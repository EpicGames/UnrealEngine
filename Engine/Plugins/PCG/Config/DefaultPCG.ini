[CoreRedirects]
+FunctionRedirects=(OldName="PCGBlueprintElement.MultiPointLoopBody",NewName="PCGBlueprintElement.VariableLoopBody")
+FunctionRedirects=(OldName="PCGBlueprintElement.PointPairLoopBody",NewName="PCGBlueprintElement.NestedLoopBody")
+FunctionRedirects=(OldName="PCGBlueprintElement.LoopOnPoints",NewName="PCGBlueprintElement.PointLoop")
+FunctionRedirects=(OldName="PCGBlueprintElement.MultiLoopOnPoints",NewName="PCGBlueprintElement.VariableLoop")
+FunctionRedirects=(OldName="PCGBlueprintElement.LoopOnPointPairs",NewName="PCGBlueprintElement.NestedLoop")
+FunctionRedirects=(OldName="PCGBlueprintElement.LoopNTimes",NewName="PCGBlueprintElement.IterationLoop")
+FunctionRedirects=(OldName="PCGBlueprintElement.InputLabels",NewName="PCGBlueprintElement.CustomInputLabels")
+FunctionRedirects=(OldName="PCGBlueprintElement.OutputLabels",NewName="PCGBlueprintElement.CustomOutputLabels")
+FunctionRedirects=(OldName="PCGBlueprintHelpers.GetRandomStream",NewName="PCGBlueprintHelpers.GetRandomStreamFromPoint")
+FunctionRedirects=(OldName="PCGDataFunctionLibrary.GetInputsByPin",NewName="PCGDataFunctionLibrary.GetInputsByPinLabel")
+FunctionRedirects=(OldName="PCGDataFunctionLibrary.GetTaggedInputs",NewName="PCGDataFunctionLibrary.GetInputsByTag")
+FunctionRedirects=(OldName="PCGDataFunctionLibrary.GetParamsByPin",NewName="PCGDataFunctionLibrary.GetParamsByPinLabel")
+FunctionRedirects=(OldName="PCGDataFunctionLibrary.GetTaggedParams",NewName="PCGDataFunctionLibrary.GetParamsByTag")
+FunctionRedirects=(OldName="PCGMetadata.InitializeAsCopy",NewName="PCGMetadata.K2_InitializeAsCopy")
+FunctionRedirects=(OldName="PCGMetadata.InitializeAsCopyWithAttributeFilter",NewName="PCGMetadata.K2_InitializeAsCopyWithAttributeFilter")
+FunctionRedirects=(OldName="PCGSpatialData.SamplePoint",NewName="PCGSpatialData.K2_SamplePoint")
+FunctionRedirects=(OldName="PCGSpatialData.ProjectPoint",NewName="PCGSpatialData.K2_ProjectPoint")
+FunctionRedirects=(OldName="PCGSpatialData.IntersectWith",NewName="PCGSpatialData.K2_IntersectWith")
+FunctionRedirects=(OldName="PCGSpatialData.ProjectOn",NewName="PCGSpatialData.K2_ProjectOn")
+FunctionRedirects=(OldName="PCGSpatialData.UnionWith",NewName="PCGSpatialData.K2_UnionWith")
+FunctionRedirects=(OldName="PCGSpatialData.Subtract",NewName="PCGSpatialData.K2_Subtract")
+FunctionRedirects=(OldName="PCGDifferenceData.AddDifference",NewName="PCGDifferenceData.K2_AddDifference")
+FunctionRedirects=(OldName="PCGParamData.FilterParamsByName",NewName="PCGParamData.K2_FilterParamsByName")
+FunctionRedirects=(OldName="PCGParamData.FilterParamsByKey",NewName="PCGParamData.K2_FilterParamsByKey")
+FunctionRedirects=(OldName="PCGMetadata.HasAttribute",NewName="PCGMetadata.BP_HasAttribute")
+FunctionRedirects=(OldName="PCGMetadata.AddAttribute",NewName="PCGMetadata.BP_AddAttribute")
+FunctionRedirects=(OldName="PCGMetadata.CopyAttribute",NewName="PCGMetadata.BP_CopyAttribute")
+FunctionRedirects=(OldName="PCGMetadata.DeleteAttribute",NewName="PCGMetadata.BP_DeleteAttribute")
+FunctionRedirects=(OldName="PCGMetadata.CopyExistingAttribute",NewName="PCGMetadata.BP_CopyExistingAttribute")
+FunctionRedirects=(OldName="PCGMetadata.RenameAttribute",NewName="PCGMetadata.BP_RenameAttribute")
+FunctionRedirects=(OldName="PCGMetadata.ClearAttribute",NewName="PCGMetadata.BP_ClearAttribute")
+FunctionRedirects=(OldName="PCGBasePointData.CopyPointsFrom",NewName="PCGBasePointData.BP_SetPointsFrom")
+FunctionRedirects=(OldName="PCGBasePointData.BP_CopyPointsFrom",NewName="PCGBasePointData.BP_SetPointsFrom")

+PropertyRedirects=(OldName="PCGSplineSamplerSettings.Params",NewName="PCGSplineSamplerSettings.SamplerParams")
+PropertyRedirects=(OldName="PCGProjectionSettings.Params",NewName="PCGProjectionSettings.ProjectionParams")
+PropertyRedirects=(OldName="PCGBlueprintElement.IterationLoop.InA",NewName="PCGBlueprintElement.IterationLoop.OptionalA")
+PropertyRedirects=(OldName="PCGBlueprintElement.IterationLoop.InB",NewName="PCGBlueprintElement.IterationLoop.OptionalB")
+PropertyRedirects=(OldName="PCGBlueprintElement.NestedLoop.InA",NewName="PCGBlueprintElement.NestedLoop.InOuterData")
+PropertyRedirects=(OldName="PCGBlueprintElement.NestedLoop.InB",NewName="PCGBlueprintElement.NestedLoop.InInnerData")
+PropertyRedirects=(OldName="PCGBlueprintElement.NestedLoopBody.InA",NewName="PCGBlueprintElement.NestedLoopBody.InOuterData")
+PropertyRedirects=(OldName="PCGBlueprintElement.NestedLoopBody.InB",NewName="PCGBlueprintElement.NestedLoopBody.InInnerData")
+PropertyRedirects=(OldName="PCGBlueprintElement.NestedLoopBody.InPointA",NewName="PCGBlueprintElement.NestedLoopBody.InOuterPoint")
+PropertyRedirects=(OldName="PCGBlueprintElement.NestedLoopBody.InPointB",NewName="PCGBlueprintElement.NestedLoopBody.InInnerPoint")
+PropertyRedirects=(OldName="PCGStaticMeshSpawnerSettings.MeshSelectorInstance",NewName="PCGStaticMeshSpawnerSettings.MeshSelectorParameters")
+PropertyRedirects=(OldName="PCGStaticMeshSpawnerSettings.InstancePackerType",NewName="PCGStaticMeshSpawnerSettings.InstanceDataPackerType")
+PropertyRedirects=(OldName="PCGStaticMeshSpawnerSettings.InstancePackerInstance",NewName="PCGStaticMeshSpawnerSettings.InstanceDataPackerParameters")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.IsOverridden.GraphInstance",NewName="PCGGraphParametersHelpers.IsOverridden.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetFloatParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetFloatParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetDoubleParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetDoubleParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetBoolParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetBoolParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetByteParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetByteParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetInt32Parameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetInt32Parameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetInt64Parameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetInt64Parameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetNameParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetNameParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetStringParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetStringParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetSoftObjectParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetSoftObjectParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetSoftClassParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetSoftClassParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetVectorParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetVectorParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetRotatorParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetRotatorParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.GetTransformParameter.GraphInstance",NewName="PCGGraphParametersHelpers.GetTransformParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetFloatParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetFloatParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetDoubleParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetDoubleParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetBoolParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetBoolParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetByteParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetByteParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetInt32Parameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetInt32Parameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetInt64Parameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetInt64Parameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetNameParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetNameParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetStringParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetStringParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetEnumParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetEnumParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetSoftObjectParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetSoftObjectParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetSoftClassParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetSoftClassParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetVectorParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetVectorParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetRotatorParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetRotatorParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGGraphParametersHelpers.SetTransformParameter.GraphInstance",NewName="PCGGraphParametersHelpers.SetTransformParameter.GraphInterface")
+PropertyRedirects=(OldName="PCGPropertyToParamDataSettings.bExtractObjectAndStruct",NewName="PCGPropertyToParamDataSettings.bForceObjectAndStructExtraction")
+PropertyRedirects=(OldName="PCGAttributePropertySelectorBlueprintHelpers.CopyAndFixSource.Selector",NewName="PCGAttributePropertySelectorBlueprintHelpers.CopyAndFixSource.OutputSelector")
+PropertyRedirects=(OldName="PCGAttributePropertySelectorBlueprintHelpers.CopyAndFixSource.InSelector",NewName="PCGAttributePropertySelectorBlueprintHelpers.CopyAndFixSource.InputSelector")
+PropertyRedirects=(OldName="PCGAttributeTransferSettings.SourceAttributeProperty",NewName="PCGAttributeTransferSettings.InputSource")
+PropertyRedirects=(OldName="PCGAttributeTransferSettings.TargetAttributeProperty",NewName="PCGAttributeTransferSettings.OutputTarget")
+PropertyRedirects=(OldName="PCGRandomChoiceSettings.bOutputDiscardedPoints",NewName="PCGRandomChoiceSettings.bOutputDiscardedEntries")
+PropertyRedirects=(OldName="/Script/PCG.PCGAttributePropertySelector.LayerName",NewName="/Script/PCG.PCGAttributePropertySelector.DomainName")

+ClassRedirects=(OldName="PCGPointSamplerSettings",NewName="/Script/PCG.PCGSelectPointsSettings")
+ClassRedirects=(OldName="PCGDensityNoiseSettings",NewName="/Script/PCG.PCGAttributeNoiseSettings")
+ClassRedirects=(OldName="PCGInstancePackerBase",NewName="/Script/PCG.PCGInstanceDataPackerBase")
+ClassRedirects=(OldName="PCGInstancePackerByAttribute",NewName="/Script/PCG.PCGInstanceDataPackerByAttribute")
+ClassRedirects=(OldName="PCGInstancePackerByRegex",NewName="/Script/PCG.PCGInstanceDataPackerByRegex")
+ClassRedirects=(OldName="PCGIntersectionSettings",NewName="/Script/PCG.PCGInnerIntersectionSettings")
+ClassRedirects=(OldName="PCGCreateAttributeSettings",NewName="/Script/PCG.PCGAddAttributeSettings")
+ClassRedirects=(OldName="PCGPointFilterSettings",NewName="/Script/PCG.PCGAttributeFilteringSettings")
+ClassRedirects=(OldName="PCGPointFilterRangeSettings",NewName="/Script/PCG.PCGAttributeFilteringRangeSettings")
+ClassRedirects=(OldName="PCGAttributeFilterSettings",NewName="/Script/PCG.PCGDeleteAttributesSettings")
+ClassRedirects=(OldName="PCGAttributeFilterNamesSettings",NewName="/Script/PCG.PCGDeleteAttributesSettings")
+ClassRedirects=(OldName="PCGPropertyToParamDataSettings",NewName="/Script/PCG.PCGGetActorPropertySettings")
+ClassRedirects=(OldName="PCGSortPointsSettings",NewName="/Script/PCG.PCGSortAttributesSettings")
+ClassRedirects=(OldName="PCGMetadataOperationSettings",NewName="/Script/PCG.PCGCopyAttributesSettings")
+ClassRedirects=(OldName="PCGCreateSplineMeshSettings",NewName="/Script/PCG.PCGSpawnSplineMeshSettings")
+ClassRedirects=(OldName="PCGGenerateGrassMapsSettings",NewName="/Script/PCG.PCGGenerateLandscapeTexturesSettings")
+ClassRedirects=(OldName="/Script/PCGGeometryScriptInterop.PCGDynamicMeshData",NewName="/Script/PCG.PCGDynamicMeshData")
+ClassRedirects=(OldName="/Script/PCG.PCGNumberOfEntriesSettings",NewName="/Script/PCG.PCGNumberOfElementsSettings")
+ClassRedirects=(OldName="/Script/PCG.PCGNumberOfPointsSettings",NewName="/Script/PCG.PCGNumberOfElementsSettings")
+ClassRedirects=(OldName="/Script/PCG.PCGBaseTextureData",NewName="/Script/PCG.PCGTexture2DSingleBaseData")

+StructRedirects=(OldName="PCGPointFilterThresholdSettings",NewName="/Script/PCG.PCGAttributeFilterThresholdSettings")
+StructRedirects=(OldName="PCGActorPropertyOverrideDescription",NewName="/Script/PCG.PCGObjectPropertyOverrideDescription")
+StructRedirects=(OldName="PCGDataTypeInfoTexture",NewName="/Script/PCG.PCGDataTypeInfoTexture2D")
+StructRedirects=(OldName="PCGDataTypeInfoRenderTarget",NewName="/Script/PCG.PCGDataTypeInfoRenderTarget2D")
+StructRedirects=(OldName="PCGDataTypeInfoBaseTexture",NewName="/Script/PCG.PCGDataTypeInfoTexture2DSingleBase")
+StructRedirects=(OldName="PCGDataTypeInfoBaseTexture2D",NewName="/Script/PCG.PCGDataTypeInfoTexture2DSingleBase")

+EnumRedirects=(OldName="EPCGDensityNoiseMode",NewName="EPCGAttributeNoiseMode")
+EnumRedirects=(OldName="EPCGPointFilterOperator",NewName="EPCGAttributeFilterOperator")
+EnumRedirects=(OldName="EPCGMedadataTransformOperation",NewName="EPCGMetadataTransformOperation")
+EnumRedirects=(OldName="EPCGMedadataBooleanOperation",NewName="EPCGMetadataBooleanOperation")
+EnumRedirects=(OldName="EPCGMedadataMathsOperation",NewName="EPCGMetadataMathsOperation")
+EnumRedirects=(OldName="EPCGMedadataTrigOperation",NewName="EPCGMetadataTrigOperation")
+EnumRedirects=(OldName="EPCGMedadataCompareOperation",NewName="EPCGMetadataCompareOperation")
+EnumRedirects=(OldName="EPCGMedadataVectorOperation",NewName="EPCGMetadataVectorOperation")
+EnumRedirects=(OldName="EPCGMedadataBitwiseOperation",NewName="EPCGMetadataBitwiseOperation")
+EnumRedirects=(OldName="EPCGMedadataRotatorOperation",NewName="EPCGMetadataRotatorOperation")
+EnumRedirects=(OldName="EPCGWorldQueryFilterByTag",NewName="EPCGWorldQueryFilter")
