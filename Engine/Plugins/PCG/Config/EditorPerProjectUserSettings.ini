[PCGEditorSpawnNodes]
; Preset
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key=One Shift=false Ctrl=false Alt=false Index=1 Label="Create Constant (Double)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key=Two Shift=false Ctrl=false Alt=false Index=4 Label="Create Constant (Vector 2D)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key=Three Shift=false Ctrl=false Alt=false Index=5 Label="Create Constant (Vector 3D)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key=Four Shift=false Ctrl=false Alt=false Index=6 Label="Create Constant (Vector 4D)")
+Node = (Class=/Script/PCG.PCGUserParameterGetSettings Key=One Shift=true Ctrl=false Alt=false Index=1 Label="Create Graph Parameter (Double)")
+Node = (Class=/Script/PCG.PCGUserParameterGetSettings Key=Two Shift=true Ctrl=false Alt=false Index=4 Label="Create Graph Parameter (Vector 2D)")
+Node = (Class=/Script/PCG.PCGUserParameterGetSettings Key=Three Shift=true Ctrl=false Alt=false Index=5 Label="Create Graph Parameter (Vector 3D)")
+Node = (Class=/Script/PCG.PCGUserParameterGetSettings Key=Four Shift=true Ctrl=false Alt=false Index=6 Label="Create Graph Parameter (Vector 4D)")
+Node = (Class=/Script/PCG.PCGDebugSettings Key=D Shift=true Ctrl=false Alt=false)
+Node = (Class=/Script/PCG.PCGSpawnActorSettings Key=A Shift=true Ctrl=false Alt=false)
+Node = (Class=/Script/PCG.PCGStaticMeshSpawnerSettings Key=M Shift=false Ctrl=false Alt=false)
+Node = (Class=/Script/PCG.PCGPrintElementSettings Key=P Shift=false Ctrl=false Alt=false)
+Node = (Class=/Script/PCG.PCGBranchSettings Key=B Shift=false Ctrl=false Alt=false)
+Node = (Class=/Script/PCG.PCGGetLandscapeSettings Key=L Shift=false Ctrl=false Alt=false)
+Node = (Class=/Script/PCG.PCGGetVolumeSettings Key=V Shift=false Ctrl=false Alt=false)
+Node = (Class=/Script/PCG.PCGGetPrimitiveSettings Key=R Shift=false Ctrl=false Alt=false)
+Node = (Class=/Script/PCG.PCGGetSplineSettings Key=S Shift=false Ctrl=false Alt=false)
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key=Z Shift=false Ctrl=false Alt=false Index=9 Label="Create Constant (String)")
; Uninitialized
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=false Ctrl=false Alt=false Index=0 Label="Create Constant (Float)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=false Ctrl=false Alt=false Index=2 Label="Create Constant (Int32)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=false Ctrl=false Alt=false Index=3 Label="Create Constant (Int64)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=false Ctrl=false Alt=false Index=7 Label="Create Constant (Quaternion)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=false Ctrl=false Alt=false Index=8 Label="Create Constant (Transform)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=false Ctrl=false Alt=false Index=10 Label="Create Constant (Boolean)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=false Ctrl=false Alt=false Index=11 Label="Create Constant (Rotator)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=false Ctrl=false Alt=false Index=12 Label="Create Constant (Name)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=false Ctrl=false Alt=false Index=13 Label="Create Constant (SoftObjectPath)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=false Ctrl=false Alt=false Index=14 Label="Create Constant (SoftClassPath)")

+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=true Ctrl=false Alt=false Index=0 Label="Create Graph Parameter (Float)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=true Ctrl=false Alt=false Index=2 Label="Create Graph Parameter (Int32)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=true Ctrl=false Alt=false Index=3 Label="Create Graph Parameter (Int64)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=true Ctrl=false Alt=false Index=7 Label="Create Graph Parameter (Quaternion)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=true Ctrl=false Alt=false Index=8 Label="Create Graph Parameter (Transform)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=true Ctrl=false Alt=false Index=10 Label="Create Graph Parameter (Boolean)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=true Ctrl=false Alt=false Index=11 Label="Create Graph Parameter (Rotator)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=true Ctrl=false Alt=false Index=12 Label="Create Graph Parameter (Name)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=true Ctrl=false Alt=false Index=13 Label="Create Graph Parameter (SoftObjectPath)")
+Node = (Class=/Script/PCG.PCGCreateAttributeSetSettings Key= Shift=true Ctrl=false Alt=false Index=14 Label="Create Graph Parameter (SoftClassPath)")
