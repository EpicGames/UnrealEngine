{
	"FileVersion": 3,
	"Version": 8,
	"VersionName": "1.0",
	"FriendlyName": "Procedural Content Generation Framework (PCG)",
	"Description": "Visual scripting framework for procedurally populating worlds with content in editor and/or at run-time.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/latest/en-US/procedural-content-generation--framework-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "PCG",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PCGEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PCGCompute",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "ComputeFramework",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true
		}
	]
}
