// Copyright Epic Games, Inc. All Rights Reserved.

// Counts how many times each unique value of an attribute is encountered in the input data elements.

#define PCG_OUTPUT_TO_RAW_BUFFER {RawDataOutputEnabled}

// Input: Input elements with attribute.
// Output (PCG_OUTPUT_TO_RAW_BUFFER): Uint array consisting of (string key value, value instance count) pairs. Number of pairs is the number of unique values (known CPU-side).
// Output (!PCG_OUTPUT_TO_RAW_BUFFER): Attribute set with a ValueCount attribute which has the count of each value. Number of elements is the number of unique values (known CPU-side).

void InitializeValueKey(uint InDataIndex, uint InValueKey)
{
#if PCG_OUTPUT_TO_RAW_BUFFER
	const uint FirstElementIndex = InValueKey * 2u;
	if (FirstElementIndex < Out_GetNumElements())
	{
		// We currently have an output element per possible string key value. This may be compacted in the future.
		Out_Store(FirstElementIndex, InValueKey);
	}
#else
	// Write value of attribute. We currently have an output element per possible string key value.
 	const int OutputValueAttributeId = CountUniqueValues_GetOutputValueAttributeId();
 	Out_SetInt(InDataIndex, InValueKey, OutputValueAttributeId, InValueKey);
#endif
}

[numthreads(64, 1, 1)]
void PCGCountUniqueAttributeValuesCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);

	// Pre-job - if this thread maps to any element in the output, initialize the value key. The dispatch thread count will be large enough to ensure
	// all output elements are initialized.
	{
#if PCG_OUTPUT_TO_RAW_BUFFER
		// If outputting to raw buffer, always single data.
		InitializeValueKey(/*DataIndex=*/0, ThreadIndex);
#else
		uint Out_DataIndex;
		uint Out_ElementIndex;
		if (Out_GetThreadData(ThreadIndex, Out_DataIndex, Out_ElementIndex))
		{
			InitializeValueKey(Out_DataIndex, Out_ElementIndex);
		}
#endif
	}

	// Main job - one thread per input data element. Read the string key value and atomic increment the output counter.
	{
		uint In_DataIndex, In_ElementIndex;
		if (!In_GetThreadData(ThreadIndex, In_DataIndex, In_ElementIndex))
		{
			return;
		}

		if (In_IsPointRemoved(In_DataIndex, In_ElementIndex))
		{
			return;
		}

		const int AttributeToCountId = CountUniqueValues_GetAttributeToCountId();

		// Get the value of the attribute. Assumes int type currently (StringKeys are stored as ints).
		const int AttributeValue = In_GetInt(In_DataIndex, In_ElementIndex, AttributeToCountId);

		if (AttributeValue < 0)
		{
			// Should not happen (0 is null/empty string, negative is invalid).
			return;
		}

		// If not outputting per-data counts, then all counters are aggregated on a single data.
		const uint Out_DataIndex = CountUniqueValues_GetEmitPerDataCounts() ? In_DataIndex : 0;

#if PCG_OUTPUT_TO_RAW_BUFFER
		const uint ElementIndex = AttributeValue * 2u + 1;
		Out_AtomicAdd(ElementIndex, 1u);
#else
		const int OutputCountAttributeId = CountUniqueValues_GetOutputCountAttributeId();

		// Increment count of this attribute.
		Out_AtomicAddInt(Out_DataIndex, AttributeValue, OutputCountAttributeId, 1);
#endif
	}
}

#undef PCG_OUTPUT_TO_RAW_BUFFER
