// Copyright Epic Games, Inc. All Rights Reserved.

// Returns the total data count
[numthreads(64, 1, 1)]
void PCGDataNumCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);
	if (ThreadIndex >= GetNumThreads().x) return;

	uint InDataIndex, InElemIndex;
	if (!In_GetThreadData(ThreadIndex, InDataIndex, InElemIndex))
	{
		return;
	}

	uint OutDataIndex, OutElemIndex;
	if (!Out_GetThreadData(ThreadIndex, OutDataIndex, OutElemIndex))
	{
		return;
	}

	const int AttributeId = DataNum_GetOutputAttributeId();
	Out_SetInt(OutDataIndex, OutElemIndex, AttributeId, (int)In_GetNumData());
}
