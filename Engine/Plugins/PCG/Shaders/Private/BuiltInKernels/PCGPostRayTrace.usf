// Copyright Epic Games, Inc. All Rights Reserved.

// This is a post-process kernel that unpacks a raw buffer (of packed data emitted by PCGRayTrace.usf) into one point per ray.

[numthreads(64, 1, 1)]
void PCGPostRayTraceCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);
	if (ThreadIndex >= GetNumThreads().x) return;

	uint In_DataIndex, In_ElementIndex;
	if (!In_GetThreadData(ThreadIndex, In_DataIndex, In_ElementIndex))
	{
		return;
	}

	uint Out_DataIndex, Out_ElementIndex;
	if (!Out_GetThreadData(ThreadIndex, Out_DataIndex, Out_ElementIndex))
	{
		return;
	}

	// Propagate the 'Removed' status to the output point. Otherwise, this point will not be culled.
	if (In_IsPointRemoved(In_DataIndex, In_ElementIndex))
	{
		Out_RemovePoint(Out_DataIndex, Out_ElementIndex);
		return;
	}
	
	const uint RayTraceResults_FirstElementIndex = ThreadIndex * PCG_RAY_TRACE_PACKED_BUFFER_STRIDE_UINTS;
	const uint PackedPositionIndex = RayTraceResults_FirstElementIndex;
	const uint PackedTexCoordsIndex = RayTraceResults_FirstElementIndex + 4;
	const uint PackedNormalsIndex = RayTraceResults_FirstElementIndex + 8;

	if (PackedPositionIndex >= RayTraceResults_GetNumElements())
	{
		// Should not happen - invalid results data.
		return;
	}

	const uint4 PackedPositionAndValid = RayTraceResults_Load4(PackedPositionIndex);
	const float3 HitPosition = asfloat(PackedPositionAndValid.xyz);
	bool bMissedOrOutOfBounds = false;

	// Handle if the ray was missed, culled, or outside the bounds.
	if (PackedPositionAndValid.w == PCG_RAY_CULLED || !(WorldRaycast_GetUnbounded() != 0 || (all(HitPosition >= GetComponentBoundsMin()) && all(HitPosition < GetComponentBoundsMax()))))
	{
		if (WorldRaycast_GetKeepOriginalPointOnMiss() != 0)
		{
			bMissedOrOutOfBounds = true;
		}
		else
		{
			Out_RemovePoint(Out_DataIndex, Out_ElementIndex);
			return;
		}
	}

	PCG_COPY_ALL_ATTRIBUTES_TO_OUTPUT(Out, In, Out_DataIndex, Out_ElementIndex, In_DataIndex, In_ElementIndex);

	if (!bMissedOrOutOfBounds)
	{
		Out_SetPosition(Out_DataIndex, Out_ElementIndex, HitPosition);
	}

	const int TexCoordsAttributeId = PostRayTrace_GetTexCoordsAttributeId();

	if (TexCoordsAttributeId >= 0)
	{
		if (!bMissedOrOutOfBounds)
		{
			const uint4 PackedTexCoords = RayTraceResults_Load4(PackedTexCoordsIndex);
			const float2 TexCoords = asfloat(PackedTexCoords.xy);
			Out_SetFloat2(Out_DataIndex, Out_ElementIndex, TexCoordsAttributeId, TexCoords);
		}
		else
		{
			Out_SetFloat2(Out_DataIndex, Out_ElementIndex, TexCoordsAttributeId, (float2)0.0f);
		}
	}

	const int NormalsAttributeId = PostRayTrace_GetNormalsAttributeId();

	if (NormalsAttributeId >= 0)
	{
		if (!bMissedOrOutOfBounds)
		{
			const uint4 PackedNormals = RayTraceResults_Load4(PackedNormalsIndex);
			const float3 Normals = asfloat(PackedNormals.xyz);
			Out_SetFloat3(Out_DataIndex, Out_ElementIndex, NormalsAttributeId, Normals);
		}
		else
		{
			Out_SetFloat3(Out_DataIndex, Out_ElementIndex, NormalsAttributeId, float3(0.0f, 0.0f, 1.0f));
		}
	}
}
