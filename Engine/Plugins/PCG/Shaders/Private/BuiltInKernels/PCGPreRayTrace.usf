// Copyright Epic Games, Inc. All Rights Reserved.

// This is a pre-process kernel that packs a raw buffer (of packed data to be fed into PCGRayTrace.usf) with one ray per point.

#define PCG_END_POINTS_ENABLED {EndPointsEnabled}

#define PCG_RAYCAST_MODE_INFINITE 0
#define PCG_RAYCAST_MODE_SCALED_VECTOR 1
#define PCG_RAYCAST_MODE_NORMALIZED_WITH_LENGTH 2
#define PCG_RAYCAST_MODE_SEGMENTS 3

[numthreads(64, 1, 1)]
void PCGPreRayTraceCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);
	if (ThreadIndex >= GetNumThreads().x) return;

	uint In_DataIndex, In_ElementIndex;
	if (!In_GetThreadData(ThreadIndex, In_DataIndex, In_ElementIndex))
	{
		return;
	}

	const uint Out_DataIndex = 0;
	const uint Out_FirstElementIndex = ThreadIndex * PCG_RAY_TRACE_PACKED_BUFFER_STRIDE_UINTS;
	const uint PackedPositionIndex = Out_FirstElementIndex;
	const uint PackedDirectionIndex = Out_FirstElementIndex + 4;
	const uint PackedNormalsIndex = Out_FirstElementIndex + 8;

	// If the point is culled, the ray should also be culled.
	if (In_IsPointRemoved(In_DataIndex, In_ElementIndex))
	{
		Out_Store4(PackedPositionIndex, (uint4)PCG_RAY_CULLED);
		return;
	}

	const int RaycastMode = WorldRaycast_GetRaycastMode();
	const int RayOriginAttributeId = WorldRaycast_GetRayOriginAttributeId();
	const int RayDirectionAttributeId = WorldRaycast_GetRayDirectionAttributeId();
	const int RayLengthAttributeId = WorldRaycast_GetRayLengthAttributeId();

	float3 RayOrigin = (float3)0.0f;
	float3 RayDirection = (float3)0.0f;
	float RayLength = 0.0f;

	if (RayOriginAttributeId != -1)
	{
		RayOrigin = In_GetFloat3(In_DataIndex, In_ElementIndex, RayOriginAttributeId);
	}
	else
	{
		RayOrigin = In_GetPosition(In_DataIndex, In_ElementIndex);
	}

	if (RayDirectionAttributeId != -1)
	{
		RayDirection = In_GetFloat3(In_DataIndex, In_ElementIndex, RayDirectionAttributeId);
	}
	else
	{
		RayDirection = WorldRaycast_GetRayDirection();
	}

	// Compute ray length.
	if (RaycastMode == PCG_RAYCAST_MODE_INFINITE)
	{
		RayLength = 1e30f;
	}
	else if (RaycastMode == PCG_RAYCAST_MODE_SCALED_VECTOR)
	{
		RayLength = length(RayDirection);
	}
	else if (RaycastMode == PCG_RAYCAST_MODE_NORMALIZED_WITH_LENGTH)
	{
		if (RayLengthAttributeId != -1)
		{
			RayLength = In_GetFloat(In_DataIndex, In_ElementIndex, RayLengthAttributeId);
		}
		else
		{
			RayLength = WorldRaycast_GetRayLength();
		}
	}
#if PCG_END_POINTS_ENABLED
	else if (RaycastMode == PCG_RAYCAST_MODE_SEGMENTS)
	{
		const int RayEndAttributeId = WorldRaycast_GetRayEndAttributeId();
		float3 RayEndPosition = (float3)0.0f;

		if (RayEndAttributeId != -1)
		{
			RayEndPosition = End_Points_GetFloat3(In_DataIndex, In_ElementIndex, RayEndAttributeId);
		}
		else
		{
			RayEndPosition = End_Points_GetPosition(In_DataIndex, In_ElementIndex);
		}

		// In Segments mode, the ray direction is also controlled by the segment.
		RayDirection = RayEndPosition - RayOrigin;
		RayLength = length(RayDirection);
	}
#endif

	// Normalize ray direction.
	RayDirection = dot(RayDirection, RayDirection) != 0.0f ? normalize(RayDirection) : (float3)0.0f;

	Out_Store4(PackedPositionIndex, uint4(asuint(RayOrigin), 0u));
	Out_Store4(PackedDirectionIndex, uint4(asuint(RayDirection), asuint(RayLength)));
	Out_Store4(PackedNormalsIndex, (uint4)0u); // Initialize buffer for normal values that will be written later.
}
