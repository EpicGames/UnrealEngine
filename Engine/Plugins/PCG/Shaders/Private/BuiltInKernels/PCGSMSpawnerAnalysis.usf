// Copyright Epic Games, Inc. All Rights Reserved.

// Counts how many times each unique value of an attribute is encountered in the input data elements.
// Dedicated analysis kernel for the Static Mesh Spawner.

#define PCG_OUTPUT_TO_RAW_BUFFER {RawDataOutputEnabled}

// Input: Input elements with attribute.
// Output (PCG_OUTPUT_TO_RAW_BUFFER): Uint array consisting of (string key value, value instance count) pairs. Number of pairs is the number of unique values (known CPU-side).
// Output (!PCG_OUTPUT_TO_RAW_BUFFER): Attribute set with a ValueCount attribute which has the count of each value. Number of elements is the number of unique values (known CPU-side).

// Number of uint32 entries per (primitive, culling cell) in the CullingCellMinMaxPositions buffer: 3 inv-min position, 3 max position.
static const uint CullingCellMinMaxPositionScaleStride = 6;

void InitializeValueKey(uint InDataIndex, uint InValueKey, uint InCullingCellIndex)
{
#if PCG_OUTPUT_TO_RAW_BUFFER
	const uint FirstElementIndex = (InValueKey * SMSpawnerAnalysis_GetNumCullingCellsTotal() + InCullingCellIndex) * 2u;
	if (FirstElementIndex < Out_GetNumElements())
	{
		// We currently have an output element per possible string key value. This may be compacted in the future.
		Out_Store(FirstElementIndex, InValueKey);
	}
#else
	// Write value of attribute. We currently have an output element per possible string key value.
	const uint ElementIndex = InValueKey * SMSpawnerAnalysis_GetNumCullingCellsTotal() + InCullingCellIndex;
 	const int OutputValueAttributeId = SMSpawnerAnalysis_GetOutputValueAttributeId();
 	Out_SetInt(InDataIndex, ElementIndex, OutputValueAttributeId, InValueKey);
#endif
}

[numthreads(64, 1, 1)]
void PCGSMSpawnerAnalysisCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);
	const int NumCullingCellsTotal = SMSpawnerAnalysis_GetNumCullingCellsTotal(); // Should always be greater than 0

	// Pre-job - if this thread maps to any element in the output, initialize the value key. The dispatch thread count will be large enough to ensure
	// all output elements are initialized.
	{
#if PCG_OUTPUT_TO_RAW_BUFFER
		// If outputting to raw buffer, always single data.
		InitializeValueKey(/*DataIndex=*/0, ThreadIndex / NumCullingCellsTotal, ThreadIndex % NumCullingCellsTotal);
#else
		uint Out_DataIndex;
		uint Out_ElementIndex;
		if (Out_GetThreadData(ThreadIndex, Out_DataIndex, Out_ElementIndex))
		{
			InitializeValueKey(Out_DataIndex, Out_ElementIndex / NumCullingCellsTotal, Out_ElementIndex % NumCullingCellsTotal);
		}
#endif
	}

	// Main job - one thread per input data element. Read the string key value and atomic increment the output counter.
	{
		uint In_DataIndex, In_ElementIndex;
		if (!In_GetThreadData(ThreadIndex, In_DataIndex, In_ElementIndex))
		{
			return;
		}

		if (In_IsPointRemoved(In_DataIndex, In_ElementIndex))
		{
			return;
		}

		const int AttributeToCountId = SMSpawnerAnalysis_GetAttributeToCountId();

		// Get the value of the attribute. Assumes int type currently (StringKeys are stored as ints).
		const int AttributeValue = In_GetInt(In_DataIndex, In_ElementIndex, AttributeToCountId);

		if (AttributeValue < 0)
		{
			// Should not happen (0 is null/empty string, negative is invalid).
			return;
		}

		// If not outputting per-data counts, then all counters are aggregated on a single data.
		const uint Out_DataIndex = SMSpawnerAnalysis_GetEmitPerDataCounts() ? In_DataIndex : 0;

		uint CullingCellIndex = 0;
		if (NumCullingCellsTotal > 1)
		{
			const float3 Pos = In_GetPosition(In_DataIndex, In_ElementIndex);
			const int3 CullingCellCoord = int3(floor(Pos / SMSpawnerAnalysis_GetCullingCellExtent()));
			int3 LocalCullingCellCoord = CullingCellCoord - int3(floor(SMSpawnerAnalysis_GetBoundsMin() / SMSpawnerAnalysis_GetCullingCellExtent()));

			const int3 NumCullingCells = SMSpawnerAnalysis_GetNumCullingCells();
			LocalCullingCellCoord = clamp(LocalCullingCellCoord, 0, NumCullingCells - 1);

			CullingCellIndex = (NumCullingCells.y * NumCullingCells.x * LocalCullingCellCoord.z) + (NumCullingCells.x * LocalCullingCellCoord.y) + LocalCullingCellCoord.x;
		}

		// Expand culling cell bounds to this instance.
		{
			const float3 Pos = In_GetPosition(In_DataIndex, In_ElementIndex);
			const float3 Scale = In_GetScale(In_DataIndex, In_ElementIndex);
			const float4 Rotation = In_GetRotation(In_DataIndex, In_ElementIndex);

			// Get local mesh bounds. Zero when mesh is not yet loaded (first frame), which degrades to point-only tracking.
			const float3 MeshLocalBoundsMin = SMSpawnerAnalysis_GetMeshBoundsMin(AttributeValue);
			const float3 MeshLocalBoundsMax = SMSpawnerAnalysis_GetMeshBoundsMax(AttributeValue);

			// Compute scale-aware local center and half-extent.
			const float3 LocalCenter = (MeshLocalBoundsMin + MeshLocalBoundsMax) * 0.5;
			const float3 LocalHalfExtent = (MeshLocalBoundsMax - MeshLocalBoundsMin) * 0.5;
			const float3 ScaledCenter = Scale * LocalCenter;
			const float3 ScaledHalfExtent = abs(Scale) * LocalHalfExtent;

			// Compute rotation-aware world AABB of the scaled mesh OBB.
			// WorldCenter = instance position + rotated scaled center.
			const float3 WorldCenter = Pos + QuatRotateVector(Rotation, ScaledCenter);

			// WorldHalfExtent = |R| * ScaledHalfExtent, where |R| is the element-wise abs rotation matrix.
			// Computed by rotating the three canonical basis vectors and taking their abs values.
			const float3 WorldHalfExtent =
				abs(QuatRotateVector(Rotation, float3(1, 0, 0))) * ScaledHalfExtent.x +
				abs(QuatRotateVector(Rotation, float3(0, 1, 0))) * ScaledHalfExtent.y +
				abs(QuatRotateVector(Rotation, float3(0, 0, 1))) * ScaledHalfExtent.z;

			// Normalize world AABB relative to a 50%-expanded generation domain, saturated to [0,1].
			// The expansion allows mesh bounds that extend outside the generation domain to be represented,
			// up to one half-extent in each direction. Anything beyond that is clamped.
			// Positive IEEE 754 floats are monotonically ordered as uint32, so asuint() preserves ordering for AtomicMax.
			const float3 DomainMin = SMSpawnerAnalysis_GetBoundsMin();
			const float3 DomainExtent = SMSpawnerAnalysis_GetBoundsExtent();
			const float DomainExpandProportion = 0.5f;
			const float TotalDomainExpansion = 2.0f;
			const float3 ExpandedMin = DomainMin - DomainExpandProportion * DomainExtent;
			const float3 ExpandedExtent = TotalDomainExpansion * DomainExtent;
			const float3 PosMinNorm = saturate((WorldCenter - WorldHalfExtent - ExpandedMin) / ExpandedExtent);
			const float3 PosMaxNorm = saturate((WorldCenter + WorldHalfExtent - ExpandedMin) / ExpandedExtent);

			// Per-culling-cell bounds, indexed by (primitive, culling cell).
			// Buffer is zero-initialized before the kernel runs. AtomicMax tracks the running maximum, so we store
			// min components as (1.0 - pos) - taking the max of inverted positions is equivalent to taking the min of positions.
			const uint CullingCellBase = CullingCellMinMaxPositionScaleStride * (AttributeValue * NumCullingCellsTotal + CullingCellIndex);
			CullingCellMinMaxPositions_AtomicMax(CullingCellBase + 0, asuint(1.0f - PosMinNorm[0]));
			CullingCellMinMaxPositions_AtomicMax(CullingCellBase + 1, asuint(1.0f - PosMinNorm[1]));
			CullingCellMinMaxPositions_AtomicMax(CullingCellBase + 2, asuint(1.0f - PosMinNorm[2]));
			CullingCellMinMaxPositions_AtomicMax(CullingCellBase + 3, asuint(PosMaxNorm[0]));
			CullingCellMinMaxPositions_AtomicMax(CullingCellBase + 4, asuint(PosMaxNorm[1]));
			CullingCellMinMaxPositions_AtomicMax(CullingCellBase + 5, asuint(PosMaxNorm[2]));
		}

#if PCG_OUTPUT_TO_RAW_BUFFER
		const uint ElementIndex = (AttributeValue * NumCullingCellsTotal + CullingCellIndex) * 2u + 1;
		Out_AtomicAdd(ElementIndex, 1u);
#else
		// Always one culling cell per attribute value.
		const uint Out_ElementIndex = AttributeValue * NumCullingCellsTotal + CullingCellIndex;

		const int OutputCountAttributeId = SMSpawnerAnalysis_GetOutputCountAttributeId();

		// Increment count of this attribute.
		Out_AtomicAddInt(Out_DataIndex, Out_ElementIndex, OutputCountAttributeId, 1);
#endif
	}
}

#undef PCG_OUTPUT_TO_RAW_BUFFER
