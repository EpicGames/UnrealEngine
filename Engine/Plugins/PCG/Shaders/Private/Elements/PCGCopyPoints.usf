// Copyright Epic Games, Inc. All Rights Reserved.

[numthreads(64, 1, 1)]
void PCGCopyPointsCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);
	if (ThreadIndex >= GetNumThreads().x) return;

	uint Out_DataIndex, Out_ElemIndex;
	if (!Out_GetThreadData(ThreadIndex, Out_DataIndex, Out_ElemIndex))
	{
		return;
	}

	const uint SourceNumData = Source_GetNumData();
	const uint TargetNumData = Target_GetNumData();
	const uint CopyEachSourceOnEveryTarget = bCopyEachSourceOnEveryTarget_GetOverridableValue();

	// Get source and target data indices for this output data index.
	uint SourceDataIndex, TargetDataIndex;
	CopyPoints_GetSourceAndTargetDataIndices(Out_DataIndex, SourceNumData, TargetNumData, SourceDataIndex, TargetDataIndex);

	const uint NumTargetPoints = Target_GetNumElements(TargetDataIndex);
	uint SourcePointIndexInData = Out_ElemIndex / NumTargetPoints;
	uint TargetPointIndexInData = Out_ElemIndex % NumTargetPoints;

	// Propagate the 'Removed' status to the output point if either source or target point is removed. Otherwise, this point will not be culled.
	if (Source_IsPointRemoved(SourceDataIndex, SourcePointIndexInData) || Target_IsPointRemoved(TargetDataIndex, TargetPointIndexInData))
	{
		Out_RemovePoint(Out_DataIndex, Out_ElemIndex);
		return;
	}

	const float3 SourcePosition = Source_GetPosition(SourceDataIndex, SourcePointIndexInData);
	const float4 SourceRotation = Source_GetRotation(SourceDataIndex, SourcePointIndexInData);
	const float3 SourceScale = Source_GetScale(SourceDataIndex, SourcePointIndexInData);
	const float4 SourceColor = Source_GetColor(SourceDataIndex, SourcePointIndexInData);
	const uint SourceSeed = Source_GetSeed(SourceDataIndex, SourcePointIndexInData);
	
	const float3 TargetPosition = Target_GetPosition(TargetDataIndex, TargetPointIndexInData);
	const float4 TargetRotation = Target_GetRotation(TargetDataIndex, TargetPointIndexInData);
	const float3 TargetScale = Target_GetScale(TargetDataIndex, TargetPointIndexInData);
	const float4 TargetColor = Target_GetColor(TargetDataIndex, TargetPointIndexInData);
	const uint TargetSeed = Target_GetSeed(TargetDataIndex, TargetPointIndexInData);

	const uint RotationInheritance = RotationInheritance_GetOverridableValue();

	uint ApplyTargetRotationToPositions = 1;
	if (RotationInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Source)
	{
		ApplyTargetRotationToPositions = bApplyTargetRotationToPositions_GetOverridableValue();
	}

	const uint ScaleInheritance = ScaleInheritance_GetOverridableValue();
	uint ApplyTargetScaleToPositions = 1;
	if (ScaleInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Source)
	{
		ApplyTargetScaleToPositions = bApplyTargetScaleToPositions_GetOverridableValue();
	}

	// @todo_pcg: Must be known statically, so graph splits are required in order to support overriding these properties.
	const uint ColorInheritance = ColorInheritance_GetOverridableValue();
	const uint SeedInheritance = SeedInheritance_GetOverridableValue();
	const uint AttributeInheritance = AttributeInheritance_GetOverridableValue();

	const float3 ScaleForPosition = ApplyTargetScaleToPositions > 0 ? TargetScale : float3(1, 1, 1);

	float3 OutPosition = TargetPosition;
	if (ApplyTargetRotationToPositions > 0)
	{
		OutPosition += QuatRotateVector(TargetRotation, (SourcePosition * ScaleForPosition));
	}
	else
	{
		OutPosition += SourcePosition * ScaleForPosition;
	}

	float4 OutRotation = float4(0, 0, 0, 1);
	float3 OutScale = (float3)0;
	float4 OutColor = (float4)0;
	uint OutSeed = 0;

	if (RotationInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Relative)
	{
		OutRotation = QuatMultiply(TargetRotation, SourceRotation);
	}
	else if (RotationInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Source)
	{
		OutRotation = SourceRotation;
	}
	else if (RotationInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Target)
	{
		OutRotation = TargetRotation;
	}

	if (ScaleInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Relative)
	{
		OutScale = SourceScale * TargetScale;
	}
	else if (ScaleInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Source)
	{
		OutScale = SourceScale;
	}
	else if (ScaleInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Target)
	{
		OutScale = TargetScale;
	}

	if (ColorInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Relative)
	{
		OutColor = SourceColor * TargetColor;
	}
	else if (ColorInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Source)
	{
		OutColor = SourceColor;
	}
	else if (ColorInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Target)
	{
		OutColor = TargetColor;
	}

	if (SeedInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Relative)
	{
		OutSeed = ComputeSeed(SourceSeed, TargetSeed);
	}
	else if (SeedInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Source)
	{
		OutSeed = SourceSeed;
	}
	else if (SeedInheritance == PCG_COPY_POINTS_INHERITANCE_MODE_Target)
	{
		OutSeed = TargetSeed;
	}
	
	// Write output
	Out_SetPosition(Out_DataIndex, Out_ElemIndex, OutPosition);
	Out_SetRotation(Out_DataIndex, Out_ElemIndex, OutRotation);
	Out_SetScale(Out_DataIndex, Out_ElemIndex, OutScale);
	Out_SetColor(Out_DataIndex, Out_ElemIndex, OutColor);
	Out_SetSeed(Out_DataIndex, Out_ElemIndex, OutSeed);

	// These properties do not have an inheritance mode, simply copy from the source data.
	Out_SetBoundsMin(Out_DataIndex, Out_ElemIndex, Source_GetBoundsMin(SourceDataIndex, SourcePointIndexInData));
	Out_SetBoundsMax(Out_DataIndex, Out_ElemIndex, Source_GetBoundsMax(SourceDataIndex, SourcePointIndexInData));
	Out_SetDensity(Out_DataIndex, Out_ElemIndex, Source_GetDensity(SourceDataIndex, SourcePointIndexInData));
	Out_SetSteepness(Out_DataIndex, Out_ElemIndex, Source_GetSteepness(SourceDataIndex, SourcePointIndexInData));

	// Copy attributes
	if (AttributeInheritance == PCG_COPY_POINTS_METADATA_INHERITANCE_MODE_SourceOnly)
	{
		PCG_COPY_ATTRIBUTES_TO_OUTPUT(Out, Source, Out_DataIndex, Out_ElemIndex, SourceDataIndex, SourcePointIndexInData, /*bMetadataOnly=*/true, /*bInitNonCopiedAttributes=*/true);
	}
	else if (AttributeInheritance == PCG_COPY_POINTS_METADATA_INHERITANCE_MODE_TargetOnly)
	{
		PCG_COPY_ATTRIBUTES_TO_OUTPUT(Out, Target, Out_DataIndex, Out_ElemIndex, TargetDataIndex, TargetPointIndexInData, /*bMetadataOnly=*/true, /*bInitNonCopiedAttributes=*/true);
	}
	else if (AttributeInheritance == PCG_COPY_POINTS_METADATA_INHERITANCE_MODE_SourceFirst)
	{
		PCG_COPY_ATTRIBUTES_TO_OUTPUT(Out, Target, Out_DataIndex, Out_ElemIndex, TargetDataIndex, TargetPointIndexInData, /*bMetadataOnly=*/true, /*bInitNonCopiedAttributes=*/true);

		// Don't allow Source to initialize non-copied attributes in the output, otherwise it will overwrite attributes from Target.
		PCG_COPY_ATTRIBUTES_TO_OUTPUT(Out, Source, Out_DataIndex, Out_ElemIndex, SourceDataIndex, SourcePointIndexInData, /*bMetadataOnly=*/true, /*bInitNonCopiedAttributes=*/false);
	}
	else if (AttributeInheritance == PCG_COPY_POINTS_METADATA_INHERITANCE_MODE_TargetFirst)
	{
		PCG_COPY_ATTRIBUTES_TO_OUTPUT(Out, Source, Out_DataIndex, Out_ElemIndex, SourceDataIndex, SourcePointIndexInData, /*bMetadataOnly=*/true, /*bInitNonCopiedAttributes=*/true);

		// Don't allow Target to initialize non-copied attributes in the output, otherwise it will overwrite attributes from Source.
		PCG_COPY_ATTRIBUTES_TO_OUTPUT(Out, Target, Out_DataIndex, Out_ElemIndex, TargetDataIndex, TargetPointIndexInData, /*bMetadataOnly=*/true, /*bInitNonCopiedAttributes=*/false);
	}
}
