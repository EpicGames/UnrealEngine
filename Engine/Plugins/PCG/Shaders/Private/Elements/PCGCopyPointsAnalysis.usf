// Copyright Epic Games, Inc. All Rights Reserved.

[numthreads(64, 1, 1)]
void PCGCopyPointsAnalysisCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);

	// We dispatched one thread per copy points output data and the output data is a single attribute set with num data elements.
	uint Dummy, CopyOutputDataIndex;
	if (!Out_GetThreadData(ThreadIndex, Dummy, CopyOutputDataIndex))
	{
		return;
	}

	const int MatchAttributeId = CopyPoints_GetMatchAttributeId();
	if (MatchAttributeId == -1)
	{
		// No match attribute, early out with no copy.
		Out_SetBool(0, CopyOutputDataIndex, CopyPoints_GetSelectedFlagAttributeId(), false);
		return;
	}

	const uint SourceNumData = Source_GetNumData();
	const uint TargetNumData = Target_GetNumData();

	uint SourceDataIndex = 0;
	uint TargetDataIndex = 0;

	if (bCopyEachSourceOnEveryTarget_GetOverridableValue())
	{
		// Output is attribute set with SourceNumData * TargetNumData bools. Kernel executes one thread per output.
		// copypoints(S = ((src 0), (src 1)), T = ((tgt 0), (tgt 1))) = (src 0 x tgt 0), (src 0 x tgt 1), (src 1 x tgt 0), (src 1 tgt 1))
		SourceDataIndex = CopyOutputDataIndex / TargetNumData;
		TargetDataIndex = CopyOutputDataIndex % TargetNumData;
	}
	else
	{
		// Output is attribute set with max(SourceNumData, TargetNumData) bools. (S, T) can be (N, N), (1, N) or (N, 1).
		SourceDataIndex = clamp(CopyOutputDataIndex, 0u, (uint)(SourceNumData - 1));
		TargetDataIndex = clamp(CopyOutputDataIndex, 0u, (uint)(TargetNumData - 1));
	}

	if (Source_GetNumElements(SourceDataIndex) > 0 && Target_GetNumElements(TargetDataIndex) > 0)
	{
		// If first point is removed, assume no copy.
		// todo_pcg: This is temporary and will use a data domain attribute once support has been added.
		if (!Source_IsPointRemoved(SourceDataIndex, /*ElementIndex=*/0) && !Target_IsPointRemoved(TargetDataIndex, /*ElementIndex=*/0))
		{
			const int SourceValue = Source_GetInt(SourceDataIndex, /*ElementIndex=*/0, MatchAttributeId);
			const int TargetValue = Target_GetInt(TargetDataIndex, /*ElementIndex=*/0, MatchAttributeId);

			if (SourceValue == TargetValue)
			{
				Out_SetBool(/*DataIndex=*/0, CopyOutputDataIndex, CopyPoints_GetSelectedFlagAttributeId(), true);
				return;
			}
		}
	}

	Out_SetBool(0, CopyOutputDataIndex, CopyPoints_GetSelectedFlagAttributeId(), false);
}
