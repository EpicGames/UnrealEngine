// Copyright Epic Games, Inc. All Rights Reserved.

[numthreads(64, 1, 1)]
void PCGCullPointsOutsideActorBoundsCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);
	if (ThreadIndex >= GetNumThreads().x) return;

	uint InDataIndex, InElemIndex;
	if (!In_GetThreadData(ThreadIndex, InDataIndex, InElemIndex))
	{
		return;
	}

	uint OutDataIndex, OutElemIndex;
	if (!Out_GetThreadData(ThreadIndex, OutDataIndex, OutElemIndex))
	{
		return;
	}

	// Propagate the 'Removed' status to the output point. Otherwise, this point will not be culled.
	if (In_IsPointRemoved(InDataIndex, InElemIndex))
	{
		Out_RemovePoint(OutDataIndex, OutElemIndex);
		return;
	}

	const float BoundsExpansion = CullPointsOutsideActorBounds_GetBoundsExpansion();
	const float3 InPosition = In_GetPosition(InDataIndex, InElemIndex);
	const float3 ComponentBoundsMin = GetComponentBoundsMin() - BoundsExpansion;
	const float3 ComponentBoundsMax = GetComponentBoundsMax() + BoundsExpansion;
	const bool bInsideBounds = all(InPosition >= ComponentBoundsMin) && all(InPosition < ComponentBoundsMax);

	if (!bInsideBounds)
	{
		Out_RemovePoint(OutDataIndex, OutElemIndex);
		return;
	}

	PCG_COPY_ATTRIBUTES_TO_OUTPUT(Out, In, OutDataIndex, OutElemIndex, InDataIndex, InElemIndex, /*bMetadataOnly=*/false, /*bInitNonCopiedAttributes=*/true);
}
