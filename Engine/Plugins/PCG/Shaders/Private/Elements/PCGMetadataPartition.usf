// Copyright Epic Games, Inc. All Rights Reserved.

// Kernel for GPU implementation of Metadata Partition node.

[numthreads(64, 1, 1)]
void PCGMetadataPartitionCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);

	uint In_DataIndex, In_ElementIndex;
	if (!In_GetThreadData(ThreadIndex, In_DataIndex, In_ElementIndex))
	{
		return;
	}

	if (In_IsPointRemoved(In_DataIndex, In_ElementIndex))
	{
		// Don't need to remove output point as we're appending points to the buffer below.
		return;
	}

	const int PartitionAttributeId = MetadataPartition_GetPartitionAttributeId();
	const int AttributeValue = In_GetStringKey(In_DataIndex, In_ElementIndex, PartitionAttributeId);

	uint Out_DataIndex, Out_ElementIndex;
	MetadataPartition_GetOutputIndices(In_DataIndex, AttributeValue, Out_DataIndex, Out_ElementIndex);

	// Write output point.
	PCG_COPY_ALL_ATTRIBUTES_TO_OUTPUT(Out, In, Out_DataIndex, Out_ElementIndex, In_DataIndex, In_ElementIndex);
}
