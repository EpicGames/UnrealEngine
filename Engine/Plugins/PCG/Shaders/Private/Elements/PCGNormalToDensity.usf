// Copyright Epic Games, Inc. All Rights Reserved.

// Mirrors PCGNormalToDensityMode
#define NORMAL_TO_DENSITY_MODE_Set 0
#define NORMAL_TO_DENSITY_MODE_Minimum 1
#define NORMAL_TO_DENSITY_MODE_Maximum 2
#define NORMAL_TO_DENSITY_MODE_Add 3
#define NORMAL_TO_DENSITY_MODE_Subtract 4
#define NORMAL_TO_DENSITY_MODE_Multiply 5
#define NORMAL_TO_DENSITY_MODE_Divide 6

[numthreads(64, 1, 1)]
void PCGNormalToDensityCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);
	if (ThreadIndex >= GetNumThreads().x) return;

	uint InDataIndex, InElemIndex;
	if (!In_GetThreadData(ThreadIndex, InDataIndex, InElemIndex))
	{
		return;
	}

	uint OutDataIndex, OutElemIndex;
	if (!Out_GetThreadData(ThreadIndex, OutDataIndex, OutElemIndex))
	{
		return;
	}

	// Propagate the 'Removed' status to the output point. Otherwise, this point will not be culled.
	if (In_IsPointRemoved(InDataIndex, InElemIndex))
	{
		Out_RemovePoint(OutDataIndex, OutElemIndex);
		return;
	}

	// Do the actual 'work' of this node.
	const float3 Normal = normalize(NormalToDensity_GetNormal());
	const float Offset = NormalToDensity_GetOffset();
	const float Strength = max(NormalToDensity_GetStrength(), 0.0001);
	const uint DensityMode = NormalToDensity_GetDensityMode();

	const float InDensity = In_GetDensity(InDataIndex, InElemIndex);
	const float4 InRotation =  In_GetRotation(InDataIndex, InElemIndex);
	const float3 PointUp = QuatRotateVector(InRotation, float3(0, 0, 1));
	float NormalDensity = pow(clamp(dot(PointUp, Normal) + Offset, 0.0, 1.0), 1 / Strength);

	float OutDensity = 0;

	if (DensityMode == NORMAL_TO_DENSITY_MODE_Set)
	{
		OutDensity = NormalDensity;
	}
	if (DensityMode == NORMAL_TO_DENSITY_MODE_Minimum)
	{
		OutDensity = min(InDensity, NormalDensity);
	}
	else if (DensityMode == NORMAL_TO_DENSITY_MODE_Maximum)
	{
		OutDensity = max(InDensity, NormalDensity);
	}
	else if (DensityMode == NORMAL_TO_DENSITY_MODE_Add)
	{
		OutDensity = InDensity + NormalDensity;
	}
	else if (DensityMode == NORMAL_TO_DENSITY_MODE_Subtract)
	{
		OutDensity = InDensity - NormalDensity;
	}
	else if (DensityMode == NORMAL_TO_DENSITY_MODE_Multiply)
	{
		OutDensity = InDensity * NormalDensity;
	}
	else if (DensityMode == NORMAL_TO_DENSITY_MODE_Divide)
	{
		OutDensity = (NormalDensity != 0) ? InDensity / NormalDensity  : 0.0f;
	}
	
	// Copy all attributes
	PCG_COPY_ALL_ATTRIBUTES_TO_OUTPUT(Out, In, OutDataIndex, OutElemIndex, InDataIndex, InElemIndex);
	
	// Write output density.
	Out_SetDensity(OutDataIndex, OutElemIndex, OutDensity);
}
