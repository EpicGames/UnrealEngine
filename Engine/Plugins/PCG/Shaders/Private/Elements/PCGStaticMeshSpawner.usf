// Copyright Epic Games, Inc. All Rights Reserved.

// Static mesh spawner kernel. Selects a primitive either using a primitive cumulative distribution function (CDF) or by an attribute, and packs attributes in custom floats.
// Passes through points to output.

// When spawning by attribute, we don't know in advance how many instances of each mesh will be spawned, so the worst case NumPoints * NumPrims is allocated in the GPU Scene.

#define PCG_PRODUCE_OUTPUT_POINTS {ProduceOutputPoints}

[numthreads(64, 1, 1)]
void PCGStaticMeshSpawnerCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);

	uint In_DataIndex, In_ElementIndex;
	if (!In_GetThreadData(ThreadIndex, In_DataIndex, In_ElementIndex))
	{
		return;
	}

#if PCG_PRODUCE_OUTPUT_POINTS
	uint Out_DataIndex, Out_ElementIndex;
	if (!Out_GetThreadData(ThreadIndex, Out_DataIndex, Out_ElementIndex))
	{
		// Should never happen.
		return;
	}
#endif // PCG_PRODUCE_OUTPUT_POINTS

	if (In_IsPointRemoved(In_DataIndex, In_ElementIndex))
	{
#if PCG_PRODUCE_OUTPUT_POINTS
		// Propagate the 'Removed' status to the output point if input point is removed. Otherwise, this point will not be culled.
		Out_RemovePoint(Out_DataIndex, Out_ElementIndex);
#endif // PCG_PRODUCE_OUTPUT_POINTS

		return;
	}

	uint PrimitiveIndex = (uint)-1;
	uint CullingCellId = 0;

	const int NumPrimitives = InstanceData_GetNumPrimitives();

	if (NumPrimitives > 0)
	{
		const uint SelectorAttributeId = SMSpawner_GetSelectorAttributeId();
		
		if (SelectorAttributeId != (uint)-1)
		{
			// Primitive ID is either the stringkey of the mesh path, or an index into a primitive data table.
			const int PrimitiveId = In_GetInt(In_DataIndex, In_ElementIndex, SelectorAttributeId);
			PrimitiveIndex = SMSpawner_GetPrimitiveIndexFromPrimitiveId(PrimitiveId);
		}
		else
		{
			// Randomly select a primitive by inverting the cumulative distribution function.
			
			// Exactly matches UPCGBlueprintHelpers::GetRandomStreamFromPoint - three seeds passed in to consecutive arguments of ComputeSeed
			// (argument count & order matters).
			const uint SeedPoint = In_GetSeed(In_DataIndex, In_ElementIndex);
			const uint SeedSettings = GetSettingsSeed();
			const uint SeedComponent = GetComponentSeed();
			uint Seed = ComputeSeed(SeedPoint, SeedSettings, SeedComponent);

			const float RandomDraw = FRand(Seed);

			for (PrimitiveIndex = 0; PrimitiveIndex < uint(NumPrimitives) - 1; ++PrimitiveIndex)
			{
				if (RandomDraw < SMSpawner_GetPrimitiveSelectionCDF(PrimitiveIndex))
				{
					break;
				}
			}
		}

		const int NumCullingCellsTotal = SMSpawner_GetNumCullingCellsTotal();
		if (NumCullingCellsTotal > 1)
		{
			const float3 Pos = In_GetPosition(In_DataIndex, In_ElementIndex);
			const int3 CullingCellCoord = int3(floor(Pos / SMSpawner_GetBucketExtent()));
			int3 LocalCullingCellCoord = CullingCellCoord - int3(floor(SMSpawner_GetBoundsMin() / SMSpawner_GetBucketExtent()));

			const int3 NumCullingCells = SMSpawner_GetNumCullingCells();
			LocalCullingCellCoord = clamp(LocalCullingCellCoord, 0, NumCullingCells - 1);

			CullingCellId = (NumCullingCells.y * NumCullingCells.x * LocalCullingCellCoord.z) + (NumCullingCells.x * LocalCullingCellCoord.y) + LocalCullingCellCoord.x;
		}
	}

	// Get instance index to write to. Will return a negative value if we have saturated instances for this primitive, in which case we will not push the data.
	int InstanceIndexInt = -1;
	if (PrimitiveIndex != (uint)-1)
	{
		InstanceIndexInt = InstanceData_GetIndexToWriteTo(PrimitiveIndex, CullingCellId);
	}

	// Write instance transform, only if not culled by primitive instance index
	if (InstanceIndexInt >= 0)
	{
		const uint InstanceIndex = InstanceIndexInt;
		const float4x4 Transform = In_GetPointTransform(In_DataIndex, In_ElementIndex);
		InstanceData_WriteInstanceLocalToWorld(InstanceIndex, transpose(Transform));

		// Write instance custom float data.
		const uint NumAttributes = SMSpawner_GetNumAttributes();
		for (uint AttributeIndex = 0; AttributeIndex < NumAttributes; ++AttributeIndex)
		{
			const uint4 AttributeIdOffsetStride = SMSpawner_GetAttributeIdOffsetStride(AttributeIndex);
			const uint AttributeId = AttributeIdOffsetStride[0];
			const uint AttributeOffset = AttributeIdOffsetStride[1];
			const uint AttributeStride = AttributeIdOffsetStride[2];

			for (uint FloatIndex = 0; FloatIndex < AttributeStride; ++FloatIndex)
			{
				const uint AttributeValueAddress = In_GetElementAddressInternal(In_DataIndex, In_ElementIndex, AttributeId);
				if (AttributeValueAddress > 0)
				{
					// Assumes floating point data. TODO add type information and deal with different types in different ways?
					InstanceData_WriteCustomFloatRaw(InstanceIndex, AttributeOffset + FloatIndex, In_LoadBufferInternal(AttributeValueAddress + 4 * FloatIndex));
				}
			}
		}
	}

#if PCG_PRODUCE_OUTPUT_POINTS
	// Write output point data (pass through).
	Out_SetPosition(Out_DataIndex, Out_ElementIndex, In_GetPosition(In_DataIndex, In_ElementIndex));
	Out_SetRotation(Out_DataIndex, Out_ElementIndex, In_GetRotation(In_DataIndex, In_ElementIndex));
	Out_SetScale(Out_DataIndex, Out_ElementIndex, In_GetScale(In_DataIndex, In_ElementIndex));
	Out_SetColor(Out_DataIndex, Out_ElementIndex, In_GetColor(In_DataIndex, In_ElementIndex));
	Out_SetSeed(Out_DataIndex, Out_ElementIndex, In_GetSeed(In_DataIndex, In_ElementIndex));
	Out_SetDensity(Out_DataIndex, Out_ElementIndex, In_GetDensity(In_DataIndex, In_ElementIndex));
	Out_SetSteepness(Out_DataIndex, Out_ElementIndex, In_GetSteepness(In_DataIndex, In_ElementIndex));

	// Either apply SM bounds to points or pass bounds through.
	if (PrimitiveIndex != (uint)-1 && SMSpawner_ShouldApplyBounds())
	{
		Out_SetBoundsMin(Out_DataIndex, Out_ElementIndex, SMSpawner_GetPrimitiveMeshBoundsMin(PrimitiveIndex));
		Out_SetBoundsMax(Out_DataIndex, Out_ElementIndex, SMSpawner_GetPrimitiveMeshBoundsMax(PrimitiveIndex));
	}
	else
	{
		Out_SetBoundsMin(Out_DataIndex, Out_ElementIndex, In_GetBoundsMin(In_DataIndex, In_ElementIndex));
		Out_SetBoundsMax(Out_DataIndex, Out_ElementIndex, In_GetBoundsMax(In_DataIndex, In_ElementIndex));
	}

	PCG_COPY_METADATA_ATTRIBUTES_TO_OUTPUT(Out, In, Out_DataIndex, Out_ElementIndex, In_DataIndex, In_ElementIndex);

	if (SMSpawner_GetSelectedMeshAttributeId() != (uint)-1)
	{
		int OutputMeshStringKey = -1;
		if (PrimitiveIndex != (uint)-1)
		{
			OutputMeshStringKey = SMSpawner_GetPrimitiveStringKey(PrimitiveIndex);
		}

		Out_SetStringKey(Out_DataIndex, Out_ElementIndex, SMSpawner_GetSelectedMeshAttributeId(), OutputMeshStringKey);
	}
#endif // PCG_PRODUCE_OUTPUT_POINTS
}

#undef PCG_PRODUCE_OUTPUT_POINTS
