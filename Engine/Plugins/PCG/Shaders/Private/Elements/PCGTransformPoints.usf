// Copyright Epic Games, Inc. All Rights Reserved.

[numthreads(64, 1, 1)]
void PCGTransformPointsCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, 64);
	if (ThreadIndex >= GetNumThreads().x) return;

	uint InDataIndex, InElemIndex;
	if (!In_GetThreadData(ThreadIndex, InDataIndex, InElemIndex))
	{
		return;
	}

	uint OutDataIndex, OutElemIndex;
	if (!Out_GetThreadData(ThreadIndex, OutDataIndex, OutElemIndex))
	{
		return;
	}

	// Propagate the 'Removed' status to the output point. Otherwise, this point will not be culled.
	if (In_IsPointRemoved(InDataIndex, InElemIndex))
	{
		Out_RemovePoint(OutDataIndex, OutElemIndex);
		return;
	}

	// Transform points code
	const float3 OffsetMin = OffsetMin_GetOverridableValue();
	const float3 OffsetMax = OffsetMax_GetOverridableValue();
	const bool bAbsoluteOffset = bAbsoluteOffset_GetOverridableValue() != 0;
	const float3 RotationMin = RotationMin_GetOverridableValue();
	const float3 RotationMax = RotationMax_GetOverridableValue();
	const bool bAbsoluteRotation = bAbsoluteRotation_GetOverridableValue() != 0;
	const float3 ScaleMin = ScaleMin_GetOverridableValue();
	const float3 ScaleMax = ScaleMax_GetOverridableValue();
	const bool bAbsoluteScale = bAbsoluteScale_GetOverridableValue() != 0;
	const bool bUniformScale = bUniformScale_GetOverridableValue() != 0;
	const bool bRecomputeSeed = bRecomputeSeed_GetOverridableValue() != 0;
	const int AttributeId = TransformPoints_GetAttributeId();
	const bool bApplyToAttribute = (TransformPoints_GetApplyToAttribute() != 0) && (AttributeId != -1);

	float3 InPosition;
	FQuat InRotation;
	float3 InScale;

	if (bApplyToAttribute)
	{
		const float4x4 InTransform = In_GetTransform(InDataIndex, InElemIndex, AttributeId);

		InScale = float3(length(InTransform._m00_m10_m20), length(InTransform._m01_m11_m21), length(InTransform._m02_m12_m22));

		// Assumes Transform axes have the correct handedness. We could do more work to fix this up too if required.
		float3x3 RotationMatix = (float3x3)transpose(InTransform);
		// Required unfortunately, Quat_FromMatrix expects orthonormalized.
		RotationMatix[0] /= InScale.x;
		RotationMatix[1] /= InScale.y;
		RotationMatix[2] /= InScale.z;
		InRotation = Quat_FromMatrix(RotationMatix);

		InPosition = InTransform._m03_m13_m23;
	}
	else
	{
		InPosition = In_GetPosition(InDataIndex, InElemIndex);
		InRotation = In_GetRotation(InDataIndex, InElemIndex);
		InScale = In_GetScale(InDataIndex, InElemIndex);
	}
	
	const uint InSeed = In_GetSeed(InDataIndex, InElemIndex);
	uint Seed = ComputeSeed(GetSeed(), InSeed);

	// Position
	float3 PositionOffset = float3(
		lerp(OffsetMin.x, OffsetMax.x, FRand(Seed)),
		lerp(OffsetMin.y, OffsetMax.y, FRand(Seed)),
		lerp(OffsetMin.z, OffsetMax.z, FRand(Seed)));

	if (!bAbsoluteOffset)
	{
		PositionOffset = Quat_RotateVector(InRotation, PositionOffset);
	}

	const float3 OutPosition = InPosition + PositionOffset;

	// Rotation
	FQuat OutRotation = bAbsoluteRotation ? FQuat(0, 0, 0, 1) : InRotation;
	// RotationMin/Max are in raw FRotator order: (Pitch, Yaw, Roll).
	// Random numbers drawn in same order as CPU: Pitch, Yaw, Roll.
	const float RandomPitch = lerp(RotationMin.x, RotationMax.x, FRand(Seed));
	const float RandomYaw   = lerp(RotationMin.y, RotationMax.y, FRand(Seed));
	const float RandomRoll  = lerp(RotationMin.z, RotationMax.z, FRand(Seed));

	// Quat_FromRotator expects Euler order: (Roll, Pitch, Yaw).
	OutRotation = Quat_Multiply(OutRotation, Quat_FromRotator(float3(RandomRoll, RandomPitch, RandomYaw)));

	// Scale
	float3 OutScale = bAbsoluteScale ? float3(1, 1, 1) : InScale;
	if (bUniformScale)
	{
		OutScale *= lerp(ScaleMin.x, ScaleMax.x, FRand(Seed));
	}
	else
	{
		OutScale.x *= lerp(ScaleMin.x, ScaleMax.x, FRand(Seed));
		OutScale.y *= lerp(ScaleMin.y, ScaleMax.y, FRand(Seed));
		OutScale.z *= lerp(ScaleMin.z, ScaleMax.z, FRand(Seed));
	}
	
	// Copy metadata attributes first.
	PCG_COPY_ATTRIBUTES_TO_OUTPUT(Out, In, OutDataIndex, OutElemIndex, InDataIndex, InElemIndex, /*bMetadataOnly=*/true, /*bInitNonCopiedAttributes=*/true);

	// Write output.
	if (bApplyToAttribute)
	{
		const float3x3 RotationMatrix = Quat_ToMatrix(OutRotation);

		const float3 Axis0 = OutScale.x * RotationMatrix[0];
		const float3 Axis1 = OutScale.y * RotationMatrix[1];
		const float3 Axis2 = OutScale.z * RotationMatrix[2];
		
		float4x4 OutTransform = float4x4(
			Axis0.x, Axis1.x, Axis2.x, OutPosition.x,
			Axis0.y, Axis1.y, Axis2.y, OutPosition.y,
			Axis0.z, Axis1.z, Axis2.z, OutPosition.z,
			(float3)0.0, 1.0);

		Out_SetTransform(OutDataIndex, OutElemIndex, AttributeId, OutTransform);

		// Copy original
		Out_SetPosition(OutDataIndex, OutElemIndex, In_GetPosition(InDataIndex, InElemIndex));
		Out_SetRotation(OutDataIndex, OutElemIndex, In_GetRotation(InDataIndex, InElemIndex));
		Out_SetScale(OutDataIndex, OutElemIndex, In_GetScale(InDataIndex, InElemIndex));
	}
	else
	{
		Out_SetPosition(OutDataIndex, OutElemIndex, OutPosition);
		Out_SetRotation(OutDataIndex, OutElemIndex, OutRotation);
		Out_SetScale(OutDataIndex, OutElemIndex, OutScale);
	}

	if (bApplyToAttribute || !bRecomputeSeed)
	{
		Out_SetSeed(OutDataIndex, OutElemIndex, InSeed);
	}
	else
	{
		Out_SetSeed(OutDataIndex, OutElemIndex, ComputeSeedFromPosition(OutPosition));
	}

	// We didn't modify these properties, so simply copy from the input data.
	Out_SetColor(OutDataIndex, OutElemIndex, In_GetColor(InDataIndex, InElemIndex));
	Out_SetBoundsMin(OutDataIndex, OutElemIndex, In_GetBoundsMin(InDataIndex, InElemIndex));
	Out_SetBoundsMax(OutDataIndex, OutElemIndex, In_GetBoundsMax(InDataIndex, InElemIndex));
	Out_SetDensity(OutDataIndex, OutElemIndex, In_GetDensity(InDataIndex, InElemIndex));
	Out_SetSteepness(OutDataIndex, OutElemIndex, In_GetSteepness(InDataIndex, InElemIndex));
}
