// Copyright Epic Games, Inc. All Rights Reserved.

// Buffer format:
//
// -- START HEADER --
// NumData (4B)
// For each data (1036B when MAX_NUM_ATTR=128):
//   TypeId (4B)
//   Attribute Count (4B)
//   Element Count (4B)
//   For attribute ID in [0, MAX_NUM_ATTR-1]:
//     PackedAttributeInfo (4B)
//         Stride (8b)
//         Id (23b)
//         Allocated (1b - MSB)
//     AttributeElementStartAddress (4B)
// -- END HEADER --
// For each data:
//   For each present attribute:
//     <Packed element data> (element count * stride bytes) - Note: element count may be 1 if the attribute is not 'Allocated' (that is, uses a single value for the entire range).

ByteAddressBuffer {DataInterfaceName}_DataCollectionBufferSRV;
RWByteAddressBuffer {DataInterfaceName}_DataCollectionBufferUAV;

// Multiplier that can be configured on output pins.
uint {DataInterfaceName}_ElementCountMultiplier;

// Union of EPCGPointNativeProperties bits across ALL output data items. Non-zero bit = that property is N-slot (fully allocated)
// in at least one data item. Set from C++ at dispatch time; cheaper to read than querying the buffer header per-thread.
// Consequence for multi-data: if a property is N-slot in any data item, the preamble does per-element copies for ALL data
// items, including those where the property is CVR - those writes are redundant but harmless (all hit slot 0 via bAllocated clamping).
// todo_pcg: could be moved into the per-data GPU buffer header, making it truly per-data and eliminating redundant preamble copies.
uint {DataInterfaceName}_AllocatedNativePropertiesMask;

StructuredBuffer<int> {DataInterfaceName}_GraphToBufferAttributeId;

int {DataInterfaceName}_FirstRemappedAttributeId;

StructuredBuffer<int> {DataInterfaceName}_BufferToGraphStringKey;

int {DataInterfaceName}_NumRemappedStringKeys;

struct FPCGAttributeHeader_{DataInterfaceName}
{
	/** Absolute address of this attribute's first data element in the buffer. */
	uint FirstElementAddress;

	/** Stride (in bytes) of this attribute. */
	uint Stride;

	/** Whether or not this attribute is full-width allocated. */
	bool bAllocated;
};

// #################### SHADER PARAM GETTERS ##########################

// @todo_pcg: This parameter is broader than this data interface. Needs to be shared in some way (e.g. with TextureDataInterface).
uint GetElementCountMultiplier_{DataInterfaceName}()
{
	return {DataInterfaceName}_ElementCountMultiplier;
}

// Returns the union of EPCGPointNativeProperties bits across all output data items. Non-zero bit = that property is N-slot
// (fully allocated) in at least one data item. Used in the processor preamble to branch between CVR (write at element 0)
// and N-slot (write per-element) copies. See comment on the uniform above for multi-data implications.
uint GetAllocatedNativePropertiesMask_{DataInterfaceName}()
{
	return {DataInterfaceName}_AllocatedNativePropertiesMask;
}

// #################### HEADER READERS ##########################

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for both SRV and UAV use.
{{BeginDuplicateForSRVUAV}}

uint GetNumData_{NameSuffix}_{DataInterfaceName}()
{
	return {DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load(0);
}

uint GetNumElements_{NameSuffix}_{DataInterfaceName}(uint InDataIndex)
{
	const uint ReadAddress =
		PCG_DATA_COLLECTION_HEADER_SIZE_BYTES +
		InDataIndex * PCG_DATA_HEADER_SIZE_BYTES +
		/*TypeId*/4 + /*Attribute Count*/4;

	return {DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load(ReadAddress);
}

bool GetThreadData_{NameSuffix}_{DataInterfaceName}(uint InThreadIndex, out uint OutDataIndex, out uint OutElementIndex)
{
	int ElementIndex = (int)InThreadIndex;

	const uint NumData = GetNumData_{NameSuffix}_{DataInterfaceName}();
	for (uint DataIndex = 0; DataIndex < NumData; ++DataIndex)
	{
		const int ElemCount = (int)GetNumElements_{NameSuffix}_{DataInterfaceName}(DataIndex);

		if (ElementIndex < ElemCount)
		{
			OutDataIndex = DataIndex;
			OutElementIndex = (uint)ElementIndex;
			return true;
		}

		ElementIndex -= ElemCount;
	}

	OutDataIndex = (uint)-1;
	OutElementIndex = (uint)-1;
	return false;
}

uint GetNumElements_{NameSuffix}_{DataInterfaceName}()
{
	uint NumElements = 0;

	const uint NumData = GetNumData_{NameSuffix}_{DataInterfaceName}();
	for (uint DataIndex = 0; DataIndex < NumData; ++DataIndex)
	{
		NumElements += GetNumElements_{NameSuffix}_{DataInterfaceName}(DataIndex);
	}

	return NumElements;
}

{{EndDuplicateForSRVUAV}}

// #################### INTERNAL HELPERS ##########################

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for both SRV and UAV use.
{{BeginDuplicateForSRVUAV}}

uint LoadBufferInternal_{NameSuffix}_{DataInterfaceName}(uint InAddress)
{
	return {DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load(InAddress);
}

{{EndDuplicateForSRVUAV}}

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for UAV use.
{{BeginUAVOnly}}

void StoreBufferInternal_{NameSuffix}_{DataInterfaceName}(uint InAddress, uint InValue)
{
	{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store(InAddress, InValue);
}

{{EndUAVOnly}}

int GetRemappedAttributeIdInternal_{DataInterfaceName}(int InGraphAttributeId)
{
	if (InGraphAttributeId >= {DataInterfaceName}_FirstRemappedAttributeId)
	{
		// Convert from current compute graph attribute ID present in buffer (required if buffer came from a different upstream compute graph).
		// Note: InGraphAttributeId should always be >= PCG_NUM_RESERVED_ATTRS, but max() added to appease compilers.
		return PCG_NUM_RESERVED_ATTRS + {DataInterfaceName}_GraphToBufferAttributeId[max(0, InGraphAttributeId - PCG_NUM_RESERVED_ATTRS)];
	}
	else
	{
		return InGraphAttributeId;
	}
}

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for both SRV and UAV use.
{{BeginDuplicateForSRVUAV}}
FPCGAttributeHeader_{DataInterfaceName} GetAttributeHeaderInternal_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, int InAttributeId)
{
	const int AttributeId = GetRemappedAttributeIdInternal_{DataInterfaceName}(InAttributeId);

	const uint ReadAddress =
		PCG_DATA_COLLECTION_HEADER_SIZE_BYTES +
		InDataIndex * PCG_DATA_HEADER_SIZE_BYTES +
		/*TypeId*/4 + /*Attribute Count*/4 + /*Element Count*/4 +
		AttributeId * PCG_ATTRIBUTE_HEADER_SIZE_BYTES;

	const uint2 PackedAttributeHeader = {DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load2(ReadAddress);

	FPCGAttributeHeader_{DataInterfaceName} AttributeHeader;
	AttributeHeader.FirstElementAddress = PackedAttributeHeader.y;
	AttributeHeader.Stride = PackedAttributeHeader.x & PCG_ATTRIBUTE_STRIDE_MASK;
	AttributeHeader.bAllocated = (PackedAttributeHeader.x & PCG_ATTRIBUTE_ALLOCATED_MASK) != 0;

	return AttributeHeader;
}

uint2 GetElementAddressAndStrideInternal_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	const FPCGAttributeHeader_{DataInterfaceName} AttributeHeader = GetAttributeHeaderInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InAttributeId);
	const uint ElementIndex = AttributeHeader.bAllocated ? InElementIndex : 0u;

	return uint2(ElementIndex * AttributeHeader.Stride + AttributeHeader.FirstElementAddress, AttributeHeader.Stride);
}

uint GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	return GetElementAddressAndStrideInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId).x;
}

uint GetFirstValidAttributeElementInternal_{NameSuffix}_{DataInterfaceName}()
{
	for (int AttributeId = 0; AttributeId < PCG_MAX_NUM_ATTRS; ++AttributeId)
	{
		const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(/*DataIndex=*/0, /*ElementIndex=*/0, AttributeId);

		if (ElementAddress > 0)
		{
			return ElementAddress;
		}
	}

	return 0;
}

uint GetDataNumAttributesInternal_{NameSuffix}_{DataInterfaceName}(uint InDataIndex)
{
	const uint ReadAddress =
		PCG_DATA_COLLECTION_HEADER_SIZE_BYTES +
		InDataIndex * PCG_DATA_HEADER_SIZE_BYTES +
		/*TypeId*/4;

	return {DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load(ReadAddress);
}
{{EndDuplicateForSRVUAV}}

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for UAV use.
{{BeginUAVOnly}}

// Deprecated (5.8)
void SetAsExecutedInternal_{NameSuffix}_{DataInterfaceName}()
{
}

{{EndUAVOnly}}

// #################### ATTRIBUTE GETTERS ##########################

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for both SRV and UAV use.
{{BeginDuplicateForSRVUAV}}

uint GetUint_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		return {DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load(ElementAddress);
	}
	else
	{
		return 0u;
	}
}

bool GetBool_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	return GetUint_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId) != 0;
}

int GetInt_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	return asint(GetUint_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId));
}

float GetFloat_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	return asfloat(GetUint_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId));
}

float2 GetFloat2_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		return asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load2(ElementAddress));
	}
	else
	{
		return (float2)0;
	}
}

float3 GetFloat3_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		return asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load3(ElementAddress));
	}
	else
	{
		return (float3)0;
	}
}

float4 GetFloat4_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		return asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load4(ElementAddress));
	}
	else
	{
		return (float4)0;
	}
}

float3 GetRotator_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	return GetFloat3_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);
}

float4 GetQuat_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	return GetFloat4_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);
}

float4x4 GetTransform_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		return float4x4(
			asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load4(ElementAddress)),
			asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load4(ElementAddress + 4 * 4)),
			asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load4(ElementAddress + 8 * 4)),
			asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load4(ElementAddress + 12 * 4)));
	}
	else
	{
		return (float4x4)0;
	}
}

int GetStringKey_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	// String keys are represented as ints.
	const int StringKey = GetInt_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);
	if (StringKey < {DataInterfaceName}_NumRemappedStringKeys)
	{
		return {DataInterfaceName}_BufferToGraphStringKey[StringKey];
	}
	else
	{
		return StringKey;
	}
}

int GetName_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId)
{
	// Names are currently put in the string table.
	return GetStringKey_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);
}

{{EndDuplicateForSRVUAV}}

// #################### ATTRIBUTE SETTERS ##########################

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for UAV use.
{{BeginUAVOnly}}

void SetBool_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, bool InValue)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store(ElementAddress, InValue ? 1u : 0u);
	}
}

void SetInt_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, int InValue)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store(ElementAddress, asuint(InValue));
	}
}

void SetUint_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, uint InValue)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store(ElementAddress, InValue);
	}
}

void SetFloat_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, float InValue)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store(ElementAddress, asuint(InValue));
	}
}

void SetFloat2_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, float2 InValue)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store2(ElementAddress, asuint(InValue));
	}
}

void SetFloat3_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, float3 InValue)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store3(ElementAddress, asuint(InValue));
	}
}

void SetFloat4_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, float4 InValue)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store4(ElementAddress, asuint(InValue));
	}
}

void SetRotator_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, float3 InValue)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store3(ElementAddress, asuint(InValue));
	}
}

void SetQuat_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, float4 InValue)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store4(ElementAddress, asuint(InValue));
	}
}

void SetTransform_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, float4x4 InValue)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store4(ElementAddress, asuint(InValue[0]));
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store4(ElementAddress + 4 * 4, asuint(InValue[1]));
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store4(ElementAddress + 8 * 4, asuint(InValue[2]));
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Store4(ElementAddress + 12 * 4, asuint(InValue[3]));
	}
}

void SetStringKey_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, int InValue)
{
	// String keys are represented as ints.
	SetInt_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId, InValue);
}

void SetName_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, int InValue)
{
	// Names are represented as ints.
	SetInt_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId, InValue);
}

{{EndUAVOnly}}

// #################### FIRST ATTRIBUTE GETTERS ##########################

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for both SRV and UAV use.
{{BeginDuplicateForSRVUAV}}
uint GetFirstAttributeAsUint_{NameSuffix}_{DataInterfaceName}()
{
	const uint AttributeAddress = GetFirstValidAttributeElementInternal_{NameSuffix}_{DataInterfaceName}();

	if (AttributeAddress > 0)
	{
		return {DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load(AttributeAddress);
	}
	else
	{
		return 0;
	}
}

bool GetFirstAttributeAsBool_{NameSuffix}_{DataInterfaceName}()
{
	return GetFirstAttributeAsUint_{NameSuffix}_{DataInterfaceName}() != 0;
}

int GetFirstAttributeAsInt_{NameSuffix}_{DataInterfaceName}()
{
	return asint(GetFirstAttributeAsUint_{NameSuffix}_{DataInterfaceName}());
}

float GetFirstAttributeAsFloat_{NameSuffix}_{DataInterfaceName}()
{
	return asfloat(GetFirstAttributeAsUint_{NameSuffix}_{DataInterfaceName}());
}

float2 GetFirstAttributeAsFloat2_{NameSuffix}_{DataInterfaceName}()
{
	const uint AttributeAddress = GetFirstValidAttributeElementInternal_{NameSuffix}_{DataInterfaceName}();

	if (AttributeAddress > 0)
	{
		return asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load2(AttributeAddress));
	}
	else
	{
		return (float2)0;
	}
}

float3 GetFirstAttributeAsFloat3_{NameSuffix}_{DataInterfaceName}()
{
	const uint AttributeAddress = GetFirstValidAttributeElementInternal_{NameSuffix}_{DataInterfaceName}();

	if (AttributeAddress > 0)
	{
		return asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load3(AttributeAddress));
	}
	else
	{
		return (float3)0;
	}
}

float4 GetFirstAttributeAsFloat4_{NameSuffix}_{DataInterfaceName}()
{
	const uint AttributeAddress = GetFirstValidAttributeElementInternal_{NameSuffix}_{DataInterfaceName}();

	if (AttributeAddress > 0)
	{
		return asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load4(AttributeAddress));
	}
	else
	{
		return (float4)0;
	}
}

float3 GetFirstAttributeAsRotator_{NameSuffix}_{DataInterfaceName}()
{
	return GetFirstAttributeAsFloat3_{NameSuffix}_{DataInterfaceName}();
}

float4 GetFirstAttributeAsQuat_{NameSuffix}_{DataInterfaceName}()
{
	return GetFirstAttributeAsFloat4_{NameSuffix}_{DataInterfaceName}();
}

float4x4 GetFirstAttributeAsTransform_{NameSuffix}_{DataInterfaceName}()
{
	const uint AttributeAddress = GetFirstValidAttributeElementInternal_{NameSuffix}_{DataInterfaceName}();
	if (AttributeAddress == 0)
	{
		return (float4x4)0;
	}

	return float4x4(
		asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load4(AttributeAddress)),
		asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load4(AttributeAddress + 4 * 4)),
		asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load4(AttributeAddress + 8 * 4)),
		asfloat({DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.Load4(AttributeAddress + 12 * 4))
	);
}

int GetFirstAttributeAsStringKey_{NameSuffix}_{DataInterfaceName}()
{
	// String keys are represented as ints.
	return GetFirstAttributeAsInt_{NameSuffix}_{DataInterfaceName}();
}

int GetFirstAttributeAsName_{NameSuffix}_{DataInterfaceName}()
{
	// Names are represented as ints.
	return GetFirstAttributeAsInt_{NameSuffix}_{DataInterfaceName}();
}
{{EndDuplicateForSRVUAV}}

// #################### ATOMICS ##########################

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for UAV use.
{{BeginUAVOnly}}

// Returns value of attribute before incrementing.
int AtomicAddInt_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, int InValueToAdd)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		uint OriginalValue;
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.InterlockedAdd(ElementAddress, (uint)InValueToAdd, OriginalValue);

		return (int)OriginalValue;
	}
	else
	{
		return 0;
	}
}

// Returns value of attribute before max operation.
uint AtomicMaxUint_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, int InAttributeId, uint InValueToMax)
{
	const uint ElementAddress = GetElementAddressInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, InAttributeId);

	if (ElementAddress > 0)
	{
		uint OriginalValue;
		{DataInterfaceName}_DataCollectionBuffer{BufferSuffix}.InterlockedMax(ElementAddress, InValueToMax, OriginalValue);

		return OriginalValue;
	}
	else
	{
		return 0u;
	}
}

{{EndUAVOnly}}

// #################### POINT ATTRIBUTE GETTERS ##########################

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for both SRV and UAV use.
{{BeginDuplicateForSRVUAV}}
float3 GetPosition_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	return GetFloat3_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_POSITION_ID);
}

float4 GetRotation_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	return GetFloat4_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_ROTATION_ID);
}

float3 GetScale_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	return GetFloat3_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_SCALE_ID);
}

float3 GetBoundsMin_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	return GetFloat3_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_BOUNDS_MIN_ID);
}

float3 GetBoundsMax_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	return GetFloat3_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_BOUNDS_MAX_ID);
}

float4 GetColor_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	return GetFloat4_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_COLOR_ID);
}

float GetDensity_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	return GetFloat_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_DENSITY_ID);
}

float GetSteepness_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	return GetFloat_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_STEEPNESS_ID);
}

uint GetSeed_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	return GetUint_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_SEED_ID);
}

float4x4 GetPointTransform_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	const FQuat Rotation = GetRotation_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex);
	const half3x3 RotationMatrix = QuatToMatrix(Rotation);
	const float3 Position = GetPosition_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex);
	const float3 Scale = GetScale_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex);

	const float3 Axis0 = Scale.x * RotationMatrix[0];
	const float3 Axis1 = Scale.y * RotationMatrix[1];
	const float3 Axis2 = Scale.z * RotationMatrix[2];

	return float4x4(
		Axis0.x, Axis1.x, Axis2.x, Position.x,
		Axis0.y, Axis1.y, Axis2.y, Position.y,
		Axis0.z, Axis1.z, Axis2.z, Position.z,
		(float3)0.0, 1.0);
}

bool IsPointRemoved_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	return GetDensity_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex) == PCG_INVALID_DENSITY;
}
{{EndDuplicateForSRVUAV}}

// #################### POINT ATTRIBUTE SETTERS ##########################

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for UAV use.
{{BeginUAVOnly}}

void SetPosition_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, float3 Position)
{
	SetFloat3_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_POSITION_ID, Position);
}

void SetRotation_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, float4 Rotation)
{
	SetFloat4_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_ROTATION_ID, Rotation);
}

void SetScale_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, float3 Scale)
{
	SetFloat3_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_SCALE_ID, Scale);
}

void SetBoundsMin_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, float3 BoundsMin)
{
	SetFloat3_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_BOUNDS_MIN_ID, BoundsMin);
}

void SetBoundsMax_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, float3 BoundsMax)
{
	SetFloat3_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_BOUNDS_MAX_ID, BoundsMax);
}

void SetColor_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, float4 Color)
{
	SetFloat4_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_COLOR_ID, Color);
}

void SetDensity_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, float Density)
{
	SetFloat_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_DENSITY_ID, Density);
}

void SetSteepness_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, float Steepness)
{
	SetFloat_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_STEEPNESS_ID, Steepness);
}

void SetSeed_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, uint Seed)
{
	SetUint_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_POINT_SEED_ID, Seed);
}

void SetPointTransform_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex, float4x4 Transform)
{
	const float3 Scale = float3(length(Transform._m00_m10_m20), length(Transform._m01_m11_m21), length(Transform._m02_m12_m22));
	SetScale_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, Scale);

	// Assumes Transform axes have the correct handedness. We could do more work to fix this up too if required.
	float3x3 RotationMatix = (float3x3)transpose(Transform);
	// Required unfortunately, QuatFromMatrix expects orthonormalized.
	RotationMatix[0] /= Scale.x;
	RotationMatix[1] /= Scale.y;
	RotationMatix[2] /= Scale.z;
	const FQuat Rotation = QuatFromMatrix(RotationMatix);
	SetRotation_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, Rotation);

	const float3 Position = Transform._m03_m13_m23;
	SetPosition_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, Position);
}

void RemovePoint_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	SetDensity_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, PCG_INVALID_DENSITY);
}

// Zero-initialize all metadata (user-defined) attributes for a single element.
void InitializeMetadata_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	int NumAttributesRemaining = (int)GetDataNumAttributesInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex);

	for (int AttributeId = PCG_NUM_RESERVED_ATTRS; AttributeId < PCG_MAX_NUM_ATTRS; ++AttributeId)
	{
		const uint2 ElementAddressAndStride = GetElementAddressAndStrideInternal_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, AttributeId);
		const uint ElementAddress = ElementAddressAndStride.x;
		const uint Stride = ElementAddressAndStride.y;

		if (Stride == 0)
		{
			// No output attribute to write to.
			continue;
		}

		// We don't bother checking element address is nonzero as we already check Stride above.
		for (uint Offset = 0; Offset < Stride; Offset += 4)
		{
			// TODO: In future could upload actual default values for attributes rather than 0-initializing.
			StoreBufferInternal_{NameSuffix}_{DataInterfaceName}(ElementAddress + Offset, 0u);
		}

		if (--NumAttributesRemaining <= 0)
		{
			// Early-out when we've looked at all the possible attributes
			break;
		}
	}
}

// Initialize a single point with default values (native properties + metadata attributes).
void InitializePoint_{NameSuffix}_{DataInterfaceName}(uint InDataIndex, uint InElementIndex)
{
	SetPosition_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, 0.0f);
	SetRotation_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, float4(0.0f, 0.0f, 0.0f, 1.0f));
	SetScale_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, 1.0f);
	SetBoundsMin_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, -50.0f);
	SetBoundsMax_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, 50.0f);
	SetColor_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, 1.0f);
	SetDensity_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, 1.0f);
	SetSeed_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, 42);
	SetSteepness_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex, 1.0f);
	InitializeMetadata_{NameSuffix}_{DataInterfaceName}(InDataIndex, InElementIndex);
}

{{EndUAVOnly}}
