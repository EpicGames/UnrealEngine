// Copyright Epic Games, Inc. All Rights Reserved.

// Pixel shader that grows valid (alpha == 1) data outward by one texel per pass into uncovered neighbors.
// Iterating N times produces an N-texel ring of padding around each valid region.

#include "/Engine/Public/Platform.ush"

Texture2D<float4> InputTexture;
int2 ResolutionMinusOne;

void MainPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	const int2 Coord = int2(SvPosition.xy);
	const float4 Self = InputTexture.Load(int3(Coord, 0));

	// If this texel alreaDy has data, pass it through unchanged. Otherwise average all covered neighbors.
	float4 Result = Self;
	if (Self.a != 1.0)
	{
		float4 Sum = 0;
		float Weight = 0;

		UNROLL
		for (int Dy = -1; Dy <= 1; ++Dy)
		{
			UNROLL
			for (int Dx = -1; Dx <= 1; ++Dx)
			{
				if (Dx == 0 && Dy == 0)
				{
					continue;
				}

				const int2 NeighborCoord = clamp(Coord + int2(Dx, Dy), int2(0, 0), ResolutionMinusOne);
				const float4 Neighbor = InputTexture.Load(int3(NeighborCoord, 0));

				if (Neighbor.a == 1.0)
				{
					Sum += Neighbor;
					Weight += 1.0;
				}
			}
		}

		if (Weight > 0.0)
		{
			Result = Sum / Weight;
	
			// Alpha is set to 1 explicitly so the validity bit propagates regardless of how the kernel evolves.
			Result.a = 1.0;
		}
	}

	OutColor = Result;
}
