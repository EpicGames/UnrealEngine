// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Landscape/LandscapeCommon.ush"

// Input:
// First 2 channels of first texture always reserved for heightmap (see constructor of FLandscapeGrassWeightExporter_RenderThread::FComponentInfo).
// So first two grass maps will be in z & w channels of first set of pass, otherwise packed in remaining channels.
// Generated grass maps all end up in one big row (see FLandscapeGrassWeightExporter::FLandscapeGrassWeightExporter()).
//
//  -----------------------------------------
// | LC0  | LC0  | LC1  | LC1  | LC1  | LC2  |
// | x=h0 | x=g2 | x=h0 | x=g2 | x=g6 | x=h0 |  InLandscapeComponentResolution
// | y=h1 | y=   | y=h1 | y=g3 | y=g7 | y=h1 |
// | z=g0 | z=   | z=g0 | z=g4 | z=g8 | z=g0 |
// | w=g1 | w=   | w=g1 | w=g5 | w=   | w=   |
//  -----------------------------------------
// InLandscapeComponentResolution*NumExportedTiles

Texture2D<float4> InPackedGrassMaps;

// This dispatch will in general target a sub area of the output texture corresponding to the exported landscape component(s), defined by InDispatchOffset and InDispatchSize.
// The bottom left corner of the output texture is the origin of the coordinates, and the tiles of landscape components are indexed relative to this origin.
// The shader writes to an output texture with height data and a texture array where each slice is a single grass combined grass map.
// InLandscapeComponentResolution (Res below) is vert count and includes outer edges, so a row/col is removed from each internal boundary.
// Below shows grass map configuration, height map is same mapping but only a single slice.
//
//                         --------------------------------
//                       --------------------------------  |
//                      |  Res     |   Res-1  |   Res-1  | |
//                      |          |          |          | |
//  (Res-1) * NumTilesY |Res-1   +-------------(OW,OH)   | |
//         + 1          |        | Output Texture |      | |
//                      |--------|                |------| |       OW, OH: OutputWidth, InOutputHeight - output texture dimensions
//                      |        |     +--(DO+DS) |      | |       DO, DS: InDispatchOffset, InDispatchSize - domain of dispatched threads
//                      |Res-1   |     |      |   |OH    | |
//                      |        |     |      |   |      | |
//                      |        |    (DO)----+   |      | |
//                      |--------|                |------| |
//                      |      (0,0)--------------+      | | .
//                      |          |         |           | | |
//                      |Res       |         |           | |-
//                      |          |         |           | | NumGrassMaps slices
//                      |   Res    |  Res-1  |  Res-1    |-
//                       --------------------------------
//                            (Res-1) * NumTilesX + 1
//

// Grass map output. Dimension is (OutputWidth, InOutputHeight, NumGrassMaps).
RWTexture2DArray<float> OutUnpackedGrassMaps;

// Height map output. Dimension is (OutputWidth, InOutputHeight).
RWTexture2D<float> OutUnpackedHeight;

StructuredBuffer<int> InComponentHeightOffsets;
StructuredBuffer<int2> InComponentGrassmapOffsetAndChannels;
StructuredBuffer<int> InOutputIndexToExportIndex;
int InNumExportedGrassMaps;

int4 InLinearTileIndexToComponentIndex[PCG_MAX_NUM_LANDSCAPE_COMPONENTS]; // Only x component used, 16 byte alignment on arrays
uint2 InNumTiles;
int2 InTileGridOrigin; // Coordinate of bottom left corner of export tile grid, in output coordinates.
uint InLandscapeComponentResolution; // One texel per vert, so number of quads + 1
float InLandscapeScaleZ; // LS Actor DrawScale.z / 128.0f (ZSCALE)
float InLandscapeRootZ;
uint InOutputHeight;
uint2 InDispatchOffset; // Bottom left coordinate of this dispatch, in output coordinates.
uint2 InDispatchSize; // Number of XY threads in this dispatch.

// Dispatch group count is (ceil(InDispatchSize / 8), Max(1, NumGrassMaps)).
[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, THREADGROUPSIZE_Z)]
void PCGGrassMapUnpacker_CS(const uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId.xy >= InDispatchSize))
	{
		return;
	}

	const uint2 OutputCoord = DispatchThreadId.xy + InDispatchOffset;
	const uint OutputSlice = DispatchThreadId.z;

	const uint2 TileGridCoord = uint2((int2)OutputCoord - InTileGridOrigin);

	// The rendered landscape components in the input are not in any particular order, so lookup which component index for this output tile.
	// Clamp because we are dividing by resolution-1 and last col/row of texture will overflow.
	const uint2 TileCoords = min(TileGridCoord / (InLandscapeComponentResolution - 1u), InNumTiles - 1u);

	const uint TileLinearIndex = TileCoords.y * InNumTiles.x + TileCoords.x;
	const int ComponentIndex = InLinearTileIndexToComponentIndex[TileLinearIndex][0];

	// Invalid component index means there was no component present for this output tile.
	if (ComponentIndex == -1)
	{
		// First layer of threads also deals with height.
		if (InOutputHeight == 1 && OutputSlice == 0)
		{
			OutUnpackedHeight[OutputCoord] = 0.0f;
		}

		OutUnpackedGrassMaps[uint3(OutputCoord, OutputSlice)] = 0.0f;
		return;
	}

	const uint2 LocalTileCoords = uint2(TileGridCoord.x - TileCoords.x * (InLandscapeComponentResolution - 1), TileGridCoord.y - TileCoords.y * (InLandscapeComponentResolution - 1));

	// Heights. The first layer threads also deal with height.
	if (InOutputHeight == 1 && OutputSlice == 0)
	{
		const int ComponentXOffset = InComponentHeightOffsets[ComponentIndex];

		const uint2 InputCoords = uint2(LocalTileCoords.x + ComponentXOffset, LocalTileCoords.y);

		const float RelativeHeight = (UnpackHeight(InPackedGrassMaps[InputCoords].xy) - LANDSCAPE_MID_VALUE) * InLandscapeScaleZ * TERRAIN_ZSCALE;
		OutUnpackedHeight[OutputCoord] = RelativeHeight + InLandscapeRootZ;
	}

	// Coordinates into input height & grass maps (see packing above).
	const int ExportSlice = InOutputIndexToExportIndex[OutputSlice];
	const int2 ComponentAndGrassXOffsetAndChannel = (ExportSlice > -1) ? InComponentGrassmapOffsetAndChannels[ComponentIndex * InNumExportedGrassMaps + ExportSlice] : -1;
	if (ComponentAndGrassXOffsetAndChannel.x != -1)
	{
		const uint ComponentXOffset = ComponentAndGrassXOffsetAndChannel.x;
		const uint ExportChannelIndex = ComponentAndGrassXOffsetAndChannel.y;

		// Coordinates into packed data which are laid out horizontally (see above).
		const uint2 InputCoords = uint2(LocalTileCoords.x + ComponentXOffset, LocalTileCoords.y);

		// Grass maps
		OutUnpackedGrassMaps[uint3(OutputCoord, OutputSlice)] = InPackedGrassMaps[InputCoords][ExportChannelIndex];

		// TODO: position of height data can move around based on slices, so also needs to be in mapping somehow
		// Heights. The first layer threads also deal with height.
		if (InOutputHeight == 1 && OutputSlice == 0)
		{
			const float RelativeHeight = (UnpackHeight(InPackedGrassMaps[InputCoords].xy) - LANDSCAPE_MID_VALUE) * InLandscapeScaleZ * TERRAIN_ZSCALE;
			OutUnpackedHeight[OutputCoord] = RelativeHeight + InLandscapeRootZ;
		}
	}
	else
	{
		// Grass map was not exported, write 0. Happens if for example user specifies a grass type name that isn't present on any of the landscape components.
		OutUnpackedGrassMaps[uint3(OutputCoord, OutputSlice)] = 0.0f;
	}
}
