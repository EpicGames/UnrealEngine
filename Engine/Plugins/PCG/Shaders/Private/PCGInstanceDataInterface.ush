// Copyright Epic Games, Inc. All Rights Reserved.

uint {DataInterfaceName}_NumPrimitives;
uint {DataInterfaceName}_NumInstancesAllPrimitives;
uint {DataInterfaceName}_NumCustomFloatsPerInstance;
int {DataInterfaceName}_NumCullingCellsTotal;

StructuredBuffer<uint2> {DataInterfaceName}_PrimitiveParameters; // Per primitive, per culling cell: (NumInstancesAllocated, InstanceOffset)

RWStructuredBuffer<uint4> {DataInterfaceName}_InstanceData;
RWStructuredBuffer<uint> {DataInterfaceName}_InstanceCustomFloatData;
RWStructuredBuffer<uint> {DataInterfaceName}_WriteCounters;

uint InstanceData_GetNumPrimitives_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumPrimitives;
}

uint InstanceData_GetNumInstancesAllPrimitives_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumInstancesAllPrimitives;
}

int InstanceData_GetIndexToWriteTo_{DataInterfaceName}(uint InPrimitiveIndex, uint InCullingCellId)
{
	const uint Index = {DataInterfaceName}_NumCullingCellsTotal * InPrimitiveIndex + InCullingCellId;

	uint ValueBefore;
	InterlockedAdd({DataInterfaceName}_WriteCounters[Index], 1u, ValueBefore);

	uint2 PrimitiveParams = {DataInterfaceName}_PrimitiveParameters[Index];

	const uint NumInstancesAllocated = PrimitiveParams[0];
	if (ValueBefore >= NumInstancesAllocated)
	{
		// All instances for this primitive have been written to. Should not happen if instance counts were computed correctly.
		return -1;
	}

	const uint InstanceOffset = PrimitiveParams[1];
	return (int)(ValueBefore + InstanceOffset);
}

void InstanceData_WriteInstanceLocalToWorld_{DataInterfaceName}(uint InInstanceIndex, float4x4 InLocalToRelativeWorld)
{
	InLocalToRelativeWorld = transpose(InLocalToRelativeWorld);
	const uint StrideUint4s = 3;
	{DataInterfaceName}_InstanceData[InInstanceIndex * StrideUint4s + 0] = asuint(InLocalToRelativeWorld[0]);
	{DataInterfaceName}_InstanceData[InInstanceIndex * StrideUint4s + 1] = asuint(InLocalToRelativeWorld[1]);
	{DataInterfaceName}_InstanceData[InInstanceIndex * StrideUint4s + 2] = asuint(InLocalToRelativeWorld[2]);
}

void InstanceData_WriteCustomFloat_{DataInterfaceName}(uint InInstanceIndex, uint InCustomFloatIndex, float InValue)
{
	{DataInterfaceName}_InstanceCustomFloatData[{DataInterfaceName}_NumCustomFloatsPerInstance * InInstanceIndex + InCustomFloatIndex] = asuint(InValue);
}

void InstanceData_WriteCustomFloatRaw_{DataInterfaceName}(uint InInstanceIndex, uint InCustomFloatIndex, uint InValueAsUint)
{
	{DataInterfaceName}_InstanceCustomFloatData[{DataInterfaceName}_NumCustomFloatsPerInstance * InInstanceIndex + InCustomFloatIndex] = InValueAsUint;
}
