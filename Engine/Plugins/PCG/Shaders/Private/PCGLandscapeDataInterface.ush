// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/PCG/Private/PCGVirtualTextureCommon.ush"

// Height RVT
Texture2D			{DataInterfaceName}_HeightVirtualTexture;
Texture2D<uint4>	{DataInterfaceName}_HeightVirtualTexturePageTable;
Texture2D<uint>		{DataInterfaceName}_HeightVirtualTexturePageTableIndirection;
uint				{DataInterfaceName}_HeightVirtualTextureAdaptive;
SamplerState		{DataInterfaceName}_HeightVirtualTextureSampler;
float3				{DataInterfaceName}_HeightVirtualTextureLWCTile;
float4x4			{DataInterfaceName}_HeightVirtualTextureWorldToUVTransform;
uint				{DataInterfaceName}_HeightVirtualTextureEnabled;
uint4				{DataInterfaceName}_HeightVirtualTexturePackedUniform0;
uint4				{DataInterfaceName}_HeightVirtualTexturePackedUniform1;
uint4				{DataInterfaceName}_HeightVirtualTextureUniforms;

// Normal RVT
Texture2D			{DataInterfaceName}_NormalVirtualTexture0;
Texture2D			{DataInterfaceName}_NormalVirtualTexture1;
Texture2D<uint4>	{DataInterfaceName}_NormalVirtualTexturePageTable;
Texture2D<uint>		{DataInterfaceName}_NormalVirtualTexturePageTableIndirection;
uint				{DataInterfaceName}_NormalVirtualTextureAdaptive;
uint2				{DataInterfaceName}_NormalVirtualTexturePageTableLayers;
SamplerState		{DataInterfaceName}_NormalVirtualTexture0Sampler;
SamplerState		{DataInterfaceName}_NormalVirtualTexture1Sampler;
float3				{DataInterfaceName}_NormalVirtualTextureLWCTile;
float4x4			{DataInterfaceName}_NormalVirtualTextureWorldToUVTransform;
uint4				{DataInterfaceName}_NormalVirtualTexturePackedUniform0;
uint4				{DataInterfaceName}_NormalVirtualTexturePackedUniform1;
uint4				{DataInterfaceName}_NormalVirtualTextureUniforms0;
uint4				{DataInterfaceName}_NormalVirtualTextureUniforms1;
int					{DataInterfaceName}_NormalVirtualTextureUnpackMode;
uint				{DataInterfaceName}_NormalVirtualTextureEnabled;

// Base Color RVT
Texture2D			{DataInterfaceName}_BaseColorVirtualTexture;
Texture2D<uint4>	{DataInterfaceName}_BaseColorVirtualTexturePageTable;
Texture2D<uint>		{DataInterfaceName}_BaseColorVirtualTexturePageTableIndirection;
uint				{DataInterfaceName}_BaseColorVirtualTextureAdaptive;
SamplerState		{DataInterfaceName}_BaseColorVirtualTextureSampler;
float3				{DataInterfaceName}_BaseColorVirtualTextureLWCTile;
float4x4			{DataInterfaceName}_BaseColorVirtualTextureWorldToUVTransform;
uint				{DataInterfaceName}_BaseColorVirtualTextureUnpackSRGB;
uint				{DataInterfaceName}_BaseColorVirtualTextureUnpackYCoCg;
uint				{DataInterfaceName}_BaseColorVirtualTextureEnabled;
uint4				{DataInterfaceName}_BaseColorVirtualTexturePackedUniform0;
uint4				{DataInterfaceName}_BaseColorVirtualTexturePackedUniform1;
uint4				{DataInterfaceName}_BaseColorVirtualTextureUniforms;

// Collision based height texture
Texture2D			{DataInterfaceName}_CollisionHeightTexture;
SamplerState		{DataInterfaceName}_CollisionHeightTextureSampler;
uint				{DataInterfaceName}_CollisionHeightTextureEnabled;
float3				{DataInterfaceName}_CollisionHeightTextureLWCTile;
float4x4			{DataInterfaceName}_CollisionHeightTextureWorldToUVTransform;
float4x4			{DataInterfaceName}_CollisionHeightTextureUVToWorldTransform;
float4				{DataInterfaceName}_CollisionHeightTextureUVScaleBias;
int2				{DataInterfaceName}_CollisionHeightTextureDimension;

// Misc parameters
float2				{DataInterfaceName}_LandscapeGridSize;

float2 GetCollisionHeightTextureUV_{DataInterfaceName}(float3 InWorldPos)
{
	// @todo_pcg: Support LWC landscape sampling
	const float3 LWCTile = (float3)0;
	const FLWCVector4 LwcWorldPos = MakeLWCVector4(LWCTile, float4(InWorldPos, 1.0f));
	const FLWCInverseMatrix LwcWorldToUV = MakeLWCInverseMatrix({DataInterfaceName}_CollisionHeightTextureLWCTile, {DataInterfaceName}_CollisionHeightTextureWorldToUVTransform);

	float2 UV = LWCMultiply(LwcWorldPos, LwcWorldToUV).xy;
	UV = UV * {DataInterfaceName}_CollisionHeightTextureUVScaleBias.xy + {DataInterfaceName}_CollisionHeightTextureUVScaleBias.zw;

	return UV;
}

float GetHeight_{DataInterfaceName}(float3 InWorldPos)
{
	if ({DataInterfaceName}_HeightVirtualTextureEnabled != 0)
	{
		const float3 UnpackHeightOrigin = {DataInterfaceName}_HeightVirtualTextureWorldToUVTransform[0].xyz;
		const float3 UnpackHeightU = {DataInterfaceName}_HeightVirtualTextureWorldToUVTransform[1].xyz;
		const float3 UnpackHeightV = {DataInterfaceName}_HeightVirtualTextureWorldToUVTransform[2].xyz;
		const float2 UnpackHeightScaleBias = {DataInterfaceName}_HeightVirtualTextureWorldToUVTransform[3].xy;
		const float3 LWCTile = (float3)0;

		const FDFVector3 LwcWorldPos = DFFromTileOffset_Hack(MakeLWCVector3(LWCTile, InWorldPos));
		const FDFVector3 LwcHeightOrigin = DFFromTileOffset_Hack(MakeLWCVector3({DataInterfaceName}_HeightVirtualTextureLWCTile, UnpackHeightOrigin));
		const float2 SampleUV = VirtualTextureWorldToUV(LwcWorldPos, LwcHeightOrigin, UnpackHeightU, UnpackHeightV);

		VTPageTableResult PageTable;

		if ({DataInterfaceName}_HeightVirtualTextureAdaptive == 0)
		{
			PageTable = TextureLoadVirtualPageTableLevel(
				{DataInterfaceName}_HeightVirtualTexturePageTable,
				VTPageTableUniform_Unpack({DataInterfaceName}_HeightVirtualTexturePackedUniform0, {DataInterfaceName}_HeightVirtualTexturePackedUniform1),
				SampleUV,
				VTADDRESSMODE_CLAMP,
				VTADDRESSMODE_CLAMP,
				0.0f);
		}
		else
		{
			PageTable = TextureLoadVirtualPageTableAdaptiveLevel(
				{DataInterfaceName}_HeightVirtualTexturePageTable,
				{DataInterfaceName}_HeightVirtualTexturePageTableIndirection,
				VTPageTableUniform_Unpack({DataInterfaceName}_HeightVirtualTexturePackedUniform0, {DataInterfaceName}_HeightVirtualTexturePackedUniform1),
				SampleUV,
				VTADDRESSMODE_CLAMP,
				VTADDRESSMODE_CLAMP,
				0.0f);
		}

		const float4 PackedValue = TextureVirtualSample(
			{DataInterfaceName}_HeightVirtualTexture,
			{DataInterfaceName}_HeightVirtualTextureSampler,
			PageTable,
			0, // Height is always in layer 0.
			VTUniform_Unpack({DataInterfaceName}_HeightVirtualTextureUniforms));

		return VirtualTextureUnpackHeight(PackedValue, UnpackHeightScaleBias);
	}
	else if ({DataInterfaceName}_CollisionHeightTextureEnabled != 0)
	{
		const float2 UV = GetCollisionHeightTextureUV_{DataInterfaceName}(InWorldPos);
		return {DataInterfaceName}_CollisionHeightTexture.SampleLevel({DataInterfaceName}_CollisionHeightTextureSampler, UV, 0).x;
	}
	else
	{
		return InWorldPos.z;
	}
}

float3 CalculateNormalFromHeights_{DataInterfaceName}(float3 InWorldPos)
{
	float3 Pos1 = InWorldPos;
	float3 Pos2 = InWorldPos + float3(0.0f, {DataInterfaceName}_LandscapeGridSize.y, 0.0f);
	float3 Pos3 = InWorldPos + float3({DataInterfaceName}_LandscapeGridSize.x, 0.0f, 0.0f);

	Pos1.z = GetHeight_{DataInterfaceName}(Pos1);
	Pos2.z = GetHeight_{DataInterfaceName}(Pos2);
	Pos3.z = GetHeight_{DataInterfaceName}(Pos3);

	const float3 DeltaX = Pos2 - Pos1;
	const float3 DeltaY = Pos3 - Pos1;

	return normalize(cross(DeltaY, DeltaX));
}

float3 GetNormal_{DataInterfaceName}(float3 InWorldPos)
{
	if ({DataInterfaceName}_NormalVirtualTextureEnabled != 0)
	{
		const float3 UnpackNormalOrigin = {DataInterfaceName}_NormalVirtualTextureWorldToUVTransform[0].xyz;
		const float3 UnpackNormalU = {DataInterfaceName}_NormalVirtualTextureWorldToUVTransform[1].xyz;
		const float3 UnpackNormalV = {DataInterfaceName}_NormalVirtualTextureWorldToUVTransform[2].xyz;
		const float3 LWCTile = (float3)0;

		const FDFVector3 LwcWorldPos = DFFromTileOffset_Hack(MakeLWCVector3(LWCTile, InWorldPos));
		const FDFVector3 LwcNormalOrigin = DFFromTileOffset_Hack(MakeLWCVector3({DataInterfaceName}_NormalVirtualTextureLWCTile, UnpackNormalOrigin));
		const float2 SampleUV = VirtualTextureWorldToUV(LwcWorldPos, LwcNormalOrigin, UnpackNormalU, UnpackNormalV);

		VTPageTableResult PageTable;

		if ({DataInterfaceName}_NormalVirtualTextureAdaptive == 0)
		{
			PageTable = TextureLoadVirtualPageTableLevel(
				{DataInterfaceName}_NormalVirtualTexturePageTable,
				VTPageTableUniform_Unpack({DataInterfaceName}_NormalVirtualTexturePackedUniform0, {DataInterfaceName}_NormalVirtualTexturePackedUniform1),
				SampleUV,
				VTADDRESSMODE_CLAMP,
				VTADDRESSMODE_CLAMP,
				0.0f);
		}
		else
		{
			PageTable = TextureLoadVirtualPageTableAdaptiveLevel(
				{DataInterfaceName}_NormalVirtualTexturePageTable,
				{DataInterfaceName}_NormalVirtualTexturePageTableIndirection,
				VTPageTableUniform_Unpack({DataInterfaceName}_NormalVirtualTexturePackedUniform0, {DataInterfaceName}_NormalVirtualTexturePackedUniform1),
				SampleUV,
				VTADDRESSMODE_CLAMP,
				VTADDRESSMODE_CLAMP,
				0.0f);
		}

		const float4 PackedValue0 = TextureVirtualSample(
			{DataInterfaceName}_NormalVirtualTexture0,
			{DataInterfaceName}_NormalVirtualTexture0Sampler,
			PageTable,
			{DataInterfaceName}_NormalVirtualTexturePageTableLayers.x,
			VTUniform_Unpack({DataInterfaceName}_NormalVirtualTextureUniforms0));

		const float4 PackedValue1 = TextureVirtualSample(
			{DataInterfaceName}_NormalVirtualTexture1,
			{DataInterfaceName}_NormalVirtualTexture1Sampler,
			PageTable,
			{DataInterfaceName}_NormalVirtualTexturePageTableLayers.y,
			VTUniform_Unpack({DataInterfaceName}_NormalVirtualTextureUniforms1));

		if ({DataInterfaceName}_NormalVirtualTextureUnpackMode == VIRTUAL_TEXTURE_UNPACK_NORMAL_BC3BC3)
		{
			return VirtualTextureUnpackNormalBC3BC3(PackedValue0, PackedValue1);
		}
		else if ({DataInterfaceName}_NormalVirtualTextureUnpackMode == VIRTUAL_TEXTURE_UNPACK_NORMAL_BC5BC1)
		{
			return VirtualTextureUnpackNormalBC5BC1(PackedValue0, PackedValue1);
		}
		else if ({DataInterfaceName}_NormalVirtualTextureUnpackMode == VIRTUAL_TEXTURE_UNPACK_NORMAL_B5G6R5)
		{
			return VirtualTextureUnpackNormalBGR565(PackedValue1);
		}
		else
		{
			return CalculateNormalFromHeights_{DataInterfaceName}(InWorldPos);
		}
	}
	else
	{
		return CalculateNormalFromHeights_{DataInterfaceName}(InWorldPos);
	}
}

float3 GetBaseColor_{DataInterfaceName}(float3 InWorldPos)
{
	if ({DataInterfaceName}_BaseColorVirtualTextureEnabled != 0)
	{
		const float3 UnpackBaseColorOrigin = {DataInterfaceName}_BaseColorVirtualTextureWorldToUVTransform[0].xyz;
		const float3 UnpackBaseColorU = {DataInterfaceName}_BaseColorVirtualTextureWorldToUVTransform[1].xyz;
		const float3 UnpackBaseColorV = {DataInterfaceName}_BaseColorVirtualTextureWorldToUVTransform[2].xyz;
		const float3 LWCTile = (float3)0;

		const FDFVector3 LwcWorldPos = DFFromTileOffset_Hack(MakeLWCVector3(LWCTile, InWorldPos));
		const FDFVector3 LwcBaseColorOrigin = DFFromTileOffset_Hack(MakeLWCVector3({DataInterfaceName}_BaseColorVirtualTextureLWCTile, UnpackBaseColorOrigin));
		const float2 SampleUV = VirtualTextureWorldToUV(LwcWorldPos, LwcBaseColorOrigin, UnpackBaseColorU, UnpackBaseColorV);

		VTPageTableResult PageTable;

		if ({DataInterfaceName}_NormalVirtualTextureAdaptive == 0)
		{
			PageTable = TextureLoadVirtualPageTableLevel(
				{DataInterfaceName}_BaseColorVirtualTexturePageTable,
				VTPageTableUniform_Unpack({DataInterfaceName}_BaseColorVirtualTexturePackedUniform0, {DataInterfaceName}_BaseColorVirtualTexturePackedUniform1),
				SampleUV,
				VTADDRESSMODE_CLAMP,
				VTADDRESSMODE_CLAMP,
				0.0f);
		}
		else
		{
			PageTable = TextureLoadVirtualPageTableAdaptiveLevel(
				{DataInterfaceName}_BaseColorVirtualTexturePageTable,
				{DataInterfaceName}_BaseColorVirtualTexturePageTableIndirection,
				VTPageTableUniform_Unpack({DataInterfaceName}_BaseColorVirtualTexturePackedUniform0, {DataInterfaceName}_BaseColorVirtualTexturePackedUniform1),
				SampleUV,
				VTADDRESSMODE_CLAMP,
				VTADDRESSMODE_CLAMP,
				0.0f);
		}

		const float4 PackedValue = TextureVirtualSample(
			{DataInterfaceName}_BaseColorVirtualTexture,
			{DataInterfaceName}_BaseColorVirtualTextureSampler,
			PageTable,
			0, // BaseColor is always in layer 0.
			VTUniform_Unpack({DataInterfaceName}_BaseColorVirtualTextureUniforms));

		if ({DataInterfaceName}_BaseColorVirtualTextureUnpackSRGB)
		{
			return VirtualTextureUnpackBaseColorSRGB(PackedValue);
		}
		else if ({DataInterfaceName}_BaseColorVirtualTextureUnpackYCoCg)
		{
			return VirtualTextureUnpackBaseColorYCoCg(PackedValue);
		}
		else
		{
			return PackedValue.rgb;
		}
	}
	else
	{
		return (float3)0.0f;
	}
}

#undef VIRTUAL_TEXTURE_UNPACK_NORMAL_NONE
#undef VIRTUAL_TEXTURE_UNPACK_NORMAL_BC3BC3
#undef VIRTUAL_TEXTURE_UNPACK_NORMAL_BC5BC1
#undef VIRTUAL_TEXTURE_UNPACK_NORMAL_B5G6R5
