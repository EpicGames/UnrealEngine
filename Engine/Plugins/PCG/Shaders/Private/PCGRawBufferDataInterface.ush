// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Raw buffer: Flat list of raw bytes, read/written in uints.
 * 
 * No multi-data support currently. Would need software-bindless, in order to keep each buffer pure rather than adding headers.
 */

ByteAddressBuffer {DataInterfaceName}_DataSRV;
RWByteAddressBuffer {DataInterfaceName}_DataUAV;

// Number of bytes in buffer.
uint {DataInterfaceName}_SizeBytes;

// #################### META INFO ##########################

uint GetNumElements_{DataInterfaceName}()
{
	return {DataInterfaceName}_SizeBytes >> 2;
}

// #################### READ ACCESSORS ##########################

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for both SRV and UAV use.
{{BeginDuplicateForSRVUAV}}
uint Load_{NameSuffix}_{DataInterfaceName}(uint InElementIndex)
{
	return {DataInterfaceName}_Data{BufferSuffix}.Load(InElementIndex << 2);
}

uint4 Load4_{NameSuffix}_{DataInterfaceName}(uint InFirstElementIndex)
{
	return {DataInterfaceName}_Data{BufferSuffix}.Load4(InFirstElementIndex << 2);
}
{{EndDuplicateForSRVUAV}}

// Begin section which will have {NameSuffix} and {BufferSuffix} injected for UAV use.
{{BeginUAVOnly}}

// #################### WRITE ACCESSORS #########################

void Store_{NameSuffix}_{DataInterfaceName}(uint InElementIndex, uint InValue)
{
	{DataInterfaceName}_Data{BufferSuffix}.Store(InElementIndex << 2, InValue);
}

void Store4_{NameSuffix}_{DataInterfaceName}(uint InFirstElementIndex, uint4 InValue)
{
	{DataInterfaceName}_Data{BufferSuffix}.Store4(InFirstElementIndex << 2, InValue);
}

// #################### ATOMICS #################################

// Returns value before incrementing.
uint AtomicAdd_{NameSuffix}_{DataInterfaceName}(uint InElementIndex, uint InValueToAdd)
{
	uint OriginalValue;
	{DataInterfaceName}_Data{BufferSuffix}.InterlockedAdd(InElementIndex << 2, (uint)InValueToAdd, OriginalValue);

	return OriginalValue;
}

// Returns value of attribute before max operation.
uint AtomicMax_{NameSuffix}_{DataInterfaceName}(uint InElementIndex, uint InValueToMax)
{
	uint OriginalValue;
	{DataInterfaceName}_Data{BufferSuffix}.InterlockedMax(InElementIndex << 2, (uint)InValueToMax, OriginalValue);

	return OriginalValue;
}

{{EndUAVOnly}}
