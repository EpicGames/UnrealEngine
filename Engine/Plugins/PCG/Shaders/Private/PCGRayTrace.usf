// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/RayTracing/TraceRayInline.ush"
#include "/Engine/Public/BindlessResources.ush"
#include "/Engine/Private/Visualization.ush"

#define PCG_DEFAULT_NORMAL float3(0.0f, 0.0f, 1.0f)

// @todo_pcg: Consider splitting into separate resources per output type (position, normals, UVs) so that unused outputs don't consume memory/bandwidth.
/**
 * Input packed data for each ray:
 * - Ray Position    - 3 uints
 * - Ray Culled      - 1 uint
 * - Ray Direction   - 3 uints
 * - Ray Max Length  - 1 uint
 * - Empty packing   - 4 uints
 *
 * Output packed data for each ray:
 *  - Impact Position   - 3 uints
 *  - Ray Culled/Missed - 1 uint
 *  - Impact TexCoords  - 2 uints
 *  - Empty packing     - 2 uints
 *  - Impact Normals    - 3 uints
 *  - Empty packing     - 1 uint
 */
RWByteAddressBuffer InOutPackedData;
RaytracingAccelerationStructure InTLAS;
#if PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
FResourceCollection InPrimitiveUVCollection;
Buffer<uint> InPrimitiveUVOffsets;
StructuredBuffer<FRayTracingSceneMetadataRecord> InRayTracingSceneMetadata;
#endif // PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
float3 InPreViewTranslation;
int InNumRays;
int InTexCoordsChannelIndex;

/** Unpack the data for this ray. */
void ReadInput(int InRayIndex, out float3 OutOrigin, out float3 OutDirection, out float OutLength, out int OutStatus)
{
	const uint PackedPositionAddress = InRayIndex * (PCG_RAY_TRACE_PACKED_BUFFER_STRIDE_UINTS << 2);
	const uint PackedDirectionAddress = PackedPositionAddress + (4 << 2);
	const uint4 PositionAndValid = InOutPackedData.Load4(PackedPositionAddress);
	const uint4 DirectionAndLength = InOutPackedData.Load4(PackedDirectionAddress);

	// Account for local view when computing the ray origin.
	OutOrigin = asfloat(PositionAndValid.xyz) + InPreViewTranslation;
	OutDirection = asfloat(DirectionAndLength.xyz);
	OutLength = asfloat(DirectionAndLength.w);
	OutStatus = PositionAndValid.w;
}

/** Pack the impact data for this ray. */
void WriteOutput(int InRayIndex, float3 InHitPosition, float2 InTexCoords, float3 InNormals, int InStatus)
{
	const uint PackedPositionAddress = InRayIndex * (PCG_RAY_TRACE_PACKED_BUFFER_STRIDE_UINTS << 2);
	const uint PackedTexCoordsAddress = PackedPositionAddress + (4 << 2);
	const uint PackedNormalsAddress = PackedPositionAddress + (8 << 2);
	const uint4 PositionAndValid = uint4(asuint(InHitPosition - InPreViewTranslation), (uint)InStatus);
	const uint4 TexCoords = uint4(asuint(InTexCoords).xy, 0u, 0u);
	const uint4 Normals = uint4(asuint(InNormals).xyz, 0u);

	InOutPackedData.Store4(PackedPositionAddress, PositionAndValid);
	InOutPackedData.Store4(PackedTexCoordsAddress, TexCoords);
	InOutPackedData.Store4(PackedNormalsAddress, Normals);
}

void PCGRayTraceCommon(int InRayIndex, float3 InOrigin, float3 InDirection, float InLength)
{
	FRayDesc Ray;
	Ray.Origin = InOrigin;
	Ray.Direction = InDirection;
	Ray.TMin = 0.001f;
	Ray.TMax = InLength;

	FTraceRayInlineContext TraceRayInlineContext = CreateTraceRayInlineContext();
	TraceRayInlineContext.MaxIterations = 32;
	
	const FTraceRayInlineResult TraceResult = TraceRayInline(InTLAS, /*RayFlags=*/0x0, /*InstanceInclusionMask=*/~0x0, Ray.GetNativeDesc(), TraceRayInlineContext);
	
	if (TraceResult.IsHit())
	{
		float2 TexCoords = 0.0f;
		float3 Normal;

#if PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
		const FTriangleBaseAttributes Attributes = LoadInlineRayTracingTriangleAttributes(
			InRayTracingSceneMetadata,
			TraceResult.InstanceContributionToHitGroupIndex,
			RAY_TRACING_NUM_SHADER_SLOTS,
			TraceResult.GeometryIndex,
			TraceResult.PrimitiveIndex);

		const FInstanceSceneData InstanceData = GetInstanceSceneData(TraceResult.InstanceID);
		const FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceData.PrimitiveId);

		// First uint packed into InPrimitiveUVOffsets will be the offset into the TexCoords buffer.
		const uint TexCoordsBufferOffset = InPrimitiveUVOffsets[PrimitiveData.PersistentPrimitiveIndex * 2 + 0];

		// If the primitive has a valid tex coord offset, grab the texture coordinates.
		if (TexCoordsBufferOffset != ~0u)
		{
			// Second uint packed into InPrimitiveUVOffsets is the number of UV channels on the primitive.
			const uint NumUVChannels = InPrimitiveUVOffsets[PrimitiveData.PersistentPrimitiveIndex * 2 + 1];

			if (InTexCoordsChannelIndex >= 0 && InTexCoordsChannelIndex < NumUVChannels)
			{
				const Buffer_float2 TexCoordsBuffer = BufferFromCollection_Buffer_float2(InPrimitiveUVCollection, TexCoordsBufferOffset);

				// Get the UVs for the hit triangle's vertices. Account for the selected UV channel.
				const float2 VertexUVs[] =
				{
					TexCoordsBuffer.Load(NumUVChannels * Attributes.Indices[0] + InTexCoordsChannelIndex),
					TexCoordsBuffer.Load(NumUVChannels * Attributes.Indices[1] + InTexCoordsChannelIndex),
					TexCoordsBuffer.Load(NumUVChannels * Attributes.Indices[2] + InTexCoordsChannelIndex)
				};

				const float3 Barycentrics = float3(1.0f - TraceResult.Barycentrics.x - TraceResult.Barycentrics.y, TraceResult.Barycentrics.x, TraceResult.Barycentrics.y);
				TexCoords = VertexUVs[0] * Barycentrics.x + VertexUVs[1] * Barycentrics.y + VertexUVs[2] * Barycentrics.z;
			}
		}
#endif // PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES

#if PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_NORMALS
		Normal = TraceResult.WorldGeometryNormal;
#elif PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
		// Transform local-space geometry normal to world space using inverse transpose (matching VFXTraceRay.ush).
		Normal = cross(Attributes.LocalPositions[2] - Attributes.LocalPositions[0], Attributes.LocalPositions[1] - Attributes.LocalPositions[0]);
		Normal = normalize(mul(Normal, (float3x3)TraceResult.WorldToObject3x4));
#else
		Normal = PCG_DEFAULT_NORMAL;
#endif

		WriteOutput(InRayIndex, Ray.Origin + Ray.Direction * TraceResult.HitT, TexCoords, Normal, /*Status=*/0u);
	}
	else
	{
		// If the ray missed, mark it as culled.
		WriteOutput(InRayIndex, Ray.Origin + Ray.Direction * Ray.TMax, /*TexCoords=*/0.0f, PCG_DEFAULT_NORMAL, PCG_RAY_CULLED);
	}
}

[numthreads(32, 1, 1)]
void PCGRayTraceMainCS(uint DispatchThreadId : SV_DispatchThreadID)
{
	const int RayIndex = DispatchThreadId;

	if (RayIndex >= InNumRays)
	{
		return;
	}

	float3 Origin, Direction;
	float Length;
	int Status;

	ReadInput(RayIndex, Origin, Direction, Length, Status);

	// If the ray was already culled, do nothing.
	if (Status == PCG_RAY_CULLED)
	{
		WriteOutput(RayIndex, Origin, /*TexCoords=*/0.0f, PCG_DEFAULT_NORMAL, Status);
		return;
	}

	PCGRayTraceCommon(RayIndex, Origin, Direction, Length);
}

#undef PCG_DEFAULT_NORMAL
