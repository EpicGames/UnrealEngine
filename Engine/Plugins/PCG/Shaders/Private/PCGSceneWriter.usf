// Copyright Epic Games, Inc. All Rights Reserved.

// Scene writer - used to inject final instance data into the GPU Scene. Dispatched in the scene update timeline.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ComputeShaderUtils.ush"
#include "/Engine/Private/GPUScene/GPUSceneWriter.ush"
#include "/Engine/Public/Platform.ush"

uint InPrimitiveIndex;
uint InNumInstancesAllocatedInGPUScene;
uint InInstanceOffset;
StructuredBuffer<uint4> InInstanceData;
StructuredBuffer<uint> InInstanceCustomFloatData;
StructuredBuffer<uint> InWriteCounters;
uint InPrimitiveId;
uint InCustomDataCount;
uint InPayloadDataFlags;
uint InSeed;
uint InNumCullingCellsTotal;

// This follows the implementation in PCGHelpers.
uint ComputeSeed(uint A, uint B)
{
	return ((A * 196314165U) + 907633515U) ^ ((B * 73148459U) + 453816763U);
}

// This follows the implementation in FRandomStream.
float FRand(inout uint InOutSeed)
{
	InOutSeed = (InOutSeed * 196314165U) + 907633515U;
	
	const float Result = asfloat(0x3F800000U | (InOutSeed >> 9));

	return Result - 1.0f;
}

void CullInstance(uint InstanceId)
{
	// This will minimally initialize the instance data such that it will be ignored and culled
	WriteInstancePrimitiveIdAndFlags(InstanceId, InPrimitiveId, INSTANCE_SCENE_DATA_FLAG_HIDDEN);
}

[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void SceneWriter_CS(const uint3 GroupId : SV_GroupID, const uint GroupIndex : SV_GroupIndex)
{
	const uint LocalInstanceId = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, NUM_THREADS_PER_GROUP);

	if (LocalInstanceId >= InNumInstancesAllocatedInGPUScene)
	{
		return;
	}

	// Retrieve the primitive data
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InPrimitiveId);
	
	const uint InstanceId = PrimitiveData.InstanceSceneDataOffset + LocalInstanceId;
	
	// Loop over all culling cells for this primitive and add up total instance count for this primitive.
	uint InstanceCount = 0;
	for (uint CullingCellIndex = 0; CullingCellIndex < InNumCullingCellsTotal; ++CullingCellIndex)
	{
		InstanceCount += InWriteCounters[InNumCullingCellsTotal * InPrimitiveIndex + CullingCellIndex];
	}

	// Cull any instances at tail end (assume all valid instances are written).
	if (LocalInstanceId >= InstanceCount)
	{
		CullInstance(InstanceId);
		return;
	}
	
	const uint StrideFloat4s = 3;
	float4x4 LocalToWorld;
	LocalToWorld[0] = asfloat(InInstanceData[(InInstanceOffset + LocalInstanceId) * StrideFloat4s + 0]);
	LocalToWorld[1] = asfloat(InInstanceData[(InInstanceOffset + LocalInstanceId) * StrideFloat4s + 1]);
	LocalToWorld[2] = asfloat(InInstanceData[(InInstanceOffset + LocalInstanceId) * StrideFloat4s + 2]);
	LocalToWorld[3] = float4(0.0f, 0.0f, 0.0f, 1.0f);
	LocalToWorld = transpose(LocalToWorld);

	uint Seed = ComputeSeed(InSeed, InstanceId);

	// todo_pcg: We should write local space transforms to the instance data buffer from the SM Spawner and use InitializeInstanceSceneData (non-WS) here instead.
	InitializeInstanceSceneDataWS(
		InstanceId,
		InPrimitiveId,
		LocalInstanceId,
 		InPayloadDataFlags,
 		InCustomDataCount,
 		FRand(Seed),
		DFFromTileOffset(LWCPromote(LocalToWorld))
	);
	
	FInstancePayloadDataOffsets PayloadOffsets = GetInstancePayloadDataOffsets(InPrimitiveId, InPayloadDataFlags, LocalInstanceId);

	const uint NumCustomFloat4s = (InCustomDataCount + 3) / 4;
	for (uint Float4Index = 0; Float4Index < NumCustomFloat4s; ++Float4Index)
	{
		const uint BaseIndex = (InInstanceOffset + LocalInstanceId) * InCustomDataCount + Float4Index * 4;

		uint4 CustomData;
		CustomData[0] = InInstanceCustomFloatData[BaseIndex];
		CustomData[1] = InInstanceCustomFloatData[BaseIndex + 1];
		CustomData[2] = InInstanceCustomFloatData[BaseIndex + 2];
		CustomData[3] = InInstanceCustomFloatData[BaseIndex + 3];
		WriteInstanceCustomData(PayloadOffsets, Float4Index, asfloat(CustomData));
	}
}
