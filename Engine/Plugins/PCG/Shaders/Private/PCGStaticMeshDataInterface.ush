// Copyright Epic Games, Inc. All Rights Reserved.

// Reference NiagaraDataInterfaceStaticMeshTemplate.ush

/** @todo_pcg:
 * Sections/Sockets
 * Distance fields
 * Spatial sampling
 */

int {DataInterfaceName}_NumVertices;
int {DataInterfaceName}_NumTriangles;
int {DataInterfaceName}_NumUVs;
uint {DataInterfaceName}_HasColors;

Buffer<uint> {DataInterfaceName}_IndexBuffer;
Buffer<float> {DataInterfaceName}_PositionBuffer;
Buffer<float4> {DataInterfaceName}_TangentBuffer;
Buffer<float2> {DataInterfaceName}_UVBuffer;
Buffer<float4> {DataInterfaceName}_ColorBuffer;

float3 {DataInterfaceName}_BoundsExtents;

// #################### VERTEX BASED SAMPLING ##########################

int GetNumVertices_{DataInterfaceName}(int DataIndex)
{
	return {DataInterfaceName}_NumVertices;
}

void GetVertex_{DataInterfaceName}(int DataIndex, int VertexIndex, out float3 OutPosition, out float3 OutNormal, out float3 OutTangent, out float3 OutBitangent)
{
	if ({DataInterfaceName}_NumVertices > 0)
	{
		VertexIndex = clamp(VertexIndex, 0, {DataInterfaceName}_NumVertices - 1);

		OutPosition.x = {DataInterfaceName}_PositionBuffer[VertexIndex * 3 + 0];
		OutPosition.y = {DataInterfaceName}_PositionBuffer[VertexIndex * 3 + 1];
		OutPosition.z = {DataInterfaceName}_PositionBuffer[VertexIndex * 3 + 2];

		float4 TangentX = TangentBias({DataInterfaceName}_TangentBuffer[VertexIndex * 2 + 0]);
		float4 TangentZ = TangentBias({DataInterfaceName}_TangentBuffer[VertexIndex * 2 + 1]);
		float3 TangentY = cross(TangentZ.xyz, TangentX.xyz) * TangentZ.w;

		// Sometimes we have bogus tangent data in the buffer, i.e. X==Z so ensure we handle this and don't end up with NaNs in the buffer
		if (length2(TangentY) > 0.001f)
		{
			OutNormal = TangentZ.xyz;
			OutTangent = TangentX.xyz;
			OutBitangent = TangentY;
		}
		else
		{
			OutNormal = float3(0, 0, 1);
			OutTangent = float3(1, 0, 0);
			OutBitangent = float3(0, 1, 0);
		}
	}
	else
	{
		OutPosition = float3(0, 0, 0);
		OutNormal = float3(0, 0, 1);
		OutBitangent = float3(0, 1, 0);
		OutTangent = float3(1, 0, 0);
	}
}

float4 GetVertexColor_{DataInterfaceName}(int DataIndex, int VertexIndex)
{
	VertexIndex = clamp(VertexIndex, 0, {DataInterfaceName}_NumVertices - 1);

	if ({DataInterfaceName}_HasColors > 0)
	{
		return {DataInterfaceName}_ColorBuffer[VertexIndex] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE;
	}
	else
	{
		return float4(1.0f, 1.0f, 1.0f, 1.0f);
	}
}

float2 GetVertexUV_{DataInterfaceName}(int DataIndex, int VertexIndex, int UVSet)
{
	VertexIndex = clamp(VertexIndex, 0, {DataInterfaceName}_NumVertices - 1);
	UVSet = clamp(UVSet, 0, {DataInterfaceName}_NumUVs - 1);

	return {DataInterfaceName}_UVBuffer[VertexIndex * {DataInterfaceName}_NumUVs + UVSet];
}

// #################### TRIANGLE BASED SAMPLING ##########################

int GetNumTriangles_{DataInterfaceName}(int DataIndex)
{
	return {DataInterfaceName}_NumTriangles;
}

void GetTriangleIndices_{DataInterfaceName}(int DataIndex, int TriangleIndex, out int OutIndex0, out int OutIndex1, out int OutIndex2)
{
	if ({DataInterfaceName}_NumTriangles > 0)
	{
		TriangleIndex = clamp(TriangleIndex, 0, {DataInterfaceName}_NumTriangles - 1);
		OutIndex0 = {DataInterfaceName}_IndexBuffer[TriangleIndex * 3 + 0];
		OutIndex1 = {DataInterfaceName}_IndexBuffer[TriangleIndex * 3 + 1];
		OutIndex2 = {DataInterfaceName}_IndexBuffer[TriangleIndex * 3 + 2];
	}
	else
	{
		OutIndex0 = 0;
		OutIndex1 = 0;
		OutIndex2 = 0;
	}
}

void SampleTriangle_{DataInterfaceName}(
	int DataIndex,
	int TriangleIndex,
	float3 BaryCoord,
	out float3 OutPosition,
	out float3 OutNormal,
	out float3 OutTangent,
	out float3 OutBitangent)
{
	int Index[3];
	GetTriangleIndices_{DataInterfaceName}(DataIndex, TriangleIndex, Index[0], Index[1], Index[2]);

	float3 Positions[3], Normals[3], Bitangents[3], Tangents[3];
	GetVertex_{DataInterfaceName}(DataIndex, Index[0], Positions[0], Normals[0], Tangents[0], Bitangents[0]);
	GetVertex_{DataInterfaceName}(DataIndex, Index[1], Positions[1], Normals[1], Tangents[1], Bitangents[1]);
	GetVertex_{DataInterfaceName}(DataIndex, Index[2], Positions[2], Normals[2], Tangents[2], Bitangents[2]);

	OutPosition = Positions[0] * BaryCoord.x + Positions[1] * BaryCoord.y + Positions[2] * BaryCoord.z;
	OutNormal = Normals[0] * BaryCoord.x + Normals[1] * BaryCoord.y + Normals[2] * BaryCoord.x;
	OutTangent = Tangents[0] * BaryCoord.x + Tangents[1] * BaryCoord.y + Tangents[2] * BaryCoord.x;
	OutBitangent = Bitangents[0] * BaryCoord.x + Bitangents[1] * BaryCoord.y + Bitangents[2] * BaryCoord.x;
}

float4 SampleTriangleColor_{DataInterfaceName}(int DataIndex, int TriangleIndex, float3 BaryCoord)
{
	int Index[3];
	GetTriangleIndices_{DataInterfaceName}(DataIndex, TriangleIndex, Index[0], Index[1], Index[2]);

	float4 Colors[3];
	Colors[0] = GetVertexColor_{DataInterfaceName}(DataIndex, Index[0]);
	Colors[1] = GetVertexColor_{DataInterfaceName}(DataIndex, Index[1]);
	Colors[2] = GetVertexColor_{DataInterfaceName}(DataIndex, Index[2]);

	return Colors[0] * BaryCoord.x + Colors[1] * BaryCoord.y + Colors[2] * BaryCoord.z;
}

float2 SampleTriangleUV_{DataInterfaceName}(int DataIndex, int TriangleIndex, float3 BaryCoord, int UVSet)
{
	int Index[3];
	GetTriangleIndices_{DataInterfaceName}(DataIndex, TriangleIndex, Index[0], Index[1], Index[2]);

	float2 UVs[3];
	UVs[0] = GetVertexUV_{DataInterfaceName}(DataIndex, Index[0], UVSet);
	UVs[1] = GetVertexUV_{DataInterfaceName}(DataIndex, Index[1], UVSet);
	UVs[2] = GetVertexUV_{DataInterfaceName}(DataIndex, Index[2], UVSet);

	return UVs[0] * BaryCoord.x + UVs[1] * BaryCoord.y + UVs[2] * BaryCoord.z;
}

// #################### MISC ##########################

float3 GetMeshBoundsExtents_{DataInterfaceName}(int DataIndex)
{
	return {DataInterfaceName}_BoundsExtents;
}
