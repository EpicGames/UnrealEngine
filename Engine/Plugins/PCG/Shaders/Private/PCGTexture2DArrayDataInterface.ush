// Copyright Epic Games, Inc. All Rights Reserved.

Texture2DArray {DataInterfaceName}_TextureSRV;
RWTexture2DArray<float4> {DataInterfaceName}_TextureUAV;
SamplerState {DataInterfaceName}_Sampler;
uint2 {DataInterfaceName}_Resolution;
uint {DataInterfaceName}_ArraySize;
float4x4 {DataInterfaceName}_TextureTransform;
float4x4 {DataInterfaceName}_TextureInverseTransform;
float2 {DataInterfaceName}_TexelSizeWorld;
uint {DataInterfaceName}_NumMips;

// #################### GENERAL HELPERS ##########################

uint GetNumData_{DataInterfaceName}()
{
	return 1u;
}

uint2 GetResolution_{DataInterfaceName}(uint InDataIndex)
{
	return {DataInterfaceName}_Resolution;
}

uint GetArraySize_{DataInterfaceName}(uint InDataIndex)
{
	return {DataInterfaceName}_ArraySize;
}

uint3 GetNumElements_{DataInterfaceName}(uint InDataIndex)
{
	return uint3({DataInterfaceName}_Resolution.xy, {DataInterfaceName}_ArraySize);
}

bool GetThreadData_{DataInterfaceName}(uint InThreadIndex, out uint OutDataIndex, out uint3 OutElementIndex)
{
	const uint TexelsPerSlice = {DataInterfaceName}_Resolution.x * {DataInterfaceName}_Resolution.y;
	const uint SliceIndex = InThreadIndex / TexelsPerSlice;

	if (SliceIndex < 0 || SliceIndex >= {DataInterfaceName}_ArraySize)
	{
		OutDataIndex = (uint)-1;
		OutElementIndex = (uint3)-1;
		return false;
	}

	// Linear texel index within the current slice.
	const uint LinearTexelIndex = InThreadIndex - SliceIndex * TexelsPerSlice;

	OutElementIndex.x = LinearTexelIndex % {DataInterfaceName}_Resolution.x;
	OutElementIndex.y = LinearTexelIndex / {DataInterfaceName}_Resolution.x;
	OutElementIndex.z = SliceIndex;

	// This data interface is single-data.
	OutDataIndex = 0;

	return true;
}

// #################### INPUT FUNCTIONS ##########################

float2 GetTexCoords_{DataInterfaceName}(uint InDataIndex, float3 InWorldPos)
{
	const float4 LocalPos = mul({DataInterfaceName}_TextureInverseTransform, float4(InWorldPos, 1.0));

	// LocalPos will be in [-1, 1] so transform to a texture coordinate in [0, 1]
	return LocalPos.xy * 0.5f + 0.5f;
}

float2 GetTexelSize_{DataInterfaceName}(uint InDataIndex)
{
	return 1.0f / float2(GetResolution_{DataInterfaceName}(InDataIndex));
}

float2 GetTexelSizeWorld_{DataInterfaceName}(uint InDataIndex)
{
	return {DataInterfaceName}_TexelSizeWorld;
}

float4x4 GetTransform_{DataInterfaceName}(uint InDataIndex)
{
	return {DataInterfaceName}_TextureTransform;
}

uint GetNumMips_{DataInterfaceName}(uint InDataIndex)
{
	return {DataInterfaceName}_NumMips;
}

float4 Sample_{DataInterfaceName}(uint InDataIndex, float2 InTextureUV, uint InSliceIndex)
{
	return {DataInterfaceName}_TextureSRV.SampleLevel({DataInterfaceName}_Sampler, float3(InTextureUV, InSliceIndex), 0);
}

float4 SampleWorldPos_{DataInterfaceName}(uint InDataIndex, float3 WorldPos, uint InSliceIndex)
{
	const float2 TextureUV = GetTexCoords_{DataInterfaceName}(InDataIndex, WorldPos);
	return {DataInterfaceName}_TextureSRV.SampleLevel({DataInterfaceName}_Sampler, float3(TextureUV, InSliceIndex), 0);
}

float4 Load_{DataInterfaceName}(uint InDataIndex, uint3 InElementIndex)
{
	return {DataInterfaceName}_TextureSRV[InElementIndex];
}

float4 Load_{DataInterfaceName}(uint InDataIndex, uint3 InElementIndex, uint InMipIndex)
{
	return {DataInterfaceName}_TextureSRV.Load(int4(InElementIndex, InMipIndex));
}

// #################### OUTPUT FUNCTIONS ##########################

void Store_{DataInterfaceName}(uint InDataIndex, uint3 InElementIndex, float4 InValue)
{
	{DataInterfaceName}_TextureUAV[InElementIndex] = InValue;
}
