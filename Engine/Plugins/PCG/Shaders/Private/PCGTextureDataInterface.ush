// Copyright Epic Games, Inc. All Rights Reserved.

// todo_pcg: Runtime diagnostic errors would be useful if texture resource types don't match expectations below.

#define PCG_ENABLE_MULTIPLE_TEXTURE_OBJECTS {EnableMultipleTextureObjects}

// Mirror of ETextureDimension
#define PCG_TextureDimension_Texture2D          0
#define PCG_TextureDimension_Texture2DArray     1
//#define PCG_TextureDimension_Texture3D        2
//#define PCG_TextureDimension_TextureCube      3
//#define PCG_TextureDimension_TextureCubeArray 4

// @todo_pcg: The Texture2DArray binding can go away in the future if we support a UPCGTexture2DArrayData.
#define PCG_TEXTURE_SRV_DECLARATION(Index) \
	Texture2D {DataInterfaceName}_Texture_##Index; \
	Texture2DArray {DataInterfaceName}_TextureArray_##Index;

PCG_TEXTURE_SRV_DECLARATION(0);
#if PCG_ENABLE_MULTIPLE_TEXTURE_OBJECTS
PCG_TEXTURE_SRV_DECLARATION(1);
PCG_TEXTURE_SRV_DECLARATION(2);
PCG_TEXTURE_SRV_DECLARATION(3);
PCG_TEXTURE_SRV_DECLARATION(4);
PCG_TEXTURE_SRV_DECLARATION(5);
PCG_TEXTURE_SRV_DECLARATION(6);
PCG_TEXTURE_SRV_DECLARATION(7);
#endif // PCG_ENABLE_MULTIPLE_TEXTURE_OBJECTS

#undef PCG_TEXTURE_SRV_DECLARATION

#define PCG_TEXTURE_UAV_DECLARATION(Index) \
	RWTexture2D<float4> {DataInterfaceName}_TextureUAV_##Index; \

PCG_TEXTURE_UAV_DECLARATION(0);

#undef PCG_TEXTURE_UAV_DECLARATION

SamplerState {DataInterfaceName}_Sampler;

float4x4 {DataInterfaceName}_TextureTransforms[PCG_MAX_NUM_SRV_BINDINGS];
float4x4 {DataInterfaceName}_TextureInverseTransforms[PCG_MAX_NUM_SRV_BINDINGS];

/** Layout of TextureInfos buffer:
 * - TextureInfo0
 *   - uint32 - BindingIndex
 *   - uint32 - SliceIndex (aka TextureIndex if bound to a TextureArray, 0 otherwise.)
 *   - uint32 - Dimension
 *   - uint32 - NumMips
 *   - uint32 - ResolutionX
 *   - uint32 - ResolutionY
 *   - float  - TexelSizeWorldX (cm)
 *   - float  - TexelSizeWorldY (cm)
 * - TextureInfo1
 * - TextureInfo2
 * - ...
 */
StructuredBuffer<uint4> {DataInterfaceName}_TextureInfos;
uint {DataInterfaceName}_NumTextureInfos;

// #################### INTERNAL HELPERS ##########################

void GetTextureInfoInternal_{DataInterfaceName}(uint InDataIndex, out uint OutBindingIndex, out uint OutSliceIndex, out uint OutTextureDimension, out uint OutNumMips)
{
	if (InDataIndex >= 0u && InDataIndex < {DataInterfaceName}_NumTextureInfos)
	{
		const uint4 TextureInfo = {DataInterfaceName}_TextureInfos[InDataIndex * 2 + 0];
		OutBindingIndex = TextureInfo.x;
		OutSliceIndex = TextureInfo.y;
		OutTextureDimension = TextureInfo.z;
		OutNumMips = TextureInfo.w;
	}
	else
	{
		// Out-of-bounds: use an invalid binding index and dimension so all sample/load functions return (float4)0.
		OutBindingIndex = PCG_MAX_NUM_SRV_BINDINGS;
		OutSliceIndex = 0u;
		OutTextureDimension = (uint)-1;
		OutNumMips = 1u;
	}
}

void GetTextureResolutionInternal_{DataInterfaceName}(uint InDataIndex, out uint2 OutResolution)
{
	if (InDataIndex >= 0u && InDataIndex < {DataInterfaceName}_NumTextureInfos)
	{
		OutResolution = {DataInterfaceName}_TextureInfos[InDataIndex * 2 + 1].xy;
	}
	else
	{
		OutResolution = (uint2)0;
	}
}

// #################### GENERAL HELPERS ##########################

uint GetNumData_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumTextureInfos;
}

uint2 GetNumElements_{DataInterfaceName}(uint InDataIndex)
{
	if (InDataIndex >= 0u && InDataIndex < {DataInterfaceName}_NumTextureInfos)
	{
		return {DataInterfaceName}_TextureInfos[InDataIndex * 2 + 1].xy;
	}
	else
	{
		return uint2(0u, 0u);
	}
}

bool GetThreadData_{DataInterfaceName}(uint InThreadIndex, out uint OutDataIndex, out uint2 OutElementIndex)
{
	uint LinearElementIndex = InThreadIndex;

	for (uint DataIndex = 0; DataIndex < {DataInterfaceName}_NumTextureInfos; ++DataIndex)
	{
		const uint2 NumElements = GetNumElements_{DataInterfaceName}(DataIndex);
		const uint ElementCount = NumElements.x * NumElements.y;

		if (LinearElementIndex < ElementCount)
		{
			OutDataIndex = DataIndex;
			OutElementIndex.x = LinearElementIndex % NumElements.x;
			OutElementIndex.y = LinearElementIndex / NumElements.x;
			return true;
		}

		LinearElementIndex -= ElementCount;
	}

	OutDataIndex = (uint)-1;
	OutElementIndex = (uint2)-1;
	return false;
}

// #################### INPUT FUNCTIONS ##########################

float2 GetTexCoords_{DataInterfaceName}(uint InDataIndex, float3 InWorldPos)
{
	const float4x4 InverseTransform = {DataInterfaceName}_TextureInverseTransforms[InDataIndex];
	const float4 LocalPos = mul(InverseTransform, float4(InWorldPos, 1.0));

	// LocalPos will be in [-1, 1] so transform to a texture coordinate in [0, 1]
	return LocalPos.xy * 0.5 + 0.5;
}

float2 GetTexelSize_{DataInterfaceName}(uint InDataIndex)
{
	uint2 Resolution;
	GetTextureResolutionInternal_{DataInterfaceName}(InDataIndex, Resolution);

	return 1.0f / float2(Resolution);
}

float2 GetTexelSizeWorld_{DataInterfaceName}(uint InDataIndex)
{
	return asfloat({DataInterfaceName}_TextureInfos[InDataIndex * 2 + 1].zw);
}

uint GetNumMips_{DataInterfaceName}(uint InDataIndex)
{
	uint BindingIndex = 0;
	uint SliceIndex = 0;
	uint TextureDimension = 0;
	uint NumMips = 1;
	GetTextureInfoInternal_{DataInterfaceName}(InDataIndex, BindingIndex, SliceIndex, TextureDimension, NumMips);
	return NumMips;
}

float4x4 GetTransform_{DataInterfaceName}(uint InDataIndex)
{
	return {DataInterfaceName}_TextureTransforms[InDataIndex];
}

#define PCG_CONDITIONAL_TEXTURE_SAMPLE(Index) \
	if (BindingIndex == Index) \
	{ \
		if (TextureDimension == PCG_TextureDimension_Texture2D) \
		{ \
			return {DataInterfaceName}_Texture_##Index.SampleLevel({DataInterfaceName}_Sampler, InTextureUV, 0); \
		} \
		else if (TextureDimension == PCG_TextureDimension_Texture2DArray) \
		{ \
			return {DataInterfaceName}_TextureArray_##Index.SampleLevel({DataInterfaceName}_Sampler, float3(InTextureUV, SliceIndex), 0); \
		} \
		else \
		{ \
			return (float4)0; \
		} \
	} \
    else

float4 Sample_{DataInterfaceName}(uint InDataIndex, float2 InTextureUV)
{
	uint BindingIndex = 0;
	uint SliceIndex = 0;
	uint TextureDimension = 0;
	uint NumMips = 1;
	GetTextureInfoInternal_{DataInterfaceName}(InDataIndex, BindingIndex, SliceIndex, TextureDimension, NumMips);

#if PCG_ENABLE_MULTIPLE_TEXTURE_OBJECTS
	PCG_CONDITIONAL_TEXTURE_SAMPLE(0)
	PCG_CONDITIONAL_TEXTURE_SAMPLE(1)
	PCG_CONDITIONAL_TEXTURE_SAMPLE(2)
	PCG_CONDITIONAL_TEXTURE_SAMPLE(3)
	PCG_CONDITIONAL_TEXTURE_SAMPLE(4)
	PCG_CONDITIONAL_TEXTURE_SAMPLE(5)
	PCG_CONDITIONAL_TEXTURE_SAMPLE(6)
	PCG_CONDITIONAL_TEXTURE_SAMPLE(7)
	// else - from macro
	{
		return (float4)0;
	}
#else
	if (TextureDimension == PCG_TextureDimension_Texture2D)
	{
		return {DataInterfaceName}_Texture_0.SampleLevel({DataInterfaceName}_Sampler, InTextureUV, 0);
	}
	else if (TextureDimension == PCG_TextureDimension_Texture2DArray)
	{
		return {DataInterfaceName}_TextureArray_0.SampleLevel({DataInterfaceName}_Sampler, float3(InTextureUV, SliceIndex), 0);
	}
	else
	{
		return (float4)0;
	}
#endif
}

#undef PCG_CONDITIONAL_TEXTURE_SAMPLE

#define PCG_CONDITIONAL_TEXTURE_SAMPLE_WORLD_POS(Index) \
	if (BindingIndex == Index) \
	{ \
		const float2 TextureUV = GetTexCoords_{DataInterfaceName}(Index, WorldPos); \
		if (TextureDimension == PCG_TextureDimension_Texture2D) \
		{ \
			return {DataInterfaceName}_Texture_##Index.SampleLevel({DataInterfaceName}_Sampler, TextureUV, 0); \
		} \
		else if (TextureDimension == PCG_TextureDimension_Texture2DArray) \
		{ \
			return {DataInterfaceName}_TextureArray_##Index.SampleLevel({DataInterfaceName}_Sampler, float3(TextureUV, SliceIndex), 0); \
		} \
		else \
		{ \
			return (float4)0; \
		} \
	} \
	else

float4 SampleWorldPos_{DataInterfaceName}(uint InDataIndex, float3 WorldPos)
{
	uint BindingIndex = 0;
	uint SliceIndex = 0;
	uint TextureDimension = 0;
	uint NumMips = 1;
	GetTextureInfoInternal_{DataInterfaceName}(InDataIndex, BindingIndex, SliceIndex, TextureDimension, NumMips);

#if PCG_ENABLE_MULTIPLE_TEXTURE_OBJECTS
	PCG_CONDITIONAL_TEXTURE_SAMPLE_WORLD_POS(0)
	PCG_CONDITIONAL_TEXTURE_SAMPLE_WORLD_POS(1)
	PCG_CONDITIONAL_TEXTURE_SAMPLE_WORLD_POS(2)
	PCG_CONDITIONAL_TEXTURE_SAMPLE_WORLD_POS(3)
	PCG_CONDITIONAL_TEXTURE_SAMPLE_WORLD_POS(4)
	PCG_CONDITIONAL_TEXTURE_SAMPLE_WORLD_POS(5)
	PCG_CONDITIONAL_TEXTURE_SAMPLE_WORLD_POS(6)
	PCG_CONDITIONAL_TEXTURE_SAMPLE_WORLD_POS(7)
	// else - from macro
	{
		return (float4)0;
	}
#else
	const float2 TextureUV = GetTexCoords_{DataInterfaceName}(0, WorldPos);

	if (TextureDimension == PCG_TextureDimension_Texture2D)
	{
		return {DataInterfaceName}_Texture_0.SampleLevel({DataInterfaceName}_Sampler, TextureUV, 0);
	}
	else if (TextureDimension == PCG_TextureDimension_Texture2DArray)
	{
		return {DataInterfaceName}_TextureArray_0.SampleLevel({DataInterfaceName}_Sampler, float3(TextureUV, SliceIndex), 0);
	}
	else
	{
		return (float4)0;
	}
#endif
}

#undef PCG_CONDITIONAL_TEXTURE_SAMPLE_WORLD_POS

#define PCG_CONDITIONAL_TEXTURE_LOAD(Index) \
	if (BindingIndex == Index) \
	{ \
		if (TextureDimension == PCG_TextureDimension_Texture2D) \
		{ \
			return {DataInterfaceName}_Texture_##Index.Load(uint3(InElementIndex, InMipIndex)); \
		} \
		else if (TextureDimension == PCG_TextureDimension_Texture2DArray) \
		{ \
			return {DataInterfaceName}_TextureArray_##Index.Load(uint4(InElementIndex, SliceIndex, InMipIndex)); \
		} \
		else \
		{ \
			return (float4)0; \
		} \
	} \
	else

float4 Load_{DataInterfaceName}(uint InDataIndex, uint2 InElementIndex, uint InMipIndex)
{
	uint BindingIndex = 0;
	uint SliceIndex = 0;
	uint TextureDimension = 0;
	uint NumMips = 1;
	GetTextureInfoInternal_{DataInterfaceName}(InDataIndex, BindingIndex, SliceIndex, TextureDimension, NumMips);

#if PCG_ENABLE_MULTIPLE_TEXTURE_OBJECTS
	PCG_CONDITIONAL_TEXTURE_LOAD(0)
	PCG_CONDITIONAL_TEXTURE_LOAD(1)
	PCG_CONDITIONAL_TEXTURE_LOAD(2)
	PCG_CONDITIONAL_TEXTURE_LOAD(3)
	PCG_CONDITIONAL_TEXTURE_LOAD(4)
	PCG_CONDITIONAL_TEXTURE_LOAD(5)
	PCG_CONDITIONAL_TEXTURE_LOAD(6)
	PCG_CONDITIONAL_TEXTURE_LOAD(7)
	// else - from macro
	{
		return (float4)0;
	}
#else
	if (TextureDimension == PCG_TextureDimension_Texture2D)
	{
		return {DataInterfaceName}_Texture_0.Load(uint3(InElementIndex, InMipIndex));
	}
	else if (TextureDimension == PCG_TextureDimension_Texture2DArray)
	{
		return {DataInterfaceName}_TextureArray_0.Load(uint4(InElementIndex, SliceIndex, InMipIndex));
	}
	else
	{
		return (float4)0;
	}
#endif
}

float4 Load_{DataInterfaceName}(uint InDataIndex, uint2 InElementIndex)
{
	return Load_{DataInterfaceName}(InDataIndex, InElementIndex, 0u);
}

#undef PCG_CONDITIONAL_TEXTURE_LOAD

#define PCG_CONDITIONAL_TEXTURE_STORE(Index) \
	if (BindingIndex == Index) \
	{ \
		{DataInterfaceName}_TextureUAV_##Index[InElementIndex] = InValue; \
		return; \
	}

// #################### OUTPUT FUNCTIONS ##########################

void Store_{DataInterfaceName}(uint InDataIndex, uint2 InElementIndex, float4 InValue)
{
	uint BindingIndex = 0;
	uint SliceIndex = 0;
	uint TextureDimension = 0;
	uint NumMips = 1;
	GetTextureInfoInternal_{DataInterfaceName}(InDataIndex, BindingIndex, SliceIndex, TextureDimension, NumMips);

#if PCG_ENABLE_MULTIPLE_TEXTURE_OBJECTS
	PCG_CONDITIONAL_TEXTURE_STORE(0)
#else
	{DataInterfaceName}_TextureUAV_0[InElementIndex] = InValue;
#endif
}

#undef PCG_CONDITIONAL_TEXTURE_STORE

// #################### DEPRECATED FUNCTIONS ##########################

// Deprecated (5.6). Fallback to using first data.
float4 Sample_{DataInterfaceName}(float2 TexCoords)
{
	return Sample_{DataInterfaceName}(/*InDataIndex=*/0, TexCoords);
}

// Deprecated (5.6). Fallback to using first data.
float4 SampleWorldPos_{DataInterfaceName}(float2 WorldPos)
{
	return SampleWorldPos_{DataInterfaceName}(/*InDataIndex=*/0, float3(WorldPos, 0.0f));
}

// Deprecated (5.7). Does not take data index, and assumes axis-aligned Z projection.
float2 GetTexCoords_{DataInterfaceName}(float2 WorldPos, float2 Min, float2 Max)
{
	// Inverse lerp
	const float2 Pos = WorldPos - Min;
	const float2 Extents = Max - Min;

	if (!all(Extents))
	{
		return (float2)0;
	}

	return Pos / Extents;
}

// Deprecated (5.7). Switched to using 3D world pos to support any texture orientation.
float4 SampleWorldPos_{DataInterfaceName}(uint InDataIndex, float2 InWorldPos)
{
	return SampleWorldPos_{DataInterfaceName}(InDataIndex, float3(InWorldPos, 0.0f));
}

#undef PCG_TextureDimension_Texture2D
#undef PCG_TextureDimension_Texture2DArray
//#undef PCG_TextureDimension_Texture3D
//#undef PCG_TextureDimension_TextureCube
//#undef PCG_TextureDimension_TextureCubeArray
