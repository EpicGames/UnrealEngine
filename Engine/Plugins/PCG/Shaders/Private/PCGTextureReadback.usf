// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Public/Platform.ush"

Texture2D<float4> SourceTexture;
Texture2DArray<float4> SourceTextureArray;
SamplerState SourceSampler;
float2 SourceDimensions;
int SourceTextureIndex;

RWTexture2D<float4> OutputTexture;

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, THREADGROUPSIZE_Z)]
void PCGTextureReadback_CS(uint3 DispatchThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	float2 ThreadID = DispatchThreadId.xy + 0.5;
	float2 TexCoord = ThreadID / SourceDimensions;

	if (SourceTextureIndex == -1)
	{
		OutputTexture[ThreadID] = Texture2DSample(SourceTexture, SourceSampler, TexCoord.xy);
	}
	else
	{
		OutputTexture[ThreadID] = Texture2DArraySample(SourceTextureArray, SourceSampler, float3(TexCoord.xy, SourceTextureIndex));
	}
}
