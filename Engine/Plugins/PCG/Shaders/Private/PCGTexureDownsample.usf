// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"

#define PCG_DOWNSAMPLE_MODE_AVERAGE		0
#define PCG_DOWNSAMPLE_MODE_MIN			1
#define PCG_DOWNSAMPLE_MODE_MAX			2
#define PCG_DOWNSAMPLE_MODE_SUM			3

SamplerState MipSampler;
uint2 MipOutSize;
float2 TexelSize;

#if PCG_DOWNSAMPLE_SRGB
#define PCG_DATA_TYPE half4
#else
#define PCG_DATA_TYPE float4
#endif

#if PCG_DOWNSAMPLE_TEXTUREARRAY
Texture2DArray MipArrayInSRV;
RWTexture2DArray<PCG_DATA_TYPE> MipArrayOutUAV;
#define PCG_TEXTURE_SAMPLE() MipArrayInSRV.SampleLevel(MipSampler, float3(UV, InThreadID.z), 0)
#define PCG_TEXTURE_LOAD(Coords) MipArrayInSRV[uint3(Coords, InThreadID.z)]
#define PCG_STORE_RESULT() MipArrayOutUAV[InThreadID] = OutColor
#else
Texture2D MipInSRV;
RWTexture2D<PCG_DATA_TYPE> MipOutUAV;
#define PCG_TEXTURE_SAMPLE() MipInSRV.SampleLevel(MipSampler, UV, 0)
#define PCG_TEXTURE_LOAD(Coords) MipInSRV[Coords]
#define PCG_STORE_RESULT() MipOutUAV[InThreadID.xy] = OutColor
#endif

[numthreads(8, 8, 1)]
void TextureDownsampleCS(const uint3 InThreadID : SV_DispatchThreadID)
{
	if (any(InThreadID.xy >= MipOutSize))
	{
		return;
	}

#if PCG_DOWNSAMPLE_MODE == PCG_DOWNSAMPLE_MODE_AVERAGE

	const float2 UV = TexelSize * (InThreadID.xy + 0.5f);

    PCG_DATA_TYPE OutColor = PCG_TEXTURE_SAMPLE();
#if PCG_DOWNSAMPLE_SRGB
    // Sample operation already did sRGB->linear conversion, so just need to convert back.
    OutColor = PCG_DATA_TYPE(LinearToSrgb(OutColor.xyz), OutColor.w);
#endif

#else // PCG_DOWNSAMPLE_MODE == PCG_DOWNSAMPLE_MODE_AVERAGE

	PCG_DATA_TYPE InColor00 = PCG_TEXTURE_LOAD(InThreadID.xy * 2);
	PCG_DATA_TYPE InColor01 = PCG_TEXTURE_LOAD(InThreadID.xy * 2 + uint2(0, 1));
	PCG_DATA_TYPE InColor10 = PCG_TEXTURE_LOAD(InThreadID.xy * 2 + uint2(1, 0));
	PCG_DATA_TYPE InColor11 = PCG_TEXTURE_LOAD(InThreadID.xy * 2 + uint2(1, 1));

#if PCG_DOWNSAMPLE_MODE == PCG_DOWNSAMPLE_MODE_MIN

	// Min is color space agnostic - load and store in whatever color space is in use.
	const PCG_DATA_TYPE OutColor = min(min(InColor00, InColor01), min(InColor10, InColor11));

#elif PCG_DOWNSAMPLE_MODE == PCG_DOWNSAMPLE_MODE_MAX

	// Max is color space agnostic - load and store in whatever color space is in use.
	const PCG_DATA_TYPE OutColor = max(max(InColor00, InColor01), max(InColor10, InColor11));

#elif PCG_DOWNSAMPLE_MODE == PCG_DOWNSAMPLE_MODE_SUM

#if PCG_DOWNSAMPLE_SRGB
	// Convert to linear space before doing math, then convert result back.
	InColor00.xyz = sRGBToLinear(InColor00.xyz);
	InColor01.xyz = sRGBToLinear(InColor01.xyz);
	InColor10.xyz = sRGBToLinear(InColor10.xyz);
	InColor11.xyz = sRGBToLinear(InColor11.xyz);
    const PCG_DATA_TYPE OutColor = PCG_DATA_TYPE(LinearToSrgb(InColor00.xyz + InColor01.xyz + InColor10.xyz + InColor11.xyz), InColor00.w + InColor01.w + InColor10.w + InColor11.w);
#else // PCG_DOWNSAMPLE_SRGB
	const PCG_DATA_TYPE OutColor = InColor00 + InColor01 + InColor10 + InColor11;
#endif // PCG_DOWNSAMPLE_SRGB

#endif // PCG_DOWNSAMPLE_MODE == PCG_DOWNSAMPLE_MODE_SUM

#endif // PCG_DOWNSAMPLE_MODE == PCG_DOWNSAMPLE_MODE_AVERAGE
	
	PCG_STORE_RESULT();
}

#undef PCG_DOWNSAMPLE_MODE_AVERAGE
#undef PCG_DOWNSAMPLE_MODE_MIN
#undef PCG_DOWNSAMPLE_MODE_MAX
#undef PCG_DOWNSAMPLE_MODE_SUM
#undef PCG_DATA_TYPE
#undef PCG_TEXTURE_SAMPLE
#undef PCG_TEXTURE_LOAD
#undef PCG_STORE_RESULT
