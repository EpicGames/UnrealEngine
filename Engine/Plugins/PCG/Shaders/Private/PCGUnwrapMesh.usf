// Copyright Epic Games, Inc. All Rights Reserved.

// Vertex shader maps UV to clip space so triangles rasterize into the output texture's UV domain.
// Pixel shader emits the selected per-texel mesh attribute selected by PCG_MESH_ATTRIBUTE.

#include "/Engine/Public/Platform.ush"

struct FUnwrapInterpolants
{
	float3 LocalPosition : TEXCOORD0;
};

void MainVS(
	float3 InPosition : ATTRIBUTE0,
	float2 InUV       : ATTRIBUTE1,
	out FUnwrapInterpolants OutInterps,
	out float4 OutPosition : SV_POSITION)
{
	OutInterps.LocalPosition = InPosition;
	// UV (0..1) to clip space. The "flip" on Y converts between the UV convention (v=0 at top) and clip-space convention (y=+1 at top).
	OutPosition = float4(InUV.x * 2.0f - 1.0f, 1.0f - InUV.y * 2.0f, 0.0f, 1.0f);
}

void MainPS(
	FUnwrapInterpolants Interps,
	out float4 OutColor : SV_Target0)
{
#if PCG_MESH_ATTRIBUTE == PCG_MESH_ATTRIBUTE_LOCALPOSITION
	// Local position. Alpha=1 marks a "written" texel so downstream samplers can mask cleared zero texels.
	OutColor = float4(Interps.LocalPosition, 1.0f);
#elif PCG_MESH_ATTRIBUTE == PCG_MESH_ATTRIBUTE_MASK
	// Coverage mask. Writes 1 on every channel so the mask is readable from any RT format (R8, R16F, RGBA, ...).
	OutColor = float4(1.0f, 1.0f, 1.0f, 1.0f);
#else
	#error "Unhandled PCG_MESH_ATTRIBUTE value"
#endif
}
