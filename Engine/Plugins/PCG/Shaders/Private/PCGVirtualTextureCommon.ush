// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define NUM_VIRTUALTEXTURE_SAMPLES 1
// Workaround for unbound View uniform buffer. This pattern is used elsewhere in the engine as well.
#define VT_DISABLE_VIEW_UNIFORM_BUFFER 1

#include "/Engine/Private/VirtualTextureCommon.ush"

// Mirror of ERuntimeVirtualTextureMaterialType
#define ERuntimeVirtualTextureMaterialType_BaseColor								0
#define ERuntimeVirtualTextureMaterialType_Mask4									1
#define ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Roughness				2
#define ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular				3
#define ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular_YCoCg			4
#define ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular_Mask_YCoCg		5
#define ERuntimeVirtualTextureMaterialType_WorldHeight								6
#define ERuntimeVirtualTextureMaterialType_Displacement								7
#define ERuntimeVirtualTextureMaterialType_Count									8

// Mirror of PCGVirtualTextureCommon::ENormalUnpackType
#define VIRTUAL_TEXTURE_UNPACK_NORMAL_NONE		0
#define VIRTUAL_TEXTURE_UNPACK_NORMAL_BC3BC3	1
#define VIRTUAL_TEXTURE_UNPACK_NORMAL_BC5BC1	2
#define VIRTUAL_TEXTURE_UNPACK_NORMAL_B5G6R5	3
