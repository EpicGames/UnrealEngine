// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/PCG/Private/PCGVirtualTextureCommon.ush"

#define PCG_VIRTUAL_TEXTURE_DECLARATION(DataIndex) \
	Texture2D {DataInterfaceName}_VirtualTexture0_##DataIndex; \
	Texture2D {DataInterfaceName}_VirtualTexture1_##DataIndex; \
	Texture2D {DataInterfaceName}_VirtualTexture2_##DataIndex; \
	Texture2D<uint4> {DataInterfaceName}_PageTable_##DataIndex; \
	Texture2D<uint> {DataInterfaceName}_PageTableIndirection_##DataIndex;

PCG_VIRTUAL_TEXTURE_DECLARATION(0)
PCG_VIRTUAL_TEXTURE_DECLARATION(1)
PCG_VIRTUAL_TEXTURE_DECLARATION(2)
PCG_VIRTUAL_TEXTURE_DECLARATION(3)

#undef PCG_VIRTUAL_TEXTURE_DECLARATION

// Binding infos
uint4 {DataInterfaceName}_PageTableUniforms[PCG_MAX_NUM_BINDINGS * 2];
uint4 {DataInterfaceName}_PageTableInfo[PCG_MAX_NUM_BINDINGS];
float4 {DataInterfaceName}_LWCTiles[PCG_MAX_NUM_BINDINGS];
float4x4 {DataInterfaceName}_WorldToUVTransforms[PCG_MAX_NUM_BINDINGS];

// Layer infos
uint4 {DataInterfaceName}_Uniforms[PCG_MAX_NUM_BINDINGS * PCG_MAX_NUM_LAYERS];
uint4 {DataInterfaceName}_ValidLayerMasks[PCG_MAX_NUM_BINDINGS];

// Shared
uint {DataInterfaceName}_NumVirtualTextures;
SamplerState {DataInterfaceName}_Sampler;

// Sample the page table
VTPageTableResult SamplePageTable_{DataInterfaceName}(
	uint InDataIndex,
	float2 InSampleUV,
	Texture2D<uint4> InPageTable,
	Texture2D<uint> InPageTableIndirection,
	uint4 PageTableUniforms[2],
	bool InAdaptive)
{
	VTPageTableResult PageTable;

	if (InAdaptive == 0)
	{
		PageTable = TextureLoadVirtualPageTableLevel(
			InPageTable,
			VTPageTableUniform_Unpack(PageTableUniforms[0], PageTableUniforms[1]),
			InSampleUV,
			VTADDRESSMODE_CLAMP,
			VTADDRESSMODE_CLAMP,
			0.0f);
	}
	else
	{
		PageTable = TextureLoadVirtualPageTableAdaptiveLevel(
			InPageTable,
			InPageTableIndirection,
			VTPageTableUniform_Unpack(PageTableUniforms[0], PageTableUniforms[1]),
			InSampleUV,
			VTADDRESSMODE_CLAMP,
			VTADDRESSMODE_CLAMP,
			0.0f);
	}

	return PageTable;
}

// Sample the physical texture
float4 SampleLayer_{DataInterfaceName}(
	VTPageTableResult InPageTableResult,
	Texture2D InTexture,
	uint4 InUniforms,
	uint InLayerIndex)
{
	return TextureVirtualSample(InTexture, {DataInterfaceName}_Sampler, InPageTableResult, InLayerIndex, VTUniform_Unpack(InUniforms));
}

#define PCG_CONDITIONAL_VIRTUAL_TEXTURE_SAMPLE(DataIndex) \
	if (InDataIndex == DataIndex) \
	{ \
		VTPageTableResult PageTableResult = SamplePageTable_{DataInterfaceName}( \
			InDataIndex, \
			SampleUV, \
			{DataInterfaceName}_PageTable_##DataIndex, \
			{DataInterfaceName}_PageTableIndirection_##DataIndex, \
			PageTableUniforms, \
			bAdaptive); \
		if (ValidLayerMask & 0x1) \
		{ \
			LayerSample[0] = SampleLayer_{DataInterfaceName}(PageTableResult, {DataInterfaceName}_VirtualTexture0_##DataIndex, Uniforms0, 0); \
		} \
		else \
		{ \
			LayerSample[0] = (float4)0; \
		} \
		if (ValidLayerMask & 0x2) \
		{ \
			LayerSample[1] = SampleLayer_{DataInterfaceName}(PageTableResult, {DataInterfaceName}_VirtualTexture1_##DataIndex, Uniforms1, bSinglePhysicalSpace ? 0 : 1); \
		} \
		else \
		{ \
			LayerSample[1] = (float4)0; \
		} \
		if (ValidLayerMask & 0x4) \
		{ \
			LayerSample[2] = SampleLayer_{DataInterfaceName}(PageTableResult, {DataInterfaceName}_VirtualTexture2_##DataIndex, Uniforms2, bSinglePhysicalSpace ? 0 : 2); \
		} \
		else \
		{ \
			LayerSample[2] = (float4)0; \
		} \
	}

void SampleVirtualTexture_{DataInterfaceName}(
	uint InDataIndex,
	float3 InWorldPos,
	out bool bOutInsideVolume,
	out float3 OutBaseColor,
	out float OutSpecular,
	out float OutRoughness,
	out float OutWorldHeight,
	out float3 OutNormal,
	out float OutDisplacement,
	out float OutMask,
	out float4 OutMask4)
{
	bOutInsideVolume = false;
	OutBaseColor = float3(0.0f, 0.0f, 0.0f);
	OutSpecular = 0.5f;
	OutRoughness = 0.5f;
	OutWorldHeight = 0.0f;
	OutNormal = float3(0.0f, 0.0f, 1.0f);
	OutDisplacement = 0.0f;
	OutMask = 1.0f;
	OutMask4 = float4(0.0f, 0.0f, 0.0f, 0.0f);

	if (InDataIndex < 0 || InDataIndex >= {DataInterfaceName}_NumVirtualTextures)
	{
		return;
	}

	// Get sample location.
	const float4x4 WorldToUVTransform = {DataInterfaceName}_WorldToUVTransforms[InDataIndex];
	const float3 UnpackOrigin = WorldToUVTransform[0].xyz;
	const float3 UnpackU = WorldToUVTransform[1].xyz;
	const float3 UnpackV = WorldToUVTransform[2].xyz;
	const float3 LWCTile = (float3)0;

	const FDFVector3 LWCWorldPos = DFFromTileOffset_Hack(MakeLWCVector3(LWCTile, InWorldPos));
	const FDFVector3 LWCUVOrigin = DFFromTileOffset_Hack(MakeLWCVector3({DataInterfaceName}_LWCTiles[InDataIndex].xyz, UnpackOrigin));
	const float2 SampleUV = VirtualTextureWorldToUV(LWCWorldPos, LWCUVOrigin, UnpackU, UnpackV);

	// Test to see if we are inside the volume, but still take the samples as it will clamp to the edge.
	bOutInsideVolume = all(SampleUV >= 0.0f) && all(SampleUV < 1.0f);

	// Sample RVTs.
	float4 LayerSample[3];
	const uint ValidLayerMask = {DataInterfaceName}_ValidLayerMasks[InDataIndex].x;

	{
		uint4 PageTableUniforms[2] =
		{
			{DataInterfaceName}_PageTableUniforms[InDataIndex * 2 + 0],
			{DataInterfaceName}_PageTableUniforms[InDataIndex * 2 + 1],
		};

		bool bAdaptive = {DataInterfaceName}_PageTableInfo[InDataIndex].y != 0;
		bool bSinglePhysicalSpace = {DataInterfaceName}_PageTableInfo[InDataIndex].z != 0;

		const uint PackedLayerIndex = InDataIndex * PCG_MAX_NUM_LAYERS;

		uint4 Uniforms0 = {DataInterfaceName}_Uniforms[PackedLayerIndex + 0];
		uint4 Uniforms1 = {DataInterfaceName}_Uniforms[PackedLayerIndex + 1];
		uint4 Uniforms2 = {DataInterfaceName}_Uniforms[PackedLayerIndex + 2];

		PCG_CONDITIONAL_VIRTUAL_TEXTURE_SAMPLE(0)
#if ENABLE_MULTIPLE_VIRTUAL_TEXTURES
		PCG_CONDITIONAL_VIRTUAL_TEXTURE_SAMPLE(1)
		PCG_CONDITIONAL_VIRTUAL_TEXTURE_SAMPLE(2)
		PCG_CONDITIONAL_VIRTUAL_TEXTURE_SAMPLE(3)
#endif
	}

	// Unpack layer samples based on material type.
	const uint MaterialType = {DataInterfaceName}_PageTableInfo[InDataIndex].x;

	switch (MaterialType)
	{
		case ERuntimeVirtualTextureMaterialType_BaseColor:
		{
			OutBaseColor = LayerSample[0].rgb;
			break;
		}
		case ERuntimeVirtualTextureMaterialType_Mask4:
		{
			OutMask4 = LayerSample[0];
			break;
		}
		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Roughness:
		{
			OutBaseColor = VirtualTextureUnpackBaseColorSRGB(LayerSample[0]);
			OutRoughness = LayerSample[1].y;
			OutNormal = VirtualTextureUnpackNormalBGR565(LayerSample[1]);
			break;
		}
		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular:
		{
			OutBaseColor = LayerSample[0].rgb;
			OutSpecular = LayerSample[1].x;
			OutRoughness = LayerSample[1].y;
			OutNormal = VirtualTextureUnpackNormalBC3BC3(LayerSample[0], LayerSample[1]);
			break;
		}
		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular_YCoCg:
		{
			OutBaseColor = VirtualTextureUnpackBaseColorYCoCg(LayerSample[0]);
			OutSpecular = LayerSample[2].x;
			OutRoughness = LayerSample[2].y;
			OutNormal = VirtualTextureUnpackNormalBC5BC1(LayerSample[1], LayerSample[2]);
			break;
		}
		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular_Mask_YCoCg:
		{
			OutBaseColor = VirtualTextureUnpackBaseColorYCoCg(LayerSample[0]);
			OutSpecular = LayerSample[2].x;
			OutRoughness = LayerSample[2].y;
			OutNormal = VirtualTextureUnpackNormalBC5BC1(LayerSample[1], LayerSample[2]);
			OutMask = LayerSample[2].w;
			break;
		}
		case ERuntimeVirtualTextureMaterialType_WorldHeight:
		{
			const float2 UnpackScaleBias = WorldToUVTransform[3].xy;
			OutWorldHeight = VirtualTextureUnpackHeight(LayerSample[0], UnpackScaleBias);
			break;
		}
		case ERuntimeVirtualTextureMaterialType_Displacement:
		{
			OutDisplacement = LayerSample[0].x;
			break;
		}
	}
}

#undef PCG_CONDITIONAL_VIRTUAL_TEXTURE_SAMPLE
