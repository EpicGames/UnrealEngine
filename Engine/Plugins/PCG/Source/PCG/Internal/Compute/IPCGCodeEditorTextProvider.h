// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IPCGCodeEditorTextProvider.generated.h"

UINTERFACE(MinimalAPI)
class UPCGCodeEditorTextProvider : public UInterface
{
	GENERATED_BODY()
};

/**
* Interface for objects that provide source code to the code editor widget.
*/
class IPCGCodeEditorTextProvider
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	/** Get text for declarations pane. */
	virtual FString GetDeclarationsText() const = 0;

	/** Get text for functions pane. */
	virtual FString GetFunctionsText() const = 0;

	/** Get text for source pane. */
	virtual FString GetSourceText() const = 0;

	/** Set functions text after edit modifications. */
	virtual void SetFunctionsText(const FString& InText) = 0;

	/** Set source text after edit modifications. */
	virtual void SetSourceText(const FString& InText) = 0;

	/** Whether the text can be edited. */
	virtual bool IsReadOnly() const = 0;
#endif
};
