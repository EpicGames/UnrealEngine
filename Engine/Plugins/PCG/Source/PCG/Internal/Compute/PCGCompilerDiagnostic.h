// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** [EXPERIMENTAL] Compiler diagnostic severity.
* Note: This struct is subject to change without deprecation.
*/
enum class EPCGDiagnosticLevel : uint8
{
	None,
	Info,
	Warning,
	Error
};

/** [EXPERIMENTAL] Diagnostic message emitted by shader compiler.
* Note: This struct is subject to change without deprecation.
*/
struct FPCGCompilerDiagnostic
{
	FPCGCompilerDiagnostic() = default;
	FPCGCompilerDiagnostic(const FPCGCompilerDiagnostic&) = default;
	FPCGCompilerDiagnostic& operator=(const FPCGCompilerDiagnostic&) = default;

	// The severity of the issue.
	EPCGDiagnosticLevel Level = EPCGDiagnosticLevel::None;

	// The actual diagnostic message.
	FText Message;

	// Line location in source.
	int32 Line = INDEX_NONE;

	// Starting column (inclusive).
	int32 ColumnStart = INDEX_NONE;

	// Ending column (inclusive).
	int32 ColumnEnd = INDEX_NONE;
};

/** [EXPERIMENTAL] Convenience struct to wrap up array of diagnostics.
 * Note: This struct is subject to change without deprecation.
 */
struct FPCGCompilerDiagnostics
{
	TArray<FPCGCompilerDiagnostic> Diagnostics;
};
