// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Editor/PCGSyntaxTokenizerParams.h"

#include "Text/HLSLSyntaxTokenizer.h"

/**
 * Tokenize the text based on HLSL keywords and operators. Also considers PCG specific keywords as tokens if provided through AddAdditionalKeywords().
 */
class FPCGHLSLSyntaxTokenizer : public FHlslSyntaxTokenizer
{
public:
	PCG_API FPCGHLSLSyntaxTokenizer(const FPCGSyntaxTokenizerParams& InParams = {});
};

#endif // WITH_EDITOR
