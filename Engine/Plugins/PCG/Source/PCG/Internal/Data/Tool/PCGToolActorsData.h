// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/Tool/PCGToolBaseData.h"

#include "PCGToolActorsData.generated.h"


/** Tool Data that stores actors. */
USTRUCT(BlueprintType, DisplayName="Actors Tool Data")
struct FPCGInteractiveToolWorkingData_Actors : public FPCGInteractiveToolWorkingData
{
	GENERATED_BODY()
	
public:
	FPCGInteractiveToolWorkingData_Actors() = default;

	PCG_API virtual void InitializeRuntimeElementData(FPCGContext* InContext) const override;

	PCG_API virtual bool IsValid() const override;
	
	PCG_API UPCGParamData* GetParamData() const;

#if WITH_EDITORONLY_DATA
	PCG_API virtual void InitializeInternal(const FPCGInteractiveToolWorkingDataContext& Context) override;
#endif
	
protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Working Data")
	TObjectPtr<UPCGParamData> SelectedActorsData;

};
