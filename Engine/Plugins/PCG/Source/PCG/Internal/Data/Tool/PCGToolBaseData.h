// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGToolData.h"

#include "PCGToolBaseData.generated.h"

class UInteractiveTool;
class UInteractiveToolPropertySet;
struct FPCGContext;

/** [EXPERIMENTAL]
* A transient struct that contains resources generated during tool usage for a specific tool data struct. Used to finalize or clean up when a tool is applied or cancelled.
*/
USTRUCT()
struct FPCGInteractiveToolWorkingDataGeneratedResources
{
	GENERATED_BODY()

	FPCGInteractiveToolWorkingDataGeneratedResources() = default;
	virtual ~FPCGInteractiveToolWorkingDataGeneratedResources() = default;

#if WITH_EDITORONLY_DATA
	/** Finalizes generated resources (e.g. remove Transient flag). */
	void Finalize();

	/** Remove and destroy generated resources. */
	void Cleanup();

	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<UActorComponent>> GeneratedComponents;
#endif
};

/** [EXPERIMENTAL]
* A transient context struct that is passed in so the tool data can manage its data properly during tool usage. Used at editor-time only.
*/
USTRUCT()
struct FPCGInteractiveToolWorkingDataContext
{
	GENERATED_BODY()

	FPCGInteractiveToolWorkingDataContext() = default;
	virtual ~FPCGInteractiveToolWorkingDataContext() = default;

#if WITH_EDITORONLY_DATA
	/** The owning PCG Component. Should always be valid. Can be used as an Outer for new objects. */
	UPROPERTY(Transient)
	TWeakObjectPtr<UPCGComponent> OwningPCGComponent;

	/** An optional DataInstanceIdentifier. Can be used to find or create a specific object, such as a Spline Component with a certain tag.
	 *  Represents a Data Instance of a tool within a PCG graph.*/
	UPROPERTY(Transient)
	FName DataInstanceIdentifier;

	/** An optional WorkingDataIdentifier. Takes shape of {ToolTag}.{DataInstanceIdentifier}. Needs to be unique within component storage. */
	UPROPERTY(Transient)
	FName WorkingDataIdentifier;

	/** The owning actor. The owning PCG component's outer. Should always be valid. Convenience accessor. */
	UPROPERTY(Transient)
	TWeakObjectPtr<AActor> OwningActor;

	/** The interactive tool currently editing the tool data. */
	UPROPERTY(Transient)
	TWeakObjectPtr<UInteractiveTool> InteractiveTool;

	/** The PCG Setting properties of the Interactive Tool. */
	UPROPERTY(Transient)
	TWeakObjectPtr<UInteractiveToolPropertySet> PCGSettings;
#endif
};

/** [EXPERIMENTAL]
* The base struct for working data for a tool. This data is created by the tool, stored on the PCG Component,
* then read back by the tool on subsequent tool invocations.
*/
USTRUCT()
struct FPCGInteractiveToolWorkingData : public FPCGInteractiveToolWorkingBaseData
{
	GENERATED_BODY()

	PCG_API bool IsInitialized() const;
	
	PCG_API virtual bool IsValid() const;

	/** Construct and add pcg data to the runtime context.
	 * This will translate data from this struct into UPCGData to be processed by the graph.
	 * Add dynamic tracking keys in here if you want to refresh when a change to your source data happens. */
	virtual void InitializeRuntimeElementData(FPCGContext* InContext) const;

#if WITH_EDITORONLY_DATA
public:
	/** Initializes data used by the tool working data. Initialize is called once when the working data is created. */
	PCG_API void Initialize(const FPCGInteractiveToolWorkingDataContext& Context);

	/** OnToolStart is called every time the tool is started up, or when the data is created.
	 * Can be used to save the initial state of persistent data to restore to on Cancel, or to Modify existing data. */
	PCG_API virtual void OnToolStart(const FPCGInteractiveToolWorkingDataContext& Context);

	/** Called when the tool is applied. Finalize changes here. */
	PCG_API virtual void OnToolApply(const FPCGInteractiveToolWorkingDataContext& Context);

	/** Called when the tool is canceled. Revert changes you have made here. */
	PCG_API virtual void OnToolCancel(const FPCGInteractiveToolWorkingDataContext& Context);

	/** Called when reset data was specifically requested, for example by the user. */
	virtual void OnResetToolDataRequested(const FPCGInteractiveToolWorkingDataContext& Context) {}

private:
	/** Implement your initialization logic here. Called by Initialize. */
	virtual void InitializeInternal(const FPCGInteractiveToolWorkingDataContext& Context);

protected:
	/** Cleanup call that will be done at the end of the OnToolApply and OnToolCancel calls. */
	virtual void OnToolShutdown(const FPCGInteractiveToolWorkingDataContext& Context) {}

	/** Each working data struct keeps track of its generated resources. They have to be reverted during OnToolCancel. */
	UPROPERTY(Transient)
	FPCGInteractiveToolWorkingDataGeneratedResources GeneratedResources;
#endif
};
