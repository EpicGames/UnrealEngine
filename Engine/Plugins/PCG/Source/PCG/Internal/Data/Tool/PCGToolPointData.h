// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/Tool/PCGToolBaseData.h"

#include "Components/BoxComponent.h"

#include "PCGToolPointData.generated.h"

/** [EXPERIMENTAL]
* Tool Data that stores points.
*/
USTRUCT(DisplayName = "Point Array Tool Data")
struct FPCGInteractiveToolWorkingData_PointArrayData : public FPCGInteractiveToolWorkingData
{
	GENERATED_BODY()

public:
	FPCGInteractiveToolWorkingData_PointArrayData() = default;

	PCG_API virtual bool IsValid() const override;
	virtual void InitializeRuntimeElementData(FPCGContext* InContext) const override;

	PCG_API UPCGPointArrayData* GetPointArrayData() const;

#if WITH_EDITOR
	PCG_API virtual void InitializeInternal(const FPCGInteractiveToolWorkingDataContext& Context) override;
	PCG_API virtual void OnToolStart(const FPCGInteractiveToolWorkingDataContext& Context) override;
	PCG_API virtual void OnToolApply(const FPCGInteractiveToolWorkingDataContext& Context) override;
	PCG_API virtual void OnToolCancel(const FPCGInteractiveToolWorkingDataContext& Context) override;
	PCG_API virtual void OnResetToolDataRequested(const FPCGInteractiveToolWorkingDataContext& Context) override;
protected:
	PCG_API virtual void OnToolShutdown(const FPCGInteractiveToolWorkingDataContext& Context) override;
#endif // WITH_EDITOR

protected:
	/** Contents of the tool data. */
	UPROPERTY(VisibleAnywhere, Category = "Working Data")
	TObjectPtr<UPCGPointArrayData> GeneratedPointData;

#if WITH_EDITORONLY_DATA
	/** A transient copy of the point array data, created on tool start/data access. We revert back to these points on Cancel. */
	UPROPERTY(Transient)
	TObjectPtr<UPCGPointArrayData> OnToolStartPointArray;

	/** Box component optionally created when the actor doesn't have valid bounds. */
	UPROPERTY(VisibleAnywhere, Category = "Working Data")
	TSoftObjectPtr<UBoxComponent> BoxComponent;
#endif // WITH_EDITORONLY_DATA
};