// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGSplineData.h"
#include "Data/Tool/PCGToolBaseData.h"

#include "PCGToolSplineData.generated.h"

/** [EXPERIMENTAL]
* Tool-created working data for spline interaction.
*/
USTRUCT(DisplayName = "Spline Tool Data")
struct FPCGInteractiveToolWorkingData_Spline : public FPCGInteractiveToolWorkingData
{
	GENERATED_BODY()
public:
	FPCGInteractiveToolWorkingData_Spline() = default;
	virtual ~FPCGInteractiveToolWorkingData_Spline() override = default;

	PCG_API virtual bool IsValid() const override;
	virtual void InitializeRuntimeElementData(FPCGContext* InContext) const override;

	PCG_API USplineComponent* GetSplineComponent() const;

#if WITH_EDITOR
public:
	PCG_API virtual void OnToolStart(const FPCGInteractiveToolWorkingDataContext& Context) override;
	PCG_API virtual void OnToolCancel(const FPCGInteractiveToolWorkingDataContext& Context) override;
	PCG_API virtual void OnResetToolDataRequested(const FPCGInteractiveToolWorkingDataContext& Context) override;
protected:
	PCG_API virtual void OnToolShutdown(const FPCGInteractiveToolWorkingDataContext& Context) override;

private:
	virtual void InitializeInternal(const FPCGInteractiveToolWorkingDataContext& Context) override;

private:
	USplineComponent* FindOrGenerateSplineComponent(const FPCGInteractiveToolWorkingDataContext& Context);

	virtual USplineComponent* FindMatchingSplineComponent(const FPCGInteractiveToolWorkingDataContext& Context);
	virtual USplineComponent* GenerateMatchingSplineComponent(const FPCGInteractiveToolWorkingDataContext& Context);
#endif

protected:
	/** Spline this working data interacts with. */
	UPROPERTY(VisibleAnywhere, Category = "Working Data")
	TSoftObjectPtr<USplineComponent> SplineComponent;

#if WITH_EDITORONLY_DATA
	/** Copy of the spline at the beginning of the interaction, for rollback purposes. */
	UPROPERTY(Transient)
	TObjectPtr<USplineComponent> OnToolStartSplineComponent;

	/** Transform of the spline at the beginning of the interaction, for rollback purposes. */
	UPROPERTY(Transient)
	FTransform OnToolStartSplineTransform;
#endif // WITH_EDITORONLY_DATA
};

USTRUCT(DisplayName = "Spline Surface Tool Data")
struct FPCGInteractiveToolWorkingData_SplineSurface : public FPCGInteractiveToolWorkingData_Spline
{
	GENERATED_BODY()

	FPCGInteractiveToolWorkingData_SplineSurface() = default;
	virtual ~FPCGInteractiveToolWorkingData_SplineSurface() override = default;

#if WITH_EDITOR
private:
	/** We look for and generate a closed spline component. */
	virtual USplineComponent* FindMatchingSplineComponent(const FPCGInteractiveToolWorkingDataContext& Context) override;
	virtual USplineComponent* GenerateMatchingSplineComponent(const FPCGInteractiveToolWorkingDataContext& Context) override;
#endif
};