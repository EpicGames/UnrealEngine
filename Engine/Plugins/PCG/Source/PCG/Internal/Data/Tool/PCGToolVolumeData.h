// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/Tool/PCGToolBaseData.h"

#include "PCGToolVolumeData.generated.h"

class AVolume;
class UBoxComponent;
class UBrushComponent;
class UBrushBuilder;

/** [EXPERIMENTAL] 
* Data to hold interaction data with volumes.
*/
USTRUCT(DisplayName = "Volume Tool Data")
struct FPCGInteractiveToolWorkingData_Volume : public FPCGInteractiveToolWorkingData
{
	GENERATED_BODY()
public:
	/** Volume actor this working data is affecting. */
	UPROPERTY(VisibleAnywhere, Category = "Working Data")
	TObjectPtr<AVolume> Volume = nullptr;

	/** Brush component this working data is affecting. */
	UPROPERTY(VisibleAnywhere, Category = "Working Data")
	TSoftObjectPtr<UBrushComponent> BrushComponent = nullptr;

	/** Box component this working data is affecting. */
	UPROPERTY(VisibleAnywhere, Category = "Working Data")
	TSoftObjectPtr<UBoxComponent> BoxComponent = nullptr;

	//~Begin FPCGInteractiveToolWorkingData interface
public:
	virtual bool IsValid() const override;
	virtual void InitializeRuntimeElementData(FPCGContext* InContext) const override;

#if WITH_EDITOR
public:
	PCG_API virtual void OnToolStart(const FPCGInteractiveToolWorkingDataContext& Context) override;
	PCG_API virtual void OnToolCancel(const FPCGInteractiveToolWorkingDataContext& Context) override;
protected:
	PCG_API virtual void OnToolShutdown(const FPCGInteractiveToolWorkingDataContext& Context) override;

private:
	virtual void InitializeInternal(const FPCGInteractiveToolWorkingDataContext& Context) override;
#endif //WITH_EDITOR
	//~End FPCGInteractiveToolWorkingData interface

#if WITH_EDITOR
	UBrushComponent* FindOrCreateBrushComponent(const FPCGInteractiveToolWorkingDataContext& Context);
	virtual UBrushComponent* FindMatchingBrushComponent(const FPCGInteractiveToolWorkingDataContext& Context);
	virtual UBrushComponent* GenerateMatchingBrushComponent(const FPCGInteractiveToolWorkingDataContext& Context);
	UBoxComponent* FindBoxComponent(const FPCGInteractiveToolWorkingDataContext& Context);
#endif

protected:
#if WITH_EDITORONLY_DATA
	/** Copy of the original brush component on tool initialization, used for rollback purposes. */
	UPROPERTY(Transient)
	TObjectPtr<UBrushComponent> OnToolStartBrushComponent = nullptr;

	/** Copy of the original brush builder on tool initialization, used for rollback purposes. */
	UPROPERTY(Transient)
	TObjectPtr<UBrushBuilder> OnToolStartBrushBuilder = nullptr;

	/** Original transform of the actor (volume or otherwise), used for rollback. */
	UPROPERTY(Transient)
	FTransform OnToolStartActorTransform;

	/** Copy of the original box component on tool initialization, used for rollback purposes. */
	UPROPERTY(Transient)
	TObjectPtr<UBoxComponent> OnToolStartBoxComponent = nullptr;

	/** Original transform of the box component, used for rollback. */
	UPROPERTY(Transient)
	FTransform OnToolStartBoxTransform;
#endif
};