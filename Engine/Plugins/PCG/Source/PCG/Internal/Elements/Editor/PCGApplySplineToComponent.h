// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGApplySplineToComponent.generated.h"

namespace PCGApplySplineToComponentConstants
{
	const FName ComponentPinLabel = TEXT("Component");
	const FName SplinePinLabel    = TEXT("Spline");
}

/**
 * Applies PCG spline data to existing spline component(s) referenced by a component reference path in an attribute set.
 * Intended for use in level asset graphs only.
 *
 * Supports 1:N and N:N mappings: if a single (1) attribute set is used, the number of entries should be equal to the number of spline data (N).
 * Otherwise, this assumes that there is a single entry per attribute set (N) matching the number of input spline data (N).
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGApplySplineToComponentSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGApplySplineToComponentSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ApplySplineToComponent")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGApplySplineToComponent", "NodeTitle", "Apply Spline To Component"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGApplySplineToComponent", "NodeTooltip", "Applies PCG spline data to an existing spline component."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Attribute on the Component param data that holds the soft object path to each target USplineComponent. Defaults to "ComponentReference" (the standard output attribute of Spawn Spline Component). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector ComponentReferenceAttribute;

	/** Silence warnings when a component path is empty or cannot be resolved. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bSilenceWarningOnUnresolvedPath = false;
};

class FPCGApplySplineToComponentElement : public IPCGElement
{
public:
	// Components must be mutated on the game thread, and changes like these are never cacheable
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
