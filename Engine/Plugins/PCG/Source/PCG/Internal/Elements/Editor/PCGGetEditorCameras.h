// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGGetEditorCameras.generated.h"

/**
 * Produces one point per editor viewport camera, with the camera world transform as the point transform. 
 * Perspective viewports appear first, followed by orthographic viewports. Each point carries the following attributes:
 *   IsPerspective        (bool)   - true for perspective viewports
 *   ViewportType         (string) - "Perspective", "Top", "Bottom", "Front", "Back", "Left", "Right", "OrthoFreelook" or "Unknown"
 *   IsActive             (bool)   - true for the last active camera
 *   FOV                  (float)  - horizontal field of view in degrees (perspective only; 0 otherwise)
 *   OrthoZoom            (float)  - orthographic zoom amount (orthographic only; 0 otherwise)
 *   NearClipPlane        (float)  - near clipping plane distance
 *   FarClipPlaneOverride (float)  - far clipping plane override; negative if not overridden (engine default applies)
 *
 * Note: returns no data in non-editor (cooked) builds.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetEditorCamerasSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetEditorCameras")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return {}; }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGGetEditorCamerasElement : public IPCGElement
{
public:
	// Editor viewport clients must be queried from the main thread.
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	// Viewport transforms change every frame; never cache results.
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
