// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGGetEditorSelection.generated.h"

/**
 * Produces a single point per editor-selected actor, with the actor transform and bounds.
 * Mirrors the GetSinglePoint mode of the Get Actor Data node but sources actors from the
 * current editor viewport selection instead of the actor selector.
 *
 * Note: returns no data in non-editor (cooked) builds.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetEditorSelectionSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetEditorSelection")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetEditorSelectionElement", "NodeTitle", "Get Editor Selection"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGGetEditorSelectionElement", "NodeTooltip", "Produces a single point per editor-selected actor, carrying the actor transform and bounds."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return {}; }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return UPCGSettings::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Silence warnings that attribute names were sanitized to replace invalid characters. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bSilenceSanitizedAttributeNameWarnings = false;
};

class FPCGGetEditorSelectionElement : public IPCGElement
{
public:
	// Editor selection must be queried from the main thread.
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	// Selection changes constantly; never cache results.
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
