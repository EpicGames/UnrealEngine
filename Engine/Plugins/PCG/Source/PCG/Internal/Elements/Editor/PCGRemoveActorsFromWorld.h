// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGRemoveActorsFromWorld.generated.h"

/**
 * Removes actors from the world based on actor references stored in point data or param data.
 * The actor reference attribute defaults to the standard PCG actor reference attribute used by
 * data assets, spawn actor nodes, etc.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class PCG_API UPCGRemoveActorsFromWorldSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGRemoveActorsFromWorldSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("RemoveActorsFromWorld")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif // WITH_EDITOR
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface

protected:
	virtual FPCGElementPtr CreateElement() const override;

public:
	/** Attribute to read actor references from. Accepts both point data and param data.
	 *  Defaults to the standard PCG actor reference attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector ActorReferenceAttribute;

	/** Silence warnings logged when a referenced actor cannot be found in the world.
	 *  An actor may be absent because it was already deleted, is not loaded, or the reference is stale. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	bool bSilenceActorNotFoundWarnings = false;
};

class PCG_API FPCGRemoveActorsFromWorldElement : public IPCGElement
{
public:
	// Interacting with actors in the world must be done on the main thread.
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	// This node has world side-effects and must not be cached.
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
