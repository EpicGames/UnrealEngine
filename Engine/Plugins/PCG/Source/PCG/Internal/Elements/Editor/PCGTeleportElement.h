// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGTeleportElement.generated.h"

/**
 * Teleports actors and scene components to the transform specified per input point.
 * Reads a soft object path attribute to resolve each target object and a transform
 * attribute (defaulting to $Transform) for the destination.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGTeleportSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGTeleportSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("TeleportActorsAndComponents")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGTeleportElement", "NodeTitle", "Teleport Actors And Components"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGTeleportElement", "NodeTooltip", "Teleports actors and scene components to the transform specified per input point."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Attribute containing soft object paths to the actors or scene components to teleport. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector ObjectReferenceAttribute;

	/** Attribute containing the target world transform. Defaults to $Transform. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector TransformAttribute;

	/** Silence warnings when an object path is empty or cannot be resolved. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bSilenceWarningOnUnresolvedPath = false;
};

class FPCGTeleportElement : public IPCGElement
{
public:
	// Actors and components must be moved on the main thread.
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	// This node has world side-effects and must not be cached.
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
