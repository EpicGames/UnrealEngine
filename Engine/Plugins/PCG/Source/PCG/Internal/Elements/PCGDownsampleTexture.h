// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "PCGTextureDownsample.h"

#include "PCGDownsampleTexture.generated.h"

class UPCGTextureData;
class UPCGTexture2DArrayData;
class UPCGTexture2DBaseData;

UENUM()
enum class EPCGDownsampleTextureType : uint8
{
	Texture2D,
	Texture2DArray,
};

/** Downsample a texture and populate its mip chain. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDownsampleTextureSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return TEXT("DownsampleTexture"); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif
	virtual bool HasDynamicPins() const override { return true; }

	virtual bool PadResolutionToSquare() const { return bPadResolutionToSquare; }
	virtual bool PadResolutionToNextPowerOfTwo() const { return bPadResolutionToNextPowerOfTwo; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

public:
	EPCGTextureDownsampleMode GetMode() const { return Mode; }

protected:
	UPROPERTY(EditAnywhere, Category = Settings)
	EPCGTextureDownsampleMode Mode = EPCGTextureDownsampleMode::Average;

	UPROPERTY(EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bPadResolutionToSquare = false;

	UPROPERTY(EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bPadResolutionToNextPowerOfTwo = false;
};

struct FPCGDownsampleTextureContext : public FPCGContext
{
public:
	bool bSubmittedRenderCommands = false;
	bool bCreatedOutputData = false;

	int32 NumSubmittedRenderCommands = 0;
	int32 NumCompletedRenderCommands = 0;

	struct FTextureOutputInfo
	{
		TRefCountPtr<IPooledRenderTarget> OutputHandle;
		FTransform Transform;
		EPCGTextureFilter Filter = EPCGTextureFilter::Bilinear;
		int32 ArraySize = 1;
		bool bIsArray = false;
		UPCGTexture2DBaseData* OutputData = nullptr;
	};

	TArray<FTextureOutputInfo> TextureOutputInfo;
};

class FPCGDownsampleTextureElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const { return true; }
	virtual FPCGContext* CreateContext() override { return new FPCGDownsampleTextureContext(); }
};
