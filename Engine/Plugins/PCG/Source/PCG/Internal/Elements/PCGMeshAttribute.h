// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/TextureRenderTarget2D.h"

#include "PCGMeshAttribute.generated.h"

/** Mesh attribute to bake into the output texture. Mirrors PCGUnwrapMesh::EMeshAttribute. */
UENUM(BlueprintType)
enum class EPCGMeshAttribute : uint8
{
	/** RGB = local-space vertex position interpolated to the texel; alpha = 1 marks "covered". */
	LocalPosition = 0,

	/** Coverage mask. Writes 1 on every channel so the mask is readable from any RT format (R8, R16F, RGBA, ...). */
	Mask = 1,
};

/** Settings shared by mesh-attribute bake nodes. */
USTRUCT(BlueprintType)
struct FPCGBakeMeshAttributesParams
{
	GENERATED_BODY()

	/** Output render target resolution in texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = "1", ClampMax = "8192"))
	FIntPoint Resolution = FIntPoint(256, 256);

	/** Output render target pixel format. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TEnumAsByte<ETextureRenderTargetFormat> Format = RTF_RGBA16f;

	/** Which per-texel attribute the pixel shader writes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGMeshAttribute Attribute = EPCGMeshAttribute::LocalPosition;

	/** Number of texels of edge padding applied to UV islands after the bake. Does not apply to the Mask attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = "0", ClampMax = "16", EditCondition = "Attribute != EPCGMeshAttribute::Mask", EditConditionHides))
	int32 Padding = 1;

	/** Mesh UV channel to rasterize into the output texture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, DisplayName = "UV Channel Index", meta = (PCG_Overridable, ClampMin = "0", ClampMax = "7"))
	int32 UVChannelIndex = 0;
};
