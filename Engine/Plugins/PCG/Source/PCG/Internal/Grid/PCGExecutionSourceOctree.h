// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/BoxSphereBounds.h"
#include "Math/GenericOctree.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "UObject/ObjectPtr.h"
#include "Templates/SharedPointer.h"
#include "Templates/SharedPointerFwd.h"

template <typename ElementType, typename OctreeSemantics> class TOctree2;

class IPCGGraphExecutionSource;

struct FPCGExecutionSourceOctreeID : public TSharedFromThis<FPCGExecutionSourceOctreeID, ESPMode::ThreadSafe>
{
	FOctreeElementId2 Id;
};

using FPCGExecutionSourceOctreeIDSharedRef = TSharedRef<struct FPCGExecutionSourceOctreeID, ESPMode::ThreadSafe>;

struct FPCGExecutionSourceRef
{
	FPCGExecutionSourceRef(IPCGGraphExecutionSource* InExecutionSource, const FPCGExecutionSourceOctreeIDSharedRef& InIdShared);

	void UpdateBounds();

	FPCGExecutionSourceOctreeIDSharedRef IdShared;
	IPCGGraphExecutionSource* ExecutionSource = nullptr;
	FBoxSphereBounds Bounds;
};

struct FPCGExecutionSourceRefSemantics
{
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	inline static const FBoxSphereBounds& GetBoundingBox(const FPCGExecutionSourceRef& InVolume)
	{
		return InVolume.Bounds;
	}

	inline static const bool AreElementsEqual(const FPCGExecutionSourceRef& A, const FPCGExecutionSourceRef& B)
	{
		return A.ExecutionSource == B.ExecutionSource;
	}

	inline static void ApplyOffset(FPCGExecutionSourceRef& InVolume, const FVector& Offset)
	{
		InVolume.Bounds.Origin += Offset;
	}

	inline static void SetElementId(const FPCGExecutionSourceRef& Element, FOctreeElementId2 OctreeElementID)
	{
		Element.IdShared->Id = OctreeElementID;
	}
};

using FPCGExecutionSourceOctree = TOctree2<FPCGExecutionSourceRef, FPCGExecutionSourceRefSemantics> ;
using FPCGExecutionSourceToIdMap = TMap<IPCGGraphExecutionSource*, FPCGExecutionSourceOctreeIDSharedRef>;

class FPCGExecutionSourceOctreeAndMap
{
public:
	FPCGExecutionSourceOctreeAndMap() = default;
	FPCGExecutionSourceOctreeAndMap(const FVector& InOrigin, FVector::FReal InExtent);

	void Reset(const FVector& InOrigin, FVector::FReal InExtent);

	TSet<IPCGGraphExecutionSource*> GetAllExecutionSources() const;

	template<typename IterateBoundsFunc>
	inline void FindElementsWithBoundsTest(const FBoxCenterAndExtent& BoxBounds, const IterateBoundsFunc& Func) const
	{
		UE::TReadScopeLock ReadLock(Lock);
		return Octree.FindElementsWithBoundsTest(BoxBounds, Func);
	}

	bool Contains(const IPCGGraphExecutionSource* InExecutionSource) const;
	FBox GetBounds(const IPCGGraphExecutionSource* InExecutionSource) const;

	void AddOrUpdateExecutionSource(IPCGGraphExecutionSource* InExecutionSource, FBox& OutBounds, bool& bOutExecutionSourceHasChanged, bool& bOutExecutionSourcetWasAdded);
	bool RemapExecutionSource(const IPCGGraphExecutionSource* InOldExecutionSource, IPCGGraphExecutionSource* InNewExecutionSource, bool& bOutBoundsHasChanged);
	bool RemoveExecutionSource(IPCGGraphExecutionSource* InExecutionSource);

private:
	FPCGExecutionSourceOctree Octree;
	FPCGExecutionSourceToIdMap ExecutionSourceToIdMap;
	mutable FTransactionallySafeRWLock Lock;
};