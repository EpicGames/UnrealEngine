// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

namespace PCGLandscapeCVars
{
	PCG_API extern TAutoConsoleVariable<int> CVarLandscapeRefreshTimeDelay;
	PCG_API extern TAutoConsoleVariable<bool> CVarLandscapeDisableRefreshTracking;
	PCG_API extern TAutoConsoleVariable<bool> CVarLandscapeDisableRefreshTrackingInLandscapeEditingMode;
	PCG_API extern TAutoConsoleVariable<bool> CVarLandscapeForceRefreshRuntimeGen;
}