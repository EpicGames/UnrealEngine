// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/PCGPropertyHelpers.h"
#include "Metadata/Accessors/PCGPropertyAccessor.h"

#include "UObject/UnrealType.h"

namespace PCGAttributeAccessorHelpers
{
namespace Internal
{
	// Empty signature to passthrough types to functors.
	template <typename T>
	struct Signature
	{
		using Type = T;
	};

	template <typename Func>
	decltype(auto) DispatchPropertyTypes(const FProperty* InProperty, Func&& Functor)
	{
		// Use the FPCGPropertyPathAccessor (on soft object path) as dummy type to get the functor return type
		// because this accessor takes a generic FProperty, while others take a more specialized type (like FNumericProperty).
		using ReturnType = decltype(Functor(Signature<FPCGPropertyPathAccessor<FSoftObjectPath>>{}, InProperty));

		if (!InProperty)
		{
			return ReturnType{};
		}

		if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
		{
			if (NumericProperty->IsFloatingPoint())
			{
				// As floating properties are mostly all double in UE, convert float to double attributes.
				return Functor(Signature<FPCGNumericPropertyAccessor<double>>{}, NumericProperty);
			}
			else if (NumericProperty->IsInteger())
			{
				// But for int32/int64 we can distinguish between the two. Everything of size 32 or less is will be an int32, 64bits integers will be int64.
				if (NumericProperty->IsA<FInt64Property>() || NumericProperty->IsA<FUInt64Property>())
				{
					return Functor(Signature<FPCGNumericPropertyAccessor<int64>>{}, NumericProperty);
				}
				else
				{
					return Functor(Signature<FPCGNumericPropertyAccessor<int32>>{}, NumericProperty);
				}
			}
		}
		else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
		{
			return Functor(Signature<FPCGPropertyAccessor<bool, FBoolProperty>>{}, BoolProperty);
		}
		else if (const FStrProperty* StringProperty = CastField<FStrProperty>(InProperty))
		{
			return Functor(Signature<FPCGPropertyAccessor<FString, FStrProperty>>{}, StringProperty);
		}
		else if (const FNameProperty* NameProperty = CastField<FNameProperty>(InProperty))
		{
			return Functor(Signature<FPCGPropertyAccessor<FName, FNameProperty>>{}, NameProperty);
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
		{
			return Functor(Signature<FPCGEnumPropertyAccessor>{}, EnumProperty);
		}
		else if (const FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(InProperty))
		{
			return Functor(Signature<FPCGPropertySoftClassPathAccessor>{}, SoftClassProperty);
		}
		else if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(InProperty))
		{
			return Functor(Signature<FPCGPropertySoftObjectPathAccessor>{}, SoftObjectProperty);
		}
		else if (const FClassProperty* ClassProperty = CastField<FClassProperty>(InProperty))
		{
			return Functor(Signature<FPCGPropertyObjectPtrAccessor<FClassProperty>>{}, ClassProperty);
		}
		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
		{
			return Functor(Signature<FPCGPropertyObjectPtrAccessor<FObjectProperty>>{}, ObjectProperty);
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			if (StructProperty->Struct == TBaseStructure<FVector>::Get())
			{
				return Functor(Signature<FPCGPropertyStructAccessor<FVector>>{}, StructProperty);
			}
			else if (StructProperty->Struct == TBaseStructure<FVector4>::Get())
			{
				return Functor(Signature<FPCGPropertyStructAccessor<FVector4>>{}, StructProperty);
			}
			else if (StructProperty->Struct == TBaseStructure<FQuat>::Get())
			{
				return Functor(Signature<FPCGPropertyStructAccessor<FQuat>>{}, StructProperty);
			}
			else if (StructProperty->Struct == TBaseStructure<FTransform>::Get())
			{
				return Functor(Signature<FPCGPropertyStructAccessor<FTransform>>{}, StructProperty);
			}
			else if (StructProperty->Struct == TBaseStructure<FRotator>::Get())
			{
				return Functor(Signature<FPCGPropertyStructAccessor<FRotator>>{}, StructProperty);
			}
			else if (StructProperty->Struct == TBaseStructure<FVector2D>::Get())
			{
				return Functor(Signature<FPCGPropertyStructAccessor<FVector2D>>{}, StructProperty);
			}
			else if (StructProperty->Struct == TBaseStructure<FSoftObjectPath>::Get())
			{
				return Functor(Signature<FPCGPropertyStructAccessor<FSoftObjectPath>>{}, StructProperty);
			}
			else if (StructProperty->Struct == TBaseStructure<FSoftClassPath>::Get())
			{
				return Functor(Signature<FPCGPropertyStructAccessor<FSoftClassPath>>{}, StructProperty);
			}
			else if (StructProperty->Struct == TBaseStructure<FLinearColor>::Get())
			{
				return Functor(Signature<FPCGLinearColorAccessor>{}, StructProperty);
			}
			else if (StructProperty->Struct == TBaseStructure<FColor>::Get())
			{
				return Functor(Signature<FPCGColorAccessor>{}, StructProperty);
			}
			else if (StructProperty->Struct && StructProperty->Struct->IsChildOf(FPCGAttributePropertySelector::StaticStruct()))
			{
				return Functor(Signature<FPCGAttributePropertySelectorAccessor>{}, StructProperty);
			}
		}

		return ReturnType{};
	}

	template <typename Func>
	decltype(auto) DispatchPropertyTypes(const FName InPropertyName, const UStruct* InStruct, Func&& Functor)
	{
		if (InStruct)
		{
			if (const FProperty* Property = PCGPropertyHelpers::FindPropertyByName(InStruct, InPropertyName))
			{
				return DispatchPropertyTypes(Property, std::forward<Func>(Functor));
			}
		}

		using ReturnType = decltype(Functor(static_cast<FPCGPropertyPathAccessor<FSoftObjectPath>*>(nullptr), static_cast<const FProperty*>(nullptr)));
		return ReturnType{};
	}
}// namespace Internal
} // namespace PCGAttributeAccessorHelpers
