// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/Accessors/PCGContainerAccessor.h"

#include "Metadata/Accessors/PCGAttributeAccessorHelpersInternal.h"

#define LOCTEXT_NAMESPACE "PCGContainerAccessor"

IPCGArrayWrapperAccessor::IPCGArrayWrapperAccessor(const FArrayProperty* InProperty, TArray<const FProperty*>&& ExtraPropertiesToArray, const FProperty* UnderlyingProperty, TArray<const FProperty*>&& ExtraPropertiesToUnderlying, bool bUseGenericAccessor)
	: IPCGAttributeAccessor(/*bInReadOnly=*/ false, static_cast<int16>(EPCGMetadataTypes::Unknown))
	, IPCGPropertyChain(InProperty, std::move(ExtraPropertiesToArray))
	, Property(InProperty)
{
	check(Property && UnderlyingProperty);
	
	if (ExtraPropertiesToUnderlying.IsEmpty())
	{
		UnderlyingAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(UnderlyingProperty, bUseGenericAccessor);
	}
	else
	{
		if (ExtraPropertiesToUnderlying.Last() != UnderlyingProperty)
		{
			ExtraPropertiesToUnderlying.Add(UnderlyingProperty);
		}
		
		UnderlyingAccessor = PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(MoveTemp(ExtraPropertiesToUnderlying), bUseGenericAccessor);
	}
	
	if (!UnderlyingAccessor)
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidUnderlyingAccessor", "[IPCGArrayWrapperAccessor] Failed to create a property accessor for property {0}."), FText::FromString(UnderlyingProperty->GetName())));
		return;
	}
	
	if (!UnderlyingAccessor->GetUnderlyingDesc().IsSingleValue())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NotSingleValue", "[IPCGArrayWrapperAccessor] Tried to create an accessor with 2 containers in the path. Unsupported."));
		UnderlyingAccessor.Reset();
		return;
	}
	
	// We match the underlying accessor desc, but forcing it to be an array.
	UnderlyingDesc = UnderlyingAccessor->GetUnderlyingDesc();
	UnderlyingType = UnderlyingAccessor->GetUnderlyingType();
	
	UnderlyingDesc.ContainerTypes = {EPCGMetadataAttributeContainerTypes::Array};
}

bool IPCGArrayWrapperAccessor::SetRangeVirtual(PCG::Private::FInValues InValues, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
{
	using namespace PCG::Private;
	
	if (!InValues.IsType<FInValuesAsArray>() || !UnderlyingAccessor)
	{
		return false;
	}
	
	const FInValuesAsArray& InValuesAsArray = InValues.Get<FInValuesAsArray>();
	
	// Since we "copy" InValues into a single array, we can only work on a single key.
	TArray<void*, TInlineAllocator<256>> ContainerKeys;
	PCGPropertyAccessor::GetContainerKeys(0, Count, Keys, ContainerKeys);
	if (ContainerKeys.IsEmpty())
	{
		return false;
	}
	
	// Update the addresses
	PCGPropertyAccessor::AddressOffset<void*>(GetPropertyChain(), ContainerKeys);
	
	TArray<void*> Addresses;

	for (int32 i = 0; i < Count; ++i)
	{
		auto [DataPtr, ArrayNum] = InValuesAsArray.InValues[i];
		
		FScriptArrayHelper ArrayHelper(Property, ContainerKeys[i]);
		ArrayHelper.Resize(ArrayNum);

		// Create subkeys for the forwarding call
		Addresses.SetNumUninitialized(ArrayNum, EAllowShrinking::No);
		for (int32 j = 0; j < ArrayNum; ++j)
		{
			Addresses[j] = ArrayHelper.GetRawPtr(j);
		}

		FPCGAttributeAccessorKeysGenericPtrs ArrayKeys{Addresses};
		if (!UnderlyingAccessor->SetRangeVirtual(FInValues{TInPlaceType<FInValuesByValue>{}, DataPtr, ArrayNum}, /*Count=*/ArrayNum, /*Index=*/0, ArrayKeys, Flags))
		{
			return false;
		}
	}
	
	return true;
}

TUniquePtr<IPCGAttributeAccessor> PCGContainerAccessorHelpers::MakeContainerAccessor(TConstArrayView<const FProperty*> PropertyChain, bool bUseGenericAccessor)
{
	// Find the container property in the chain. Make sure that there is just a single one.
	int32 ContainerPropertyIndex = INDEX_NONE;
	int32 Index = 0;
	const FArrayProperty* ArrayProperty = nullptr;
	for (const FProperty* Property : PropertyChain)
	{
		check(Property);
		if (Property->IsA<FArrayProperty>())
		{
			if (ContainerPropertyIndex != INDEX_NONE)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("InvalidChain", "[PCGContainerAccessorHelpers::MakeContainerAccessor] Tried to create an accessor on a property chain with multiple containers. Unsupported."));
				return nullptr;
			}
				
			ContainerPropertyIndex = Index;
			ArrayProperty = CastField<FArrayProperty>(Property);
		}
			
		++Index;
	}

	if (ContainerPropertyIndex == INDEX_NONE || ContainerPropertyIndex >= PropertyChain.Num() - 1)
	{
		return nullptr;
	}

	check(ArrayProperty);

	TArray<const FProperty*> ParentProperties(PropertyChain.GetData(), ContainerPropertyIndex + 1);
	TArray<const FProperty*> ChildProperties(PropertyChain.GetData() + ContainerPropertyIndex + 1, PropertyChain.Num() - ContainerPropertyIndex - 1);
	
	if (!PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(ChildProperties.Last(), bUseGenericAccessor))
	{
		return nullptr;
	}

	// If there is no more than 1 child property and use bUseGenericAccessor, we can just have a generic accessor. Otherwise we create a custom accessor that will allow to set specific elements in a struct contained in an array.
	if (ChildProperties.Num() <= 1 && bUseGenericAccessor)
	{
		return MakeUnique<FPCGPropertyGenericAccessor>(ArrayProperty, MoveTemp(ParentProperties));
	}
	else
	{
		return MakeUnique<IPCGArrayWrapperAccessor>(ArrayProperty, MoveTemp(ParentProperties), ChildProperties.Last(), MoveTemp(ChildProperties), bUseGenericAccessor);
	}
}

#undef LOCTEXT_NAMESPACE