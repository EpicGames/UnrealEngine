// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGModule.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGPropertyAccessor.h"

#include "UObject/UnrealType.h"

/**
 * DISCLAIMER:
 * Container accessors are a first try and temporary to support containers for accessors.
 * Because of how the Accessor API is working, it will only works to set values in single containers, with no support for multi-containers.
 * It is meant to set the full container in one go, no in-place editing or appending, meaning that Index > 0 will be rejected.
 */
class IPCGArrayWrapperAccessor : public IPCGAttributeAccessor, IPCGPropertyChain
{
public:
	// ExtraProperties are properties to arrive to the ArrayProperty. UnderlyingAccessor is for writing into an element of the array.
	PCG_API IPCGArrayWrapperAccessor(const FArrayProperty* InProperty, TArray<const FProperty*>&& ExtraPropertiesToArray, const FProperty* UnderlyingProperty, TArray<const FProperty*>&& ExtraPropertiesToUnderlying, bool bUseGenericAccessor);
	
protected:
	virtual bool GetRangeVirtual(PCG::Private::FOutValues OutValues, int32 Count, int32 Index, const IPCGAttributeAccessorKeys& Keys) const override
	{
		// Unsupported
		return false;
	}

	PCG_API virtual bool SetRangeVirtual(PCG::Private::FInValues InValues, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) override;

private:
	const FArrayProperty* Property = nullptr;
	TUniquePtr<IPCGAttributeAccessor> UnderlyingAccessor;
};

namespace PCGContainerAccessorHelpers
{
	TUniquePtr<IPCGAttributeAccessor> MakeContainerAccessor(TConstArrayView<const FProperty*> PropertyChain, bool bUseGenericAccessor = false);
}