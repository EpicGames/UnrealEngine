// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"

#include "GameFramework/Actor.h"

#include "PCGIsolatedActor.generated.h"

/** 
* Simnple actor wrapper class to store PCG generated data.
*/
UCLASS(MinimalAPI, NotPlaceable)
class APCGIsolatedActor : public AActor
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleInstanceOnly, Category = PCG)
	FPCGDataCollection Data;

	UPROPERTY(VisibleInstanceOnly, Category = PCG)
	TSoftObjectPtr<AActor> Origin = nullptr;
};