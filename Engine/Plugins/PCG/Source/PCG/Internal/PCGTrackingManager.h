// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "ChangeTracking/PCGChangeTrackingRegistry.h"
#include "Elements/PCGActorSelector.h"
#include "Grid/PCGGridDescriptor.h"
#include "Grid/PCGExecutionSourceOctree.h"
#include "RuntimeGen/PCGRuntimeGenScheduler.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Templates/Function.h"
#include "UObject/ObjectKey.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class ALandscapeProxy;
class APCGPartitionActor;
class FLandscapeProxyComponentDataChangedParams;
class ILevelInstanceInterface;
class UObject;
class UPCGComponent;
class UPCGGraph;
class UPCGSubsystem;
class UWorld;
class UWorldPartition;
class FWorldPartitionActorDescInstance;
class FPackageReloadedEvent;

enum class EPackageReloadPhase : uint8;

#if WITH_EDITOR
namespace PCGTrackingManager
{
	static const FPCGTrackingChangeID ObjectChanged(0x8C0A272F, 0xF1854FD0, 0xB821301B, 0x99D38D8D);
	static const FPCGTrackingChangeID ExecutionSourceGenerated(0xA73B5462, 0x3CB24DB2, 0x9E9CADFE, 0xEE346EB5);
}
#endif

/**
* This class currently serves two purposes:
* - Keep a mapping between original and local execution sources.
* - Do change tracking to refresh execution sources.
*
* Its meant to be part of the PCG Subsystem and owned by it. We offload some logic to this class to avoid to clutter the subsystem.
*/
class FPCGTrackingManager
{
public:
	friend UPCGSubsystem;
	friend FPCGRuntimeGenScheduler;

	~FPCGTrackingManager() = default;

	/** Initializes callbacks, etc, tied to the PCG subsystem */
	void Initialize();

	/** Deinitializes callbacks, etc, tied to the PCG subsystem */
	void Deinitialize();
	
	/** Should be called by the subsystem to handle delayed operations. */
	void Tick();

	/** Returns the PCGSubystem's World */
	PCG_API UWorld* GetWorld() const;

#if WITH_EDITOR
	PCG_API bool IsTracking() const;
	PCG_API bool IsKeyTracked(const FPCGSelectionKey& InKey) const;
	PCG_API void ForEachTrackedKey(TFunctionRef<bool(const FPCGSelectionKey&, const TSet<IPCGGraphExecutionSource*>&)> InCallback) const;
	/** Gather all settings from a given execution source that track the key, and clear the cache for them. Returns true if we should dirty afterwards (aka at least one settings was cleared and/or landscape changed). */
	PCG_API bool ClearCacheForKeys(TArrayView<FPCGSelectionKey> InKeys, const IPCGGraphExecutionSource* InExecutionSource, const bool bIntersect, const UObject* InOriginatingChange) const;

	PCG_API void OnSelectionKeyChanged(const FPCGSelectionKey& InSelectionKey);
	PCG_API void OnSelectionKeyChanged(const FPCGSelectionKey& InSelectionKey, const UObject* InOriginatingChangeObject, TArrayView<const FBox> InBounds);
	PCG_API void OnSelectionKeyChanged(const FPCGSelectionKey& InSelectionKey, const UObject* InOriginatingChangeObject, const TSet<FName>& InRemovedTags, bool bInNoRefreshOwner, TArrayView<const FBox> InBounds);

	PCG_API void NotifyObjectChanged(const UObject* InObject, const FPCGTrackingChangeID& InChangeID, const UObject* InOriginatingChangeObject = nullptr);

	static void BuildPartitionActorRecords(APCGWorldActor* PCGWorldActor, UWorldPartition* WorldPartition, TMap<FPCGGridCellDescriptor, FGuid>& OutPartitionActorRecords, TSet<FGuid>& OutInvalidPartitionActors);

	/** Returns true if originating execution source is null or if dependent execution source has no more dependencies */
	PCG_API bool RemoveExecutionSourceDependency(const IPCGGraphExecutionSource* InOriginatingExecutionSource, const IPCGGraphExecutionSource* InDependentExecutionSource);
#endif // WITH_EDITOR

	/** Iterate other all the int coordinates given a box and call a callback. Thread safe */
	PCG_API void ForAllIntersectingPartitionActors(const FBox& InBounds, TFunctionRef<void(APCGPartitionActor*)> InFunc) const;

	/** Register a new execution source or update it. Returns true if it was added/updated. Thread safe */
	bool RegisterOrUpdateExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bDoPartitionMapping = true);

	/** In case of blueprints, we need to remap the old execution source destroyed by the construction script to the new one. Returns true if a re-mapping was done. Thread safe */
	bool RemapExecutionSource(const IPCGGraphExecutionSource* OldExecutionSource, IPCGGraphExecutionSource* NewExecutionSource, bool bDoPartitionMapping);

	/** Unregister a execution source. Can force it, if we have a delayed unregister. Thread safe */
	void UnregisterExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bForce = false);

	/** Register a new Partition actor, will be added to a map and will query all intersecting volume to bind to them if asked. Thread safe */
	void RegisterPartitionActor(APCGPartitionActor* InActor);

	/** Unregister a Partition actor, will be removed from the map and remove itself to all intersecting volumes. Thread safe */
	void UnregisterPartitionActor(APCGPartitionActor* InActor);

	/** Return a copy of all the registered partitioned execution sources. Thread safe */
	TSet<IPCGGraphExecutionSource*> GetAllRegisteredPartitionedExecutionSources() const;

	/** Return a copy of all the registered non-partitioned execution sources. Thread safe */
	TSet<IPCGGraphExecutionSource*> GetAllRegisteredNonPartitionedExecutionSources() const;

	/** Return a copy of all the registered execution sources. Thread safe */
	TSet<IPCGGraphExecutionSource*> GetAllRegisteredExecutionSources() const;

	/** Retrieves a local component using grid descriptor and grid coordinates, returns nullptr if no such component is found. */
	UPCGComponent* GetLocalComponent(const FPCGGridDescriptor& GridDescriptor, const FIntVector& CellCoords, const UPCGComponent* InOriginalComponent) const;

	APCGPartitionActor* GetPartitionActor(const FPCGGridDescriptor& GridDescriptor, const FIntVector& CellCoords) const;

#if WITH_EDITOR
	/** Returns true if there is record of a partition actor living in a certain grid cell, regardless of whether or not it is loaded. */
	bool DoesPartitionActorRecordExist(const FPCGGridDescriptor& GridDescriptor, const FIntVector& GridCoords) const;
#endif
private:

#if WITH_EDITOR
	void RegisterDelegates();
	void UnregisterDelegates();

	/** If the partition grid size change, call this to empty the Partition actors map */
	void ResetPartitionActorsMap();

	void RegisterTracking(IPCGGraphExecutionSource* InExecutionSource);
	void UpdateTracking(IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGSelectionKey>* OptionalChangedKeys = nullptr);
#endif
	bool RegisterOrUpdateExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bDoPartitionMapping, bool bForce);

	// This class is only meant to be used as part of the PCG Subsytem and owned by it.
	// So we put constructors private.
	// We also need this class to be default constructible, since the PCGSubsytem needs to be default constructible.
	FPCGTrackingManager() = default;
	explicit FPCGTrackingManager(UPCGSubsystem* PCGSubsystem);

	bool RegisterOrUpdatePartitionedExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bDoPartitionMapping = true);
	bool RegisterOrUpdateNonPartitionedExecutionSource(IPCGGraphExecutionSource* InExecutionSource);

	void UnregisterPartitionedExecutionSource(IPCGGraphExecutionSource* InExecutionSource);
	void UnregisterNonPartitionedExecutionSource(IPCGGraphExecutionSource* InExecutionSource);

	/* Call the InFunc function to all local component registered to the original component. Return the list of all the tasks scheduled. Thread safe*/
	TArray<FPCGTaskId> DispatchToRegisteredLocalComponents(UPCGComponent* OriginalComponent, const TFunctionRef<FPCGTaskId(UPCGComponent*, const TArray<FPCGTaskId>&)>& InFunc) const;

	/* Call the InFunc function to all local component from the set of partition actors. Return the list of all the tasks scheduled. */
	TArray<FPCGTaskId> DispatchToLocalComponents(UPCGComponent* OriginalComponent, const TSet<TObjectPtr<APCGPartitionActor>>& PartitionActors, const TFunctionRef<FPCGTaskId(UPCGComponent*, const TArray<FPCGTaskId>&)>& InFunc) const;

	/** Call the InFunc function to all partitioned execution sources which bounds intersect 'InBounds'. */
	void ForAllIntersectingPartitionedExecutionSources(const FBoxCenterAndExtent& InBounds, TFunctionRef<void(IPCGGraphExecutionSource*)> InFunc) const;

	/** Call the InFunc function for all original execution sources (regardless of partitioned or not). */
	void ForAllOriginalExecutionSources(TFunctionRef<void(IPCGGraphExecutionSource*)> InFunc);

	/** Gather all the execution sources within some bounds. */
	TArray<IPCGGraphExecutionSource*> GetAllIntersectingExecutionSources(const FBoxCenterAndExtent& InBounds) const;

	/** Update the current mapping between a PCG component and its PCG Partition actors */
	void UpdateMappingPCGComponentPartitionActor(UPCGComponent* InComponent, bool bInChangeGraphInstances = true);

	/** Returns the current mapping between a PCG component and its PCG Partition actors */
	TSet<TObjectPtr<APCGPartitionActor>> GetPCGComponentPartitionActorMappings(UPCGComponent* InComponent) const;
	
	/** Delete the current mapping between a PCG component and its PCG Partition actors */
	void DeleteMappingPCGComponentPartitionActor(UPCGComponent* InComponent);

	/** Returns true if an executon source is registered */
	bool IsExecutionSourceRegistered(const IPCGGraphExecutionSource* InExecutionSource) const;

	/** Return true if there are any Original or Non-Partitioned execution sources set to GenerateAtRuntime. */
	bool AnyRuntimeGenExecutionSourcesExist() const;

#if WITH_EDITOR
	void OnPackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);
	void OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* InActorDescInstance);
	void OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* InActorDescInstance);
	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InEvent);
	void OnPCGGraphStartsGenerating(IPCGGraphExecutionSource* InExecutionSource);
	void OnPCGGraphGeneratedOrCleaned(IPCGGraphExecutionSource* InExecutionSource, TOptional<FBox> OptionalExecutionBounds = {});
	void OnPCGGraphCancelled(IPCGGraphExecutionSource* InExecutionSource);

	/** Remap the tracking in case of blueprints. */
	void RemapTracking(const IPCGGraphExecutionSource* InOldExecutionSource, IPCGGraphExecutionSource* InNewExecutionSource);

	/** Unregister tracking when a execution source is removed */
	bool UnregisterTracking(const IPCGGraphExecutionSource* InExecutionSource);

	/** Unregister tracking when a execution source is removed or keys are unregistered */
	bool UnregisterTracking(const IPCGGraphExecutionSource* InExecutionSource, const TSet<FPCGSelectionKey>* OptionalKeysToUntrack);

	void OnObjectChanged(const UObject* InObject, const FPCGTrackingChangeID& InChangeID, const UObject* InOriginatingChangeObject = nullptr);

	/** Build initial records for existing partition actors */
	void BuildPartitionActorRecords();

	/** Used by the PCG Builder to force dispatched local components to be dependency chained together so that they do not run at the same time for determinism */
	void SetChainedDispatchToLocalComponents(bool bInChainedDispatch) { bChainedDispatchToLocalComponents = bInChainedDispatch; }
#endif // WITH_EDITOR

private:
	// Cached subsystem
	UPCGSubsystem* PCGSubsystem = nullptr;

	// Mapping Component <-> PartitionActor
	/** Octree tracking all partitioned pcg components */
	FPCGExecutionSourceOctreeAndMap PartitionedOctree;

	/** Mapping from grid size and grid coords to partition actor. We can only have 1 partition actor per grid cell. */
	TMap<FPCGGridDescriptor, TMap<FIntVector, TObjectPtr<APCGPartitionActor>>> PartitionActorsMap;
	mutable PCG::FSharedLock PartitionActorsMapLock;

	/** Mapping between original components and its overlapping partition actors. */
	TMap<const UPCGComponent*, TSet<TObjectPtr<APCGPartitionActor>>> ComponentToPartitionActorsMap;
	mutable PCG::FSharedLock ComponentToPartitionActorsMapLock;

	/** Will hold all the components that are not partitioned (and not local) and are tracking something. Will be use to dispatch actor tracking updates. */
	FPCGExecutionSourceOctreeAndMap NonPartitionedOctree;

#if WITH_EDITOR
	TArray<TUniquePtr<IPCGChangeTracker>> ChangeTrackers;
	TArray<TUniquePtr<IPCGChangeHandler>> ChangeHandlers;

	/** Components to be registered/unregistered on the next frame. see RegisterOrUpdatePCGComponent/UnregisterPCGComponent for a better understanding on why it is needed. */
	mutable PCG::FLock DelayedComponentsRegistrationLock;
	TSet<TWeakObjectPtr<UPCGComponent>> DelayedComponentsToUnregister;
	TSet<TWeakObjectPtr<UPCGComponent>> DelayedComponentsToRegister;
	
	// Tracking actors
	/** Keep a mapping between tracked keys and the execution sources that track them, and the tracking needs to be culled.*/
	TMap<FPCGSelectionKey, TSet<IPCGGraphExecutionSource*>> CulledTrackedKeysToExecutionSourcesMap;

	/** Same mapping but for always tracked keys */
	TMap<FPCGSelectionKey, TSet<IPCGGraphExecutionSource*>> AlwaysTrackedKeysToExecutionSourcesMap;

	mutable PCG::FSharedLock TrackedComponentsLock;

	/** Transient map that keep track of all execution source that depend on other execution sources component currently generating, to trigger the refresh only once, when all are done. */
	TMap<FObjectKey, TArray<FObjectKey>> ExecutionSourcesToDependencyMap;

	// Set of existing PCG Partition Actors (for World Partition worlds)
	TMap<FPCGGridCellDescriptor, FGuid> PartitionActorRecords;
	// Previously generated PCG Partition Actors to ignore (will prevent them from getting registered and mark them for deletion)
	TSet<FGuid> InvalidPartitionActors;

	// If dispatch needs to run the local comopnents one after the other through dependency chain for determinism
	bool bChainedDispatchToLocalComponents = false;
#endif // WITH_EDITOR
};