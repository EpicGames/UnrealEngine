// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "RuntimeGen/PCGRuntimeGenExecutionSource.h"

#include "UObject/ObjectKey.h"

class IPCGGenSourceBase;

namespace PCGRuntimeGenChangeDetection
{
	using FDetectionInputsBase = TTuple</*OriginalExecutionSource*/TObjectKey<UObject>, const IPCGGenSourceBase*, FPCGRuntimeGenerationRadii, /*RadiusMultiplier*/float, /*UseGenSourceDirection*/bool, /*MinGridSize*/uint32>;

	/** The key used for the state cache. */
	struct FDetectionInputs : FDetectionInputsBase
	{
		FDetectionInputs(const IPCGGraphExecutionSource* InOriginalExecutionSource, const IPCGGenSourceBase* InGenSource, const FPCGRuntimeGenerationRadii& InGenerationRadii, float InRadiusMultiplier, bool bInUseGenSourceDirection, uint32 InMinGridSize)
			: FDetectionInputsBase(Cast<UObject>(InOriginalExecutionSource), InGenSource, InGenerationRadii, InRadiusMultiplier, bInUseGenSourceDirection, InMinGridSize)
		{}

		const IPCGGraphExecutionSource* GetOriginalExecutionSource() const { return Cast<IPCGGraphExecutionSource>(Get<0>().ResolveObjectPtr()); }
		const IPCGGenSourceBase* GetGenSource() const { return Get<1>(); }
		const FPCGRuntimeGenerationRadii& GetGenerationRadii() const { return Get<2>(); }
		float GetGenerationRadiusMultiplier() const { return Get<3>(); }
		bool UseGenSourceDirection() const { return Get<4>(); }
		uint32 GetMinGridSize() const { return Get<5>(); }
	};

	/** Used to detect when world state (generation sources, cvars etc) have changed enough to warrant rescanning generation cells. */
	struct FDetector
	{
		/** The value in the state cache. Stores transient values so they can later used to detect change. */
		struct FCachedState
		{
			void UpdateFromInputs(const FDetectionInputs& InInputs);

			bool ShouldScanCells(const FDetectionInputs& InInputs) const;

			TOptional<FVector> GenSourcePosition;
			TOptional<FVector> GenSourceDirection;
			float GenerationRadiusMultiplier = 1.0f;
		};

		void PreTick();
		void PostTick();

		/** Compare current state against cached state to determine if a rescan of cells is required. */
		bool IsCellScanRequired(const FDetectionInputs& InInputs);

		/** Called after all cells have been scanned to update the cached state. */
		void OnCellsScanned(const FDetectionInputs& InInputs);

		/** Flush any recorded state, resulting in a rescan of the component on next tick. If nullptr provided all components will rescan. */
		void FlushCachedState(const IPCGGraphExecutionSource* InOptionalOriginalExecutionSource = nullptr);

	private:
		TMap<FDetectionInputs, FCachedState> StateCache;

		/** Any inputs in this array will be removed from the StateCache during PostTick(). */
		TArray<FDetectionInputs> InputsPendingDeletion;
	};
}
