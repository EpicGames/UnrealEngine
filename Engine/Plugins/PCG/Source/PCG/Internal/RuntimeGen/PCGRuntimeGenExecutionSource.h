// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGraphExecutionInspection.h"
#include "PCGManagedResourceContainer.h"
#include "PCGRuntimeGenExecutionState.h"
#include "ChangeTracking/PCGChangeTracker.h"

#include "PCGRuntimeGenExecutionSource.generated.h"

/**
 * Runtime generated execution source, managed by the RuntimeGenScheduler. This execution source is never serialized
 * and does not produce actors/actor-components. It is owned by a concrete original execution source (e.g. a UPCGComponent in the level)
 * which is responsible for providing state (e.g. World, Seed, Transform, etc.).
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGRuntimeGenExecutionSource : public UObject, public IPCGGraphExecutionSource
{
	GENERATED_BODY()

public:
	/** ~Begin UObject interface */
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	/** ~End UObject interface */

	/** ~Begin IPCGGraphExecutionSource interface */
	virtual IPCGGraphExecutionState& GetExecutionState() override { return ExecutionState; }
	virtual const IPCGGraphExecutionState& GetExecutionState() const override { return ExecutionState; }
	/** ~End IPCGGraphExecutionSource interface */

	UPCGRuntimeGenExecutionSource(const FObjectInitializer& InObjectInitializer);

	void Register(IPCGGraphExecutionSource* InOwningSource);
	void Unregister();
	void SetOwningSource(IPCGGraphExecutionSource* InOwningSource);

	/** Begin mirror of FPCGRuntimeGenExecutionState */
	UPCGData* GetSelfData() const;
	int32 GetSeed() const;
	FString GetDebugName() const;
	UWorld* GetWorld() const;
	UObject* GetTarget() const;
	bool IsActive() const;
	bool HasAuthority() const;
	FTransform GetTransform() const;
	FTransform GetOriginalTransform() const;
	FBox GetBounds() const;
	FBox GetOriginalBounds() const;
	FBox GetLocalSpaceBounds() const;
	FBox GetOriginalLocalSpaceBounds() const;
	UPCGGraph* GetGraph() const;
	UPCGGraphInstance* GetGraphInstance() const;
	void Cancel();
	void OnGraphExecutionAborted(bool bQuiet = false, bool bCleanupUnusedResources = true);
	bool IsGenerating() const;
	bool IsGenerated() const;
	FPCGTaskId GetGenerationTaskId() const;
	FPCGTaskId Generate(const IPCGGraphExecutionState::FGenerateParams& InGenerateParams);
	FPCGTaskId Cleanup(const IPCGGraphExecutionState::FCleanupParams& InCleanupParams);
	void StoreOutputDataForPin(const FString& InResourceKey, const FPCGDataCollection& InData);
	const FPCGDataCollection* RetrieveOutputDataForPin(const FString& InResourceKey) const;
	FPCGManagedResourceContainer* GetManagedResourceContainer();
	FTransactionallySafeCriticalSection* GetManagedResourceContainerLock();
	void ExecutePreGraph(FPCGContext* InContext);

#if PCG_PROFILING_ENABLED
	const PCGUtils::FExtraCapture& GetExtraCapture() const;
	PCGUtils::FExtraCapture& GetExtraCapture();
	const FPCGGraphExecutionInspection& GetInspection() const;
	FPCGGraphExecutionInspection& GetInspection();
#endif // PCG_PROFILING_ENABLED

#if WITH_EDITOR
	FPCGDynamicTrackingPriority GetDynamicTrackingPriority() const;
	void RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling);
	void RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings);
	TArray<FPCGSelectionKey> GatherTrackingKeys() const;
	bool IsKeyTrackedAndCulled(const FPCGSelectionKey& Key, bool& bOutIsCulled) const;
	void ForEachSettingTrackingKey(const FPCGSelectionKey& InKey, const TFunctionRef<void(const FPCGSelectionKey&, const FPCGSettingsAndCulling&)> InCallback) const;
	void AddLocalSourceDynamicTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceDynamicTrackingKeys);
	void AddLocalSourceCurrentTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceCurrentExecutionTrackingKeys, const TSet<const UPCGSettings*>& LocalSourceCurrentExecutionTrackingSettings);
	bool IsRefreshInProgress() const;
	bool WasGeneratedThisSession() const;
#endif

	IPCGGraphExecutionSource* GetOriginalSource() const;
	IPCGGraphExecutionSource* GetLocalSource(const FPCGGridDescriptor& InGridDescriptor, const FIntVector& InGridCoords) const;
	bool IsPartitioned() const;
	bool IsLocalSource() const;
	bool Use2DGrid() const;
	uint32 GetGenerationGridSize() const;
	FIntVector GetGenerationGridCoords() const;
	bool IsManagedByRuntimeGenSystem() const;
	const UPCGSchedulingPolicyBase* GetRuntimeGenSchedulingPolicy() const;
	const FPCGRuntimeGenerationRadii& GetGenerationRadii() const;
	double GetGenerationRadiusFromGrid(uint32 InGrid) const;
	double GetCleanupRadiusFromGrid(uint32 InGrid) const;
	/** End mirror of FPCGRuntimeGenExecutionState */

	/** Creates a local execution source that is automatically setup and registered with this original execution source. Nullptr if this is not a partitioned original source. */
	UPCGRuntimeGenExecutionSource* CreateLocalSource(uint32 InGridSize, const FIntVector& InGridCoords);

	/** Removes a local source from the tracking set allowing it to be garbage collected. */
	void DestroyLocalSource(UPCGRuntimeGenExecutionSource* InLocalSource);

	/** The owning execution source drives the properties for the RuntimeGen execution source. */
	IPCGGraphExecutionSource* GetOwningSource();

private:
	/** Clear any data stored for any pins. */
	void ClearPerPinGeneratedOutput();

	void CreateSelfData();
	FBox GetBoundsInternal() const;
	FBox GetOriginalBoundsInternal() const;
	FBox GetLocalSpaceBoundsInternal() const;
	FBox GetOriginalLocalSpaceBoundsInternal() const;

#if WITH_EDITOR
	bool UpdateTrackingCache(TArray<FPCGSelectionKey>* InOptionalChangedKeys = nullptr);
#endif

private:
	FPCGRuntimeGenExecutionState ExecutionState;

	IPCGGraphExecutionSource* OwningSource = nullptr;
	UPCGRuntimeGenExecutionSource* OriginalSource = nullptr;
	uint32 GridSize = PCGHiGenGrid::UnboundedGridSize();
	FIntVector GridCoords = FIntVector::ZeroValue;

	UPROPERTY()
	TSet<TObjectPtr<UPCGRuntimeGenExecutionSource>> LocalSources;

	FPCGTaskId GenerationTaskId = InvalidPCGTaskId;
	bool bGenerated = false;

	/** If any graph edges cross execution grid sizes, data on the edge is stored / retrieved from this map. */
	UPROPERTY(Transient)
	TMap<FString, FPCGDataCollection> PerPinGeneratedOutput;
	mutable FTransactionallySafeRWLock PerPinGeneratedOutputLock;

	UPROPERTY()
	FPCGManagedResourceContainer ManagedResourceContainer;
	mutable FTransactionallySafeCriticalSection ManagedResourcesLock;

#if PCG_PROFILING_ENABLED
	mutable PCGUtils::FExtraCapture ExtraCapture;
	FPCGGraphExecutionInspection ExecutionInspection;
#endif // PCG_PROFILING_ENABLED

#if WITH_EDITOR
	FPCGChangeTracker ChangeTracker;
#endif

	UPROPERTY()
	TObjectPtr<UPCGData> SelfData = nullptr;
};
