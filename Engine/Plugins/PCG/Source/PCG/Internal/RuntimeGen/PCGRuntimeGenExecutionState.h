// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGraphExecutionStateInterface.h"

class UPCGRuntimeGenExecutionSource;

class FPCGRuntimeGenExecutionState : public IPCGGraphExecutionState
{
	/** ~Begin IPCGGraphExecutionState interface */
public:
	virtual UPCGData* GetSelfData() const override;
	virtual int32 GetSeed() const override;
	virtual FString GetDebugName() const override;
	virtual UWorld* GetWorld() const override;
	virtual UObject* GetTarget() const override;
	virtual bool IsActive() const override;
	virtual bool HasAuthority() const override;
	virtual FTransform GetTransform() const override;
	virtual FTransform GetOriginalTransform() const override;
	virtual FBox GetBounds() const override;
	virtual FBox GetOriginalBounds() const override;
	virtual FBox GetLocalSpaceBounds() const override;
	virtual FBox GetOriginalLocalSpaceBounds() const override;
	virtual UPCGGraph* GetGraph() const override;
	virtual UPCGGraphInstance* GetGraphInstance() const override;
	virtual void Cancel() override;
	virtual void OnGraphExecutionAborted(bool bQuiet = false, bool bCleanupUnusedResources = true) override;
	virtual bool IsGenerating() const override;
	virtual bool IsGenerated() const override;
	virtual FPCGTaskId GetGenerationTaskId() const override;
	virtual FPCGTaskId Generate(const FGenerateParams& InGenerateParams) override;
	virtual FPCGTaskId Cleanup(const FCleanupParams& InCleanupParams) override;
	virtual void StoreOutputDataForPin(const FString& InResourceKey, const FPCGDataCollection& InData) const override;
	virtual const FPCGDataCollection* RetrieveOutputDataForPin(const FString& InResourceKey) const override;
	virtual FPCGManagedResourceContainer* GetManagedResourceContainer() override;
	virtual FTransactionallySafeCriticalSection* GetManagedResourceContainerLock() override;
	virtual void ExecutePreGraph(FPCGContext* InContext) override;

#if PCG_PROFILING_ENABLED
	virtual const PCGUtils::FExtraCapture& GetExtraCapture() const override;
	virtual PCGUtils::FExtraCapture& GetExtraCapture() override;
	virtual const FPCGGraphExecutionInspection& GetInspection() const override;
	virtual FPCGGraphExecutionInspection& GetInspection() override;
#endif // PCG_PROFILING_ENABLED

#if WITH_EDITOR
	PCG_API virtual FPCGDynamicTrackingPriority GetDynamicTrackingPriority() const override;
	virtual void RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling) override;
	virtual void RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings) override;
	virtual TArray<FPCGSelectionKey> GatherTrackingKeys() const override;
	virtual bool IsKeyTrackedAndCulled(const FPCGSelectionKey& Key, bool& bOutIsCulled) const override;
	virtual void ForEachSettingTrackingKey(const FPCGSelectionKey& InKey, const TFunctionRef<void(const FPCGSelectionKey&, const FPCGSettingsAndCulling&)> InCallback) const override;
	virtual void AddLocalSourceDynamicTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceDynamicTrackingKeys) override;
	virtual void AddLocalSourceCurrentTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceCurrentExecutionTrackingKeys, const TSet<const UPCGSettings*>& LocalSourceCurrentExecutionTrackingSettings) override;
	virtual bool IsRefreshInProgress() const override;
	virtual bool WasGeneratedThisSession() const override;
#endif

	virtual IPCGGraphExecutionSource* GetOriginalSource() const override;
	virtual IPCGGraphExecutionSource* GetLocalSource(const FPCGGridDescriptor& GridDescriptor, const FIntVector& CellCoords) const override;
	virtual bool IsPartitioned() const override;
	virtual bool IsLocalSource() const override;
	virtual bool Use2DGrid() const override;
	virtual uint32 GetGenerationGridSize() const override;
	virtual FIntVector GetGenerationGridCoords() const override;
	virtual bool IsManagedByRuntimeGenSystem() const override;
	virtual const UPCGSchedulingPolicyBase* GetRuntimeGenSchedulingPolicy() const override;
	virtual const FPCGRuntimeGenerationRadii& GetGenerationRadii() const override;
	virtual double GetGenerationRadiusFromGrid(uint32 InGrid) const override;
	virtual double GetCleanupRadiusFromGrid(uint32 InGrid) const override;
	/** ~End IPCGGraphExecutionState interface */

	void SetSource(UPCGRuntimeGenExecutionSource* InSource);

private:
	UPCGRuntimeGenExecutionSource* Source = nullptr;
};
