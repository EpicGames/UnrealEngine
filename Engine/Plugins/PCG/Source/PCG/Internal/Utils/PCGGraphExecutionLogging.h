// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

class IPCGGraphExecutionSource;
class UPCGComponent;
class UPCGGraph;
enum class EPCGHiGenGrid : uint32;
enum class EPCGIgnoreChangeOriginReason : uint8;
struct FPCGContext;
struct FPCGGraphActiveTask;
struct FPCGGraphTask;
struct FPCGPinDependencyExpression;
struct FPCGStack;

namespace PCGGraphExecutionLogging
{
	bool LogEnabled();
	bool CullingLogEnabled();

	void LogGraphTask(FPCGTaskId TaskId, const FPCGGraphTask& Task, const TSet<FPCGTaskId>* SuccessorIds = nullptr);
	void LogGraphTasks(const TMap<FPCGTaskId, FPCGGraphTask>& Tasks, const TMap<FPCGTaskId, TSet<FPCGTaskId>>* TaskSuccessors = nullptr);
	void LogGraphTasks(const TArray<FPCGGraphTask>& Tasks);

	void LogGraphSchedule(const IPCGGraphExecutionSource* InExecutionSource, const UPCGGraph* InScheduledGraph);
	void LogGraphScheduleDependency(const IPCGGraphExecutionSource* InExecutionSource, const FPCGStack* InFromStack);
	void LogGraphScheduleDependencyFailed(const IPCGGraphExecutionSource* InExecutionSource, const FPCGStack* InFromStack);

	void LogGraphPostSchedule(const TMap<FPCGTaskId, FPCGGraphTask>& Tasks, const TMap<FPCGTaskId, TSet<FPCGTaskId>>& TaskSuccessors);
	void LogPostProcessGraph(const IPCGGraphExecutionSource* InExecutionSource);

	void LogExecutionSourceCancellation(const TSet<IPCGGraphExecutionSource*>& CancelledComponents);

	PCG_API void LogChangeOriginIgnoredForComponent(const UObject* InObject, const IPCGGraphExecutionSource* InExecutionSource, EPCGIgnoreChangeOriginReason InReason);

	void LogGraphExecuteFrameFinished();

	void LogTaskExecute(const FPCGGraphTask& Task);
	void LogTaskExecuteCachingDisabled(const FPCGGraphTask& Task);

	void LogTaskCullingBegin(FPCGTaskId CompletedTaskId, uint64 InactiveOutputPinBitmask, const TArray<FPCGPinId>& PinIdsToDeactivate);
	void LogTaskCullingBeginLoop(FPCGTaskId PinTaskId, uint64 PinIndex, const TArray<FPCGPinId>& PinIdsToDeactivate);
	void LogTaskCullingUpdatedPinDeps(FPCGTaskId TaskId, const FPCGPinDependencyExpression& PinDependency, bool bDependencyExpressionBecameFalse);

	void LogGridLinkageTaskExecuteStore(const FPCGContext* InContext, uint32 InGenerationGrid, uint32 InFromGridSize, uint32 InToGridSize, const FString& InResourcePath, int32 InDataItemCount);
	void LogGridLinkageTaskExecuteRetrieve(const FPCGContext* InContext, uint32 InGenerationGrid, uint32 InFromGridSize, uint32 InToGridSize, const FString& InResourcePath);
	void LogGridLinkageTaskExecuteRetrieveSuccess(const FPCGContext* InContext, const IPCGGraphExecutionSource* InExecutionSource, const FString& InResourcePath, int32 InDataItemCount);
	void LogGridLinkageTaskExecuteRetrieveScheduleGraph(const FPCGContext* InContext, const IPCGGraphExecutionSource* InScheduledSource, const FString& InResourcePath);
	void LogGridLinkageTaskExecuteRetrieveWaitOnScheduledGraph(const FPCGContext* InContext, const IPCGGraphExecutionSource* InWaitOnSource, const FString& InResourcePath);
	void LogGridLinkageTaskExecuteRetrieveWakeUp(const FPCGContext* InContext, const IPCGGraphExecutionSource* InWokenBySource);
	void LogGridLinkageTaskExecuteRetrieveNoLocalSource(const FPCGContext* InContext, const FString& InResourcePath);
	void LogGridLinkageTaskExecuteRetrieveNoData(const FPCGContext* InContext, const IPCGGraphExecutionSource* InExecutionSource, const FString& InResourcePath);
	void LogGridLinkageTaskStoreNullDataOnCulled(const IPCGGraphExecutionSource* InExecutionSource, uint32 InGenerationGrid, uint32 InFromGridSize, uint32 InToGridSize, const FString& InResourcePath);
}
