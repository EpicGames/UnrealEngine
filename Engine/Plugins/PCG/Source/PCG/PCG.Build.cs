// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PCG : ModuleRules
	{
		public PCG(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"ComputeFramework",
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"Foliage",
					"GeometryAlgorithms",
					"GeometryCore",
					"GeometryFramework",
					"Landscape",
					"PhysicsCore",
					"Projects",
					"RenderCore",
					"RHI",
				});
			SetupModulePhysicsSupport(Target);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"PCGCompute",
					"Renderer",
					"Voronoi",
					"Json"
				});

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
						"Settings",
						"SourceControl",
						"InteractiveToolsFramework",
						"DataHierarchyEditor"
					});
			}

			//PublicDefinitions.Add("PCG_DATA_USAGE_LOGGING"); // Generates log to help debug lifetimes of transient resources.
			//PublicDefinitions.Add("PCG_GPU_KERNEL_PROFILING"); // Enables repeating kernel dispatches every frame to help profiling.
		}
	}
}
