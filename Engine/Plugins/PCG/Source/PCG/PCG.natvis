<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">
  
  <!-- Copyright Epic Games, Inc. All Rights Reserved. -->
  
  <Type Name="UPCGPin">
    <DisplayString Condition="Edges.ArrayNum==0">Name={Properties.Label}, Type={Properties.AllowedTypes}</DisplayString>
    <DisplayString Condition="Edges.ArrayNum &gt; 0">Name={Properties.Label}, Type={Properties.AllowedTypes}, Edges={Edges.ArrayNum}</DisplayString>
  </Type>

  <Type Name="FPCGSelectionKey">
    <DisplayString Condition="Selection==EPCGActorSelection::ByPath">ByPath={ObjectPath}, ExtraDep={OptionalExtraDependency}</DisplayString>
    <DisplayString Condition="Selection==EPCGActorSelection::ByTag">ByTag={Tag}, ExtraDep={OptionalExtraDependency}</DisplayString>
    <DisplayString Condition="Selection==EPCGActorSelection::ByClass">ByClass={SelectionClass}, ExtraDep={OptionalExtraDependency}</DisplayString>
    <DisplayString Condition="ActorFilter!=EPCGActorFilter::AllWorldActors">{ActorFilter}, ExtraDep={OptionalExtraDependency}</DisplayString>
  </Type>

    <!-- FPCGMetadataAttributeBase visualizer -->
  <Type Name="FPCGMetadataAttributeBase">
    <Intrinsic Name="_IsContainer" Expression="CachedDesc.ContainerTypes.ArrayNum != 0"/>
    <Intrinsic Name="_ContainerType" Expression="*(EPCGMetadataAttributeContainerTypes*)CachedDesc.ContainerTypes.AllocatorInstance._GetData()"/>
    <Intrinsic Name="_IsSingleValue" Expression="!_IsContainer() || _ContainerType() == EPCGMetadataAttributeContainerTypes::None"/>
    <Intrinsic Name="_IsArray" Expression="_IsContainer() &amp;&amp; _ContainerType() == EPCGMetadataAttributeContainerTypes::Array"/>
    <Intrinsic Name="_IsSet" Expression="_IsContainer() &amp;&amp; _ContainerType() == EPCGMetadataAttributeContainerTypes::Set"/>
    <Intrinsic Name="_IsMap" Expression="_IsContainer() &amp;&amp; _ContainerType() == EPCGMetadataAttributeContainerTypes::Map"/>
    <Intrinsic Name="_GetProperty" Expression="UnderlyingProperty->Property->Inner"/>
    <Intrinsic Name="_GetArrayInnerProperty" Expression="(*(FArrayProperty**)(&amp;UnderlyingProperty->Property->Inner))->Inner"/>
    <Intrinsic Name="_GetSetInnerProperty" Expression="(*(FSetProperty**)(&amp;UnderlyingProperty->Property->Inner))->ElementProp"/>
    <Intrinsic Name="_GetMapKeyProperty" Expression="(*(FMapProperty**)(&amp;UnderlyingProperty->Property->Inner))->KeyProp"/>
    <Intrinsic Name="_GetMapValueProperty" Expression="(*(FMapProperty**)(&amp;UnderlyingProperty->Property->Inner))->ValueProp"/>
    <Intrinsic Name="_GetPropertyTypeName" Expression="Property->ClassPrivate->Name">
      <Parameter Name="Property" Type="FProperty*"/>
    </Intrinsic>
    <Intrinsic Name="_GetNumValues" Expression="Values.ArrayNum"/>
    <Intrinsic Name="_GetValuesCapacity" Expression="Values.ArrayMax"/>
    <Intrinsic Name="_ValuesAsScriptArray" Expression="(FScriptArray*)Values.Data"/>
    <Intrinsic Name="_DefaultValueAsScriptArray" Expression="(FScriptArray*)DefaultValue.Memory"/>
    <Intrinsic Name="_GetValueObjectName" Expression="CachedDesc.ValueTypeObject.ObjectPtr.DebugPtr->NamePrivate.Name"/>
    <Intrinsic Name="_GetKeyObjectName" Expression="CachedDesc.KeyTypeObject.ObjectPtr.DebugPtr->NamePrivate.Name"/>
    
    <!-- Displaying the type -->
    <DisplayString Condition="UnderlyingProperty.Object == nullptr">Invalid</DisplayString>
    
    <!-- Single values -->
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Struct">{{{CachedDesc.Name,sb} - F{_GetValueObjectName(),sb}}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector2">{{{CachedDesc.Name,sb} - FVector2D}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector">{{{CachedDesc.Name,sb} - FVector}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector4">{{{CachedDesc.Name,sb} - FVector4}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Rotator">{{{CachedDesc.Name,sb} - FRotator}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Quaternion">{{{CachedDesc.Name,sb} - FQuat}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Transform">{{{CachedDesc.Name,sb} - FTransform}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftObjectPath">{{{CachedDesc.Name,sb} - FSoftObjectPath}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftClassPath">{{{CachedDesc.Name,sb} - FSoftClassPath}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Object">{{{CachedDesc.Name,sb} - U{_GetValueObjectName(),sb}}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftObject">{{{CachedDesc.Name,sb} - U{_GetValueObjectName(),sb}}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftClass">{{{CachedDesc.Name,sb} - U{_GetValueObjectName(),sb}}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Enum">{{{CachedDesc.Name,sb} - E{_GetValueObjectName(),sb}}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSingleValue()">{{{CachedDesc.Name,sb} - {_GetPropertyTypeName(_GetProperty()),sb}}}</DisplayString>

    <!-- Arrays -->
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Struct">{{{CachedDesc.Name,sb} - Array&lt;F{_GetValueObjectName(),sb}&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector2">{{{CachedDesc.Name,sb} - Array&lt;FVector2D&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector">{{{CachedDesc.Name,sb} - Array&lt;FVector&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector4">{{{CachedDesc.Name,sb} - Array&lt;FVector4&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Rotator">{{{CachedDesc.Name,sb} - Array&lt;FRotator&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Quaternion">{{{CachedDesc.Name,sb} - Array&lt;FQuat&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Transform">{{{CachedDesc.Name,sb} - Array&lt;FTransform&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftObjectPath">{{{CachedDesc.Name,sb} - Array&lt;FSoftObjectPath&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftClassPath">{{{CachedDesc.Name,sb} - Array&lt;FSoftClassPath&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Object">{{{CachedDesc.Name,sb} - Array&lt;U{_GetValueObjectName(),sb}&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftObject">{{{CachedDesc.Name,sb} - Array&lt;U{_GetValueObjectName(),sb}&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftClass">{{{CachedDesc.Name,sb} - Array&lt;U{_GetValueObjectName(),sb}&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Enum">{{{CachedDesc.Name,sb} - Array&lt;E{_GetValueObjectName(),sb}&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsArray()">{{{CachedDesc.Name,sb} - Array&lt;{_GetPropertyTypeName(_GetArrayInnerProperty()),sb}&gt;}}</DisplayString>

    <!-- Set -->
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Struct">{{{CachedDesc.Name,sb} - Set&lt;F{_GetValueObjectName(),sb}&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector2">{{{CachedDesc.Name,sb} - Set&lt;FVector2D&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector">{{{CachedDesc.Name,sb} - Set&lt;FVector&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector4">{{{CachedDesc.Name,sb} - Set&lt;FVector4&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Rotator">{{{CachedDesc.Name,sb} - Set&lt;FRotator&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Quaternion">{{{CachedDesc.Name,sb} - Set&lt;FQuat&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Transform">{{{CachedDesc.Name,sb} - Set&lt;FTransform&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftObjectPath">{{{CachedDesc.Name,sb} - Set&lt;FSoftObjectPath&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftClassPath">{{{CachedDesc.Name,sb} - Set&lt;FSoftClassPath&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Object">{{{CachedDesc.Name,sb} - Set&lt;U{_GetValueObjectName(),sb}&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftObject">{{{CachedDesc.Name,sb} - Set&lt;U{_GetValueObjectName(),sb}&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftClass">{{{CachedDesc.Name,sb} - Set&lt;U{_GetValueObjectName(),sb}&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Enum">{{{CachedDesc.Name,sb} - Set&lt;E{_GetValueObjectName(),sb}&gt;}}</DisplayString>
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsSet()">{{{CachedDesc.Name,sb} - Set&lt;{_GetPropertyTypeName(_GetSetInnerProperty()),sb}&gt;}}</DisplayString>
    
    <!-- Map, too many combinations so keep it simple-->
    <DisplayString Condition="UnderlyingProperty.Object != nullptr &amp;&amp; _IsMap()">{{{CachedDesc.Name,sb} - Map&lt;{_GetPropertyTypeName(_GetMapKeyProperty()),sb},{_GetPropertyTypeName(_GetMapValueProperty()),sb}&gt;</DisplayString>

    <Expand>
      <Item Name="EntryToValueKeyMap">EntryToValueKeyMap</Item>
      <Item Name="Metadata">Metadata</Item>
      <Item Name="Parent">Parent</Item>
      <Item Name="bAllowsInterpolation">bAllowsInterpolation</Item>
      <Item Name="TypeId (Legacy)">(EPCGMetadataTypes)TypeId</Item>
      <Item Name="CachedDesc">CachedDesc</Item>
      <Item Name="UnderlyingProperty">UnderlyingProperty</Item>
      <Item Name="ValueKeyOffset">ValueKeyOffset</Item>
      
      <!-- All possible default values. For more complex types, the user will be required to cast it. -->
      <!-- Single values -->
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Float">*(float*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Double">*(double*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Integer32">*(int32*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Integer64">*(int64*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Boolean">*(bool*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector2">*(FVector2D*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector">*(FVector*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector4">*(FVector4*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Quaternion">*(FQuat*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Rotator">*(FRotator*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Transform">*(FTransform*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::String">*(FString*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Name">*(FName*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Text">*(FText*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Byte">*(uint8*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftObjectPath">*(FSoftObjectPath*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftClassPath">*(FSoftClassPath*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Struct">(void*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Enum">*(int8*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftObject">*(FSoftObjectPtr*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftClass">*(FSoftObjectPtr*)DefaultValue.Memory</Item>
      <Item Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsSingleValue() &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Object">*(UObject**)DefaultValue.Memory</Item>
      
      <!-- Array -->
      <Synthetic Name="DefaultValue" Condition="DefaultValue.Memory &amp;&amp; _IsArray()">
        <DisplayString>{{Size={_DefaultValueAsScriptArray()->ArrayNum}, Capacity={_DefaultValueAsScriptArray()->ArrayMax}}}</DisplayString>
        <Expand>
          <ArrayItems>
            <Size>_DefaultValueAsScriptArray()->ArrayNum</Size>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Float">(float*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Double">(double*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Integer32">(int32*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Integer64">(int64*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Boolean">(bool*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Vector2">(FVector2D*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Vector">(FVector*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Vector4">(FVector4*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Quaternion">(FQuat*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Rotator">(FRotator*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Transform">(FTransform*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::String">(FString*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Name">(FName*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Text">(FText*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Byte">(uint8*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::SoftObjectPath">(FSoftObjectPath*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::SoftClassPath">(FSoftClassPath*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Object">(UObject**)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::SoftObject">(FSoftObjectPtr*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::SoftClass">(FSoftObjectPtr*)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Struct">(void**)_DefaultValueAsScriptArray()->Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Enum">(uint8*)_DefaultValueAsScriptArray()->Data</ValuePointer>
          </ArrayItems>
        </Expand>
      </Synthetic>
      
      <!-- Values -->
      <Synthetic Name="[Values]" Condition="UnderlyingProperty.Object != nullptr">
        <DisplayString>{{Size={_GetNumValues()}, Capacity={_GetValuesCapacity()}}}</DisplayString>
        <Expand>
          <!-- Single values -->
          <ArrayItems Condition="_IsSingleValue()">
            <Size>_GetNumValues()</Size>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Float">(float*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Double">(double*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Integer32">(int32*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Integer64">(int64*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Boolean">(bool*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Vector2">(FVector2D*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Vector">(FVector*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Vector4">(FVector4*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Quaternion">(FQuat*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Rotator">(FRotator*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Transform">(FTransform*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::String">(FString*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Name">(FName*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Text">(FText*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Byte">(uint8*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::SoftObjectPath">(FSoftObjectPath*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::SoftClassPath">(FSoftClassPath*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Object">(UObject**)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::SoftObject">(FSoftObjectPtr*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::SoftClass">(FSoftObjectPtr*)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Struct">(void**)Values.Data</ValuePointer>
            <ValuePointer Condition="CachedDesc.ValueType == EPCGMetadataTypes::Enum">(int8*)Values.Data</ValuePointer>
          </ArrayItems>

          <!-- Array -->
          <CustomListItems Condition="_IsArray()">
            <Variable Name="Index" InitialValue="0"/>
            <Variable Name="ArrayPtr" InitialValue="(FScriptArray*)_ValuesAsScriptArray()"/>
            <Variable Name="ArrayNum" InitialValue="0"/>
            <Loop Condition="Index &lt; _GetNumValues()">
              <Exec>ArrayNum = ArrayPtr->ArrayNum</Exec>
              <Item Name="[{Index}]" Condition="ArrayNum == 0">"Empty",sb</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Float">(float*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Double">(double*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Integer32">(int32*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Integer64">(int64*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Boolean">(bool*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector2">(FVector2D*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector">(FVector*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Vector4">(FVector4*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Quaternion">(FQuat*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Rotator">(FRotator*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Transform">(FTransform*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::String">(FString*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Name">(FName*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Text">(FText*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Byte">(uint8*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftObjectPath">(FSoftObjectPath*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftClassPath">(FSoftClassPath*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Object">(UObject**)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftObject">(FSoftObjectPtr*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::SoftClass">(FSoftObjectPtr*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Struct">(void**)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Item Name="[{Index}]" Condition="ArrayNum > 0 &amp;&amp; CachedDesc.ValueType == EPCGMetadataTypes::Enum">(int8*)ArrayPtr->Data,[ArrayNum]nag</Item>
              <Exec>++ArrayPtr</Exec>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

</AutoVisualizer>
