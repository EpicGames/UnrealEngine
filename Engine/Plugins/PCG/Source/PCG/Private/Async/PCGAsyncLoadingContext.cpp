// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/PCGAsyncLoadingContext.h"

#include "PCGContext.h"

#include "Engine/AssetManager.h"

IPCGAsyncLoadingContext::~IPCGAsyncLoadingContext()
{
	// We need to make sure to cancel any loading requested.
	CancelLoading();
}

void IPCGAsyncLoadingContext::CancelLoading()
{
	if (LoadHandle.IsValid() && LoadHandle->IsActive())
	{
		LoadHandle->CancelHandle();
	}

	LoadHandle.Reset();
}

bool IPCGAsyncLoadingContext::ResetLoadRequestIfDone()
{
	if (bLoadRequested && LoadHandle.IsValid() && LoadHandle->HasLoadCompleted())
	{
		LoadHandle.Reset();
		bLoadRequested = false;
		return true;
	}
	else
	{
		return false;
	}
}

void IPCGAsyncLoadingContext::ResolveAlreadyLoadedObjects(TArray<FSoftObjectPath>& ObjectsToLoad)
{
	for (int32 i = ObjectsToLoad.Num() - 1; i >= 0; --i)
	{
		UObject* ResolvedObject = ObjectsToLoad[i].ResolveObject();
		if (ResolvedObject || ObjectsToLoad[i].IsNull())
		{
			ObjectsToLoad.RemoveAtSwap(i);
			if (ResolvedObject)
			{
				AlreadyLoadedObjects.Emplace(ResolvedObject);
			}
		}
	}
}

bool IPCGAsyncLoadingContext::RequestResourceLoad(FPCGContext* ThisContext, TArray<FSoftObjectPath>&& ObjectsToLoad, bool bAsynchronous)
{
	if (!ObjectsToLoad.IsEmpty() && !bLoadRequested)
	{
		ResolveAlreadyLoadedObjects(ObjectsToLoad);

		// If there is nothing to load, early out.
		if (ObjectsToLoad.IsEmpty())
		{
			bLoadRequested = true;
			return true;
		}

		if (!bAsynchronous)
		{
			LoadHandle = UAssetManager::GetStreamableManager().RequestSyncLoad(std::move(ObjectsToLoad));
			bLoadRequested = true;

			return true;
		}
		else
		{
			ThisContext->bIsPaused = true;

			LoadHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(std::move(ObjectsToLoad),
				[ContextHandle = ThisContext->GetOrCreateHandle()]()
				{
					FPCGContext::FSharedContext<FPCGContext> SharedContext(ContextHandle);
					if (FPCGContext* Context = SharedContext.Get())
					{
						Context->bIsPaused = false;
					}
				});

			bLoadRequested = true;

			// If the load handle is not active it means objects were invalid
			if (!LoadHandle->IsActive())
			{
				ThisContext->bIsPaused = false;
				return true;
			}
			else
			{
				return false;
			}
		}
	}

	return true;
}