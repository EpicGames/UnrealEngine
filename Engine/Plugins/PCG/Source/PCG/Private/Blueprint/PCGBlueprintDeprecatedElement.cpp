// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Blueprint/PCGBlueprintDeprecatedElement.h"

#include "Helpers/PCGAsync.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBlueprintDeprecatedElement)

#define LOCTEXT_NAMESPACE "PCGBlueprintElement"

namespace PCGBlueprintConstants
{
	constexpr int32 RunawayResetFrequency = 1024;
}

void UPCGBlueprintElement::ExecuteWithContext_Implementation(FPCGContext& InContext, const FPCGDataCollection& Input, FPCGDataCollection& Output)
{
	Execute(Input, Output);
}

void UPCGBlueprintElement::PointLoop(FPCGContext& InContext, const UPCGPointData* InData, UPCGPointData*& OutData, UPCGPointData* OptionalOutData, bool bGameThread) const
{
	if (!InData)
	{
		PCGE_LOG_C(Error, GraphAndLog, &InContext, LOCTEXT("InvalidInputDataPointLoop", "Invalid input data in PointLoop"));
		return;
	}

	if (bGameThread && !IsInGameThread())
	{
		ThrowRequiresGameThreadException(&InContext, TEXT("PointLoop"));
		return;
	}

	if (OptionalOutData)
	{
		OutData = OptionalOutData;
	}
	else
	{
		OutData = FPCGContext::NewObject_AnyThread<UPCGPointData>(&InContext);
		OutData->InitializeFromData(InData);
	}

	const TArray<FPCGPoint>& InPoints = InData->GetPoints();
	TArray<FPCGPoint>& OutPoints = OutData->GetMutablePoints();

	TGuardValue<bool> IsCallingBlueprint(CurrentContext->AsyncState.bIsCallingBlueprint, true);
	TGuardValue<int32> NumTaskOverride(CurrentContext->AsyncState.NumAvailableTasks, bGameThread ? 1 : CurrentContext->AsyncState.NumAvailableTasks);

	FPCGAsync::AsyncPointProcessing(&InContext, InPoints.Num(), OutPoints, [this, &InContext, InData, OutData, &InPoints](int32 Index, FPCGPoint& OutPoint)
	{
		if (Index % PCGBlueprintConstants::RunawayResetFrequency == 0)
		{
			GInitRunaway(); // Reset periodically the iteration count, because we know we're in a fixed size loop.
		}

		return PointLoopBody(InContext, InData, InPoints[Index], OutPoint, OutData->Metadata, Index);
	});
}

void UPCGBlueprintElement::VariableLoop(FPCGContext& InContext, const UPCGPointData* InData, UPCGPointData*& OutData, UPCGPointData* OptionalOutData, bool bGameThread) const
{
	if (!InData)
	{
		PCGE_LOG_C(Error, GraphAndLog, &InContext, LOCTEXT("InvalidInputDataVariableLoop", "Invalid input data in VariableLoop"));
		return;
	}

	if (bGameThread && !IsInGameThread())
	{
		ThrowRequiresGameThreadException(&InContext, TEXT("VariableLoop"));
		return;
	}

	if (OptionalOutData)
	{
		OutData = OptionalOutData;
	}
	else
	{
		OutData = FPCGContext::NewObject_AnyThread<UPCGPointData>(&InContext);
		OutData->InitializeFromData(InData);
	}

	const TArray<FPCGPoint>& InPoints = InData->GetPoints();
	TArray<FPCGPoint>& OutPoints = OutData->GetMutablePoints();

	TGuardValue<bool> IsCallingBlueprint(CurrentContext->AsyncState.bIsCallingBlueprint, true);
	TGuardValue<int32> NumTaskOverride(CurrentContext->AsyncState.NumAvailableTasks, bGameThread ? 1 : CurrentContext->AsyncState.NumAvailableTasks);

	FPCGAsync::AsyncMultiPointProcessing(&InContext, InPoints.Num(), OutPoints, [this, &InContext, InData, OutData, &InPoints](int32 Index)
	{
		if (Index % PCGBlueprintConstants::RunawayResetFrequency == 0)
		{
			GInitRunaway(); // Reset periodically the iteration count, because we know we're in a fixed size loop.
		}

		return VariableLoopBody(InContext, InData, InPoints[Index], OutData->Metadata, Index);
	});
}

void UPCGBlueprintElement::NestedLoop(FPCGContext& InContext, const UPCGPointData* InOuterData, const UPCGPointData* InInnerData, UPCGPointData*& OutData, UPCGPointData* OptionalOutData, bool bGameThread) const
{
	if (!InOuterData || !InInnerData)
	{
		PCGE_LOG_C(Error, GraphAndLog, &InContext, LOCTEXT("InvalidInputDataNestedLoop", "Invalid input data in NestedLoop"));
		return;
	}

	if (bGameThread && !IsInGameThread())
	{
		ThrowRequiresGameThreadException(&InContext, TEXT("NestedLoop"));
		return;
	}

	if (OptionalOutData)
	{
		OutData = OptionalOutData;
	}
	else
	{
		OutData = FPCGContext::NewObject_AnyThread<UPCGPointData>(&InContext);
		OutData->InitializeFromData(InOuterData);
		OutData->Metadata->AddAttributes(InInnerData->Metadata);
	}

	const TArray<FPCGPoint>& InOuterPoints = InOuterData->GetPoints();
	const TArray<FPCGPoint>& InInnerPoints = InInnerData->GetPoints();
	TArray<FPCGPoint>& OutPoints = OutData->GetMutablePoints();

	TGuardValue<bool> IsCallingBlueprint(CurrentContext->AsyncState.bIsCallingBlueprint, true);
	TGuardValue<int32> NumTaskOverride(CurrentContext->AsyncState.NumAvailableTasks, bGameThread ? 1 : CurrentContext->AsyncState.NumAvailableTasks);

	FPCGAsync::AsyncPointProcessing(&InContext, InOuterPoints.Num() * InInnerPoints.Num(), OutPoints, [this, &InContext, InOuterData, InInnerData, OutData, &InOuterPoints, &InInnerPoints](int32 Index, FPCGPoint& OutPoint)
	{
		if (Index % PCGBlueprintConstants::RunawayResetFrequency == 0)
		{
			GInitRunaway(); // Reset periodically the iteration count, because we know we're in a fixed size loop.
		}

		const int32 OuterIndex = Index / InInnerPoints.Num();
		const int32 InnerIndex = Index % InInnerPoints.Num();
		return NestedLoopBody(InContext, InOuterData, InInnerData, InOuterPoints[OuterIndex], InInnerPoints[InnerIndex], OutPoint, OutData->Metadata, OuterIndex, InnerIndex);
	});
}

void UPCGBlueprintElement::IterationLoop(FPCGContext& InContext, int64 NumIterations, UPCGPointData*& OutData, const UPCGSpatialData* InA, const UPCGSpatialData* InB, UPCGPointData* OptionalOutData, bool bGameThread) const
{
	if (NumIterations < 0)
	{
		PCGE_LOG_C(Error, GraphAndLog, &InContext, FText::Format(LOCTEXT("InvalidIterationCount", "Invalid number of iterations ({0})"), NumIterations));
		return;
	}

	if (bGameThread && !IsInGameThread())
	{
		ThrowRequiresGameThreadException(&InContext, TEXT("IterationLoop"));
		return;
	}

	if (OptionalOutData)
	{
		OutData = OptionalOutData;
	}
	else
	{
		const UPCGSpatialData* Owner = (InA ? InA : InB);
		OutData = FPCGContext::NewObject_AnyThread<UPCGPointData>(&InContext);

		if (Owner)
		{
			OutData->InitializeFromData(Owner);
		}
	}

	TArray<FPCGPoint>& OutPoints = OutData->GetMutablePoints();

	TGuardValue<bool> IsCallingBlueprint(CurrentContext->AsyncState.bIsCallingBlueprint, true);
	TGuardValue<int32> NumTaskOverride(CurrentContext->AsyncState.NumAvailableTasks, bGameThread ? 1 : CurrentContext->AsyncState.NumAvailableTasks);

	FPCGAsync::AsyncPointProcessing(&InContext, NumIterations, OutPoints, [this, &InContext, InA, InB, OutData](int32 Index, FPCGPoint& OutPoint)
	{
		if (Index % PCGBlueprintConstants::RunawayResetFrequency == 0)
		{
			GInitRunaway(); // Reset periodically the iteration count, because we know we're in a fixed size loop.
		}

		return IterationLoopBody(InContext, Index, InA, InB, OutPoint, OutData->Metadata);
	});
}

FPCGContext& UPCGBlueprintElement::GetContext() const
{
	checkf(CurrentContext, TEXT("Execution context is not ready - do not call the GetContext method inside of non-execution methods"));
	return *CurrentContext;
}

int UPCGBlueprintElement::GetSeed(FPCGContext& InContext) const
{
	return InContext.GetSeed();
}

FRandomStream UPCGBlueprintElement::GetRandomStream(FPCGContext& InContext) const
{
	return FRandomStream(GetSeed(InContext));
}


#undef LOCTEXT_NAMESPACE
