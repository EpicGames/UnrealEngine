// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChangeTracking/PCGChangeTracker.h"

#if WITH_EDITOR

#include "PCGGraph.h"
#include "PCGGraphExecutionStateInterface.h"
#include "Subsystems/PCGSubsystem.h"

namespace PCGChangeTracker
{
	/** Have a CVar for dynamic tracking for runtime until we have a better solution to also have "culling" for local components and dynamic tracking. */
	static TAutoConsoleVariable<bool> CVarDisableDynamicTrackingForRuntimeGen(
		TEXT("pcg.RuntimeGeneration.DisableDynamicTracking"),
		false,
		TEXT("In Editor and with runtime gen, a change with one tracked element will refresh all the local components. If it is too resource intensive, it can be disabled."));
}

void FPCGChangeTracker::ResetCurrentExecution()
{
	PCG::TScopeLock Lock(CurrentExecutionDynamicTrackingLock);
	CurrentExecutionDynamicTracking.Empty();
	CurrentExecutionDynamicTrackingSettings.Empty();
}

TArray<FPCGSelectionKey> FPCGChangeTracker::GatherTrackingKeys() const
{
	TArray<FPCGSelectionKey> Keys;
	Keys.Reserve(StaticallyTrackedKeysToSettings.Num() + DynamicallyTrackedKeysToSettings.Num());
	for (const auto& It : StaticallyTrackedKeysToSettings)
	{
		Keys.Add(It.Key);
	}

	for (const auto& It : DynamicallyTrackedKeysToSettings)
	{
		Keys.Add(It.Key);
	}

	return Keys;
}

bool FPCGChangeTracker::IsKeyTrackedAndCulled(const FPCGSelectionKey& Key, bool& bOutIsCulled) const
{
	bool bIsTracked = false;

	bool bStaticallyCulled = true;
	bool bDynamicallyCulled = true;

	if (auto* It = StaticallyTrackedKeysToSettings.Find(Key))
	{
		bIsTracked = true;
		bStaticallyCulled = PCGSettings::IsKeyCulled(*It);
	}

	if (auto* It = DynamicallyTrackedKeysToSettings.Find(Key))
	{
		bIsTracked = true;
		bDynamicallyCulled = PCGSettings::IsKeyCulled(*It);
	}

	// If it is tracked statically and dynamically, we will cull only and only if both are culling.
	// Otherwise, it means that at least one key requires to always track, so bOutIsCulled needs to be False.
	bOutIsCulled = bIsTracked && bStaticallyCulled && bDynamicallyCulled;

	return bIsTracked;
}

bool FPCGChangeTracker::UpdateTrackingCache(const UPCGGraph* InGraph, TArray<FPCGSelectionKey>* OptionalChangedKeys)
{
	FPCGSelectionKeyToSettingsMap NewTrackedKeysToSettings;

	const int32 PreviousKeyCount = StaticallyTrackedKeysToSettings.Num();

	bool bHasChanged = false;

	if (InGraph)
	{
		bHasChanged = InGraph->UpdateTrackedSelectionKeys(StaticallyTrackedKeysToSettings, NewTrackedKeysToSettings, OptionalChangedKeys);
	}
	else
	{
		// Handle case were we no longer have a graph
		bHasChanged = StaticallyTrackedKeysToSettings.Num() > 0;
		if (OptionalChangedKeys)
		{
			OptionalChangedKeys->Reserve(StaticallyTrackedKeysToSettings.Num());
			for (const TPair<FPCGSelectionKey, TArray<FPCGSettingsAndCulling>>& It : StaticallyTrackedKeysToSettings)
			{
				OptionalChangedKeys->Add(It.Key);
			}
		}
	}

	StaticallyTrackedKeysToSettings = MoveTemp(NewTrackedKeysToSettings);

	return bHasChanged;
}

void FPCGChangeTracker::ForEachSettingTrackingKey(const FPCGSelectionKey& InKey, const TFunctionRef<void(const FPCGSelectionKey&, const FPCGSettingsAndCulling&)> InCallback) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGChangeTracker::ForEachSettingTrackingKey);

	auto FindAndApplyInMap = [&InKey, &InCallback](const FPCGSelectionKeyToSettingsMap& InMap)
	{
		if (const TArray<FPCGSettingsAndCulling>* StaticallyTrackedSettings = InMap.Find(InKey))
		{
			for (const FPCGSettingsAndCulling& SettingsAndCulling : *StaticallyTrackedSettings)
			{
				InCallback(InKey, SettingsAndCulling);
			}
		}
	};

	FindAndApplyInMap(StaticallyTrackedKeysToSettings);
	FindAndApplyInMap(DynamicallyTrackedKeysToSettings);
}

void FPCGChangeTracker::RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling)
{
	if (!InSettings)
	{
		return;
	}

	UE::TScopeLock Lock(CurrentExecutionDynamicTrackingLock);
	CurrentExecutionDynamicTrackingSettings.Add(InSettings);

	for (const TPair<FPCGSelectionKey, bool>& It : InDynamicKeysAndCulling)
	{
		// Make sure to not register null assets
		if (It.Key.Selection == EPCGActorSelection::ByPath && It.Key.ObjectPath.IsNull())
		{
			continue;
		}

		CurrentExecutionDynamicTracking.FindOrAdd(It.Key).Emplace(InSettings, It.Value);
	}
}

void FPCGChangeTracker::RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings)
{
	if (InKeysToSettings.IsEmpty())
	{
		return;
	}

	UE::TScopeLock Lock(CurrentExecutionDynamicTrackingLock);

	CurrentExecutionDynamicTracking.Append(InKeysToSettings);

	for (const auto& It : InKeysToSettings)
	{
		Algo::Transform(It.Value, CurrentExecutionDynamicTrackingSettings, [](const FPCGSettingsAndCulling& SettingsAndCulling) { return SettingsAndCulling.Key.Get(); });
	}
}

void FPCGChangeTracker::UpdateDynamicTracking(IPCGGraphExecutionSource* InExecutionSource)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGChangeTracker::UpdateDynamicTracking);
	check(InExecutionSource);

	UPCGSubsystem* Subsystem = Cast<UPCGSubsystem>(InExecutionSource->GetExecutionState().GetSubsystem());
	if (!Subsystem)
	{
		return;
	}

	IPCGGraphExecutionSource* OriginalSource = InExecutionSource->GetExecutionState().GetOriginalSource();
	if (!OriginalSource)
	{
		return;
	}

	const bool bIsLocal = InExecutionSource->GetExecutionState().IsLocalSource();
	const bool bIsRuntime = InExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem();

	if (bIsRuntime && PCGChangeTracker::CVarDisableDynamicTrackingForRuntimeGen.GetValueOnAnyThread())
	{
		return;
	}

	// If the component is local, we defer the tracking to the original component.
	// So move everything to the original (while making sure we are not duplicating keys/settings).
	// Since it can happen in parallel, we need to lock.
	if (bIsLocal && !bIsRuntime)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGChangeTracker::UpdateDynamicTracking::LocalComponent);
		
		OriginalSource->GetExecutionState().AddLocalSourceCurrentTrackingKeys(CurrentExecutionDynamicTracking, CurrentExecutionDynamicTrackingSettings);

		CurrentExecutionDynamicTracking.Empty();
		CurrentExecutionDynamicTrackingSettings.Empty();

		return;
	}

	TArray<FPCGSelectionKey> ChangedKeys;

	// Locking to make sure we never hit this multiple times.
	{
		UE::TScopeLock Lock(CurrentExecutionDynamicTrackingLock);

		// Go over all dynamic keys gathered during this execution.
		// If they are not already tracked, we need to register this key.
		// Otherwise, we need to gather all the settings that tracked this key that were not executed (because of caching).
		for (auto& It : CurrentExecutionDynamicTracking)
		{
			if (TArray<FPCGSettingsAndCulling>* AllSettingsAndCulling = DynamicallyTrackedKeysToSettings.Find(It.Key))
			{
				for (FPCGSettingsAndCulling& SettingsAndCulling : *AllSettingsAndCulling)
				{
					if (SettingsAndCulling.Key.Get() && !CurrentExecutionDynamicTrackingSettings.Contains(SettingsAndCulling.Key.Get()))
					{
						It.Value.AddUnique(std::move(SettingsAndCulling));
					}
				}
			}
			else
			{
				ChangedKeys.Add(It.Key);
			}
		}

		// Go over all already registered dynamic keys
		// If they are not in the current execution gathered keys, we check if they are associated with settings that 
		// were executed. If so, we re-add them to the current execution gathered keys.
		// If not, it means that the key is no longer tracked and should be unregistered.
		for (auto& It : DynamicallyTrackedKeysToSettings)
		{
			if (!CurrentExecutionDynamicTracking.Contains(It.Key))
			{
				TArray<FPCGSettingsAndCulling>* AllSettingsAndCulling = nullptr;

				for (FPCGSettingsAndCulling& SettingsAndCulling : It.Value)
				{
					if (SettingsAndCulling.Key.Get() && !CurrentExecutionDynamicTrackingSettings.Contains(SettingsAndCulling.Key.Get()))
					{
						if (!AllSettingsAndCulling)
						{
							AllSettingsAndCulling = &CurrentExecutionDynamicTracking.Add(It.Key);
						}

						check(AllSettingsAndCulling);
						// No need for Add Unique since they are already unique in the original map.
						AllSettingsAndCulling->Add(std::move(SettingsAndCulling));
					}
				}

				if (!AllSettingsAndCulling)
				{
					ChangedKeys.Add(It.Key);
				}
			}
		}

		DynamicallyTrackedKeysToSettings = std::move(CurrentExecutionDynamicTracking);
		CurrentExecutionDynamicTracking.Empty();
		CurrentExecutionDynamicTrackingSettings.Empty();
	}

	if (!ChangedKeys.IsEmpty())
	{
		if (bIsLocal)
		{
			OriginalSource->GetExecutionState().AddLocalSourceDynamicTrackingKeys(DynamicallyTrackedKeysToSettings);
		}

		Subsystem->UpdateTracking(OriginalSource, &ChangedKeys);
	}
}

void FPCGChangeTracker::AddLocalSourceDynamicTrackingKeys(const FPCGSelectionKeyToSettingsMap& InLocalSourceDynamicTrackingKeys)
{
	PCG::TScopeLock Lock(CurrentExecutionDynamicTrackingLock);
	for (const auto& It : InLocalSourceDynamicTrackingKeys)
	{
		TArray<FPCGSettingsAndCulling>& OriginalSettingsAndCulling = DynamicallyTrackedKeysToSettings.FindOrAdd(It.Key);
		for (FPCGSettingsAndCulling SettingsAndCulling : It.Value)
		{
			OriginalSettingsAndCulling.AddUnique(std::move(SettingsAndCulling));
		}
	}
}

void FPCGChangeTracker::AddLocalSourceCurrentTrackingKeys(const FPCGSelectionKeyToSettingsMap& InLocalSourceCurrentExecutionTrackingKeys, const TSet<const UPCGSettings*>& InLocalSourceCurrentExecutionTrackingSettings)
{
	PCG::TScopeLock Lock(CurrentExecutionDynamicTrackingLock);
	for (const auto& It : InLocalSourceCurrentExecutionTrackingKeys)
	{
		TArray<FPCGSettingsAndCulling>& OriginalSettingsAndCulling = CurrentExecutionDynamicTracking.FindOrAdd(It.Key);
		for (FPCGSettingsAndCulling SettingsAndCulling : It.Value)
		{
			OriginalSettingsAndCulling.AddUnique(std::move(SettingsAndCulling));
		}
	}
	CurrentExecutionDynamicTrackingSettings.Append(InLocalSourceCurrentExecutionTrackingSettings);
}

void FPCGChangeTracker::Serialize(FArchive& Ar)
{
	if (!Ar.IsCooking() && !Ar.IsLoadingFromCookedPackage())
	{
		Ar << DynamicallyTrackedKeysToSettings;
	}
}

#endif