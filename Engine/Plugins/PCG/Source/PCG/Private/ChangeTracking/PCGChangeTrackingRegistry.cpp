// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChangeTracking/PCGChangeTrackingRegistry.h"

#if WITH_EDITOR

void FPCGChangeTrackingRegistry::RegisterTrackerFactory(FName InName, FPCGCreateTrackerInstance InTrackerFactory)
{
	check(!TrackerFactories.Contains(InName));
	TrackerFactories.Add(InName, MoveTemp(InTrackerFactory));
}

void FPCGChangeTrackingRegistry::UnregisterTrackerFactory(FName InName)
{
	TrackerFactories.Remove(InName);
}

TArray<TUniquePtr<IPCGChangeTracker>> FPCGChangeTrackingRegistry::CreateChangeTrackers(FPCGTrackingManager* InManager) const
{
	TArray<TUniquePtr<IPCGChangeTracker>> ChangeTrackers;

	for (const auto& KeyValuePair : TrackerFactories)
	{
		ChangeTrackers.Add(KeyValuePair.Value.Execute(InManager));
	}

	return ChangeTrackers;
}

void FPCGChangeTrackingRegistry::RegisterHandlerFactory(FName InName, FPCGCreateHandlerInstance InHandlerFactory)
{
	check(!HandlerFactories.Contains(InName));
	HandlerFactories.Add(InName, MoveTemp(InHandlerFactory));
}

void FPCGChangeTrackingRegistry::UnregisterHandlerFactory(FName InName)
{
	HandlerFactories.Remove(InName);
}

TArray<TUniquePtr<IPCGChangeHandler>> FPCGChangeTrackingRegistry::CreateChangeHandlers(FPCGTrackingManager* InManager) const
{
	TArray<TUniquePtr<IPCGChangeHandler>> ChangeHandlers;

	for (const auto& KeyValuePair : HandlerFactories)
	{
		ChangeHandlers.Add(KeyValuePair.Value.Execute(InManager));
	}

	return ChangeHandlers;
}

#endif // WITH_EDITOR

TArray<IPCGGraphExecutionSource*> FPCGChangeTrackingRegistry::GetExecutionSourcesFromSelectionKey(const FPCGGetExecutionSourcesFromSelectionKeyParams& InParams) const
{
	const UObject* SelectedObject = InParams.SelectionKey.GetObjectFromPath();
	if (SelectedObject)
	{
		UClass* ObjectClass = SelectedObject->GetClass();
		while (ObjectClass)
		{
			if (const FPCGGetExecutionSourcesFromSelectionKey* Found = GetExecutionSourcesFromSelectionKeyEntries.Find(ObjectClass))
			{
				if (ensure(Found->IsBound()))
				{
					return Found->Execute(InParams);
				}
			}

			ObjectClass = ObjectClass->GetSuperClass();
		}
	}

	return {};
}

void FPCGChangeTrackingRegistry::RegisterGetExecutionSourcesFromSelectionKey(UClass* InSelectedObjectClass, FPCGGetExecutionSourcesFromSelectionKey InGetExecutionSourcesFromKey)
{
	check(!GetExecutionSourcesFromSelectionKeyEntries.Contains(InSelectedObjectClass));
	GetExecutionSourcesFromSelectionKeyEntries.Add(InSelectedObjectClass, MoveTemp(InGetExecutionSourcesFromKey));
}

void FPCGChangeTrackingRegistry::UnregisterGetExecutionSourcesFromSelectionKey(UClass* InSelectedObjectClass)
{
	GetExecutionSourcesFromSelectionKeyEntries.Remove(InSelectedObjectClass);
}
