// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PCGProceduralISMComponent.h"

#include "PCGComponent.h"
#include "PCGModule.h"
#include "PCGManagedResource.h"
#include "PCGSubsystem.h"
#include "Helpers/PCGHelpers.h"
#include "MeshSelectors/PCGISMDescriptor.h"

#include "InstanceDataSceneProxy.h"
#include "InstancedStaticMeshSceneProxyDesc.h"
#include "NaniteSceneProxy.h"
#include "PrimitiveComponentHelper.h"
#include "PrimitiveSceneDesc.h"
#include "PrimitiveSceneProxyDesc.h"
#include "SceneInterface.h"
#include "StaticMeshSceneProxyDesc.h"
#include "Algo/Compare.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGProceduralISMComponent)

UPCGManagedProceduralISMComponent* PCGManagedProceduralISMComponent::GetOrCreateManagedProceduralISMC(AActor* InTargetActor, UPCGComponent* InSourceComponent, uint64 InSettingsUID, FPCGProceduralISMCBuilderParameters&& InParams, FPCGContext* OptionalContext)
{
	check(InTargetActor && InSourceComponent);

	const UStaticMesh* StaticMesh = InParams.Descriptor.StaticMesh.Get();
	if (!ensure(StaticMesh))
	{
		return nullptr;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedISMC);

	FPCGProceduralISMComponentDescriptor& Descriptor = InParams.Descriptor;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedISMC::FindMatchingMISMC);
		UPCGManagedProceduralISMComponent* MatchingResource = nullptr;
		InSourceComponent->ForEachManagedResource([&MatchingResource, &InParams, &InTargetActor, &Descriptor, InSettingsUID](UPCGManagedResource* InResource)
		{
			// Early out if already found a match
			if (MatchingResource)
			{
				return;
			}

			if (UPCGManagedProceduralISMComponent* Resource = Cast<UPCGManagedProceduralISMComponent>(InResource))
			{
				// Note: Contrary to other managed resources, PISMCs can't be extended after being used so only allow each PISMC to be used once.
				if (!Resource->CanBeUsed() || !Resource->IsMarkedUnused() || Resource->GetSettingsUID() != InSettingsUID)
				{
					return;
				}

				if (UPCGProceduralISMComponent* ISMC = Resource->GetComponent())
				{
					if (IsValid(ISMC) &&
						ISMC->GetOwner() == InTargetActor &&
						Resource->GetDescriptor() == Descriptor &&
						Algo::Compare(ISMC->GetDefaultCustomPrimitiveData().Data, InParams.CustomPrimitiveData, [](const float LHS, const float RHS) { return FMath::IsNearlyEqual(LHS, RHS); }))
					{
						MatchingResource = Resource;
					}
				}
			}
		});

		if (MatchingResource)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedISMC::MarkAsUsed);
			MatchingResource->MarkAsUsed();

			if (UPCGProceduralISMComponent* ISMC = MatchingResource->GetComponent())
			{
				ISMC->ComponentTags = Descriptor.ComponentTags;
				ISMC->ComponentTags.AddUnique(PCGHelpers::DefaultPCGTag);
				ISMC->ComponentTags.AddUnique(InSourceComponent->GetFName());
			}

			return MatchingResource;
		}
	}

	// No matching ISM component found, let's create a new one
	FString ComponentName = TEXT("PISM_") + StaticMesh->GetName();

	const EObjectFlags ObjectFlags = RF_Transient;
	UPCGProceduralISMComponent* ISMC = NewObject<UPCGProceduralISMComponent>(InTargetActor, UPCGProceduralISMComponent::StaticClass(), MakeUniqueObjectName(InTargetActor, UPCGProceduralISMComponent::StaticClass(), FName(ComponentName)), ObjectFlags);
	ISMC->InitFromDescriptor(Descriptor);
	ISMC->PrecachePSOs();

	ISMC->RegisterComponent();
	InTargetActor->AddInstanceComponent(ISMC);

	if (!ISMC->AttachToComponent(InTargetActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, false)))
	{
		PCGLog::Component::LogComponentAttachmentFailedWarning(OptionalContext);
	}

	ISMC->SetCustomPrimitiveDataFloatArray(0, InParams.CustomPrimitiveData);

	// Create managed resource on source component
	UPCGManagedProceduralISMComponent* Resource = NewObject<UPCGManagedProceduralISMComponent>(InSourceComponent);
	Resource->SetComponent(ISMC);
	Resource->SetDescriptor(MoveTemp(Descriptor));
	if (InTargetActor->GetRootComponent())
	{
		Resource->SetRootLocation(InTargetActor->GetRootComponent()->GetComponentLocation());
	}

	Resource->SetSettingsUID(InSettingsUID);
	InSourceComponent->AddToManagedResources(Resource);

	ISMC->ComponentTags.AddUnique(PCGHelpers::DefaultPCGTag);
	ISMC->ComponentTags.AddUnique(InSourceComponent->GetFName());

	return Resource;
}

UPCGProceduralISMComponent::UPCGProceduralISMComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Disable unsupported rendering features (currently require instance data on CPU).
	bAffectDynamicIndirectLighting = false;
	bAffectDistanceFieldLighting = false;

	BodyInstance.bSimulatePhysics = false;
	SetGenerateOverlapEvents(false);

	bEnableVertexColorMeshPainting = false;

	bNavigationRelevant = false;
	bCanEverAffectNavigation = false;
	
	bEnableAutoLODGeneration = false;

	bIsEditorOnly = true;

#if STATS
	{
		UObject const* StatObject = this->AdditionalStatObject();
		if (!StatObject)
		{
			StatObject = this;
		}
		StatId = StatObject->GetStatID(true);
	}
#endif
}

void UPCGProceduralISMComponent::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (LocalBounds.IsValid)
	{
		WorldBounds = LocalBounds.TransformBy(GetComponentTransform());
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR
void UPCGProceduralISMComponent::OnRegister()
{
	Super::OnRegister();

	USceneComponent::MarkRenderStateDirtyEvent.RemoveAll(this);
	USceneComponent::MarkRenderStateDirtyEvent.AddUObject(this, &UPCGProceduralISMComponent::OnRenderStateDirty);
}

void UPCGProceduralISMComponent::OnUnregister()
{
	USceneComponent::MarkRenderStateDirtyEvent.RemoveAll(this);

	Super::OnUnregister();
}
#endif

void UPCGProceduralISMComponent::InitFromDescriptor(const FPCGProceduralISMComponentDescriptor& InDescriptor)
{
	SetStaticMesh(InDescriptor.StaticMesh.Get());
	OverrideMaterials = InDescriptor.OverrideMaterials;
	OverlayMaterial = InDescriptor.OverlayMaterial;
	RuntimeVirtualTextures = InDescriptor.RuntimeVirtualTextures;
	SetNumInstances(InDescriptor.NumInstances);
	SetNumCustomDataFloats(InDescriptor.NumCustomFloats);
	SetBounds(InDescriptor.WorldBounds);
	SetMinDrawDistance(InDescriptor.InstanceMinDrawDistance);
	SetCullDistances(InDescriptor.InstanceStartCullDistance, InDescriptor.InstanceEndCullDistance);
	ComponentTags = InDescriptor.ComponentTags;
	Mobility = InDescriptor.Mobility;
	VirtualTextureRenderPassType = InDescriptor.VirtualTextureRenderPassType;
	LightingChannels = InDescriptor.LightingChannels;
	CustomDepthStencilWriteMask = InDescriptor.CustomDepthStencilWriteMask;
	VirtualTextureCullMips = InDescriptor.VirtualTextureCullMips;
	TranslucencySortPriority = InDescriptor.TranslucencySortPriority;
	CustomDepthStencilValue = InDescriptor.CustomDepthStencilValue;
	CastShadow = InDescriptor.bCastShadow;
	bEmissiveLightSource = InDescriptor.bEmissiveLightSource;
	bCastDynamicShadow = InDescriptor.bCastDynamicShadow;
	bCastStaticShadow = InDescriptor.bCastStaticShadow;
	bCastContactShadow = InDescriptor.bCastContactShadow;
	bCastShadowAsTwoSided = InDescriptor.bCastShadowAsTwoSided;
	bCastHiddenShadow = InDescriptor.bCastHiddenShadow;
	bReceivesDecals = InDescriptor.bReceivesDecals;
	bUseAsOccluder = InDescriptor.bUseAsOccluder;
	bRenderCustomDepth = InDescriptor.bRenderCustomDepth;
	bEvaluateWorldPositionOffset = InDescriptor.bEvaluateWorldPositionOffset;
	bReverseCulling = InDescriptor.bReverseCulling;
	WorldPositionOffsetDisableDistance = InDescriptor.WorldPositionOffsetDisableDistance;
	ShadowCacheInvalidationBehavior = InDescriptor.ShadowCacheInvalidationBehavior;
	DetailMode = InDescriptor.DetailMode;
	bVisibleInRayTracing = InDescriptor.bVisibleInRayTracing;
	bVisibleInReflections = InDescriptor.bVisibleInReflections;
	RayTracingGroupId = InDescriptor.RayTracingGroupId;
	RayTracingGroupCullingPriority = InDescriptor.RayTracingGroupCullingPriority;
}

FPrimitiveSceneProxy* UPCGProceduralISMComponent::CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	LLM_SCOPE(ELLMTag::InstancedMesh);

	if (!ensure(GetWorld()) || !ensure(GetWorld()->Scene))
	{
		return nullptr;
	}
	
	if (!UseGPUScene(GetWorld()->Scene->GetShaderPlatform()))
	{
		UE_LOGF(LogPCG, Warning, "PCGProceduralISMComponent depends on GPUScene functionality which is not available on this platform, component will not render.");
		return nullptr;
	}

	if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogPCG, Verbose, "Skipping CreateSceneProxy for PCGProceduralISMComponent %ls (PCGProceduralISMComponent PSOs are still compiling)", *GetFullName());
		return nullptr;
	}

	FInstancedStaticMeshSceneProxyDesc Desc;
	GetSceneProxyDesc(Desc);

	if (bCreateNanite)
	{
		return ::new Nanite::FSceneProxy(NaniteMaterials, Desc);
	}
	else
	{
		return ::new FInstancedStaticMeshSceneProxy(Desc, GetWorld()->GetFeatureLevel());
	}
}

FPrimitiveSceneProxy* UPCGProceduralISMComponent::CreateSceneProxy()
{
	if (NumInstances <= 0)
	{
		return nullptr;
	}

	ValidateComponentSetup();

	return Super::CreateSceneProxy();
}

FMatrix UPCGProceduralISMComponent::GetRenderMatrix() const
{
	// Apply the translated space to the render matrix.
	return GetComponentTransform().ToMatrixWithScale();
}

FBoxSphereBounds UPCGProceduralISMComponent::CalcBounds(const FTransform& BoundTransform) const
{
	return WorldBounds.InverseTransformBy(GetComponentTransform().Inverse() * BoundTransform);
}

#if WITH_EDITOR
FBox UPCGProceduralISMComponent::GetStreamingBounds() const
{
	return FBox::BuildAABB(Bounds.Origin, Bounds.BoxExtent);
}
#endif

bool UPCGProceduralISMComponent::GetMaterialStreamingData(int32 MaterialIndex, FPrimitiveMaterialInfo& MaterialData) const
{
	// Same thing as StaticMesh but we take the full bounds to cover the instances.
	if (GetStaticMesh())
	{
		MaterialData.Material = GetMaterial(MaterialIndex);
		MaterialData.UVChannelData = GetStaticMesh()->GetUVChannelData(MaterialIndex);
		MaterialData.PackedRelativeBox = PackedRelativeBox_Identity;
	}
	return MaterialData.IsValid();
}

bool UPCGProceduralISMComponent::BuildTextureStreamingDataImpl(ETextureStreamingBuildType BuildType, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, TSet<FGuid>& DependentResources, bool& bOutSupportsBuildTextureStreamingData)
{
#if WITH_EDITORONLY_DATA // Only rebuild the data in editor 
	if (GetNumInstances() > 0)
	{
		return Super::BuildTextureStreamingDataImpl(BuildType, QualityLevel, FeatureLevel, DependentResources, bOutSupportsBuildTextureStreamingData);
	}
#endif
	return true;
}

void UPCGProceduralISMComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	// Don't only look the instance count but also if the bound is valid, as derived classes might not set PerInstanceSMData.
	if (GetNumInstances() > 0 || Bounds.SphereRadius > 0)
	{
		return Super::GetStreamingRenderAssetInfo(LevelContext, OutStreamingRenderAssets);
	}
}

void UPCGProceduralISMComponent::SetCullDistances(int32 InStartCullDistance, int32 InEndCullDistance)
{
	if (InstanceStartCullDistance != InStartCullDistance || InstanceEndCullDistance != InEndCullDistance)
	{
		InstanceStartCullDistance = InStartCullDistance;
		InstanceEndCullDistance = InEndCullDistance;

		if (GetScene() && SceneProxy)
		{
			GetScene()->UpdateInstanceCullDistance(this, InStartCullDistance, InEndCullDistance);
		}
	}
}

void UPCGProceduralISMComponent::SetMinDrawDistance(int32 InMinDrawDistance)
{
	if (InstanceMinDrawDistance != InMinDrawDistance)
	{
		InstanceMinDrawDistance = InMinDrawDistance;

		if (GetScene() && SceneProxy)
		{
			GetScene()->UpdatePrimitiveDrawDistance(this, InMinDrawDistance, /*MaxDrawDistance=*/0, /*VirtualTextureMaxDrawDistance=*/0);
		}
	}
}

void UPCGProceduralISMComponent::SetSpatialHashes(TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem>&& InSpatialHashes)
{
	// A simple ordered compare of the hashes used as this will be set infrequently, and hash order impacts what cell merging in the renderer.
	if (SpatialHashes != InSpatialHashes)
	{
		SpatialHashes = MoveTemp(InSpatialHashes);
#if WITH_EDITOR
		TGuardValue<bool> SelfDirtyScope(bIsSelfDirtying, true);
#endif
		MarkRenderStateDirty();
	}
}

void UPCGProceduralISMComponent::GetSceneProxyDesc(FInstancedStaticMeshSceneProxyDesc& OutSceneProxyDesc) const
{
	const UWorld* World = GetWorld();
	const bool bHasPerInstanceRandom = ensure(World && World->Scene) && FPrimitiveComponentHelper::GetUsedMaterialPropertyDesc(*this, World->Scene->GetShaderPlatform()).bAnyMaterialHasPerInstanceRandom;

	FInstanceSceneDataBuffers InstanceSceneDataBuffers(/*InbInstanceDataIsGPUOnly=*/true);
	{
		FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
		FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);

		InstanceSceneDataBuffers.SetPrimitiveLocalToWorld(GetRenderMatrix(), AccessTag);

		ProxyData.NumInstancesGPUOnly = GetNumInstances();
		ProxyData.NumCustomDataFloats = GetNumCustomDataFloats();
		ProxyData.InstanceLocalBounds.SetNum(1);
		ProxyData.InstanceLocalBounds[0] = GetStaticMesh()->GetBounds();

		ProxyData.Flags.bHasPerInstanceCustomData = ProxyData.NumCustomDataFloats > 0;
		ProxyData.Flags.bHasPerInstanceRandom = bHasPerInstanceRandom;

		if (!SpatialHashes.IsEmpty())
		{
			InstanceSceneDataBuffers.SetImmutable(FInstanceSceneDataImmutable(SpatialHashes), AccessTag);
		}

		InstanceSceneDataBuffers.EndWriteAccess(AccessTag);
		InstanceSceneDataBuffers.ValidateData();
	}

	OutSceneProxyDesc.InitializeFromStaticMeshComponent(this);
	OutSceneProxyDesc.InstanceDataSceneProxy = MakeShared<FInstanceDataSceneProxy, ESPMode::ThreadSafe>(MoveTemp(InstanceSceneDataBuffers));
	OutSceneProxyDesc.InstanceMinDrawDistance = InstanceMinDrawDistance;
	OutSceneProxyDesc.InstanceStartCullDistance = InstanceStartCullDistance;
	OutSceneProxyDesc.InstanceEndCullDistance = InstanceEndCullDistance;
	OutSceneProxyDesc.bUseGpuLodSelection = true;
}

void UPCGProceduralISMComponent::ValidateComponentSetup()
{
	if (bAffectDynamicIndirectLighting)
	{
		UE_LOGF(LogPCG, Warning, "Affecting indirect lighting is not currently supported by PCGProceduralISMComponent, disabling.");
		bAffectDynamicIndirectLighting = false;
	}

	if (bAffectDistanceFieldLighting)
	{
		UE_LOGF(LogPCG, Warning, "Affecting distance field lighting is not currently supported by PCGProceduralISMComponent, disabling.");
		bAffectDistanceFieldLighting = false;
	}
}

#if WITH_EDITOR
void UPCGProceduralISMComponent::OnRenderStateDirty(UActorComponent& InComponent)
{
	// Currently, there is no explicit persistence of instance data in the GPU scene. When this component is dirtied, the instance data is cleared.

	if (&InComponent != this)
	{
		return;
	}

	// Skip when we triggered this dirty ourselves, otherwise refreshing the graph would loop forever.
	if (bIsSelfDirtying)
	{
		return;
	}

	UPCGSubsystem* Subsystem = GetOwner() ? UPCGSubsystem::GetInstance(GetOwner()->GetWorld()) : nullptr;

	if (!Subsystem)
	{
		return;
	}

	// Helper that returns true if the given PCG component is managing this PISMC.
	auto PCGComponentManagesThisPISMC = [this](UPCGComponent* InComponent) -> bool
	{
		bool bManagesThis = false;

		if (InComponent->AreProceduralInstancesInUse() && InComponent->AreManagedResourcesAccessible())
		{
			InComponent->ForEachManagedResource([this, InComponent, &bManagesThis](UPCGManagedResource* InResource)
			{
				if (!bManagesThis)
				{
					const UPCGManagedProceduralISMComponent* PISMC = Cast<UPCGManagedProceduralISMComponent>(InResource);
					if (PISMC && PISMC->GetComponent() == this)
					{
						bManagesThis = true;
					}
				}
			});
		}

		return bManagesThis;
	};

	Subsystem->RefreshAllComponentsFiltered(
		[this, Subsystem, &PCGComponentManagesThisPISMC](UPCGComponent* InComponent) -> bool
		{
			// If the original component manages this PISMC, request a refresh of it and we're done. If it has local components they will also be refreshed.
			if (PCGComponentManagesThisPISMC(InComponent))
			{
				return true;
			}

			// A local component of the original component might manage this, so check those.
			bool bManagesThis = false;

			if (InComponent->bIsComponentPartitioned)
			{
				Subsystem->ForAllRegisteredLocalComponents(
					InComponent,
					[this, &bManagesThis, &PCGComponentManagesThisPISMC](UPCGComponent* InComponent)
					{
						if (!bManagesThis)
						{
							bManagesThis = PCGComponentManagesThisPISMC(InComponent);
						}
					});
			}

			return bManagesThis;
		},
		EPCGChangeType::Structural);
}
#endif // WITH_EDITOR

void UPCGProceduralISMComponent::SetNumInstances(int32 InNumInstances)
{
	if (NumInstances != InNumInstances)
	{
		NumInstances = InNumInstances;
#if WITH_EDITOR
		TGuardValue<bool> SelfDirtyScope(bIsSelfDirtying, true);
#endif
		MarkRenderStateDirty();
	}
}

void UPCGProceduralISMComponent::SetNumCustomDataFloats(int32 InNumCustomDataFloats)
{
	if (FMath::Max(InNumCustomDataFloats, 0) != NumCustomDataFloats)
	{
		NumCustomDataFloats = FMath::Max(InNumCustomDataFloats, 0);
#if WITH_EDITOR
		TGuardValue<bool> SelfDirtyScope(bIsSelfDirtying, true);
#endif
		MarkRenderStateDirty();
	}
}

FBox UPCGProceduralISMComponent::GetLocalBounds()
{
	return WorldBounds.InverseTransformBy(GetComponentTransform());
}

void UPCGProceduralISMComponent::GetBounds(FBox& OutWorldBounds)
{
	OutWorldBounds = WorldBounds;
}

void UPCGProceduralISMComponent::SetBounds(const FBox& InWorldBounds)
{
	if (InWorldBounds != WorldBounds)
	{
		WorldBounds = InWorldBounds;
#if WITH_EDITOR
		TGuardValue<bool> SelfDirtyScope(bIsSelfDirtying, true);
#endif
		MarkRenderStateDirty();
	}
}

#if WITH_EDITOR
void UPCGProceduralISMComponent::PostEditUndo()
{
	Super::PostEditUndo();

	MarkRenderStateDirty();
}
#endif

void UPCGManagedProceduralISMComponent::PostLoad()
{
	Super::PostLoad();

	// Cache raw ptr
	GetComponent();
}

void UPCGManagedProceduralISMComponent::SetDescriptor(const FPCGProceduralISMComponentDescriptor& InDescriptor)
{
	Descriptor = InDescriptor;
}

void UPCGManagedProceduralISMComponent::SetDescriptor(FPCGProceduralISMComponentDescriptor&& InDescriptor)
{
	Descriptor = MoveTemp(InDescriptor);
}

bool UPCGManagedProceduralISMComponent::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	if (Super::ReleaseIfUnused(OutActorsToDelete) || !GetComponent())
	{
		return true;
	}
	else if (GetComponent()->GetNumInstances() == 0)
	{
		GeneratedComponent->DestroyComponent();
		ForgetComponent();
		return true;
	}
	else
	{
		return false;
	}
}

void UPCGManagedProceduralISMComponent::ResetComponent()
{
	if (UPCGProceduralISMComponent* ISMC = GetComponent())
	{
		ISMC->UpdateBounds();
	}
}

void UPCGManagedProceduralISMComponent::MarkAsUsed()
{
	const bool bWasMarkedUnused = bIsMarkedUnused;
	Super::MarkAsUsed();

	if (!bWasMarkedUnused)
	{
		return;
	}

	if (UPCGProceduralISMComponent* ISMC = GetComponent())
	{
		// Keep track of the current root location so if we reuse this later we are able to update this appropriately
		if (USceneComponent* RootComponent = ISMC->GetAttachmentRoot())
		{
			bHasRootLocation = true;
			RootLocation = RootComponent->GetComponentLocation();
		}
		else
		{
			bHasRootLocation = false;
			RootLocation = FVector::ZeroVector;
		}

		// Reset the rotation/scale to be identity otherwise if the root component transform has changed, the final transform will be wrong
		// Since this is technically 'moving' the ISM, we need to unregister it before moving otherwise we could get a warning that we're moving a component with static mobility
		ISMC->UnregisterComponent();
		ISMC->SetWorldTransform(FTransform(FQuat::Identity, RootLocation, FVector::OneVector));
		ISMC->RegisterComponent();
	}
}

void UPCGManagedProceduralISMComponent::MarkAsReused()
{
	Super::MarkAsReused();

	if (UPCGProceduralISMComponent* ISMC = GetComponent())
	{
		// Reset the rotation/scale to be identity otherwise if the root component transform has changed, the final transform will be wrong
		FVector TentativeRootLocation = RootLocation;

		if (!bHasRootLocation)
		{
			if (USceneComponent* RootComponent = ISMC->GetAttachmentRoot())
			{
				TentativeRootLocation = RootComponent->GetComponentLocation();
			}
		}

		// Since this is technically 'moving' the ISM, we need to unregister it before moving otherwise we could get a warning that we're moving a component with static mobility
		ISMC->UnregisterComponent();
		ISMC->SetWorldTransform(FTransform(FQuat::Identity, TentativeRootLocation, FVector::OneVector));
		ISMC->RegisterComponent();
	}
}

void UPCGManagedProceduralISMComponent::SetRootLocation(const FVector& InRootLocation)
{
	bHasRootLocation = true;
	RootLocation = InRootLocation;
}

UPCGProceduralISMComponent* UPCGManagedProceduralISMComponent::GetComponent() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGManagedProceduralISMComponent::GetComponent);
	return Cast<UPCGProceduralISMComponent>(GeneratedComponent.Get());
}

void UPCGManagedProceduralISMComponent::SetComponent(UPCGProceduralISMComponent* InComponent)
{
	GeneratedComponent = InComponent;
}
