// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/StaticMeshComponent.h"
#include "PCGManagedResource.h"

#include "Components/PCGProceduralISMComponentDescriptor.h"

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "InstanceDataSceneProxy.h"
#include "StaticMeshResources.h"
#include "StaticMeshSceneProxy.h"
#include "Algo/Compare.h"
#include "Misc/Guid.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"

#include "PCGProceduralISMComponent.generated.h"

#define UE_API PCG_API

class FPrimitiveSceneProxy;
class UPCGComponent;
class UPCGManagedProceduralISMComponent;
struct FInstancedStaticMeshSceneProxyDesc;
struct FPCGContext;
struct FPrimitiveSceneDesc;
struct FPrimitiveSceneProxyDesc;

/** Instanced primitive that is not backed by CPU instance data. Instance data must be written direct to the GPU scene. */
UCLASS(ClassGroup = Rendering, meta = (BlueprintSpawnableComponent), Blueprintable, MinimalAPI, hideCategories = (Collision, Cooking, HLOD, MeshPainting, Navigation, Physics, RayTracing))
class UPCGProceduralISMComponent : public UStaticMeshComponent
{
	GENERATED_UCLASS_BODY()

public:
	//~Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~End UObject interface

#if WITH_EDITOR
	//~Begin UActorComponent Interface
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
	//~End UActorComponent Interface
#endif

	void InitFromDescriptor(const FPCGProceduralISMComponentDescriptor& InDescriptor);

	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC")
	int32 GetNumInstances() const { return NumInstances; }

	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC")
	void SetNumInstances(int32 InNumInstances);

	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC")
	int32 GetNumCustomDataFloats() const { return NumCustomDataFloats; }

	/** Update number of custom data entries per instance. */
	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC")
	void SetNumCustomDataFloats(int32 InNumCustomDataFloats);

	/** Get local bounds that were previously set for all instances of this component. */
	FBox GetLocalBounds();

	/** Get world bounds for all instances of this component. */
	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC")
	void GetBounds(FBox& OutWorldBounds);

	/** Set world bounds for all instances of this component (bounds are not computed automatically from instances and must be set using this function). */
	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC")
	void SetBounds(const FBox& InWorldBounds);

	/** Clear all instances being rendered by this component. */
	UFUNCTION(BlueprintCallable, Category="Components|PCGProceduralISMC")
	void ClearInstances() { SetNumInstances(0); }

	/** Sets the fading start and culling end distances for this component. */
	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC")
	void SetCullDistances(int32 InStartCullDistance, int32 InEndCullDistance);

	/** Gets the fading start and culling end distances for this component. */
	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC")
	void GetCullDistances(int32& OutStartCullDistance, int32& OutEndCullDistance) const { OutStartCullDistance = InstanceStartCullDistance; OutEndCullDistance = InstanceEndCullDistance; }

	/** Sets the minimum distance at which instances will render for this component. */
	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC")
	void SetMinDrawDistance(int32 InMinDrawDistance);

	/** Gets the minimum distance at which instances will render for this component. */
	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC")
	void GetMinDrawDistance(int32& OutMinDrawDistance) const { OutMinDrawDistance = InstanceMinDrawDistance; }

	/** Set spatial hashes used by the renderer for early culling of subranges of instances. */
	void SetSpatialHashes(TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem>&& InSpatialHashes);

	//~ Begin UActorComponent Interface
	bool ShouldCreatePhysicsState() const override { return false; }
	bool IsHLODRelevant() const override { return false; }
	//~ End UActorComponent Interface

	//~ Begin UPrimitiveComponent Interface
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual FMatrix GetRenderMatrix() const override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& BoundTransform) const override;
	bool SupportsStaticLighting() const override { return false; }
#if WITH_EDITOR
	virtual FBox GetStreamingBounds() const override;
#endif
	virtual bool BuildTextureStreamingDataImpl(ETextureStreamingBuildType BuildType, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, TSet<FGuid>& DependentResources, bool& bOutSupportsBuildTextureStreamingData) override;
	virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
	//~ End UPrimitiveComponent Interface

	//~ Begin INavRelevantInterface Interface.
	bool IsNavigationRelevant() const override { return false; }
	//~ End INavRelevantInterface Interface.

	//~ Begin UStaticMeshComponentInterface
public:
	virtual bool GetMaterialStreamingData(int32 MaterialIndex, FPrimitiveMaterialInfo& MaterialData) const override;

protected:
	virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite) override;
	//~ End UStaticMeshComponentInterface

	//~ Begin UObject Interface
#if WITH_EDITOR
protected:
	virtual void PostEditUndo() override;
#endif
	//~ End UObject Interface

public:
	void GetSceneProxyDesc(FInstancedStaticMeshSceneProxyDesc& OutSceneProxyDesc) const;

protected:
	/** Validate component setup, disable features that are not supported (typically those that require instance transforms on CPU). */
	virtual void ValidateComponentSetup();

#if WITH_EDITOR
private:
	/** Executed when the render state of any actor component is dirtied. */
	void OnRenderStateDirty(UActorComponent& InComponent);

	bool bIsSelfDirtying = false;
#endif

protected:
	UPROPERTY(EditAnywhere, Category = Instances)
	int32 NumInstances = 0;

	/** Defines the number of floats that will be available per instance for custom data */
	UPROPERTY(EditAnywhere, Category = Instances)
	int32 NumCustomDataFloats;

	static inline const float DefaultBoundsExtent = 500.0f;

	UPROPERTY(EditAnywhere, Category = Instances)
	FBox WorldBounds = FBox(-FVector::OneVector * DefaultBoundsExtent, FVector::OneVector * DefaultBoundsExtent);

	/** Distance from camera at which each instance begins to draw. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Culling)
	int32 InstanceMinDrawDistance;

	/** Distance from camera at which each instance begins to fade out. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Culling)
	int32 InstanceStartCullDistance;

	/** Distance from camera at which each instance completely fades out. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Culling)
	int32 InstanceEndCullDistance;

#if STATS
	/** Used for dynamic stats */
	TStatId StatId;
#endif

	UE_DEPRECATED(5.6, "Use WorldBounds instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use WorldBounds instead."))
	FBox LocalBounds;

	UE_DEPRECATED(5.6, "Use SetBounds to set world-space bounds instead.")
	UFUNCTION(BlueprintCallable, Category = "Components|PCGProceduralISMC", meta = (DeprecatedProperty, DeprecationMessage = "Use SetBounds to set world-space bounds instead."))
	void SetLocalBounds(const FBox& InWorldBounds) {}

	/** Spatial hashes provide bounds for sub-ranges of instances within this primitive, enabling early CPU culling. */
	TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem> SpatialHashes;
};

struct FPCGProceduralISMCBuilderParameters
{
	friend inline uint32 GetTypeHash(const FPCGProceduralISMCBuilderParameters& Key)
	{
		uint32 Hash = GetTypeHash(Key.Descriptor);
		Hash = HashCombine(Hash, Key.bAllowDescriptorChanges ? 2 : 1);
		Hash = GetArrayHash(Key.CustomPrimitiveData.GetData(), Key.CustomPrimitiveData.Num(), Hash);
		return Hash;
	}

	inline bool operator==(const FPCGProceduralISMCBuilderParameters& Other) const
	{
		return Descriptor == Other.Descriptor
		&& bAllowDescriptorChanges == Other.bAllowDescriptorChanges
		&& Algo::Compare(CustomPrimitiveData, Other.CustomPrimitiveData);
	}

	FPCGProceduralISMComponentDescriptor Descriptor;
	TConstArrayView<float> CustomPrimitiveData;

	bool bAllowDescriptorChanges = true;
};

UCLASS(MinimalAPI, BlueprintType)
class UPCGManagedProceduralISMComponent : public UPCGManagedComponent
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGManagedResource interface
	UE_API virtual bool ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	//~End UPCGManagedResource interface

	//~Begin UPCGManagedComponents interface
	UE_API virtual void ResetComponent() override;
	virtual bool SupportsComponentReset() const override { return true; }
	UE_API virtual void MarkAsUsed() override;
	UE_API virtual void MarkAsReused() override;
	//~End UPCGManagedComponents interface

	UE_API UPCGProceduralISMComponent* GetComponent() const;
	UE_API void SetComponent(UPCGProceduralISMComponent* InComponent);

	UE_API void SetDescriptor(const FPCGProceduralISMComponentDescriptor& InDescriptor);
	UE_API void SetDescriptor(FPCGProceduralISMComponentDescriptor&& InDescriptor);
	
	const FPCGProceduralISMComponentDescriptor& GetDescriptor() const { return Descriptor; }

	UE_API void SetRootLocation(const FVector& InRootLocation);

	uint64 GetSettingsUID() const { return SettingsUID; }
	void SetSettingsUID(uint64 InSettingsUID) { SettingsUID = InSettingsUID; }

protected:
	UPROPERTY()
	FPCGProceduralISMComponentDescriptor Descriptor;

	UPROPERTY()
	bool bHasRootLocation = false;

	UPROPERTY()
	FVector RootLocation = FVector::ZeroVector;

	UPROPERTY(Transient)
	uint64 SettingsUID = -1; // purposefully a value that will never happen in data
};

namespace PCGManagedProceduralISMComponent
{
	UPCGManagedProceduralISMComponent* GetOrCreateManagedProceduralISMC(AActor* InTargetActor, UPCGComponent* InSourceComponent, uint64 InSettingsUID, FPCGProceduralISMCBuilderParameters&& InParams, FPCGContext* OptionalContext = nullptr);
}

#undef UE_API
