// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PCGProceduralISMComponentDescriptor.h"

#include "MeshSelectors/PCGISMDescriptor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGProceduralISMComponentDescriptor)

FPCGProceduralISMComponentDescriptor& FPCGProceduralISMComponentDescriptor::operator=(const FPCGSoftISMComponentDescriptor& Other)
{
	FProceduralISMComponentDescriptor::operator=(static_cast<const FSoftISMComponentDescriptor&>(Other));

	ComponentTags = Other.ComponentTags;

	return *this;
}

bool FPCGProceduralISMComponentDescriptor::operator==(const FPCGProceduralISMComponentDescriptor& Other) const
{
	return FProceduralISMComponentDescriptor::operator==(Other)
		&& ComponentTags == Other.ComponentTags;
}
