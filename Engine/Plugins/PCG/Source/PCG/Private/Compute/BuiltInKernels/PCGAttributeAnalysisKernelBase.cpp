// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/BuiltInKernels/PCGAttributeAnalysisKernelBase.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include "ShaderCompilerCore.h"
#include "Algo/MaxElement.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeAnalysisKernelBase)

#define LOCTEXT_NAMESPACE "PCGAttributeAnalysisKernelBase"

bool UPCGAttributeAnalysisKernelBase::IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGAttributeAnalysisKernelBase::IsKernelDataValid);

	if (!Super::IsKernelDataValid(InDataBinding, InContext))
	{
		return false;
	}

	if (InDataBinding)
	{
		const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InDataBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultInputLabel, /*bIsInputPin=*/true);

		if (!ensure(InputDataDesc))
		{
			return false;
		}

		if (!InputDataDesc->GetDataDescriptions().IsEmpty())
		{
			FPCGKernelAttributeDesc AttributeDesc;
			bool bConflictingTypesFound = false;
			bool bPresentOnAllData = false;
			InputDataDesc->GetAttributeDesc(AttributeName, AttributeDesc, bConflictingTypesFound, bPresentOnAllData);

			if (!bPresentOnAllData)
			{
				if (!InputDataDesc->GetDataDescriptions().IsEmpty())
				{
					PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), FText::Format(
						LOCTEXT("AttributeMissing", "Count attribute '{0}' not found, this attribute must be present on all input data, and be of type String Key."),
						FText::FromName(AttributeName)));
				}

				return false;
			}

			if (bConflictingTypesFound)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), FText::Format(
					LOCTEXT("AttributeHasMultipleTypes", "Count attribute '{0}' found with multiple types in input data, all attributes must be of type String Key."),
					FText::FromName(AttributeName)));

				return false;
			}

			const bool bRequireStringKeyAttribute = MaxNumUniqueValuesSource == EPCGMaxNumUniqueValuesSource::StringKeyValues;
			const bool bTypeSupported = (!bRequireStringKeyAttribute && AttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::Int)
				|| AttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::StringKey;

			if (!bTypeSupported)
			{
				// Attribute value counting only currently supported for attributes of type Int or StringKey.
				PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), FText::Format(
					LOCTEXT("AttributeTypeInvalid", "Cannot count values for attribute '{0}'. Attribute type must either be Int, or can be String Key when counting via String Key."),
					FText::FromName(AttributeName)));

				return false;
			}

			if (MaxNumUniqueValuesSource == EPCGMaxNumUniqueValuesSource::UniqueValueTable)
			{
				const TSharedPtr<const FPCGDataCollectionDesc> InputTableDesc = InDataBinding->GetCachedKernelPinDataDesc(this, PCGAttributeAnalysisKernelConstants::UniqueValueTablePinLabel, /*bIsInputPin=*/true);
				if (!InputTableDesc || InputTableDesc->ComputeTotalElementCount() == 0)
				{
					PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), FText::Format(
						LOCTEXT("NoInputDataToDriveUniqueValueCount", "No input data provided on kernel pin '{0}' to drive unique value count calculation."),
						FText::FromName(PCGAttributeAnalysisKernelConstants::UniqueValueTablePinLabel)));

					return false;
				}
			}
			else if (MaxNumUniqueValuesSource == EPCGMaxNumUniqueValuesSource::ExplicitMaxValue && MaxIntValue <= 0)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), FText::Format(
					LOCTEXT("InvalidExplicitAttributeValueCount", "Explicit maximum value count {0} is invalid, must be greater than 0."),
					FText::AsNumber(MaxIntValue)));

				return false;
			}
		}
	}

	return true;
}

int32 UPCGAttributeAnalysisKernelBase::ComputeNumCountersRequired(UPCGDataBinding* InBinding, TSharedPtr<const FPCGDataCollectionDesc> InInputDesc) const
{
	return GetNumValues(InBinding, InInputDesc);
}

int32 UPCGAttributeAnalysisKernelBase::GetNumValues(UPCGDataBinding* InBinding, TSharedPtr<const FPCGDataCollectionDesc> InInputDesc) const
{
	int32 MaxNumUniqueValues = INDEX_NONE;

	if (MaxNumUniqueValuesSource == EPCGMaxNumUniqueValuesSource::StringKeyValues)
	{
		// If counted attribute is a stringkey, use max string key value as num elements.
		const int32 StringKeyAttributeId = InBinding->GetAttributeId(AttributeName, EPCGKernelAttributeType::StringKey);

		if (StringKeyAttributeId == INDEX_NONE)
		{
			return 0;
		}

		TArray<int32> UniqueStringKeyValues;
		check(InInputDesc);
		InInputDesc->GetUniqueStringKeyValues(StringKeyAttributeId, UniqueStringKeyValues);

		// If the highest string key value is 3, we'll allocate 4 counters. String key value 0 is reserved for empty/null string.
		const int32* MaxStringKeyValue = Algo::MaxElement(UniqueStringKeyValues);
		MaxNumUniqueValues = (MaxStringKeyValue ? *MaxStringKeyValue : 0) + 1;
	}
	else if (MaxNumUniqueValuesSource == EPCGMaxNumUniqueValuesSource::ExplicitMaxValue)
	{
		if (MaxIntValue < 0)
		{
			UE_LOGF(LogPCG, Warning, "PCGAttributeAnalysisKernelBase: number of unique values (%d) not set.", MaxIntValue);
			return 0;
		}

		MaxNumUniqueValues = MaxIntValue + 1;
	}
	else if (MaxNumUniqueValuesSource == EPCGMaxNumUniqueValuesSource::UniqueValueTable)
	{
		const FPCGKernelPin TableKernelPin(GetKernelIndex(), PCGAttributeAnalysisKernelConstants::UniqueValueTablePinLabel, /*bIsInput=*/true);
		const TSharedPtr<const FPCGDataCollectionDesc> TableDesc = InBinding->ComputeKernelPinDataDesc(TableKernelPin);
		if (!ensure(TableDesc))
		{
			return 0;
		}

		MaxNumUniqueValues = TableDesc->ComputeTotalElementCount();
	}
	else
	{
		UE_LOGF(LogPCG, Error, "PCGAttributeAnalysisKernelBase: MaxNumUniqueValuesSource (%d) is not valid.", EnumToUnderlyingType(MaxNumUniqueValuesSource));
		return 0;
	}

	return MaxNumUniqueValues;
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGAttributeAnalysisKernelBase::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	const FPCGKernelPin InputKernelPin(GetKernelIndex(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InBinding->ComputeKernelPinDataDesc(InputKernelPin);

	if (!ensure(InputDesc))
	{
		return nullptr;
	}

	if (InOutputPinLabel == PCGPinConstants::DefaultOutputLabel)
	{
		TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeShared();
		TArray<FPCGDataDesc>& OutDataDescs = OutDataDesc->GetDataDescriptionsMutable();

		int32 NumCounters = 0;
		if (InBinding->GetAttributeId(AttributeName, EPCGKernelAttributeType::StringKey) != INDEX_NONE || InBinding->GetAttributeId(AttributeName, EPCGKernelAttributeType::Int) != INDEX_NONE)
		{
			NumCounters = ComputeNumCountersRequired(InBinding, InputDesc);
		}

		if (NumCounters > 0)
		{
			// Base data description object, copied N times below to populate the final description.
			FPCGDataDesc CountDataDesc;

			if (bOutputRawBuffer)
			{
				CountDataDesc = FPCGDataDesc(FPCGDataTypeInfoRawBuffer::AsId(), NumCounters * 2);
			}
			else
			{
				CountDataDesc = FPCGDataDesc(EPCGDataType::Param, NumCounters);
				CountDataDesc.AddAttribute(FPCGKernelAttributeKey(PCGAttributeAnalysisKernelConstants::ValueAttributeName, EPCGKernelAttributeType::Int), InBinding);
				CountDataDesc.AddAttribute(FPCGKernelAttributeKey(PCGAttributeAnalysisKernelConstants::ValueCountAttributeName, EPCGKernelAttributeType::Int), InBinding);
			}

			// Raw buffer output not supported for multiple data currently.
			ensure(!bEmitPerDataCounts || !bOutputRawBuffer);

			if (bEmitPerDataCounts)
			{
				const TConstArrayView<FPCGDataDesc> InputDataDescs = InputDesc->GetDataDescriptions();

				// Output data is array of (value, count) pairs for each data.
				for (int InputIndex = 0; InputIndex < InputDataDescs.Num(); ++InputIndex)
				{
					// Data descriptions have multiple collection objects so worth to move last element for efficiency, otherwise copy.
					if (InputIndex < InputDataDescs.Num() - 1)
					{
						OutDataDescs.Add(CountDataDesc);
					}
					else
					{
						OutDataDescs.Add(MoveTemp(CountDataDesc));
					}
				}
			}
			else
			{
				OutDataDescs.Add(MoveTemp(CountDataDesc));
			}
		}
		else
		{
			// Attribute not present on input data. We can't create a zero sized buffer on the GPU, so create a null output.
			if (bOutputRawBuffer)
			{
				// Null output: One pair (string key 0 (invalid), value count 0).
				OutDataDescs.Emplace(FPCGDataTypeInfoRawBuffer::AsId(), 2);
			}
			else
			{
				// Null output: One entry: string key 0, value count 0.
				FPCGDataDesc& CountDataDesc = OutDataDescs.Emplace_GetRef(EPCGDataType::Param, 1);
				CountDataDesc.AddAttribute(FPCGKernelAttributeKey(PCGAttributeAnalysisKernelConstants::ValueAttributeName, EPCGKernelAttributeType::Int), InBinding);
				CountDataDesc.AddAttribute(FPCGKernelAttributeKey(PCGAttributeAnalysisKernelConstants::ValueCountAttributeName, EPCGKernelAttributeType::Int), InBinding);
			}
		}

		return OutDataDesc;
	}

	ensure(false);
	return nullptr;
}

int UPCGAttributeAnalysisKernelBase::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	if (const TSharedPtr<const FPCGDataCollectionDesc> InputPinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true); ensure(InputPinDesc))
	{
		const int InputElementCount = InputPinDesc->ComputeTotalElementCount();
		const int OutputElementCount = ComputeNumCountersRequired(const_cast<UPCGDataBinding*>(InBinding), InputPinDesc);

		// One thread per element, but also make sure we have enough threads to write each unique attribute value (rather than leaving uninitialized).
		return FMath::Max(InputElementCount, OutputElementCount);
	}
	else
	{
		return 0;
	}
}

bool UPCGAttributeAnalysisKernelBase::DoesOutputPinRequireZeroInitialization(FName InOutputPinLabel) const
{
	// We are atomically incrementing the values on the output so we need to ensure the values are 0-initialized.
	return InOutputPinLabel == PCGPinConstants::DefaultOutputLabel;
}

#if WITH_EDITOR
FString UPCGAttributeAnalysisKernelBase::GetCookedSource(FPCGGPUCompilationContext& InOutContext) const
{
	FString SourceFile;
	ensure(LoadShaderSourceFile(GetSourceFilePath(), EShaderPlatform::SP_PCD3D_SM5, &SourceFile, nullptr));

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("RawDataOutputEnabled"), bOutputRawBuffer ? 1 : 0 },
	};

	SourceFile = FString::Format(*SourceFile, TemplateArgs);

	return SourceFile;
}
#endif

void UPCGAttributeAnalysisKernelBase::GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const
{
	// Register the attribute this node creates.
	if (!bOutputRawBuffer)
	{
		OutKeys.AddUnique(FPCGKernelAttributeKey(PCGAttributeAnalysisKernelConstants::ValueAttributeName, EPCGKernelAttributeType::Int));
		OutKeys.AddUnique(FPCGKernelAttributeKey(PCGAttributeAnalysisKernelConstants::ValueCountAttributeName, EPCGKernelAttributeType::Int));
	}
}

void UPCGAttributeAnalysisKernelBase::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	OutPins.Emplace(PCGAttributeAnalysisKernelConstants::UniqueValueTablePinLabel, EPCGDataType::Param);
}

void UPCGAttributeAnalysisKernelBase::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	// Attribute set with a value count attribute, element count equal to number of unique values of the counted attribute.
	OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, bOutputRawBuffer ? FPCGDataTypeInfoRawBuffer::AsId() : FPCGDataTypeInfoParam::AsId());
}

#undef LOCTEXT_NAMESPACE
