// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeKernel.h"

#include "PCGAttributeAnalysisKernelBase.generated.h"

namespace PCGAttributeAnalysisKernelConstants
{
	const FName ValueAttributeName = TEXT("UniqueValue");
	const FName ValueCountAttributeName = TEXT("UniqueValueCount");
	const FName UniqueValueTablePinLabel = TEXT("UniqueValueTable");
}

/** Source used by analysis kernels to determine the maximum number of unique attribute values. */
UENUM()
enum class EPCGMaxNumUniqueValuesSource : uint8
{
	Invalid = 0,
	/** Treat the attribute type as string key and derive maximum unique value count from static data descriptions. */
	StringKeyValues = 1,
	/** Explicitly set maximum value count. */
	ExplicitMaxValue = 2,
	/** Treat the attribute value as an element index into a table, and derive the max count based on the number of elements encountered in input data. */
	UniqueValueTable = 3,
};

/**
* Abstract base for kernels that analyse an input data collection by counting unique values of a string-key or int attribute.
* The output is an attribute set of (value, count) pairs - either one per input data or one across all data.
*
* Subclasses provide the shader source, entry point, and a kernel-specific data interface. They may also extend the
* input/output pin set and the validation pipeline by overriding the relevant virtuals below and chaining to Super::.
*/
UCLASS(Abstract)
class UPCGAttributeAnalysisKernelBase : public UPCGComputeKernel
{
	GENERATED_BODY()

public:
	void SetAttributeName(FName InAttributeName) { AttributeName = InAttributeName; }

	/** Whether to produce one set of element counts that count across all input data, rather than producing counters per input data. */
	void SetEmitPerDataCounts(bool bInEmitPerDataCounts) { bEmitPerDataCounts = bInEmitPerDataCounts; }

	/** Set largest value that attribute can have. Required when counting values of int attributes (but not string key attributes). */
	void SetMaxIntValue(int32 InMaxIntValue) { MaxIntValue = InMaxIntValue; }

	/** Whether to output a raw array of (string key value, instance count) values from analysis instead of constructing an attribute set. */
	void SetOutputRawBuffer(bool bInOutputRawBuffer) { bOutputRawBuffer = bInOutputRawBuffer; }

	EPCGMaxNumUniqueValuesSource GetMaxNumUniqueValuesSource() const { return MaxNumUniqueValuesSource; }
	void SetMaxNumUniqueValuesSource(EPCGMaxNumUniqueValuesSource InSource) { MaxNumUniqueValuesSource = InSource; }

	virtual int32 ComputeNumCountersRequired(UPCGDataBinding* InBinding, TSharedPtr<const FPCGDataCollectionDesc> InInputDesc) const;
	int32 GetNumValues(UPCGDataBinding* InBinding, TSharedPtr<const FPCGDataCollectionDesc> InInputDesc) const;

	//~ Begin UPCGComputeKernel Interface
	virtual bool IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const override;
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const override;
	virtual bool DoesOutputPinRequireZeroInitialization(FName InOutputPinLabel) const override;
	virtual void GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const override;
	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;
#if WITH_EDITOR
	virtual FString GetCookedSource(FPCGGPUCompilationContext& InOutContext) const override;
	// Split graph to read back analysis results.
	virtual bool SplitGraphAtOutput() const override { return true; }
#endif
	//~ End UPCGComputeKernel Interface

protected:
	UPROPERTY()
	FName AttributeName;

	/** Set largest value that attribute can have. Required when counting values of int attributes (but not string key attributes). */
	UPROPERTY()
	int32 MaxIntValue = INDEX_NONE;

	/** Produce one set of element counts that count across all input data, rather than producing counters per input data. */
	UPROPERTY()
	bool bEmitPerDataCounts = true;

	UPROPERTY()
	bool bOutputRawBuffer = false;

	UPROPERTY()
	EPCGMaxNumUniqueValuesSource MaxNumUniqueValuesSource = EPCGMaxNumUniqueValuesSource::Invalid;
};
