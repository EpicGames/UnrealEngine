// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/BuiltInKernels/PCGCountUniqueAttributeValuesKernel.h"

#include "Compute/DataInterfaces/BuiltInKernels/PCGCountUniqueAttributeValuesDataInterface.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCountUniqueAttributeValuesKernel)

#if WITH_EDITOR
void UPCGCountUniqueAttributeValuesKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGCountUniqueAttributeValuesDataInterface> KernelDI = InOutContext.NewObject_AnyThread<UPCGCountUniqueAttributeValuesDataInterface>(InObjectOuter);
	KernelDI->SetProducerKernel(this);
	KernelDI->SetAttributeToCountName(AttributeName);
	KernelDI->SetEmitPerDataCounts(bEmitPerDataCounts);
	KernelDI->SetOutputRawBuffer(bOutputRawBuffer);

	OutDataInterfaces.Add(KernelDI);
}
#endif
