// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/BuiltInKernels/PCGAttributeAnalysisKernelBase.h"

#include "PCGCountUniqueAttributeValuesKernel.generated.h"

/**
* Counts how many unique values of a string key or int attribute are present in an input data collection. Output attribute set is a table of unique
* values and corresponding counts. Can output an attribute set per input data, or a single attribute set that counts across all data.
*/
UCLASS()
class UPCGCountUniqueAttributeValuesKernel : public UPCGAttributeAnalysisKernelBase
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual const TCHAR* GetSourceFilePath() const override { return TEXT("/Plugin/PCG/Private/BuiltInKernels/PCGCountUniqueAttributeValues.usf"); }
	virtual FString GetEntryPoint() const override { return TEXT("PCGCountUniqueAttributeValuesCS"); }
	virtual void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
#endif
};
