// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDataNumKernel.h"

#include "PCGContext.h"
#include "Compute/DataInterfaces/BuiltInKernels/PCGDataNumDataInterface.h"
#include "Elements/PCGDataNum.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataNumKernel)

#define LOCTEXT_NAMESPACE "PCGDataNumKernel"

TSharedPtr<const FPCGDataCollectionDesc> UPCGDataNumKernel::ComputeOutputBindingDataDesc(const FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	// Only single output pin
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeShared();

	FPCGDataDesc& OutputDataDesc = OutDataDesc->GetDataDescriptionsMutable().Emplace_GetRef(EPCGDataType::Param, 1);
	OutputDataDesc.AddAttribute(OutCountAttributeKey, InBinding);

	return OutDataDesc;
}

int UPCGDataNumKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	// One element returned. Only need 1 thread.
	return 1;
}

#if WITH_EDITOR
void UPCGDataNumKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGDataNumDataInterface> NodeDI = InOutContext.NewObject_AnyThread<UPCGDataNumDataInterface>(InObjectOuter);
	NodeDI->SetProducerKernel(this);

	OutDataInterfaces.Add(NodeDI);
}
#endif

void UPCGDataNumKernel::GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const
{
	OutKeys.AddUnique(OutCountAttributeKey);
}

void UPCGDataNumKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
}

void UPCGDataNumKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);
}

#if WITH_EDITOR
void UPCGDataNumKernel::InitializeInternal()
{
	Super::InitializeInternal();

	const UPCGDataNumSettings* Settings = CastChecked<UPCGDataNumSettings>(GetSettings());
	OutCountAttributeKey = FPCGKernelAttributeKey(FPCGAttributeIdentifier(Settings->OutputAttributeName), EPCGKernelAttributeType::Int);
}
#endif

#undef LOCTEXT_NAMESPACE
