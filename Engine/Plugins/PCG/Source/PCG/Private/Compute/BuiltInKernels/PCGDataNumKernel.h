// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeKernel.h"

#include "Compute/PCGPinPropertiesGPU.h"

#include "PCGDataNumKernel.generated.h"

UCLASS()
class UPCGDataNumKernel : public UPCGComputeKernel
{
	GENERATED_BODY()

public:
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const override;

#if WITH_EDITOR
	virtual const TCHAR* GetSourceFilePath() const override { return TEXT("/Plugin/PCG/Private/BuiltInKernels/PCGDataNum.usf"); }
	virtual FString GetEntryPoint() const override { return TEXT("PCGDataNumCS"); }
	virtual void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
#endif
	virtual void GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const override;
	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;

protected:
#if WITH_EDITOR
	virtual void InitializeInternal() override;
#endif

private:
	UPROPERTY()
	FPCGKernelAttributeKey OutCountAttributeKey;
};
