// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/BuiltInKernels/PCGPostRayTraceKernel.h"

#include "PCGContext.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Compute/DataInterfaces/PCGPostRayTraceDataInterface.h"
#include "Compute/DataInterfaces/PCGRawBufferDataInterface.h"
#include "Compute/DataInterfaces/Elements/PCGWorldRaycastDataInterface.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Elements/PCGWorldRaycast.h"
#include "Helpers/PCGWorldQueryHelpers.h"

#include "ShaderCompilerCore.h"

#define LOCTEXT_NAMESPACE "PCGPostRayTraceKernel"

TSharedPtr<const FPCGDataCollectionDesc> UPCGPostRayTraceKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	// Code assumes single output pin.
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	const FPCGKernelPin InputKernelPin(GetKernelIndex(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InBinding->ComputeKernelPinDataDesc(InputKernelPin);

	if (!ensure(InputDataDesc))
	{
		return nullptr;
	}

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeSharedFrom(InputDataDesc);

	const UPCGWorldRaycastElementSettings* Settings = CastChecked<UPCGWorldRaycastElementSettings>(GetSettings());

	if (Settings->bGetTextureCoordinates)
	{
		OutDataDesc->AddAttributeToAllData(FPCGKernelAttributeKey(PCGWorldQueryConstants::UVCoordAttribute, EPCGKernelAttributeType::Float2), InBinding);
	}

	if (Settings->bGetNormals)
	{
		OutDataDesc->AddAttributeToAllData(FPCGKernelAttributeKey(PCGWorldQueryConstants::ImpactNormalAttribute, EPCGKernelAttributeType::Float3), InBinding);
	}

	for (FPCGDataDesc& DataDesc : OutDataDesc->GetDataDescriptionsMutable())
	{
		DataDesc.AllocateProperties(EPCGPointNativeProperties::Transform);
	}

	return OutDataDesc;
}

int UPCGPostRayTraceKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const TSharedPtr<const FPCGDataCollectionDesc> OutputPinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultOutputLabel, /*bIsInput=*/false);
	return ensure(OutputPinDesc) ? OutputPinDesc->ComputeTotalElementCount() : 0;
}

#if WITH_EDITOR
void UPCGPostRayTraceKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGPostRayTraceDataInterface> PostRayTraceDI = InOutContext.NewObject_AnyThread<UPCGPostRayTraceDataInterface>(InObjectOuter);
	PostRayTraceDI->SetProducerKernel(this);
	OutDataInterfaces.Add(PostRayTraceDI);

	TObjectPtr<UPCGWorldRaycastDataInterface> WorldRaycastDI = InOutContext.NewObject_AnyThread<UPCGWorldRaycastDataInterface>(InObjectOuter);
	WorldRaycastDI->SetProducerKernel(this);

	OutDataInterfaces.Add(WorldRaycastDI);
}
#endif

void UPCGPostRayTraceKernel::GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const
{
	OutKeys.Add(FPCGKernelAttributeKey(PCGWorldQueryConstants::UVCoordAttribute, EPCGKernelAttributeType::Float2));
	OutKeys.Add(FPCGKernelAttributeKey(PCGWorldQueryConstants::ImpactNormalAttribute, EPCGKernelAttributeType::Float3));
}

void UPCGPostRayTraceKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	OutPins.Emplace(PCGPostRayTraceKernel::RayTraceResultsLabel, FPCGDataTypeInfoRawBuffer::AsId());
}

void UPCGPostRayTraceKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
}

#undef LOCTEXT_NAMESPACE
