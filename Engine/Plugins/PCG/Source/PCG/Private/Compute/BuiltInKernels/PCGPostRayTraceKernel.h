// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeKernel.h"

#include "Compute/PCGPinPropertiesGPU.h"

#include "PCGPostRayTraceKernel.generated.h"

namespace PCGPostRayTraceKernel
{
	const FName RayTraceResultsLabel = TEXT("RayTraceResults");
}

/** Post-process kernel that unpacks data after raytracing. Data is unpacked into points (one per ray). */
UCLASS()
class UPCGPostRayTraceKernel : public UPCGComputeKernel
{
	GENERATED_BODY()

public:
	//~Begin UPCGComputeKernel interface
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const override;
#if WITH_EDITOR
	virtual const TCHAR* GetSourceFilePath() const override { return TEXT("/Plugin/PCG/Private/BuiltInKernels/PCGPostRayTrace.usf"); }
	virtual FString GetEntryPoint() const override { return TEXT("PCGPostRayTraceCS"); }
	void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
#endif
	virtual void GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const override;
	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;
	bool RequiresPostTLASBuildExecutionGroup() const override { return true; }
	//~End UPCGComputeKernel interface
};
