// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/BuiltInKernels/PCGPreRayTraceKernel.h"

#include "PCGContext.h"
#include "PCGRayTrace.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Compute/DataInterfaces/PCGExportToRayTracingDataInterface.h"
#include "Compute/DataInterfaces/Elements/PCGWorldRaycastDataInterface.h"
#include "Elements/PCGWorldRaycast.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include "RHIGlobals.h"
#include "RendererUtils.h"
#include "ShaderCompilerCore.h"

#define LOCTEXT_NAMESPACE "PCGPreRayTraceKernel"

bool UPCGPreRayTraceKernel::IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGPreRayTraceKernel::IsKernelDataValid);

	if (!Super::IsKernelDataValid(InDataBinding, InContext))
	{
		return false;
	}

	const UPCGWorldRaycastElementSettings* Settings = CastChecked<UPCGWorldRaycastElementSettings>(GetSettings());

	if (!IsRayTracingEnabled())
	{
		PCG_KERNEL_VALIDATION_ERR(InContext, Settings, LOCTEXT("RayTracingDisabled", "Ray tracing is not enabled."));
		return false;
	}

	if (!GRHISupportsInlineRayTracing)
	{
		PCG_KERNEL_VALIDATION_ERR(InContext, Settings, LOCTEXT("InlineRayTracingUnsupported", "Inline ray tracing is not supported."));
		return false;
	}

	if (InDataBinding)
	{
		const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InDataBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultInputLabel, /*bIsInputPin=*/true);

		if (!ensure(InputDataDesc))
		{
			return false;
		}

		// Validate the attribute exists and is the correct type.
		auto ValidateAttribute = [InContext, Settings](const TSharedPtr<const FPCGDataCollectionDesc> InDataDesc, FName InAttributeName, EPCGKernelAttributeType InAttributeType)
		{
			FPCGKernelAttributeDesc AttributeDesc;
			bool bConflictingTypesFound = false;
			bool bPresentOnAllData = false;
			InDataDesc->GetAttributeDesc(InAttributeName, AttributeDesc, bConflictingTypesFound, bPresentOnAllData);

			if (!bPresentOnAllData)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings, FText::Format(LOCTEXT("AttributeMissing", "Attribute '{0}' not found, this attribute must be present on all input data."), FText::FromName(InAttributeName)));
				return false;
			}

			if (bConflictingTypesFound)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings, FText::Format(LOCTEXT("AttributeHasMultipleTypes", "Attribute '{0}' found with multiple types in input data."), FText::FromName(InAttributeName)));
				return false;
			}

			if (AttributeDesc.GetAttributeKey().GetType() != InAttributeType)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings, FText::Format(
					LOCTEXT("AttributeTypeInvalid", "Attribute '{0}', must be of type '{1}'."),
					FText::FromName(InAttributeName),
					StaticEnum<EPCGKernelAttributeType>()->GetDisplayNameTextByValue(static_cast<int64>(InAttributeType))));

				return false;
			}

			return true;
		};

		if (!InputDataDesc->GetDataDescriptions().IsEmpty())
		{
			if (Settings->OriginInputAttribute.GetSelection() == EPCGAttributePropertySelection::Attribute && !ValidateAttribute(InputDataDesc, Settings->OriginInputAttribute.GetAttributeName(), EPCGKernelAttributeType::Float3))
			{
				return false;
			}

			if (Settings->bOverrideRayDirections && !ValidateAttribute(InputDataDesc, Settings->RayDirectionAttribute.GetAttributeName(), EPCGKernelAttributeType::Float3))
			{
				return false;
			}

			if (Settings->bOverrideRayLengths && !ValidateAttribute(InputDataDesc, Settings->RayLengthAttribute.GetAttributeName(), EPCGKernelAttributeType::Float))
			{
				return false;
			}
		}

		if (Settings->RaycastMode == EPCGWorldRaycastMode::Segments)
		{
			const TSharedPtr<const FPCGDataCollectionDesc> EndPointsDataDesc = InDataBinding->GetCachedKernelPinDataDesc(this, PCGWorldRaycastElement::Constants::Labels::EndPointsInput, /*bIsInputPin=*/true);

			if (!ensure(EndPointsDataDesc))
			{
				return false;
			}

			const uint32 NumStartPoints = InputDataDesc->ComputeTotalElementCount();
			const uint32 NumEndPoints = EndPointsDataDesc->ComputeTotalElementCount();

			if (NumStartPoints != NumEndPoints)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings, FText::Format(LOCTEXT("InputMismatch", "Mismatch between number of input points ({0}) and number of end points ({1}). Must be N:N for GPU execution."), NumStartPoints, NumEndPoints));
				return false;
			}

			if (!EndPointsDataDesc->GetDataDescriptions().IsEmpty())
			{
				if (Settings->EndPointAttribute.GetSelection() == EPCGAttributePropertySelection::Attribute && !ValidateAttribute(EndPointsDataDesc, Settings->EndPointAttribute.GetAttributeName(), EPCGKernelAttributeType::Float3))
				{
					return false;
				}
			}
		}
	}

	return true;
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGPreRayTraceKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	// Code assumes single output pin.
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	const FPCGKernelPin InputKernelPin(GetKernelIndex(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InBinding->ComputeKernelPinDataDesc(InputKernelPin);

	if (!ensure(InputDataDesc))
	{
		return nullptr;
	}

	const uint32 PointElementCount = InputDataDesc->ComputeTotalElementCount();

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeShared();
	OutDataDesc->GetDataDescriptionsMutable().Add(FPCGDataDesc(FPCGDataTypeInfoRawBuffer::AsId(), PointElementCount * PCGRaytraceConstants::RAY_TRACE_PACKED_BUFFER_STRIDE_UINTS));

	return OutDataDesc;
}

int UPCGPreRayTraceKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const TSharedPtr<const FPCGDataCollectionDesc> PinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	return (ensure(PinDesc)) ? PinDesc->ComputeTotalElementCount() : 0;
}

#if WITH_EDITOR
FString UPCGPreRayTraceKernel::GetCookedSource(FPCGGPUCompilationContext& InOutContext) const
{
	FString SourceFile;
	ensure(LoadShaderSourceFile(GetSourceFilePath(), EShaderPlatform::SP_PCD3D_SM5, &SourceFile, nullptr));
	
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("EndPointsEnabled"), bIncludeEndPointsInputPin ? 1 : 0 },
	};

	SourceFile = FString::Format(*SourceFile, TemplateArgs);

	return SourceFile;
}

void UPCGPreRayTraceKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGWorldRaycastDataInterface> WorldRaycastDI = InOutContext.NewObject_AnyThread<UPCGWorldRaycastDataInterface>(InObjectOuter);
	WorldRaycastDI->SetProducerKernel(this);

	OutDataInterfaces.Add(WorldRaycastDI);
}

UPCGComputeDataInterface* UPCGPreRayTraceKernel::CreateOutputPinDataInterface(const PCGComputeHelpers::FCreateDataInterfaceParams& InParams) const
{
	check(InParams.Context);
	check(InParams.PinProperties);
	check(InParams.ObjectOuter);

	if (InParams.PinProperties->Label == PCGPinConstants::DefaultOutputLabel && ensure(InParams.PinProperties->AllowedTypes == FPCGDataTypeInfoRawBuffer::AsId()))
	{
		// Use a special data interface for the raw buffer output - one that dispatches a ray trace pass after the buffer is populated.
		UPCGExportToRayTracingDataInterface* ExportToRTDI = InParams.Context->NewObject_AnyThread<UPCGExportToRayTracingDataInterface>(InParams.ObjectOuter);
		ExportToRTDI->SetInputPointsPinLabel(PCGPinConstants::DefaultInputLabel);
		
		return ExportToRTDI;
	}
	else
	{
		// Fallback to default DI creation logic.
		return Super::CreateOutputPinDataInterface(InParams);
	}
}
#endif

void UPCGPreRayTraceKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	OutPins.Emplace(PCGWorldRaycastElement::Constants::Labels::EndPointsInput, EPCGDataType::Point);
}

void UPCGPreRayTraceKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeInfoRawBuffer::AsId());
}

#if WITH_EDITOR
bool UPCGPreRayTraceKernel::PerformStaticValidation()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGPreRayTraceKernel::PerformStaticValidation);

	if (!Super::PerformStaticValidation())
	{
		return false;
	}

	const UPCGWorldRaycastElementSettings* Settings = CastChecked<UPCGWorldRaycastElementSettings>(GetSettings());

#if PCG_KERNEL_LOGGING_ENABLED
	auto LogUnsupportedAttribute = [this](const FPCGAttributePropertyInputSelector& InputSelector)
	{
		AddStaticLogEntry(FText::Format(LOCTEXT("InvalidAttribute", "Invalid attribute '{0}', unsupported for GPU execution. Must be a basic attribute (not a property and no extra names)."), InputSelector.GetDisplayText()), EPCGKernelLogVerbosity::Error);
	};

	auto LogUnsupportedProperty = [this](const FPCGAttributePropertyInputSelector& InputSelector)
	{
		AddStaticLogEntry(FText::Format(LOCTEXT("InvalidProperty", "Invalid property '{0}', unsupported for GPU execution. Must be '$Position'."), InputSelector.GetDisplayText()), EPCGKernelLogVerbosity::Error);
	};
#endif

	if (Settings->OriginInputAttribute.GetSelection() == EPCGAttributePropertySelection::Property && Settings->OriginInputAttribute.GetPointProperty() != EPCGPointProperties::Position)
	{
#if PCG_KERNEL_LOGGING_ENABLED
		LogUnsupportedProperty(Settings->OriginInputAttribute);
#endif
		return false;
	}

	if (Settings->bOverrideRayDirections && !Settings->RayDirectionAttribute.IsBasicAttribute())
	{
#if PCG_KERNEL_LOGGING_ENABLED
		LogUnsupportedAttribute(Settings->RayDirectionAttribute);
#endif
		return false;
	}

	if (Settings->bOverrideRayLengths && !Settings->RayLengthAttribute.IsBasicAttribute())
	{
#if PCG_KERNEL_LOGGING_ENABLED
		LogUnsupportedAttribute(Settings->RayLengthAttribute);
#endif
		return false;
	}

	if (Settings->RaycastMode == EPCGWorldRaycastMode::Segments)
	{
		const UPCGNode* Node = GetNode();
		check(Node);

		if (!Node->IsInputPinConnected(PCGWorldRaycastElement::Constants::Labels::EndPointsInput))
		{
#if PCG_KERNEL_LOGGING_ENABLED
			AddStaticLogEntry(FText::Format(LOCTEXT("MissingOutputPoints", "Missing connection on input pin '{0}'"), FText::FromName(PCGWorldRaycastElement::Constants::Labels::EndPointsInput)), EPCGKernelLogVerbosity::Error);
#endif
			return false;
		}

		if (Settings->EndPointAttribute.GetSelection() == EPCGAttributePropertySelection::Property && Settings->EndPointAttribute.GetPointProperty() != EPCGPointProperties::Position)
		{
	#if PCG_KERNEL_LOGGING_ENABLED
			LogUnsupportedProperty(Settings->EndPointAttribute);
	#endif
			return false;
		}
	}

	if (Settings->CollisionShape.ShapeType != EPCGCollisionShapeType::Line)
	{
#if PCG_KERNEL_LOGGING_ENABLED
		AddStaticLogEntry(FText::Format(LOCTEXT("InvalidCollisionShape", "Invalid collision shape '{0}', unsupported for GPU execution. Must be '{1}'."),
			StaticEnum<EPCGCollisionShapeType>()->GetDisplayNameTextByValue((int64)Settings->CollisionShape.ShapeType),
			StaticEnum<EPCGCollisionShapeType>()->GetDisplayNameTextByValue((int64)EPCGCollisionShapeType::Line)),
			EPCGKernelLogVerbosity::Error);
#endif
		return false;
	}

	return true;
}

void UPCGPreRayTraceKernel::SetIncludeEndPointsInputPin(bool bEnabled)
{
	bIncludeEndPointsInputPin = bEnabled;
}
#endif

#undef LOCTEXT_NAMESPACE
