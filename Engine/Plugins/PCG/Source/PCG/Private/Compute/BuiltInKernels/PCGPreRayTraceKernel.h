// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeKernel.h"

#include "Compute/PCGPinPropertiesGPU.h"

#include "PCGPreRayTraceKernel.generated.h"

/** Pre-process kernel that packs data for raytracing, producing one ray per input point. Dispatches an inlined raytrace in PostSubmit. */
UCLASS()
class UPCGPreRayTraceKernel : public UPCGComputeKernel
{
	GENERATED_BODY()

public:
	//~Begin UPCGComputeKernel interface
	virtual bool IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const override;
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const override;
#if WITH_EDITOR
	virtual FString GetCookedSource(FPCGGPUCompilationContext& InOutContext) const override;
	virtual const TCHAR* GetSourceFilePath() const override { return TEXT("/Plugin/PCG/Private/BuiltInKernels/PCGPreRayTrace.usf"); }
	virtual FString GetEntryPoint() const override { return TEXT("PCGPreRayTraceCS"); }
	virtual void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
	virtual UPCGComputeDataInterface* CreateOutputPinDataInterface(const PCGComputeHelpers::FCreateDataInterfaceParams& InParams) const override;
#endif
	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;
	bool RequiresPostTLASBuildExecutionGroup() const override { return true; }

protected:
#if WITH_EDITOR
	virtual bool PerformStaticValidation() override;
#endif
	//~End UPCGComputeKernel interface

#if WITH_EDITOR
public:
	void SetIncludeEndPointsInputPin(bool bEnabled);

private:
	bool bIncludeEndPointsInputPin = false;
#endif
};
