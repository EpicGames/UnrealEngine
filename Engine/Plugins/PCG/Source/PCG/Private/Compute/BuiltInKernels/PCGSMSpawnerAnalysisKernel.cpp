// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/BuiltInKernels/PCGSMSpawnerAnalysisKernel.h"

#include "Compute/PCGDataBinding.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Compute/DataInterfaces/BuiltInKernels/PCGSMSpawnerAnalysisDataInterface.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSMSpawnerAnalysisKernel)

int32 UPCGSMSpawnerAnalysisKernel::ComputeNumCountersRequired(UPCGDataBinding* InBinding, TSharedPtr<const FPCGDataCollectionDesc> InInputDesc) const
{
	const int32 NumValues = Super::ComputeNumCountersRequired(InBinding, InInputDesc);
	const FIntVector SubCounters = InBinding->GetMaxNumCullingCells(GetCullingCellExtent());
	return NumValues * SubCounters.X * SubCounters.Y * SubCounters.Z;
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGSMSpawnerAnalysisKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	if (InOutputPinLabel != PCGSMSpawnerAnalysisConstants::CullingCellMinMaxPositionsPinLabel)
	{
		return Super::ComputeOutputBindingDataDesc(InOutputPinLabel, InBinding);
	}

	check(InBinding);

	const FPCGKernelPin InputKernelPin(GetKernelIndex(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InBinding->ComputeKernelPinDataDesc(InputKernelPin);
	if (!ensure(InputDesc))
	{
		return nullptr;
	}

	int32 NumValues = 0;
	if (InBinding->GetAttributeId(AttributeName, EPCGKernelAttributeType::StringKey) != INDEX_NONE || InBinding->GetAttributeId(AttributeName, EPCGKernelAttributeType::Int) != INDEX_NONE)
	{
		NumValues = GetNumValues(InBinding, InputDesc);
	}

	const int32 NumCullingCellsTotal = ComputeNumCountersRequired(InBinding, InputDesc) / FMath::Max(1, NumValues);

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeShared();
	FPCGDataDesc& CullingCellMinMaxPositionsDataDesc = OutDataDesc->GetDataDescriptionsMutable().Emplace_GetRef();

	// PositionStride uints per (primitive, culling cell): MinX, MinY, MinZ, MaxX, MaxY, MaxZ
	CullingCellMinMaxPositionsDataDesc = FPCGDataDesc(FPCGDataTypeInfoRawBuffer::AsId(), PCGSMSpawnerAnalysisConstants::PositionStride * FMath::Max(1, NumValues) * FMath::Max(1, NumCullingCellsTotal));

	return OutDataDesc;
}

bool UPCGSMSpawnerAnalysisKernel::DoesOutputPinRequireZeroInitialization(FName InOutputPinLabel) const
{
	return Super::DoesOutputPinRequireZeroInitialization(InOutputPinLabel)
		|| InOutputPinLabel == PCGSMSpawnerAnalysisConstants::CullingCellMinMaxPositionsPinLabel;
}

void UPCGSMSpawnerAnalysisKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	Super::GetOutputPins(OutPins);

	// Per-culling-cell min/max bounds: one entry per (primitive, culling cell).
	OutPins.Emplace(PCGSMSpawnerAnalysisConstants::CullingCellMinMaxPositionsPinLabel, FPCGDataTypeInfoRawBuffer::AsId());
}

#if WITH_EDITOR
void UPCGSMSpawnerAnalysisKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGSMSpawnerAnalysisDataInterface> KernelDI = InOutContext.NewObject_AnyThread<UPCGSMSpawnerAnalysisDataInterface>(InObjectOuter);
	KernelDI->SetProducerKernel(this);
	KernelDI->SetAttributeToCountName(AttributeName);
	KernelDI->SetEmitPerDataCounts(bEmitPerDataCounts);
	KernelDI->SetOutputRawBuffer(bOutputRawBuffer);
	KernelDI->SetBucketingCellExtent(GetCullingCellExtent());
	KernelDI->SetMeshAttributeName(MeshAttributeName);

	OutDataInterfaces.Add(KernelDI);
}
#endif
