// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/BuiltInKernels/PCGAttributeAnalysisKernelBase.h"

#include "PCGSMSpawnerAnalysisKernel.generated.h"

namespace PCGSMSpawnerAnalysisConstants
{
	const FName CullingCellMinMaxPositionsPinLabel = TEXT("CullingCellMinMaxPositions");

	/** Number of uint32 entries per-(primitive, culling cell) in the CullingCellMinMaxPositions buffer: 3 inv-min position, 3 max position. */
	constexpr int32 PositionStride = 6;
}

/**
* Dedicated analysis kernel for the Static Mesh Spawner: counts unique attribute values like the generic analysis kernel,
* plus produces a per-(primitive, culling cell) min/max position buffer used by the spawner's culling pipeline.
*/
UCLASS()
class UPCGSMSpawnerAnalysisKernel : public UPCGAttributeAnalysisKernelBase
{
	GENERATED_BODY()

public:
	FName GetMeshAttributeName() const { return MeshAttributeName; }
	/** Attribute name in the primitive table that holds the static mesh path. Only set when spawning by primitive data. */
	void SetMeshAttributeName(FName InMeshAttributeName) { MeshAttributeName = InMeshAttributeName; }

	int32 GetCullingCellExtent() const { return CullingCellExtent; }
	void SetCullingCellExtent(int32 InCullingCellExtent) { CullingCellExtent = InCullingCellExtent; }

	//~ Begin UPCGComputeKernel Interface
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
	virtual bool DoesOutputPinRequireZeroInitialization(FName InOutputPinLabel) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;
#if WITH_EDITOR
	virtual const TCHAR* GetSourceFilePath() const override { return TEXT("/Plugin/PCG/Private/BuiltInKernels/PCGSMSpawnerAnalysis.usf"); }
	virtual FString GetEntryPoint() const override { return TEXT("PCGSMSpawnerAnalysisCS"); }
	virtual void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
#endif
	//~ End UPCGComputeKernel Interface

	//~ Begin UPCGAttributeAnalysisKernelBase Interface
	virtual int32 ComputeNumCountersRequired(UPCGDataBinding* InBinding, TSharedPtr<const FPCGDataCollectionDesc> InInputDesc) const override;
	//~ End UPCGAttributeAnalysisKernelBase Interface

protected:
	/** Attribute name in the primitive table that holds the static mesh path. Only set when spawning by primitive data. */
	UPROPERTY()
	FName MeshAttributeName;

	/** Extent of a culling cell in cm. */
	UPROPERTY()
	int32 CullingCellExtent = 0;
};
