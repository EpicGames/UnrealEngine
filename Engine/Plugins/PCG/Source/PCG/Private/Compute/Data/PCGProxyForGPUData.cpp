// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/Data/PCGProxyForGPUData.h"

#include "PCGModule.h"
#include "Compute/Packing/PCGDataCollectionPacking.h"
#include "Helpers/PCGAsync.h"

#include "RHIGPUReadback.h"
#include "RenderGraphResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGProxyForGPUData)

#define LOCTEXT_NAMESPACE "PCGProxyForGPUData"

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoProxyForGPU, UPCGProxyForGPUData)

#if WITH_EDITOR
static TAutoConsoleVariable<bool> CVarWarnOnGPUReadbacks(
	TEXT("pcg.Graph.GPU.WarnOnGPUReadbacks"),
	false,
	TEXT("Emits warnings on nodes which trigger a readback of data from GPU to CPU."));
#endif

void UPCGProxyForGPUData::Initialize(TSharedPtr<FPCGProxyForGPUDataCollection> InDataCollection, int InDataIndexInCollection)
{
	DataCollectionOnGPU = InDataCollection;
	DataIndexInCollection = InDataIndexInCollection;
}

void UPCGProxyForGPUData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use the unique object instance UID.
	AddUIDToCrc(Ar);
}

void UPCGProxyForGPUData::ReleaseTransientResources(const TCHAR* InReason)
{
#ifdef PCG_DATA_USAGE_LOGGING
	UE_LOGF(LogPCG, Warning, "Releasing GPU data for '%ls' due to %ls", *GetName(), InReason ? InReason : TEXT("NOREASON"));
#endif

	DataCollectionOnGPU.Reset();
}

EPCGDataType UPCGProxyForGPUData::GetUnderlyingDataType() const
{
	return GetUnderlyingDataTypeId().AsLegacyType();
}

FPCGDataTypeBaseId UPCGProxyForGPUData::GetUnderlyingDataTypeId() const
{
	if (TSharedPtr<const FPCGProxyForGPUDataCollection> DataCollectionGPU = GetGPUInfo())
	{
		if (const TSharedPtr<const FPCGDataCollectionDesc> DataCollectionDesc = DataCollectionGPU->GetDescription())
		{
			return DataCollectionDesc->GetDataDescriptions()[DataIndexInCollection].GetType().GetId();
		}
	}

	return FPCGDataTypeBaseId{};
}

FIntVector4 UPCGProxyForGPUData::GetNumElements() const
{
	if (TSharedPtr<const FPCGProxyForGPUDataCollection> DataCollectionGPU = GetGPUInfo())
	{
		if (TSharedPtr<const FPCGDataCollectionDesc> Desc = DataCollectionGPU->GetDescription())
		{
			if (Desc->GetDataDescriptions().IsValidIndex(DataIndexInCollection))
			{
				return Desc->GetDataDescriptions()[DataIndexInCollection].GetElementCount();
			}
		}
	}

	return FIntVector4::ZeroValue;
}

TSharedPtr<const FPCGProxyForGPUDataCollection> UPCGProxyForGPUData::GetInputDataCollectionInfo() const
{
	return DataCollectionOnGPU;
}

int UPCGProxyForGPUData::GetDataIndexInCollection() const
{
	return DataIndexInCollection;
}

FIntVector4 UPCGProxyForGPUData::GetElementCount() const
{
	if (TSharedPtr<const FPCGProxyForGPUDataCollection> DataCollectionGPU = GetGPUInfo())
	{
		if (const TSharedPtr<const FPCGDataCollectionDesc> DataCollectionDesc = DataCollectionGPU->GetDescription())
		{
			return DataCollectionDesc->GetDataDescriptions()[DataIndexInCollection].GetElementCount();
		}
	}

	return FIntVector4::ZeroValue;
}

bool UPCGProxyForGPUData::GetDescription(FPCGDataDesc& OutDescription) const
{
	if (TSharedPtr<const FPCGProxyForGPUDataCollection> DataCollectionGPU = GetGPUInfo())
	{
		if (const TSharedPtr<const FPCGDataCollectionDesc> DataCollectionDesc = DataCollectionGPU->GetDescription())
		{
			OutDescription = DataCollectionDesc->GetDataDescriptions()[DataIndexInCollection];
			return true;
		}
	}

	return false;
}

UPCGProxyForGPUData::FReadbackResult UPCGProxyForGPUData::GetCPUData(FPCGContext* InContext) const
{
	TSharedPtr<FPCGProxyForGPUDataCollection> DataOnGPU = GetGPUInfoMutable();
	if (!DataOnGPU)
	{
		UE_LOGF(LogPCG, Error, "Data collection lost! Enabling the define PCG_DATA_USAGE_LOGGING may help to identify when resource was released.");

		return FReadbackResult{ .bComplete = true };
	}

	FPCGTaggedData ResultData;
	if (!DataOnGPU->GetCPUData(InContext, DataIndexInCollection, ResultData))
	{
		return FReadbackResult{ .bComplete = false };
	}

	return FReadbackResult{ .bComplete = true, .TaggedData = MoveTemp(ResultData) };
}

TSharedPtr<const FPCGProxyForGPUDataCollection> UPCGProxyForGPUData::GetGPUInfo() const
{
	return GetGPUInfoMutable();
}

TSharedPtr<FPCGProxyForGPUDataCollection> UPCGProxyForGPUData::GetGPUInfoMutable() const
{
	if (!DataCollectionOnGPU || !DataCollectionOnGPU->GetDescription())
	{
		ensureMsgf(false, TEXT("Data %s: GPU data collection lost. Enabling the define 'PCG_DATA_USAGE_LOGGING' may help to identify when resource was released."), *GetName());
		return nullptr;
	}
	else if (!DataCollectionOnGPU->GetDescription()->GetDataDescriptions().IsValidIndex(DataIndexInCollection))
	{
		ensureMsgf(false, TEXT("Data %s: DataIndexInCollection (%d) was out of range [0, %d)."), *GetName(), DataIndexInCollection, DataCollectionOnGPU->GetDescription()->GetDataDescriptions().Num());
		return nullptr;
	}

	return DataCollectionOnGPU;
}

FPCGProxyForGPUDataCollection::FPCGProxyForGPUDataCollection(TRefCountPtr<FRDGPooledBuffer> InBuffer, uint32 InBufferSizeBytes, const TSharedPtr<const FPCGDataCollectionDesc> InDescription, const TArray<FString>& InStringTable)
	: Buffer(InBuffer)
	, BufferSizeBytes(InBufferSizeBytes)
	, Description(InDescription)
	, StringTable(InStringTable)
{
}

bool FPCGProxyForGPUDataCollection::GetCPUData(FPCGContext* InContext, int32 InDataIndex, FPCGTaggedData& OutData)
{
	PCG::TScopeLock Lock(ReadbackLock);

	if (bReadbackDataProcessed)
	{
		OutData = ReadbackData.IsValidIndex(InDataIndex) ? ReadbackData[InDataIndex] : FPCGTaggedData();
		return true;
	}

	if (!ReadbackRequest && !bReadbackDataArrived)
	{
#if WITH_EDITOR
		if (CVarWarnOnGPUReadbacks.GetValueOnAnyThread())
		{
			PCGLog::LogWarningOnGraph(LOCTEXT("PerformingReadback", "Performing readback"), InContext);
		}
#endif

		ReadbackRequest = MakeShared<FRHIGPUBufferReadback>(TEXT("FPCGProxyForGPUDataCollectionReadback"));

		ENQUEUE_RENDER_COMMAND(ReadbackDataCollection)([WeakThis=AsWeak(), ReadbackRequest=ReadbackRequest, Buffer=Buffer, BufferSizeBytes=BufferSizeBytes](FRHICommandListImmediate& RHICmdList)
		{
			LLM_SCOPE_BYTAG(PCG);

			FRDGPooledBuffer* RDGBuffer = Buffer.GetReference();

			ReadbackRequest->EnqueueCopy(RHICmdList, Buffer->GetRHI());

			auto ExecuteAsync = [](auto&& RunnerFunc) -> void
			{
				if (IsInActualRenderingThread())
				{
					AsyncTask(ENamedThreads::ActualRenderingThread, [RunnerFunc]()
					{
						RunnerFunc(RunnerFunc);
					});
				}
				else
				{
					// In specific cases (Server, -onethread, etc) the RenderingThread is actually the same as the GameThread.
					// When this happens we want to avoid calling AsyncTask which could put us in a infinite task execution loop. 
					// The reason is that if we are running this callback through the task graph we might stay in an executing loop until it has no tasks to execute,
					// since we are pushing a new task as long as our data isn't ready and we are not advancing the GameThread as we are already on the GameThread this causes a infinite task execution.
					// Instead delay to GameThread with ExecuteOnGameThread
					ExecuteOnGameThread(UE_SOURCE_LOCATION, [RunnerFunc]()
					{
						RunnerFunc(RunnerFunc);
					});
				}
			};

			auto RunnerFunc = [WeakThis, ReadbackRequest, BufferSizeBytes, ExecuteAsync](auto&& RunnerFunc) -> void
			{
				LLM_SCOPE_BYTAG(PCG);

				if (TSharedPtr<FPCGProxyForGPUDataCollection> This = WeakThis.Pin())
				{
					if (ReadbackRequest->IsReady())
					{
						if (void* RawData = ReadbackRequest->Lock(BufferSizeBytes))
						{
							uint32* DataAsUint = static_cast<uint32*>(RawData);

							This->RawReadbackData.SetNumUninitialized(BufferSizeBytes);
							FMemory::Memcpy(This->RawReadbackData.GetData(), RawData, BufferSizeBytes);
						}

						ReadbackRequest->Unlock();

						This->bReadbackDataArrived = true;
					}
					else
					{
						ExecuteAsync(RunnerFunc);
					}
				}
			};

			ExecuteAsync(RunnerFunc);

		});
	}

	if (!bReadbackDataArrived)
	{
		OutData = FPCGTaggedData();
		return false;
	}

	ReadbackRequest.Reset();

	bReadbackDataProcessed = true;

	if (!RawReadbackData.IsEmpty())
	{
		FPCGDataCollection DataFromGPU;
		const EPCGUnpackDataCollectionResult Result = PCGDataCollectionPackingHelpers::UnpackDataCollection(InContext, Description, RawReadbackData, /*OutputPinLabelAlias*/NAME_None, StringTable, DataFromGPU);

		RawReadbackData.Empty();

		if (Result != EPCGUnpackDataCollectionResult::Success)
		{
			OutData = FPCGTaggedData();
			return true;
		}

		ReadbackData.Reset();
		ReadbackDataRefs.Reset();

		for (const FPCGTaggedData& TaggedData : DataFromGPU.TaggedData)
		{
			ReadbackData.Add(TaggedData);
			ReadbackDataRefs.Add(TStrongObjectPtr<const UPCGData>(TaggedData.Data));
		}
	}

	OutData = ReadbackData.IsValidIndex(InDataIndex) ? ReadbackData[InDataIndex] : FPCGTaggedData();
	return true;
}

#undef LOCTEXT_NAMESPACE
