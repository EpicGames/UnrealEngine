// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/Data/PCGRawBufferData.h"

#include "PCGModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGRawBufferData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoRawBuffer, UPCGRawBufferData)

void UPCGRawBufferData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use the unique object instance UID.
	AddUIDToCrc(Ar);
}

void UPCGRawBufferData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Data.GetAllocatedSize());
}

void UPCGRawBufferData::ReleaseTransientResources(const TCHAR* InReason)
{
	Super::ReleaseTransientResources(InReason);

	Data.Empty();
}
