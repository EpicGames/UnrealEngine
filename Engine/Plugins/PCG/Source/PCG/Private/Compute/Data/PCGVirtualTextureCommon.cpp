// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGVirtualTextureCommon.h"

#include "RHIResources.h"
#include "VT/RuntimeVirtualTexture.h"

namespace PCGVirtualTextureCommon
{
	void FVirtualTextureLayer::Initialize(const URuntimeVirtualTexture* VirtualTexture, uint32 LayerIndex, bool bSRGB)
	{
		Reset();
	
		if (::IsValid(VirtualTexture))
		{
			if (IAllocatedVirtualTexture* AllocatedTexture = VirtualTexture->GetAllocatedVirtualTexture())
			{
				if (FRHIShaderResourceView* PhysicalTextureSRV = AllocatedTexture->GetPhysicalTextureSRV(LayerIndex, bSRGB))
				{
					TextureSRV = PhysicalTextureSRV;
					AllocatedTexture->GetPackedUniform(&TextureUniforms, LayerIndex);
				}
			}
		}
	}
	
	void FVirtualTextureLayer::Reset()
	{
		TextureSRV = nullptr;
		TextureUniforms = FUintVector4::ZeroValue;
	}
	
	bool FVirtualTextureLayer::IsValid() const
	{
		return TextureSRV.IsValid();
	}
	
	void FVirtualTexturePageTable::Initialize(const URuntimeVirtualTexture* VirtualTexture)
	{
		Reset();
	
		if (::IsValid(VirtualTexture))
		{
			if (IAllocatedVirtualTexture* AllocatedTexture = VirtualTexture->GetAllocatedVirtualTexture())
			{
				// RVT should always have a single PageTable texture.
				check(AllocatedTexture->GetNumPageTableTextures() == 1);
				PageTableRef = AllocatedTexture->GetPageTableTexture(0);
				PageTableIndirectionRef = AllocatedTexture->GetPageTableIndirectionTexture();

				bIsAdaptive = VirtualTexture->GetAdaptivePageTable();
				bSinglePhysicalSpace = VirtualTexture->GetSinglePhysicalSpace();
	
				if (PageTableRef.IsValid())
				{
					AllocatedTexture->GetPackedPageTableUniform(PageTableUniforms);
	
					WorldToUVParameters[0] = VirtualTexture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldToUVTransform0);
					WorldToUVParameters[1] = VirtualTexture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1);
					WorldToUVParameters[2] = VirtualTexture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2);
					WorldToUVParameters[3] = VirtualTexture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldHeightUnpack);
				}
			}
		}
	}
	
	void FVirtualTexturePageTable::Reset()
	{
		PageTableRef = nullptr;
		PageTableIndirectionRef = nullptr;
		bIsAdaptive = false;
		bSinglePhysicalSpace = false;
		PageTableUniforms[0] = FUintVector4::ZeroValue;
		PageTableUniforms[1] = FUintVector4::ZeroValue;
		WorldToUVParameters[0] = FVector4::Zero();
		WorldToUVParameters[1] = FVector4::Zero();
		WorldToUVParameters[2] = FVector4::Zero();
		WorldToUVParameters[3] = FVector4::Zero();
	}
	
	bool FVirtualTexturePageTable::IsValid() const
	{
		// If this is an adaptive RVT then we require the indirection page table.
		if (bIsAdaptive && !PageTableIndirectionRef.IsValid())
		{
			return false;
		}
	
		return PageTableRef.IsValid();
	}
}