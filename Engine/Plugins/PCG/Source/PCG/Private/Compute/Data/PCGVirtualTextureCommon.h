// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIFwd.h"
#include "VT/RuntimeVirtualTextureEnum.h"

class URuntimeVirtualTexture;

namespace PCGVirtualTextureCommon
{
	enum class ENormalUnpackType : int32
	{
		None = 0,
		BC3BC3,
		BC5BC1,
		B5G6R5,
	};

	enum class EBaseColorUnpackType : int32
	{
		/** No unpacking is required. */
		None = 0,
		/** Base color is manually packed as SRGB. */
		SRGBUnpack,
		/** Base color is manually packed as YCoCg. */
		YCoCgUnpack,
	};

	struct FVirtualTextureLayer
	{
		void Initialize(const URuntimeVirtualTexture* VirtualTexture, uint32 LayerIndex, bool bSRGB);
		void Reset();
		bool IsValid() const;

		FShaderResourceViewRHIRef TextureSRV;
		FUintVector4 TextureUniforms = FUintVector4::ZeroValue;
	};

	struct FVirtualTexturePageTable
	{
		void Initialize(const URuntimeVirtualTexture* VirtualTexture);
		void Reset();
		bool IsValid() const;

		FTextureRHIRef PageTableRef;
		FTextureRHIRef PageTableIndirectionRef;
		bool bSinglePhysicalSpace = false;
		bool bIsAdaptive = false;
		FUintVector4 PageTableUniforms[2];
		FVector4 WorldToUVParameters[4];
	};
}
