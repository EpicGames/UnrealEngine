// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGCountUniqueAttributeValuesDataInterface.h"

#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGDataDescription.h"
#include "Compute/BuiltInKernels/PCGAttributeAnalysisKernelBase.h"

#include "RHIResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCountUniqueAttributeValuesDataInterface)

void UPCGCountUniqueAttributeValuesDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	const FString Prefix(GetShaderFunctionPrefix());

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetAttributeToCountId"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetOutputCountAttributeId"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetOutputValueAttributeId"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetEmitPerDataCounts"))
		.AddReturnType(EShaderFundamentalType::Bool);
}

void UPCGCountUniqueAttributeValuesDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGAttributeAnalysisCommonShaderParameters>(UID);
}

void UPCGCountUniqueAttributeValuesDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
		{ TEXT("Prefix"), FString(GetShaderFunctionPrefix()) },
	};

	OutHLSL += FString::Format(TEXT(
		"int {DataInterfaceName}_AttributeToCountId;\n"
		"int {DataInterfaceName}_OutputValueAttributeId;\n"
		"int {DataInterfaceName}_OutputCountAttributeId;\n"
		"uint {DataInterfaceName}_EmitPerDataCounts;\n"
		"\n"
		"int {Prefix}_GetAttributeToCountId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_AttributeToCountId;\n"
		"}\n"
		"\n"
		"int {Prefix}_GetOutputValueAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_OutputValueAttributeId;\n"
		"}\n"
		"\n"
		"int {Prefix}_GetOutputCountAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_OutputCountAttributeId;\n"
		"}\n"
		"\n"
		"bool {Prefix}_GetEmitPerDataCounts_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_EmitPerDataCounts > 0;\n"
		"}\n"
	), TemplateArgs);
}

UComputeDataProvider* UPCGCountUniqueAttributeValuesDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGCountUniqueAttributeValuesDataProvider>();
}

void UPCGCountUniqueAttributeValuesDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCountUniqueAttributeValuesDataProvider::Initialize);

	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGCountUniqueAttributeValuesDataInterface* DataInterface = CastChecked<UPCGCountUniqueAttributeValuesDataInterface>(InDataInterface);

	AttributeToCountName = DataInterface->GetAttributeToCountName();
	bEmitPerDataCounts = DataInterface->GetEmitPerDataCounts();
	bOutputRawBuffer = DataInterface->GetOutputRawBuffer();
}

bool UPCGCountUniqueAttributeValuesDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCountUniqueAttributeValuesDataProvider::PrepareForExecute_GameThread);
	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	if (!bOutputRawBuffer)
	{
		OutputValueAttributeId = InBinding->GetAttributeId(PCGAttributeAnalysisKernelConstants::ValueAttributeName, EPCGKernelAttributeType::Int);
		check(OutputValueAttributeId != -1);
		OutputCountAttributeId = InBinding->GetAttributeId(PCGAttributeAnalysisKernelConstants::ValueCountAttributeName, EPCGKernelAttributeType::Int);
		check(OutputCountAttributeId != -1);
	}

	AttributeToCountId = InBinding->GetAttributeId(AttributeToCountName, EPCGKernelAttributeType::StringKey);
	if (AttributeToCountId == INDEX_NONE)
	{
		AttributeToCountId = InBinding->GetAttributeId(AttributeToCountName, EPCGKernelAttributeType::Int);
	}

	return true;
}

FComputeDataProviderRenderProxy* UPCGCountUniqueAttributeValuesDataProvider::GetRenderProxy()
{
	FPCGAttributeAnalysisCommonDispatchData Data =
	{
		.AttributeToCountId = AttributeToCountId,
		.OutputValueAttributeId = OutputValueAttributeId,
		.OutputCountAttributeId = OutputCountAttributeId,
		.bEmitPerDataCounts = bEmitPerDataCounts,
	};

	return new FPCGCountUniqueAttributeValuesProviderProxy(Data);
}

void UPCGCountUniqueAttributeValuesDataProvider::Reset()
{
	AttributeToCountName = NAME_None;
	AttributeToCountId = INDEX_NONE;
	OutputValueAttributeId = INDEX_NONE;
	OutputCountAttributeId = INDEX_NONE;
	bEmitPerDataCounts = true;
	bOutputRawBuffer = false;

	Super::Reset();
}

bool FPCGCountUniqueAttributeValuesProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return (InValidationData.ParameterStructSize == sizeof(FParameters));
}

void FPCGCountUniqueAttributeValuesProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		PCGAttributeAnalysis::WriteCommonDispatchParameters(ParameterArray[InvocationIndex], Data);
	}
}
