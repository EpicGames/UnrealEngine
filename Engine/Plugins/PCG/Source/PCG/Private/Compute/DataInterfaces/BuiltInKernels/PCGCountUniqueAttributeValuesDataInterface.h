// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "ShaderParameterMacros.h"

#include "PCGCountUniqueAttributeValuesDataInterface.generated.h"

class UPCGDataBinding;

/** Shader parameter struct holding the scalars shared by every attribute-analysis kernel. */
BEGIN_SHADER_PARAMETER_STRUCT(FPCGAttributeAnalysisCommonShaderParameters,)
	SHADER_PARAMETER(int32, AttributeToCountId)
	SHADER_PARAMETER(int32, OutputValueAttributeId)
	SHADER_PARAMETER(int32, OutputCountAttributeId)
	SHADER_PARAMETER(uint32, EmitPerDataCounts)
END_SHADER_PARAMETER_STRUCT()

/** Common dispatch-time scalars shared by attribute-analysis proxy classes. */
struct FPCGAttributeAnalysisCommonDispatchData
{
	int32 AttributeToCountId = INDEX_NONE;
	int32 OutputValueAttributeId = INDEX_NONE;
	int32 OutputCountAttributeId = INDEX_NONE;
	bool bEmitPerDataCounts = true;
};

namespace PCGAttributeAnalysis
{
	/**
	 * Copy the common dispatch data into a flat shader parameter struct.
	 * Works for any TParameters that names these four fields directly (kept flat so both subclass param structs match).
	 */
	template<typename TParameters>
	void WriteCommonDispatchParameters(TParameters& Params, const FPCGAttributeAnalysisCommonDispatchData& Data)
	{
		Params.AttributeToCountId = Data.AttributeToCountId;
		Params.OutputValueAttributeId = Data.OutputValueAttributeId;
		Params.OutputCountAttributeId = Data.OutputCountAttributeId;
		Params.EmitPerDataCounts = Data.bEmitPerDataCounts ? 1u : 0u;
	}
}

/**
 * Data Interface to marshal Count Unique Values kernel data to GPU.
 * Acts as the canonical base for attribute-analysis DIs: provides the four shared scalar UPROPERTYs, the shared HLSL getter
 * emission, and the shared shader-function-prefix hook. Kernel-specific subclasses (e.g. SMSpawner Analysis) override the
 * prefix and append extras to GetSupportedInputs/GetHLSL.
 */
UCLASS(ClassGroup = (Procedural))
class UPCGCountUniqueAttributeValuesDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGCountUniqueAttributeValues"); }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	FName GetAttributeToCountName() const { return AttributeToCountName; }
	void SetAttributeToCountName(FName InAttributeToCountName) { AttributeToCountName = InAttributeToCountName; }

	bool GetEmitPerDataCounts() const { return bEmitPerDataCounts; }
	void SetEmitPerDataCounts(bool bInEmitPerDataCounts) { bEmitPerDataCounts = bInEmitPerDataCounts; }

	bool GetOutputRawBuffer() const { return bOutputRawBuffer; }
	void SetOutputRawBuffer(bool bInOutputRawBuffer) { bOutputRawBuffer = bInOutputRawBuffer; }

protected:
	/** Kernel-specific shader function prefix (e.g. "CountUniqueValues", "SMSpawnerAnalysis"). */
	virtual const TCHAR* GetShaderFunctionPrefix() const { return TEXT("CountUniqueValues"); }

protected:
	UPROPERTY()
	FName AttributeToCountName;

	UPROPERTY()
	bool bEmitPerDataCounts = true;

	UPROPERTY()
	bool bOutputRawBuffer = false;
};

UCLASS()
class UPCGCountUniqueAttributeValuesDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

public:
	FName AttributeToCountName;

	int32 AttributeToCountId = INDEX_NONE;
	int32 OutputValueAttributeId = INDEX_NONE;
	int32 OutputCountAttributeId = INDEX_NONE;

	bool bEmitPerDataCounts = true;
	bool bOutputRawBuffer = false;
};

/**
 * Render proxy carrying only the common dispatch data. Suitable for the Count Unique Values kernel and any future
 * attribute-analysis kernel whose Provider has no extras beyond what FPCGAttributeAnalysisCommonDispatchData represents.
 */
class FPCGCountUniqueAttributeValuesProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGCountUniqueAttributeValuesProviderProxy(FPCGAttributeAnalysisCommonDispatchData InData)
		: Data(InData)
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGAttributeAnalysisCommonShaderParameters;

	FPCGAttributeAnalysisCommonDispatchData Data;
};
