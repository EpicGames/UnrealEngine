// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDataNumDataInterface.h"

#include "PCGContext.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/BuiltInKernels/PCGDataNumKernel.h"
#include "Elements/PCGDataNum.h"

#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataNumDataInterface)

void UPCGDataNumDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("DataNum_GetOutputAttributeId"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Int));
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGDataNumDataInterfaceParameters,)
	SHADER_PARAMETER(int32, OutputAttributeId)
END_SHADER_PARAMETER_STRUCT()

void UPCGDataNumDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGDataNumDataInterfaceParameters>(UID);
}

void UPCGDataNumDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	const TMap<FString, FStringFormatArg> DataNumArgs = {{TEXT("DataInterfaceName"), InDataInterfaceName}};

	OutHLSL += FString::Format(TEXT(
		"int {DataInterfaceName}_OutputAttributeId;\n"
		"\n"
		"int DataNum_GetOutputAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_OutputAttributeId;\n"
		"}\n"
	), DataNumArgs);
}

UComputeDataProvider* UPCGDataNumDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGDataNumDataProvider>();
}

void UPCGDataNumDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, const uint64 InInputMask, const uint64 InOutputMask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataNumDataProvider::Initialize);

	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGDataNumSettings* Settings = CastChecked<UPCGDataNumSettings>(GetProducerKernel()->GetSettings());
	OutputAttributeName = Settings->OutputAttributeName;
}

FComputeDataProviderRenderProxy* UPCGDataNumDataProvider::GetRenderProxy()
{
	FPCGDataNumDataProviderProxy::FDataNumData_RenderThread ProxyData;
	ProxyData.OutputAttributeId = OutputAttributeId;
	return new FPCGDataNumDataProviderProxy(std::move(ProxyData));
}

bool UPCGDataNumDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataNumDataProvider::PrepareForExecute_GameThread);
	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	OutputAttributeId = InBinding->GetAttributeId(OutputAttributeName, EPCGKernelAttributeType::Int);
	check(OutputAttributeId != INDEX_NONE);

	return true;
}

void UPCGDataNumDataProvider::Reset()
{
	Super::Reset();

	OutputAttributeName = NAME_None;
	OutputAttributeId = INDEX_NONE;
}

bool FPCGDataNumDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGDataNumDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.OutputAttributeId = Data.OutputAttributeId;
	}
}