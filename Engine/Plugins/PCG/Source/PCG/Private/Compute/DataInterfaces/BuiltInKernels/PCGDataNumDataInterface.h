// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "PCGDataNumDataInterface.generated.h"

class FPCGDataNumDataInterfaceParameters;

/** Data Interface to marshal UPCGDataNumSettings settings to the GPU. */
UCLASS(ClassGroup = (Procedural))
class UPCGDataNumDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	virtual TCHAR const* GetClassName() const override { return TEXT("PCGDataNum"); }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface
};

UCLASS()
class UPCGDataNumDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	FName OutputAttributeName = NAME_None;
	int32 OutputAttributeId = INDEX_NONE;
};

class FPCGDataNumDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FDataNumData_RenderThread
	{
		int32 OutputAttributeId = INDEX_NONE;
	};

	FPCGDataNumDataProviderProxy(FDataNumData_RenderThread InData)
		: Data(MoveTemp(InData)) {}

	//~ Begin FComputeDataProviderRenderProxy Interface
	virtual bool IsValid(FValidationData const& InValidationData) const override;
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGDataNumDataInterfaceParameters;

	FDataNumData_RenderThread Data;
};
