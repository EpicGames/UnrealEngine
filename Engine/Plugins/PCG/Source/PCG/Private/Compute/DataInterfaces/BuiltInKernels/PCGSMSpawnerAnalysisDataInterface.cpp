// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGSMSpawnerAnalysisDataInterface.h"

#include "PCGGraphExecutionStateInterface.h"
#include "PCGParamData.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGDataDescription.h"
#include "Compute/BuiltInKernels/PCGAttributeAnalysisKernelBase.h"

#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHIResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SystemTextures.h"
#include "Algo/MaxElement.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Engine/AssetManager.h"
#include "Engine/StaticMesh.h"
#include "Engine/StreamableManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSMSpawnerAnalysisDataInterface)

void UPCGSMSpawnerAnalysisDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	Super::GetSupportedInputs(OutFunctions);

	const FString Prefix(GetShaderFunctionPrefix());

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetBoundsMin"))
		.AddReturnType(EShaderFundamentalType::Float, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetBoundsExtent"))
		.AddReturnType(EShaderFundamentalType::Float, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetCullingCellExtent"))
		.AddReturnType(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetNumCullingCells"))
		.AddReturnType(EShaderFundamentalType::Int, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetNumCullingCellsTotal"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetMeshBoundsMin"))
		.AddReturnType(EShaderFundamentalType::Float, 3) // Local bounds min
		.AddParam(EShaderFundamentalType::Uint); // InAttributeValue (primitive table index)

	OutFunctions.AddDefaulted_GetRef()
		.SetName(Prefix + TEXT("_GetMeshBoundsMax"))
		.AddReturnType(EShaderFundamentalType::Float, 3) // Local bounds max
		.AddParam(EShaderFundamentalType::Uint); // InAttributeValue (primitive table index)
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGSMSpawnerAnalysisDataInterfaceParameters,)
	SHADER_PARAMETER(int32, AttributeToCountId)
	SHADER_PARAMETER(int32, OutputValueAttributeId)
	SHADER_PARAMETER(int32, OutputCountAttributeId)
	SHADER_PARAMETER(uint32, EmitPerDataCounts)
	SHADER_PARAMETER(FVector3f, BoundsMin)
	SHADER_PARAMETER(FVector3f, BoundsExtent)
	SHADER_PARAMETER(float, CullingCellExtent)
	SHADER_PARAMETER(FIntVector, NumCullingCells)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector4f>, MeshBoundsMin)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector4f>, MeshBoundsMax)
END_SHADER_PARAMETER_STRUCT()

void UPCGSMSpawnerAnalysisDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGSMSpawnerAnalysisDataInterfaceParameters>(UID);
}

void UPCGSMSpawnerAnalysisDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	Super::GetHLSL(OutHLSL, InDataInterfaceName);

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
		{ TEXT("Prefix"), FString(GetShaderFunctionPrefix()) },
	};

	OutHLSL += FString::Format(TEXT(
		"float3 {DataInterfaceName}_BoundsMin;\n"
		"float3 {DataInterfaceName}_BoundsExtent;\n"
		"float {DataInterfaceName}_CullingCellExtent;\n"
		"int3 {DataInterfaceName}_NumCullingCells;\n"
		"StructuredBuffer<float4> {DataInterfaceName}_MeshBoundsMin;\n"
		"StructuredBuffer<float4> {DataInterfaceName}_MeshBoundsMax;\n"
		"\n"
		"float3 {Prefix}_GetBoundsMin_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_BoundsMin;\n"
		"}\n"
		"\n"
		"float3 {Prefix}_GetBoundsExtent_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_BoundsExtent;\n"
		"}\n"
		"\n"
		"float {Prefix}_GetCullingCellExtent_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_CullingCellExtent;\n"
		"}\n"
		"\n"
		"int3 {Prefix}_GetNumCullingCells_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_NumCullingCells;\n"
		"}\n"
		"\n"
		"int {Prefix}_GetNumCullingCellsTotal_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_NumCullingCells.x * {DataInterfaceName}_NumCullingCells.y * {DataInterfaceName}_NumCullingCells.z;\n"
		"}\n"
		"\n"
		"float3 {Prefix}_GetMeshBoundsMin_{DataInterfaceName}(uint InAttributeValue)\n"
		"{\n"
		"	return {DataInterfaceName}_MeshBoundsMin[InAttributeValue].xyz;\n"
		"}\n"
		"\n"
		"float3 {Prefix}_GetMeshBoundsMax_{DataInterfaceName}(uint InAttributeValue)\n"
		"{\n"
		"	return {DataInterfaceName}_MeshBoundsMax[InAttributeValue].xyz;\n"
		"}\n"
	), TemplateArgs);
}

UComputeDataProvider* UPCGSMSpawnerAnalysisDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGSMSpawnerAnalysisDataProvider>();
}

void UPCGSMSpawnerAnalysisDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSMSpawnerAnalysisDataProvider::Initialize);

	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGSMSpawnerAnalysisDataInterface* DataInterface = CastChecked<UPCGSMSpawnerAnalysisDataInterface>(InDataInterface);

	MeshAttributeName = DataInterface->GetMeshAttributeName();

	UPCGDataBinding* Binding = CastChecked<UPCGDataBinding>(InBinding);
	BoundsMin = FVector3f(Binding->GetExecutionSource()->GetExecutionState().GetBounds().Min);
	BoundsExtent = FVector3f(Binding->GetExecutionSource()->GetExecutionState().GetBounds().GetSize());

	NumCullingCells = Binding->GetMaxNumCullingCells(DataInterface->GetBucketingCellExtent());
	CullingCellExtent = static_cast<double>(DataInterface->GetBucketingCellExtent());
}

bool UPCGSMSpawnerAnalysisDataProvider::PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSMSpawnerAnalysisDataProvider::PerformPreExecuteReadbacks_GameThread);
	check(InBinding);

	if (!Super::PerformPreExecuteReadbacks_GameThread(InBinding))
	{
		return false;
	}

	if (MeshAttributeName.IsNone())
	{
		// Not in by-primitive-data mode, nothing to read back.
		return true;
	}

	if (UniqueValueTableDataIndex == INDEX_NONE)
	{
		UniqueValueTableDataIndex = GetProducerKernel() ? InBinding->GetFirstInputDataIndex(GetProducerKernel(), PCGAttributeAnalysisKernelConstants::UniqueValueTablePinLabel) : INDEX_NONE;
	}

	// The primitive table is CPU data, so ReadbackInputDataToCPU returns true immediately.
	return UniqueValueTableDataIndex == INDEX_NONE || InBinding->ReadbackInputDataToCPU(UniqueValueTableDataIndex);
}

bool UPCGSMSpawnerAnalysisDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSMSpawnerAnalysisDataProvider::PrepareForExecute_GameThread);
	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	// Mesh bounds: Step 1 populates MeshPaths for whichever spawning mode is active.
	// A shared block then deduplicates and kicks off async loading.
	// Step 2 waits for the load and fills MeshBoundsMin/Max from MeshPaths.
	if (!MeshAttributeName.IsNone() && UniqueValueTableDataIndex != INDEX_NONE)
	{
		// Step 1 (by-primitive-data): extract mesh paths from the primitive table metadata.
		if (!bMeshPathsCreated)
		{
			const bool bIndexValid = InBinding->GetInputDataCollection().TaggedData.IsValidIndex(UniqueValueTableDataIndex);
			const UPCGParamData* PrimitiveData = bIndexValid ? Cast<UPCGParamData>(InBinding->GetInputDataCollection().TaggedData[UniqueValueTableDataIndex].Data) : nullptr;
			const UPCGMetadata* PrimitiveMetadata = PrimitiveData ? PrimitiveData->ConstMetadata() : nullptr;

			if (const FPCGMetadataAttributeBase* MeshAttributeBase = PrimitiveMetadata ? PrimitiveMetadata->GetConstAttribute(MeshAttributeName) : nullptr)
			{
				const bool bIsSoftObjectPath = MeshAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FSoftObjectPath>::Id;
				const bool bIsString = MeshAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FString>::Id;

				if (bIsSoftObjectPath || bIsString)
				{
					const FPCGMetadataAttribute<FSoftObjectPath>* SOPAttr = bIsSoftObjectPath ? static_cast<const FPCGMetadataAttribute<FSoftObjectPath>*>(MeshAttributeBase) : nullptr;
					const FPCGMetadataAttribute<FString>* StringAttr = bIsString ? static_cast<const FPCGMetadataAttribute<FString>*>(MeshAttributeBase) : nullptr;

					const int32 NumElements = PrimitiveMetadata->GetItemCountForChild();
					MeshPaths.SetNum(NumElements);

					for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
					{
						const PCGMetadataValueKey ValueKey = MeshAttributeBase->GetValueKey(ElementIndex);
						MeshPaths[ElementIndex] = SOPAttr ? SOPAttr->GetValue(ValueKey) : FSoftObjectPath(StringAttr->GetValue(ValueKey));
					}
				}
			}

			bMeshPathsCreated = true;
		}
	}
	else if (MeshAttributeName.IsNone() && AttributeToCountId != INDEX_NONE)
	{
		// Step 1 (by-attribute): resolve mesh paths from the string table keyed by attribute value.
		if (!bMeshPathsCreated)
		{
			const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);

			if (InputDesc)
			{
				TArray<int32> UniqueKeys;
				InputDesc->GetUniqueStringKeyValues(AttributeToCountId, UniqueKeys);

				if (!UniqueKeys.IsEmpty())
				{
					const int32* MaxKeyValue = Algo::MaxElement(UniqueKeys);
					MeshPaths.SetNum(*MaxKeyValue + 1);

					const TArray<FString>& StringTable = InBinding->GetStringTable();
					for (int32 Key : UniqueKeys)
					{
						if (StringTable.IsValidIndex(Key))
						{
							MeshPaths[Key] = FSoftObjectPath(StringTable[Key]);
						}
					}
				}
			}

			bMeshPathsCreated = true;
		}
	}

	// Shared: deduplicate MeshPaths and kick off async load (fires once, after Step 1 sets bMeshPathsCreated).
	if (bMeshPathsCreated && !bMeshBoundsLoaded && !LoadHandle)
	{
		TSet<FSoftObjectPath> UniquePaths;
		UniquePaths.Reserve(MeshPaths.Num());
		for (const FSoftObjectPath& Path : MeshPaths)
		{
			if (!Path.IsNull())
			{
				UniquePaths.Add(Path);
			}
		}
		TArray<FSoftObjectPath> PathsToLoad = UniquePaths.Array();

		if (!PathsToLoad.IsEmpty())
		{
			LoadHandle = UAssetManager::Get().GetStreamableManager().RequestAsyncLoad(MoveTemp(PathsToLoad));
		}
	}

	// Step 2 (shared): wait for async load to complete, then fill MeshBoundsMin/Max from MeshPaths.
	if (bMeshPathsCreated && !bMeshBoundsLoaded)
	{
		if (LoadHandle && !LoadHandle->HasLoadCompleted())
		{
			return false;
		}

		const int32 NumMeshes = MeshPaths.Num();

		// Build cache: one ResolveObject()/GetBounds() call per unique mesh path.
		TMap<FSoftObjectPath, FBox> BoundsCache;
		BoundsCache.Reserve(NumMeshes);
		for (const FSoftObjectPath& Path : MeshPaths)
		{
			if (!Path.IsNull() && !BoundsCache.Contains(Path))
			{
				const UStaticMesh* Mesh = Cast<UStaticMesh>(Path.ResolveObject());
				BoundsCache.Add(Path, Mesh ? Mesh->GetBounds().GetBox() : FBox(EForceInit::ForceInit));
			}
		}

		MeshBoundsMin.SetNumUninitialized(NumMeshes);
		MeshBoundsMax.SetNumUninitialized(NumMeshes);
		for (int32 Index = 0; Index < NumMeshes; ++Index)
		{
			if (const FBox* FoundBounds = BoundsCache.Find(MeshPaths[Index]); FoundBounds && FoundBounds->IsValid)
			{
				MeshBoundsMin[Index] = FVector4f((float)FoundBounds->Min.X, (float)FoundBounds->Min.Y, (float)FoundBounds->Min.Z, /*Unused*/0.0f);
				MeshBoundsMax[Index] = FVector4f((float)FoundBounds->Max.X, (float)FoundBounds->Max.Y, (float)FoundBounds->Max.Z, /*Unused*/0.0f);
			}
			else
			{
				MeshBoundsMin[Index] = FVector4f::Zero();
				MeshBoundsMax[Index] = FVector4f::Zero();
			}
		}

		bMeshBoundsLoaded = true;
		LoadHandle.Reset();
	}

	return true;
}

FComputeDataProviderRenderProxy* UPCGSMSpawnerAnalysisDataProvider::GetRenderProxy()
{
	FPCGSMSpawnerAnalysisProviderProxy::FSMSpawnerAnalysisData_RenderThread Data =
	{
		.Common =
		{
			.AttributeToCountId = AttributeToCountId,
			.OutputValueAttributeId = OutputValueAttributeId,
			.OutputCountAttributeId = OutputCountAttributeId,
			.bEmitPerDataCounts = bEmitPerDataCounts,
		},
		.BoundsMin = BoundsMin,
		.BoundsExtent = BoundsExtent,
		.CullingCellExtent = static_cast<float>(CullingCellExtent),
		.NumCullingCells = NumCullingCells,
		.MeshBoundsMin = MeshBoundsMin,
		.MeshBoundsMax = MeshBoundsMax,
	};

	return new FPCGSMSpawnerAnalysisProviderProxy(MoveTemp(Data));
}

void UPCGSMSpawnerAnalysisDataProvider::Reset()
{
	BoundsMin = FVector3f::ZeroVector;
	BoundsExtent = FVector3f::ZeroVector;
	NumCullingCells = FIntVector();
	CullingCellExtent = 0.0;
	MeshAttributeName = NAME_None;
	UniqueValueTableDataIndex = INDEX_NONE;
	MeshPaths.Reset();
	MeshBoundsMin.Reset();
	MeshBoundsMax.Reset();
	LoadHandle.Reset();
	bMeshPathsCreated = false;
	bMeshBoundsLoaded = false;

	Super::Reset();
}

bool FPCGSMSpawnerAnalysisProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return (InValidationData.ParameterStructSize == sizeof(FParameters));
}

void FPCGSMSpawnerAnalysisProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	MeshBoundsMin = MoveTemp(Data.MeshBoundsMin);
	MeshBoundsMax = MoveTemp(Data.MeshBoundsMax);

	if (!MeshBoundsMin.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(MeshBoundsMin.GetTypeSize(), MeshBoundsMin.Num());
		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGSMSpawnerAnalysis_MeshBoundsMin"));
		MeshBoundsMinBufferSRV = GraphBuilder.CreateSRV(Buffer);
		GraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(MeshBoundsMin));
	}
	else
	{
		MeshBoundsMinBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, MeshBoundsMin.GetTypeSize())));
	}

	if (!MeshBoundsMax.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(MeshBoundsMax.GetTypeSize(), MeshBoundsMax.Num());
		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGSMSpawnerAnalysis_MeshBoundsMax"));
		MeshBoundsMaxBufferSRV = GraphBuilder.CreateSRV(Buffer);
		GraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(MeshBoundsMax));
	}
	else
	{
		MeshBoundsMaxBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, MeshBoundsMax.GetTypeSize())));
	}
}

void FPCGSMSpawnerAnalysisProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];

		PCGAttributeAnalysis::WriteCommonDispatchParameters(Parameters, Data.Common);

		Parameters.BoundsMin = Data.BoundsMin;
		Parameters.BoundsExtent = Data.BoundsExtent;
		Parameters.CullingCellExtent = Data.CullingCellExtent;
		Parameters.NumCullingCells = Data.NumCullingCells;
		Parameters.MeshBoundsMin = MeshBoundsMinBufferSRV;
		Parameters.MeshBoundsMax = MeshBoundsMaxBufferSRV;
	}
}
