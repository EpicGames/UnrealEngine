// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/BuiltInKernels/PCGCountUniqueAttributeValuesDataInterface.h"

#include "PCGSMSpawnerAnalysisDataInterface.generated.h"

class FPCGSMSpawnerAnalysisDataInterfaceParameters;
struct FStreamableHandle;

/** Data Interface to marshal SM Spawner Analysis kernel data to GPU. */
UCLASS(ClassGroup = (Procedural))
class UPCGSMSpawnerAnalysisDataInterface : public UPCGCountUniqueAttributeValuesDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGSMSpawnerAnalysis"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	int32 GetBucketingCellExtent() const { return BucketingCellExtent; }
	void SetBucketingCellExtent(int32 InGrid) { BucketingCellExtent = InGrid; }

	FName GetMeshAttributeName() const { return MeshAttributeName; }
	/** Attribute name in the primitive table that holds the static mesh path. Only set when spawning by primitive data. */
	void SetMeshAttributeName(FName InMeshAttributeName) { MeshAttributeName = InMeshAttributeName; }

protected:
	virtual const TCHAR* GetShaderFunctionPrefix() const override { return TEXT("SMSpawnerAnalysis"); }

protected:
	UPROPERTY()
	int32 BucketingCellExtent = 0;

	/** Attribute name in the primitive table that holds the static mesh path. Only set when spawning by primitive data. */
	UPROPERTY()
	FName MeshAttributeName;
};

UCLASS()
class UPCGSMSpawnerAnalysisDataProvider : public UPCGCountUniqueAttributeValuesDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding) override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

public:
	FVector3f BoundsMin = FVector3f::ZeroVector;
	FVector3f BoundsExtent = FVector3f::ZeroVector;
	double CullingCellExtent = 0.0;
	FIntVector NumCullingCells = FIntVector(1, 1, 1);

	/** Mesh attribute name in the primitive table. Empty when not in by-primitive-data mode. */
	FName MeshAttributeName;

	/** Data index for UniqueValueTable pin (primitive table). INDEX_NONE if not in by-primitive-data mode. */
	int32 UniqueValueTableDataIndex = INDEX_NONE;

	/** Mesh paths indexed by attribute value (= element index in primitive table). */
	TArray<FSoftObjectPath> MeshPaths;

	/** Per-primitive local mesh bounds min, indexed by attribute value. Uploaded as SRV. */
	TArray<FVector4f> MeshBoundsMin;

	/** Per-primitive local mesh bounds max, indexed by attribute value. Uploaded as SRV. */
	TArray<FVector4f> MeshBoundsMax;

	/** Keeps loaded mesh assets alive. */
	TSharedPtr<FStreamableHandle> LoadHandle;

	bool bMeshPathsCreated = false;
	bool bMeshBoundsLoaded = false;
};

class FPCGSMSpawnerAnalysisProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FSMSpawnerAnalysisData_RenderThread
	{
		FPCGAttributeAnalysisCommonDispatchData Common;
		FVector3f BoundsMin = FVector3f::ZeroVector;
		FVector3f BoundsExtent = FVector3f::ZeroVector;
		float CullingCellExtent = 0.0f;
		FIntVector NumCullingCells = FIntVector(1, 1, 1);
		TArray<FVector4f> MeshBoundsMin;
		TArray<FVector4f> MeshBoundsMax;
	};

	FPCGSMSpawnerAnalysisProviderProxy(FSMSpawnerAnalysisData_RenderThread InData)
		: Data(MoveTemp(InData))
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGSMSpawnerAnalysisDataInterfaceParameters;

	FSMSpawnerAnalysisData_RenderThread Data;

	TArray<FVector4f> MeshBoundsMin;
	FRDGBufferSRVRef MeshBoundsMinBufferSRV = nullptr;

	TArray<FVector4f> MeshBoundsMax;
	FRDGBufferSRVRef MeshBoundsMaxBufferSRV = nullptr;
};
