// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGCopyPointsAnalysisDataInterface.h"

#include "PCGContext.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGDataDescription.h"
#include "Elements/PCGCopyPoints.h"

#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHIResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCopyPointsAnalysisDataInterface)

void UPCGCopyPointsAnalysisDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	Super::GetSupportedInputs(OutFunctions);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("CopyPoints_GetMatchAttributeId"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Int));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("CopyPoints_GetSelectedFlagAttributeId"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Int));
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGCopyPointsAnalysisDataInterfaceParameters,)
	SHADER_PARAMETER(int32, MatchAttributeId)
	SHADER_PARAMETER(int32, SelectedFlagAttributeId)
	SHADER_PARAMETER(uint32, bCopyEachSourceOnEveryTarget)
END_SHADER_PARAMETER_STRUCT()

void UPCGCopyPointsAnalysisDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGCopyPointsAnalysisDataInterfaceParameters>(UID);
}

void UPCGCopyPointsAnalysisDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	Super::GetHLSL(OutHLSL, InDataInterfaceName);

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	OutHLSL += FString::Format(TEXT(
		"int {DataInterfaceName}_MatchAttributeId;\n"
		"int {DataInterfaceName}_SelectedFlagAttributeId;\n"
		"\n"
		"int CopyPoints_GetMatchAttributeId_{DataInterfaceName}() { return {DataInterfaceName}_MatchAttributeId; }\n"
		"int CopyPoints_GetSelectedFlagAttributeId_{DataInterfaceName}() { return {DataInterfaceName}_SelectedFlagAttributeId; }\n"
		), TemplateArgs);
}

UComputeDataProvider* UPCGCopyPointsAnalysisDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGCopyPointsAnalysisDataProvider>();
}

bool UPCGCopyPointsAnalysisDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	const FPCGKernelParams* KernelParams = InBinding->GetCachedKernelParams(GetProducerKernel());

	if (!ensure(KernelParams))
	{
		return true;
	}

	Params.bCopyEachSourceOnEveryTarget = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bCopyEachSourceOnEveryTarget));

	// Resolve attribute IDs from incoming data.
	const FName MatchAttributeName = KernelParams->GetValueName(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, MatchAttribute));
	Params.MatchAttributeId = InBinding->GetAttributeId(MatchAttributeName, EPCGKernelAttributeType::Int);
	ensure(Params.MatchAttributeId != INDEX_NONE);

	Params.SelectedFlagAttributeId = InBinding->GetAttributeId(PCGCopyPointsConstants::SelectedFlagAttributeName, EPCGKernelAttributeType::Bool);
	ensure(Params.SelectedFlagAttributeId != INDEX_NONE);

	return true;
}

FComputeDataProviderRenderProxy* UPCGCopyPointsAnalysisDataProvider::GetRenderProxy()
{
	return new FPCGCopyPointsAnalysisDataProviderProxy(Params);
}

void UPCGCopyPointsAnalysisDataProvider::Reset()
{
	Params = {};
	Super::Reset();
}

bool FPCGCopyPointsAnalysisDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters)
		&& Params.MatchAttributeId != INDEX_NONE
		&& Params.SelectedFlagAttributeId != INDEX_NONE;
}

void FPCGCopyPointsAnalysisDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.MatchAttributeId = Params.MatchAttributeId;
		Parameters.SelectedFlagAttributeId = Params.SelectedFlagAttributeId;
		Parameters.bCopyEachSourceOnEveryTarget = Params.bCopyEachSourceOnEveryTarget ? 1 : 0;
	}
}
