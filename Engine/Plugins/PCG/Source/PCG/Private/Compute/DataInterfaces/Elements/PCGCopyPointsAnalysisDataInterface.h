// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "PCGCopyPointsAnalysisDataInterface.generated.h"

class FPCGCopyPointsAnalysisDataInterfaceParameters;
class UPCGCopyPointsSettings;

struct FPCGCopyPointsAnalysisParams
{
	int32 MatchAttributeId = INDEX_NONE;
	int32 SelectedFlagAttributeId = INDEX_NONE;
	bool bCopyEachSourceOnEveryTarget = true;
};

/** Data Interface to marshal Copy Points settings to the GPU. */
UCLASS(ClassGroup = (Procedural))
class UPCGCopyPointsAnalysisDataInterface : public UPCGKernelParamsDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGCopyPointsAnalysis"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface
};

UCLASS()
class UPCGCopyPointsAnalysisDataProvider : public UPCGKernelParamsDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

public:
	FPCGCopyPointsAnalysisParams Params;
};

class FPCGCopyPointsAnalysisDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGCopyPointsAnalysisDataProviderProxy(const FPCGCopyPointsAnalysisParams& InParams)
		: Params(InParams)
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGCopyPointsAnalysisDataInterfaceParameters;

	FPCGCopyPointsAnalysisParams Params;
};
