// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGCopyPointsDataInterface.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Elements/PCGCopyPoints.h"
#include "Elements/PCGCopyPointsKernel.h"

#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHIResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SystemTextures.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCopyPointsDataInterface)

void UPCGCopyPointsDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	Super::GetSupportedInputs(OutFunctions);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("CopyPoints_GetSourceAndTargetDataIndices"))
		.AddParam(EShaderFundamentalType::Uint) // InOutputDataIndex
		.AddParam(EShaderFundamentalType::Uint) // InSourceDataCount
		.AddParam(EShaderFundamentalType::Uint) // InTargetDataCount
		.AddParam(EShaderFundamentalType::Uint, 0, 0, EShaderParamModifier::Out) // OutSourceIndex
		.AddParam(EShaderFundamentalType::Uint, 0, 0, EShaderParamModifier::Out); // OutTargetIndex
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGCopyPointsDataInterfaceParameters,)
	SHADER_PARAMETER(uint32, RotationInheritance)
	SHADER_PARAMETER(uint32, bApplyTargetRotationToPositions)
	SHADER_PARAMETER(uint32, ScaleInheritance)
	SHADER_PARAMETER(uint32, bApplyTargetScaleToPositions)
	SHADER_PARAMETER(uint32, ColorInheritance)
	SHADER_PARAMETER(uint32, SeedInheritance)
	SHADER_PARAMETER(uint32, AttributeInheritance)
	SHADER_PARAMETER(uint32, bCopyEachSourceOnEveryTarget)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FUintVector2>, SourceAndTargetDataIndices)
END_SHADER_PARAMETER_STRUCT()

void UPCGCopyPointsDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGCopyPointsDataInterfaceParameters>(UID);
}

void UPCGCopyPointsDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	OutPermutationVector.AddPermutation(TEXT("USE_INPUT_DATA_INDICES_BUFFER"), 2);
}

void UPCGCopyPointsDataInterface::GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const
{
	Super::GetDefines(OutDefinitionSet);

	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_COPY_POINTS_INHERITANCE_MODE_Relative"), FString::FromInt(static_cast<int32>(EPCGCopyPointsInheritanceMode::Relative))));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_COPY_POINTS_INHERITANCE_MODE_Source"), FString::FromInt(static_cast<int32>(EPCGCopyPointsInheritanceMode::Source))));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_COPY_POINTS_INHERITANCE_MODE_Target"), FString::FromInt(static_cast<int32>(EPCGCopyPointsInheritanceMode::Target))));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_COPY_POINTS_METADATA_INHERITANCE_MODE_SourceFirst"), FString::FromInt(static_cast<int32>(EPCGCopyPointsMetadataInheritanceMode::SourceFirst))));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_COPY_POINTS_METADATA_INHERITANCE_MODE_TargetFirst"), FString::FromInt(static_cast<int32>(EPCGCopyPointsMetadataInheritanceMode::TargetFirst))));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_COPY_POINTS_METADATA_INHERITANCE_MODE_SourceOnly"), FString::FromInt(static_cast<int32>(EPCGCopyPointsMetadataInheritanceMode::SourceOnly))));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_COPY_POINTS_METADATA_INHERITANCE_MODE_TargetOnly"), FString::FromInt(static_cast<int32>(EPCGCopyPointsMetadataInheritanceMode::TargetOnly))));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_COPY_POINTS_METADATA_INHERITANCE_MODE_None"), FString::FromInt(static_cast<int32>(EPCGCopyPointsMetadataInheritanceMode::None))));
}

void UPCGCopyPointsDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	Super::GetHLSL(OutHLSL, InDataInterfaceName);

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};
	
	OutHLSL += FString::Format(TEXT(
		"StructuredBuffer<uint2> {DataInterfaceName}_SourceAndTargetDataIndices;\n"
		"\n"
		"void CopyPoints_GetSourceAndTargetDataIndices_{DataInterfaceName}(uint InOutputDataIndex, int InSourceDataCount, int InTargetDataCount, out uint OutSourceIndex, out uint OutTargetIndex)\n"
		"{\n"
		"#if USE_INPUT_DATA_INDICES_BUFFER\n"
		"	const uint2 Indices = {DataInterfaceName}_SourceAndTargetDataIndices[InOutputDataIndex];\n"
		"	OutSourceIndex = Indices[0];\n"
		"	OutTargetIndex = Indices[1];\n"
		"#else\n"
		// Note: Because bCopyEachSourceOnEveryTarget requires a graph split to be overidden, it is safe to directly access instead of calling GetOverridableValue(), which can only be called from kernel source.
		"	if ({DataInterfaceName}_bCopyEachSourceOnEveryTarget)\n"
		"	{\n"
		"		OutSourceIndex = InOutputDataIndex / InTargetDataCount;\n"
		"		OutTargetIndex = InOutputDataIndex % InTargetDataCount;\n"
		"	}\n"
		"	else\n"
		"	{\n"
		"		OutSourceIndex = clamp(InOutputDataIndex, 0u, (uint)(InSourceDataCount - 1));\n"
		"		OutTargetIndex = clamp(InOutputDataIndex, 0u, (uint)(InTargetDataCount - 1));\n"
		"	}\n"
		"#endif\n"
		"}\n"
		), TemplateArgs);
}

UComputeDataProvider* UPCGCopyPointsDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGCopyPointsDataProvider>();
}

bool UPCGCopyPointsDataProvider::PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCopyPointsDataProvider::PerformPreExecuteReadbacks_GameThread);
	check(InBinding);

	if (!Super::PerformPreExecuteReadbacks_GameThread(InBinding))
	{
		return false;
	}

	KernelParams = InBinding->GetCachedKernelParams(GetProducerKernel());

	if (!ensure(KernelParams))
	{
		return true;
	}

	const bool bMatchBasedOnAttribute = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bMatchBasedOnAttribute));

	if (!bMatchBasedOnAttribute)
	{
		// If we're not matching based on attribute then we don't need to do any readbacks.
		return true;
	}

	const int32 AnalysisDataIndex = GetProducerKernel() ? InBinding->GetFirstInputDataIndex(GetProducerKernel(), PCGCopyPointsConstants::SelectedFlagsPinLabel) : INDEX_NONE;
	if (AnalysisDataIndex == INDEX_NONE)
	{
		// Analysis data was not produced, nothing to read back.
		return true;
	}

	// Readback analysis data - poll until readback complete.
	return InBinding->ReadbackInputDataToCPU(AnalysisDataIndex);
}

bool UPCGCopyPointsDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCopyPointsDataProvider::PrepareForExecute_GameThread);
	check(InBinding);

	if (!ensure(KernelParams))
	{
		return true;
	}

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	const bool bMatchBasedOnAttribute = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bMatchBasedOnAttribute));

	if (bMatchBasedOnAttribute)
	{
		const int32 AnalysisDataIndex = InBinding->GetFirstInputDataIndex(GetProducerKernel(), PCGCopyPointsConstants::SelectedFlagsPinLabel);
		if (AnalysisDataIndex == INDEX_NONE)
		{
			return true;
		}

		const TSharedPtr<const FPCGDataCollectionDesc> SourcePinDesc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), PCGCopyPointsConstants::SourcePointsLabel, /*bIsInput=*/true);
		const TSharedPtr<const FPCGDataCollectionDesc> TargetPinDesc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), PCGCopyPointsConstants::TargetPointsLabel, /*bIsInput=*/true);

		if (!ensure(SourcePinDesc) || !ensure(TargetPinDesc))
		{
			return true;
		}

		const TConstArrayView<FPCGDataDesc> SourceDataDescs = SourcePinDesc->GetDataDescriptions();
		const TConstArrayView<FPCGDataDesc> TargetDataDescs = TargetPinDesc->GetDataDescriptions();

		const bool bCopyEachSourceOnEveryTarget = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bCopyEachSourceOnEveryTarget));

		const int32 NumOutputs = bCopyEachSourceOnEveryTarget ? SourceDataDescs.Num() * TargetDataDescs.Num() : FMath::Max(SourceDataDescs.Num(), TargetDataDescs.Num());
		SourceAndTargetDataIndices.Reserve(NumOutputs);

		const UPCGParamData* AnalysisResultsData = Cast<UPCGParamData>(InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data);
		const UPCGMetadata* AnalysisMetadata = AnalysisResultsData ? AnalysisResultsData->ConstMetadata() : nullptr;
		const FPCGMetadataAttributeBase* AnalysisAttributeBase = AnalysisMetadata ? AnalysisMetadata->GetConstAttribute(PCGCopyPointsConstants::SelectedFlagAttributeName) : nullptr;
		const int32 NumElements = AnalysisAttributeBase ? AnalysisMetadata->GetItemCountForChild() : INDEX_NONE;
		const FPCGMetadataAttribute<bool>* SelectedFlagsAttribute = ((NumElements == NumOutputs) && AnalysisAttributeBase && (AnalysisAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<bool>::Id)) ? static_cast<const FPCGMetadataAttribute<bool>*>(AnalysisAttributeBase) : nullptr;

		if (SelectedFlagsAttribute)
		{
			if (bCopyEachSourceOnEveryTarget)
			{
				for (int32 S = 0; S < SourceDataDescs.Num(); ++S)
				{
					for (int32 T = 0; T < TargetDataDescs.Num(); ++T)
					{
						if (SelectedFlagsAttribute->GetValue(S * TargetDataDescs.Num() + T))
						{
							SourceAndTargetDataIndices.Emplace(S, T);
						}
					}
				}
			}
			else
			{
				for (int32 OutputIndex = 0; OutputIndex < NumOutputs; ++OutputIndex)
				{
					const int32 S = FMath::Clamp(OutputIndex, 0, SourceDataDescs.Num() - 1);
					const int32 T = FMath::Clamp(OutputIndex, 0, TargetDataDescs.Num() - 1);

					if (SelectedFlagsAttribute->GetValue(OutputIndex))
					{
						SourceAndTargetDataIndices.Emplace(S, T);
					}
				}
			}
		}
	}

	return true;
}

FComputeDataProviderRenderProxy* UPCGCopyPointsDataProvider::GetRenderProxy()
{
	FPCGCopyPointsDataProviderProxy::FCopyPointsData_RenderThread ProxyData;

	if (ensure(KernelParams))
	{
		const EPCGCopyPointsInheritanceMode RotationInheritance = KernelParams->GetValueEnum<EPCGCopyPointsInheritanceMode>(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, RotationInheritance));
		const bool bApplyTargetRotationToPositions = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bApplyTargetRotationToPositions));
		const EPCGCopyPointsInheritanceMode ScaleInheritance = KernelParams->GetValueEnum<EPCGCopyPointsInheritanceMode>(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, ScaleInheritance));
		const bool bApplyTargetScaleToPositions = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bApplyTargetScaleToPositions));
		const EPCGCopyPointsInheritanceMode ColorInheritance = KernelParams->GetValueEnum<EPCGCopyPointsInheritanceMode>(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, ColorInheritance));
		const EPCGCopyPointsInheritanceMode SeedInheritance = KernelParams->GetValueEnum<EPCGCopyPointsInheritanceMode>(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, SeedInheritance));
		const EPCGCopyPointsMetadataInheritanceMode AttributeInheritance = KernelParams->GetValueEnum<EPCGCopyPointsMetadataInheritanceMode>(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, AttributeInheritance));
		const bool bCopyEachSourceOnEveryTarget = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bCopyEachSourceOnEveryTarget));

		ProxyData.RotationInheritance = static_cast<uint32>(RotationInheritance);
		ProxyData.ApplyTargetRotationToPositions = bApplyTargetRotationToPositions ? 1u : 0u;
		ProxyData.ScaleInheritance = static_cast<uint32>(ScaleInheritance);
		ProxyData.ApplyTargetScaleToPositions = bApplyTargetScaleToPositions ? 1u : 0u;
		ProxyData.ColorInheritance = static_cast<uint32>(ColorInheritance);
		ProxyData.SeedInheritance = static_cast<uint32>(SeedInheritance);
		ProxyData.AttributeInheritance = static_cast<uint32>(AttributeInheritance);
		ProxyData.bCopyEachSourceOnEveryTarget = static_cast<uint32>(bCopyEachSourceOnEveryTarget);
		ProxyData.SourceAndTargetDataIndices = SourceAndTargetDataIndices;
	}

	return new FPCGCopyPointsDataProviderProxy(MoveTemp(ProxyData));
}

void UPCGCopyPointsDataProvider::Reset()
{
	Super::Reset();

	KernelParams = nullptr;
	SourceAndTargetDataIndices.Empty();
}

bool FPCGCopyPointsDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

struct FCopyPointsDataInterfacePermutationIds
{
	FCopyPointsDataInterfacePermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		{
			static FString Name(TEXT("USE_INPUT_DATA_INDICES_BUFFER"));
			static uint32 Hash = GetTypeHash(Name);
			UseInputDataIndicesBuffer = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
	}

	uint32 UseInputDataIndicesBuffer = 0;
};

void FPCGCopyPointsDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	FCopyPointsDataInterfacePermutationIds PermutationIds(InOutPermutationData.PermutationVector);

	for (int32 InvocationIndex = 0; InvocationIndex < InOutPermutationData.NumInvocations; ++InvocationIndex)
	{
		// Select permutation based on whether we have data indices or not.
		InOutPermutationData.PermutationIds[InvocationIndex] |= (Data.SourceAndTargetDataIndices.IsEmpty() ? 0 : PermutationIds.UseInputDataIndicesBuffer);
	}
}

void FPCGCopyPointsDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCopyPointsDataProviderProxy::AllocateResources);
	LLM_SCOPE_BYTAG(PCG);

	if (!Data.SourceAndTargetDataIndices.IsEmpty())
	{
		const FRDGBufferDesc IndexPairsDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FUintVector2), Data.SourceAndTargetDataIndices.Num());

		SourceAndTargetDataIndicesBuffer = GraphBuilder.CreateBuffer(IndexPairsDesc, TEXT("PCGCopyPoints_SourceAndTargetDataIndices"));
		SourceAndTargetDataIndicesBufferSRV = GraphBuilder.CreateSRV(SourceAndTargetDataIndicesBuffer);

		GraphBuilder.QueueBufferUpload(SourceAndTargetDataIndicesBuffer, MakeArrayView(Data.SourceAndTargetDataIndices));
	}
	else
	{
		// Fallback
		SourceAndTargetDataIndicesBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FUintVector2))));
	}
}

void FPCGCopyPointsDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.RotationInheritance = Data.RotationInheritance;
		Parameters.bApplyTargetRotationToPositions = Data.ApplyTargetRotationToPositions;
		Parameters.ScaleInheritance = Data.ScaleInheritance;
		Parameters.bApplyTargetScaleToPositions = Data.ApplyTargetScaleToPositions;
		Parameters.ColorInheritance = Data.ColorInheritance;
		Parameters.SeedInheritance = Data.SeedInheritance;
		Parameters.AttributeInheritance = Data.AttributeInheritance;
		Parameters.bCopyEachSourceOnEveryTarget = Data.bCopyEachSourceOnEveryTarget;
		Parameters.SourceAndTargetDataIndices = SourceAndTargetDataIndicesBufferSRV;
	}
}
