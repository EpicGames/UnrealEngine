// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "PCGCopyPointsDataInterface.generated.h"

class FPCGCopyPointsDataInterfaceParameters;
struct FPCGKernelParams;

/**
 * Data Interface to marshal Copy Points settings to the GPU.
 * This subclass handles its own marshalling of overridable params because it also manages RDG buffer
 * resources, shader permutations, and defines that are tightly coupled to the override param layout.
 * New nodes should prefer the automatic UPCGKernelParamsDataInterface (see its class comment).
 */
UCLASS(ClassGroup = (Procedural))
class UPCGCopyPointsDataInterface : public UPCGKernelParamsDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGCopyPoints"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	void GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface
};

UCLASS()
class UPCGCopyPointsDataProvider : public UPCGKernelParamsDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding) override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

public:
	/** The kernel params are owned by the KernelParamsCache, so we are not responsible for its lifetime. */
	const FPCGKernelParams* KernelParams = nullptr;

	TArray<FUintVector2> SourceAndTargetDataIndices;
};

class FPCGCopyPointsDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FCopyPointsData_RenderThread
	{
		uint32 RotationInheritance = 0;
		uint32 ApplyTargetRotationToPositions = 0;
		uint32 ScaleInheritance = 0;
		uint32 ApplyTargetScaleToPositions = 0;
		uint32 ColorInheritance = 0;
		uint32 SeedInheritance = 0;
		uint32 AttributeInheritance = 0;
		uint32 bCopyEachSourceOnEveryTarget = 0;
		TArray<FUintVector2> SourceAndTargetDataIndices;
	};

	FPCGCopyPointsDataProviderProxy(FCopyPointsData_RenderThread InData)
		: Data(MoveTemp(InData))
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherPermutations(FPermutationData& InOutPermutationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGCopyPointsDataInterfaceParameters;

	FCopyPointsData_RenderThread Data;

	FRDGBufferRef SourceAndTargetDataIndicesBuffer = nullptr;
	FRDGBufferSRVRef SourceAndTargetDataIndicesBufferSRV = nullptr;
};
