// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGCullPointsOutsideActorBoundsDataInterface.h"

#include "PCGContext.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Elements/PCGCullPointsOutsideActorBounds.h"
#include "Compute/Elements/PCGCullPointsOutsideActorBoundsKernel.h"

#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHIResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCullPointsOutsideActorBoundsDataInterface)

void UPCGCullPointsOutsideActorBoundsDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("CullPointsOutsideActorBounds_GetBoundsExpansion"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Float));
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGCullPointsOutsideActorBoundsDataInterfaceParameters,)
	SHADER_PARAMETER(float, BoundsExpansion)
END_SHADER_PARAMETER_STRUCT()

void UPCGCullPointsOutsideActorBoundsDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGCullPointsOutsideActorBoundsDataInterfaceParameters>(UID);
}

void UPCGCullPointsOutsideActorBoundsDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> CullPointsOutsideActorBoundsArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	OutHLSL += FString::Format(TEXT(
		"float {DataInterfaceName}_BoundsExpansion;\n"
		"\n"
		"float CullPointsOutsideActorBounds_GetBoundsExpansion_{DataInterfaceName}() { return {DataInterfaceName}_BoundsExpansion; }\n"
		), CullPointsOutsideActorBoundsArgs);
}

UComputeDataProvider* UPCGCullPointsOutsideActorBoundsDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGCullPointsOutsideActorBoundsDataProvider>();
}

FComputeDataProviderRenderProxy* UPCGCullPointsOutsideActorBoundsDataProvider::GetRenderProxy()
{
	const UPCGCullPointsOutsideActorBoundsSettings* Settings = CastChecked<UPCGCullPointsOutsideActorBoundsSettings>(GetProducerKernel()->GetSettings());

	FPCGCullPointsOutsideActorBoundsDataProviderProxy::FCullPointsOutsideActorBoundsData_RenderThread ProxyData;
	ProxyData.BoundsExpansion = Settings->BoundsExpansion;

	return new FPCGCullPointsOutsideActorBoundsDataProviderProxy(MoveTemp(ProxyData));
}

bool FPCGCullPointsOutsideActorBoundsDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGCullPointsOutsideActorBoundsDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.BoundsExpansion = Data.BoundsExpansion;
	}
}
