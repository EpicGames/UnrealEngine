// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "PCGCullPointsOutsideActorBoundsDataInterface.generated.h"

class FPCGCullPointsOutsideActorBoundsDataInterfaceParameters;

/** Data Interface to marshal CullPointsOutsideActorBounds settings to the GPU. */
UCLASS(ClassGroup = (Procedural))
class UPCGCullPointsOutsideActorBoundsDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGCullPointsOutsideActorBounds"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface
};

UCLASS()
class UPCGCullPointsOutsideActorBoundsDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FPCGCullPointsOutsideActorBoundsDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FCullPointsOutsideActorBoundsData_RenderThread
	{
		float BoundsExpansion = 0.0f;
	};

	FPCGCullPointsOutsideActorBoundsDataProviderProxy(FCullPointsOutsideActorBoundsData_RenderThread InData)
		: Data(MoveTemp(InData))
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGCullPointsOutsideActorBoundsDataInterfaceParameters;

	FCullPointsOutsideActorBoundsData_RenderThread Data;
};
