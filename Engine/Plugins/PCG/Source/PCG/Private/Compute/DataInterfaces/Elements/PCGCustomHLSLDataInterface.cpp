// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGCustomHLSLDataInterface.h"

#include "PCGContext.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Compute/Elements/PCGCustomHLSLKernel.h"

#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCustomHLSLDataInterface)

BEGIN_SHADER_PARAMETER_STRUCT(FPCGCustomHLSLDataInterfaceParameters,)
	SHADER_PARAMETER(int32, NumElements)
	SHADER_PARAMETER(int32, NumElementsX)
	SHADER_PARAMETER(int32, NumElementsY)
	SHADER_PARAMETER(int32, NumElementsZ)
	SHADER_PARAMETER(int32, ThreadCountMultiplier)
	SHADER_PARAMETER(int32, FixedThreadCount)
END_SHADER_PARAMETER_STRUCT()

void UPCGCustomHLSLDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGCustomHLSLDataInterfaceParameters>(UID);
}

void UPCGCustomHLSLDataInterface::GetShaderHash(FString& InOutKey) const
{
	// Required because the shader code didn't change when we added this param to the shader struct.
	InOutKey += TEXT("PCGCustomHLSL_DI_v2_AddNumElementsZToShaderParamStruct");
}

UComputeDataProvider* UPCGCustomHLSLDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGCustomHLSLDataProvider>();
}

FComputeDataProviderRenderProxy* UPCGCustomHLSLDataProvider::GetRenderProxy()
{
	FPCGCustomHLSLDataProviderProxy::FCustomHLSLData ProxyData;

	if (ensure(KernelParams))
	{
		ProxyData.NumElements = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElements));
		ProxyData.NumElementsX = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsX));
		ProxyData.NumElementsY = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsY));
		ProxyData.NumElementsZ = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsZ));
		ProxyData.ThreadCountMultiplier = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, ThreadCountMultiplier));
		ProxyData.FixedThreadCount = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, FixedThreadCount));
	}

	return new FPCGCustomHLSLDataProviderProxy(MoveTemp(ProxyData));
}

void UPCGCustomHLSLDataProvider::Reset()
{
	Super::Reset();

	KernelParams = nullptr;
}

bool UPCGCustomHLSLDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCustomHLSLDataProvider::PrepareForExecute_GameThread);
	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	KernelParams = InBinding->GetCachedKernelParams(GetProducerKernel());

	return true;
}

bool FPCGCustomHLSLDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGCustomHLSLDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumElements = Data.NumElements;
		Parameters.NumElementsX = Data.NumElementsX;
		Parameters.NumElementsY = Data.NumElementsY;
		Parameters.NumElementsZ = Data.NumElementsZ;
		Parameters.ThreadCountMultiplier = Data.ThreadCountMultiplier;
		Parameters.FixedThreadCount = Data.FixedThreadCount;
	}
}
