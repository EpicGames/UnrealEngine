// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "PCGCustomHLSLDataInterface.generated.h"

class FPCGCustomHLSLDataInterfaceParameters;
struct FPCGKernelParams;

/**
 * Data Interface to marshal UPCGCustomHLSLSettings settings to the GPU.
 * This subclass handles its own marshalling of overridable params because it has bespoke thread
 * configuration logic that is tightly coupled to the override param layout.
 * New nodes should prefer the automatic UPCGKernelParamsDataInterface (see its class comment).
 */
UCLASS(ClassGroup = (Procedural))
class UPCGCustomHLSLDataInterface : public UPCGKernelParamsDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	virtual TCHAR const* GetClassName() const override { return TEXT("PCGCustomHLSL"); }
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual void GetShaderHash(FString& InOutKey) const override;
	virtual UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface
};

UCLASS()
class UPCGCustomHLSLDataProvider : public UPCGKernelParamsDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	/** The kernel params are owned by the KernelParamsCache, so we are not responsible for its lifetime. */
	const FPCGKernelParams* KernelParams = nullptr;
};

class FPCGCustomHLSLDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FCustomHLSLData
	{
		int32 NumElements = 0;
		int32 NumElementsX = 0;
		int32 NumElementsY = 0;
		int32 NumElementsZ = 0;
		int32 ThreadCountMultiplier = 0;
		int32 FixedThreadCount = 0;
	};

	FPCGCustomHLSLDataProviderProxy(FCustomHLSLData InData)
		: Data(MoveTemp(InData)) {}

	//~ Begin FComputeDataProviderRenderProxy Interface
	virtual bool IsValid(FValidationData const& InValidationData) const override;
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGCustomHLSLDataInterfaceParameters;

	FCustomHLSLData Data;
};
