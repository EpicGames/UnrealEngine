// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGMetadataPartitionDataInterface.h"

#include "PCGParamData.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/BuiltInKernels/PCGCountUniqueAttributeValuesKernel.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Elements/Metadata/PCGMetadataPartition.h"
#include "Elements/Metadata/PCGMetadataPartitionKernel.h"

#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHIResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataPartitionDataInterface)

void UPCGMetadataPartitionDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("MetadataPartition_GetPartitionAttributeId"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("MetadataPartition_GetOutputIndices"))
		.AddParam(EShaderFundamentalType::Uint) // InInputDataIndex
		.AddParam(EShaderFundamentalType::Int) // InAttributeValue
		.AddParam(EShaderFundamentalType::Uint, 0, 0, EShaderParamModifier::Out) // OutDataIndex
		.AddParam(EShaderFundamentalType::Uint, 0, 0, EShaderParamModifier::Out); // OutElementIndex
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGMetadataPartitionDataInterfaceParameters,)
	SHADER_PARAMETER(int32, PartitionAttributeId)
	SHADER_PARAMETER(int32, NumPartitions)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int32>, AttributeValueToOutputDataIndex)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint32>, WriteCounters)
END_SHADER_PARAMETER_STRUCT()

void UPCGMetadataPartitionDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGMetadataPartitionDataInterfaceParameters>(UID);
}

void UPCGMetadataPartitionDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	OutHLSL += FString::Format(TEXT(
		"StructuredBuffer<int> {DataInterfaceName}_AttributeValueToOutputDataIndex;\n"
		"int {DataInterfaceName}_PartitionAttributeId;\n"
		"int {DataInterfaceName}_NumPartitions;\n"
		"RWStructuredBuffer<uint> {DataInterfaceName}_WriteCounters;\n"
		"\n"
		"int MetadataPartition_GetPartitionAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_PartitionAttributeId;\n"
		"}\n"
		"\n"
		"int MetadataPartition_GetOutputDataIndex_{DataInterfaceName}(uint InInputDataIndex, int InAttributeValue)\n"
		"{\n"
		"	const int TableIndex = {DataInterfaceName}_NumPartitions * (int)InInputDataIndex + InAttributeValue;"
		"	return {DataInterfaceName}_AttributeValueToOutputDataIndex[TableIndex];\n"
		"}\n"
		"\n"
		"void MetadataPartition_GetOutputIndices_{DataInterfaceName}(uint InInputDataIndex, int InAttributeValue, out uint OutDataIndex, out uint OutElementIndex)\n"
		"{\n"
		"	const int CounterIndex = (int)InInputDataIndex * {DataInterfaceName}_NumPartitions + InAttributeValue;\n"
		"	\n"
		"	OutDataIndex = MetadataPartition_GetOutputDataIndex_{DataInterfaceName}(InInputDataIndex, InAttributeValue);"
		"	InterlockedAdd({DataInterfaceName}_WriteCounters[CounterIndex], 1u, OutElementIndex);\n"
		"}\n"
	), TemplateArgs);
}

UComputeDataProvider* UPCGMetadataPartitionDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGMetaDataPartitionDataProvider>();
}

bool UPCGMetaDataPartitionDataProvider::PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMetaDataPartitionDataProvider::PerformPreExecuteReadbacks_GameThread);
	check(InBinding);

	if (!Super::PerformPreExecuteReadbacks_GameThread(InBinding))
	{
		return false;
	}

	if (AnalysisDataIndices.IsEmpty())
	{
		AnalysisDataIndices = InBinding->GetPinInputDataIndices(GetProducerKernel(), PCGMetadataPartitionConstants::ElementCountsPinLabel);

		if (AnalysisDataIndices.IsEmpty())
		{
			UE_LOGF(LogPCG, Verbose, "UPCGMetaDataPartitionDataProvider: No analysis data found from count kernel.");
			return true;
		}

	}

	bool bAllReadBack = true;

	for (int32 AnalysisDataIndex : AnalysisDataIndices)
	{
		// Readback analysis data - poll until readback complete (return true).
		if (!InBinding->ReadbackInputDataToCPU(AnalysisDataIndex))
		{
			bAllReadBack = false;
		}
	}

	return bAllReadBack;
}

bool UPCGMetaDataPartitionDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMetadataPartitionDataInterface::PrepareForExecute_GameThread);
	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	if (AnalysisDataIndices.IsEmpty())
	{
		UE_LOGF(LogPCG, Verbose, "UPCGMetaDataPartitionDataProvider: No analysis data found from count kernel.");
		return true;
	}

	const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	if (!InputDesc)
	{
		UE_LOGF(LogPCG, Error, "UPCGMetaDataPartitionDataProvider: Could not retrieve input data description.");
		return true;
	}

	if (AnalysisDataIndices.Num() != InputDesc->GetDataDescriptions().Num())
	{
		UE_LOGF(LogPCG, Error, "UPCGMetaDataPartitionDataProvider: Number of received analysis data items (%d) does not match input data count (%d).",
			AnalysisDataIndices.Num(), InputDesc->GetDataDescriptions().Num());
		return true;
	}

	UniqueStringKeyValuesPerInputData.SetNum(AnalysisDataIndices.Num());
	MaxAttributeValuePerInputData.SetNumZeroed(AnalysisDataIndices.Num());

	for (int InputDataIndex = 0; InputDataIndex < AnalysisDataIndices.Num(); ++InputDataIndex)
	{
		const uint32 AnalysisDataIndex = AnalysisDataIndices[InputDataIndex];

		if (!ensure(InBinding->GetInputDataCollection().TaggedData.IsValidIndex(AnalysisDataIndex)))
		{
			continue;
		}

		const UPCGParamData* AnalysisResultsData = Cast<UPCGParamData>(InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data);
		const UPCGMetadata* AnalysisMetadata = AnalysisResultsData ? AnalysisResultsData->ConstMetadata() : nullptr;

		const FPCGMetadataAttributeBase* AnalysisCountAttributeBase = AnalysisMetadata ? AnalysisMetadata->GetConstAttribute(PCGAttributeAnalysisKernelConstants::ValueCountAttributeName) : nullptr;

		if (AnalysisCountAttributeBase && AnalysisCountAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<int32>::Id)
		{
			const FPCGMetadataAttribute<int32>* CountAttribute = static_cast<const FPCGMetadataAttribute<int32>*>(AnalysisCountAttributeBase);

			const int32 NumPartitionsInAnalysisData = AnalysisMetadata->GetItemCountForChild();

			// TODO: Range based get would scale better.
			for (int64 MetadataKey = 0; MetadataKey < NumPartitionsInAnalysisData; ++MetadataKey)
			{
				if (CountAttribute->GetValue(MetadataKey) > 0)
				{
					// The count kernel emits a count for each attribute value index, and the value attribute in the analysis data is just the identity map,
					// so we don't bother reading the value attribute.
					const int32 AttributeValue = static_cast<int32>(MetadataKey);

					UniqueStringKeyValuesPerInputData[InputDataIndex].AddUnique(AttributeValue);

					MaxAttributeValuePerInputData[InputDataIndex] = FMath::Max(MaxAttributeValuePerInputData[InputDataIndex], AttributeValue);
				}
			}
		}
		else
		{
			if (InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data->IsA<UPCGProxyForGPUData>())
			{
				UE_LOGF(LogPCG, Error, "Data was not read back.");
			}
			else
			{
				UE_LOGF(LogPCG, Error, "Analysis results attributes missing from read back data.");
			}

			return true;
		}
	}

	const UPCGMetadataPartitionSettings* Settings = Cast<UPCGMetadataPartitionSettings>(GetProducerKernel()->GetSettings());
	check(Settings);

	if (ensure(Settings->PartitionAttributeSelectors.Num() == 1 && Settings->PartitionAttributeSelectors[0].IsBasicAttribute()))
	{
		const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), PCGPinConstants::DefaultInputLabel, /*bIsInput*/true);

		if (!ensure(InputDataDesc))
		{
			return true;
		}

		PartitionAttributeId = InBinding->GetAttributeId(Settings->PartitionAttributeSelectors[0].GetAttributeName(), EPCGKernelAttributeType::StringKey);

		NumInputData = InputDataDesc->GetDataDescriptions().Num();
	}

	return true;
}

FComputeDataProviderRenderProxy* UPCGMetaDataPartitionDataProvider::GetRenderProxy()
{
	return new FPCGMetaDataPartitionProviderProxy(PartitionAttributeId, NumInputData/*, MaxAttributeValuePerInputData*/, UniqueStringKeyValuesPerInputData);
}

void UPCGMetaDataPartitionDataProvider::Reset()
{
	PartitionAttributeId = INDEX_NONE;
	NumInputData = INDEX_NONE;
	MaxAttributeValuePerInputData.Empty(MaxAttributeValuePerInputData.Num());
	UniqueStringKeyValuesPerInputData.Empty(UniqueStringKeyValuesPerInputData.Num());
	AnalysisDataIndices.Empty(AnalysisDataIndices.Num());

	Super::Reset();
}

FPCGMetaDataPartitionProviderProxy::FPCGMetaDataPartitionProviderProxy(int32 InPartitionAttributeId, int32 InNumInputData, TArray<TArray<int32>> InUniqueStringKeyValuesPerInputData)
	: PartitionAttributeId(InPartitionAttributeId)
	, NumInputData(InNumInputData)
	, UniqueStringKeyValuesPerInputData(MoveTemp(InUniqueStringKeyValuesPerInputData))
{
	int32 MaxAttributeValueAcrossAllData = INDEX_NONE;

	for (const TArray<int32>& UniqueStringKeyValues : UniqueStringKeyValuesPerInputData)
	{
		for (int32 StringKeyValue : UniqueStringKeyValues)
		{
			MaxAttributeValueAcrossAllData = FMath::Max(MaxAttributeValueAcrossAllData, StringKeyValue);
		}
	}

	// We'll allocate space for all possible partitions across all input data.
	NumPartitions = MaxAttributeValueAcrossAllData + 1;
}

bool FPCGMetaDataPartitionProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	for (const TArray<int32>& UniqueStringKeyValues : UniqueStringKeyValuesPerInputData)
	{
		if (UniqueStringKeyValues.IsEmpty())
		{
			UE_LOGF(LogPCG, Warning, "Metadata Partition will not execute, proxy invalid due to missing UniqueStringKeyValues.");
			return false;
		}
	}

	if (NumPartitions <= 0)
	{
		UE_LOGF(LogPCG, Warning, "Metadata Partition will not execute, no partitions to generate.");
		return false;
	}

	if (PartitionAttributeId < 0)
	{
		UE_LOGF(LogPCG, Warning, "Metadata Partition will not execute, partition attribute ID not set.");
		return false;
	}

	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGMetaDataPartitionProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	LLM_SCOPE_BYTAG(PCG);

	// For each data, precompute a target partition index for each attribute value.
	{
		const int32 NumIndicesToAllocate = FMath::Max(1, NumInputData * NumPartitions);

		TArray<int32, TInlineAllocator<32>> Indices;
		Indices.SetNumZeroed(NumIndicesToAllocate);

		FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), NumIndicesToAllocate);
		Desc.Usage |= BUF_SourceCopy;

		FRDGBufferRef AttributeValueToOutputDataIndex = GraphBuilder.CreateBuffer(Desc, TEXT("PCGAttributeValueToOutputDataIndex"));
		AttributeValueToOutputDataIndexSRV = GraphBuilder.CreateSRV(AttributeValueToOutputDataIndex);

		int OutputDataIndex = 0;

		for (int InputDataIndex = 0; InputDataIndex < NumInputData; ++InputDataIndex)
		{
			for (int ValueIndex = 0; ValueIndex < NumPartitions; ++ValueIndex)
			{
				const int PartitionIndex = UniqueStringKeyValuesPerInputData[InputDataIndex].IndexOfByKey(ValueIndex);
				// Each possible attribute value that is present in the input data should go to one output data.
				const int OutputIndex = (PartitionIndex == -1) ? -1 : (OutputDataIndex++);
				Indices[InputDataIndex * NumPartitions + ValueIndex] = OutputIndex;
			}
		}

		GraphBuilder.QueueBufferUpload(AttributeValueToOutputDataIndex, MakeConstArrayView(Indices));
	}

	// For each data, set up and 0-initialize a write counter for each partition that GPU threads can atomically increment and use as write index.
	{
		const int32 NumWriteCountersToAllocate = FMath::Max(1, NumInputData * NumPartitions);

		FRDGBufferDesc CountersDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NumWriteCountersToAllocate);
		CountersDesc.Usage |= BUF_SourceCopy;

		TArray<uint32, TInlineAllocator<32>> Zeroes;
		Zeroes.SetNumZeroed(NumWriteCountersToAllocate);

		FRDGBufferRef WriteCounters = GraphBuilder.CreateBuffer(CountersDesc, TEXT("PCGDataCollection_Counters"));
		GraphBuilder.QueueBufferUpload(WriteCounters, MakeConstArrayView(Zeroes));

		WriteCountersUAV = GraphBuilder.CreateUAV(WriteCounters);
	}
}

void FPCGMetaDataPartitionProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];

		Parameters.PartitionAttributeId = PartitionAttributeId;
		Parameters.NumPartitions = NumPartitions;
		Parameters.AttributeValueToOutputDataIndex = AttributeValueToOutputDataIndexSRV;
		Parameters.WriteCounters = WriteCountersUAV;
	}
}
