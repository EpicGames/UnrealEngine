// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "PCGMetadataPartitionDataInterface.generated.h"

class FPCGMetadataPartitionDataInterfaceParameters;

/** Data Interface to marshal Attribute Partition settings to the GPU. */
UCLASS(ClassGroup = (Procedural))
class UPCGMetadataPartitionDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGMetadataPartition"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface
};

UCLASS()
class UPCGMetaDataPartitionDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding) override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

public:
	int32 PartitionAttributeId = INDEX_NONE;

	int32 NumInputData = INDEX_NONE;

	TArray<int32> MaxAttributeValuePerInputData;

	TArray<TArray<int32>> UniqueStringKeyValuesPerInputData;

	TArray<int32> AnalysisDataIndices;
};

class FPCGMetaDataPartitionProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGMetaDataPartitionProviderProxy(int32 InPartitionAttributeId, int32 InNumInputData, TArray<TArray<int32>> InUniqueStringKeyValuesPerInputData);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData);
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGMetadataPartitionDataInterfaceParameters;

	int32 PartitionAttributeId = INDEX_NONE;

	int32 NumInputData = INDEX_NONE;

	int32 NumPartitions = INDEX_NONE;

	TArray<TArray<int32>> UniqueStringKeyValuesPerInputData;

	FRDGBufferSRVRef AttributeValueToOutputDataIndexSRV = nullptr;

	FRDGBufferUAVRef WriteCountersUAV = nullptr;
};
