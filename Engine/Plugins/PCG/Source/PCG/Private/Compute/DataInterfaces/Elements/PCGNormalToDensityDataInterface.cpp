// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGNormalToDensityDataInterface.h"

#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/Elements/PCGNormalToDensityKernel.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHIResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGNormalToDensityDataInterface)

void UPCGNormalToDensityDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("NormalToDensity_GetNormal"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Float, 3));
	
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("NormalToDensity_GetOffset"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Float));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("NormalToDensity_GetStrength"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Float));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("NormalToDensity_GetDensityMode"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint));
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGNormalToDensityDataInterfaceParameters,)
	SHADER_PARAMETER(FVector3f, Normal)
	SHADER_PARAMETER(float, Offset)
	SHADER_PARAMETER(float, Strength)
	SHADER_PARAMETER(uint32, DensityMode)
END_SHADER_PARAMETER_STRUCT()

void UPCGNormalToDensityDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGNormalToDensityDataInterfaceParameters>(UID);
}

void UPCGNormalToDensityDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	OutHLSL += FString::Format(TEXT(
		"float3 {DataInterfaceName}_Normal;\n"
		"float {DataInterfaceName}_Offset;\n"
		"float {DataInterfaceName}_Strength;\n"
		"uint {DataInterfaceName}_DensityMode;\n"
		"\n"
		"float3 NormalToDensity_GetNormal_{DataInterfaceName}() { return {DataInterfaceName}_Normal; }\n"
		"float NormalToDensity_GetOffset_{DataInterfaceName}() { return {DataInterfaceName}_Offset; }\n"
		"float NormalToDensity_GetStrength_{DataInterfaceName}() { return {DataInterfaceName}_Strength; }\n"
		"uint NormalToDensity_GetDensityMode_{DataInterfaceName}() { return {DataInterfaceName}_DensityMode; }\n"
		), TemplateArgs);
}

UComputeDataProvider* UPCGNormalToDensityDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGNormalToDensityProvider>();
}

FComputeDataProviderRenderProxy* UPCGNormalToDensityProvider::GetRenderProxy()
{
	const UPCGNormalToDensitySettings* Settings = CastChecked<UPCGNormalToDensitySettings>(GetProducerKernel()->GetSettings());

	FPCGNormalToDensityDataProviderProxy::FNormalToDensityData_RenderThread ProxyData;
	ProxyData.Normal = Settings->Normal;
	ProxyData.Offset = Settings->Offset;
	ProxyData.Strength = Settings->Strength;
	ProxyData.DensityMode = Settings->DensityMode;

	return new FPCGNormalToDensityDataProviderProxy(MoveTemp(ProxyData));
}

bool FPCGNormalToDensityDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGNormalToDensityDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.Normal = static_cast<FVector3f>(Data.Normal);
		Parameters.Offset = static_cast<float>(Data.Offset);
		Parameters.Strength = static_cast<float>(Data.Strength);
		Parameters.DensityMode = static_cast<uint32>(Data.DensityMode);
	}
}
