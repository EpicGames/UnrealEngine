// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "Elements/PCGNormalToDensity.h"

#include "PCGNormalToDensityDataInterface.generated.h"

class FPCGNormalToDensityDataInterfaceParameters;

/** Data Interface to marshal Normal to density settings to the GPU. */
UCLASS(ClassGroup = (Procedural))
class UPCGNormalToDensityDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGNormalToDensity"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface
};

UCLASS()
class UPCGNormalToDensityProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FPCGNormalToDensityDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FNormalToDensityData_RenderThread
	{
		// The normal to compare against
		FVector Normal = FVector::UpVector;

		// This biases the value towards or against the normal (positive or negative)
		double Offset = 0.0;

		// This applies a curve to scale the result density with Result = Result^(1/Strength)
		double Strength = 1.0;

		// The operator to apply to the output density 
		PCGNormalToDensityMode DensityMode = PCGNormalToDensityMode::Set;
	};

	FPCGNormalToDensityDataProviderProxy(FNormalToDensityData_RenderThread InData)
		: Data(MoveTemp(InData))
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGNormalToDensityDataInterfaceParameters;

	FNormalToDensityData_RenderThread Data;
};
