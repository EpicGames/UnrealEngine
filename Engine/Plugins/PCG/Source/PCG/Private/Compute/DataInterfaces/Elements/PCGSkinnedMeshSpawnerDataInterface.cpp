// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGSkinnedMeshSpawnerDataInterface.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGManagedResource.h"
#include "PCGParamData.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/BuiltInKernels/PCGCountUniqueAttributeValuesKernel.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/Packing/PCGMeshSpawnerPacking.h"
#include "Compute/PrimitiveFactories/PCGPrimitiveFactoryISKMC.h"
#include "Elements/PCGSkinnedMeshSpawner.h"
#include "Elements/PCGSkinnedMeshSpawnerKernel.h"
#include "InstanceDataPackers/PCGSkinnedMeshInstanceDataPackerBase.h"
#include "MeshSelectors/PCGSkinnedMeshSelector.h"
#include "Helpers/PCGActorHelpers.h"

#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHIResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SystemTextures.h"
#include "Engine/SkinnedAsset.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Components/InstancedSkinnedMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSkinnedMeshSpawnerDataInterface)

#define LOCTEXT_NAMESPACE "PCGSkinnedMeshSpawnerDataInterface"

namespace PCGSkinnedMeshSpawnerDataInterface
{
	static const FText CouldNotLoadStaticMeshFormat = LOCTEXT("CouldNotLoadSkinnedMesh", "Could not load Skinned mesh from path '{0}'.");
	static const FText TooManyPrimitivesFormat = LOCTEXT("TooManyPrimitives", "Attempted to emit too many primitive components, terminated after creating '{0}'.");
	static const FText NoMeshEntriesFormat = LOCTEXT("NoMeshEntries", "No mesh entries provided in weighted mesh selector.");
}

void UPCGSkinnedMeshSpawnerDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetSelectorAttributeId"))
		.AddReturnType(EShaderFundamentalType::Uint); // Attribute id to get mesh path string key from, or invalid if we should use CDF instead.

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetNumAttributes"))
		.AddReturnType(EShaderFundamentalType::Uint); // Num attributes

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetNumPrimitives"))
		.AddReturnType(EShaderFundamentalType::Uint); // Num primitives

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_ShouldApplyBounds"))
		.AddReturnType(EShaderFundamentalType::Bool);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetPrimitiveMeshBoundsMin"))
		.AddReturnType(EShaderFundamentalType::Float, 3) // Local bounds min
		.AddParam(EShaderFundamentalType::Uint); // InPrimitiveIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetPrimitiveMeshBoundsMax"))
		.AddReturnType(EShaderFundamentalType::Float, 3) // Local bounds max
		.AddParam(EShaderFundamentalType::Uint); // InPrimitiveIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetAttributeIdOffsetStride"))
		.AddReturnType(EShaderFundamentalType::Uint, 4)
		.AddParam(EShaderFundamentalType::Uint); // InAttributeIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetPrimitiveStringKey"))
		.AddReturnType(EShaderFundamentalType::Int) // String key
		.AddParam(EShaderFundamentalType::Uint); // InPrimitiveIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetPrimitiveIndexFromPrimitiveId"))
		.AddReturnType(EShaderFundamentalType::Uint) // Primitive index
		.AddParam(EShaderFundamentalType::Int); // InMeshPathStringKey

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetPrimitiveSelectionCDF"))
		.AddReturnType(EShaderFundamentalType::Float) // CDF value
		.AddParam(EShaderFundamentalType::Uint); // InPrimitiveIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetSelectedMeshAttributeId"))
		.AddReturnType(EShaderFundamentalType::Uint); // Attribute id to output mesh path string key to

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetNumCullingCells"))
		.AddReturnType(EShaderFundamentalType::Int, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetNumCullingCellsTotal"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetBoundsMin"))
		.AddReturnType(EShaderFundamentalType::Float, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetBucketExtent"))
		.AddReturnType(EShaderFundamentalType::Float);
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGSkinnedMeshSpawnerDataInterfaceParameters,)
	SHADER_PARAMETER_ARRAY(FUintVector4, AttributeIdOffsetStrides, [UPCGSkinnedMeshSpawnerDataInterface::MAX_ATTRIBUTES])
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int32>, PrimitiveStringKeys)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector4f>, PrimitiveMeshBoundsMin)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector4f>, PrimitiveMeshBoundsMax)
	SHADER_PARAMETER_SCALAR_ARRAY(float, SelectionCDF, [PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER])
	SHADER_PARAMETER(uint32, NumAttributes)
	SHADER_PARAMETER(uint32, NumPrimitives)
	SHADER_PARAMETER(int32, SelectorAttributeId)
	SHADER_PARAMETER(uint32, ApplyBounds)
END_SHADER_PARAMETER_STRUCT()

void UPCGSkinnedMeshSpawnerDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGSkinnedMeshSpawnerDataInterfaceParameters>(UID);
}

void UPCGSkinnedMeshSpawnerDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
		{ TEXT("MaxAttributes"), MAX_ATTRIBUTES },
		{ TEXT("MaxPrimitives"), PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER },
	};

	OutHLSL += FString::Format(TEXT(
		"int {DataInterfaceName}_SelectorAttributeId;\n"
		"uint {DataInterfaceName}_NumAttributes;\n"
		"uint {DataInterfaceName}_NumPrimitives;\n"
		"uint {DataInterfaceName}_ApplyBounds;\n"
		"uint4 {DataInterfaceName}_AttributeIdOffsetStrides[{MaxAttributes}];\n"
		"StructuredBuffer<float4> {DataInterfaceName}_PrimitiveMeshBoundsMin;\n"
		"StructuredBuffer<float4> {DataInterfaceName}_PrimitiveMeshBoundsMax;\n"
		"StructuredBuffer<int> {DataInterfaceName}_PrimitiveStringKeys;\n"
		"DECLARE_SCALAR_ARRAY(float, {DataInterfaceName}_SelectionCDF, {MaxPrimitives});\n"
		"\n"
		"int SMSpawner_GetSelectorAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_SelectorAttributeId;\n"
		"}\n"
		"\n"
		"uint SMSpawner_GetNumAttributes_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_NumAttributes;\n"
		"}\n"
		"\n"
		"uint4 SMSpawner_GetAttributeIdOffsetStride_{DataInterfaceName}(uint InAttributeIndex)\n"
		"{\n"
		"	return {DataInterfaceName}_AttributeIdOffsetStrides[InAttributeIndex];\n"
		"}\n"
		"\n"
		"uint SMSpawner_GetNumPrimitives_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_NumPrimitives;\n"
		"}\n"
		"\n"
		"bool SMSpawner_ShouldApplyBounds_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_ApplyBounds > 0;\n"
		"}\n"
		"\n"
		"float3 SMSpawner_GetPrimitiveMeshBoundsMin_{DataInterfaceName}(uint InPrimitiveIndex)\n"
		"{\n"
		"	return {DataInterfaceName}_PrimitiveMeshBoundsMin[InPrimitiveIndex].xyz;\n"
		"}\n"
		"\n"
		"float3 SMSpawner_GetPrimitiveMeshBoundsMax_{DataInterfaceName}(uint InPrimitiveIndex)\n"
		"{\n"
		"	return {DataInterfaceName}_PrimitiveMeshBoundsMax[InPrimitiveIndex].xyz;\n"
		"}\n"
		"\n"
		"int SMSpawner_GetPrimitiveStringKey_{DataInterfaceName}(uint InPrimitiveIndex)\n"
		"{\n"
		"	return {DataInterfaceName}_PrimitiveStringKeys[InPrimitiveIndex];\n"
		"}\n"
		"\n"
		"uint SMSpawner_GetPrimitiveIndexFromPrimitiveId_{DataInterfaceName}(int InMeshPathStringKey)\n"
		"{\n"
		"	for (uint Index = 0; Index < {DataInterfaceName}_NumPrimitives; ++Index)\n"
		"	{\n"
		"		if ({DataInterfaceName}_PrimitiveStringKeys[Index] == InMeshPathStringKey)\n"
		"		{\n"
		"			return Index;\n"
		"		}\n"
		"	}\n"
		"	\n"
		"	return (uint)-1;\n"
		"}\n"
		"\n"
		"float SMSpawner_GetPrimitiveSelectionCDF_{DataInterfaceName}(uint InPrimitiveIndex)\n"
		"{\n"
		"	return GET_SCALAR_ARRAY_ELEMENT({DataInterfaceName}_SelectionCDF, InPrimitiveIndex);\n"
		"}\n"
		"\n"
		"int SMSpawner_GetSelectedMeshAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return -1; \n" // Not currently supported by skinned mesh spawner node. Stubbed here so that single spawner usf can be used.
		"}\n"
		"\n"
		// todo_pcg: Culling cell functions are not implemented for the skinned mesh spawner. Stubs return the
		// 'no culling' defaults so that the if (NumCullingCellsTotal > 1) block in the USF is skipped.
		"int3 SMSpawner_GetNumCullingCells_{DataInterfaceName}()\n"
		"{\n"
		"	return int3(1, 1, 1);\n"
		"}\n"
		"\n"
		"int SMSpawner_GetNumCullingCellsTotal_{DataInterfaceName}()\n"
		"{\n"
		"	return 1;\n"
		"}\n"
		"\n"
		"float3 SMSpawner_GetBoundsMin_{DataInterfaceName}()\n"
		"{\n"
		"	return float3(0, 0, 0);\n"
		"}\n"
		"\n"
		"float SMSpawner_GetBucketExtent_{DataInterfaceName}()\n"
		"{\n"
		"	return 0.0;\n"
		"}\n\n"
	), TemplateArgs);
}

UComputeDataProvider* UPCGSkinnedMeshSpawnerDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGSkinnedMeshSpawnerDataProvider>();
}

bool UPCGSkinnedMeshSpawnerDataProvider::PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSkinnedMeshSpawnerDataProvider::PerformPreExecuteReadbacks_GameThread);
	check(InBinding);

	if (!Super::PerformPreExecuteReadbacks_GameThread(InBinding))
	{
		return false;
	}

	const UPCGSkinnedMeshSpawnerSettings* Settings = CastChecked<UPCGSkinnedMeshSpawnerSettings>(GetProducerKernel()->GetSettings());

	// Obtain index of analysis data that we need to read back.
	if (AnalysisDataIndex == INDEX_NONE)
	{
		if (!ensure(Settings->MeshSelectorParameters))
		{
			// Non by-attribute selection does not need to do any readbacks.
			return true;
		}

		AnalysisDataIndex = GetProducerKernel() ? InBinding->GetFirstInputDataIndex(GetProducerKernel(), PCGSkinnedMeshSpawnerConstants::InstanceCountsPinLabel) : INDEX_NONE;

		if (AnalysisDataIndex == INDEX_NONE)
		{
			// No analysis data to read back.
			return true;
		}
	}

	// Readback analysis data - poll until readback complete (return true).
	if (!InBinding->ReadbackInputDataToCPU(AnalysisDataIndex))
	{
		return false;
	}

	const UPCGParamData* AnalysisResultsData = Cast<UPCGParamData>(InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data);
	const UPCGMetadata* AnalysisMetadata = AnalysisResultsData ? AnalysisResultsData->ConstMetadata() : nullptr;

	const FPCGMetadataAttributeBase* AnalysisValueAttributeBase = AnalysisMetadata ? AnalysisMetadata->GetConstAttribute(PCGAttributeAnalysisKernelConstants::ValueAttributeName) : nullptr;
	const FPCGMetadataAttributeBase* AnalysisCountAttributeBase = AnalysisMetadata ? AnalysisMetadata->GetConstAttribute(PCGAttributeAnalysisKernelConstants::ValueCountAttributeName) : nullptr;

	if (AnalysisValueAttributeBase && AnalysisValueAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<int32>::Id &&
		AnalysisCountAttributeBase && AnalysisCountAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<int32>::Id)
	{
		const FPCGMetadataAttribute<int32>* ValueAttribute = static_cast<const FPCGMetadataAttribute<int32>*>(AnalysisValueAttributeBase);
		const FPCGMetadataAttribute<int32>* CountAttribute = static_cast<const FPCGMetadataAttribute<int32>*>(AnalysisCountAttributeBase);

		const int32 NumElements = AnalysisMetadata->GetItemCountForChild();

		StringKeyToInstanceCount.Reserve(NumElements);

		// TODO: Range based get would scale better.
		for (int64 MetadataKey = 0; MetadataKey < NumElements; ++MetadataKey)
		{
			StringKeyToInstanceCount.Add(ValueAttribute->GetValue(MetadataKey), CountAttribute->GetValue(MetadataKey));
		}
	}
	else
	{
		UE_LOGF(LogPCG, Warning, "No analysis data received by static mesh spawner kernel, worst case instance allocations will be made.");

		if (InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data->IsA<UPCGProxyForGPUData>())
		{
			UE_LOGF(LogPCG, Error, "Data was not read back.");
		}

		return true;
	}

	return true;
}

bool UPCGSkinnedMeshSpawnerDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSkinnedMeshSpawnerDataProvider::PrepareForExecute_GameThread);
	check(InBinding);
	const UPCGSkinnedMeshSpawnerSettings* Settings = CastChecked<UPCGSkinnedMeshSpawnerSettings>(GetProducerKernel()->GetSettings());

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	if (!bRegisteredPrimitives)
	{
		InBinding->AddMeshSpawnerPrimitives(GetProducerKernel(), {});
		bRegisteredPrimitives = true;
	}

	TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
	FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr;
	if (!ensure(Context))
	{
		InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
		return true;
	}

	UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get());
	if (!ensure(SourceComponent))
	{
		InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
		return true;
	}

	if (SelectorAttributeId == INDEX_NONE && ensure(Settings->MeshSelectorParameters))
	{
		const FName SelectorName = Settings->MeshSelectorParameters->MeshAttribute.GetName();
		const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);

		if (!ensure(InputDataDesc))
		{
			InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
			return true;
		}

		bool bAnyPointsPresent = false;

		for (const FPCGDataDesc& Desc : InputDataDesc->GetDataDescriptions())
		{
			if (Desc.GetElementCount().X <= 0)
			{
				continue;
			}

			bAnyPointsPresent = true;

			for (const FPCGKernelAttributeDesc& AttributeDesc : Desc.GetAttributeDescriptions())
			{
				if (AttributeDesc.GetAttributeKey().GetIdentifier().Name == SelectorName && AttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::StringKey)
				{
					SelectorAttributeId = AttributeDesc.GetAttributeId();
					break;
				}
			}

			if (SelectorAttributeId != INDEX_NONE)
			{
				break;
			}
		}

		if (SelectorAttributeId == INDEX_NONE)
		{
			// Mute this error if the point data is empty.
			if (bAnyPointsPresent)
			{
				PCG_KERNEL_VALIDATION_ERR(Context, Settings, FText::Format(
					LOCTEXT("MeshSelectorAttributeNotFound", "Mesh selector attribute '{0}' not found."),
					FText::FromName(SelectorName)));
			}

			InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
			return true;
		}

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}

	if (!bPrimitiveDescriptorsCreated)
	{
		CreatePrimitiveDescriptors(Context, InBinding);

		for (const FPCGSoftSkinnedMeshComponentDescriptor& PrimitiveDescriptor : PrimitiveDescriptors)
		{
			UE_LOGF(LogPCG, Verbose, "Request '%ls' to load.", *PrimitiveDescriptor.SkinnedAsset.ToString());

			PrimitiveDescriptor.SkinnedAsset.LoadAsync({});
		}

		bPrimitiveDescriptorsCreated = true;

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}

	if (!bStaticMeshesLoaded)
	{
		for (const FPCGSoftSkinnedMeshComponentDescriptor& PrimitiveDescriptor : PrimitiveDescriptors)
		{
			if (PrimitiveDescriptor.SkinnedAsset.IsPending())
			{
				UE_LOGF(LogPCG, Verbose, "Waiting for '%ls' to load.", *PrimitiveDescriptor.SkinnedAsset.ToString());

				return false;
			}
		}

		bStaticMeshesLoaded = true;
	}

	if (!bPrimitivesSetUp)
	{
		if (!SetupPrimitives(Context, InBinding))
		{
			return false;
		}

		bPrimitivesSetUp = true;

		if (NumPrimitivesSetup > 0)
		{
			SourceComponent->NotifyProceduralInstancesInUse();

			// Signal not finished. Compute graph element will wait a frame, giving time for GPU Scene to be updated.
			return false;
		}
		else
		{
			// No component set up means we have no more work to do.
			InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
			return true;
		}
	}

	InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
	return true;
}

FComputeDataProviderRenderProxy* UPCGSkinnedMeshSpawnerDataProvider::GetRenderProxy()
{
	return new FPCGSkinnedMeshSpawnerDataProviderProxy(AttributeIdOffsetStrides, SelectorAttributeId, PrimitiveStringKeys, PrimitiveSelectionCDF, PrimitiveMeshBounds);
}

void UPCGSkinnedMeshSpawnerDataProvider::Reset()
{
	Super::Reset();

	AttributeIdOffsetStrides.Empty();
	PrimitiveStringKeys.Empty();
	PrimitiveMeshBounds.Empty();
	PrimitiveSelectionCDF.Empty();
	SelectorAttributeId = INDEX_NONE;
	NumInputPoints = 0;
	StringKeyToInstanceCount.Empty();
	AnalysisDataIndex = INDEX_NONE;
	bPrimitiveDescriptorsCreated = false;
	PrimitiveDescriptors.Empty();
	CustomFloatCount = 0;
	bPrimitivesSetUp = false;
	NumPrimitivesSetup = 0;
	bRegisteredPrimitives = false;
	bStaticMeshesLoaded = false;
}

void UPCGSkinnedMeshSpawnerDataProvider::CreatePrimitiveDescriptors(FPCGContext* InContext, UPCGDataBinding* InBinding)
{
	PrimitiveDescriptors.Empty();

	const UPCGSkinnedMeshSpawnerSettings* Settings = CastChecked<UPCGSkinnedMeshSpawnerSettings>(GetProducerKernel()->GetSettings());

	AActor* TargetActor = Settings->TargetActor.Get() ? Settings->TargetActor.Get() : InContext->GetTypedExecutionTarget<AActor>();
	if (!ensure(Settings->MeshSelectorParameters) || !ensure(TargetActor))
	{
		return;
	}

	const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);

	if (!ensure(InputDataDesc))
	{
		return;
	}

	const int32 TotalInputPointCount = InputDataDesc->ComputeTotalElementCount();
	if (TotalInputPointCount == 0)
	{
		return;
	}

	if (TotalInputPointCount >= MAX_INSTANCE_ID)
	{
		PCG_KERNEL_VALIDATION_ERR(InContext, Settings, FText::Format(LOCTEXT("TooManyInstances", "Tried to spawn too many instances ({0}), procedural ISM component creation skipped and instances will not be rendered."), TotalInputPointCount));
		return;
	}

	UPCGComponent* SourceComponent = Cast<UPCGComponent>(InContext->ExecutionSource.Get());
	if (!SourceComponent)
	{
		return;
	}

	const TOptional<TConstArrayView<FPCGAttributePropertyInputSelector>> AttributeSelectors = Settings->InstanceDataPackerParameters ? Settings->InstanceDataPackerParameters->GetAttributeSelectors() : TOptional<TConstArrayView<FPCGAttributePropertyInputSelector>>();

	if (AttributeSelectors.IsSet())
	{
		for (const FPCGAttributePropertyInputSelector& AttributeSelector : AttributeSelectors.GetValue())
		{
			const FName AttributeName = AttributeSelector.GetAttributeName();

			if (!AttributeSelector.IsBasicAttribute() || AttributeName == PCGMetadataAttributeConstants::LastAttributeName)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings,
					FText::Format(LOCTEXT("OnlyBasicAttributesSupported", "Attribute '{0}' is invalid. GPU instance packer implementation currently only supports basic attributes."), AttributeSelector.GetDisplayText()));
				continue;
			}

			if (AttributeName == NAME_None)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings,
					FText::Format(LOCTEXT("InstanceDataAttributeInvalid", "Invalid instance data attribute specified '{0}'."), FText::FromName(AttributeName)));

				continue;
			}

			if (!InputDataDesc->ContainsAttributeOnAnyData(AttributeName))
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings,
					FText::Format(LOCTEXT("InstanceDataAttributeNotFound", "Instance data attribute '{0}' not found."), FText::FromName(AttributeName)));

				continue;
			}
		}

		PCGMeshSpawnerPackingHelpers::ComputeCustomFloatPacking(InContext, Settings, AttributeSelectors.GetValue(), InputDataDesc, CustomFloatCount, AttributeIdOffsetStrides);
	}

	if (Settings->MeshSelectorParameters)
	{
		const FName SelectorName = Settings->MeshSelectorParameters->MeshAttribute.GetName(); // getname ok?

		// Compute how many instances we expect for each mesh, if we do not already have the answer from analysis.
		if (StringKeyToInstanceCount.IsEmpty())
		{
			for (const FPCGDataDesc& Desc : InputDataDesc->GetDataDescriptions())
			{
				if (Desc.GetElementCount().X <= 0)
				{
					continue;
				}

				for (const FPCGKernelAttributeDesc& AttributeDesc : Desc.GetAttributeDescriptions())
				{
					if (AttributeDesc.GetAttributeKey().GetIdentifier().Name == SelectorName)
					{
						if (AttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::StringKey)
						{
							for (int32 AttributeStringKeyValue : AttributeDesc.GetUniqueStringKeys())
							{
								if (AttributeStringKeyValue > 0)
								{
									StringKeyToInstanceCount.FindOrAdd(AttributeStringKeyValue, 0) += Desc.GetElementCount().X;
								}
							}

							// We could early out but we currently continue looping to generate the below warning for mismatched types.
						}
						else
						{
							// Currently only a single type per attribute name is supported (the name/type in the attribute table). It's possible to wire a graph
							// for which an attribute is present with multiple types. Warn if this is encountered.
							PCG_KERNEL_VALIDATION_WARN(InContext, Settings, FText::Format(
								LOCTEXT("MeshSelectorAttributeNotUsable", "Attribute '{0}' not usable for mesh selection, only attributes of type String Key are supported."),
								FText::FromName(SelectorName)));
						}
					}
				}
			}
		}

		PrimitiveDescriptors.Reserve(StringKeyToInstanceCount.Num());

		for (const TPair<int32, uint32>& StringKeyAndInstanceCount : StringKeyToInstanceCount)
		{
			if (StringKeyAndInstanceCount.Value == 0)
			{
				continue;
			}

			if (!ensure(InBinding->GetStringTable().IsValidIndex(StringKeyAndInstanceCount.Key)))
			{
				continue;
			}

			const FString& MeshPathString = InBinding->GetStringTable()[StringKeyAndInstanceCount.Key];
			if (MeshPathString.IsEmpty())
			{
				continue;
			}

			if (PrimitiveDescriptors.Num() >= PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER)
			{
				PCG_KERNEL_VALIDATION_WARN(InContext, Settings,
					FText::Format(PCGSkinnedMeshSpawnerDataInterface::TooManyPrimitivesFormat, FText::FromString(FString::FromInt(PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER))));

				break;
			}

			FPCGSoftSkinnedMeshComponentDescriptor Descriptor{};
			
			Descriptor = Settings->MeshSelectorParameters->TemplateDescriptor;
			Descriptor.bIsInstanceDataGPUOnly = true;
			Descriptor.PrimitiveBoundsOverride = SourceComponent->GetGridBounds();
			Descriptor.NumInstancesGPUOnly = StringKeyAndInstanceCount.Value;
			Descriptor.NumCustomDataFloatsGPUOnly = CustomFloatCount;
			Descriptor.SkinnedAsset = FSoftObjectPath(MeshPathString);
			Descriptor.bAffectDynamicIndirectLighting = false; // Not supported
			Descriptor.bAffectDistanceFieldLighting = false; // Not supported

			// Sanity check instance count.
			if (!ensure(Descriptor.NumInstancesGPUOnly <= TotalInputPointCount))
			{
				Descriptor.NumInstancesGPUOnly = TotalInputPointCount;
			}

			PrimitiveStringKeys.Add(StringKeyAndInstanceCount.Key);
			PrimitiveDescriptors.Add(MoveTemp(Descriptor));
		}

		PrimitiveSelectionCDF.SetNumZeroed(PrimitiveDescriptors.Num());
	}

	FPCGSpawnerPrimitives& Primitives = InBinding->FindOrAddMeshSpawnerPrimitives(GetProducerKernel());
	Primitives.NumCustomFloats = CustomFloatCount;
	Primitives.AttributeIdOffsetStrides = AttributeIdOffsetStrides;
	Primitives.SelectorAttributeId = SelectorAttributeId;
	Primitives.SelectionCDF = PrimitiveSelectionCDF;
	Primitives.PrimitiveMeshBounds = PrimitiveMeshBounds;

	// Useful for debugging instance counts.
	//UE_LOGF(LogPCG, Log, "Input points %d, total instance count %d (+ %.2f%%)", TotalInputPointCount, Primitives.NumInstancesAllPrimitives, (float(Primitives.NumInstancesAllPrimitives - TotalInputPointCount) / TotalInputPointCount) * 100.0f);
}

bool UPCGSkinnedMeshSpawnerDataProvider::SetupPrimitives(FPCGContext* InContext, UPCGDataBinding* InBinding)
{
	if (PrimitiveDescriptors.IsEmpty())
	{
		return true;
	}

	const UPCGSkinnedMeshSpawnerSettings* Settings = CastChecked<UPCGSkinnedMeshSpawnerSettings>(GetProducerKernel()->GetSettings());

	AActor* TargetActor = Settings->TargetActor.Get() ? Settings->TargetActor.Get() : InContext->GetTypedExecutionTarget<AActor>();
	if (!ensure(TargetActor))
	{
		return true;
	}

	FPCGSpawnerPrimitives* Primitives = InBinding->FindMeshSpawnerPrimitives(GetProducerKernel());
	if (!Primitives)
	{
		return true;
	}

	TSharedPtr<FPCGPrimitiveFactoryISKMC> Factory;

	if (!Primitives->PrimitiveFactory)
	{
		Factory = MakeShared<FPCGPrimitiveFactoryISKMC>();

		Primitives->PrimitiveFactory = Factory;

		FPCGPrimitiveFactoryISKMC::FParameters Params;
		Params.Descriptors = PrimitiveDescriptors;
		Params.TargetActor = TargetActor;

		Factory->Initialize(MoveTemp(Params));
	}
	else
	{
		Factory = StaticCastSharedPtr<FPCGPrimitiveFactoryISKMC>(Primitives->PrimitiveFactory);
	}

	if (!Factory->Create(InContext))
	{
		// Not finished, continue next tick.
		return false;
	}

	if (Settings->bApplyMeshBoundsToPoints)
	{
		for (int32 PrimitiveIndex = 0; PrimitiveIndex < Factory->GetNumPrimitives(); ++PrimitiveIndex)
		{
			PrimitiveMeshBounds.Add(Factory->GetMeshBounds(PrimitiveIndex));
		}
	}

	NumPrimitivesSetup = Factory->GetNumPrimitives();

	return true;
}

FPCGSkinnedMeshSpawnerDataProviderProxy::FPCGSkinnedMeshSpawnerDataProviderProxy(
	const TArray<FUintVector4>& InAttributeIdOffsetStrides,
	int32 InSelectorAttributeId,
	const TArray<int32>& InPrimitiveStringKeys,
	TArray<float> InSelectionCDF,
	const TArray<FBox>& InPrimitiveMeshBounds)
	: AttributeIdOffsetStrides(InAttributeIdOffsetStrides)
	, SelectionCDF(InSelectionCDF)
	, SelectorAttributeId(InSelectorAttributeId)
	, PrimitiveStringKeys(InPrimitiveStringKeys)
{
	PrimitiveMeshBoundsMin.Reserve(InPrimitiveMeshBounds.Num());
	PrimitiveMeshBoundsMax.Reserve(InPrimitiveMeshBounds.Num());

	for (int Index = 0; Index < InPrimitiveMeshBounds.Num(); ++Index)
	{
		PrimitiveMeshBoundsMin.Add(FVector4f(InPrimitiveMeshBounds[Index].Min.X, InPrimitiveMeshBounds[Index].Min.Y, InPrimitiveMeshBounds[Index].Min.Z, /*Unused*/0.0f));
		PrimitiveMeshBoundsMax.Add(FVector4f(InPrimitiveMeshBounds[Index].Max.X, InPrimitiveMeshBounds[Index].Max.Y, InPrimitiveMeshBounds[Index].Max.Z, /*Unused*/0.0f));
	}
}

bool FPCGSkinnedMeshSpawnerDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGSkinnedMeshSpawnerDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	LLM_SCOPE_BYTAG(PCG);

	if (!PrimitiveStringKeys.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(PrimitiveStringKeys.GetTypeSize(), PrimitiveStringKeys.Num());

		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGSkinnedMeshSpawner_PrimitiveStringKeys"));
		PrimitiveStringKeysBufferSRV = GraphBuilder.CreateSRV(Buffer);

		GraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(PrimitiveStringKeys));
	}
	else
	{
		PrimitiveStringKeysBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, PrimitiveStringKeys.GetTypeSize())));
	}

	if (!PrimitiveMeshBoundsMin.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(PrimitiveMeshBoundsMin.GetTypeSize(), PrimitiveMeshBoundsMin.Num());

		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGSkinnedMeshSpawner_PrimitiveMeshBoundsMin"));
		PrimitiveMeshBoundsMinBufferSRV = GraphBuilder.CreateSRV(Buffer);

		GraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(PrimitiveMeshBoundsMin));
	}
	else
	{
		PrimitiveMeshBoundsMinBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, PrimitiveMeshBoundsMin.GetTypeSize())));
	}

	if (!PrimitiveMeshBoundsMax.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(PrimitiveMeshBoundsMax.GetTypeSize(), PrimitiveMeshBoundsMax.Num());

		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGSkinnedMeshSpawner_PrimitiveMeshBoundsMax"));
		PrimitiveMeshBoundsMaxBufferSRV = GraphBuilder.CreateSRV(Buffer);

		GraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(PrimitiveMeshBoundsMax));
	}
	else
	{
		PrimitiveMeshBoundsMaxBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, PrimitiveMeshBoundsMax.GetTypeSize())));
	}
}

void FPCGSkinnedMeshSpawnerDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];

		Parameters.NumAttributes = AttributeIdOffsetStrides.Num();
		Parameters.NumPrimitives = SelectionCDF.Num();
		Parameters.SelectorAttributeId = SelectorAttributeId;

		for (int32 Index = 0; Index < AttributeIdOffsetStrides.Num(); ++Index)
		{
			Parameters.AttributeIdOffsetStrides[Index] = AttributeIdOffsetStrides[Index];
		}

		Parameters.PrimitiveStringKeys = PrimitiveStringKeysBufferSRV;

		for (int32 Index = 0; Index < SelectionCDF.Num(); ++Index)
		{
			GET_SCALAR_ARRAY_ELEMENT(Parameters.SelectionCDF, Index) = SelectionCDF[Index];
		}

		Parameters.ApplyBounds = PrimitiveMeshBoundsMin.IsEmpty() ? 0 : 1;

		Parameters.PrimitiveMeshBoundsMin = PrimitiveMeshBoundsMinBufferSRV;
		Parameters.PrimitiveMeshBoundsMax = PrimitiveMeshBoundsMaxBufferSRV;
	}
}

#undef LOCTEXT_NAMESPACE
