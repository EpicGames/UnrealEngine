// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "MeshSelectors/PCGSkinnedMeshDescriptor.h"

#include "PCGSkinnedMeshSpawnerDataInterface.generated.h"

class FPCGSkinnedMeshSpawnerDataInterfaceParameters;
class UPCGSkinnedMeshSpawnerSettings;
class UPCGSettings;
class UPCGSkinnedMeshSpawnerKernel;
struct FPCGContext;

UCLASS(ClassGroup = (Procedural))
class UPCGSkinnedMeshSpawnerDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGSkinnedMeshSpawner"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	static constexpr uint32 MAX_ATTRIBUTES = 64;
};

UCLASS()
class UPCGSkinnedMeshSpawnerDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual bool PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding) override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	void CreatePrimitiveDescriptors(FPCGContext* InContext, UPCGDataBinding* InBinding);

	/** Setup Procedural ISMCs. Returns true if at least one component was set up. */
	virtual bool SetupPrimitives(FPCGContext* InContext, UPCGDataBinding* InBinding);

public:
	/** Attributes to use for writing per-instance custom floats. */
	UPROPERTY()
	TArray<FUintVector4> AttributeIdOffsetStrides;

	UPROPERTY()
	TArray<int32> PrimitiveStringKeys;

	UPROPERTY()
	TArray<FBox> PrimitiveMeshBounds;

	UPROPERTY()
	TArray<float> PrimitiveSelectionCDF;

	/** Attribute Id for mesh selector. */
	UPROPERTY()
	int32 SelectorAttributeId = INDEX_NONE;

	UPROPERTY()
	int32 NumInputPoints = 0;

	/** The number of instances per string key value, used for by-attribute spawning. */
	TMap<int32, uint32> StringKeyToInstanceCount;

	int32 AnalysisDataIndex = INDEX_NONE;

	bool bPrimitiveDescriptorsCreated = false;

	TArray<FPCGSoftSkinnedMeshComponentDescriptor> PrimitiveDescriptors;

	uint32 CustomFloatCount = 0;

	bool bRegisteredPrimitives = false;

	bool bStaticMeshesLoaded = false;

	bool bPrimitivesSetUp = false;

	/** Number of primitives created during primitive setup. */
	int32 NumPrimitivesSetup = 0;
};

class FPCGSkinnedMeshSpawnerDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGSkinnedMeshSpawnerDataProviderProxy(
		const TArray<FUintVector4>& InAttributeIdOffsetStrides,
		int32 InSelectorAttributeId,
		const TArray<int32>& InPrimitiveStringKeys,
		TArray<float> InSelectionCDF,
		const TArray<FBox>& InPrimitiveMeshBounds);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGSkinnedMeshSpawnerDataInterfaceParameters;

	TArray<FUintVector4> AttributeIdOffsetStrides;
	TArray<float> SelectionCDF;

	int32 SelectorAttributeId = INDEX_NONE;

	TArray<int32> PrimitiveStringKeys;
	FRDGBufferSRVRef PrimitiveStringKeysBufferSRV = nullptr;

	TArray<FVector4f> PrimitiveMeshBoundsMin;
	FRDGBufferSRVRef PrimitiveMeshBoundsMinBufferSRV = nullptr;

	TArray<FVector4f> PrimitiveMeshBoundsMax;
	FRDGBufferSRVRef PrimitiveMeshBoundsMaxBufferSRV = nullptr;
};
