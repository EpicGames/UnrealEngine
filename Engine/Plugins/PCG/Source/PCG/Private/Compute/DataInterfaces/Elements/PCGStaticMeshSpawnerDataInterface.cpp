// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/DataInterfaces/Elements/PCGStaticMeshSpawnerDataInterface.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/BuiltInKernels/PCGSMSpawnerAnalysisKernel.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Compute/Packing/PCGMeshSpawnerPacking.h"
#include "Elements/PCGStaticMeshSpawner.h"
#include "Elements/PCGStaticMeshSpawnerKernel.h"
#include "InstanceDataPackers/PCGInstanceDataPackerBase.h"
#include "MeshSelectors/PCGMeshSelectorByAttribute.h"
#include "MeshSelectors/PCGMeshSelectorPrimitiveData.h"
#include "MeshSelectors/PCGMeshSelectorWeighted.h"
#include "Metadata/PCGObjectPropertyOverride.h"

#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHIResources.h"
#include "SceneDefinitions.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SystemTextures.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Engine/AssetManager.h"
#include "Engine/StaticMesh.h"
#include "Engine/StreamableManager.h"
#include "EngineDefines.h"
#include "Engine/World.h"
#include "Materials/MaterialInterface.h"
#if UE_ENABLE_DEBUG_DRAWING
#include "DrawDebugHelpers.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGStaticMeshSpawnerDataInterface)

#define LOCTEXT_NAMESPACE "PCGStaticMeshSpawnerDataInterface"

namespace PCGStaticMeshSpawnerDataInterface
{
	static const FText CouldNotLoadStaticMeshFormat = LOCTEXT("CouldNotLoadStaticMesh", "Could not load static mesh from path '{0}'.");
	static const FText TooManyPrimitivesFormat = LOCTEXT("TooManyPrimitives", "Attempted to emit too many primitive components, terminated after creating '{0}'.");
	static const FText NoMeshEntriesFormat = LOCTEXT("NoMeshEntries", "No mesh entries provided in weighted mesh selector.");

	static TAutoConsoleVariable<bool> CVarCreatePrimitivesComponentless(
		TEXT("pcg.RuntimeGeneration.ISM.ComponentlessPrimitives"),
		false,
		TEXT("Uses a component-less path for creating primitives."));

#if UE_ENABLE_DEBUG_DRAWING
	static TAutoConsoleVariable<bool> CVarCullingCellBoundsDebugDraw(
		TEXT("pcg.RuntimeGeneration.ISM.CullingCellBoundsDebugDraw"),
		false,
		TEXT("When true, draws red debug boxes for per-culling-cell world bounds read back from GPU analysis."));
#endif
}

void UPCGStaticMeshSpawnerDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetSelectorAttributeId"))
		.AddReturnType(EShaderFundamentalType::Uint); // Attribute id to get mesh path string key from, or invalid if we should use CDF instead.

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetNumAttributes"))
		.AddReturnType(EShaderFundamentalType::Uint); // Num attributes

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetNumPrimitives"))
		.AddReturnType(EShaderFundamentalType::Uint); // Num primitives

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_ShouldApplyBounds"))
		.AddReturnType(EShaderFundamentalType::Bool);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetPrimitiveMeshBoundsMin"))
		.AddReturnType(EShaderFundamentalType::Float, 3) // Local bounds min
		.AddParam(EShaderFundamentalType::Uint); // InPrimitiveIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetPrimitiveMeshBoundsMax"))
		.AddReturnType(EShaderFundamentalType::Float, 3) // Local bounds max
		.AddParam(EShaderFundamentalType::Uint); // InPrimitiveIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetAttributeIdOffsetStride"))
		.AddReturnType(EShaderFundamentalType::Uint, 4)
		.AddParam(EShaderFundamentalType::Uint); // InAttributeIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetPrimitiveStringKey"))
		.AddReturnType(EShaderFundamentalType::Int) // String key
		.AddParam(EShaderFundamentalType::Uint); // InPrimitiveIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetPrimitiveIndexFromPrimitiveId"))
		.AddReturnType(EShaderFundamentalType::Uint) // Primitive index
		.AddParam(EShaderFundamentalType::Int); // InPrimitiveId

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetPrimitiveSelectionCDF"))
		.AddReturnType(EShaderFundamentalType::Float) // CDF value
		.AddParam(EShaderFundamentalType::Uint); // InPrimitiveIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetSelectedMeshAttributeId"))
		.AddReturnType(EShaderFundamentalType::Uint); // Attribute id to output mesh path string key to

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetNumCullingCells"))
		.AddReturnType(EShaderFundamentalType::Int, 3); // Number of culling cells that the primitive volume overlaps, per dimension

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetNumCullingCellsTotal"))
		.AddReturnType(EShaderFundamentalType::Int); // Total number of culling cells that the primitive volume overlaps

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetBoundsMin"))
		.AddReturnType(EShaderFundamentalType::Float, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SMSpawner_GetBucketExtent"))
		.AddReturnType(EShaderFundamentalType::Float);
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGStaticMeshSpawnerDataInterfaceParameters,)
	SHADER_PARAMETER_ARRAY(FUintVector4, AttributeIdOffsetStrides, [UPCGStaticMeshSpawnerDataInterface::MAX_ATTRIBUTES])
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int32>, PrimitiveIds)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int32>, PrimitiveStringKeys)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector4f>, PrimitiveMeshBoundsMin)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector4f>, PrimitiveMeshBoundsMax)
	SHADER_PARAMETER_SCALAR_ARRAY(float, SelectionCDF, [PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER])
	SHADER_PARAMETER(uint32, NumAttributes)
	SHADER_PARAMETER(uint32, NumPrimitives)
	SHADER_PARAMETER(int32, SelectorAttributeId)
	SHADER_PARAMETER(int32, SelectedMeshAttributeId)
	SHADER_PARAMETER(uint32, ApplyBounds)
	SHADER_PARAMETER(FIntVector, NumCullingCells)
	SHADER_PARAMETER(int32, NumCullingCellsTotal)
	SHADER_PARAMETER(FVector3f, BoundsMin)
	SHADER_PARAMETER(float, BucketExtent)
END_SHADER_PARAMETER_STRUCT()

void UPCGStaticMeshSpawnerDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGStaticMeshSpawnerDataInterfaceParameters>(UID);
}

void UPCGStaticMeshSpawnerDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
		{ TEXT("MaxAttributes"), MAX_ATTRIBUTES },
		{ TEXT("MaxPrimitives"), PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER },
	};

	OutHLSL += FString::Format(TEXT(
		"int {DataInterfaceName}_SelectorAttributeId;\n"
		"uint {DataInterfaceName}_NumAttributes;\n"
		"uint {DataInterfaceName}_NumPrimitives;\n"
		"uint {DataInterfaceName}_ApplyBounds;\n"
		"int {DataInterfaceName}_SelectedMeshAttributeId;\n"
		"uint4 {DataInterfaceName}_AttributeIdOffsetStrides[{MaxAttributes}];\n"
		"int3 {DataInterfaceName}_NumCullingCells;\n"
		"int {DataInterfaceName}_NumCullingCellsTotal;\n"
		"float3 {DataInterfaceName}_BoundsMin;\n"
		"float {DataInterfaceName}_BucketExtent;\n"
		"\n"
		"StructuredBuffer<float4> {DataInterfaceName}_PrimitiveMeshBoundsMin;\n"
		"StructuredBuffer<float4> {DataInterfaceName}_PrimitiveMeshBoundsMax;\n"
		"StructuredBuffer<int> {DataInterfaceName}_PrimitiveIds;\n"
		"StructuredBuffer<int> {DataInterfaceName}_PrimitiveStringKeys;\n"
		"DECLARE_SCALAR_ARRAY(float, {DataInterfaceName}_SelectionCDF, {MaxPrimitives});\n"
		"\n"
		"int SMSpawner_GetSelectorAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_SelectorAttributeId;\n"
		"}\n"
		"\n"
		"uint SMSpawner_GetNumAttributes_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_NumAttributes;\n"
		"}\n"
		"\n"
		"uint4 SMSpawner_GetAttributeIdOffsetStride_{DataInterfaceName}(uint InAttributeIndex)\n"
		"{\n"
		"	return {DataInterfaceName}_AttributeIdOffsetStrides[InAttributeIndex];\n"
		"}\n"
		"\n"
		"uint SMSpawner_GetNumPrimitives_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_NumPrimitives;\n"
		"}\n"
		"\n"
		"bool SMSpawner_ShouldApplyBounds_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_ApplyBounds > 0;\n"
		"}\n"
		"\n"
		"float3 SMSpawner_GetPrimitiveMeshBoundsMin_{DataInterfaceName}(uint InPrimitiveIndex)\n"
		"{\n"
		"	return {DataInterfaceName}_PrimitiveMeshBoundsMin[InPrimitiveIndex].xyz;\n"
		"}\n"
		"\n"
		"float3 SMSpawner_GetPrimitiveMeshBoundsMax_{DataInterfaceName}(uint InPrimitiveIndex)\n"
		"{\n"
		"	return {DataInterfaceName}_PrimitiveMeshBoundsMax[InPrimitiveIndex].xyz;\n"
		"}\n"
		"\n"
		"int SMSpawner_GetPrimitiveStringKey_{DataInterfaceName}(uint InPrimitiveIndex)\n"
		"{\n"
		"	return {DataInterfaceName}_PrimitiveStringKeys[InPrimitiveIndex];\n"
		"}\n"
		"\n"
		"uint SMSpawner_GetPrimitiveIndexFromPrimitiveId_{DataInterfaceName}(int InPrimitiveId)\n"
		"{\n"
		"	for (uint Index = 0; Index < {DataInterfaceName}_NumPrimitives; ++Index)\n"
		"	{\n"
		"		if ({DataInterfaceName}_PrimitiveIds[Index] == InPrimitiveId)\n"
		"		{\n"
		"			return Index;\n"
		"		}\n"
		"	}\n"
		"	\n"
		"	return (uint)-1;\n"
		"}\n"
		"\n"
		"float SMSpawner_GetPrimitiveSelectionCDF_{DataInterfaceName}(uint InPrimitiveIndex)\n"
		"{\n"
		"	return GET_SCALAR_ARRAY_ELEMENT({DataInterfaceName}_SelectionCDF, InPrimitiveIndex);\n"
		"}\n"
		"\n"
		"int SMSpawner_GetSelectedMeshAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_SelectedMeshAttributeId;\n"
		"}\n"
		"\n"
		"int3 SMSpawner_GetNumCullingCells_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_NumCullingCells;\n"
		"}\n"
		"\n"
		"int SMSpawner_GetNumCullingCellsTotal_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_NumCullingCellsTotal;\n"
		"}\n"
		"\n"
		"float3 SMSpawner_GetBoundsMin_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_BoundsMin;\n"
		"}\n"
		"\n"
		"float SMSpawner_GetBucketExtent_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_BucketExtent;\n"
		"}\n"
	), TemplateArgs);
}

UComputeDataProvider* UPCGStaticMeshSpawnerDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGStaticMeshSpawnerDataProvider>();
}

void UPCGStaticMeshSpawnerDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGStaticMeshSpawnerSettings* Settings = CastChecked<UPCGStaticMeshSpawnerSettings>(GetProducerKernel()->GetSettings());
	bSpawningByAttribute = !!Cast<UPCGMeshSelectorByAttribute>(Settings->MeshSelectorParameters);
	bSpawningByPrimitiveData = !!Cast<UPCGMeshSelectorPrimitiveData>(Settings->MeshSelectorParameters);

	UPCGDataBinding* Binding = CastChecked<UPCGDataBinding>(InBinding);

	NumCullingCells = Binding->GetMaxNumCullingCells(Settings->GetCullingCellSize());
	CullingCellExtent = static_cast<float>(Settings->GetCullingCellSize());

	NumCullingCellsTotal = NumCullingCells.X * NumCullingCells.Y * NumCullingCells.Z;

	BoundsMin = FVector3f(Binding->GetExecutionSource()->GetExecutionState().GetBounds().Min);
}

bool UPCGStaticMeshSpawnerDataProvider::ReadbackMeshSelectionData(UPCGDataBinding* InBinding)
{
	if (!bSpawningByAttribute && !bSpawningByPrimitiveData)
	{
		// Only by-attribute and primitive data table selection need to do readbacks.
		return true;
	}

	if (AnalysisDataIndex == INDEX_NONE)
	{
		AnalysisDataIndex = GetProducerKernel() ? InBinding->GetFirstInputDataIndex(GetProducerKernel(), PCGStaticMeshSpawnerKernelConstants::InstanceCountsPinLabel) : INDEX_NONE;
	}

	return AnalysisDataIndex == INDEX_NONE || InBinding->ReadbackInputDataToCPU(AnalysisDataIndex);
}

bool UPCGStaticMeshSpawnerDataProvider::ReadbackPrimitiveData(UPCGDataBinding* InBinding)
{
	if (!bSpawningByPrimitiveData)
	{
		return true;
	}

	if (PrimitiveDataIndex == INDEX_NONE)
	{
		PrimitiveDataIndex = GetProducerKernel() ? InBinding->GetFirstInputDataIndex(GetProducerKernel(), PCGStaticMeshSpawnerKernelConstants::PrimitiveTablePinLabel) : INDEX_NONE;
	}

	return PrimitiveDataIndex == INDEX_NONE || InBinding->ReadbackInputDataToCPU(PrimitiveDataIndex);
}

bool UPCGStaticMeshSpawnerDataProvider::ReadbackCullingCellMinMaxPositionData(UPCGDataBinding* InBinding)
{
	if (!bSpawningByAttribute && !bSpawningByPrimitiveData)
	{
		return true;
	}

	if (CullingCellMinMaxPositionDataIndex == INDEX_NONE)
	{
		CullingCellMinMaxPositionDataIndex = GetProducerKernel() ? InBinding->GetFirstInputDataIndex(GetProducerKernel(), PCGStaticMeshSpawnerKernelConstants::CullingCellMinMaxPositionsPinLabel) : INDEX_NONE;
	}

	return CullingCellMinMaxPositionDataIndex == INDEX_NONE || InBinding->ReadbackInputDataToCPU(CullingCellMinMaxPositionDataIndex);
}

bool UPCGStaticMeshSpawnerDataProvider::PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGStaticMeshSpawnerDataProvider::PerformPreExecuteReadbacks_GameThread);
	check(InBinding);

	if (!Super::PerformPreExecuteReadbacks_GameThread(InBinding))
	{
		return false;
	}

	const bool bAnalysisDataReadBack = ReadbackMeshSelectionData(InBinding);
	const bool bPrimitiveDataReadBack = ReadbackPrimitiveData(InBinding);
	const bool bCullingCellMinMaxPositionDataReadBack = ReadbackCullingCellMinMaxPositionData(InBinding);

	if (!bAnalysisDataReadBack || !bPrimitiveDataReadBack || !bCullingCellMinMaxPositionDataReadBack)
	{
		return false;
	}

	const UPCGData* AnalysisData = InBinding->GetInputDataCollection().TaggedData.IsValidIndex(AnalysisDataIndex) ? InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data : nullptr;

	if (const UPCGParamData* AnalysisResultsData = Cast<UPCGParamData>(AnalysisData))
	{
		const UPCGMetadata* AnalysisMetadata = AnalysisResultsData->ConstMetadata();
		check(AnalysisMetadata);

		const FPCGMetadataAttributeBase* AnalysisValueAttributeBase = AnalysisMetadata->GetConstAttribute(PCGAttributeAnalysisKernelConstants::ValueAttributeName);
		const FPCGMetadataAttributeBase* AnalysisCountAttributeBase = AnalysisMetadata->GetConstAttribute(PCGAttributeAnalysisKernelConstants::ValueCountAttributeName);

		if (AnalysisValueAttributeBase && AnalysisValueAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<int32>::Id &&
			AnalysisCountAttributeBase && AnalysisCountAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<int32>::Id)
		{
			const FPCGMetadataAttribute<int32>* ValueAttribute = static_cast<const FPCGMetadataAttribute<int32>*>(AnalysisValueAttributeBase);
			const FPCGMetadataAttribute<int32>* CountAttribute = static_cast<const FPCGMetadataAttribute<int32>*>(AnalysisCountAttributeBase);

			const int32 NumElements = AnalysisMetadata->GetItemCountForChild();

			int32 LastPrimitiveId = INDEX_NONE;
			uint32 CellId = 0;
			TArray<FPCGInstanceRange>* CurrentInstanceRanges = nullptr;

			// TODO: Range based get would scale better.
			for (int64 MetadataKey = 0; MetadataKey < NumElements; ++MetadataKey)
			{
				const int32 PrimitiveId = ValueAttribute->GetValue(MetadataKey);
				const int32 InstanceCount = CountAttribute->GetValue(MetadataKey);

				if (LastPrimitiveId != PrimitiveId)
				{
					// Reset cell ID if we encounter a new primitive.
					CellId = 0;

					// Cache pointer to avoid repeated map finds.
					CurrentInstanceRanges = &PrimitiveIdToInstanceRanges.FindOrAdd(PrimitiveId);
				}

				LastPrimitiveId = PrimitiveId;

				if (InstanceCount > 0)
				{
					CurrentInstanceRanges->Add(FPCGInstanceRange(InstanceCount, CellId));
				}

				++CellId;
			}
		}
		else
		{
			UE_LOGF(LogPCG, Warning, "No analysis data received by static mesh spawner kernel, worst case instance allocations will be made.");

			if (InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data->IsA<UPCGProxyForGPUData>())
			{
				UE_LOGF(LogPCG, Error, "Data was not read back.");
			}

			return true;
		}
	}
	else if (const UPCGRawBufferData* RawAnalysisData = Cast<UPCGRawBufferData>(AnalysisData))
	{
		// Hardcoded data format - uint array containing pairs of (primitive ID, instance count) pairs.
		// This path is not supported for by-attribute spawning, for which the primitive ID is the string key value, because
		// we can't currently remap string keys from upstream compute graph.
		const int32 NumUints = RawAnalysisData->GetNumUint32s();
		ensure((NumUints % 2) == 0);

		int32 LastPrimitiveId = INDEX_NONE;
		uint32 CellId = 0;
		TArray<FPCGInstanceRange>* CurrentInstanceRanges = nullptr;

		const uint32* RawData = RawAnalysisData->GetConstData().GetData();
		for (const uint32* It = RawData; It < RawData + NumUints; It += 2)
		{
			const int32 PrimitiveId = static_cast<int32>(It[0]);
			const uint32 InstanceCount = It[1];

			if (LastPrimitiveId != PrimitiveId)
			{
				// Reset cell ID if we encounter a new primitive.
				CellId = 0;

				// Cache pointer to avoid repeated map finds.
				CurrentInstanceRanges = &PrimitiveIdToInstanceRanges.FindOrAdd(PrimitiveId);
			}

			LastPrimitiveId = PrimitiveId;

			if (InstanceCount > 0)
			{
				CurrentInstanceRanges->Add(FPCGInstanceRange(InstanceCount, CellId));
			}

			++CellId;
		}
	}

	const UPCGData* CullingCellMinMaxPosData = InBinding->GetInputDataCollection().TaggedData.IsValidIndex(CullingCellMinMaxPositionDataIndex) ? InBinding->GetInputDataCollection().TaggedData[CullingCellMinMaxPositionDataIndex].Data : nullptr;

	if (const UPCGRawBufferData* CullingCellMinMaxPositionData = Cast<UPCGRawBufferData>(CullingCellMinMaxPosData))
	{
		TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
		FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr;

		// Data format: PositionStride uints per (primitive, culling cell).
		// Layout: (BufferPrimitiveIndex * NumCullingCellsTotal + CullingCellIndex) * PositionStride + channel
		// BufferPrimitiveIndex is a 0-based sequential slot (= AttributeValue in the shader).
		const TArray<uint32>& CullingCellData = CullingCellMinMaxPositionData->GetConstData();
		const int32 Stride = PCGSMSpawnerAnalysisConstants::PositionStride;

		if (ensure(CullingCellData.Num() % Stride == 0) && NumCullingCellsTotal > 0 && Context && Context->ExecutionSource.IsValid())
		{
			const FBox& ExecutionSourceBounds = Context->ExecutionSource->GetExecutionState().GetBounds();
			const double DomainExpandProportion = 0.5;
			const FBox ExpandedBounds = ExecutionSourceBounds.ExpandBy(DomainExpandProportion * ExecutionSourceBounds.GetSize());
			const FVector& ExpandedMin = ExpandedBounds.Min;
			const FVector& ExpandedMax = ExpandedBounds.Max;

			const int32 NumPrimitivesInBuffer = CullingCellData.Num() / Stride / NumCullingCellsTotal;

			if (NumPrimitivesInBuffer > 0)
			{
				auto AsFloat = [&CullingCellData](int32 Index) { return FGenericPlatformMath::AsFloat(CullingCellData[Index]); };

				for (const TPair<int32, TArray<FPCGInstanceRange>>& PrimIdAndRanges : PrimitiveIdToInstanceRanges)
				{
					const int32 BufferPrimitiveIndex = PrimIdAndRanges.Key;
					const TArray<FPCGInstanceRange>& InstanceRanges = PrimIdAndRanges.Value;

					if (InstanceRanges.IsEmpty() || !ensure(BufferPrimitiveIndex >= 0 && BufferPrimitiveIndex < NumPrimitivesInBuffer))
					{
						continue;
					}

					// Per-range world bounds, aligned 1:1 with InstanceRanges.
					TArray<FBox> CullingCellBounds;
					CullingCellBounds.SetNumUninitialized(InstanceRanges.Num());
					FBox PrimitiveBounds(EForceInit::ForceInit);

					for (int32 RangeIndex = 0; RangeIndex < InstanceRanges.Num(); ++RangeIndex)
					{
						const FPCGInstanceRange& Range = InstanceRanges[RangeIndex];
						const uint32 CullingCellIndex = Range.GetCellID();
						if (!ensure(CullingCellIndex < static_cast<uint32>(NumCullingCellsTotal)))
						{
							CullingCellBounds[RangeIndex] = FBox(EForceInit::ForceInit);
							continue;
						}

						const int32 BaseIndex = Stride * (BufferPrimitiveIndex * NumCullingCellsTotal + static_cast<int32>(CullingCellIndex));

						FVector CullingCellMin, CullingCellMax;
						// Min components are stored inverted (1.0 - pos) by the shader so it can use AtomicMax against a zero-initialized buffer; undo the inversion here.
						CullingCellMin[0] = FMath::Lerp(ExpandedMin[0], ExpandedMax[0], 1.0 - AsFloat(BaseIndex + 0));
						CullingCellMin[1] = FMath::Lerp(ExpandedMin[1], ExpandedMax[1], 1.0 - AsFloat(BaseIndex + 1));
						CullingCellMin[2] = FMath::Lerp(ExpandedMin[2], ExpandedMax[2], 1.0 - AsFloat(BaseIndex + 2));
						CullingCellMax[0] = FMath::Lerp(ExpandedMin[0], ExpandedMax[0], AsFloat(BaseIndex + 3));
						CullingCellMax[1] = FMath::Lerp(ExpandedMin[1], ExpandedMax[1], AsFloat(BaseIndex + 4));
						CullingCellMax[2] = FMath::Lerp(ExpandedMin[2], ExpandedMax[2], AsFloat(BaseIndex + 5));

						if (CullingCellMin.X <= CullingCellMax.X && CullingCellMin.Y <= CullingCellMax.Y && CullingCellMin.Z <= CullingCellMax.Z)
						{
							FBox& CellBox = CullingCellBounds[RangeIndex];
							CellBox = FBox(CullingCellMin, CullingCellMax);
							PrimitiveBounds += CellBox;
						}
						else
						{
							CullingCellBounds[RangeIndex] = FBox(EForceInit::ForceInit);
						}
					}

					if (PrimitiveBounds.IsValid)
					{
						// Store raw position bounds. Scale expansion is applied later in PrepareForExecute_GameThread once PrimitiveMeshBounds is available.
						PrimitiveIdToWorldBounds.Add(BufferPrimitiveIndex, PrimitiveBounds);

#if UE_ENABLE_DEBUG_DRAWING
						if (PCGStaticMeshSpawnerDataInterface::CVarCullingCellBoundsDebugDraw.GetValueOnGameThread())
						{
							UWorld* World = Context->ExecutionSource->GetExecutionState().GetWorld();
							for (const FBox& CullingCellBox : CullingCellBounds)
							{
								if (CullingCellBox.IsValid)
								{
									DrawDebugBox(World, CullingCellBox.GetCenter(), CullingCellBox.GetExtent(), FColor::Red, /*bPersistentLines=*/false, /*LifeTime=*/3.0f);
								}
							}
						}
#endif

						PrimitiveIdToCullingCellWorldBounds.Add(BufferPrimitiveIndex, MoveTemp(CullingCellBounds));
					}
				}
			}
		}
	}

	return true;
}

void UPCGStaticMeshSpawnerDataProvider::ExpandPrimitiveBoundsByMeshBounds()
{
	for (FPCGPrimitiveInfo& PrimitiveInfo : PrimitiveInfos)
	{
		const UStaticMesh* Mesh = Cast<UStaticMesh>(PrimitiveInfo.Descriptor.StaticMesh.Get());
		if (!Mesh)
		{
			continue;
		}

		const FBox MeshLocalBounds = Mesh->GetBounds().GetBox();
		if (!MeshLocalBounds.IsValid)
		{
			continue;
		}

		// Per-axis farthest reach from the pivot (local origin), not from the mesh center.
		// This handles asymmetric meshes (e.g. trees with pivot at base).
		const FVector FarthestFromPivot(
			FMath::Max(FMath::Abs(MeshLocalBounds.Min.X), FMath::Abs(MeshLocalBounds.Max.X)),
			FMath::Max(FMath::Abs(MeshLocalBounds.Min.Y), FMath::Abs(MeshLocalBounds.Max.Y)),
			FMath::Max(FMath::Abs(MeshLocalBounds.Min.Z), FMath::Abs(MeshLocalBounds.Max.Z)));

		// Scale = 1.0: no per-instance scale data is available without a GPU readback pass.
		PrimitiveInfo.Descriptor.WorldBounds = PrimitiveInfo.Descriptor.WorldBounds.ExpandBy(FarthestFromPivot.Size());
	}
}

bool UPCGStaticMeshSpawnerDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGStaticMeshSpawnerDataProvider::PrepareForExecute_GameThread);
	check(InBinding);
	const UPCGStaticMeshSpawnerSettings* Settings = CastChecked<UPCGStaticMeshSpawnerSettings>(GetProducerKernel()->GetSettings());

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	if (!bRegisteredPrimitives)
	{
		InBinding->AddMeshSpawnerPrimitives(GetProducerKernel(), {});
		bRegisteredPrimitives = true;
	}

	TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
	FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr;
	if (!ensure(Context))
	{
		InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
		return true;
	}

	if (!ensure(Context->ExecutionSource.IsValid()))
	{
		InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
		return true;
	}

	const UPCGMeshSelectorByAttribute* SelectorByAttribute = Cast<UPCGMeshSelectorByAttribute>(Settings->MeshSelectorParameters);
	const UPCGMeshSelectorPrimitiveData* SelectorPrimitiveData = Cast<UPCGMeshSelectorPrimitiveData>(Settings->MeshSelectorParameters);

	if (SelectorAttributeId == INDEX_NONE && (SelectorByAttribute || SelectorPrimitiveData))
	{
		const FName SelectorName = SelectorByAttribute ? SelectorByAttribute->AttributeName : SelectorPrimitiveData->PrimitiveIndexAttribute;

		const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);

		if (!ensure(InputDataDesc))
		{
			InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
			return true;
		}

		bool bAnyPointsPresent = false;

		for (const FPCGDataDesc& Desc : InputDataDesc->GetDataDescriptions())
		{
			if (Desc.GetElementCount().X <= 0)
			{
				continue;
			}

			bAnyPointsPresent = true;

			for (const FPCGKernelAttributeDesc& AttributeDesc : Desc.GetAttributeDescriptions())
			{
				if (AttributeDesc.GetAttributeKey().GetIdentifier().Name == SelectorName
					&& (AttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::StringKey || AttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::Int))
				{
					SelectorAttributeId = AttributeDesc.GetAttributeId();
					break;
				}
			}

			if (SelectorAttributeId != INDEX_NONE)
			{
				break;
			}
		}

		if (SelectorAttributeId == INDEX_NONE)
		{
			// Mute this error if the point data is empty.
			if (!InputDataDesc->GetDataDescriptions().IsEmpty() && bAnyPointsPresent)
			{
				PCG_KERNEL_VALIDATION_ERR(Context, Settings, FText::Format(
					LOCTEXT("MeshSelectorAttributeNotFound", "Mesh selector attribute '{0}' not found."),
					FText::FromName(SelectorName)));
			}

			InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
			return true;
		}

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}

	if (!bPrimitiveDescriptorsCreated)
	{
		CreatePrimitiveDescriptors(Context, InBinding);

		TArray<FSoftObjectPath> PathsToLoad;
		PathsToLoad.Reserve(PrimitiveInfos.Num());

		for (const FPCGPrimitiveInfo& PrimitiveInfo : PrimitiveInfos)
		{
			if (!PrimitiveInfo.Descriptor.StaticMesh.IsNull())
			{
				UE_LOGF(LogPCG, Verbose, "Request '%ls' to load.", *PrimitiveInfo.Descriptor.StaticMesh.ToString());
				PathsToLoad.Add(PrimitiveInfo.Descriptor.StaticMesh.ToSoftObjectPath());
			}
		}

		if (!PathsToLoad.IsEmpty())
		{
			ensure(!LoadHandle);
			LoadHandle = UAssetManager::Get().GetStreamableManager().RequestAsyncLoad(MoveTemp(PathsToLoad));
		}

		bPrimitiveDescriptorsCreated = true;

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}

	if (!bStaticMeshesLoaded)
	{
		for (const FPCGPrimitiveInfo& PrimitiveInfo : PrimitiveInfos)
		{
			if (PrimitiveInfo.Descriptor.StaticMesh.IsPending())
			{
				UE_LOGF(LogPCG, Verbose, "Waiting for '%ls' to load.", *PrimitiveInfo.Descriptor.StaticMesh.ToString());

				return false;
			}
		}

		bool bAnyInvalid = false;

		// User could pass us any soft object path so verify static meshes were loaded.
		for (const FPCGPrimitiveInfo& PrimitiveInfo : PrimitiveInfos)
		{
			if (!Cast<UStaticMesh>(PrimitiveInfo.Descriptor.StaticMesh.Get()))
			{
				UE_LOGF(LogPCG, Error, "Tried to use asset '%ls' as a static mesh.", *PrimitiveInfo.Descriptor.StaticMesh.ToString());
				bAnyInvalid = true;
			}
		}

		if (bAnyInvalid)
		{
			PrimitiveInfos.Reset();
		}

		bStaticMeshesLoaded = true;
	}

	if (!bPrimitivesSetUp)
	{
		// Weighted spawning has no analysis pass, so PrimitiveIdToWorldBounds is never populated.
		// Expand WorldBounds by the mesh's local extents so edge instances aren't incorrectly culled.
		// todo_pcg: Investigate moving weighted spawning to use the analysis kernel.
		if (!bSpawningByAttribute && !bSpawningByPrimitiveData)
		{
			ExpandPrimitiveBoundsByMeshBounds();
		}

		if (!SetupPrimitives(Context, InBinding))
		{
			return false;
		}

		bPrimitivesSetUp = true;

		if (NumPrimitivesSetup > 0)
		{
			if (UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get()))
			{
				SourceComponent->NotifyProceduralInstancesInUse();
			}
		}
		else
		{
			// No component set up means we have no more work to do.
			InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
			return true;
		}

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}

	// We know the name and type of the selected mesh attribute statically and declared the attribute in GetKernelAttributeKeys,
	// so the attribute ID should be present in the attribute table.
	SelectedMeshAttributeId = InBinding->GetAttributeId(Settings->OutAttributeName, EPCGKernelAttributeType::StringKey);
	ensure(SelectedMeshAttributeId != INDEX_NONE);

	InBinding->AddCompletedMeshSpawnerKernel(GetProducerKernel());
	return true;
}

FComputeDataProviderRenderProxy* UPCGStaticMeshSpawnerDataProvider::GetRenderProxy()
{
	return new FPCGStaticMeshSpawnerDataProviderProxy(
		AttributeIdOffsetStrides, SelectorAttributeId, PrimitiveIds, PrimitiveStringKeys, PrimitiveSelectionCDF, SelectedMeshAttributeId, PrimitiveMeshBounds, NumCullingCells, NumCullingCellsTotal, BoundsMin, CullingCellExtent);
}

void UPCGStaticMeshSpawnerDataProvider::Reset()
{
	Super::Reset();

	bSpawningByAttribute = false;
	bSpawningByPrimitiveData = false;
	AttributeIdOffsetStrides.Empty();
	PrimitiveIds.Empty();
	PrimitiveStringKeys.Empty();
	PrimitiveMeshBounds.Empty();
	PrimitiveSelectionCDF.Empty();
	SelectorAttributeId = INDEX_NONE;
	SelectedMeshAttributeId = INDEX_NONE;
	PrimitiveIdToInstanceRanges.Empty();
	PrimitiveIdToWorldBounds.Empty();
	PrimitiveIdToCullingCellWorldBounds.Empty();
	AnalysisDataIndex = INDEX_NONE;
	PrimitiveDataIndex = INDEX_NONE;
	CullingCellMinMaxPositionDataIndex = INDEX_NONE;
	bPrimitiveDescriptorsCreated = false;
	PrimitiveInfos.Empty();
	CustomFloatCount = 0;
	bPrimitivesSetUp = false;
	NumPrimitivesSetup = 0;
	bRegisteredPrimitives = false;
	bStaticMeshesLoaded = false;
	NumCullingCells = FIntVector(1, 1, 1);
	NumCullingCellsTotal = 1;
	BoundsMin = FVector3f::ZeroVector;
	CullingCellExtent = 0.0;
	LoadHandle.Reset();
	CustomPrimitiveData.Reset();
}

void UPCGStaticMeshSpawnerDataProvider::CreatePrimitiveDescriptors(FPCGContext* InContext, UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGStaticMeshSpawnerDataProvider::CreatePrimitiveDescriptors);
	check(InContext);

	PrimitiveInfos.Empty();

	const UPCGStaticMeshSpawnerSettings* Settings = CastChecked<UPCGStaticMeshSpawnerSettings>(GetProducerKernel()->GetSettings());

	if (!ensure(Settings->MeshSelectorParameters))
	{
		return;
	}

	const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);

	if (!ensure(InputDataDesc))
	{
		return;
	}

	const int32 TotalInputPointCount = InputDataDesc->ComputeTotalElementCount();
	if (TotalInputPointCount == 0)
	{
		return;
	}

	if (TotalInputPointCount >= MAX_INSTANCE_ID)
	{
		PCG_KERNEL_VALIDATION_ERR(InContext, Settings, FText::Format(LOCTEXT("TooManyInstances", "Tried to spawn too many instances ({0}), procedural ISM component creation skipped and instances will not be rendered."), TotalInputPointCount));
		return;
	}

	if (!InContext->ExecutionSource.IsValid())
	{
		return;
	}

	const TOptional<TConstArrayView<FPCGAttributePropertyInputSelector>> AttributeSelectors = Settings->InstanceDataPackerParameters ? Settings->InstanceDataPackerParameters->GetAttributeSelectors() : TOptional<TConstArrayView<FPCGAttributePropertyInputSelector>>();

	if (AttributeSelectors.IsSet())
	{
		for (const FPCGAttributePropertyInputSelector& AttributeSelector : AttributeSelectors.GetValue())
		{
			const FName AttributeName = AttributeSelector.GetAttributeName();

			if (!AttributeSelector.IsBasicAttribute() || AttributeName == PCGMetadataAttributeConstants::LastAttributeName)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings,
					FText::Format(LOCTEXT("OnlyBasicAttributesSupported", "Attribute '{0}' is invalid. GPU instance packer implementation currently only supports basic attributes."), AttributeSelector.GetDisplayText()));
				continue;
			}

			if (AttributeName == NAME_None)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings,
					FText::Format(LOCTEXT("InstanceDataAttributeInvalid", "Invalid instance data attribute specified '{0}'."), FText::FromName(AttributeName)));

				continue;
			}

			if (!InputDataDesc->ContainsAttributeOnAnyData(AttributeName))
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings,
					FText::Format(LOCTEXT("InstanceDataAttributeNotFound", "Instance data attribute '{0}' not found."), FText::FromName(AttributeName)));

				continue;
			}
		}

		PCGMeshSpawnerPackingHelpers::ComputeCustomFloatPacking(InContext, Settings, AttributeSelectors.GetValue(), InputDataDesc, CustomFloatCount, AttributeIdOffsetStrides);
	}

	if (const UPCGMeshSelectorByAttribute* SelectorByAttribute = Cast<UPCGMeshSelectorByAttribute>(Settings->MeshSelectorParameters))
	{
		const FName SelectorName = SelectorByAttribute->AttributeName;

		PrimitiveInfos.Reserve(PrimitiveIdToInstanceRanges.Num());

		for (TPair<int32, TArray<FPCGInstanceRange>>& PrimitiveIdAndInstanceInfo : PrimitiveIdToInstanceRanges)
		{
			uint32 TotalInstanceCount = 0;
			for (const FPCGInstanceRange& InstanceRange : PrimitiveIdAndInstanceInfo.Value)
			{
				TotalInstanceCount += InstanceRange.GetNumInstances();
			}

			if (TotalInstanceCount == 0)
			{
				continue;
			}

			if (!ensure(InBinding->GetStringTable().IsValidIndex(PrimitiveIdAndInstanceInfo.Key)))
			{
				continue;
			}

			const FString& MeshPathString = InBinding->GetStringTable()[PrimitiveIdAndInstanceInfo.Key];
			if (MeshPathString.IsEmpty())
			{
				continue;
			}

			if (PrimitiveInfos.Num() >= PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER)
			{
				PCG_KERNEL_VALIDATION_WARN(InContext, Settings,
					FText::Format(PCGStaticMeshSpawnerDataInterface::TooManyPrimitivesFormat, FText::FromString(FString::FromInt(PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER))));

				break;
			}

			FPCGProceduralISMComponentDescriptor Descriptor;
			Descriptor = SelectorByAttribute->TemplateDescriptor;
			Descriptor.NumInstances = TotalInstanceCount;
			Descriptor.NumCustomFloats = CustomFloatCount;
			Descriptor.StaticMesh = FSoftObjectPath(MeshPathString);

			const FBox* FoundWorldBounds = PrimitiveIdToWorldBounds.Find(PrimitiveIdAndInstanceInfo.Key);
			Descriptor.WorldBounds = FoundWorldBounds ? *FoundWorldBounds : InContext->ExecutionSource->GetExecutionState().GetBounds();
			// Sanity check instance count.
			if (!ensure(Descriptor.NumInstances <= TotalInputPointCount))
			{
				Descriptor.NumInstances = TotalInputPointCount;
			}

			PrimitiveIds.Add(PrimitiveIdAndInstanceInfo.Key);

			FPCGPrimitiveInfo& PrimitiveInfo = PrimitiveInfos.Emplace_GetRef();
			PrimitiveInfo.Descriptor = MoveTemp(Descriptor);
			PrimitiveInfo.InstanceRanges = MoveTemp(PrimitiveIdAndInstanceInfo.Value);

			if (const TArray<FBox>* FoundCullingCellBounds = PrimitiveIdToCullingCellWorldBounds.Find(PrimitiveIdAndInstanceInfo.Key))
			{
				PrimitiveInfo.CullingCellWorldBounds = *FoundCullingCellBounds;
			}

			// For by-attribute spawning, the primitive ID is the mesh path string key.
			PrimitiveStringKeys.Add(PrimitiveIdAndInstanceInfo.Key);
		}

		PrimitiveSelectionCDF.SetNumZeroed(PrimitiveInfos.Num());
	}
	else if (const UPCGMeshSelectorPrimitiveData* SelectorPrimitiveData = Cast<UPCGMeshSelectorPrimitiveData>(Settings->MeshSelectorParameters))
	{
		if (PrimitiveIdToInstanceRanges.IsEmpty())
		{
			return;
		}

		const bool bIndexValid = InBinding->GetInputDataCollection().TaggedData.IsValidIndex(PrimitiveDataIndex);
		const UPCGParamData* PrimitiveData = bIndexValid ? Cast<UPCGParamData>(InBinding->GetInputDataCollection().TaggedData[PrimitiveDataIndex].Data) : nullptr;
		const UPCGMetadata* PrimitiveMetadata = PrimitiveData ? PrimitiveData->ConstMetadata() : nullptr;
		
		if (!PrimitiveMetadata)
		{
			PCG_KERNEL_VALIDATION_ERR(InContext, Settings, LOCTEXT("NoPrimitiveData", "No primitive data found, no primitives will spawn."));
			return;
		}

		const FPCGMetadataAttributeBase* MeshAttributeBase = PrimitiveMetadata->GetConstAttribute(SelectorPrimitiveData->MeshAttribute);
		const bool bIsString = MeshAttributeBase && MeshAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FString>::Id;
		const bool bIsSoftObjectPath = MeshAttributeBase && MeshAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FSoftObjectPath>::Id;
		if (!MeshAttributeBase || (!bIsString && !bIsSoftObjectPath))
		{
			PCG_KERNEL_VALIDATION_WARN(InContext, Settings,
				FText::Format(LOCTEXT("MeshAttributeInvalid", "Mesh attribute {0} not found in primitive data, or is not of required type Soft Object Path or String."), FText::FromName(SelectorPrimitiveData->MeshAttribute)));
			return;
		}

		const FPCGMetadataAttribute<FString>* MeshAttributeString = bIsString ? static_cast<const FPCGMetadataAttribute<FString>*>(MeshAttributeBase) : nullptr;
		const FPCGMetadataAttribute<FSoftObjectPath>* MeshAttributeSoftObjectPath = bIsSoftObjectPath ? static_cast<const FPCGMetadataAttribute<FSoftObjectPath>*>(MeshAttributeBase) : nullptr;

		FPCGProceduralISMComponentDescriptor Descriptor;
		Descriptor = SelectorPrimitiveData->TemplateDescriptor;
		Descriptor.NumCustomFloats = CustomFloatCount;

		FPCGObjectOverrides PropertyOverrides(&Descriptor);
		PropertyOverrides.Initialize(SelectorPrimitiveData->PrimitiveOverrideAttributes, &Descriptor, PrimitiveData, InContext);

		FPCGMeshMaterialOverrideHelper MaterialOverrideHelper;
		MaterialOverrideHelper.Initialize(*InContext, /*bInByAttributeOverride=*/true, SelectorPrimitiveData->MaterialOverrideAttributes, PrimitiveMetadata);

		SelectorPrimitiveData->PackCustomPrimitiveData(PrimitiveData, PrimitiveIdToInstanceRanges.Num(), CustomPrimitiveData, InContext);

		const int32 NumPrimitiveDataElements = PrimitiveMetadata->GetItemCountForChild();

		PrimitiveInfos.Reserve(PrimitiveIdToInstanceRanges.Num());
		PrimitiveIds.Reserve(PrimitiveIdToInstanceRanges.Num());
		PrimitiveStringKeys.Reserve(PrimitiveIdToInstanceRanges.Num());
		
		for (TPair<int32, TArray<FPCGInstanceRange>>& PrimitiveIdAndInstanceInfo : PrimitiveIdToInstanceRanges)
		{
			uint32 TotalInstanceCount = 0;
			for (const FPCGInstanceRange& InstanceRange : PrimitiveIdAndInstanceInfo.Value)
			{
				TotalInstanceCount += InstanceRange.GetNumInstances();
			}

			if (TotalInstanceCount == 0)
			{
				continue;
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(UPCGStaticMeshSpawnerDataProvider::CreatePrimitiveDescriptor);

			const int32 PrimitiveId = PrimitiveIdAndInstanceInfo.Key;
			if (PrimitiveId >= NumPrimitiveDataElements)
			{
				PCG_KERNEL_VALIDATION_WARN(InContext, Settings,
					FText::Format(LOCTEXT("OutOfBoundsPrimitiveIndex", "Primitive index {0} exceeds size of primitive table ({1})."), PrimitiveId, NumPrimitiveDataElements));
				continue;
			}

			const PCGMetadataValueKey ValueKey = MeshAttributeSoftObjectPath ? MeshAttributeSoftObjectPath->GetValueKey(PrimitiveId) : MeshAttributeString->GetValueKey(PrimitiveId);
			const FSoftObjectPath MeshPath = MeshAttributeSoftObjectPath ? MeshAttributeSoftObjectPath->GetValue(ValueKey) : FSoftObjectPath(MeshAttributeString->GetValue(ValueKey));

			if (PrimitiveInfos.Num() >= PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER)
			{
				PCG_KERNEL_VALIDATION_WARN(InContext, Settings,
					FText::Format(PCGStaticMeshSpawnerDataInterface::TooManyPrimitivesFormat, FText::FromString(FString::FromInt(PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER))));

				break;
			}

			Descriptor.NumInstances = TotalInstanceCount;
			Descriptor.StaticMesh = MeshPath;

			const FBox* FoundWorldBounds = PrimitiveIdToWorldBounds.Find(PrimitiveIdAndInstanceInfo.Key);
			Descriptor.WorldBounds = FoundWorldBounds ? *FoundWorldBounds : InContext->ExecutionSource->GetExecutionState().GetBounds();
			PropertyOverrides.Apply(PrimitiveId);

			if (MaterialOverrideHelper.OverridesMaterials())
			{
				for (const TSoftObjectPtr<UMaterialInterface>& Material : MaterialOverrideHelper.GetMaterialOverrides(PrimitiveId))
				{
					// todo_pcg: Add async load.
					Descriptor.OverrideMaterials.Add(Material.LoadSynchronous());
				}
			}

			// Sanity check instance count.
			if (!ensure(Descriptor.NumInstances <= TotalInputPointCount))
			{
				Descriptor.NumInstances = TotalInputPointCount;
			}

			PrimitiveIds.Add(PrimitiveId);

			FPCGPrimitiveInfo& PrimitiveInfo = PrimitiveInfos.Emplace_GetRef();
			PrimitiveInfo.Descriptor = MoveTemp(Descriptor);
			PrimitiveInfo.InstanceRanges = PrimitiveIdAndInstanceInfo.Value;
			PrimitiveInfo.CullingCellExtent = CullingCellExtent;
			PrimitiveInfo.NumCullingCells = NumCullingCells;

			if (const TArray<FBox>* FoundCullingCellBounds = PrimitiveIdToCullingCellWorldBounds.Find(PrimitiveIdAndInstanceInfo.Key))
			{
				PrimitiveInfo.CullingCellWorldBounds = *FoundCullingCellBounds;
			}

			// Look up the mesh path string key value.
			PrimitiveStringKeys.Add(InBinding->GetStringTable().IndexOfByKey(MeshPath));
		}

		PrimitiveSelectionCDF.SetNumZeroed(PrimitiveInfos.Num());
	}
	else if (const UPCGMeshSelectorWeighted* SelectorWeighted = Cast<UPCGMeshSelectorWeighted>(Settings->MeshSelectorParameters))
	{
		if (SelectorWeighted->MeshEntries.IsEmpty())
		{
			PCG_KERNEL_VALIDATION_ERR(InContext, Settings, PCGStaticMeshSpawnerDataInterface::NoMeshEntriesFormat);
			return;
		}

		float CumulativeWeight = 0.0f;

		float TotalWeight = 0.0f;
		for (const FPCGMeshSelectorWeightedEntry& Entry : SelectorWeighted->MeshEntries)
		{
			TotalWeight += Entry.Weight;
		}

		if (TotalWeight < UE_SMALL_NUMBER)
		{
			return;
		}

		PrimitiveSelectionCDF.Reserve(SelectorWeighted->MeshEntries.Num());

		PrimitiveInfos.Reserve(SelectorWeighted->MeshEntries.Num());

		for (const FPCGMeshSelectorWeightedEntry& Entry : SelectorWeighted->MeshEntries)
		{
			if (Entry.Descriptor.StaticMesh.IsNull())
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, Settings, FText::Format(PCGStaticMeshSpawnerDataInterface::CouldNotLoadStaticMeshFormat, FText::FromString(Entry.Descriptor.StaticMesh.ToString())));
				continue;
			}

			const float Weight = float(Entry.Weight) / TotalWeight;

			if (Weight < UE_SMALL_NUMBER)
			{
				continue;
			}

			if (PrimitiveInfos.Num() >= PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER)
			{
				PCG_KERNEL_VALIDATION_WARN(InContext, Settings,
					FText::Format(PCGStaticMeshSpawnerDataInterface::TooManyPrimitivesFormat, FText::FromString(FString::FromInt(PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER))));

				break;
			}

			CumulativeWeight += Weight;
			PrimitiveSelectionCDF.Add(CumulativeWeight);

			// For weighted spawning, primitive ID is the string key value.
			const int32 StringKeyValue = InBinding->GetStringTable().IndexOfByKey(Entry.Descriptor.StaticMesh.ToString());
			PrimitiveIds.Add(StringKeyValue);
			PrimitiveStringKeys.Add(StringKeyValue);

			FPCGProceduralISMComponentDescriptor Descriptor;
			Descriptor = Entry.Descriptor;
			Descriptor.NumCustomFloats = CustomFloatCount;
			Descriptor.StaticMesh = Entry.Descriptor.StaticMesh;

			const FBox* FoundWorldBounds = PrimitiveIdToWorldBounds.Find(StringKeyValue);
			Descriptor.WorldBounds = FoundWorldBounds ? *FoundWorldBounds : InContext->ExecutionSource->GetExecutionState().GetBounds();
			int32 InstanceCount = FMath::CeilToInt(TotalInputPointCount * Weight);

			if (SelectorWeighted->MeshEntries.Num() > 1)
			{
				// Since we'll be selecting meshes based on random draws using the point random seeds which we don't have on CPU,
				// we may pick more or less than the expected number of instances for each mesh. Use binomial variance to calculate
				// the overallocation that gives 99.7% confidence (3 sigma).
				const double Variance = double(TotalInputPointCount) * Weight * (1.0f - Weight);
				const double Sigma = FMath::Sqrt(Variance);
				const uint32 AdditionalAllocation = FMath::CeilToInt(3.0f * Sigma);

				// Useful for debugging instance counts.
				//UE_LOGF(LogPCG, Warning, "Over allocation, N = %d, P = %.2f, NP = %d, 3sig = %f, Extra = %d (%.2f%%), Calculated = %d, Final = %d", TotalInputPointCount, Weight, InstanceCount, float(3.0f * Sigma), AdditionalAllocation, float(AdditionalAllocation) / TotalInputPointCount, InstanceCount + AdditionalAllocation, FMath::Min(InstanceCount + AdditionalAllocation, TotalInputPointCount));

				InstanceCount += AdditionalAllocation;
			}

			Descriptor.NumInstances = FMath::Min(InstanceCount, TotalInputPointCount);

			FPCGPrimitiveInfo& PrimitiveInfo = PrimitiveInfos.Emplace_GetRef();
			PrimitiveInfo.Descriptor = MoveTemp(Descriptor);
			PrimitiveInfo.InstanceRanges.Emplace(Descriptor.NumInstances);
			PrimitiveInfo.CullingCellExtent = CullingCellExtent;
			PrimitiveInfo.NumCullingCells = NumCullingCells;
		}
	}

	FPCGSpawnerPrimitives& Primitives = InBinding->FindOrAddMeshSpawnerPrimitives(GetProducerKernel());
	Primitives.NumCustomFloats = CustomFloatCount;
	Primitives.AttributeIdOffsetStrides = AttributeIdOffsetStrides;
	Primitives.SelectorAttributeId = SelectorAttributeId;
	Primitives.SelectionCDF = PrimitiveSelectionCDF;
	Primitives.PrimitiveMeshBounds = PrimitiveMeshBounds;

	// Useful for debugging instance counts.
	//UE_LOGF(LogPCG, Log, "Input points %d, total instance count %d (+ %.2f%%)", TotalInputPointCount, Primitives.NumInstancesAllPrimitives, (float(Primitives.NumInstancesAllPrimitives - TotalInputPointCount) / TotalInputPointCount) * 100.0f);
}

bool UPCGStaticMeshSpawnerDataProvider::SetupPrimitives(FPCGContext* InContext, UPCGDataBinding* InBinding)
{
	FPCGSpawnerPrimitives* Primitives = InBinding->FindMeshSpawnerPrimitives(GetProducerKernel());
	if (!Primitives)
	{
		return true;
	}

	const UPCGStaticMeshSpawnerSettings* Settings = CastChecked<UPCGStaticMeshSpawnerSettings>(GetProducerKernel()->GetSettings());

	TSharedPtr<IPCGPrimitiveFactoryISMBase> Factory;

	if (!Primitives->PrimitiveFactory)
	{
		// If we have no descriptors, we can early out. Only checked when the primitive factory is not initialized,
		// as we're moving the descritors into the factory (which will emtpy PrimitiveDescriptors)
		if (PrimitiveInfos.IsEmpty())
		{
			return true;
		}

		// Ensure SMs are compiled, required to prevent ensure when partially built SM is accessed later in FastGeo code.
		for (const FPCGPrimitiveInfo& PrimitiveInfo : PrimitiveInfos)
		{
			if (ensure(PrimitiveInfo.Descriptor.StaticMesh.IsValid()) && PrimitiveInfo.Descriptor.StaticMesh->IsCompiling())
			{
				return false;
			}
		}

		bool bCreateComponentless = false;

		if (PCGStaticMeshSpawnerDataInterface::CVarCreatePrimitivesComponentless.GetValueOnGameThread() && PCGPrimitiveFactoryHelpers::GetFastGeoPrimitiveFactory())
		{
			if (UWorld* World = InContext->ExecutionSource.IsValid() ? InContext->ExecutionSource->GetExecutionState().GetWorld() : nullptr)
			{
				bCreateComponentless = true;
			}
		}

		if (bCreateComponentless)
		{
			Factory = PCGPrimitiveFactoryHelpers::GetFastGeoPrimitiveFactory();
		}
		else
		{
			Factory = MakeShared<FPCGPrimitiveFactoryPISMC>();
		}

		Primitives->PrimitiveFactory = Factory;

		IPCGPrimitiveFactoryISMBase::FParameters Params;
		Params.PrimitiveInfos = MoveTemp(PrimitiveInfos);
		Params.CustomPrimitiveData = MoveTemp(CustomPrimitiveData);
		Params.TargetActor = Settings->TargetActor.Get() ? Settings->TargetActor.Get() : InContext->GetTypedExecutionTarget<AActor>();

		Factory->Initialize(MoveTemp(Params));
	}
	else
	{
		Factory = StaticCastSharedPtr<IPCGPrimitiveFactoryISMBase>(Primitives->PrimitiveFactory);
	}

	if (!Factory->Create(InContext))
	{
		// Not finished, continue next tick.
		return false;
	}

	const int32 NumPrimitives = Factory->GetNumPrimitives();

	if (Settings->bApplyMeshBoundsToPoints)
	{
		PrimitiveMeshBounds.Reserve(PrimitiveMeshBounds.Num() + NumPrimitives);
		for (int32 PrimitiveIndex = 0; PrimitiveIndex < NumPrimitives; ++PrimitiveIndex)
		{
			PrimitiveMeshBounds.Add(Factory->GetMeshBounds(PrimitiveIndex));
		}
	}

	NumPrimitivesSetup = NumPrimitives;

	return true;
}

FPCGStaticMeshSpawnerDataProviderProxy::FPCGStaticMeshSpawnerDataProviderProxy(
	const TArray<FUintVector4>& InAttributeIdOffsetStrides,
	int32 InSelectorAttributeId,
	const TArray<int32>& InPrimitiveIds,
	const TArray<int32>& InPrimitiveStringKeys,
	TArray<float> InSelectionCDF,
	int32 InSelectedMeshAttributeId,
	const TArray<FBox>& InPrimitiveMeshBounds,
	FIntVector InNumSubdivisions,
	int32 InNumSubdivisionsTotal,
	FVector3f InBoundsMin,
	float InBucketExtent)
	: AttributeIdOffsetStrides(InAttributeIdOffsetStrides)
	, SelectionCDF(InSelectionCDF)
	, SelectorAttributeId(InSelectorAttributeId)
	, PrimitiveIds(InPrimitiveIds)
	, PrimitiveStringKeys(InPrimitiveStringKeys)
	, SelectedMeshAttributeId(InSelectedMeshAttributeId)
	, NumCullingCells(InNumSubdivisions)
	, NumCullingCellsTotal(InNumSubdivisionsTotal)
	, CullingCellExtent(InBucketExtent)
	, BoundsMin(InBoundsMin)
{
	PrimitiveMeshBoundsMin.Reserve(InPrimitiveMeshBounds.Num());
	PrimitiveMeshBoundsMax.Reserve(InPrimitiveMeshBounds.Num());

	for (int Index = 0; Index < InPrimitiveMeshBounds.Num(); ++Index)
	{
		PrimitiveMeshBoundsMin.Add(FVector4f(InPrimitiveMeshBounds[Index].Min.X, InPrimitiveMeshBounds[Index].Min.Y, InPrimitiveMeshBounds[Index].Min.Z, /*Unused*/0.0f));
		PrimitiveMeshBoundsMax.Add(FVector4f(InPrimitiveMeshBounds[Index].Max.X, InPrimitiveMeshBounds[Index].Max.Y, InPrimitiveMeshBounds[Index].Max.Z, /*Unused*/0.0f));
	}
}

bool FPCGStaticMeshSpawnerDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters) && NumCullingCellsTotal > 0;
}

void FPCGStaticMeshSpawnerDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	if (!PrimitiveIds.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(PrimitiveIds.GetTypeSize(), PrimitiveIds.Num());

		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGStaticMeshSpawner_PrimitiveIds"));
		PrimitiveIdsBufferSRV = GraphBuilder.CreateSRV(Buffer);

		GraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(PrimitiveIds));
	}
	else
	{
		PrimitiveIdsBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, PrimitiveIds.GetTypeSize())));
	}

	if (!PrimitiveStringKeys.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(PrimitiveStringKeys.GetTypeSize(), PrimitiveStringKeys.Num());

		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGStaticMeshSpawner_PrimitiveStringKeys"));
		PrimitiveStringKeysBufferSRV = GraphBuilder.CreateSRV(Buffer);

		GraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(PrimitiveStringKeys));
	}
	else
	{
		PrimitiveStringKeysBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, PrimitiveStringKeys.GetTypeSize())));
	}

	if (!PrimitiveMeshBoundsMin.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(PrimitiveMeshBoundsMin.GetTypeSize(), PrimitiveMeshBoundsMin.Num());

		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGStaticMeshSpawner_PrimitiveMeshBoundsMin"));
		PrimitiveMeshBoundsMinBufferSRV = GraphBuilder.CreateSRV(Buffer);

		GraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(PrimitiveMeshBoundsMin));
	}
	else
	{
		PrimitiveMeshBoundsMinBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, PrimitiveMeshBoundsMin.GetTypeSize())));
	}

	if (!PrimitiveMeshBoundsMax.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(PrimitiveMeshBoundsMax.GetTypeSize(), PrimitiveMeshBoundsMax.Num());

		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGStaticMeshSpawner_PrimitiveMeshBoundsMax"));
		PrimitiveMeshBoundsMaxBufferSRV = GraphBuilder.CreateSRV(Buffer);

		GraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(PrimitiveMeshBoundsMax));
	}
	else
	{
		PrimitiveMeshBoundsMaxBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, PrimitiveMeshBoundsMax.GetTypeSize())));
	}
}

void FPCGStaticMeshSpawnerDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];

		Parameters.NumAttributes = AttributeIdOffsetStrides.Num();
		Parameters.NumPrimitives = SelectionCDF.Num();
		Parameters.SelectorAttributeId = SelectorAttributeId;
		Parameters.SelectedMeshAttributeId = SelectedMeshAttributeId;
		Parameters.NumCullingCells = NumCullingCells;
		Parameters.NumCullingCellsTotal = NumCullingCellsTotal;
		Parameters.BoundsMin = BoundsMin;
		Parameters.BucketExtent = CullingCellExtent;

		for (int32 Index = 0; Index < AttributeIdOffsetStrides.Num(); ++Index)
		{
			Parameters.AttributeIdOffsetStrides[Index] = AttributeIdOffsetStrides[Index];
		}

		Parameters.PrimitiveIds = PrimitiveIdsBufferSRV;
		Parameters.PrimitiveStringKeys = PrimitiveStringKeysBufferSRV;

		for (int32 Index = 0; Index < SelectionCDF.Num(); ++Index)
		{
			GET_SCALAR_ARRAY_ELEMENT(Parameters.SelectionCDF, Index) = SelectionCDF[Index];
		}

		Parameters.ApplyBounds = PrimitiveMeshBoundsMin.IsEmpty() ? 0 : 1;

		Parameters.PrimitiveMeshBoundsMin = PrimitiveMeshBoundsMinBufferSRV;
		Parameters.PrimitiveMeshBoundsMax = PrimitiveMeshBoundsMaxBufferSRV;
	}
}

#undef LOCTEXT_NAMESPACE
