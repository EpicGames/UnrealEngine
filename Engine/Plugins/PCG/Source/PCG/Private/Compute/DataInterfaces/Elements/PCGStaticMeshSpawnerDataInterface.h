// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "Compute/PrimitiveFactories/PCGPrimitiveFactoryPISMC.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "Components/PCGProceduralISMComponentDescriptor.h"
#include "InstanceDataPackers/PCGInstanceDataPackerBase.h"

#include "PCGStaticMeshSpawnerDataInterface.generated.h"

class FPCGStaticMeshSpawnerDataInterfaceParameters;
struct FPCGContext;
struct FStreamableHandle;

/** Data Interface to marshal Static Mesh Spawner settings to the GPU. */
UCLASS(ClassGroup = (Procedural))
class UPCGStaticMeshSpawnerDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGStaticMeshSpawner"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	static constexpr uint32 MAX_ATTRIBUTES = 64;
};

UCLASS()
class UPCGStaticMeshSpawnerDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	virtual bool PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding) override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	void CreatePrimitiveDescriptors(FPCGContext* InContext, UPCGDataBinding* InBinding);

	/** Setup primitives. Returns true if at least one component was set up. */
	virtual bool SetupPrimitives(FPCGContext* InContext, UPCGDataBinding* InBinding);

protected:
	virtual bool ReadbackMeshSelectionData(UPCGDataBinding* InBinding);
	virtual bool ReadbackPrimitiveData(UPCGDataBinding* InBinding);
	virtual bool ReadbackCullingCellMinMaxPositionData(UPCGDataBinding* InBinding);

	/** Expands WorldBounds of each primitive by its mesh's local extents from the pivot.
	 *  Called before SetupPrimitives in weighted mode, which does not run analysis and has not expanded the primitive bounds. */
	void ExpandPrimitiveBoundsByMeshBounds();

public:
	bool bSpawningByAttribute = false;
	bool bSpawningByPrimitiveData = false;

	/** Attributes to use for writing per-instance custom floats. */
	TArray<FUintVector4> AttributeIdOffsetStrides;

	/** Array of primitive IDs used by the spawner kernel to look up a spawned primitive index from a selector attribute value.
	* When using direct by-attribute selection, each primitive ID is the string key value of the mesh path, otherwise it is an
	* index into a primitive data table.
	*/
	TArray<int32> PrimitiveIds;

	/** Mesh path string keys of each primitive. */
	TArray<int32> PrimitiveStringKeys;

	TArray<FBox> PrimitiveMeshBounds;

	TArray<float> PrimitiveSelectionCDF;

	/** Attribute Id for mesh selector. */
	int32 SelectorAttributeId = INDEX_NONE;

	/** Output attribute Id for the output selected mesh. */
	int32 SelectedMeshAttributeId = INDEX_NONE;

	/** Ranges of instances for each primitive. */
	TMap</*Primitive Id*/int32, TArray<FPCGInstanceRange>> PrimitiveIdToInstanceRanges;

	TMap</*Primitive Id*/int32, FBox> PrimitiveIdToWorldBounds;

	/** Per-range world bounds. Outer key is primitive ID; inner array is aligned 1:1 with the matching PrimitiveIdToInstanceRanges entry. */
	TMap</*Primitive Id*/int32, TArray<FBox>> PrimitiveIdToCullingCellWorldBounds;

	int32 AnalysisDataIndex = INDEX_NONE;
	int32 PrimitiveDataIndex = INDEX_NONE;
	int32 CullingCellMinMaxPositionDataIndex = INDEX_NONE;

	bool bPrimitiveDescriptorsCreated = false;

	TArray<FPCGPrimitiveInfo> PrimitiveInfos;

	FPCGPackedCustomData CustomPrimitiveData;

	uint32 CustomFloatCount = 0;

	bool bRegisteredPrimitives = false;

	bool bStaticMeshesLoaded = false;

	bool bPrimitivesSetUp = false;

	/** Number of primitives created during primitive setup. */
	int32 NumPrimitivesSetup = 0;

	/** Keeps loaded objects alive. */
	TSharedPtr<FStreamableHandle> LoadHandle;

	FIntVector NumCullingCells = FIntVector(1, 1, 1);
	int32 NumCullingCellsTotal = 1;
	float CullingCellExtent = 0.0;

	FVector3f BoundsMin = FVector3f::ZeroVector;
};

class FPCGStaticMeshSpawnerDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGStaticMeshSpawnerDataProviderProxy(
		const TArray<FUintVector4>& InAttributeIdOffsetStrides,
		int32 InSelectorAttributeId,
		const TArray<int32>& InPrimitiveIds,
		const TArray<int32>& InPrimitiveStringKeys,
		TArray<float> InSelectionCDF,
		int32 InSelectedMeshAttributeId,
		const TArray<FBox>& InPrimitiveMeshBounds,
		FIntVector InNumSubdivisions,
		int32 InNumSubdivisionsTotal,
		FVector3f InBoundsMin,
		float InBucketExtent);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGStaticMeshSpawnerDataInterfaceParameters;

	TArray<FUintVector4> AttributeIdOffsetStrides;
	TArray<float> SelectionCDF;

	int32 SelectorAttributeId = INDEX_NONE;

	TArray<int32> PrimitiveIds;
	FRDGBufferSRVRef PrimitiveIdsBufferSRV = nullptr;

	TArray<int32> PrimitiveStringKeys;
	FRDGBufferSRVRef PrimitiveStringKeysBufferSRV = nullptr;

	TArray<FVector4f> PrimitiveMeshBoundsMin;
	FRDGBufferSRVRef PrimitiveMeshBoundsMinBufferSRV = nullptr;

	TArray<FVector4f> PrimitiveMeshBoundsMax;
	FRDGBufferSRVRef PrimitiveMeshBoundsMaxBufferSRV = nullptr;

	int32 SelectedMeshAttributeId = INDEX_NONE;

	FIntVector NumCullingCells = FIntVector(1, 1, 1);
	int32 NumCullingCellsTotal = 1;
	float CullingCellExtent = 0.0f;
	FVector3f BoundsMin = FVector3f::ZeroVector;
};
