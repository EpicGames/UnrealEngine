// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTransformPointsDataInterface.h"

#include "PCGContext.h"
#include "Compute/PCGDataBinding.h"
#include "Elements/PCGTransformPoints.h"

#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "Algo/Find.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGTransformPointsDataInterface)

#define LOCTEXT_NAMESPACE "PCGTransformPointsDataInterface"

void UPCGTransformPointsDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("TransformPoints_GetApplyToAttribute"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint, 1));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("TransformPoints_GetAttributeId"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Int, 1));
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGTransformPointsDataInterfaceParameters,)
	SHADER_PARAMETER(uint32, bApplyToAttribute)
	SHADER_PARAMETER(int32, AttributeId)
END_SHADER_PARAMETER_STRUCT()

void UPCGTransformPointsDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGTransformPointsDataInterfaceParameters>(UID);
}

void UPCGTransformPointsDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	OutHLSL += FString::Format(TEXT(
		"uint {DataInterfaceName}_bApplyToAttribute;\n"
		"int {DataInterfaceName}_AttributeId;\n"
		"\n"
		"uint TransformPoints_GetApplyToAttribute_{DataInterfaceName}() { return {DataInterfaceName}_bApplyToAttribute; }\n"
		"int TransformPoints_GetAttributeId_{DataInterfaceName}() { return {DataInterfaceName}_AttributeId; }\n"
		), TemplateArgs);
}

UComputeDataProvider* UPCGTransformPointsDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGTransformPointsDataProvider>();
}

FComputeDataProviderRenderProxy* UPCGTransformPointsDataProvider::GetRenderProxy()
{
	const UPCGTransformPointsSettings* Settings = CastChecked<UPCGTransformPointsSettings>(GetProducerKernel()->GetSettings());

	FPCGTransformPointsDataProviderProxy::FData ProxyData;
	ProxyData.bApplyToAttribute = Settings->bApplyToAttribute;
	ProxyData.AttributeId = AttributeId;

	return new FPCGTransformPointsDataProviderProxy(MoveTemp(ProxyData));
}

void UPCGTransformPointsDataProvider::Reset()
{
	Super::Reset();
	AttributeId = INDEX_NONE;
}

bool UPCGTransformPointsDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
	FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr;
	if (!ensure(Context))
	{
		return true;
	}

	const UPCGTransformPointsSettings* Settings = CastChecked<UPCGTransformPointsSettings>(GetProducerKernel()->GetSettings());

	if (Settings->bApplyToAttribute && AttributeId == INDEX_NONE)
	{
		const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);

		if (!ensure(InputDataDesc))
		{
			return true;
		}

		bool bAnyPointsPresent = false;

		for (const FPCGDataDesc& Desc : InputDataDesc->GetDataDescriptions())
		{
			if (Desc.GetElementCount().X <= 0)
			{
				continue;
			}

			bAnyPointsPresent = true;

			const FPCGKernelAttributeDesc* It = Algo::FindByPredicate(Desc.GetAttributeDescriptions(), [Settings](const FPCGKernelAttributeDesc& AttributeDesc)
			{
				return AttributeDesc.GetAttributeKey().GetIdentifier().Name == Settings->AttributeName && AttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::Transform;
			});

			if (It)
			{
				AttributeId = It->GetAttributeId();
				break;
			}
		}

		// Mute this error if the point data is empty.
		if (AttributeId == INDEX_NONE && !InputDataDesc->GetDataDescriptions().IsEmpty() && bAnyPointsPresent)
		{
			PCG_KERNEL_VALIDATION_ERR(Context, Settings, FText::Format(
				LOCTEXT("TransformPointsAttributeNotFound", "Transform points attribute '{0}' not found."),
				FText::FromName(Settings->AttributeName)));
		}
	}

	return true;
}

bool FPCGTransformPointsDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGTransformPointsDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.bApplyToAttribute = Data.bApplyToAttribute;
		Parameters.AttributeId = Data.AttributeId;
	}
}

#undef LOCTEXT_NAMESPACE
