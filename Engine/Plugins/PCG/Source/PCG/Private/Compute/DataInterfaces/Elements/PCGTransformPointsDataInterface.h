// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "PCGTransformPointsDataInterface.generated.h"

class FPCGTransformPointsDataInterfaceParameters;

/**
 * Data interface for non-overridable transform points params (bApplyToAttribute, AttributeId).
 * Overridable params (OffsetMin, ScaleMax, etc.) are handled by the auto-created UPCGKernelParamsDataInterface.
 */
UCLASS(ClassGroup = (Procedural))
class UPCGTransformPointsDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGTransformPointsDataInterface"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface
};

UCLASS()
class UPCGTransformPointsDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UPCGComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	/** Output attribute Id. */
	UPROPERTY()
	int32 AttributeId = INDEX_NONE;
};

class FPCGTransformPointsDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FData
	{
		uint32 bApplyToAttribute = 0;
		int32 AttributeId = INDEX_NONE;
	};

	FPCGTransformPointsDataProviderProxy(FData InData)
		: Data(MoveTemp(InData))
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGTransformPointsDataInterfaceParameters;

	FData Data;
};
