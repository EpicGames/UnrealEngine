// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGWorldRaycastDataInterface.h"

#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGDataBinding.h"
#include "Elements/PCGWorldRaycast.h"

#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHIResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGWorldRaycastDataInterface)

void UPCGWorldRaycastDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WorldRaycast_GetRaycastMode"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WorldRaycast_GetRayDirection"))
		.AddReturnType(EShaderFundamentalType::Float, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WorldRaycast_GetRayLength"))
		.AddReturnType(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WorldRaycast_GetUnbounded"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WorldRaycast_GetKeepOriginalPointOnMiss"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WorldRaycast_GetRayOriginAttributeId"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WorldRaycast_GetRayEndAttributeId"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WorldRaycast_GetRayDirectionAttributeId"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WorldRaycast_GetRayLengthAttributeId"))
		.AddReturnType(EShaderFundamentalType::Int);
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGWorldRaycastDataInterfaceParameters,)
	SHADER_PARAMETER(int32, RaycastMode)
	SHADER_PARAMETER(FVector3f, RayDirection)
	SHADER_PARAMETER(float, RayLength)
	SHADER_PARAMETER(int32, Unbounded)
	SHADER_PARAMETER(int32, KeepOriginalPointOnMiss)
	SHADER_PARAMETER(int32, RayOriginAttributeId)
	SHADER_PARAMETER(int32, RayEndAttributeId)
	SHADER_PARAMETER(int32, RayDirectionAttributeId)
	SHADER_PARAMETER(int32, RayLengthAttributeId)
END_SHADER_PARAMETER_STRUCT()

void UPCGWorldRaycastDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGWorldRaycastDataInterfaceParameters>(UID);
}

void UPCGWorldRaycastDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	OutHLSL += FString::Format(TEXT(
		"int {DataInterfaceName}_RaycastMode;\n"
		"float3 {DataInterfaceName}_RayDirection;\n"
		"float {DataInterfaceName}_RayLength;\n"
		"int {DataInterfaceName}_Unbounded;\n"
		"int {DataInterfaceName}_KeepOriginalPointOnMiss;\n"
		"int {DataInterfaceName}_RayOriginAttributeId;\n"
		"int {DataInterfaceName}_RayEndAttributeId;\n"
		"int {DataInterfaceName}_RayDirectionAttributeId;\n"
		"int {DataInterfaceName}_RayLengthAttributeId;\n"
		"\n"
		"int WorldRaycast_GetRaycastMode_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_RaycastMode;\n"
		"}\n"
		"\n"
		"float3 WorldRaycast_GetRayDirection_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_RayDirection;\n"
		"}\n"
		"\n"
		"float WorldRaycast_GetRayLength_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_RayLength;\n"
		"}\n"
		"\n"
		"int WorldRaycast_GetUnbounded_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_Unbounded;\n"
		"}\n"
		"\n"
		"int WorldRaycast_GetKeepOriginalPointOnMiss_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_KeepOriginalPointOnMiss;\n"
		"}\n"
		"\n"
		"int WorldRaycast_GetRayOriginAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_RayOriginAttributeId;\n"
		"}\n"
		"\n"
		"int WorldRaycast_GetRayEndAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_RayEndAttributeId;\n"
		"}\n"
		"\n"
		"int WorldRaycast_GetRayDirectionAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_RayDirectionAttributeId;\n"
		"}\n"
		"\n"
		"int WorldRaycast_GetRayLengthAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_RayLengthAttributeId;\n"
		"}\n"
	), TemplateArgs);
}

UComputeDataProvider* UPCGWorldRaycastDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGWorldRaycastDataProvider>();
}

FComputeDataProviderRenderProxy* UPCGWorldRaycastDataProvider::GetRenderProxy()
{
	const UPCGWorldRaycastElementSettings* Settings = CastChecked<UPCGWorldRaycastElementSettings>(GetProducerKernel()->GetSettings());

	FPCGWorldRaycastProviderProxy::FData ProxyData =
	{
		.RaycastMode = static_cast<int32>(Settings->RaycastMode),
		.RayDirection = Settings->RayDirection,
		.RayLength = static_cast<float>(Settings->RayLength),
		.Unbounded = Settings->bUnbounded,
		.KeepOriginalPointOnMiss = Settings->bKeepOriginalPointOnMiss,
		.RayOriginAttributeId = RayOriginAttributeId,
		.RayEndAttributeId = RayEndAttributeId,
		.RayDirectionAttributeId = RayDirectionAttributeId,
		.RayLengthAttributeId = RayLengthAttributeId
	};

	return new FPCGWorldRaycastProviderProxy(ProxyData);
}

void UPCGWorldRaycastDataProvider::Reset()
{
	RayOriginAttributeId = INDEX_NONE;
	RayEndAttributeId = INDEX_NONE;
	RayDirectionAttributeId = INDEX_NONE;
	RayLengthAttributeId = INDEX_NONE;

	Super::Reset();
}

bool UPCGWorldRaycastDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGWorldRaycastDataProvider::PrepareForExecute_GameThread);
	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	const UPCGWorldRaycastElementSettings* Settings = CastChecked<UPCGWorldRaycastElementSettings>(GetProducerKernel()->GetSettings());
	RayOriginAttributeId = Settings->OriginInputAttribute.GetSelection() == EPCGAttributePropertySelection::Attribute ? InBinding->GetAttributeId(Settings->OriginInputAttribute.GetAttributeName(), EPCGKernelAttributeType::Float3) : INDEX_NONE;
	RayEndAttributeId = Settings->EndPointAttribute.GetSelection() == EPCGAttributePropertySelection::Attribute ? InBinding->GetAttributeId(Settings->EndPointAttribute.GetAttributeName(), EPCGKernelAttributeType::Float3) : INDEX_NONE;
	RayDirectionAttributeId = Settings->bOverrideRayDirections ? InBinding->GetAttributeId(Settings->RayDirectionAttribute.GetAttributeName(), EPCGKernelAttributeType::Float3) : INDEX_NONE;
	RayLengthAttributeId = Settings->bOverrideRayLengths ? InBinding->GetAttributeId(Settings->RayLengthAttribute.GetAttributeName(), EPCGKernelAttributeType::Float) : INDEX_NONE;

	return true;
}

bool FPCGWorldRaycastProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGWorldRaycastProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.RaycastMode = Data.RaycastMode;
		Parameters.RayDirection = static_cast<FVector3f>(Data.RayDirection);
		Parameters.RayLength = Data.RayLength;
		Parameters.Unbounded = Data.Unbounded;
		Parameters.KeepOriginalPointOnMiss = Data.KeepOriginalPointOnMiss;
		Parameters.RayOriginAttributeId = Data.RayOriginAttributeId;
		Parameters.RayEndAttributeId = Data.RayEndAttributeId;
		Parameters.RayDirectionAttributeId = Data.RayDirectionAttributeId;
		Parameters.RayLengthAttributeId = Data.RayLengthAttributeId;
	}
}
