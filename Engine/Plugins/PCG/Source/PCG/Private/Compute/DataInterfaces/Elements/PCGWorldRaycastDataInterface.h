// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "PCGWorldRaycastDataInterface.generated.h"

class FPCGWorldRaycastDataInterfaceParameters;

/** Data Interface to marshal World Raycast settings to the GPU. */
UCLASS(ClassGroup = (Procedural))
class UPCGWorldRaycastDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGWorldRaycast"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface
};

UCLASS()
class UPCGWorldRaycastDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	//~ Begin UPCGComputeDataProvider Interface
	virtual void Reset() override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

public:
	int32 RayOriginAttributeId = INDEX_NONE;
	int32 RayEndAttributeId = INDEX_NONE;
	int32 RayDirectionAttributeId = INDEX_NONE;
	int32 RayLengthAttributeId = INDEX_NONE;
};

class FPCGWorldRaycastProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FData
	{
		int32 RaycastMode = 0;
		FVector RayDirection = FVector::ZeroVector;
		float RayLength = 0.0f;
		int32 Unbounded = 0;
		int32 KeepOriginalPointOnMiss = 0;
		int32 RayOriginAttributeId = 0;
		int32 RayEndAttributeId = 0;
		int32 RayDirectionAttributeId = 0;
		int32 RayLengthAttributeId = 0;
	};

	FPCGWorldRaycastProviderProxy(const FData& InData)
		: Data(InData)
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGWorldRaycastDataInterfaceParameters;

	FData Data;
};
