// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGSettings.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGKernelOverridableParamTypeInfo.h"
#include "Compute/PCGKernelParamsCache.h"

#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ComputeMetadataBuilder.h"
#include "ComputeFramework/ShaderParameterMetadataAllocation.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGComputeDataInterface)

#if WITH_EDITOR
namespace PCGComputeDataInterfaceHelpers
{
	static TAutoConsoleVariable<bool> CVarEnableInternalKernelPinInspection(
		TEXT("pcg.GPU.EnableInternalKernelPinInspection"),
		false,
		TEXT("Allows internal kernel pins to be inspected and debugged. Note, this only works if the internal kernel pin name matches a pin name on the node."));
}
#endif

void UPCGComputeDataInterface::AddDownstreamInputPin(FName InInputPinLabel, const FName* InOptionalInputPinLabelAlias)
{
	DownstreamInputPinLabelAliases.AddUnique(InOptionalInputPinLabelAlias ? *InOptionalInputPinLabelAlias : InInputPinLabel);
}

void UPCGComputeDataInterface::SetOutputPin(FName InOutputPinLabel, const FName* InOptionalOutputPinLabelAlias)
{
	OutputPinLabel = InOutputPinLabel;
	OutputPinLabelAlias = InOptionalOutputPinLabelAlias ? *InOptionalOutputPinLabelAlias : InOutputPinLabel;
}

void UPCGComputeDataInterface::SetProducerSettings(const UPCGSettings* InProducerSettings)
{
	ResolvedProducerSettings = InProducerSettings;
	ProducerSettings = ResolvedProducerSettings;
}

const UPCGSettings* UPCGComputeDataInterface::GetProducerSettings() const
{
	if (!ResolvedProducerSettings)
	{
		FGCScopeGuard Guard;
		ResolvedProducerSettings = ProducerSettings.Get();
	}

	return ResolvedProducerSettings;
}

void UPCGComputeDataInterface::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UPCGComputeDataInterface* This = CastChecked<UPCGComputeDataInterface>(InThis);
	Collector.AddReferencedObject(This->ResolvedProducerSettings);
}

void UPCGComputeDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	// Bump counter so any async callbacks from previous usages of this provider will be invalidated.
	++GenerationCounter;

	const UPCGComputeDataInterface* DataInterface = CastChecked<UPCGComputeDataInterface>(InDataInterface);
	ProducerSettings = DataInterface->GetProducerSettings();
	bProducedByCPU = DataInterface->IsProducedByCPU();
	GraphBindingIndex = DataInterface->GetGraphBindingIndex();
	ProducerKernel = DataInterface->GetProducerKernel();
	DownstreamInputPinLabelAliases = DataInterface->GetDownstreamInputPinLabelAliases();

	// Set data that is only relevant for data produced on GPU, mainly to help prevent misuse.
	if (!bProducedByCPU)
	{
		// The original label is needed to store inspection data.
		OutputPinLabel = DataInterface->GetOutputPinLabel();

		// Use the aliased label for CPU data output as this is the output from the compute graph.
		OutputPinLabelAlias = DataInterface->GetOutputPinLabelAlias();
	}
}

void UPCGComputeDataProvider::Reset()
{
	Super::Reset();

	// Bump counter so any async callbacks from usages of this provider will be invalidated.
	++GenerationCounter;

	ProducerKernel = nullptr;
	ProducerSettings = nullptr;
	GraphBindingIndex = INDEX_NONE;
	OutputPinLabel = NAME_None;
	OutputPinLabelAlias = NAME_None;
	DownstreamInputPinLabelAliases.Empty();
	bProducedByCPU = false;
}

const UPCGSettings* UPCGComputeDataProvider::GetProducerSettings() const
{
	return ProducerSettings;
}

void UPCGComputeDataProvider::SetProducerSettings(const UPCGSettings* InSettings)
{
	ProducerSettings = InSettings;
}

#if WITH_EDITOR
void UPCGComputeDataProvider::NotifyProducerUploadedData(UPCGDataBinding* InBinding)
{
	const UPCGSettings* LocalProducerSettings = GetProducerSettings();

	if (LocalProducerSettings && !LocalProducerSettings->ShouldExecuteOnGPU())
	{
		const UPCGNode* ProducerNode = Cast<UPCGNode>(LocalProducerSettings->GetOuter());

		// Works around current issue where input output settings are outer'd to the graph rather than their node.
		if (!ProducerNode)
		{
			if (const UPCGGraph* Graph = Cast<UPCGGraph>(LocalProducerSettings->GetOuter()))
			{
				if (Graph->GetInputNode() && Graph->GetInputNode()->GetSettings() == LocalProducerSettings)
				{
					ProducerNode = Graph->GetInputNode();
				}
			}
		}

		TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
		FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr;
		if (ProducerNode && Context && Context->GetStack() && Context->ExecutionSource.IsValid())
		{
			Context->ExecutionSource->GetExecutionState().GetInspection().NotifyCPUToGPUUpload(ProducerNode, Context->GetStack());
		}
	}
}
#endif // WITH_EDITOR

void UPCGExportableDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGExportableDataInterface* DataInterface = CastChecked<UPCGExportableDataInterface>(InDataInterface);
	Binding = CastChecked<UPCGDataBinding>(InBinding);

	ExportMode = DataInterface->GetRequiresExport() ? EPCGExportMode::ComputeGraphOutput : EPCGExportMode::NoExport;

	// Debug/inspection functionality for GPU-produced data.
	if (!IsProducedByCPU())
	{
#if WITH_EDITOR
		// Some exportable data providers don't support inspect/debug because their producer settings are not available (e.g. GridLinkage).
		const UPCGSettings* LocalProducerSettings = GetProducerSettings();
		const bool bCanInspectOrDebug = LocalProducerSettings && (!GetProducerKernel() || !GetProducerKernel()->IsPinInternal(GetOutputPinLabel()) || PCGComputeDataInterfaceHelpers::CVarEnableInternalKernelPinInspection.GetValueOnAnyThread());

		if (bCanInspectOrDebug)
		{
			const IPCGGraphExecutionSource* ExecutionSource = Binding->GetExecutionSource();

			if (LocalProducerSettings->bIsInspecting && ExecutionSource && ExecutionSource->GetExecutionState().GetInspection().IsInspecting())
			{
				ExportMode |= EPCGExportMode::Inspection;
			}

			if (LocalProducerSettings->bDebug)
			{
				ExportMode |= EPCGExportMode::DebugVisualization;
			}
		}
#endif
	}
}

void UPCGExportableDataProvider::Reset()
{
	Super::Reset();

	ExportMode = EPCGExportMode::NoExport;
	OnDataExported = {};
	Binding.Reset();
	PinDataDescription = nullptr;
}

bool UPCGExportableDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	if (!PinDataDescription)
	{
		PinDataDescription = InBinding->GetCachedKernelPinDataDesc(GetGraphBindingIndex());
	}

	return true;
}

namespace PCGKernelParamsHelpers
{
	/** Write a single kernel param value from KernelParams into a byte buffer at the given offset. */
	static void WriteKernelParamValue(uint8* OutBuffer, uint32 InOffset, const FPCGKernelOverridableParam& InParam, const FPCGKernelParams& InKernelParams)
	{
		switch (InParam.UnderlyingType)
		{
		case EPCGMetadataTypes::Boolean:
		{
			const uint32 Value = InKernelParams.GetValueBool(InParam.Label) ? 1u : 0u;
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(uint32));
			break;
		}
		case EPCGMetadataTypes::Enum:
		{
			const uint32 Value = InKernelParams.GetValueEnumAsUint(InParam.Label);
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(uint32));
			break;
		}
		case EPCGMetadataTypes::Integer32:
		case EPCGMetadataTypes::Integer64:
		{
			const int32 Value = InKernelParams.GetValueInt(InParam.Label);
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(int32));
			break;
		}
		case EPCGMetadataTypes::Float:
		{
			const float Value = InKernelParams.GetValueFloat(InParam.Label);
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(float));
			break;
		}
		case EPCGMetadataTypes::Double:
		{
			const float Value = static_cast<float>(InKernelParams.GetValueDouble(InParam.Label));
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(float));
			break;
		}
		case EPCGMetadataTypes::Vector2:
		{
			const FVector2f Value(InKernelParams.GetValueVector2D(InParam.Label));
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(FVector2f));
			break;
		}
		case EPCGMetadataTypes::Vector:
		{
			const FVector3f Value(InKernelParams.GetValueVector(InParam.Label));
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(FVector3f));
			break;
		}
		case EPCGMetadataTypes::Rotator:
		{
			// Store as raw Pitch/Yaw/Roll components to match the attribute buffer layout used by the overridable system.
			const FRotator Rot = InKernelParams.GetValueRotator(InParam.Label);
			const FVector3f Value(static_cast<float>(Rot.Pitch), static_cast<float>(Rot.Yaw), static_cast<float>(Rot.Roll));
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(FVector3f));
			break;
		}
		case EPCGMetadataTypes::Vector4:
		{
			const FVector4f Value(InKernelParams.GetValueVector4(InParam.Label));
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(FVector4f));
			break;
		}
		case EPCGMetadataTypes::Quaternion:
		{
			const FQuat Quat = InKernelParams.GetValueQuat(InParam.Label);
			const FVector4f Value(static_cast<float>(Quat.X), static_cast<float>(Quat.Y), static_cast<float>(Quat.Z), static_cast<float>(Quat.W));
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(FVector4f));
			break;
		}
		case EPCGMetadataTypes::Transform:
		{
			const FMatrix44f Value(InKernelParams.GetValueTransform(InParam.Label).ToMatrixWithScale());
			FMemory::Memcpy(OutBuffer + InOffset, &Value, sizeof(FMatrix44f));
			break;
		}
		default:
			ensureMsgf(false, TEXT("Unsupported EPCGMetadataTypes in kernel param value write."));
			break;
		}
	}
}

FPCGKernelParamLayout FPCGKernelParamLayout::Build(const TArray<FPCGKernelOverridableParam>& InParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGKernelParamLayout::Build);
	FPCGKernelParamLayout Layout;

	if (InParams.IsEmpty())
	{
		return Layout;
	}

	// Build temporary metadata to determine the exact offsets and total size used by GetShaderParameters.
	// The metadata is discarded after extracting layout info.
	FShaderParametersMetadataBuilder TempBuilder;
	TArray<FShaderParametersMetadata*> TempNestedStructs;

	// Name strings must outlive TempMetadata (FShaderParametersMetadata stores raw TCHAR* pointers).
	// Param.Label.ToString() returns a temporary FString, so we must store the strings here.
	TArray<FString> TempNames;
	TempNames.Reserve(InParams.Num());

	for (const FPCGKernelOverridableParam& Param : InParams)
	{
		TempNames.Add(Param.Label.ToString());
		const FPCGKernelOverridableParamTypeInfo& TypeInfo = FPCGKernelOverridableParamTypeInfo::Get(Param.UnderlyingType);
		ComputeFramework::AddParamForType(TempBuilder, *TempNames.Last(), TypeInfo.GetShaderValueType(/*bForShaderParamStruct=*/true), TempNestedStructs);
	}

	FShaderParametersMetadata* TempMetadata = TempBuilder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("Temp"));

	const TArray<FShaderParametersMetadata::FMember>& Members = TempMetadata->GetMembers();
	Layout.ParamOffsets.SetNum(Members.Num());

	for (int32 i = 0; i < Members.Num(); ++i)
	{
		Layout.ParamOffsets[i] = Members[i].GetOffset();
	}

	Layout.BufferSize = TempMetadata->GetSize();

	// Clean up temporary metadata.
	delete TempMetadata;
	for (FShaderParametersMetadata* Nested : TempNestedStructs)
	{
		delete Nested;
	}

	return Layout;
}

const FPCGKernelParamLayout& UPCGKernelParamsDataInterface::GetKernelParamLayout() const
{
	if (!CachedKernelParamLayout.IsValid())
	{
		if (const UPCGComputeKernel* Kernel = GetProducerKernel())
		{
			CachedKernelParamLayout = FPCGKernelParamLayout::Build(Kernel->GetCachedOverridableParams());
		}
	}

	return CachedKernelParamLayout;
}

UComputeDataProvider* UPCGKernelParamsDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGKernelParamsDataProvider>();
}

void UPCGKernelParamsDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
#if WITH_EDITOR
	if (GetProducerKernel())
	{
		for (const FPCGKernelOverridableParam& OverridableParam : GetProducerKernel()->GetCachedOverridableParams())
		{
			const FPCGKernelOverridableParamTypeInfo& TypeInfo = FPCGKernelOverridableParamTypeInfo::Get(OverridableParam.UnderlyingType);

			OutFunctions.AddDefaulted_GetRef()
				.SetName(FString::Format(TEXT("Get{0}Internal"), { OverridableParam.Label.ToString(), }))
				.AddReturnType(TypeInfo.GetShaderValueType());
		}
	}
#else
	checkf(false, TEXT("GetSupportedInputs should only be called during compute graph compilation, which is editor-only."));
#endif
}

void UPCGKernelParamsDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	// This dynamically builds the same kind of metadata that per-node subclasses produce statically via BEGIN_SHADER_PARAMETER_STRUCT + AddNestedStruct<StaticType>(UID).
	// We do it dynamically because the set of kernel params varies per node and isn't known at C++ compile time.
	if (const UPCGComputeKernel* Kernel = GetProducerKernel())
	{
		if (!Kernel->GetCachedOverridableParams().IsEmpty())
		{
			FShaderParametersMetadataBuilder NestedBuilder;
			TArray<FShaderParametersMetadata*> NestedStructs;

			AddKernelShaderParams(NestedBuilder, NestedStructs, InOutAllocations);

			// FShaderParametersMetadata is a heap object that isn't serializable, so we can't pre-build it in-editor.
			FShaderParametersMetadata* Metadata = NestedBuilder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, UID);
			InOutAllocations.ShaderParameterMetadatas.Add(Metadata);
			InOutAllocations.ShaderParameterMetadatas.Append(NestedStructs);
			InOutBuilder.AddNestedStruct(UID, Metadata);
		}
	}
}

void UPCGKernelParamsDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
#if WITH_EDITOR
	if (GetProducerKernel())
	{
		for (const FPCGKernelOverridableParam& OverridableParam : GetProducerKernel()->GetCachedOverridableParams())
		{
			const FPCGKernelOverridableParamTypeInfo& TypeInfo = FPCGKernelOverridableParamTypeInfo::Get(OverridableParam.UnderlyingType);

			OutHLSL += FString::Format(TEXT(
				"{2} {0}_{1};\n"
				"{2} Get{1}Internal_{0}() {{ return {0}_{1}; }}\n"
			), { InDataInterfaceName, OverridableParam.Label.ToString(), TypeInfo.HLSLType });
		}
	}
#else
	checkf(false, TEXT("GetHLSL should only be called during compute graph compilation, which is editor-only."));
#endif
}

void UPCGKernelParamsDataInterface::AddKernelShaderParams(FShaderParametersMetadataBuilder& InOutBuilder, TArray<FShaderParametersMetadata*>& InOutNestedStructs, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	if (const UPCGComputeKernel* Kernel = GetProducerKernel())
	{
		// Add one member per overridable param. ComputeFramework::AddParamForType maps FShaderValueType
		// to the correct AddParam<T> call (e.g. Float3 -> AddParam<FVector3f>), handling alignment and
		// sizing automatically. This mirrors what BEGIN_SHADER_PARAMETER_STRUCT produces for a hand-written struct.
		for (const FPCGKernelOverridableParam& Param : Kernel->GetCachedOverridableParams())
		{
			// Name strings must be stored in InOutAllocations.Names so they outlive the FShaderParametersMetadata that references them.
			int32 NameIndex = InOutAllocations.Names.Add();
			InOutAllocations.Names[NameIndex] = Param.Label.ToString();

			const FPCGKernelOverridableParamTypeInfo& TypeInfo = FPCGKernelOverridableParamTypeInfo::Get(Param.UnderlyingType);
			ComputeFramework::AddParamForType(InOutBuilder, *InOutAllocations.Names[NameIndex], TypeInfo.GetShaderValueType(/*bForShaderParamStruct=*/true), InOutNestedStructs);
		}
	}
}

void UPCGKernelParamsDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGKernelParamsDataInterface* KernelParamsDI = CastChecked<UPCGKernelParamsDataInterface>(InDataInterface);
	CachedKernelParamLayout = KernelParamsDI->GetKernelParamLayout();
}

bool UPCGKernelParamsDataProvider::PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding)
{
	check(InBinding);

	if (!Super::PerformPreExecuteReadbacks_GameThread(InBinding))
	{
		return false;
	}

	CachedKernelParams = InBinding->GetCachedKernelParams(GetProducerKernel());

	return true;
}

void UPCGKernelParamsDataProvider::WriteKernelParamValues(uint8* OutBuffer, const FPCGKernelParamLayout& InLayout) const
{
	check(GetProducerKernel());
	check(CachedKernelParams);

	const TArray<FPCGKernelOverridableParam>& Params = GetProducerKernel()->GetCachedOverridableParams();

	for (int32 i = 0; i < Params.Num(); ++i)
	{
		PCGKernelParamsHelpers::WriteKernelParamValue(OutBuffer, InLayout.ParamOffsets[i], Params[i], *CachedKernelParams);
	}
}

FComputeDataProviderRenderProxy* UPCGKernelParamsDataProvider::GetRenderProxy()
{
	if (!ensure(CachedKernelParams))
	{
		return new FPCGKernelParamsDataProviderProxy(TArray<uint8>(), /*InParameterStructSize=*/0);
	}

	if (!ensure(CachedKernelParamLayout.IsValid()))
	{
		return new FPCGKernelParamsDataProviderProxy(TArray<uint8>(), /*InParameterStructSize=*/0);
	}

	TArray<uint8> ParameterData;
	ParameterData.SetNumZeroed(CachedKernelParamLayout.BufferSize);

	WriteKernelParamValues(ParameterData.GetData(), CachedKernelParamLayout);

	return new FPCGKernelParamsDataProviderProxy(MoveTemp(ParameterData), CachedKernelParamLayout.BufferSize);
}

void UPCGKernelParamsDataProvider::Reset()
{
	Super::Reset();
	CachedKernelParams = nullptr;
	CachedKernelParamLayout = {};
}

bool FPCGKernelParamsDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return ParameterStructSize == 0 || InValidationData.ParameterStructSize == static_cast<int32>(ParameterStructSize);
}

void FPCGKernelParamsDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	if (ParameterData.IsEmpty())
	{
		return;
	}

	for (int32 InvocationIndex = 0; InvocationIndex < InDispatchData.NumInvocations; ++InvocationIndex)
	{
		uint8* DestPtr = InDispatchData.ParameterBuffer + InDispatchData.ParameterBufferOffset + InvocationIndex * InDispatchData.ParameterBufferStride;
		FMemory::Memcpy(DestPtr, ParameterData.GetData(), ParameterData.Num());
	}
}

