// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/DataInterfaces/PCGCustomKernelDataInterface.h"

#include "PCGComponent.h"
#include "PCGModule.h"
#include "PCGSettings.h"
#include "Compute/PCGComputeKernel.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ComputeMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCustomKernelDataInterface)

void UPCGCustomKernelDataInterface::SetSettings(const UPCGSettings* InSettings)
{
	ResolvedSettings = InSettings;
	Settings = ResolvedSettings;
}

const UPCGSettings* UPCGCustomKernelDataInterface::GetSettings() const
{
	if (!ResolvedSettings)
	{
		FGCScopeGuard Guard;
		ResolvedSettings = Settings.Get();
	}
	
	return ResolvedSettings;
}

void UPCGCustomKernelDataInterface::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UPCGCustomKernelDataInterface* This = CastChecked<UPCGCustomKernelDataInterface>(InThis);
	Collector.AddReferencedObject(This->ResolvedSettings);
}

void UPCGCustomKernelDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetNumThreads"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint, 3));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetThreadCountMultiplier"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetSeed"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetSettingsSeed"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetComponentSeed"))
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint));

	// A convenient way to serve component bounds to all kernels. Could be pulled out into a PCG context DI in the future.
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetComponentBoundsMin"))
		.AddReturnType(EShaderFundamentalType::Float, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetComponentBoundsMax"))
		.AddReturnType(EShaderFundamentalType::Float, 3);
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGCustomKernelDataInterfaceParameters, )
	SHADER_PARAMETER(FUintVector3, NumThreads)
	SHADER_PARAMETER(uint32, ThreadCountMultiplier)
	SHADER_PARAMETER(uint32, Seed)
	SHADER_PARAMETER(uint32, SeedSettings)
	SHADER_PARAMETER(uint32, SeedComponent)
	SHADER_PARAMETER(FVector3f, ComponentBoundsMin)
	SHADER_PARAMETER(FVector3f, ComponentBoundsMax)
END_SHADER_PARAMETER_STRUCT()

void UPCGCustomKernelDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGCustomKernelDataInterfaceParameters>(UID);
}

void UPCGCustomKernelDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	OutHLSL += FString::Format(TEXT(
		"uint3 {DataInterfaceName}_NumThreads;\n"
		"uint {DataInterfaceName}_ThreadCountMultiplier;\n"
		"uint {DataInterfaceName}_Seed;\n"
		"uint {DataInterfaceName}_SeedSettings;\n"
		"uint {DataInterfaceName}_SeedComponent;\n"
		"float3 {DataInterfaceName}_ComponentBoundsMin;\n"
		"float3 {DataInterfaceName}_ComponentBoundsMax;\n"
		"\n"
		"uint3 GetNumThreads_{DataInterfaceName}()\n{\n\treturn {DataInterfaceName}_NumThreads;\n}\n\n"
		"uint GetThreadCountMultiplier_{DataInterfaceName}()\n{\n\treturn {DataInterfaceName}_ThreadCountMultiplier;\n}\n\n"
		"uint GetSeed_{DataInterfaceName}()\n{\n\treturn {DataInterfaceName}_Seed;\n}\n\n"
		"uint GetSettingsSeed_{DataInterfaceName}()\n{\n\treturn {DataInterfaceName}_SeedSettings;\n}\n\n"
		"uint GetComponentSeed_{DataInterfaceName}()\n{\n\treturn {DataInterfaceName}_SeedComponent;\n}\n\n"
		"float3 GetComponentBoundsMin_{DataInterfaceName}()\n{\n\treturn {DataInterfaceName}_ComponentBoundsMin;\n}\n\n"
		"float3 GetComponentBoundsMax_{DataInterfaceName}()\n{\n\treturn {DataInterfaceName}_ComponentBoundsMax;\n}\n\n"),
		TemplateArgs);
}

UComputeDataProvider* UPCGCustomKernelDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGCustomComputeKernelDataProvider>();
}

void UPCGCustomComputeKernelDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCustomComputeKernelDataProvider::Initialize);

	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGCustomKernelDataInterface* DataInterface = CastChecked<UPCGCustomKernelDataInterface>(InDataInterface);

	UPCGDataBinding* Binding = CastChecked<UPCGDataBinding>(InBinding);
	const IPCGGraphExecutionSource* ExecutionSource = Binding->GetExecutionSource();
	check(ExecutionSource);

	const UPCGSettings* DataInterfaceSettings = DataInterface->GetSettings();
	check(DataInterfaceSettings);
	check(DataInterface->Kernel);

	Kernel = DataInterface->Kernel;
	ThreadCountMultiplier = DataInterface->Kernel->GetThreadCountMultiplier();
	Seed = static_cast<uint32>(DataInterfaceSettings->GetSeed(ExecutionSource));
	SeedSettings = static_cast<uint32>(DataInterfaceSettings->Seed);
	SeedComponent = static_cast<uint32>(ExecutionSource->GetExecutionState().GetSeed());

	SourceComponentBounds = ExecutionSource->GetExecutionState().GetBounds();
}

bool UPCGCustomComputeKernelDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	ThreadCount = Kernel->ComputeThreadCount(InBinding);
	return true;
}

FComputeDataProviderRenderProxy* UPCGCustomComputeKernelDataProvider::GetRenderProxy()
{
	return new FPCGCustomComputeKernelDataProviderProxy(ThreadCount, ThreadCountMultiplier, Seed, SeedSettings, SeedComponent, SourceComponentBounds);
}

void UPCGCustomComputeKernelDataProvider::Reset()
{
	Kernel = nullptr;
	ThreadCount = -1;
	ThreadCountMultiplier = 0;
	Seed = 42;
	SeedSettings = 42;
	SeedComponent = 42;
	SourceComponentBounds = {};

	Super::Reset();
}

bool FPCGCustomComputeKernelDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters)
		&& ThreadCount >= 0;
}

int32 FPCGCustomComputeKernelDataProviderProxy::GetDispatchThreadCount(TArray<FIntVector>& InOutThreadCounts) const
{
	// Always dispatch at least one thread. This is necessary in order to flag the kernel as executed.
	InOutThreadCounts.Emplace(FMath::Max(1, ThreadCount), 1, 1);
	return InOutThreadCounts.Num();
}

void FPCGCustomComputeKernelDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < InDispatchData.NumInvocations; ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];

		// Thread count
		// Note: If this ThreadCount is zero, the kernel will still execute one thread, but terminates early when comparing thread index against this value.
		ensure(InDispatchData.bUnifiedDispatch || InDispatchData.NumInvocations == 1);
		Parameters.NumThreads.X = static_cast<uint32>(ThreadCount);
		Parameters.NumThreads.Y = Parameters.NumThreads.Z = 1u;
		Parameters.ThreadCountMultiplier = ThreadCountMultiplier;

		// Seed for the node
		Parameters.Seed = Seed;
		Parameters.SeedSettings = SeedSettings;
		Parameters.SeedComponent = SeedComponent;

		// Set component bounds
		Parameters.ComponentBoundsMin = (FVector3f)SourceComponentBounds.Min;
		Parameters.ComponentBoundsMax = (FVector3f)SourceComponentBounds.Max;
	}
}
