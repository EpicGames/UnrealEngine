// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "Compute/PCGDataBinding.h"

#include "PCGCustomKernelDataInterface.generated.h"

#define UE_API PCG_API

class FPCGCustomKernelDataInterfaceParameters;
class UPCGComputeKernel;

/** Interface for any meta data provided to the compute kernel, such as num threads. */
UCLASS(MinimalAPI, ClassGroup = (Procedural))
class UPCGCustomKernelDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	static UE_API const TCHAR* NumThreadsReservedName;
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGCustomKernel"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider() const override;
	// This DI will provide execution parameters like dispatch information.
	bool IsExecutionInterface() const override { return true; }
	//~ End UComputeDataInterface Interface
		
	void SetSettings(const UPCGSettings* InSettings);
	const UPCGSettings* GetSettings() const;

	UPROPERTY()
	TObjectPtr<const UPCGComputeKernel> Kernel = nullptr;

	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

protected:
	UPROPERTY()
	TSoftObjectPtr<const UPCGSettings> Settings = nullptr;

	mutable TObjectPtr<const UPCGSettings> ResolvedSettings = nullptr;
};

/** Compute Framework Data Provider for each custom compute kernel. */
UCLASS()
class UPCGCustomComputeKernelDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	UPROPERTY()
	TObjectPtr<const UPCGComputeKernel> Kernel = nullptr;

	int32 ThreadCount = -1;
	uint32 ThreadCountMultiplier = 0;

	uint32 Seed = 42;
	uint32 SeedSettings = 42;
	uint32 SeedComponent = 42;

	FBox SourceComponentBounds;
};

class FPCGCustomComputeKernelDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGCustomComputeKernelDataProviderProxy(int32 InThreadCount, uint32 InThreadCountMultiplier, uint32 InSeed, uint32 InSeedSettings, uint32 InSeedComponent, const FBox& InSourceComponentBounds)
		: ThreadCount(InThreadCount)
		, ThreadCountMultiplier(InThreadCountMultiplier)
		, Seed(InSeed)
		, SeedSettings(InSeedSettings)
		, SeedComponent(InSeedComponent)
		, SourceComponentBounds(InSourceComponentBounds)
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	int32 GetDispatchThreadCount(TArray<FIntVector>& InOutThreadCounts) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGCustomKernelDataInterfaceParameters;

	int32 ThreadCount = -1;
	uint32 ThreadCountMultiplier = 0;

	uint32 Seed = 42;
	uint32 SeedSettings = 42;
	uint32 SeedComponent = 42;

	FBox SourceComponentBounds;
};

#undef UE_API
