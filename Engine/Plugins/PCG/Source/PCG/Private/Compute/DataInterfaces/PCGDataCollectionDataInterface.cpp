// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/DataInterfaces/PCGDataCollectionDataInterface.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGSettings.h"
#include "PCGSubsystem.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGDataDescription.h"
#include "Compute/PCGDataUploadAdaptor.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/Packing/PCGDataCollectionPacking.h"
#include "Helpers/PCGHelpers.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SystemTextures.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataCollectionDataInterface)

#define LOCTEXT_NAMESPACE "PCGDataCollectionDataInterface"

#if WITH_EDITOR
namespace PCGDataCollectionDataInterface
{
	using FMakeFunctionName = FString(*)(const TCHAR*);

	/** Functions exposed identically on both SRV (input) and UAV (output) views. They don't touch the data collection buffer, so no duplication is needed. */
	static void AddCommonFunctions(TArray<FShaderFunctionDefinition>& OutFunctions)
	{
		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetElementCountMultiplier"))
			.AddReturnType(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetAllocatedNativePropertiesMask"))
			.AddReturnType(EShaderFundamentalType::Uint);
	}

	/** Read-path functions that need an SRV variant on input pins and a UAV variant on output pins. MakeName selects which suffix to apply. */
	static void AddReadFunctions(TArray<FShaderFunctionDefinition>& OutFunctions, FMakeFunctionName MakeName)
	{
		// Internal direct access to buffer, used to optimize access.
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("LoadBufferInternal")))
			.AddReturnType(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		// Header Readers
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetNumData")))
			.AddReturnType(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetDataNumAttributesInternal")))
			.AddReturnType(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetNumElements")))
			.AddReturnType(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetThreadData")))
			.AddParam(EShaderFundamentalType::Uint) // InThreadIndex
			.AddParam(EShaderFundamentalType::Uint, 0, 0, EShaderParamModifier::Out) // OutDataIndex
			.AddParam(EShaderFundamentalType::Uint, 0, 0, EShaderParamModifier::Out) // OutElementIndex
			.AddReturnType(EShaderFundamentalType::Bool);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetNumElements")))
			.AddReturnType(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetElementAddressInternal")))
			.AddReturnType(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint) // InDataIndex
			.AddParam(EShaderFundamentalType::Uint) // InElementIndex
			.AddParam(EShaderFundamentalType::Int); // InAttributeId

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetElementAddressAndStrideInternal")))
			.AddReturnType(EShaderFundamentalType::Uint, 2)
			.AddParam(EShaderFundamentalType::Uint) // InDataIndex
			.AddParam(EShaderFundamentalType::Uint) // InElementIndex
			.AddParam(EShaderFundamentalType::Int); // InAttributeId

		// Attribute Getters
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetBool")))
			.AddReturnType(EShaderFundamentalType::Bool)
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint) // ElementIndex
			.AddParam(EShaderFundamentalType::Uint); // AttributeId

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetInt")))
			.AddReturnType(EShaderFundamentalType::Int)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFloat")))
			.AddReturnType(EShaderFundamentalType::Float)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFloat2")))
			.AddReturnType(EShaderFundamentalType::Float, 2)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFloat3")))
			.AddReturnType(EShaderFundamentalType::Float, 3)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFloat4")))
			.AddReturnType(EShaderFundamentalType::Float, 4)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetRotator")))
			.AddReturnType(EShaderFundamentalType::Float, 3)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetQuat")))
			.AddReturnType(EShaderFundamentalType::Float, 4)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetTransform")))
			.AddReturnType(EShaderFundamentalType::Float, 4, 4)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetStringKey")))
			.AddReturnType(EShaderFundamentalType::Int) // String key represented by int
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint) // ElementIndex
			.AddParam(EShaderFundamentalType::Uint); // AttributeId

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetName")))
			.AddReturnType(EShaderFundamentalType::Int) // Name represented by int
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint) // ElementIndex
			.AddParam(EShaderFundamentalType::Uint); // AttributeId

		// First Attribute Getters. Useful for overridable params. Currently not exposed to declarations and only called from compiler generated code.
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsBool")))
			.AddReturnType(EShaderFundamentalType::Bool);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsUint")))
			.AddReturnType(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsInt")))
			.AddReturnType(EShaderFundamentalType::Int);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsFloat")))
			.AddReturnType(EShaderFundamentalType::Float);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsFloat2")))
			.AddReturnType(EShaderFundamentalType::Float, 2);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsFloat3")))
			.AddReturnType(EShaderFundamentalType::Float, 3);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsFloat4")))
			.AddReturnType(EShaderFundamentalType::Float, 4);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsRotator")))
			.AddReturnType(EShaderFundamentalType::Float, 3);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsQuat")))
			.AddReturnType(EShaderFundamentalType::Float, 4);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsTransform")))
			.AddReturnType(EShaderFundamentalType::Float, 4, 4);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsStringKey")))
			.AddReturnType(EShaderFundamentalType::Int); // String key represented by int

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetFirstAttributeAsName")))
			.AddReturnType(EShaderFundamentalType::Int); // Name represented by int

		// Point Attribute Getters
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetPosition")))
			.AddReturnType(EShaderFundamentalType::Float, 3)
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint); // ElementIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetRotation")))
			.AddReturnType(EShaderFundamentalType::Float, 4)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetScale")))
			.AddReturnType(EShaderFundamentalType::Float, 3)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetBoundsMin")))
			.AddReturnType(EShaderFundamentalType::Float, 3)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetBoundsMax")))
			.AddReturnType(EShaderFundamentalType::Float, 3)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetColor")))
			.AddReturnType(EShaderFundamentalType::Float, 4)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetDensity")))
			.AddReturnType(EShaderFundamentalType::Float)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetSeed")))
			.AddReturnType(EShaderFundamentalType::Int)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetSteepness")))
			.AddReturnType(EShaderFundamentalType::Float)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("IsPointRemoved")))
			.AddReturnType(EShaderFundamentalType::Bool)
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint); // ElementIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("GetPointTransform")))
			.AddReturnType(EShaderFundamentalType::Float, 4, 4)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);
	}

	/** Write-path functions. UAV-only, so they get a single (unsuffixed) registration on the output side. */
	static void AddWriteFunctions(TArray<FShaderFunctionDefinition>& OutFunctions, FMakeFunctionName MakeName)
	{
		// Internal direct access to buffer, used to optimize access.
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("StoreBufferInternal")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		// Deprecated (5.8)
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetAsExecutedInternal")));

		// Attribute Setters
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetBool")))
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint) // ElementIndex
			.AddParam(EShaderFundamentalType::Uint) // AttributeId
			.AddParam(EShaderFundamentalType::Bool); // Value

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetInt")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Int);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetFloat")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetFloat2")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 2);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetFloat3")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 3);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetFloat4")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 4);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetRotator")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 3);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetQuat")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 4);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetTransform")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 4, 4);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetStringKey")))
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint) // ElementIndex
			.AddParam(EShaderFundamentalType::Uint) // AttributeId
			.AddParam(EShaderFundamentalType::Int); // String key represented by int

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetName")))
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint) // ElementIndex
			.AddParam(EShaderFundamentalType::Uint) // AttributeId
			.AddParam(EShaderFundamentalType::Int); // Name represented by int

		// Atomics
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("AtomicAddInt")))
			.AddReturnType(EShaderFundamentalType::Int) // Value before it was incremented
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint) // ElementIndex
			.AddParam(EShaderFundamentalType::Int) // AttributeId
			.AddParam(EShaderFundamentalType::Int); // ValueToAdd

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("AtomicMaxUint")))
			.AddReturnType(EShaderFundamentalType::Uint) // Value before max operation
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint) // ElementIndex
			.AddParam(EShaderFundamentalType::Int) // AttributeId
			.AddParam(EShaderFundamentalType::Uint); // ValueToMax

		// Point Attribute Setters
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetPosition")))
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint) // ElementIndex
			.AddParam(EShaderFundamentalType::Float, 3); // Value

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetRotation")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 4);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetScale")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 3);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetBoundsMin")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 3);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetBoundsMax")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 3);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetColor")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 4);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetDensity")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetSeed")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Int);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetSteepness")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("SetPointTransform")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Float, 4, 4);

		// Misc
		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("RemovePoint")))
			.AddParam(EShaderFundamentalType::Uint) // DataIndex
			.AddParam(EShaderFundamentalType::Uint); // ElementIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("InitializePoint")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(MakeName(TEXT("InitializeMetadata")))
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);
	}
}
#endif // WITH_EDITOR

void UPCGDataCollectionDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
#if WITH_EDITOR
	PCGDataCollectionDataInterface::AddCommonFunctions(OutFunctions);

	if (bLegacyAllowSettersOnInputs)
	{
		// In legacy mode input pins also expose writers when bound as kernel input, which can be undefined behaviour, but was exposed previously.
		PCGDataCollectionDataInterface::AddReadFunctions(OutFunctions, &PCGComputeHelpers::GetUAVFunction);
		PCGDataCollectionDataInterface::AddWriteFunctions(OutFunctions, &PCGComputeHelpers::GetUAVFunction);
	}
	else
	{
		PCGDataCollectionDataInterface::AddReadFunctions(OutFunctions, &PCGComputeHelpers::GetSRVFunction);
	}
#endif // WITH_EDITOR
}

void UPCGDataCollectionDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
#if WITH_EDITOR
	PCGDataCollectionDataInterface::AddCommonFunctions(OutFunctions);
	PCGDataCollectionDataInterface::AddReadFunctions(OutFunctions, &PCGComputeHelpers::GetUAVFunction);
	PCGDataCollectionDataInterface::AddWriteFunctions(OutFunctions, &PCGComputeHelpers::GetUAVFunction);
#endif // WITH_EDITOR
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGDataCollectionDataInterfaceParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, DataCollectionBufferSRV)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, DataCollectionBufferUAV)
	SHADER_PARAMETER(uint32, ElementCountMultiplier)
	SHADER_PARAMETER(uint32, AllocatedNativePropertiesMask)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int32>, GraphToBufferAttributeId)
	SHADER_PARAMETER(int32, FirstRemappedAttributeId)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int32>, BufferToGraphStringKey)
	SHADER_PARAMETER(int32, NumRemappedStringKeys)
END_SHADER_PARAMETER_STRUCT()

void UPCGDataCollectionDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGDataCollectionDataInterfaceParameters>(UID);
}

TCHAR const* UPCGDataCollectionDataInterface::TemplateFilePath = TEXT("/Plugin/PCG/Private/PCGDataCollectionDataInterface.ush");

TCHAR const* UPCGDataCollectionDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UPCGDataCollectionDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UPCGDataCollectionDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
#if WITH_EDITOR
	FString TemplateFile;
	if (!ensure(LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr)))
	{
		return;
	}

	// Expand the {{BeginDuplicateForSRVUAV}} / {{EndDuplicateForSRVUAV}} regions into parallel SRV and UAV variants.
	const FString ExpandedTemplate = PCGComputeHelpers::ExpandShaderTemplateForSRVUAV(TemplateFile);

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};
	OutHLSL += FString::Format(*ExpandedTemplate, TemplateArgs);
#endif // WITH_EDITOR
}

void UPCGDataCollectionDataInterface::GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const
{
	Super::GetDefines(OutDefinitionSet);

	// Point property IDs
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_POINT_POSITION_ID"),   FString::FromInt(PCGDataCollectionPackingConstants::POINT_POSITION_ATTRIBUTE_ID)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_POINT_ROTATION_ID"),   FString::FromInt(PCGDataCollectionPackingConstants::POINT_ROTATION_ATTRIBUTE_ID)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_POINT_SCALE_ID"),      FString::FromInt(PCGDataCollectionPackingConstants::POINT_SCALE_ATTRIBUTE_ID)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_POINT_BOUNDS_MIN_ID"), FString::FromInt(PCGDataCollectionPackingConstants::POINT_BOUNDS_MIN_ATTRIBUTE_ID)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_POINT_BOUNDS_MAX_ID"), FString::FromInt(PCGDataCollectionPackingConstants::POINT_BOUNDS_MAX_ATTRIBUTE_ID)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_POINT_COLOR_ID"),      FString::FromInt(PCGDataCollectionPackingConstants::POINT_COLOR_ATTRIBUTE_ID)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_POINT_DENSITY_ID"),    FString::FromInt(PCGDataCollectionPackingConstants::POINT_DENSITY_ATTRIBUTE_ID)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_POINT_SEED_ID"),       FString::FromInt(PCGDataCollectionPackingConstants::POINT_SEED_ATTRIBUTE_ID)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_POINT_STEEPNESS_ID"),  FString::FromInt(PCGDataCollectionPackingConstants::POINT_STEEPNESS_ATTRIBUTE_ID)));

	// Header sizes
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_DATA_COLLECTION_HEADER_SIZE_BYTES"), FString::FromInt(PCGDataCollectionPackingConstants::DATA_COLLECTION_HEADER_SIZE_BYTES)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_DATA_HEADER_SIZE_BYTES"),            FString::FromInt(PCGDataCollectionPackingConstants::DATA_HEADER_SIZE_BYTES)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_ATTRIBUTE_HEADER_SIZE_BYTES"),       FString::FromInt(PCGDataCollectionPackingConstants::ATTRIBUTE_HEADER_SIZE_BYTES)));

	// Misc
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_INVALID_DENSITY"),      FString::Printf(TEXT("%f"), PCGDataCollectionPackingConstants::INVALID_DENSITY)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_NUM_RESERVED_ATTRS"),   FString::FromInt(PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS)));

	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_ATTRIBUTE_STRIDE_MASK"), FString::FromInt(PCGDataCollectionPackingConstants::AttributeStrideMask)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_ATTRIBUTE_ALLOCATED_MASK"), FString::FromInt(PCGDataCollectionPackingConstants::AttributeAllocatedMask)));
}

UComputeDataProvider* UPCGDataCollectionDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGDataCollectionDataProvider>();
}

void UPCGDataCollectionDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataCollectionDataProvider::Initialize);

	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	// Initialize settings relevant for data produced on GPU.
	if (!IsProducedByCPU())
	{
		const UPCGDataCollectionDataInterface* DataInterface = CastChecked<UPCGDataCollectionDataInterface>(InDataInterface);

		// Only applicable for outputs from GPU nodes (not uploads from CPU).
		bRequiresZeroInitialization = DataInterface->bRequiresZeroInitialization;
		ElementCountMultiplier = DataInterface->ElementCountMultiplier;
	}
}

bool UPCGDataCollectionDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataCollectionDataProvider::PrepareForExecute_GameThread);
	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	if (!GetPinDescription())
	{
		UE_LOGF(LogPCG, Warning, "UPCGDataCollectionDataProvider: missing kernel pin data description. Kernel='%ls', output pin label='%ls'",
			GetProducerKernel() ? *GetProducerKernel()->GetName() : TEXT("MISSING"),
			*GetOutputPinLabel().ToString());

		return true;
	}

	if (!bBufferSizeValidated)
	{
		SizeBytes = PCGDataCollectionPackingHelpers::ComputePackedSizeBytes(GetPinDescription());
		bIsBufferSizeValid = !PCGComputeHelpers::IsBufferSizeTooLarge(SizeBytes);

		if (!bIsBufferSizeValid)
		{
			return true;
		}

		bBufferSizeValidated = true;
	}

	if (IsProducedByCPU())
	{
		if (!DataAdaptor)
		{
			// Requires virtual input pin labels to pick the data items from the compute graph element's input data collection.
			check(!GetDownstreamInputPinLabelAliases().IsEmpty());
			DataAdaptor = MakeShared<FPCGDataUploadAdaptor>(InBinding, GetPinDescription(), GetDownstreamInputPinLabelAliases()[0]);
		}

		const bool bPreparationDone = !DataAdaptor || DataAdaptor->PrepareData_GameThread(GetProducerKernel(), GetProducerSettings());

#if WITH_EDITOR
		if (bPreparationDone && DataAdaptor && DataAdaptor->IsUploadingFromCPU())
		{
			NotifyProducerUploadedData(InBinding);
		}
#endif

		return bPreparationDone;
	}
	else
	{
		return true;
	}
}

void UPCGDataCollectionDataProvider::ReleaseTransientResources(const TCHAR* InReason)
{
	if (DataAdaptor)
	{
#ifdef PCG_DATA_USAGE_LOGGING
		UE_LOGF(LogPCG, Warning, "%ls: Releasing resources due to %ls", *GetName(), InReason ? InReason : TEXT("NOREASON"));
#endif

		DataAdaptor.Reset();
	}
}

FComputeDataProviderRenderProxy* UPCGDataCollectionDataProvider::GetRenderProxy()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataCollectionDataProvider::GetRenderProxy);

	// Compute the union of allocated native properties across all output data items. Used as a shader
	// uniform so the preamble can branch on CVR vs N-slot without reading per-element buffer headers.
	uint32 AllocatedNativePropertiesMaskValue = 0;
	if (TSharedPtr<const FPCGDataCollectionDesc> PinDescLocal = GetPinDescription())
	{
		AllocatedNativePropertiesMaskValue = (uint32)PinDescLocal->GetAllocatedPropertiesUnion();
	}

	FPCGDataCollectionDataProviderProxy::FSetupParams ProxyParams =
	{
		.DataProvider = this,
		.PinDesc = GetPinDescription(),
		.SizeBytes = SizeBytes,
		.bIsBufferSizeValid = bIsBufferSizeValid,
		.ExportMode = GetExportMode(),
		.bZeroInitialize = bRequiresZeroInitialization,
		.ElementCountMultiplier = ElementCountMultiplier,
		.AllocatedNativePropertiesMask = AllocatedNativePropertiesMaskValue,
		.OutputPinLabel = GetOutputPinLabel(),
		.OutputPinLabelAlias = GetOutputPinLabelAlias(),
		.bProducedByCPU = IsProducedByCPU(),
		.DataAdaptor = DataAdaptor,
	};

	return new FPCGDataCollectionDataProviderProxy(ProxyParams);
}

void UPCGDataCollectionDataProvider::Reset()
{
	Super::Reset();

	SizeBytes = 0;
	bIsBufferSizeValid = false;
	ElementCountMultiplier = 0;
	bRequiresZeroInitialization = false;
	DataAdaptor.Reset();
	bBufferSizeValidated = false;
}

FPCGDataCollectionDataProviderProxy::FPCGDataCollectionDataProviderProxy(const FSetupParams& InParams)
	: SizeBytes(InParams.SizeBytes)
	, bIsBufferSizeValid(InParams.bIsBufferSizeValid)
	, ExportMode(InParams.ExportMode)
	, bZeroInitialize(InParams.bZeroInitialize)
	, ElementCountMultiplier(InParams.ElementCountMultiplier)
	, AllocatedNativePropertiesMask(InParams.AllocatedNativePropertiesMask)
	, DataProviderWeakPtr(InParams.DataProvider)
	, OutputPinLabel(InParams.OutputPinLabel)
	, OutputPinLabelAlias(InParams.OutputPinLabelAlias)
	, bProducedByCPU(InParams.bProducedByCPU)
	, DataAdaptor(InParams.DataAdaptor)
{
	OriginatingGenerationCount = DataProviderWeakPtr->GetGenerationCounter();

	if (InParams.PinDesc)
	{
		PinDesc = InParams.PinDesc;

		for (const FPCGDataDesc& DataDesc : PinDesc->GetDataDescriptions())
		{
			for (const FPCGKernelAttributeDesc& AttrDesc : DataDesc.GetAttributeDescriptions())
			{
				LargestAttributeIndex = FMath::Max(LargestAttributeIndex, PCGComputeHelpers::GetMetadataAttributeIndexFromAttributeId(AttrDesc.GetAttributeId()));
			}
		}
	}
}

bool FPCGDataCollectionDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	if (!bIsBufferSizeValid)
	{
		UE_LOGF(LogPCG, Error, "FPCGDataCollectionDataProviderProxy invalid due to invalid buffer size.");
		return false;
	}

	if (!PinDesc)
	{
		UE_LOGF(LogPCG, Error, "FPCGDataCollectionDataProviderProxy invalid due to missing pin data description.");
		return false;
	}

	if (bProducedByCPU && !DataAdaptor.IsValid())
	{
		UE_LOGF(LogPCG, Warning, "FPCGDataProviderDataCollectionProxy invalid due to null DataAdaptor (and bRequiresUpload=true).");
		return false;
	}

	return true;
}

void FPCGDataCollectionDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	check(BufferSRV);
	check(BufferUAV);

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		// Both views target the same physical buffer. Shader DCE drops whichever view the shader doesn't reference.
		Parameters.DataCollectionBufferSRV = BufferSRV;
		Parameters.DataCollectionBufferUAV = BufferUAV;
		Parameters.ElementCountMultiplier = ElementCountMultiplier;
		Parameters.AllocatedNativePropertiesMask = AllocatedNativePropertiesMask;
		Parameters.GraphToBufferAttributeId = AttributeIdRemapSRV;
		Parameters.FirstRemappedAttributeId = FirstRemappedAttributeId;
		Parameters.BufferToGraphStringKey = BufferToGraphStringKeySRV;
		Parameters.NumRemappedStringKeys = NumRemappedStringKeys;
	}
}

void FPCGDataCollectionDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataCollectionDataProviderProxy::AllocateResources);
	LLM_SCOPE_BYTAG(PCG);

	if (!bProducedByCPU)
	{
		// Initialize buffer for output from a GPU kernel.

		check(SizeBytes > 0);

		AttributeIdRemapSRV = GetAttributeIdRemapBufferSRV(GraphBuilder, FirstRemappedAttributeId);

		BufferToGraphStringKeySRV = GetBufferToGraphStringKeySRV(GraphBuilder, NumRemappedStringKeys);

		FRDGBufferDesc Desc = FRDGBufferDesc::CreateByteAddressDesc(SizeBytes);
		if (ExportMode != EPCGExportMode::NoExport)
		{
			// We don't know for sure whether buffer will be read back or not, so need to flag the possibility if the buffer will be passed downstream.
			Desc.Usage |= BUF_SourceCopy;
		}

		Buffer = GraphBuilder.CreateBuffer(Desc, TEXT("PCGDataCollection"));
		BufferSRV = GraphBuilder.CreateSRV(Buffer);
		BufferUAV = GraphBuilder.CreateUAV(Buffer);

		// Initialize with an empty data collection. The kernel may not run, for example if indirect dispatch args end up being 0. This ensures
		// there is something meaningful to readback.
		TArray<uint32> PackedDataCollection;
		PackedDataCollection.Reserve(PCGDataCollectionPackingHelpers::ComputePackedHeaderSizeBytes(PinDesc));
		PCGDataCollectionPackingHelpers::WriteHeader(PinDesc, PackedDataCollection);

		if (bZeroInitialize)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataCollectionDataProviderProxy::AllocateResources::ZeroInitialize);

			// Clear buffer to 0s before uploading header below.
			PackedDataCollection.SetNumZeroed(SizeBytes / PackedDataCollection.GetTypeSize());
		}
#if !UE_BUILD_SHIPPING
		else if (PCGSystemSwitches::CVarFuzzGPUMemory.GetValueOnAnyThread())
		{
			// Allocate rest of buffer and initialize with random numbers below.
			const int NumBefore = PackedDataCollection.Num();
			PackedDataCollection.SetNumUninitialized(SizeBytes / PackedDataCollection.GetTypeSize());

			FRandomStream Rand(GFrameCounter);
			for (int Index = NumBefore; Index < PackedDataCollection.Num(); ++Index)
			{
				PackedDataCollection[Index] = Rand.GetUnsignedInt();
			}
		}
#endif // !UE_BUILD_SHIPPING

		GraphBuilder.QueueBufferUpload(Buffer, MakeConstArrayView(PackedDataCollection));

		// If buffer needs to be exported, can do it now. Pass back a reference to the main thread where it can be picked up by the compute graph element.
		if (ExportMode != EPCGExportMode::NoExport)
		{
			const TRefCountPtr<FRDGPooledBuffer> ExportedPooledBuffer = GraphBuilder.ConvertToExternalBuffer(Buffer);
			GraphBuilder.SetBufferAccessFinal(Buffer, ERHIAccess::CopySrc);

			ExecuteOnGameThread(UE_SOURCE_LOCATION, [ExportedPooledBuffer, DataProviderWeakPtr = DataProviderWeakPtr, PinDesc = PinDesc, OutputPinLabel = OutputPinLabel, SizeBytes = SizeBytes, ExportMode = ExportMode, OutputPinLabelAlias = OutputPinLabelAlias, GenerationCount = OriginatingGenerationCount]()
			{
				LLM_SCOPE_BYTAG(PCG);

				// Obtain objects. No ensures added because a graph cancellation could feasibly destroy some or all of these.
				UPCGDataCollectionDataProvider* DataProvider = DataProviderWeakPtr.Get();
				if (!DataProvider)
				{
					UE_LOGF(LogPCG, Error, "Could not resolve UPCGDataCollectionDataProvider object to pass back buffer handle.");
					return;
				}

				if (DataProvider->GetGenerationCounter() != GenerationCount)
				{
					return;
				}

				if (!ensure(PinDesc))
				{
					return;
				}

				UPCGDataBinding* Binding = DataProvider ? DataProvider->GetDataBinding() : nullptr;
				UPCGSubsystem* Subsystem = UPCGSubsystem::GetSubsystemForCurrentWorld();

				if (Binding && Subsystem)
				{
					TSharedPtr<FPCGProxyForGPUDataCollection> DataCollectionOnGPU = MakeShared<FPCGProxyForGPUDataCollection>(ExportedPooledBuffer, SizeBytes, PinDesc, Binding->GetStringTable());

					for (int DataIndex = 0; DataIndex < PinDesc->GetDataDescriptions().Num(); ++DataIndex)
					{
						UPCGProxyForGPUData* Proxy = NewObject<UPCGProxyForGPUData>();
						Proxy->Initialize(DataCollectionOnGPU, DataIndex);

						// TODO - binding is doing a lot of work. Could store a context handle in the data provider instead?
						Binding->ReceiveDataFromGPU_GameThread(Proxy, DataProvider->GetProducerSettings(), ExportMode, OutputPinLabel, OutputPinLabelAlias, PinDesc->GetDataDescriptions()[DataIndex]);
					}

					DataProvider->OnDataExported_GameThread().Broadcast();
				}
			});
		}
	}
	else
	{
		// For the CPU->GPU upload case, obtain the buffers from the data adaptor.
		Buffer = DataAdaptor->GetBuffer_RenderThread(GraphBuilder, ExportMode);
		BufferSRV = GraphBuilder.CreateSRV(Buffer);
		BufferUAV = GraphBuilder.CreateUAV(Buffer);

		AttributeIdRemapSRV = GetAttributeIdRemapBufferSRV(GraphBuilder, FirstRemappedAttributeId);
		BufferToGraphStringKeySRV = GetBufferToGraphStringKeySRV(GraphBuilder, NumRemappedStringKeys);
	}
}

FRDGBufferSRVRef FPCGDataCollectionDataProviderProxy::GetAttributeIdRemapBufferSRV(FRDGBuilder& GraphBuilder, int32& OutFirstRemappedAttributeId) const
{
	if (!bProducedByCPU || !DataAdaptor || !DataAdaptor->UsesAttributeIdRemap())
	{
		// Default implementation. No remap required, disabled remapping.
		OutFirstRemappedAttributeId = PCGDataCollectionPackingConstants::MAX_NUM_ATTRS;

		return GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(int32))));
	}
	else
	{
		return DataAdaptor->GetAttributeRemapBufferSRV(GraphBuilder, OutFirstRemappedAttributeId);
	}
}

FRDGBufferSRVRef FPCGDataCollectionDataProviderProxy::GetBufferToGraphStringKeySRV(FRDGBuilder& GraphBuilder, int32& OutNumRemappedStringKeys) const
{
	if (!bProducedByCPU || !DataAdaptor || !DataAdaptor->UsesStringKeyRemap())
	{
		// Default implementation. Disable remapping.
		OutNumRemappedStringKeys = 0;

		return GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(int32))));
	}
	else
	{
		return DataAdaptor->GetBufferToGraphStringKeySRV(GraphBuilder, OutNumRemappedStringKeys);
	}
}

#undef LOCTEXT_NAMESPACE
