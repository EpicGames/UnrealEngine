// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "RenderGraphResources.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "PCGDataCollectionDataInterface.generated.h"

struct FPCGComputeGraphContext;
struct FPCGDataCollectionDesc;
class FPCGDataCollectionDataInterfaceParameters;
class FPCGDataUploadAdaptor;
class UPCGDataBinding;

/**
* Compute Framework Data Interface for PCG data types that support being packed into data collections, which means they are composed of:
* - One or more data items
* - One or more elements
* - Elements can have system attributes (like point transform, bounds, etc)
* - Elements can have user-added attributes
*/
UCLASS(ClassGroup = (Procedural))
class UPCGDataCollectionDataInterface : public UPCGExportableDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGDataCollection"); }
	/** Return true if the associated UComputeDataProvider holds data that can be combined into a single dispatch invocation. */
	bool CanSupportUnifiedDispatch() const override { return false; } // I think this means compute shader can produce multiple buffers simultaneously?
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	TCHAR const* GetShaderVirtualPath() const override; // TODO probably easier to just inline rather than external source?
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	void GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	/** Set whether entire buffer needs to be initialized to 0 on the GPU, if not only the header is initialized for efficiency. */
	void SetRequiresZeroInitialization(bool bInZeroInit) { bRequiresZeroInitialization = bInZeroInit; }

	/** Apply multiplier to all element counts (only used for kernel output data). */
	void SetElementCountMultiplier(uint32 InElementCountMultiplier) { ElementCountMultiplier = InElementCountMultiplier; }

public:
	/** Whether to perform full 0-initialization of the buffer. */
	UPROPERTY()
	bool bRequiresZeroInitialization = false;

	/** Multiplier applied to all element counts (only used for kernel output data). */
	UPROPERTY()
	uint32 ElementCountMultiplier = 0;

	/** Make GetSupportedInputs return the union of read and write functions, to not break legacy Custom HLSL nodes. */
	UPROPERTY()
	bool bLegacyAllowSettersOnInputs = false;

protected:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading a PCG Data Collection. */
UCLASS()
class UPCGDataCollectionDataProvider : public UPCGExportableDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	virtual void ReleaseTransientResources(const TCHAR* InReason) override;
	//~ End UPCGComputeDataProvider Interface

	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

public:
	uint32 ElementCountMultiplier = 0;

	bool bRequiresZeroInitialization = false;

protected:
	uint64 SizeBytes = 0;

	bool bIsBufferSizeValid = false;

	bool bBufferSizeValidated = false;

	TSharedPtr<FPCGDataUploadAdaptor> DataAdaptor;
};

class FPCGDataCollectionDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FSetupParams
	{
		// Weak pointer useful for passing back buffer handles. Do not access this directly from the render thread.
		TWeakObjectPtr<UPCGDataCollectionDataProvider> DataProvider;
		TSharedPtr<const FPCGDataCollectionDesc> PinDesc = nullptr;
		uint64 SizeBytes = 0;
		bool bIsBufferSizeValid = false;
		EPCGExportMode ExportMode = EPCGExportMode::NoExport;
		bool bZeroInitialize = false;
		uint32 ElementCountMultiplier = 0;
		uint32 AllocatedNativePropertiesMask = 0;
		FName OutputPinLabel;
		FName OutputPinLabelAlias;
		bool bProducedByCPU = false;
		TSharedPtr<FPCGDataUploadAdaptor> DataAdaptor;
	};

	FPCGDataCollectionDataProviderProxy(const FSetupParams& InParams);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	FRDGBufferSRVRef GetAttributeIdRemapBufferSRV(FRDGBuilder& GraphBuilder, int32& OutFirstRemappedAttributeId) const;

	FRDGBufferSRVRef GetBufferToGraphStringKeySRV(FRDGBuilder& GraphBuilder, int32& OutNumRemappedStringKeys) const;

public:
	/** Generation count of the data provider when the proxy was created. */
	uint64 OriginatingGenerationCount = 0;

protected:
	uint64 SizeBytes = 0;

	bool bIsBufferSizeValid = false;

	EPCGExportMode ExportMode = EPCGExportMode::NoExport;

	bool bZeroInitialize = false;

	uint32 ElementCountMultiplier = 0;

	/** Union of EPCGPointNativeProperties bits across all output data items: which native properties are N-slot (fully allocated) vs CVR. Set from the pin description at proxy creation time. */
	uint32 AllocatedNativePropertiesMask = 0;

	using FParameters = FPCGDataCollectionDataInterfaceParameters;

	TSharedPtr<const FPCGDataCollectionDesc> PinDesc = nullptr;

	FRDGBufferRef Buffer = nullptr;
	FRDGBufferSRVRef BufferSRV = nullptr;
	FRDGBufferUAVRef BufferUAV = nullptr;

	// Weak pointer useful for passing back buffer handles. Do not access this directly from the render thread.
	TWeakObjectPtr<UPCGDataCollectionDataProvider> DataProviderWeakPtr;

	FName OutputPinLabel;
	FName OutputPinLabelAlias;

	FRDGBufferSRVRef AttributeIdRemapSRV = nullptr;

	int32 FirstRemappedAttributeId = INDEX_NONE;

	FRDGBufferSRVRef BufferToGraphStringKeySRV = nullptr;

	int32 NumRemappedStringKeys = INDEX_NONE;

	int32 LargestAttributeIndex = INDEX_NONE;

	bool bProducedByCPU = false;

	TSharedPtr<FPCGDataUploadAdaptor> DataAdaptor;
};
