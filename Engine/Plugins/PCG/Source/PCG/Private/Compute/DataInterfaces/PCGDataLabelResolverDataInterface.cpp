// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/DataInterfaces/PCGDataLabelResolverDataInterface.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGSettings.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGComputeKernel.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SystemTextures.h"
#include "Algo/AnyOf.h"
#include "ComputeFramework/ComputeMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataLabelResolverDataInterface)

void UPCGDataLabelResolverDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetDataIndexFromIdInternal"))
		.AddReturnType(EShaderFundamentalType::Uint) // Index of data in input data collection
		.AddParam(EShaderFundamentalType::Uint); // InDataId
}

void UPCGDataLabelResolverDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	// Usage symmetrical across kernel inputs and kernel outputs.
	GetSupportedInputs(OutFunctions);
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGDataLabelResolverDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, InNumData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint32>, InDataIdToDataIndexMap)
END_SHADER_PARAMETER_STRUCT()

void UPCGDataLabelResolverDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGDataLabelResolverDataInterfaceParameters>(UID);
}

void UPCGDataLabelResolverDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	OutHLSL += FString::Format(TEXT(
		"uint {DataInterfaceName}_InNumData;\n"
		"StructuredBuffer<int> {DataInterfaceName}_InDataIdToDataIndexMap;\n"
		"\n"
		"uint GetDataIndexFromIdInternal_{DataInterfaceName}(uint InDataId)\n"
		"{\n"
		"	if (InDataId >= {DataInterfaceName}_InNumData)\n"
		"	{\n"
		"		return 0;\n"
		"	}\n"
		"\n"
		"	// Data index could be -1 if the label was not resolved, so fall-back to data index 0 if necessary.\n"
		"	return (uint)max({DataInterfaceName}_InDataIdToDataIndexMap[InDataId], 0);\n"
		"}\n"
		"\n"),
		TemplateArgs);
}

UComputeDataProvider* UPCGDataLabelResolverDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGDataLabelResolverDataProvider>();
}

void UPCGDataLabelResolverDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataLabelResolverDataProvider::Initialize);

	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGDataLabelResolverDataInterface* DataInterface = CastChecked<UPCGDataLabelResolverDataInterface>(InDataInterface);

	Kernel = DataInterface->Kernel;
	PinLabel = DataInterface->PinLabel;
	bIsInput = DataInterface->bIsInput;
}

bool UPCGDataLabelResolverDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataLabelResolverDataProvider::PrepareForExecute_GameThread);
	check(InBinding);
	check(Kernel);

	const UPCGComputeGraph* ComputeGraph = InBinding->GetComputeGraph();
	check(ComputeGraph);

	const TArray<FString>& StringTable = InBinding->GetStringTable();

	const TSharedPtr<const FPCGDataCollectionDesc> PinDataDesc = InBinding->GetCachedKernelPinDataDesc(Kernel, PinLabel, bIsInput);

	if (!ensure(PinDataDesc))
	{
		return true;
	}

	const FPCGPinDataLabels* FoundPinDataLabels = ComputeGraph->GetStaticDataLabelsTable().Find(Kernel->GetKernelIndex());
	if (const FPCGDataLabels* FoundLabels = FoundPinDataLabels ? FoundPinDataLabels->PinToDataLabels.Find(PinLabel) : nullptr)
	{
		DataIdToDataIndexMap.SetNumUninitialized(FoundLabels->Labels.Num());

		for (int32 LabelIndex = 0; LabelIndex < FoundLabels->Labels.Num(); ++LabelIndex)
		{
			const FString& Label = FoundLabels->Labels[LabelIndex];
			const FString PrefixedLabel = PCGComputeHelpers::GetPrefixedDataLabel(Label);

			// Initialize with invalid indices. The data IDs are not ordered.
			DataIdToDataIndexMap[LabelIndex] = INDEX_NONE;

			for (int32 InputDataIndex = 0; InputDataIndex < PinDataDesc->GetDataDescriptions().Num(); ++InputDataIndex)
			{
				for (const int32 TagStringKey : PinDataDesc->GetDataDescriptions()[InputDataIndex].GetTagStringKeys())
				{
					if (StringTable[TagStringKey] == PrefixedLabel)
					{
						DataIdToDataIndexMap[LabelIndex] = InputDataIndex;
						break;
					}
				}
			}

			if (DataIdToDataIndexMap[LabelIndex] == INDEX_NONE)
			{
				if (TSharedPtr<FPCGContextHandle> Context = InBinding->GetContextHandle().Pin())
				{
					PCG_KERNEL_VALIDATION_ERR(Context->GetContext(), Kernel->GetSettings(), FText::Format(
						NSLOCTEXT("PCGDataLabelResolver", "LabelDoesNotExist", "Data label '{0}' was not present in the tags on the incoming data."),
						FText::FromString(Label)));
				}
			}
		}
	}

	return true;
}

FComputeDataProviderRenderProxy* UPCGDataLabelResolverDataProvider::GetRenderProxy()
{
	return new FPCGDataLabelResolverDataProviderProxy(DataIdToDataIndexMap);
}

void UPCGDataLabelResolverDataProvider::Reset()
{
	Kernel = nullptr;
	PinLabel = NAME_None;
	bIsInput = false;
	DataIdToDataIndexMap.Empty();

	Super::Reset();
}

bool FPCGDataLabelResolverDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters) && !Algo::AnyOf(DataIdToDataIndexMap, [](int32 InDataIndex) { return InDataIndex < 0; });
}

void FPCGDataLabelResolverDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < InDispatchData.NumInvocations; ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.InNumData = DataIdToDataIndexMap.Num();
		Parameters.InDataIdToDataIndexMap = DataIdToDataIndexBufferSRV;
	}
}

void FPCGDataLabelResolverDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	LLM_SCOPE_BYTAG(PCG);

	if (!DataIdToDataIndexMap.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(DataIdToDataIndexMap.GetTypeSize(), DataIdToDataIndexMap.Num());

		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGDataLabelResolver_DataIdToDataIndexMap"));
		DataIdToDataIndexBufferSRV = GraphBuilder.CreateSRV(Buffer);

		GraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(DataIdToDataIndexMap));
	}
	else
	{
		DataIdToDataIndexBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, DataIdToDataIndexMap.GetTypeSize())));
	}
}

