// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "ComputeFramework/ComputeDataInterface.h"

#include "Compute/PCGDataBinding.h"

#include "PCGDataLabelResolverDataInterface.generated.h"

#define UE_API PCG_API

class FPCGDataLabelResolverDataInterfaceParameters;

/** Data interface for mapping from data label to data index. */
UCLASS(MinimalAPI, ClassGroup = (Procedural))
class UPCGDataLabelResolverDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGDataLabelResolver"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	/** Label of the pin being resolved. */
	UPROPERTY()
	FName PinLabel;

	/** Whether the pin is an input or output pin. */
	UPROPERTY()
	bool bIsInput = false;

	/** Kernel that owns the pin being resolved. */
	UPROPERTY()
	TObjectPtr<const UPCGComputeKernel> Kernel = nullptr;
};

UCLASS()
class UPCGDataLabelResolverDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

public:
	UPROPERTY()
	TObjectPtr<const UPCGComputeKernel> Kernel;

	/** Label of the pin being resolved. */
	UPROPERTY()
	FName PinLabel;

	/** Whether the pin is an input or output pin. */
	UPROPERTY()
	bool bIsInput = false;

	/** Map from data ID to data index for accessing the input data collection. */
	TArray<int32> DataIdToDataIndexMap;
};

class FPCGDataLabelResolverDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGDataLabelResolverDataProviderProxy(const TArray<int32>& InDataIdToDataIndexMap)
		: DataIdToDataIndexMap(InDataIdToDataIndexMap)
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGDataLabelResolverDataInterfaceParameters;

	TArray<int32> DataIdToDataIndexMap;
	FRDGBufferSRVRef DataIdToDataIndexBufferSRV = nullptr;
};

#undef UE_API
