// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/DataInterfaces/PCGDebugDataInterface.h"

#include "PCGModule.h"
#include "PCGSettings.h"
#include "Compute/PCGComputeKernel.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ComputeMetadataBuilder.h"
#include "ComputeFramework/ShaderParameterMetadataAllocation.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDebugDataInterface)

void UPCGDebugDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName("WriteDebugValue")
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float);
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGDebugDataInterfaceParameters, )
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, DebugBuffer)
END_SHADER_PARAMETER_STRUCT()

void UPCGDebugDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGDebugDataInterfaceParameters>(UID);
}

void UPCGDebugDataInterface::GetShaderHash(FString& InOutKey) const
{
	// UComputeGraph::BuildKernelSource hashes the result of GetHLSL()
	// Only append additional hashes here if the HLSL contains any additional includes	
}

void UPCGDebugDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	const FString WriteDebugValueImpl = (DebugBufferSizeFloats > 0)
		? FString::Format(TEXT(
			"\n{\n"
			"	if (Index >= 0 && Index < {0})\n"
			"	{\n"
			"		{1}_DebugBuffer.Store(Index * 4, asuint(Value));\n"
			"	}\n"
			"}\n"),
			{ DebugBufferSizeFloats, *InDataInterfaceName })
		: TEXT("{ /* No-Op */ }\n");

	OutHLSL += FString::Format(TEXT(
		"RWByteAddressBuffer {0}_DebugBuffer;\n"
		"void WriteDebugValue_{0}(uint Index, float Value) {1}"),
		{ *InDataInterfaceName, WriteDebugValueImpl });
}

bool UPCGDebugDataInterface::GetRequiresReadback() const
{
	return DebugBufferSizeFloats > 0;
}

UComputeDataProvider* UPCGDebugDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGDebugDataProvider>();
}

void UPCGDebugDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDebugDataProvider::Initialize);

	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGDebugDataInterface* DataInterface = CastChecked<UPCGDebugDataInterface>(InDataInterface);

	DebugBufferSizeFloats = DataInterface->DebugBufferSizeFloats;
}

FComputeDataProviderRenderProxy* UPCGDebugDataProvider::GetRenderProxy()
{
	const UPCGSettings* Settings = GetProducerSettings();
	const FString SettingsName = Settings ? Settings->GetName() : TEXT("MISSINGSETTINGS");
	const FString KernelName = GetProducerKernel() ? GetProducerKernel()->GetName() : TEXT("MISSINGKERNEL");

	auto ProcessReadbackData_RenderThread = [SettingsName=SettingsName, KernelName=KernelName](const void* InData, int InNumBytes)
	{
		if (!InData)
		{
			return;
		}

		LLM_SCOPE_BYTAG(PCG);

		const float* DataAsFloat = static_cast<const float*>(InData);

		UE_LOGF(LogPCG, Warning, "%ls - %ls", *SettingsName, *KernelName);

		for (int Index = 0; Index < InNumBytes / sizeof(float); ++Index)
		{
			UE_LOGF(LogPCG, Warning, "\t%d:\t%f", Index, DataAsFloat[Index]);
		}
	};

	return new FPCGDebugDataProviderProxy(DebugBufferSizeFloats, MoveTemp(ProcessReadbackData_RenderThread));
}

void UPCGDebugDataProvider::Reset()
{
	DebugBufferSizeFloats = 0;

	Super::Reset();
}

bool FPCGDebugDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGDebugDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < InDispatchData.NumInvocations; ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.DebugBuffer = DebugBufferUAV;
	}
}

void FPCGDebugDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	LLM_SCOPE_BYTAG(PCG);

	const uint32 NumFloats = FMath::Max(DebugBufferSizeFloats, 1u);

	FRDGBufferDesc Desc = FRDGBufferDesc::CreateByteAddressDesc(sizeof(float) * NumFloats);
	Desc.Usage |= BUF_SourceCopy;

	DebugBuffer = GraphBuilder.CreateBuffer(Desc, TEXT("PCGDebugBuffer"));
	DebugBufferUAV = GraphBuilder.CreateUAV(DebugBuffer);

	TArray<float> ZeroInitializedBuffer;
	ZeroInitializedBuffer.SetNumZeroed(NumFloats);

	GraphBuilder.QueueBufferUpload(DebugBuffer, MakeArrayView(ZeroInitializedBuffer));
}

void FPCGDebugDataProviderProxy::GetReadbackData(TArray<FReadbackData>& OutReadbackData) const
{
	FReadbackData Data;
	Data.Buffer = DebugBuffer;
	Data.NumBytes = DebugBufferSizeFloats * sizeof(float);
	Data.ReadbackCallback_RenderThread = &AsyncReadbackCallback_RenderThread;

	OutReadbackData.Add(MoveTemp(Data));
}
