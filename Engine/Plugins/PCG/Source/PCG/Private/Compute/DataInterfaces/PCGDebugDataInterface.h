// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "PCGDebugDataInterface.generated.h"

#define UE_API PCG_API

class FPCGDebugDataInterfaceParameters;
class FRDGBuffer;
class FRDGBufferUAV;

/** Interface for data about the kernel debug, such as debug value buffer. */
UCLASS(MinimalAPI, ClassGroup = (Procedural))
class UPCGDebugDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGKernelDebug"); }
	UE_API void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API bool GetRequiresReadback() const override;
	UE_API UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	void SetDebugBufferSize(uint32 InDebugBufferSizeFloats) { DebugBufferSizeFloats = InDebugBufferSizeFloats; }

public:
	UPROPERTY()
	uint32 DebugBufferSizeFloats = 0;
};

/** Compute Framework Data Provider for each custom compute kernel. */
UCLASS()
class UPCGDebugDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

public:
	uint32 DebugBufferSizeFloats = 0;
};

class FPCGDebugDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGDebugDataProviderProxy(uint32 InDebugBufferSizeFloats, FReadbackCallback InAsyncReadbackCallback_RenderThread)
		: DebugBufferSizeFloats(InDebugBufferSizeFloats)
		, AsyncReadbackCallback_RenderThread(InAsyncReadbackCallback_RenderThread)
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GetReadbackData(TArray<FReadbackData>& OutReadbackData) const override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGDebugDataInterfaceParameters;

	FRDGBufferRef DebugBuffer = nullptr;
	FRDGBufferUAVRef DebugBufferUAV = nullptr;
	uint32 DebugBufferSizeFloats = 0;
	FReadbackCallback AsyncReadbackCallback_RenderThread;
};

#undef UE_API
