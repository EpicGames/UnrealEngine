// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGExportToRayTracingDataInterface.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGRayTrace.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/PCGDataBinding.h"
#include "Elements/PCGWorldRaycast.h"

#include "ComputeWorkerInterface.h"
#include "RenderGraphBuilder.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ShaderParameterStruct.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGExportToRayTracingDataInterface)

void UPCGExportToRayTracingDataInterface::GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const
{
	Super::GetDefines(OutDefinitionSet);

	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_RAY_CULLED"), FString::FromInt(PCGRaytraceConstants::RAY_CULLED)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_RAY_TRACE_PACKED_BUFFER_STRIDE_UINTS"), FString::FromInt(PCGRaytraceConstants::RAY_TRACE_PACKED_BUFFER_STRIDE_UINTS)));
}

UComputeDataProvider* UPCGExportToRayTracingDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGExportToRayTracingDataProvider>();
}

FComputeDataProviderRenderProxy* UPCGExportToRayTracingDataProvider::GetRenderProxy()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGExportToRayTracingDataProvider::GetRenderProxy);

	FPCGRawBufferDataProviderProxy::FParams RawBufferParams;
	RawBufferParams.SizeBytes = SizeBytes;

	FPCGExportToRayTracingDataProviderProxy::FParams Params;
	Params.NumRays = NumRays;
	Params.TexCoordsChannelIndex = TexCoordsChannelIndex;

	return new FPCGExportToRayTracingDataProviderProxy(RawBufferParams, Params);
}

void UPCGExportToRayTracingDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGExportToRayTracingDataInterface* DataInterface = CastChecked<UPCGExportToRayTracingDataInterface>(InDataInterface);
	InputPointsPinLabel = DataInterface->GetInputPointsPinLabel();
}

void UPCGExportToRayTracingDataProvider::Reset()
{
	InputPointsPinLabel = NAME_None;
	NumRays = INDEX_NONE;
	TexCoordsChannelIndex = INDEX_NONE;

	Super::Reset();
}

bool UPCGExportToRayTracingDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	const TSharedPtr<const FPCGDataCollectionDesc> Desc = InBinding->GetCachedKernelPinDataDesc(GetProducerKernel(), InputPointsPinLabel, /*bInIsInput=*/true);

	if (!ensure(Desc))
	{
		return true;
	}

	NumRays = Desc->ComputeTotalElementCount();

	const UPCGWorldRaycastElementSettings* Settings = CastChecked<UPCGWorldRaycastElementSettings>(GetProducerKernel()->GetSettings());
	TexCoordsChannelIndex = Settings->bGetTextureCoordinates ? Settings->TextureCoordinatesChannelIndex : INDEX_NONE;

	return true;
}

void FPCGExportToRayTracingDataProviderProxy::PostSubmit(FComputeContext& Context) const
{
#if RHI_RAYTRACING
	FPCGRayTraceParams RayTraceParams;
	RayTraceParams.Scene = Context.Scene;
	RayTraceParams.View = Context.View;
	RayTraceParams.NumRays = Params.NumRays;
	RayTraceParams.bNeedsUVData = Params.TexCoordsChannelIndex != INDEX_NONE;
	RayTraceParams.TexCoordsChannelIndex = Params.TexCoordsChannelIndex;
	RayTraceParams.PackedDataUAV = DataUAV;
	PCGRayTrace::RenderPCGRayTraceInline(Context.GraphBuilder, RayTraceParams);
#else
	UE_LOGF(LogPCG, Error, "Failed to submit inline raytrace shader. Requires RHI ray tracing support.");
#endif
}
