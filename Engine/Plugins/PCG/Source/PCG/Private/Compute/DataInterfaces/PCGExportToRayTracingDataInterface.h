// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGRawBufferDataInterface.h"

#include "PCGExportToRayTracingDataInterface.generated.h"

class FSceneInterface;

/** Represents a buffer of data packed from points for tracing rays against the scene. Dispatches an inlined raytrace in PostSubmit. */
UCLASS(ClassGroup = (Procedural))
class UPCGExportToRayTracingDataInterface : public UPCGRawBufferDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	void GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const override;
	bool GetRequiresPostSubmitCall() const override { return true; }
	virtual UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	FName GetInputPointsPinLabel() const { return InputPointsPinLabel; }
	void SetInputPointsPinLabel(FName InInputPointsPinLabel) { InputPointsPinLabel = InInputPointsPinLabel; }

protected:
	UPROPERTY()
	FName InputPointsPinLabel;
};

UCLASS()
class UPCGExportToRayTracingDataProvider : public UPCGRawBufferDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	//~ Begin UPCGComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	virtual void Reset() override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

protected:
	FName InputPointsPinLabel = NAME_None;
	int32 NumRays = INDEX_NONE;
	int32 TexCoordsChannelIndex = INDEX_NONE;
};

class FPCGExportToRayTracingDataProviderProxy : public FPCGRawBufferDataProviderProxy
{
public:
	struct FParams
	{
		int32 NumRays = INDEX_NONE;
		int32 TexCoordsChannelIndex = INDEX_NONE;
	};

	FPCGExportToRayTracingDataProviderProxy(const FPCGRawBufferDataProviderProxy::FParams& InRawBufferParams, const FParams& InParams)
		: FPCGRawBufferDataProviderProxy(InRawBufferParams)
		, Params(InParams)
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	void PostSubmit(FComputeContext& Context) const override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	FParams Params;
};
