// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGInstanceDataInterface.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGSettings.h"
#include "PCGSceneWriterCS.h"
#include "Components/PCGProceduralISMComponent.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PrimitiveFactories/IPCGRuntimePrimitiveFactory.h"

#include "ComputeWorkerInterface.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "GPUSceneWriter.h"
#include "InstanceDataSceneProxy.h"
#include "InstancedSkinnedMeshSceneProxyDesc.h"
#include "InstancedStaticMeshSceneProxyDesc.h"
#include "PrimitiveSceneDesc.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "RenderCaptureInterface.h"
#include "RendererUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ShaderParameterStruct.h"
#include "Algo/AnyOf.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Containers/Ticker.h"
#include "Engine/Texture.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGInstanceDataInterface)

#define PCG_INSTANCE_DATA_LOGGING 0

namespace PCGInstanceDataInterface
{
#if !UE_BUILD_SHIPPING
	static int32 TriggerGPUCaptureDispatchIndex = 0;
	static FAutoConsoleVariableRef CVarTriggerGPUCaptureDispatchIndex(
		TEXT("pcg.GPU.TriggerRenderCaptures.InstanceSceneWriter"),
		TriggerGPUCaptureDispatchIndex,
		TEXT("Index of the next dispatch to capture. I.e. if set to 2, will ignore one dispatch and then trigger a capture on the next one."),
		ECVF_RenderThreadSafe
	);
#endif // !UE_BUILD_SHIPPING
}

void UPCGInstanceDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("InstanceData_GetNumPrimitives"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("InstanceData_GetNumInstancesAllPrimitives"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("InstanceData_GetIndexToWriteTo"))
		.AddReturnType(EShaderFundamentalType::Int)
		.AddParam(EShaderFundamentalType::Uint) // InPrimitiveIndex
		.AddParam(EShaderFundamentalType::Uint); // InCullingCellId

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("InstanceData_WriteInstanceLocalToWorld"))
		.AddParam(EShaderFundamentalType::Uint) // InInstanceIndex
		.AddParam(EShaderFundamentalType::Float, 4, 4);  // InTransform

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("InstanceData_WriteCustomFloat"))
		.AddParam(EShaderFundamentalType::Uint) // InInstanceIndex
		.AddParam(EShaderFundamentalType::Uint) // InCustomFloatIndex
		.AddParam(EShaderFundamentalType::Float); // InValue

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("InstanceData_WriteCustomFloatRaw"))
		.AddParam(EShaderFundamentalType::Uint) // InInstanceIndex
		.AddParam(EShaderFundamentalType::Uint) // InCustomFloatIndex
		.AddParam(EShaderFundamentalType::Uint); // InValueAsUint
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGInstanceDataInterfaceParameters,)
	SHADER_PARAMETER(uint32, NumPrimitives)
	SHADER_PARAMETER(uint32, NumInstancesAllPrimitives)
	SHADER_PARAMETER(uint32, NumCustomFloatsPerInstance)
	SHADER_PARAMETER(int32, NumCullingCellsTotal)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FUintVector2>, PrimitiveParameters)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FUintVector4>, InstanceData)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint32>, InstanceCustomFloatData)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint32>, WriteCounters)
END_SHADER_PARAMETER_STRUCT()

void UPCGInstanceDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGInstanceDataInterfaceParameters>(UID);
}

TCHAR const* UPCGInstanceDataInterface::TemplateFilePath = TEXT("/Plugin/PCG/Private/PCGInstanceDataInterface.ush");

TCHAR const* UPCGInstanceDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UPCGInstanceDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UPCGInstanceDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	if (ensure(LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr)))
	{
		OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
	}
}

void UPCGInstanceDataInterface::GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const
{
	Super::GetDefines(OutDefinitionSet);

	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_MAX_PRIMITIVES"), FString::FromInt(PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER)));
}

UComputeDataProvider* UPCGInstanceDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGInstanceDataProvider>();
}

void UPCGInstanceDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGInstanceDataInterface* InstanceDI = CastChecked<UPCGInstanceDataInterface>(InDataInterface);

	const FIntVector NumCullingCells = CastChecked<UPCGDataBinding>(InBinding)->GetMaxNumCullingCells(InstanceDI->GetCullingCellExtent());
	NumCullingCellsTotal = NumCullingCells.X * NumCullingCells.Y * NumCullingCells.Z;
}

bool UPCGInstanceDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGInstanceDataProvider::PrepareForExecute_GameThread);
	UPCGDataBinding* Binding = CastChecked<UPCGDataBinding>(InBinding);

	if (!Binding->IsMeshSpawnerKernelComplete(GetProducerKernel()))
	{
		// The static mesh data interface(s) set this up, so wait until it is ready.
		return false;
	}

	FPCGSpawnerPrimitives* FoundPrimitives = Binding->FindMeshSpawnerPrimitives(GetProducerKernel());
	if (!FoundPrimitives)
	{
		return true;
	}

	PrimitiveFactory = FoundPrimitives->PrimitiveFactory;
	if (!PrimitiveFactory)
	{
		// Factory can be null if no primitives or no instances.
		return true;
	}

	// The factories enforce this limit, to simplify the logic here.
	if (!ensure(PrimitiveFactory->GetNumPrimitives() <= PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER))
	{
		UE_LOGF(LogPCG, Error, "FPCGInstanceDataProvider: %d primitives provided which is more than the maximum limit (%d), instances will not be created.",
			PrimitiveFactory->GetNumPrimitives(), PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER);
		return true;
	}

	if (!PrimitiveFactory->IsRenderStateCreated())
	{
		UE_LOGF(LogPCG, Verbose, "FPCGInstanceDataProvider: One or more scene proxies were not ready. Will try again on the next tick.");
		return false;
	}

	NumInstancesAllPrimitives = 0;

	for (int32 PrimitiveIndex = 0; PrimitiveIndex < PrimitiveFactory->GetNumPrimitives(); ++PrimitiveIndex)
	{
		NumInstancesAllPrimitives += PrimitiveFactory->GetNumInstancesTotal(PrimitiveIndex);
	}

	NumCustomFloatsPerInstance = FoundPrimitives->NumCustomFloats;

	ContextHandle = InBinding->GetContextHandle();

	return true;
}

bool UPCGInstanceDataProvider::PostExecute(UPCGDataBinding* InBinding)
{
	if (!Super::PostExecute(InBinding))
	{
		return false;
	}

	return bWroteInstances;
}

FComputeDataProviderRenderProxy* UPCGInstanceDataProvider::GetRenderProxy()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGInstanceDataProvider::GetRenderProxy);

	const UPCGSettings* Settings = GetProducerKernel() ? GetProducerKernel()->GetSettings() : nullptr;

	FPCGInstanceDataProviderProxy::FParams Params =
	{
		.PrimitiveFactory = PrimitiveFactory,
		.NumInstancesAllPrimitives = NumInstancesAllPrimitives,
		.NumCustomFloatsPerInstance = NumCustomFloatsPerInstance,
		.NumCullingCellsTotal = NumCullingCellsTotal,
		.Seed = Settings ? static_cast<uint32>(Settings->GetSeed()) : 42,
		.DataProvider = MakeWeakObjectPtr(this),
		.ContextHandle = ContextHandle,
	};

	return new FPCGInstanceDataProviderProxy(Params);
}

void UPCGInstanceDataProvider::Reset()
{
	Super::Reset();

	PrimitiveFactory.Reset();
	NumInstancesAllPrimitives = 0;
	NumCustomFloatsPerInstance = 0;
	NumCullingCellsTotal = 1;
	bWroteInstances = false;
	ContextHandle.Reset();
}

FPCGInstanceDataProviderProxy::FPCGInstanceDataProviderProxy(const FParams& InParams)
	: PrimitiveFactory(InParams.PrimitiveFactory)
	, NumInstancesAllPrimitives(InParams.NumInstancesAllPrimitives)
	, NumCustomFloatsPerInstance(InParams.NumCustomFloatsPerInstance)
	, NumCullingCellsTotal(InParams.NumCullingCellsTotal)
	, Seed(InParams.Seed)
	, DataProvider(InParams.DataProvider)
	, ContextHandle(InParams.ContextHandle)
{
	bIsValid = true;

	if (ensure(DataProvider.IsValid()))
	{
		DataProviderGeneration = DataProvider->GetGenerationCounter();

		if (NumInstancesAllPrimitives == 0)
		{
			DataProvider->bWroteInstances = true;
		}
	}
	else
	{
		UE_LOGF(LogPCG, Warning, "FPCGInstanceDataProviderProxy: Data provider missing, proxy is invalid, compute graph will not execute.");
		bIsValid = false;
	}

	if (PrimitiveFactory)
	{
		for (int32 Index = 0; Index < PrimitiveFactory->GetNumPrimitives(); ++Index)
		{
			if (!PrimitiveFactory->GetSceneProxy(Index))
			{
				UE_LOGF(LogPCG, Warning, "FPCGInstanceDataProviderProxy: One or more component proxies were invalid, compute graph will not execute.");
				bIsValid = false;
				break;
			}
		}
	}
}

bool FPCGInstanceDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return bIsValid && InValidationData.ParameterStructSize == sizeof(FParameters) && NumCullingCellsTotal > 0;
}

void FPCGInstanceDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	LLM_SCOPE_BYTAG(PCG);

	const uint32 StrideUint4s = 3;
	InstanceData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FUintVector4), FMath::Max(1u, NumInstancesAllPrimitives) * StrideUint4s), TEXT("PCGInstanceDataBuffer"));
	InstanceDataUAV = GraphBuilder.CreateUAV(InstanceData);

	const uint32 CustomFloatsRequired = FMath::Max(1u, NumInstancesAllPrimitives * NumCustomFloatsPerInstance);
	InstanceCustomFloatData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), CustomFloatsRequired), TEXT("PCGInstanceCustomFloatDataBuffer"));
	InstanceCustomFloatDataUAV = GraphBuilder.CreateUAV(InstanceCustomFloatData);

	const int32 NumPrimitiveCells = FMath::Max(1u, PrimitiveFactory ? PrimitiveFactory->GetNumPrimitives() : 0u) * NumCullingCellsTotal;

	WriteCounters = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NumPrimitiveCells), TEXT("PCGWriteCounters"));
	WriteCountersUAV = GraphBuilder.CreateUAV(WriteCounters);

	TArray<uint32, TInlineAllocator<128>> Zeroes;
	Zeroes.SetNumZeroed(NumPrimitiveCells);
	GraphBuilder.QueueBufferUpload(WriteCounters, MakeConstArrayView(Zeroes));

	FRDGBufferRef PrimitiveParameters = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FUintVector2), NumPrimitiveCells), TEXT("PCGPrimitiveParameters"));
	PrimitiveParametersSRV = GraphBuilder.CreateSRV(PrimitiveParameters);

	if (PrimitiveFactory && PrimitiveFactory->GetNumPrimitives() > 0)
	{
		TArray<FUintVector2, TInlineAllocator<128>> PrimitiveParams;
		uint32 CumulativeInstanceCount = 0;

		for (int32 PrimitiveIndex = 0; PrimitiveIndex < PrimitiveFactory->GetNumPrimitives(); ++PrimitiveIndex)
		{
			for (int32 CellID = 0; CellID < NumCullingCellsTotal; ++CellID)
			{
				// This is the full map (not just populated cells) - for each incoming instance the kernel will look up an instance offset from this array.
				const uint32 InstanceCount = PrimitiveFactory->GetNumInstances(PrimitiveIndex, CellID);

				PrimitiveParams.Add(FUintVector2(InstanceCount, CumulativeInstanceCount));

				CumulativeInstanceCount += InstanceCount;
			}
		}

		GraphBuilder.QueueBufferUpload(PrimitiveParameters, MakeConstArrayView(PrimitiveParams));
	}
	else
	{
		GraphBuilder.QueueBufferUpload(PrimitiveParameters, MakeConstArrayView(&FUintVector2::ZeroValue, 1));
	}
}

void FPCGInstanceDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumPrimitives = PrimitiveFactory ? PrimitiveFactory->GetNumPrimitives() : 0;
		Parameters.NumInstancesAllPrimitives = NumInstancesAllPrimitives;
		Parameters.NumCustomFloatsPerInstance = NumCustomFloatsPerInstance;
		Parameters.InstanceData = InstanceDataUAV;
		Parameters.InstanceCustomFloatData = InstanceCustomFloatDataUAV;
		Parameters.WriteCounters = WriteCountersUAV;
		Parameters.PrimitiveParameters = PrimitiveParametersSRV;
		Parameters.NumCullingCellsTotal = NumCullingCellsTotal;
	}
}

void FPCGInstanceDataProviderProxy::PostSubmit(FComputeContext& Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGInstanceDataProviderProxy::PostSubmit);

	if (NumInstancesAllPrimitives == 0)
	{
		return;
	}

	const TRefCountPtr<FRDGPooledBuffer> InstanceDataExported = Context.GraphBuilder.ConvertToExternalBuffer(InstanceData);
	const TRefCountPtr<FRDGPooledBuffer> InstanceCustomFloatDataExported = Context.GraphBuilder.ConvertToExternalBuffer(InstanceCustomFloatData);
	const TRefCountPtr<FRDGPooledBuffer> WriteCountersExported = Context.GraphBuilder.ConvertToExternalBuffer(WriteCounters);
	
	FTSTicker::GetCoreTicker().AddTicker(TEXT("ApplyPrimitiveSceneUpdates"), /*InDelay=*/0.0f, [DataProviderWeak=DataProvider, DataProviderGeneration=DataProviderGeneration, InstanceDataExported, InstanceCustomFloatDataExported, WriteCountersExported, ContextHandle=ContextHandle, Seed=Seed, NumCullingCellsTotal=NumCullingCellsTotal](float)
	{
		UPCGInstanceDataProvider* DataProviderInner = DataProviderWeak.Get();
		if (!DataProviderInner || DataProviderInner->GetGenerationCounter() != DataProviderGeneration)
		{
			UE_LOGF(LogPCG, Verbose, "Data provider object lost, GPU instancing will fail.");
			return false;
		}

		// All instance data is stored in a single buffer, so this is used to give the scene writer an index to the first primitive.
		int32 CumulativeInstanceCount = 0;

		FSceneInterface* Scene = nullptr;

		if (TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin())
		{
			if (FPCGContext* ContextPtr = SharedHandle->GetContext())
			{
				if (UWorld* World = ContextPtr->ExecutionSource.IsValid() ? ContextPtr->ExecutionSource->GetExecutionState().GetWorld() : nullptr)
				{
					Scene = World->Scene;
				}
			}
		}

		if (!Scene)
		{
			UE_LOGF(LogPCG, Error, "Missing scene encountered during instancing, should not happen.");
			return false;
		}

		TSharedPtr<IPCGRuntimePrimitiveFactory> Factory = DataProviderInner->PrimitiveFactory;

		// Defer if any primitive is awaiting render state recreate - writes to the doomed scene proxy's GPU slot get orphaned.
		if (Factory->IsAnyRenderStateDirty())
		{
			UE_LOGF(LogPCG, Verbose, "FPCGInstanceDataProvider: deferring instance write, primitive render state is dirty, awaiting recreate.");
			return true;
		}

		for (int32 PrimitiveIndex = 0; PrimitiveIndex < Factory->GetNumPrimitives(); ++PrimitiveIndex)
		{
			const int32 PrimitiveNumInstances = Factory->GetNumInstancesTotal(PrimitiveIndex);

			if (PrimitiveNumInstances <= 0)
			{
				UE_LOGF(LogPCG, Warning, "Primitive with 0 instances encountered during instancing, should not happen.");
				continue;
			}

			FPrimitiveSceneProxy* SceneProxy = Factory->GetSceneProxy(PrimitiveIndex);
			if (!SceneProxy)
			{
				if (!Factory->IsPrimitiveValid(PrimitiveIndex))
				{
					UE_LOGF(LogPCG, Verbose, "Primitive component was destroyed, aborting instance write.");
					return false;
				}

				UE_LOGF(LogPCG, Verbose, "Scene proxy not ready, retrying next frame.");
				return true;
			}

			// todo_pcg: This is technically a race with the RT and we should try remove these conditions and get to the bottom of why it was needed.
			if (!ensure(SceneProxy->GetPrimitiveSceneInfo()->GetInstanceSceneDataOffset() >= 0))
			{
				UE_LOGF(LogPCG, Verbose, "Primitive instances not allocated in GPU scene, retrying next frame.");
				return true;
			}

#if UE_BUILD_SHIPPING
			const bool bTriggerCapture = false;
#else
			const bool bTriggerCapture = (PCGInstanceDataInterface::TriggerGPUCaptureDispatchIndex > 0);
			PCGInstanceDataInterface::TriggerGPUCaptureDispatchIndex = FMath::Max(PCGInstanceDataInterface::TriggerGPUCaptureDispatchIndex - 1, 0);
#endif // UE_BUILD_SHIPPING

			FPrimitiveSceneDesc PrimitiveSceneDesc;
			PrimitiveSceneDesc.SceneProxy = SceneProxy;

			Scene->UpdatePrimitiveInstancesFromCompute(&PrimitiveSceneDesc, FGPUSceneWriteDelegate::CreateLambda([PrimitiveIndex, PrimitiveNumInstances, CumulativeInstanceCount, Seed, bTriggerCapture, InstanceDataExported, InstanceCustomFloatDataExported, WriteCountersExported, NumCullingCellsTotal=NumCullingCellsTotal](FRDGBuilder& InGraphBuilder, const FGPUSceneWriteDelegateParams& Params)
			{
				RDG_EVENT_SCOPE(InGraphBuilder, "SceneComputeUpdateInterface->EnqueueUpdate");
				check(Params.PersistentPrimitiveId != (uint32)INDEX_NONE);

#if !UE_BUILD_SHIPPING
				RenderCaptureInterface::FScopedCapture RenderCapture(bTriggerCapture, InGraphBuilder, TEXT("SceneComputeUpdateInterface->EnqueueUpdate"));
#endif // !UE_BUILD_SHIPPING

				FRDGBufferRef InstanceDataInner  = InGraphBuilder.RegisterExternalBuffer(InstanceDataExported);
				FRDGBufferRef InstanceCustomFloatDataInner = InGraphBuilder.RegisterExternalBuffer(InstanceCustomFloatDataExported);
				FRDGBufferRef WriteCountersInner = InGraphBuilder.RegisterExternalBuffer(WriteCountersExported);

				// Write instances
				FPCGSceneWriterCS::FParameters* Parameters = InGraphBuilder.AllocParameters<FPCGSceneWriterCS::FParameters>();
				Parameters->InPrimitiveIndex = PrimitiveIndex;
				Parameters->InNumInstancesAllocatedInGPUScene = PrimitiveNumInstances;
				Parameters->InInstanceOffset = CumulativeInstanceCount;
				Parameters->InInstanceData = InGraphBuilder.CreateSRV(InstanceDataInner);
				Parameters->InInstanceCustomFloatData = InGraphBuilder.CreateSRV(InstanceCustomFloatDataInner);
				Parameters->InWriteCounters = InGraphBuilder.CreateSRV(WriteCountersInner);
				Parameters->InPrimitiveId = Params.PersistentPrimitiveId;
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Parameters->GPUSceneWriterParameters = Params.GPUWriteParams;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				Parameters->InCustomDataCount = Params.NumCustomDataFloats;
				Parameters->InPayloadDataFlags = Params.PackedInstanceSceneDataFlags;
				Parameters->InSeed = Seed;
				Parameters->InNumCullingCellsTotal = NumCullingCellsTotal;

#if PCG_INSTANCE_DATA_LOGGING
				UE_LOGF(LogPCG, Log, "\tScene writer delegate [%d]:\tPrimitive ID %u,\tsource instance offset %u,\tInstanceSceneDataOffset %u, num instances %u",
					Parameters->InPrimitiveIndex,
					Parameters->InPrimitiveId,
					Parameters->InInstanceOffset,
					Params.InstanceSceneDataOffset,
					Parameters->InNumInstancesAllocatedInGPUScene);
#endif

				TShaderMapRef<FPCGSceneWriterCS> Shader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

				const FIntVector WrappedGroupCount = FComputeShaderUtils::GetGroupCountWrapped(FMath::DivideAndRoundUp<int>(PrimitiveNumInstances, FPCGSceneWriterCS::NumThreadsPerGroup));

				FComputeShaderUtils::AddPass(InGraphBuilder, RDG_EVENT_NAME("PCGWriteInstanceData"), ERDGPassFlags::Compute | ERDGPassFlags::NeverCull, Shader, Parameters, WrappedGroupCount);
			}));

			CumulativeInstanceCount += PrimitiveNumInstances;
		}

		DataProviderInner->bWroteInstances = true;

		return false;
	});
}
