// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "RHIResources.h"

#include "PCGInstanceDataInterface.generated.h"

class FPCGInstanceDataInterfaceParameters;
class FPrimitiveSceneProxy;
class IPCGRuntimePrimitiveFactory;
class UPCGSettings;
class UComputeKernel;
struct FPCGContextHandle;

/** Data Interface for writing instance data (transform and custom floats) to an intermediate buffer, which is then injected into the GPU Scene. */
UCLASS(ClassGroup = (Procedural))
class UPCGInstanceDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGInstanceData"); }
	void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	TCHAR const* GetShaderVirtualPath() const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	void GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const override;
	bool GetRequiresPostSubmitCall() const override { return true; }
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	void SetInputPinProvidingData(FName InInputPinProvidingData) { InputPinProvidingData = InInputPinProvidingData; }

	int32 GetCullingCellExtent() const { return CullingCellExtent; }
	void SetCullingCellExtent(int32 InGridSize) { CullingCellExtent = InGridSize; }

protected:
	UPROPERTY()
	FName InputPinProvidingData = NAME_None;

	/** Extent of culling cells in cm. */
	UPROPERTY()
	int32 CullingCellExtent = 0;

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for writing to the GPU Scene. */
UCLASS()
class UPCGInstanceDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	virtual bool PostExecute(UPCGDataBinding* InBinding) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	TSharedPtr<IPCGRuntimePrimitiveFactory> PrimitiveFactory;

	uint32 NumInstancesAllPrimitives = 0;

	uint32 NumCustomFloatsPerInstance = 0;

	/** Whether instance data has been applied to scene and operation is complete. */
	bool bWroteInstances = false;

	int32 NumCullingCellsTotal = 1;

	TWeakPtr<FPCGContextHandle> ContextHandle;
};

class FPCGInstanceDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FParams
	{
		TSharedPtr<IPCGRuntimePrimitiveFactory> PrimitiveFactory;
		uint32 NumInstancesAllPrimitives = 0;
		uint32 NumCustomFloatsPerInstance = 0;
		int32 NumCullingCellsTotal = 1;
		uint32 Seed = 42;
		TWeakObjectPtr<UPCGInstanceDataProvider> DataProvider;
		TWeakPtr<FPCGContextHandle> ContextHandle;
	};

	FPCGInstanceDataProviderProxy(const FParams& InParams);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData);
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void PostSubmit(FComputeContext& Context) const override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGInstanceDataInterfaceParameters;

	bool bIsValid = false;

	TSharedPtr<IPCGRuntimePrimitiveFactory> PrimitiveFactory;

	/** Instance transforms of all instances across all primitives. Three float4s per instance. */
	FRDGBufferRef InstanceData = nullptr;
	FRDGBufferUAVRef InstanceDataUAV = nullptr;

	/** Custom float data for all instances across all primitives. N per instance. Number of custom floats N comes from inputs (and is static). */
	FRDGBufferRef InstanceCustomFloatData = nullptr;
	FRDGBufferUAVRef InstanceCustomFloatDataUAV = nullptr;

	/** A per-primitive instance counter. Updated atomically and used to place instances into a segmented array (one segment per primitive). */
	FRDGBufferRef WriteCounters = nullptr;
	FRDGBufferUAVRef WriteCountersUAV = nullptr;

	FRDGBufferSRVRef PrimitiveParametersSRV = nullptr;

	uint32 NumInstancesAllPrimitives = 0;
	uint32 NumCustomFloatsPerInstance = 0;
	int32 NumCullingCellsTotal = 1;
	uint32 Seed = 42;

	TWeakObjectPtr<UPCGInstanceDataProvider> DataProvider;
	uint64 DataProviderGeneration = 0;

	TWeakPtr<FPCGContextHandle> ContextHandle;
};
