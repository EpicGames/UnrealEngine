// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGLandscapeDataInterface.h"

#include "PCGComponent.h"
#include "PCGData.h"
#include "Compute/PCGDataBinding.h"
#include "Data/PCGLandscapeData.h"

#include "EngineUtils.h"
#include "GlobalRenderResources.h"
#include "Landscape.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "LandscapeInfo.h"
#include "RenderGraphBuilder.h"
#include "RenderResource.h"
#include "RHIStaticStates.h"
#include "RHIUtilities.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Containers/ResourceArray.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "VT/RuntimeVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGLandscapeDataInterface)

#define LOCTEXT_NAMESPACE "UPCGLandscapeDataInterface"

namespace PCGLandscapeDataInterface
{
	static TAutoConsoleVariable<bool> CVarEnableLandscapeVTSampling(
		TEXT("pcg.GPU.EnableLandscapeVTSampling"),
		true,
		TEXT("Allow the landscape data interface to use landscape virtual textures if available."));
}

/** RenderResource used to hold textures generated by this DI, pulled from the collision geometry of the terrain. */
class FPCGLandscapeTextureResource : public FRenderResource
{
public:
	FPCGLandscapeTextureResource(const FIntPoint& InCellCount): CellCount(InCellCount) {}
	virtual ~FPCGLandscapeTextureResource() = default;

	FPCGLandscapeTextureResource() = delete;
	FPCGLandscapeTextureResource(const FPCGLandscapeTextureResource&) = delete;
	FPCGLandscapeTextureResource(const FPCGLandscapeTextureResource&&) = delete;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		if (HeightValues.Num())
		{
			FResourceBulkDataArrayView BulkData(HeightValues);
			HeightTexture.Initialize(RHICmdList, TEXT("FLandscapeTextureResource_HeightTexture"), sizeof(float), CellCount.X, CellCount.Y, EPixelFormat::PF_R32_FLOAT, FTextureReadBuffer2D::DefaultTextureInitFlag, &BulkData);
		}

		ReleaseSourceData();
	}

	virtual void ReleaseRHI() override { HeightTexture.Release(); }
	void ReleaseSourceData() { HeightValues.Empty(); }
	FRHITexture* GetHeightTexture() const { return HeightTexture.Buffer; }
	FIntPoint GetDimensions() const { return CellCount; }

	TArray<float>& EditHeightValues(int32 SampleCount)
	{
		const float DefaultHeight = 0.0f;
		HeightValues.Reset(SampleCount);
		HeightValues.Init(DefaultHeight, SampleCount);

		return HeightValues;
	}

private:
	FTextureReadBuffer2D HeightTexture;
	FIntPoint CellCount;

	TArray<float> HeightValues;
};

FPCGLandscapeResource::FPCGLandscapeResource(const FResourceKey& InKey)
	: ResourceKey(InKey)
{
	const ULandscapeInfo* LandscapeInfo = ResourceKey.Source->GetLandscapeInfo();
	if (!LandscapeInfo)
	{
		UE_LOGF(LogPCG, Warning, "FPCGLandscapeResource: Could not retrieve landscape info to setup landscape height data.");
		return;
	}

	const int32 ComponentQuadCount = ResourceKey.Source->ComponentSizeQuads;
	const FIntPoint RegionSpan = ResourceKey.MaxCaptureRegion - ResourceKey.MinCaptureRegion + FIntPoint(1, 1);
	const FIntPoint CaptureQuadSpan = RegionSpan * ComponentQuadCount;
	const FIntPoint CaptureVertexSpan = CaptureQuadSpan + FIntPoint(1, 1);
	const int32 SampleCount = CaptureVertexSpan.X * CaptureVertexSpan.Y;

	LandscapeTexture = new FPCGLandscapeTextureResource(CaptureVertexSpan);

	TArray<float>* HeightValues = &LandscapeTexture->EditHeightValues(SampleCount);

	const FIntPoint RegionVertexBase = ResourceKey.MinCaptureRegion * ComponentQuadCount;

	for (const FIntPoint& Region : ResourceKey.CapturedRegions)
	{
		auto FoundCollisionComponent = LandscapeInfo->XYtoCollisionComponentMap.Find(Region);
		check(FoundCollisionComponent);

		if (FoundCollisionComponent)
		{
			if (const ULandscapeHeightfieldCollisionComponent* CollisionComponent = *FoundCollisionComponent)
			{
				if (HeightValues)
				{
					const FIntPoint SectionBase = (Region - ResourceKey.MinCaptureRegion) * ComponentQuadCount;
					CollisionComponent->FillHeightTile(*HeightValues, SectionBase.X + SectionBase.Y * CaptureVertexSpan.X, CaptureVertexSpan.X);
				}
			}
		}
	}

	// Number of cells that are represented in our heights array.
	CellCount = CaptureVertexSpan;

	// Mapping to get the UV from 'cell space' which is relative to the entire terrain (not just the captured regions).
	FVector2D UVScale(1.0f / CaptureVertexSpan.X, 1.0f / CaptureVertexSpan.Y);

	UVScaleBias = FVector4(
		UVScale.X,
		UVScale.Y,
		(0.5f - RegionVertexBase.X) * UVScale.X,
		(0.5f - RegionVertexBase.Y) * UVScale.Y);

	FTransform LandscapeTransform = ResourceKey.Source->GetTransform();
	FLargeWorldRenderPosition LandscapeTransformOrigin(LandscapeTransform.GetLocation());

	LandscapeLWCTile = LandscapeTransformOrigin.GetTile();
	LandscapeTransform.SetLocation(FVector(LandscapeTransformOrigin.GetOffset()));

	ActorToWorldTransform = LandscapeTransform.ToMatrixWithScale();
	WorldToActorTransform = ActorToWorldTransform.Inverse();
}

FPCGLandscapeResource::~FPCGLandscapeResource()
{
	if (LandscapeTexture)
	{
		// Release the LandscapeTexture resource handle on the RHI and clear the pointer.
		ENQUEUE_RENDER_COMMAND(BeginDestroyCommand)([RT_Resource = LandscapeTexture](FRHICommandListImmediate& RHICmdList)
		{
			check(RT_Resource);
			RT_Resource->ReleaseResource();

			// On some RHIs textures will push data on the RHI thread
			// Therefore we are not 'released' until the RHI thread has processed all commands
			RHICmdList.EnqueueLambda([RT_Resource](FRHICommandListImmediate& RHICmdList)
			{
				delete RT_Resource;
			});
		});

		LandscapeTexture = nullptr;
	}
}

void UPCGLandscapeDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetHeight"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Float, 3); // InWorldPos

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetNormal"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Float, 3); // InWorldPos

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetBaseColor"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Float, 3); // InWorldPos
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGLandscapeDataInterfaceParameters,)
	// Height RVT
	SHADER_PARAMETER_SRV(Texture2D,				HeightVirtualTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D<uint4>,	HeightVirtualTexturePageTable)
	SHADER_PARAMETER_TEXTURE(Texture2D<uint>,	HeightVirtualTexturePageTableIndirection)
	SHADER_PARAMETER(uint32,					HeightVirtualTextureAdaptive)
	SHADER_PARAMETER_SAMPLER(SamplerState,		HeightVirtualTextureSampler)
	SHADER_PARAMETER(FVector3f,					HeightVirtualTextureLWCTile)
	SHADER_PARAMETER(FMatrix44f,				HeightVirtualTextureWorldToUVTransform)
	SHADER_PARAMETER(uint32,					HeightVirtualTextureEnabled)
	SHADER_PARAMETER(FUintVector4,				HeightVirtualTexturePackedUniform0)
	SHADER_PARAMETER(FUintVector4,				HeightVirtualTexturePackedUniform1)
	SHADER_PARAMETER(FUintVector4,				HeightVirtualTextureUniforms)

	// Normal RVT
	SHADER_PARAMETER_SRV(Texture2D,				NormalVirtualTexture0)
	SHADER_PARAMETER_SRV(Texture2D,				NormalVirtualTexture1)
	SHADER_PARAMETER_TEXTURE(Texture2D<uint4>,	NormalVirtualTexturePageTable)
	SHADER_PARAMETER_TEXTURE(Texture2D<uint>,	NormalVirtualTexturePageTableIndirection)
	SHADER_PARAMETER(uint32,					NormalVirtualTextureAdaptive)
	SHADER_PARAMETER(FUintVector2,				NormalVirtualTexturePageTableLayers)
	SHADER_PARAMETER_SAMPLER(SamplerState,		NormalVirtualTexture0Sampler)
	SHADER_PARAMETER_SAMPLER(SamplerState,		NormalVirtualTexture1Sampler)
	SHADER_PARAMETER(FVector3f,					NormalVirtualTextureLWCTile)
	SHADER_PARAMETER(FMatrix44f,				NormalVirtualTextureWorldToUVTransform)
	SHADER_PARAMETER(FUintVector4,				NormalVirtualTexturePackedUniform0)
	SHADER_PARAMETER(FUintVector4,				NormalVirtualTexturePackedUniform1)
	SHADER_PARAMETER(FUintVector4,				NormalVirtualTextureUniforms0)
	SHADER_PARAMETER(FUintVector4,				NormalVirtualTextureUniforms1)
	SHADER_PARAMETER(int32,						NormalVirtualTextureUnpackMode)
	SHADER_PARAMETER(uint32,					NormalVirtualTextureEnabled)

	// Base Color RVT
	SHADER_PARAMETER_SRV(Texture2D,				BaseColorVirtualTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D<uint4>,	BaseColorVirtualTexturePageTable)
	SHADER_PARAMETER_TEXTURE(Texture2D<uint>,	BaseColorVirtualTexturePageTableIndirection)
	SHADER_PARAMETER(uint32,					BaseColorVirtualTextureAdaptive)
	SHADER_PARAMETER_SAMPLER(SamplerState,		BaseColorVirtualTextureSampler)
	SHADER_PARAMETER(FVector3f,					BaseColorVirtualTextureLWCTile)
	SHADER_PARAMETER(FMatrix44f,				BaseColorVirtualTextureWorldToUVTransform)
	SHADER_PARAMETER(uint32,					BaseColorVirtualTextureUnpackSRGB)
	SHADER_PARAMETER(uint32,					BaseColorVirtualTextureUnpackYCoCg)
	SHADER_PARAMETER(uint32,					BaseColorVirtualTextureEnabled)
	SHADER_PARAMETER(FUintVector4,				BaseColorVirtualTexturePackedUniform0)
	SHADER_PARAMETER(FUintVector4,				BaseColorVirtualTexturePackedUniform1)
	SHADER_PARAMETER(FUintVector4,				BaseColorVirtualTextureUniforms)

	// Collision based height texture
	SHADER_PARAMETER_TEXTURE(Texture2D,			CollisionHeightTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState,		CollisionHeightTextureSampler)
	SHADER_PARAMETER(uint32,					CollisionHeightTextureEnabled)
	SHADER_PARAMETER(FVector3f,					CollisionHeightTextureLWCTile)
	SHADER_PARAMETER(FMatrix44f,				CollisionHeightTextureWorldToUVTransform)
	SHADER_PARAMETER(FMatrix44f,				CollisionHeightTextureUVToWorldTransform)
	SHADER_PARAMETER(FVector4f,					CollisionHeightTextureUVScaleBias)
	SHADER_PARAMETER(FIntPoint,					CollisionHeightTextureDimension)

	// Misc parameters
	SHADER_PARAMETER(FVector2f,					LandscapeGridSize)
END_SHADER_PARAMETER_STRUCT()

void UPCGLandscapeDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGLandscapeDataInterfaceParameters>(UID);
}

TCHAR const* UPCGLandscapeDataInterface::TemplateFilePath = TEXT("/Plugin/PCG/Private/PCGLandscapeDataInterface.ush");

TCHAR const* UPCGLandscapeDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UPCGLandscapeDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
	GetShaderFileHash(TEXT("/Engine/Private/VirtualTextureCommon.ush"), EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
	GetShaderFileHash(TEXT("/Plugin/PCG/Private/PCGVirtualTextureCommon.ush"), EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UPCGLandscapeDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	if (ensure(LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr)))
	{
		OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
	}
}

UComputeDataProvider* UPCGLandscapeDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGLandscapeDataProvider>();
}

void UPCGLandscapeDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGLandscapeDataProvider::Initialize);

	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGLandscapeDataInterface* DataInterface = CastChecked<UPCGLandscapeDataInterface>(InDataInterface);

	UPCGDataBinding* Binding = CastChecked<UPCGDataBinding>(InBinding);

	const IPCGGraphExecutionSource* ExecutionSource = Binding->GetExecutionSource();
	check(ExecutionSource);

	const UWorld* World = ExecutionSource->GetExecutionState().GetWorld();
	const FBox ComponentBounds = ExecutionSource->GetExecutionState().GetBounds();
	const UPCGComponent* SourceComponent = Cast<UPCGComponent>(ExecutionSource);

	// Sampling of virtual textures is not supported outside of runtime generation at this time, because the RuntimeGenScheduler is responsible for priming the virtual textures.
	const bool bCanSampleVirtualTextures = SourceComponent && SourceComponent->IsManagedByRuntimeGenSystem() && PCGLandscapeDataInterface::CVarEnableLandscapeVTSampling.GetValueOnAnyThread();

	auto TestLandscape = [World, &ComponentBounds](const ALandscape* InLandscape)
	{
		if (InLandscape->GetWorld() == World)
		{
			if (const ULandscapeInfo* LandscapeInfo = InLandscape->GetLandscapeInfo())
			{
				for (const auto& ComponentIt : LandscapeInfo->XYtoCollisionComponentMap)
				{
					if (ComponentBounds.IntersectXY(ComponentIt.Value->Bounds.GetBox()))
					{
						return true;
					}
				}
			}
		}

		return false;
	};

	// Take any input pin label alias to obtain the data from the input data collection.
	check(!DataInterface->GetDownstreamInputPinLabelAliases().IsEmpty());
	const TArray<FPCGTaggedData> TaggedDatas = Binding->GetInputDataCollection().GetInputsByPin(DataInterface->GetDownstreamInputPinLabelAliases()[0]);

	if (!TaggedDatas.IsEmpty())
	{
		ensure(TaggedDatas.Num() == 1); // There should only be one landscape data
		ensure(TaggedDatas[0].Data->IsA<UPCGLandscapeData>());

		if (const UPCGLandscapeData* LandscapeData = Cast<UPCGLandscapeData>(TaggedDatas[0].Data))
		{
			for (TSoftObjectPtr<ALandscapeProxy> LandscapeProxyPtr : LandscapeData->Landscapes)
			{
				if (ALandscapeProxy* LandscapeProxy = LandscapeProxyPtr.Get())
				{
					if (ALandscape* Landscape = LandscapeProxy->GetLandscapeActor())
					{
						if (TestLandscape(Landscape))
						{
							InitFromLandscape(Binding, Landscape, ComponentBounds, bCanSampleVirtualTextures && LandscapeData->CanSampleVirtualTextures(), LandscapeData->CanSampleVirtualTextureNormals());
							break;
						}
					}
				}
			}
		}
	}
}

void UPCGLandscapeDataProvider::InitFromLandscape(UPCGDataBinding* InBinding, ALandscape* InLandscape, const FBox& Bounds, bool bAllowSampleVirtualTexture, bool bAllowSampleVirtualTextureNormal)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGLandscapeDataProvider::InitFromLandscape);
	check(IsInGameThread());

	if (!ensure(InLandscape))
	{
		return;
	}

	LandscapeGridSize = FVector2D(InLandscape->GetTransform().GetScale3D());

	if (bAllowSampleVirtualTexture)
	{
		InitializeRuntimeVirtualTextures(InLandscape, bAllowSampleVirtualTextureNormal);
	}

	// If the height can't be accessed via RVT, build a physics based height texture instead.
	if (!HeightVirtualTexture)
	{
		UE_CLOGF(bAllowSampleVirtualTexture, LogPCG, Verbose, "UPCGLandscapeDataProvider: No runtime virtual texture found to use for landscape heights, falling back to uploading data from landscape collision components.");

		InitializeFromLandscapeCollision(InBinding, InLandscape, Bounds);
	}
}

void UPCGLandscapeDataProvider::InitializeRuntimeVirtualTextures(ALandscape* InLandscape, bool bAllowSampleVirtualTextureNormal)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGLandscapeDataProvider::InitializeRuntimeVirtualTextures);
	check(InLandscape);

	int32 BaseColorVirtualTextureIndex = INDEX_NONE;
	int32 HeightVirtualTextureIndex = INDEX_NONE;
	int32 NormalVirtualTextureIndex = INDEX_NONE;

	// Only worry about virtual textures if our current platform supports them.
	if (UseVirtualTexturing(GMaxRHIShaderPlatform))
	{
		const int32 RuntimeVirtualTextureCount = InLandscape->RuntimeVirtualTextures.Num();
		for (int32 TextureIndex = 0; TextureIndex < RuntimeVirtualTextureCount; ++TextureIndex)
		{
			if (const URuntimeVirtualTexture* RVT = InLandscape->RuntimeVirtualTextures[TextureIndex])
			{
				const ERuntimeVirtualTextureMaterialType VirtualMaterialType = RVT->GetMaterialType();

				switch (VirtualMaterialType)
				{
				case ERuntimeVirtualTextureMaterialType::WorldHeight:
					if (HeightVirtualTextureIndex == INDEX_NONE)
					{
						HeightVirtualTextureIndex = TextureIndex;
					}
					break;
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
					if (NormalVirtualTextureIndex == INDEX_NONE)
					{
						NormalVirtualTextureIndex = TextureIndex;
						NormalVirtualTextureMode = VirtualMaterialType;
					}
					// intentionally falling through to the BaseColor option
				case ERuntimeVirtualTextureMaterialType::BaseColor:
					if (BaseColorVirtualTextureIndex == INDEX_NONE)
					{
						BaseColorVirtualTextureIndex = TextureIndex;
					}
					break;
				}

				if (BaseColorVirtualTextureIndex == TextureIndex)
				{
					bBaseColorSRGB = RVT->IsLayerSRGB(0);

					if (RVT->IsLayerYCoCg(0))
					{
						BaseColorVirtualTextureUnpackType = PCGVirtualTextureCommon::EBaseColorUnpackType::YCoCgUnpack;
					}
					else if (VirtualMaterialType == ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness)
					{
						BaseColorVirtualTextureUnpackType = PCGVirtualTextureCommon::EBaseColorUnpackType::SRGBUnpack;
					}
				}
			}
		}
	}

	BaseColorVirtualTexture = InLandscape->RuntimeVirtualTextures.IsValidIndex(BaseColorVirtualTextureIndex)
		? BaseColorVirtualTexture = InLandscape->RuntimeVirtualTextures[BaseColorVirtualTextureIndex]
		: nullptr;

	HeightVirtualTexture = InLandscape->RuntimeVirtualTextures.IsValidIndex(HeightVirtualTextureIndex)
		?  InLandscape->RuntimeVirtualTextures[HeightVirtualTextureIndex]
		: nullptr;

	if (bAllowSampleVirtualTextureNormal)
	{
		NormalVirtualTexture = InLandscape->RuntimeVirtualTextures.IsValidIndex(NormalVirtualTextureIndex)
			? InLandscape->RuntimeVirtualTextures[NormalVirtualTextureIndex]
			: nullptr;
	}
}

void UPCGLandscapeDataProvider::InitializeFromLandscapeCollision(UPCGDataBinding* InBinding, ALandscape* InLandscape, const FBox& Bounds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGLandscapeDataProvider::InitializeFromLandscapeCollision);
	check(InLandscape);

	const ULandscapeInfo* LandscapeInfo = InLandscape->GetLandscapeInfo();

	if (!ensure(LandscapeInfo))
	{
		return;
	}

	// We want to use the bounds of the system to figure out which cells of the landscape that we need to handle.
	const int32 MaxLandscapeRegionCount = LandscapeInfo->XYtoCollisionComponentMap.Num();

	const FVector3f LWCTile(0, 0, 0); // Used to offset in LWC for precision, default to 0-vector for now.
	const FVector LWCTileOffset = FVector(LWCTile) * FLargeWorldRenderScalar::GetTileSize();
	FBox SystemWorldBounds = Bounds;
	SystemWorldBounds.Min += LWCTileOffset;
	// Expand max bounds by the LS grid size so that we don't go off the height texture when computing normals for points at the edges of the bounds.
	SystemWorldBounds.Max += LWCTileOffset + FVector(LandscapeGridSize.X, LandscapeGridSize.Y, 0.0);

	const FTransform& LandscapeActorToWorld = InLandscape->LandscapeActorToWorld();
	const FVector SystemMinInLandscape = LandscapeActorToWorld.InverseTransformPosition(SystemWorldBounds.Min);
	const FVector SystemMaxInLandscape = LandscapeActorToWorld.InverseTransformPosition(SystemWorldBounds.Max);

	FBox SystemBoundsInLandscape(
		SystemMinInLandscape.ComponentMin(SystemMaxInLandscape),
		SystemMinInLandscape.ComponentMax(SystemMaxInLandscape));

	// Rescale based on the quad size.
	const double QuadSizeScaleFactor = 1.0 / ((double)InLandscape->ComponentSizeQuads);
	SystemBoundsInLandscape.Min *= QuadSizeScaleFactor;
	SystemBoundsInLandscape.Max *= QuadSizeScaleFactor;

	// Clamp between Min/Max int32.
	const FVector MinIntValue = FVector(double(TNumericLimits<int32>::Min()));
	SystemBoundsInLandscape.Min = SystemBoundsInLandscape.Min.ComponentMax(MinIntValue);
	SystemBoundsInLandscape.Max = SystemBoundsInLandscape.Max.ComponentMax(MinIntValue);

	const FVector MaxIntValue = FVector(double(TNumericLimits<int32>::Max()));
	SystemBoundsInLandscape.Min = SystemBoundsInLandscape.Min.ComponentMin(MaxIntValue);
	SystemBoundsInLandscape.Max = SystemBoundsInLandscape.Max.ComponentMin(MaxIntValue);

	const FIntRect SystemRect = FIntRect(
		FIntPoint(FMath::FloorToInt(SystemBoundsInLandscape.Min.X), FMath::FloorToInt(SystemBoundsInLandscape.Min.Y)),
		FIntPoint(FMath::CeilToInt(SystemBoundsInLandscape.Max.X), FMath::CeilToInt(SystemBoundsInLandscape.Max.Y)));

	// For obnoxiously large system bounds we need to guard against potential overflow on the number of cells.
	const int32 MaxSystemWidth = FMath::Clamp<int32>(SystemRect.Max.X - SystemRect.Min.X, 0, MaxLandscapeRegionCount);
	const int32 MaxSystemHeight = FMath::Clamp<int32>(SystemRect.Max.Y - SystemRect.Min.Y, 0, MaxLandscapeRegionCount);

	const int64 MaxSystemRegionCount64 = int64(MaxSystemWidth) * int64(MaxSystemHeight);
	const int32 MaxSystemRegionCount = int32(FMath::Min<int64>(int64(TNumericLimits<int32>::Max()), MaxSystemRegionCount64));

	const int32 MaxRegionCount = FMath::Min(MaxSystemRegionCount, MaxLandscapeRegionCount);

	FPCGLandscapeResource::FResourceKey Key;
	Key.Source = InLandscape;
	Key.CapturedRegions.Reserve(MaxRegionCount);
	Key.MinCaptureRegion = FIntPoint(TNumericLimits<int32>::Max(), TNumericLimits<int32>::Max());
	Key.MaxCaptureRegion = FIntPoint(TNumericLimits<int32>::Min(), TNumericLimits<int32>::Min());

	auto AddRegion = [&](const FIntPoint& Region)
	{
		Key.CapturedRegions.Add(Region);
		Key.MinCaptureRegion = Key.MinCaptureRegion.ComponentMin(Region);
		Key.MaxCaptureRegion = Key.MaxCaptureRegion.ComponentMax(Region);
	};

	if (MaxSystemRegionCount > MaxLandscapeRegionCount)
	{
		for (const auto& LandscapeComponent : LandscapeInfo->XYtoCollisionComponentMap)
		{
			if (SystemRect.Contains(LandscapeComponent.Key))
			{
				AddRegion(LandscapeComponent.Key);
			}
		}
	}
	else
	{
		for (int32 GridY = SystemRect.Min.Y; GridY < SystemRect.Max.Y; ++GridY)
		{
			for (int32 GridX = SystemRect.Min.X; GridX < SystemRect.Max.X; ++GridX)
			{
				const FIntPoint CurrentRegion(GridX, GridY);
				if (LandscapeInfo->XYtoCollisionComponentMap.Contains(CurrentRegion))
				{
					AddRegion(CurrentRegion);
				}
			}
		}
	}

	Resource = MakeShared<FPCGLandscapeResource>(Key);

#if WITH_EDITOR
	NotifyProducerUploadedData(InBinding);
#endif // WITH_EDITOR
}

FComputeDataProviderRenderProxy* UPCGLandscapeDataProvider::GetRenderProxy()
{
	return new FPCGLandscapeDataProviderProxy(
		Resource,
		BaseColorVirtualTexture,
		HeightVirtualTexture,
		NormalVirtualTexture,
		bBaseColorSRGB,
		BaseColorVirtualTextureUnpackType,
		NormalVirtualTextureMode,
		LandscapeGridSize);
}

void UPCGLandscapeDataProvider::Reset()
{
	BaseColorVirtualTexture = nullptr;
	HeightVirtualTexture = nullptr;
	NormalVirtualTexture = nullptr;

	Resource.Reset();
	bBaseColorSRGB = false;
	BaseColorVirtualTextureUnpackType = PCGVirtualTextureCommon::EBaseColorUnpackType::None;
	NormalVirtualTextureMode = ERuntimeVirtualTextureMaterialType::Count;
	LandscapeGridSize = FVector2D(1.0f, 1.0f);

	Super::Reset();
}

FPCGLandscapeDataProviderProxy::FPCGLandscapeDataProviderProxy(
	TSharedPtr<const FPCGLandscapeResource> InResource,
	const URuntimeVirtualTexture* InBaseColorVirtualTexture,
	const URuntimeVirtualTexture* InHeightVirtualTexture,
	const URuntimeVirtualTexture* InNormalVirtualTexture,
	bool bInBaseColorSRGB,
	PCGVirtualTextureCommon::EBaseColorUnpackType InBaseColorVirtualTextureUnpackType,
	ERuntimeVirtualTextureMaterialType InNormalVirtualTextureMode,
	FVector2D InLandscapeGridSize)
	: Resource(InResource)
	, BaseColorVirtualTexture(InBaseColorVirtualTexture)
	, HeightVirtualTexture(InHeightVirtualTexture)
	, NormalVirtualTexture(InNormalVirtualTexture)
	, bBaseColorSRGB(bInBaseColorSRGB)
	, NormalVirtualTextureMode(InNormalVirtualTextureMode)
	, BaseColorVirtualTextureUnpackType(InBaseColorVirtualTextureUnpackType)
	, LandscapeGridSize(InLandscapeGridSize)
{
}

bool FPCGLandscapeDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGLandscapeDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		SetShaderParameters(ParameterArray[InvocationIndex]);
	}
}

void FPCGLandscapeDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	LLM_SCOPE_BYTAG(PCG);

	if (TStrongObjectPtr<const URuntimeVirtualTexture> BaseColorVT = BaseColorVirtualTexture.Pin())
	{
		BaseColorVirtualLayer.Initialize(BaseColorVT.Get(), /*PageTableIndex=*/0, bBaseColorSRGB);
		BaseColorVirtualPage.Initialize(BaseColorVT.Get());
	}

	if (TStrongObjectPtr<const URuntimeVirtualTexture> HeightVT = HeightVirtualTexture.Pin())
	{
		HeightVirtualLayer.Initialize(HeightVT.Get(), /*PageTableIndex=*/0, /*bSRGB=*/false);
		HeightVirtualPage.Initialize(HeightVT.Get());
	}

	if (TStrongObjectPtr<const URuntimeVirtualTexture> NormalVT = NormalVirtualTexture.Pin())
	{
		const URuntimeVirtualTexture* NormalVTPtr = NormalVT.Get();

		NormalVirtualPage.Initialize(NormalVTPtr);

		switch (NormalVirtualTextureMode)
		{
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
			NormalVirtualLayer0.Initialize(NormalVTPtr, /*PageTableIndex=*/0, /*bSRGB=*/false);
			NormalVirtualLayer1.Initialize(NormalVTPtr, /*PageTableIndex=*/1, /*bSRGB=*/false);
			NormalUnpackMode = PCGVirtualTextureCommon::ENormalUnpackType::B5G6R5;
			NormalPageTableLayers = NormalVirtualPage.bSinglePhysicalSpace ? FUintVector2(0, 0) : FUintVector2(0, 1);
			break;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
			NormalVirtualLayer0.Initialize(NormalVTPtr, /*PageTableIndex=*/0, /*bSRGB=*/false);
			NormalVirtualLayer1.Initialize(NormalVTPtr, /*PageTableIndex=*/1, /*bSRGB=*/false);
			NormalUnpackMode = PCGVirtualTextureCommon::ENormalUnpackType::BC3BC3;
			NormalPageTableLayers = NormalVirtualPage.bSinglePhysicalSpace ? FUintVector2(0, 0) : FUintVector2(0, 1);
			break;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
			NormalVirtualLayer0.Initialize(NormalVTPtr, /*PageTableIndex=*/1, /*bSRGB=*/false);
			NormalVirtualLayer1.Initialize(NormalVTPtr, /*PageTableIndex=*/2, /*bSRGB=*/false);
			NormalUnpackMode = PCGVirtualTextureCommon::ENormalUnpackType::BC5BC1;
			NormalPageTableLayers = NormalVirtualPage.bSinglePhysicalSpace ? FUintVector2(0, 0) : FUintVector2(1, 2);
			break;
		default:
			NormalVirtualPage.Reset();
			NormalVirtualLayer0.Reset();
			NormalVirtualLayer1.Reset();
			NormalUnpackMode = PCGVirtualTextureCommon::ENormalUnpackType::None;
			NormalPageTableLayers = FUintVector2(0, 0);
			break;
		}
	}

	if (Resource && Resource->LandscapeTexture)
	{
		Resource->LandscapeTexture->InitResource(GraphBuilder.RHICmdList);
	}
}

void FPCGLandscapeDataProviderProxy::SetShaderParameters(FParameters& OutShaderParameters)
{
	FRHISamplerState* BilinearSamplerState				= TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	FRHISamplerState* PointClampedSampler				= TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	OutShaderParameters.BaseColorVirtualTextureSampler	= BilinearSamplerState;
	OutShaderParameters.HeightVirtualTextureSampler		= BilinearSamplerState;
	OutShaderParameters.NormalVirtualTexture0Sampler	= BilinearSamplerState;
	OutShaderParameters.NormalVirtualTexture1Sampler	= BilinearSamplerState;
	OutShaderParameters.CollisionHeightTextureSampler	= RHIPixelFormatHasCapabilities(EPixelFormat::PF_R32_FLOAT, EPixelFormatCapabilities::TextureFilterable) ? BilinearSamplerState : PointClampedSampler;

	if (!SetBaseColorVirtualTextureParameters(OutShaderParameters))
	{
		SetBaseColorVirtualTextureParameters_Default(OutShaderParameters);
	}

	const bool bSetHeightVT = SetHeightVirtualTextureParameters(OutShaderParameters);

	if (!bSetHeightVT)
	{
		SetHeightVirtualTextureParameters_Default(OutShaderParameters);
	}

	if (!SetNormalVirtualTextureParameters(OutShaderParameters))
	{
		SetNormalVirtualTextureParameters_Default(OutShaderParameters);
	}

	const bool bSetCollisionHeightTexture = SetCollisionHeightTextureParameters(OutShaderParameters);

	if (!bSetCollisionHeightTexture)
	{
		SetCollisionHeightTextureParameters_Defaults(OutShaderParameters);
	}

	if (!bSetHeightVT && !bSetCollisionHeightTexture)
	{
		// Can be hit frequently in editor in unloaded areas for spatially loaded landscapes.
		UE_LOGF(LogPCG, Verbose, "PCGLandscapeDataInterface: No height texture initialized. If you are trying to use a virtual texture, make sure the RVT volumes are created. This may also occur when generating in areas where the landscape is not loaded.");
	}

	OutShaderParameters.LandscapeGridSize = (FVector2f)LandscapeGridSize;
}

bool FPCGLandscapeDataProviderProxy::SetBaseColorVirtualTextureParameters(FParameters& OutShaderParameters) const
{
	if (!BaseColorVirtualPage.IsValid() || !BaseColorVirtualLayer.IsValid())
	{
		return false;
	}

	OutShaderParameters.BaseColorVirtualTexture = BaseColorVirtualLayer.TextureSRV;
	OutShaderParameters.BaseColorVirtualTexturePageTable = BaseColorVirtualPage.PageTableRef;
	OutShaderParameters.BaseColorVirtualTexturePageTableIndirection = BaseColorVirtualPage.PageTableIndirectionRef;
	OutShaderParameters.BaseColorVirtualTextureAdaptive = BaseColorVirtualPage.bIsAdaptive;

	FLargeWorldRenderPosition BaseColorVirtualTextureOrigin(BaseColorVirtualPage.WorldToUVParameters[0]);
	OutShaderParameters.BaseColorVirtualTextureLWCTile = BaseColorVirtualTextureOrigin.GetTile();

	OutShaderParameters.BaseColorVirtualTextureWorldToUVTransform = FMatrix44f(
		BaseColorVirtualTextureOrigin.GetOffset(),
		FVector3f((FVector4f)BaseColorVirtualPage.WorldToUVParameters[1]),
		FVector3f((FVector4f)BaseColorVirtualPage.WorldToUVParameters[2]),
		FVector3f(0, 0, 0));
	OutShaderParameters.BaseColorVirtualTextureUnpackSRGB = BaseColorVirtualTextureUnpackType == PCGVirtualTextureCommon::EBaseColorUnpackType::SRGBUnpack;
	OutShaderParameters.BaseColorVirtualTextureUnpackYCoCg = BaseColorVirtualTextureUnpackType == PCGVirtualTextureCommon::EBaseColorUnpackType::YCoCgUnpack;
	OutShaderParameters.BaseColorVirtualTextureEnabled = 1;
	OutShaderParameters.BaseColorVirtualTexturePackedUniform0 = BaseColorVirtualPage.PageTableUniforms[0];
	OutShaderParameters.BaseColorVirtualTexturePackedUniform1 = BaseColorVirtualPage.PageTableUniforms[1];
	OutShaderParameters.BaseColorVirtualTextureUniforms = BaseColorVirtualLayer.TextureUniforms;

	return true;
}

bool FPCGLandscapeDataProviderProxy::SetHeightVirtualTextureParameters(FParameters& OutShaderParameters) const
{
	if (!HeightVirtualPage.IsValid() || !HeightVirtualLayer.IsValid())
	{
		return false;
	}

	OutShaderParameters.HeightVirtualTexture = HeightVirtualLayer.TextureSRV;
	OutShaderParameters.HeightVirtualTexturePageTable = HeightVirtualPage.PageTableRef;
	OutShaderParameters.HeightVirtualTexturePageTableIndirection = HeightVirtualPage.PageTableIndirectionRef;
	OutShaderParameters.HeightVirtualTextureAdaptive = HeightVirtualPage.bIsAdaptive;

	FLargeWorldRenderPosition HeightVirtualTextureOrigin(HeightVirtualPage.WorldToUVParameters[0]);
	OutShaderParameters.HeightVirtualTextureLWCTile = HeightVirtualTextureOrigin.GetTile();

	OutShaderParameters.HeightVirtualTextureWorldToUVTransform = FMatrix44f(
		HeightVirtualTextureOrigin.GetOffset(),
		FVector3f((FVector4f)HeightVirtualPage.WorldToUVParameters[1]),
		FVector3f((FVector4f)HeightVirtualPage.WorldToUVParameters[2]),
		FVector3f((FVector4f)HeightVirtualPage.WorldToUVParameters[3]));

	OutShaderParameters.HeightVirtualTextureEnabled = 1;
	OutShaderParameters.HeightVirtualTexturePackedUniform0 = HeightVirtualPage.PageTableUniforms[0];
	OutShaderParameters.HeightVirtualTexturePackedUniform1 = HeightVirtualPage.PageTableUniforms[1];
	OutShaderParameters.HeightVirtualTextureUniforms = HeightVirtualLayer.TextureUniforms;

	return true;
}

bool FPCGLandscapeDataProviderProxy::SetNormalVirtualTextureParameters(FParameters& OutShaderParameters) const
{
	if (!NormalVirtualPage.IsValid() || !NormalVirtualLayer0.IsValid() || !NormalVirtualLayer1.IsValid())
	{
		return false;
	}

	OutShaderParameters.NormalVirtualTexture0 = NormalVirtualLayer0.TextureSRV;
	OutShaderParameters.NormalVirtualTexture1 = NormalVirtualLayer1.TextureSRV;
	OutShaderParameters.NormalVirtualTexturePageTable = NormalVirtualPage.PageTableRef;
	OutShaderParameters.NormalVirtualTexturePageTableIndirection = NormalVirtualPage.PageTableIndirectionRef;
	OutShaderParameters.NormalVirtualTextureAdaptive = NormalVirtualPage.bIsAdaptive;

	FLargeWorldRenderPosition NormalVirtualTextureOrigin(NormalVirtualPage.WorldToUVParameters[0]);
	OutShaderParameters.NormalVirtualTextureLWCTile = NormalVirtualTextureOrigin.GetTile();

	OutShaderParameters.NormalVirtualTextureWorldToUVTransform = FMatrix44f(
		FPlane4f(NormalVirtualTextureOrigin.GetOffset()),
		FPlane4f((FVector4f)NormalVirtualPage.WorldToUVParameters[1]),
		FPlane4f((FVector4f)NormalVirtualPage.WorldToUVParameters[2]),
		FPlane4f(0.0f, 0.0f, 0.0f, 1.0f));

	OutShaderParameters.NormalVirtualTextureEnabled = 1;
	OutShaderParameters.NormalVirtualTextureUnpackMode = (int32)NormalUnpackMode;
	OutShaderParameters.NormalVirtualTexturePackedUniform0 = NormalVirtualPage.PageTableUniforms[0];
	OutShaderParameters.NormalVirtualTexturePackedUniform1 = NormalVirtualPage.PageTableUniforms[1];
	OutShaderParameters.NormalVirtualTextureUniforms0 = NormalVirtualLayer0.TextureUniforms;
	OutShaderParameters.NormalVirtualTextureUniforms1 = NormalVirtualLayer1.TextureUniforms;
	OutShaderParameters.NormalVirtualTexturePageTableLayers = NormalPageTableLayers;
	return true;
}

bool FPCGLandscapeDataProviderProxy::SetCollisionHeightTextureParameters(FParameters& OutShaderParameters) const
{
	FRHITexture* CollisionHeightTexture = (Resource && Resource->LandscapeTexture) ? Resource->LandscapeTexture->GetHeightTexture() : nullptr;

	if (!CollisionHeightTexture)
	{
		return false;
	}

	const FIntPoint TextureDimensions(Resource->LandscapeTexture->GetDimensions());

	OutShaderParameters.CollisionHeightTextureLWCTile = Resource->LandscapeLWCTile;
	OutShaderParameters.CollisionHeightTextureWorldToUVTransform = (FMatrix44f)Resource->WorldToActorTransform;
	OutShaderParameters.CollisionHeightTextureUVToWorldTransform = (FMatrix44f)Resource->ActorToWorldTransform;
	OutShaderParameters.CollisionHeightTextureUVScaleBias = (FVector4f)Resource->UVScaleBias;
	OutShaderParameters.CollisionHeightTexture = CollisionHeightTexture;
	OutShaderParameters.CollisionHeightTextureEnabled = 1;
	OutShaderParameters.CollisionHeightTextureDimension = TextureDimensions;

	return true;
}

void FPCGLandscapeDataProviderProxy::SetBaseColorVirtualTextureParameters_Default(FParameters& OutShaderParameters)
{
	const FUintVector4 DummyUint4(ForceInitToZero);

	OutShaderParameters.BaseColorVirtualTexture = GBlackTextureWithSRV->ShaderResourceViewRHI;
	OutShaderParameters.BaseColorVirtualTexturePageTable = GBlackUintTexture->TextureRHI;
	OutShaderParameters.BaseColorVirtualTexturePageTableIndirection = GBlackUintTexture->TextureRHI;
	OutShaderParameters.BaseColorVirtualTextureAdaptive = 0;
	OutShaderParameters.BaseColorVirtualTextureLWCTile = FVector3f::ZeroVector;
	OutShaderParameters.BaseColorVirtualTextureWorldToUVTransform = FMatrix44f::Identity;
	OutShaderParameters.BaseColorVirtualTextureUnpackSRGB = 0;
	OutShaderParameters.BaseColorVirtualTextureUnpackYCoCg = 0;
	OutShaderParameters.BaseColorVirtualTextureEnabled = 0;
	OutShaderParameters.BaseColorVirtualTexturePackedUniform0 = DummyUint4;
	OutShaderParameters.BaseColorVirtualTexturePackedUniform1 = DummyUint4;
	OutShaderParameters.BaseColorVirtualTextureUniforms = DummyUint4;
}

void FPCGLandscapeDataProviderProxy::SetHeightVirtualTextureParameters_Default(FParameters& OutShaderParameters)
{
	const FUintVector4 DummyUint4(ForceInitToZero);

	OutShaderParameters.HeightVirtualTexture = GBlackTextureWithSRV->ShaderResourceViewRHI;
	OutShaderParameters.HeightVirtualTexturePageTable = GBlackUintTexture->TextureRHI;
	OutShaderParameters.HeightVirtualTexturePageTableIndirection = GBlackUintTexture->TextureRHI;
	OutShaderParameters.HeightVirtualTextureAdaptive = 0;
	OutShaderParameters.HeightVirtualTextureLWCTile = FVector3f::ZeroVector;
	OutShaderParameters.HeightVirtualTextureWorldToUVTransform = FMatrix44f::Identity;
	OutShaderParameters.HeightVirtualTextureEnabled = 0;
	OutShaderParameters.HeightVirtualTexturePackedUniform0 = DummyUint4;
	OutShaderParameters.HeightVirtualTexturePackedUniform1 = DummyUint4;
	OutShaderParameters.HeightVirtualTextureUniforms = DummyUint4;
}

void FPCGLandscapeDataProviderProxy::SetNormalVirtualTextureParameters_Default(FParameters& OutShaderParameters)
{
	const FUintVector4 DummyUint4(ForceInitToZero);

	OutShaderParameters.NormalVirtualTexture0 = GBlackTextureWithSRV->ShaderResourceViewRHI;
	OutShaderParameters.NormalVirtualTexture1 = GBlackTextureWithSRV->ShaderResourceViewRHI;
	OutShaderParameters.NormalVirtualTexturePageTable = GBlackUintTexture->TextureRHI;
	OutShaderParameters.NormalVirtualTexturePageTableIndirection = GBlackUintTexture->TextureRHI;
	OutShaderParameters.NormalVirtualTextureAdaptive = 0;
	OutShaderParameters.NormalVirtualTexturePageTableLayers = FUintVector2(0, 0);
	OutShaderParameters.NormalVirtualTextureLWCTile = FVector3f::ZeroVector;
	OutShaderParameters.NormalVirtualTextureWorldToUVTransform = FMatrix44f::Identity;
	OutShaderParameters.NormalVirtualTextureEnabled = 0;
	OutShaderParameters.NormalVirtualTextureUnpackMode = (int32)PCGVirtualTextureCommon::ENormalUnpackType::None;
	OutShaderParameters.NormalVirtualTexturePackedUniform0 = DummyUint4;
	OutShaderParameters.NormalVirtualTexturePackedUniform1 = DummyUint4;
	OutShaderParameters.NormalVirtualTextureUniforms0 = DummyUint4;
	OutShaderParameters.NormalVirtualTextureUniforms1 = DummyUint4;
}

void FPCGLandscapeDataProviderProxy::SetCollisionHeightTextureParameters_Defaults(FParameters& OutShaderParameters)
{
	const FVector4f DummyVector4 = FVector4f::Zero();

	OutShaderParameters.CollisionHeightTexture = GBlackTexture->TextureRHI;
	OutShaderParameters.CollisionHeightTextureLWCTile = FVector3f::ZeroVector;
	OutShaderParameters.CollisionHeightTextureWorldToUVTransform = FMatrix44f::Identity;
	OutShaderParameters.CollisionHeightTextureUVToWorldTransform = FMatrix44f::Identity;
	OutShaderParameters.CollisionHeightTextureUVScaleBias = DummyVector4;
	OutShaderParameters.CollisionHeightTextureEnabled = 0;
}

#undef LOCTEXT_NAMESPACE
