// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/Data/PCGVirtualTextureCommon.h"
#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "ComputeFramework/ComputeDataProvider.h"

#include "PCGLandscapeDataInterface.generated.h"

class ALandscape;
class FPCGLandscapeDataInterfaceParameters;
class FPCGLandscapeTextureResource;
class URuntimeVirtualTexture;

/** Manages the textures created from landscape height data. */
class FPCGLandscapeResource
{
public:
	struct FResourceKey
	{
		TWeakObjectPtr<const ALandscape> Source = nullptr;
		TArray<FIntPoint> CapturedRegions;
		FIntPoint MinCaptureRegion = FIntPoint(ForceInitToZero);
		FIntPoint MaxCaptureRegion = FIntPoint(ForceInitToZero);
	};

	FPCGLandscapeResource() = default;
	FPCGLandscapeResource(const FResourceKey& InKey);
	~FPCGLandscapeResource();

	FPCGLandscapeTextureResource* LandscapeTexture = nullptr;
	FVector3f LandscapeLWCTile = FVector3f::ZeroVector;
	FMatrix ActorToWorldTransform = FMatrix::Identity;
	FMatrix WorldToActorTransform = FMatrix::Identity;
	FVector4 UVScaleBias = FVector4(1.0f, 1.0f, 0.0f, 0.0f);
	FIntPoint CellCount = FIntPoint(ForceInitToZero);

private:
	FResourceKey ResourceKey;
};

/** Data Interface allowing sampling of a Landscape. */
UCLASS(ClassGroup = (Procedural))
class UPCGLandscapeDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGLandscape"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	TCHAR const* GetShaderVirtualPath() const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading a landscape. */
UCLASS()
class UPCGLandscapeDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

private:
	void InitFromLandscape(UPCGDataBinding* InBinding, ALandscape* InLandscape, const FBox& Bounds, bool bAllowSampleVirtualTexture, bool bAllowSampleVirtualTextureNormal);
	void InitializeRuntimeVirtualTextures(ALandscape* InLandscape, bool bAllowSampleVirtualTextureNormal);
	void InitializeFromLandscapeCollision(UPCGDataBinding* InBinding, ALandscape* InLandscape, const FBox& Bounds);

private:
	UPROPERTY()
	TObjectPtr<const URuntimeVirtualTexture> BaseColorVirtualTexture = nullptr;

	UPROPERTY()
	TObjectPtr<const URuntimeVirtualTexture> HeightVirtualTexture = nullptr;

	UPROPERTY()
	TObjectPtr<const URuntimeVirtualTexture> NormalVirtualTexture = nullptr;

	TSharedPtr<FPCGLandscapeResource> Resource;
	bool bBaseColorSRGB = false;
	PCGVirtualTextureCommon::EBaseColorUnpackType BaseColorVirtualTextureUnpackType = PCGVirtualTextureCommon::EBaseColorUnpackType::None;
	ERuntimeVirtualTextureMaterialType NormalVirtualTextureMode = ERuntimeVirtualTextureMaterialType::Count;
	FVector2D LandscapeGridSize = FVector2D(1.0f, 1.0f);
};

class FPCGLandscapeDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGLandscapeDataProviderProxy(TSharedPtr<const FPCGLandscapeResource> InResource,
		const URuntimeVirtualTexture* InBaseColorVirtualTexture,
		const URuntimeVirtualTexture* InHeightVirtualTexture,
		const URuntimeVirtualTexture* InNormalVirtualTexture,
		bool bBaseColorSRGB,
		PCGVirtualTextureCommon::EBaseColorUnpackType BaseColorVirtualTextureUnpackType,
		ERuntimeVirtualTextureMaterialType NormalVirtualTextureMode,
		FVector2D InWorldGridSize);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGLandscapeDataInterfaceParameters;

	/** Sets all shader parameters. */
	void SetShaderParameters(FParameters& OutShaderParameters);

	/** Set shader parameters for each texture. */
	bool SetBaseColorVirtualTextureParameters(FParameters& OutShaderParameters) const;
	bool SetHeightVirtualTextureParameters(FParameters& OutShaderParameters) const;
	bool SetNormalVirtualTextureParameters(FParameters& OutShaderParameters) const;
	bool SetCollisionHeightTextureParameters(FParameters& OutShaderParameters) const;
	
	/** Set default values for shader parameters. Fallback in case the textures were invalid. */
	static void SetBaseColorVirtualTextureParameters_Default(FParameters& OutShaderParameters);
	static void SetHeightVirtualTextureParameters_Default(FParameters& OutShaderParameters);
	static void SetNormalVirtualTextureParameters_Default(FParameters& OutShaderParameters);
	static void SetCollisionHeightTextureParameters_Defaults(FParameters& OutShaderParameters);

protected:
	TSharedPtr<const FPCGLandscapeResource> Resource;

	TWeakObjectPtr<const URuntimeVirtualTexture> BaseColorVirtualTexture;
	TWeakObjectPtr<const URuntimeVirtualTexture> HeightVirtualTexture;
	TWeakObjectPtr<const URuntimeVirtualTexture> NormalVirtualTexture;

	bool bBaseColorSRGB = false;
	ERuntimeVirtualTextureMaterialType NormalVirtualTextureMode = ERuntimeVirtualTextureMaterialType::Count;

	PCGVirtualTextureCommon::FVirtualTexturePageTable BaseColorVirtualPage;
	PCGVirtualTextureCommon::FVirtualTextureLayer BaseColorVirtualLayer;
	PCGVirtualTextureCommon::EBaseColorUnpackType BaseColorVirtualTextureUnpackType = PCGVirtualTextureCommon::EBaseColorUnpackType::None;
	
	PCGVirtualTextureCommon::FVirtualTexturePageTable HeightVirtualPage;
	PCGVirtualTextureCommon::FVirtualTextureLayer HeightVirtualLayer;
	
	PCGVirtualTextureCommon::FVirtualTexturePageTable NormalVirtualPage;
	PCGVirtualTextureCommon::FVirtualTextureLayer NormalVirtualLayer0;
	PCGVirtualTextureCommon::FVirtualTextureLayer NormalVirtualLayer1;
	PCGVirtualTextureCommon::ENormalUnpackType NormalUnpackMode;
	FUintVector2 NormalPageTableLayers;

	FVector2D LandscapeGridSize = FVector2D(1.0f, 1.0f);
};
