// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGPostRayTraceDataInterface.h"

#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGDataBinding.h"
#include "Elements/PCGWorldRaycast.h"
#include "Helpers/PCGWorldQueryHelpers.h"

#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHIResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPostRayTraceDataInterface)

void UPCGPostRayTraceDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("PostRayTrace_GetTexCoordsAttributeId"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("PostRayTrace_GetNormalsAttributeId"))
		.AddReturnType(EShaderFundamentalType::Int);
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGPostRayTraceDataInterfaceParameters,)
	SHADER_PARAMETER(int32, TexCoordsAttributeId)
	SHADER_PARAMETER(int32, NormalsAttributeId)
END_SHADER_PARAMETER_STRUCT()

void UPCGPostRayTraceDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGPostRayTraceDataInterfaceParameters>(UID);
}

void UPCGPostRayTraceDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	OutHLSL += FString::Format(TEXT(
		"int {DataInterfaceName}_TexCoordsAttributeId;\n"
		"int {DataInterfaceName}_NormalsAttributeId;\n"
		"\n"
		"int PostRayTrace_GetTexCoordsAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_TexCoordsAttributeId;\n"
		"}\n"
		"\n"
		"int PostRayTrace_GetNormalsAttributeId_{DataInterfaceName}()\n"
		"{\n"
		"	return {DataInterfaceName}_NormalsAttributeId;\n"
		"}\n"
	), TemplateArgs);
}

UComputeDataProvider* UPCGPostRayTraceDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGPostRayTraceDataProvider>();
}

FComputeDataProviderRenderProxy* UPCGPostRayTraceDataProvider::GetRenderProxy()
{
	FPCGPostRayTraceProviderProxy::FData ProxyData =
	{
		.TexCoordsAttributeId = TexCoordsAttributeId,
		.NormalsAttributeId = NormalsAttributeId
	};

	return new FPCGPostRayTraceProviderProxy(ProxyData);
}

void UPCGPostRayTraceDataProvider::Reset()
{
	TexCoordsAttributeId = INDEX_NONE;
	NormalsAttributeId = INDEX_NONE;

	Super::Reset();
}

bool UPCGPostRayTraceDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGPostRayTraceDataInterface::PrepareForExecute_GameThread);
	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	const UPCGWorldRaycastElementSettings* Settings = CastChecked<UPCGWorldRaycastElementSettings>(GetProducerKernel()->GetSettings());
	TexCoordsAttributeId = Settings->bGetTextureCoordinates ? InBinding->GetAttributeId(PCGWorldQueryConstants::UVCoordAttribute, EPCGKernelAttributeType::Float2) : INDEX_NONE;
	NormalsAttributeId = Settings->bGetNormals ? InBinding->GetAttributeId(PCGWorldQueryConstants::ImpactNormalAttribute, EPCGKernelAttributeType::Float3) : INDEX_NONE;

	return true;
}

bool FPCGPostRayTraceProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGPostRayTraceProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.TexCoordsAttributeId = Data.TexCoordsAttributeId;
		Parameters.NormalsAttributeId = Data.NormalsAttributeId;
	}
}
