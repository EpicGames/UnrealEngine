// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "PCGPostRayTraceDataInterface.generated.h"

class FPCGPostRayTraceDataInterfaceParameters;

/** Data Interface to marshal post ray trace settings to the GPU. */
UCLASS(ClassGroup = (Procedural))
class UPCGPostRayTraceDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGPostRayTrace"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface
};

UCLASS()
class UPCGPostRayTraceDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	//~ Begin UPCGComputeDataProvider Interface
	virtual void Reset() override;
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

public:
	int32 TexCoordsAttributeId = INDEX_NONE;
	int32 NormalsAttributeId = INDEX_NONE;
};

class FPCGPostRayTraceProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FData
	{
		int32 TexCoordsAttributeId = 0;
		int32 NormalsAttributeId = 0;
	};

	FPCGPostRayTraceProviderProxy(const FData& InData)
		: Data(InData)
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGPostRayTraceDataInterfaceParameters;

	FData Data;
};
