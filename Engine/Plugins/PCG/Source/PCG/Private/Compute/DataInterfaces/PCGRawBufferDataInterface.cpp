// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/DataInterfaces/PCGRawBufferDataInterface.h"

#include "PCGCommon.h"
#include "PCGModule.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/Data/PCGRawBufferData.h"

#include "RenderGraphBuilder.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ShaderParameterStruct.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGRawBufferDataInterface)

namespace PCGRawBufferDataInterface
{
	static TCHAR const* TemplateFilePath = TEXT("/Plugin/PCG/Private/PCGRawBufferDataInterface.ush");

	/** Computes the size (in bytes) of the data description after packing. */
	uint64 ComputePackedSizeBytes(const FPCGDataDesc& InDataDesc)
	{
		return InDataDesc.GetElementCount().X * sizeof(uint32);
	}

	/** Computes the size (in bytes) of the data collection after packing. */
	uint64 ComputePackedSizeBytes(TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGRawBufferPacking::ComputePackedSizeBytes);

		uint64 TotalDataSizeBytes = 0;

		if (InDataCollectionDesc)
		{
			for (const FPCGDataDesc& DataDesc : InDataCollectionDesc->GetDataDescriptions())
			{
				TotalDataSizeBytes += ComputePackedSizeBytes(DataDesc);
			}
		}
	
		return TotalDataSizeBytes;
	}

	/** Adds function definitions common to both SRV (input) and UAV (output) views. */
	static void AddCommonFunctions(TArray<FShaderFunctionDefinition>& OutFunctions)
	{
		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetNumElements"))
			.AddReturnType(EShaderFundamentalType::Uint);
	}
}

void UPCGRawBufferDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
#if WITH_EDITOR
	PCGRawBufferDataInterface::AddCommonFunctions(OutFunctions);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(PCGComputeHelpers::GetSRVFunction(TEXT("Load")))
		.AddParam(EShaderFundamentalType::Uint) // InElementIndex
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(PCGComputeHelpers::GetSRVFunction(TEXT("Load4")))
		.AddParam(EShaderFundamentalType::Uint) // InFirstElementIndex
		.AddReturnType(EShaderFundamentalType::Uint, 4);
#endif // WITH_EDITOR
}

void UPCGRawBufferDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
#if WITH_EDITOR
	PCGRawBufferDataInterface::AddCommonFunctions(OutFunctions);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(PCGComputeHelpers::GetUAVFunction(TEXT("Load")))
		.AddParam(EShaderFundamentalType::Uint) // InElementIndex
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(PCGComputeHelpers::GetUAVFunction(TEXT("Load4")))
		.AddParam(EShaderFundamentalType::Uint) // InFirstElementIndex
		.AddReturnType(EShaderFundamentalType::Uint, 4);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(PCGComputeHelpers::GetUAVFunction(TEXT("Store")))
		.AddParam(EShaderFundamentalType::Uint) // InElementIndex
		.AddParam(EShaderFundamentalType::Uint); // InValue

	OutFunctions.AddDefaulted_GetRef()
		.SetName(PCGComputeHelpers::GetUAVFunction(TEXT("Store4")))
		.AddParam(EShaderFundamentalType::Uint) // InFirstElementIndex
		.AddParam(EShaderFundamentalType::Uint, 4); // InValue

	OutFunctions.AddDefaulted_GetRef()
		.SetName(PCGComputeHelpers::GetUAVFunction(TEXT("AtomicAdd")))
		.AddParam(EShaderFundamentalType::Uint) // InElementIndex
		.AddParam(EShaderFundamentalType::Uint) // InValueToAdd
		.AddReturnType(EShaderFundamentalType::Uint); // Value before add

	OutFunctions.AddDefaulted_GetRef()
		.SetName(PCGComputeHelpers::GetUAVFunction(TEXT("AtomicMax")))
		.AddParam(EShaderFundamentalType::Uint) // InElementIndex
		.AddParam(EShaderFundamentalType::Uint) // InValueToMax
		.AddReturnType(EShaderFundamentalType::Uint); // Value before max
#endif // WITH_EDITOR
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGRawBufferDataInterfaceParameters,)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, DataSRV)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, DataUAV)
	SHADER_PARAMETER(uint32, SizeBytes)
END_SHADER_PARAMETER_STRUCT()

void UPCGRawBufferDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGRawBufferDataInterfaceParameters>(UID);
}

TCHAR const* UPCGRawBufferDataInterface::GetShaderVirtualPath() const
{
	return PCGRawBufferDataInterface::TemplateFilePath;
}

void UPCGRawBufferDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(PCGRawBufferDataInterface::TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UPCGRawBufferDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
#if WITH_EDITOR
	FString TemplateFile;
	if (!ensure(LoadShaderSourceFile(PCGRawBufferDataInterface::TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr)))
	{
		return;
	}

	// Expand the {{BeginDuplicateForSRVUAV}} / {{EndDuplicateForSRVUAV}} region into parallel SRV and UAV variants.
	const FString ExpandedTemplate = PCGComputeHelpers::ExpandShaderTemplateForSRVUAV(TemplateFile);

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};
	OutHLSL += FString::Format(*ExpandedTemplate, TemplateArgs);
#endif // WITH_EDITOR
}

UComputeDataProvider* UPCGRawBufferDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGRawBufferDataProvider>();
}

void UPCGRawBufferDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGRawBufferDataInterface* DataInterface = CastChecked<UPCGRawBufferDataInterface>(InDataInterface);
	bZeroInitialize = DataInterface->GetRequiresZeroInitialization();
}

bool UPCGRawBufferDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	if (const TSharedPtr<const FPCGDataCollectionDesc> ProducerPinDesc = InBinding->GetCachedKernelPinDataDesc(GetGraphBindingIndex()); ensure(ProducerPinDesc))
	{
		const uint64 PackedSizeBytes = PCGRawBufferDataInterface::ComputePackedSizeBytes(ProducerPinDesc);

		if (!PCGComputeHelpers::IsBufferSizeTooLarge(PackedSizeBytes))
		{
			SizeBytes = static_cast<uint32>(PackedSizeBytes);
		}
	}

	if (IsProducedByCPU())
	{
		check(!GetDownstreamInputPinLabelAliases().IsEmpty());

		const TArray<FPCGTaggedData> TaggedDatas = InBinding->GetInputDataCollection().GetInputsByPin(GetDownstreamInputPinLabelAliases()[0]);

		if (!TaggedDatas.IsEmpty())
		{
			ensureMsgf(TaggedDatas.Num() == 1, TEXT("UPCGRawBufferDataProvider: Multi-data is not currently supported (received %d data items)."), TaggedDatas.Num());

			DataToUpload = Cast<UPCGRawBufferData>(TaggedDatas[0].Data);
		}
	}

	ReadbackState = MakeShared<FReadbackState>();

	return true;
}

FComputeDataProviderRenderProxy* UPCGRawBufferDataProvider::GetRenderProxy()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGRawBufferDataProvider::GetRenderProxy);

	TWeakObjectPtr<UPCGRawBufferDataProvider> ThisWeak = MakeWeakObjectPtr(this);

	FPCGRawBufferDataProviderProxy::FParams Params =
	{
		.SizeBytes = SizeBytes,
		.bZeroInitialize = bZeroInitialize,
		.DataToUpload = DataToUpload,
		.ExportMode = GetExportMode(),
		.OutputPinLabel = GetOutputPinLabel(),
		.OutputPinLabelAlias = GetOutputPinLabelAlias(),
		.DataProviderWeakPtr = ThisWeak,
	};

	ensure(ReadbackState);

	TSharedPtr<const FPCGDataCollectionDesc> PinDesc = GetPinDescription();
	ensure(PinDesc);

	Params.AsyncReadbackCallback_RenderThread = [ReadbackState=ReadbackState, DataProviderWeakPtr=ThisWeak, GenerationCount=GetGenerationCounter(), ExportMode=GetExportMode(), OutputPinLabel=GetOutputPinLabel(), OutputPinLabelAlias = GetOutputPinLabelAlias(), BindingWeakPtr=TWeakObjectPtr<UPCGDataBinding>(GetDataBinding()), PinDesc](const void* InData, int InNumBytes)
	{
		if (!InData || !ReadbackState || !PinDesc || PinDesc->GetDataDescriptions().IsEmpty())
		{
			return;
		}

		LLM_SCOPE_BYTAG(PCG);

		ReadbackState->Data.Append(static_cast<const uint32*>(InData), InNumBytes / sizeof(uint32));

		ExecuteOnGameThread(UE_SOURCE_LOCATION, [ReadbackState, DataProviderWeakPtr, GenerationCount, ExportMode, OutputPinLabel, OutputPinLabelAlias, BindingWeakPtr, PinDesc]()
		{
			LLM_SCOPE_BYTAG(PCG);

			UPCGDataBinding* Binding = BindingWeakPtr.Get();
			if (!Binding)
			{
				return;
			}

			UPCGRawBufferDataProvider* DataProvider = DataProviderWeakPtr.Get();
			if (!DataProvider)
			{
				UE_LOGF(LogPCG, Error, "Could not resolve UPCGRawBufferDataProvider pointer to pass back buffer handle.");
				return;
			}

			if (DataProvider->GetGenerationCounter() != GenerationCount)
			{
				// Data provider was returned to the pool, possibly due to cancellation. Nothing to do here.
				return;
			}

			UPCGRawBufferData* RawBufferData = NewObject<UPCGRawBufferData>();
			RawBufferData->SetData(MoveTemp(ReadbackState->Data));

			Binding->ReceiveDataFromGPU_GameThread(RawBufferData, DataProvider->GetProducerSettings(), ExportMode, OutputPinLabel, OutputPinLabelAlias, PinDesc->GetDataDescriptions()[0]);

			DataProvider->OnDataExported_GameThread().Broadcast();
		});
	};

	return new FPCGRawBufferDataProviderProxy(Params);
}

void UPCGRawBufferDataProvider::Reset()
{
	Super::Reset();

	SizeBytes = INDEX_NONE;
	bZeroInitialize = false;
	DataToUpload = nullptr;
	ReadbackState.Reset();
}

FPCGRawBufferDataProviderProxy::FPCGRawBufferDataProviderProxy(const FParams& InParams)
	: bZeroInitialize(InParams.bZeroInitialize)
	, ExportMode(InParams.ExportMode)
	, OutputPinLabel(InParams.OutputPinLabel)
	, OutputPinLabelAlias(InParams.OutputPinLabelAlias)
	, DataProviderWeakPtr(InParams.DataProviderWeakPtr)
	, AsyncReadbackCallback_RenderThread(InParams.AsyncReadbackCallback_RenderThread)
{
	// Ensure we will create at least a minimal valid resource (4 bytes).
	SizeBytes = FMath::Max(4, InParams.SizeBytes);

	if (InParams.DataToUpload)
	{
		DataToUpload.Append(InParams.DataToUpload->GetConstData());
	}
}

bool FPCGRawBufferDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if ((SizeBytes % sizeof(uint32)) != 0)
	{
		UE_LOGF(LogPCG, Warning, "FPCGRawBufferDataProviderProxy not valid due to invalid size: %d bytes.", SizeBytes);
		return false;
	}

	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGRawBufferDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	FRDGBufferDesc Desc = FRDGBufferDesc::CreateByteAddressDesc(SizeBytes);

	if (ExportMode != EPCGExportMode::NoExport)
	{
		// We don't know for sure whether buffer will be read back or not, so need to flag the possibility if the buffer will be passed downstream.
		Desc.Usage |= BUF_SourceCopy;
	}

	Data = GraphBuilder.CreateBuffer(Desc, TEXT("PCGRawBuffer"));
	DataSRV = GraphBuilder.CreateSRV(Data);
	DataUAV = GraphBuilder.CreateUAV(Data);

	if (!DataToUpload.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRawBufferDataProviderProxy::AllocateResources::UploadData);
		GraphBuilder.QueueBufferUpload(Data, MakeConstArrayView(DataToUpload));
	}
	else if (bZeroInitialize)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRawBufferDataProviderProxy::AllocateResources::UploadZeroes);
		DataToUpload.SetNumZeroed(SizeBytes / sizeof(uint32));
		GraphBuilder.QueueBufferUpload(Data, MakeConstArrayView(DataToUpload));
	}
#if !UE_BUILD_SHIPPING
	else if (PCGSystemSwitches::CVarFuzzGPUMemory.GetValueOnAnyThread())
	{
		DataToUpload.SetNumUninitialized(SizeBytes / sizeof(uint32));

		FRandomStream Rand(GFrameCounter);
		for (uint32& Value : DataToUpload)
		{
			Value = Rand.GetUnsignedInt();
		}

		GraphBuilder.QueueBufferUpload(Data, MakeConstArrayView(DataToUpload));
	}
#endif
}

void FPCGRawBufferDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		// Both views target the same physical buffer. ValidateShaderParameters requires every slot filled; shader
		// DCE drops whichever view the shader doesn't reference, so the unused side consumes no GPU register.
		Parameters.DataSRV = DataSRV;
		Parameters.DataUAV = DataUAV;
		Parameters.SizeBytes = SizeBytes;
	}
}

void FPCGRawBufferDataProviderProxy::GetReadbackData(TArray<FReadbackData>& OutReadbackData) const
{
	if (ExportMode != EPCGExportMode::NoExport)
	{
		FReadbackData ReadbackData;
		ReadbackData.Buffer = Data;
		ReadbackData.NumBytes = SizeBytes;
		ReadbackData.ReadbackCallback_RenderThread = &AsyncReadbackCallback_RenderThread;

		OutReadbackData.Add(MoveTemp(ReadbackData));
	}
}
