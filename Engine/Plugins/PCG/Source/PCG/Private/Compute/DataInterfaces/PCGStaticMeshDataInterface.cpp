// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGStaticMeshDataInterface.h"

#include "PCGData.h"
#include "PCGModule.h"
#include "Compute/PCGDataBinding.h"
#include "Data/PCGStaticMeshResourceData.h"

#include "RHIResources.h"
#include "RenderGraphBuilder.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "StaticMeshResources.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGStaticMeshDataInterface)

#define LOCTEXT_NAMESPACE "PCGStaticMeshDataInterface"

void UPCGStaticMeshDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	// Vertex functions
	{
		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetNumVertices"))
			.AddReturnType(EShaderFundamentalType::Int)
			.AddParam(EShaderFundamentalType::Int); // DataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetVertex"))
			.AddParam(EShaderFundamentalType::Int) // DataIndex
			.AddParam(EShaderFundamentalType::Int) // VertexIndex
			.AddParam(EShaderFundamentalType::Float, 3, 0, EShaderParamModifier::Out) // OutPosition
			.AddParam(EShaderFundamentalType::Float, 3, 0, EShaderParamModifier::Out) // OutNormal
			.AddParam(EShaderFundamentalType::Float, 3, 0, EShaderParamModifier::Out) // OutTangent
			.AddParam(EShaderFundamentalType::Float, 3, 0, EShaderParamModifier::Out); // OutBitangent

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetVertexColor"))
			.AddReturnType(EShaderFundamentalType::Float, 4)
			.AddParam(EShaderFundamentalType::Int) // DataIndex
			.AddParam(EShaderFundamentalType::Int); // VertexIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetVertexUV"))
			.AddReturnType(EShaderFundamentalType::Float, 2)
			.AddParam(EShaderFundamentalType::Int) // DataIndex
			.AddParam(EShaderFundamentalType::Int) // VertexIndex
			.AddParam(EShaderFundamentalType::Int); // UVSet
	}

	// Triangle functions
	{
		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetNumTriangles"))
			.AddReturnType(EShaderFundamentalType::Int)
			.AddParam(EShaderFundamentalType::Int); // DataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetTriangleIndices"))
			.AddParam(EShaderFundamentalType::Int) // DataIndex
			.AddParam(EShaderFundamentalType::Int) // TriangleIndex
			.AddParam(EShaderFundamentalType::Int, 0, 0, EShaderParamModifier::Out) // OutIndex0
			.AddParam(EShaderFundamentalType::Int, 0, 0, EShaderParamModifier::Out) // OutIndex1
			.AddParam(EShaderFundamentalType::Int, 0, 0, EShaderParamModifier::Out); // OutIndex2

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("SampleTriangle"))
			.AddParam(EShaderFundamentalType::Int) // DataIndex
			.AddParam(EShaderFundamentalType::Int) // TriangleIndex
			.AddParam(EShaderFundamentalType::Float, 3) // BaryCoord
			.AddParam(EShaderFundamentalType::Float, 3, 0, EShaderParamModifier::Out) // OutPosition
			.AddParam(EShaderFundamentalType::Float, 3, 0, EShaderParamModifier::Out) // OutNormal
			.AddParam(EShaderFundamentalType::Float, 3, 0, EShaderParamModifier::Out) // OutTangent
			.AddParam(EShaderFundamentalType::Float, 3, 0, EShaderParamModifier::Out); // OutBitangent

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("SampleTriangleColor"))
			.AddReturnType(EShaderFundamentalType::Float, 4)
			.AddParam(EShaderFundamentalType::Int) // DataIndex
			.AddParam(EShaderFundamentalType::Int) // TriangleIndex
			.AddParam(EShaderFundamentalType::Float, 3); // BaryCoord

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("SampleTriangleUV"))
			.AddReturnType(EShaderFundamentalType::Float, 2)
			.AddParam(EShaderFundamentalType::Int) // DataIndex
			.AddParam(EShaderFundamentalType::Int) // TriangleIndex
			.AddParam(EShaderFundamentalType::Float, 3) // BaryCoord
			.AddParam(EShaderFundamentalType::Int); // UVSet
	}

	// Misc functions
	{
		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetMeshBoundsExtents"))
			.AddReturnType(EShaderFundamentalType::Float, 3)
			.AddParam(EShaderFundamentalType::Int); // DataIndex
	}
}

// Reference NiagaraDataInterfaceStaticMesh.h
BEGIN_SHADER_PARAMETER_STRUCT(FPCGStaticMeshDataInterfaceParameters,)
	SHADER_PARAMETER(int, NumVertices)
	SHADER_PARAMETER(int, NumTriangles)
	SHADER_PARAMETER(int, NumUVs)
	SHADER_PARAMETER(uint32, HasColors)
	SHADER_PARAMETER_SRV(Buffer<uint>, IndexBuffer)
	SHADER_PARAMETER_SRV(Buffer<float>, PositionBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, TangentBuffer)
	SHADER_PARAMETER_SRV(Buffer<float2>, UVBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, ColorBuffer)
	SHADER_PARAMETER(FVector3f, BoundsExtents)
END_SHADER_PARAMETER_STRUCT()

void UPCGStaticMeshDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGStaticMeshDataInterfaceParameters>(UID);
}

TCHAR const* UPCGStaticMeshDataInterface::TemplateFilePath = TEXT("/Plugin/PCG/Private/PCGStaticMeshDataInterface.ush");

TCHAR const* UPCGStaticMeshDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UPCGStaticMeshDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UPCGStaticMeshDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	if (ensure(LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr)))
	{
		OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
	}
}

UComputeDataProvider* UPCGStaticMeshDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGStaticMeshDataProvider>();;
}

bool UPCGStaticMeshDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGStaticMeshDataProvider::PrepareForExecute_GameThread);

	if (!LoadHandle.IsValid())
	{
		// Take any input pin label alias to obtain the data from the input data collection.
		check(!GetDownstreamInputPinLabelAliases().IsEmpty());

		const UPCGDataBinding* Binding = CastChecked<UPCGDataBinding>(InBinding);
		const TArray<FPCGTaggedData> TaggedDatas = Binding->GetInputDataCollection().GetInputsByPin(GetDownstreamInputPinLabelAliases()[0]);

		if (!TaggedDatas.IsEmpty())
		{
			const UPCGStaticMeshResourceData* ResourceData = Cast<UPCGStaticMeshResourceData>(TaggedDatas[0].Data);
			ensure(TaggedDatas.Num() == 1); // There should only be one static mesh data

			if (ensure(ResourceData))
			{
				LoadHandle = ResourceData->RequestResourceLoad();
			}
		}
	}

	if (!LoadHandle.IsValid())
	{
		return true;
	}

	if (LoadHandle->HasLoadCompleted())
	{
		LoadedStaticMesh = Cast<UStaticMesh>(LoadHandle->GetLoadedAsset());
		return true;
	}

	return false;
}

FComputeDataProviderRenderProxy* UPCGStaticMeshDataProvider::GetRenderProxy()
{
	return new FPCGStaticMeshDataProviderProxy(LoadedStaticMesh.Get());
}

void UPCGStaticMeshDataProvider::Reset()
{
	LoadedStaticMesh = nullptr;
	LoadHandle.Reset();

	Super::Reset();
}

FPCGStaticMeshDataProviderProxy::FPCGStaticMeshDataProviderProxy(const UStaticMesh* InStaticMesh)
{
	const FStaticMeshRenderData* RenderData = InStaticMesh ? InStaticMesh->GetRenderData() : nullptr;

	if (RenderData && !RenderData->LODResources.IsEmpty())
	{
		MeshName = InStaticMesh->GetFName();
		LODResources = &RenderData->LODResources[0];
		Bounds = RenderData->Bounds;
	}
}

FPCGStaticMeshDataProviderProxy::~FPCGStaticMeshDataProviderProxy()
{
	IndexBufferSRV.SafeRelease();
}

bool FPCGStaticMeshDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters)
		&& LODResources;
}

void FPCGStaticMeshDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	check(LODResources);

	const FStaticMeshVertexBuffers& VertexBuffers = LODResources->VertexBuffers;
	FShaderResourceViewRHIRef PositionBufferSRV = VertexBuffers.PositionVertexBuffer.GetSRV();
	FShaderResourceViewRHIRef TangentBufferSRV = VertexBuffers.StaticMeshVertexBuffer.GetTangentsSRV();
	FShaderResourceViewRHIRef UVBufferSRV = VertexBuffers.StaticMeshVertexBuffer.GetTexCoordsSRV();
	FShaderResourceViewRHIRef ColorBufferSRV = VertexBuffers.ColorVertexBuffer.GetColorComponentsSRV();

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumTriangles = IndexBufferSRV.IsValid() ? LODResources->IndexBuffer.GetNumIndices() / 3 : 0;
		Parameters.NumVertices = PositionBufferSRV.IsValid() ? VertexBuffers.PositionVertexBuffer.GetNumVertices() : 0;
		Parameters.NumUVs = UVBufferSRV.IsValid() ? VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() : 0;
		Parameters.HasColors = ColorBufferSRV.IsValid();
		Parameters.IndexBuffer = IndexBufferSRV;
		Parameters.PositionBuffer = PositionBufferSRV;
		Parameters.TangentBuffer = TangentBufferSRV.IsValid() ? TangentBufferSRV : static_cast<FShaderResourceViewRHIRef>(PCGComputeDummies::GetDummyFloat4Buffer());
		Parameters.UVBuffer = UVBufferSRV.IsValid() ? UVBufferSRV : static_cast<FShaderResourceViewRHIRef>(PCGComputeDummies::GetDummyFloat2Buffer());
		Parameters.ColorBuffer = ColorBufferSRV.IsValid() ? ColorBufferSRV : static_cast<FShaderResourceViewRHIRef>(PCGComputeDummies::GetDummyFloat4Buffer());
		Parameters.BoundsExtents = static_cast<FVector3f>(Bounds.BoxExtent);
	}
}

void FPCGStaticMeshDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	LLM_SCOPE_BYTAG(PCG);

	FBufferRHIRef IndexBufferRHIRef = LODResources ? LODResources->IndexBuffer.IndexBufferRHI : nullptr;
	const bool bAllowCPUAccess = LODResources? LODResources->IndexBuffer.GetAllowCPUAccess() : false;
	const bool bCanCreateIndexSRV =
		IndexBufferRHIRef.IsValid() &&
		((IndexBufferRHIRef->GetUsage() & EBufferUsageFlags::ShaderResource) == EBufferUsageFlags::ShaderResource) &&
		// Added just to keep old behavior, where whether the SRV should be created has been inferred from whether GPU Skin cache is enabled or not
		// Typically if GPU Skin Cache is enabled, the platform likely does not care about memory cost from extra SRVs
		// Individual systems can further decide what to do on platforms that don't support GPU Skin Cache
		(AreBufferSRVsAlwaysCreatedByDefault(GMaxRHIShaderPlatform) || bAllowCPUAccess); 

	if (bCanCreateIndexSRV)
	{
		IndexBufferSRV = GraphBuilder.RHICmdList.CreateShaderResourceView(IndexBufferRHIRef,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(LODResources->IndexBuffer.Is32Bit() ? PF_R32_UINT : PF_R16_UINT));
	}
	else
	{
		UE_LOGF(LogPCG, Error,
			"PCGStaticMeshDataInterface used by PCG Graph but does not have SRV access on this platform. Mesh: '%ls'.",
			*MeshName.ToString());
	}
}

#undef LOCTEXT_NAMESPACE
