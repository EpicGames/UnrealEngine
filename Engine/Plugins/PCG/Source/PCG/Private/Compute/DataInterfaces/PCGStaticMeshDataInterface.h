// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "Engine/StreamableManager.h"

#include "PCGStaticMeshDataInterface.generated.h"

class FPCGStaticMeshDataInterfaceParameters;
class UPCGStaticMeshResourceData;

class FRHIShaderResourceView;
class UStaticMesh;
struct FStaticMeshLODResources;

/** Data Interface allowing sampling of a static mesh. */
UCLASS(ClassGroup = (Procedural))
class UPCGStaticMeshDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGStaticMesh"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	TCHAR const* GetShaderVirtualPath() const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading a static mesh. */
UCLASS()
class UPCGStaticMeshDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface
	
	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	UPROPERTY()
	TObjectPtr<const UStaticMesh> LoadedStaticMesh = nullptr;

	TSharedPtr<FStreamableHandle> LoadHandle = nullptr;
};

class FPCGStaticMeshDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGStaticMeshDataProviderProxy(const UStaticMesh* InStaticMesh);
	virtual ~FPCGStaticMeshDataProviderProxy();

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGStaticMeshDataInterfaceParameters;

	FName MeshName;
	TRefCountPtr<const FStaticMeshLODResources> LODResources;
	FBoxSphereBounds Bounds = FBoxSphereBounds(EForceInit::ForceInit);
	TRefCountPtr<FRHIShaderResourceView> IndexBufferSRV;
};
