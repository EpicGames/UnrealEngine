// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTexture2DArrayDataInterface.h"

#include "PCGContext.h"
#include "Compute/PCGDataBinding.h"
#include "Data/PCGTexture2DArrayData.h"

#include "RHIStaticStates.h"
#include "RenderGraphBuilder.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SystemTextures.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGTexture2DArrayDataInterface)

#define LOCTEXT_NAMESPACE "PCGTexture2DArrayDataInterface"

namespace PCGTexture2DArrayDataInterfaceHelpers
{
	void GetSharedFunctions(TArray<FShaderFunctionDefinition>& OutFunctions)
	{
		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetNumData"))
			.AddReturnType(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetNumElements"))
			.AddReturnType(EShaderFundamentalType::Uint, 3)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetResolution"))
			.AddReturnType(EShaderFundamentalType::Uint, 2)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetArraySize"))
			.AddReturnType(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetNumMips"))
			.AddReturnType(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetThreadData"))
			.AddReturnType(EShaderFundamentalType::Bool)
			.AddParam(EShaderFundamentalType::Uint) // InThreadIndex
			.AddParam(EShaderFundamentalType::Uint, 0, 0, EShaderParamModifier::Out) // OutDataIndex
			.AddParam(EShaderFundamentalType::Uint, 3, 0, EShaderParamModifier::Out); // OutElementIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetTexCoords"))
			.AddReturnType(EShaderFundamentalType::Float, 2)
			.AddParam(EShaderFundamentalType::Uint) // InDataIndex
			.AddParam(EShaderFundamentalType::Float, 3); // InWorldPos

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetTexelSize"))
			.AddReturnType(EShaderFundamentalType::Float, 2)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetTexelSizeWorld"))
			.AddReturnType(EShaderFundamentalType::Float, 2)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetTransform"))
			.AddReturnType(EShaderFundamentalType::Float, 4, 4)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex
	}
}

void UPCGTexture2DArrayDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	PCGTexture2DArrayDataInterfaceHelpers::GetSharedFunctions(OutFunctions);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("Sample"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Float, 2) // InTextureUV
		.AddParam(EShaderFundamentalType::Uint); // InSliceIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SampleWorldPos"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Float, 3) // WorldPos
		.AddParam(EShaderFundamentalType::Uint); // InSliceIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("Load"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Uint, 3); // InElementIndex = (TexelIndex, SliceIndex)

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("Load"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Uint, 3) // InElementIndex = (TexelIndex, SliceIndex)
		.AddParam(EShaderFundamentalType::Uint); // InMipIndex
}

void UPCGTexture2DArrayDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	PCGTexture2DArrayDataInterfaceHelpers::GetSharedFunctions(OutFunctions);

	// No override to select a mip as that is not supported in HLSL.
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("Store"))
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Uint, 3) // InElementIndex = (TexelIndex, SliceIndex)
		.AddParam(EShaderFundamentalType::Float, 4); // InValue
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGTexture2DArrayDataInterfaceParameters, )
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray, TextureSRV)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, TextureUAV)
	SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
	SHADER_PARAMETER(FUintVector2, Resolution)
	SHADER_PARAMETER(uint32, ArraySize)
	SHADER_PARAMETER(uint32, NumMips)
	SHADER_PARAMETER(FMatrix44f, TextureTransform)
	SHADER_PARAMETER(FMatrix44f, TextureInverseTransform)
	SHADER_PARAMETER(FVector2f, TexelSizeWorld)
END_SHADER_PARAMETER_STRUCT()

void UPCGTexture2DArrayDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGTexture2DArrayDataInterfaceParameters>(UID);
}

TCHAR const* UPCGTexture2DArrayDataInterface::TemplateFilePath = TEXT("/Plugin/PCG/Private/PCGTexture2DArrayDataInterface.ush");

TCHAR const* UPCGTexture2DArrayDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UPCGTexture2DArrayDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UPCGTexture2DArrayDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	if (ensure(LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr)))
	{
		OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
	}
}

UComputeDataProvider* UPCGTexture2DArrayDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGTexture2DArrayDataProvider>();
}

void UPCGTexture2DArrayDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGTexture2DArrayDataInterface* DataInterface = CastChecked<UPCGTexture2DArrayDataInterface>(InDataInterface);
	bInitializeFromDataCollection = DataInterface->GetInitializeFromDataCollection();
}

FComputeDataProviderRenderProxy* UPCGTexture2DArrayDataProvider::GetRenderProxy()
{
	FPCGTexture2DArrayDataProviderProxy::FData Data;
	Data.BindingInfo = BindingInfo;
	Data.ExportMode = GetExportMode();
	Data.PinDesc = GetPinDescription();
	Data.OutputPinLabel = GetOutputPinLabel();
	Data.OutputPinLabelAlias = GetOutputPinLabelAlias();
	Data.OriginatingGenerationCount = GetGenerationCounter();
	Data.WeakDataProvider_GT = this;

	return new FPCGTexture2DArrayDataProviderProxy(Data);
}

void UPCGTexture2DArrayDataProvider::Reset()
{
	Super::Reset();

	bInitializeFromDataCollection = false;
	BindingInfo = {};
}

bool UPCGTexture2DArrayDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGTexture2DArrayDataProvider::PrepareForExecute_GameThread);

	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	if (GetInitializeFromDataCollection())
	{
		InitializeFromDataCollection(InBinding);
	}
	else
	{
		InitializeFromDataDescription(InBinding);
	}

	return true;
}

void UPCGTexture2DArrayDataProvider::InitializeFromDataCollection(UPCGDataBinding* InBinding)
{
	check(InBinding);

	// Take any input pin label alias to obtain the data from the input data collection.
	check(!GetDownstreamInputPinLabelAliases().IsEmpty());
	const TArray<FPCGTaggedData> InputTaggedData = InBinding->GetInputDataCollection().GetInputsByPin(GetDownstreamInputPinLabelAliases()[0]);

	if (InputTaggedData.IsEmpty())
	{
		UE_LOGF(LogPCG, Warning, "UPCGTexture2DArrayDataProvider::InitializeFromDataCollection: No texture data found on pin '%ls' - upstream node may not have produced output.", *GetDownstreamInputPinLabelAliases()[0].ToString());
	}
	else
	{
		const FPCGTaggedData& TaggedData = InputTaggedData[0];
		const UPCGTexture2DArrayData* TextureData = Cast<UPCGTexture2DArrayData>(TaggedData.Data);

		if (ensure(TextureData))
		{
			BindingInfo = FPCGTextureBindingInfo(TextureData);
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Unsupported data type encountered by Texture2DArray data interface: '%ls'", TaggedData.Data ? *TaggedData.Data->GetName() : TEXT("NULL"));
		}
	}
}

void UPCGTexture2DArrayDataProvider::InitializeFromDataDescription(UPCGDataBinding* InBinding)
{
	check(InBinding);

	TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
	FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr;

	if (!ensure(Context))
	{
		return;
	}

	TSharedPtr<const FPCGDataCollectionDesc> PinDesc = GetPinDescription();

	if (ensure(PinDesc) && !PinDesc->GetDataDescriptions().IsEmpty())
	{
		BindingInfo = FPCGTextureBindingInfo(PinDesc->GetDataDescriptions()[0]);
	}
}

bool FPCGTexture2DArrayDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	FString* ReasonPtr = nullptr;
#if !NO_LOGGING
	FString Reason;
	ReasonPtr = &Reason;
#endif

	if (!Data.BindingInfo.IsValid(ReasonPtr))
	{
		UE_LOGF(LogPCG, Warning, "FPCGTexture2DArrayDataProviderProxy: Invalid due to invalid texture binding.");
		if (ReasonPtr)
		{
			UE_LOGF(LogPCG, Warning, "FPCGTexture2DArrayDataProviderProxy: %ls.", **ReasonPtr);
		}

		return false;
	}

	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGTexture2DArrayDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const FPCGTextureBindingInfo& BindingInfo = Data.BindingInfo;
	const FMatrix44f Transform = FMatrix44f(BindingInfo.Transform.ToMatrixWithScale()).GetTransposed();
	const FMatrix44f InverseTransform = !FMath::IsNearlyZero(BindingInfo.Transform.GetDeterminant()) ? FMatrix44f(BindingInfo.Transform.ToInverseMatrixWithScale().GetTransposed()) : FMatrix44f::Identity;
	const FVector& TextureWorldScale = BindingInfo.Transform.GetScale3D();
	const float TexelSizeWorldX = BindingInfo.Size.X != 0 ? (TextureWorldScale.X * 2.0) / BindingInfo.Size.X : 0.0f;
	const float TexelSizeWorldY = BindingInfo.Size.Y != 0 ? (TextureWorldScale.Y * 2.0) / BindingInfo.Size.Y : 0.0f;

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];

		Parameters.TextureSRV = TextureSRV;
		Parameters.TextureUAV = TextureUAV;
		Parameters.Sampler = BindingInfo.Filter == EPCGTextureFilter::Point ? TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI() : TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Parameters.Resolution = FUintVector2(BindingInfo.Size.X, BindingInfo.Size.Y);
		Parameters.ArraySize = static_cast<uint32>(BindingInfo.ArraySize);
		Parameters.NumMips = static_cast<uint32>(BindingInfo.NumMips);
		Parameters.TextureTransform = Transform;
		Parameters.TextureInverseTransform = InverseTransform;
		Parameters.TexelSizeWorld.X = TexelSizeWorldX;
		Parameters.TexelSizeWorld.Y = TexelSizeWorldY;
	}
}

void FPCGTexture2DArrayDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	LLM_SCOPE_BYTAG(PCG);

	CreateDefaultTextures(GraphBuilder);
	CreateTextures(GraphBuilder);
}

void FPCGTexture2DArrayDataProviderProxy::CreateDefaultTextures(FRDGBuilder& GraphBuilder)
{
	FRDGTextureRef DummyTextureSRV = GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture2DArray, EPixelFormat::PF_G8, 0.0f);
	FRDGTextureRef DummyTextureUAV = GraphBuilder.CreateTexture(
		FRDGTextureDesc::Create2DArray(FIntPoint(1, 1), PF_G8, FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV, /*ArraySize=*/1),
		TEXT("PCGTexture2DArrayDI_DummyUAV"));

	TextureSRV = GraphBuilder.CreateSRV(DummyTextureSRV);
	TextureUAV = GraphBuilder.CreateUAV(DummyTextureUAV);
}

void FPCGTexture2DArrayDataProviderProxy::CreateTextures(FRDGBuilder& GraphBuilder)
{
	FRDGTextureRef ExportableTexture = nullptr;

	FRDGTextureRef Texture = nullptr;
	bool bCanCreateUAV = false;

	if (Data.BindingInfo.ResourceType == EPCGTextureResourceType::TextureObject && Data.BindingInfo.Texture)
	{
		Texture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Data.BindingInfo.Texture, TEXT("PCGTexture2DArrayDI_SRV")));
	}
	else if (Data.BindingInfo.ResourceType == EPCGTextureResourceType::ExportedTexture && Data.BindingInfo.ExportedTexture)
	{
		Texture = GraphBuilder.RegisterExternalTexture(Data.BindingInfo.ExportedTexture);
	}
	else
	{
		Texture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2DArray(
				Data.BindingInfo.Size,
				Data.BindingInfo.Format,
				FClearValueBinding::Black,
				ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV,
				Data.BindingInfo.ArraySize),
			TEXT("PCGTexture2DArrayDI_UAV"));

		bCanCreateUAV = true;
	}

	TextureSRV = GraphBuilder.CreateSRV(Texture);

	if (bCanCreateUAV)
	{
		TextureUAV = GraphBuilder.CreateUAV(Texture);
	}

	ExportableTexture = Texture;

	if (Data.ExportMode != EPCGExportMode::NoExport)
	{
		ExportTextureUAV(GraphBuilder.ConvertToExternalTexture(ExportableTexture));
	}
}

void FPCGTexture2DArrayDataProviderProxy::ExportTextureUAV(TRefCountPtr<IPooledRenderTarget> ExportedTexture)
{
	LLM_SCOPE_BYTAG(PCG);

	// Export textures and pass a reference back to the main thread where it will be picked up by the compute graph element.
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [Data = Data, ExportedTexture = ExportedTexture]()
	{
		LLM_SCOPE_BYTAG(PCG);

		// Obtain objects. No ensures added because a graph cancellation could feasibly destroy some or all of these.
		UPCGTexture2DArrayDataProvider* DataProvider = Data.WeakDataProvider_GT.Get();
		if (!DataProvider)
		{
			UE_LOGF(LogPCG, Error, "Could not resolve UPCGTexture2DArrayDataProvider object to pass back buffer handle.");
			return;
		}

		if (DataProvider->GetGenerationCounter() != Data.OriginatingGenerationCount)
		{
			return;
		}

		if (DataProvider->GetInitializeFromDataCollection())
		{
			DataProvider->OnDataExported_GameThread().Broadcast();
			return;
		}

		if (!ensure(Data.PinDesc) || Data.PinDesc->GetDataDescriptions().IsEmpty())
		{
			return;
		}

		if (UPCGDataBinding* Binding = DataProvider->GetDataBinding())
		{
			const TArray<FString>& StringTable = Binding->GetStringTable();
			FTransform TextureTransform = Data.BindingInfo.Transform;
			const FVector::FReal XSize = 2.0 * TextureTransform.GetScale3D().X;
			const FVector::FReal YSize = 2.0 * TextureTransform.GetScale3D().Y;
			const FVector2D TexelSize = FVector2D(XSize, YSize) / Data.BindingInfo.Size;

			FPCGTexture2DArrayDataInitParams InitParams;
			InitParams.Transform = MoveTemp(TextureTransform);

			UPCGTexture2DArrayData* ExportedData = NewObject<UPCGTexture2DArrayData>();
			ExportedData->Initialize(ExportedTexture, InitParams);

			Binding->ReceiveDataFromGPU_GameThread(ExportedData, DataProvider->GetProducerSettings(), Data.ExportMode, Data.OutputPinLabel, Data.OutputPinLabelAlias, Data.PinDesc->GetDataDescriptions()[0]);
		}

		DataProvider->OnDataExported_GameThread().Broadcast();
	});
}

#undef LOCTEXT_NAMESPACE
