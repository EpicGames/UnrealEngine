// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGTextureInfo.h"
#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "RHIResources.h"

#include "PCGTexture2DArrayDataInterface.generated.h"

class FPCGTexture2DArrayDataInterfaceParameters;
class UPCGTexture2DArrayData;

/** Data Interface allowing read/write of a UPCGTexture2DArrayData. */
UCLASS(ClassGroup = (Procedural))
class UPCGTexture2DArrayDataInterface : public UPCGExportableDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGTexture2DArray"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	TCHAR const* GetShaderVirtualPath() const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	void SetInitializeFromDataCollection(bool bInInitializeFromDataCollection) { bInitializeFromDataCollection = bInInitializeFromDataCollection; }
	bool GetInitializeFromDataCollection() const { return bInitializeFromDataCollection; }

protected:
	UPROPERTY()
	bool bInitializeFromDataCollection = false;

private:
	static TCHAR const* TemplateFilePath;
};

/** Data Provider allowing read/write of a UPCGTexture2DArrayData. */
UCLASS()
class UPCGTexture2DArrayDataProvider : public UPCGExportableDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	bool GetInitializeFromDataCollection() const { return bInitializeFromDataCollection; }

private:
	void InitializeFromDataCollection(UPCGDataBinding* InBinding);
	void InitializeFromDataDescription(UPCGDataBinding* InBinding);

private:
	bool bInitializeFromDataCollection = false;

	FPCGTextureBindingInfo BindingInfo;
};

class FPCGTexture2DArrayDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FData
	{
		FPCGTextureBindingInfo BindingInfo;

		// For export
		EPCGExportMode ExportMode = EPCGExportMode::NoExport;
		TSharedPtr<const FPCGDataCollectionDesc> PinDesc = nullptr;
		FName OutputPinLabel;
		FName OutputPinLabelAlias;
	
		/** Generation count of the data provider when the proxy was created. */
		uint64 OriginatingGenerationCount = 0;

		// Weak pointer useful for passing back texture handles. Do not access this directly from the render thread.
		TWeakObjectPtr<UPCGTexture2DArrayDataProvider> WeakDataProvider_GT;
	};

	explicit FPCGTexture2DArrayDataProviderProxy(FData InData)
		: Data(MoveTemp(InData))
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	void CreateDefaultTextures(FRDGBuilder& GraphBuilder);
	void CreateTextures(FRDGBuilder& GraphBuilder);
	void ExportTextureUAV(TRefCountPtr<IPooledRenderTarget> ExportedTexture);

private:
	using FParameters = FPCGTexture2DArrayDataInterfaceParameters;

	FData Data;

	// Allocated resources
	FRDGTextureSRVRef TextureSRV;
	FRDGTextureUAVRef TextureUAV;
};
