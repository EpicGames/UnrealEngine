// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTextureDataInterface.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGData.h"
#include "PCGSettings.h"
#include "Compute/PCGDataBinding.h"
#include "Data/PCGTextureData.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGHelpers.h"

#include "GlobalRenderResources.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "RenderGraphBuilder.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SystemTextures.h"
#include "TextureResource.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Engine/Texture.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGTextureDataInterface)

#define LOCTEXT_NAMESPACE "PCGTextureDataInterface"

namespace PCGTextureDataInterfaceHelpers
{
	void GetSharedFunctions(TArray<FShaderFunctionDefinition>& OutFunctions)
	{
		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetNumData"))
			.AddReturnType(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetNumElements"))
			.AddReturnType(EShaderFundamentalType::Uint, 2)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetThreadData"))
			.AddReturnType(EShaderFundamentalType::Bool)
			.AddParam(EShaderFundamentalType::Uint) // InThreadIndex
			.AddParam(EShaderFundamentalType::Uint, 0, 0, EShaderParamModifier::Out) // OutDataIndex
			.AddParam(EShaderFundamentalType::Uint, 2, 0, EShaderParamModifier::Out); // OutElementIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetTexCoords"))
			.AddReturnType(EShaderFundamentalType::Float, 2)
			.AddParam(EShaderFundamentalType::Uint) // InDataIndex
			.AddParam(EShaderFundamentalType::Float, 3); // InWorldPos

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetTexelSize"))
			.AddReturnType(EShaderFundamentalType::Float, 2)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetTexelSizeWorld"))
			.AddReturnType(EShaderFundamentalType::Float, 2)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("GetNumMips"))
			.AddReturnType(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint); // InDataIndex
	}
}

void UPCGTextureDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	// TODO: Support texture3d, etc. UNiagaraDataInterfaceRenderTarget2D and related are useful references.

	PCGTextureDataInterfaceHelpers::GetSharedFunctions(OutFunctions);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("Sample"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Float, 2); // InTextureUV

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SampleWorldPos"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Float, 3); // WorldPos

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("Load"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Uint, 2); // InElementIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("Load"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Uint, 2) // InElementIndex
		.AddParam(EShaderFundamentalType::Uint); // InMipIndex

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetTransform"))
		.AddReturnType(EShaderFundamentalType::Float, 4, 4)
		.AddParam(EShaderFundamentalType::Uint); // InDataIndex

	// Deprecated 5.6. Assumes data index is 0.
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("Sample"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Float, 2); // InTextureUV

	// Deprecated 5.6. Assumes data index is 0.
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SampleWorldPos"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Float, 2); // WorldPos

	// Deprecated 5.7. Does not take data index, and assumes axis-aligned Z projection.
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetTexCoords"))
		.AddReturnType(EShaderFundamentalType::Float, 2)
		.AddParam(EShaderFundamentalType::Float, 2) // WorldPos
		.AddParam(EShaderFundamentalType::Float, 2) // Min
		.AddParam(EShaderFundamentalType::Float, 2); // Max

	// Deprecated 5.7. Switch to using 3D world pos to support any texture orientation.
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SampleWorldPos"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Float, 2); // WorldPos
}

void UPCGTextureDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	PCGTextureDataInterfaceHelpers::GetSharedFunctions(OutFunctions);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("Store"))
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Uint, 2) // InElementIndex
		.AddParam(EShaderFundamentalType::Float, 4); // InValue

	// @todo_pcg: We could also add a LoadUAV function. Note that we can't use the existing input Load function because it loads from the SRV, not the UAV.
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGTextureDataInterfaceParameters,)
	SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2D, Texture, [PCGTextureDataInterfaceConstants::MAX_NUM_SRV_BINDINGS])
	SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2DArray, TextureArray, [PCGTextureDataInterfaceConstants::MAX_NUM_SRV_BINDINGS])
	SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<float4>, TextureUAV, [PCGTextureDataInterfaceConstants::MAX_NUM_UAV_BINDINGS])
	SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
	SHADER_PARAMETER_ARRAY(FMatrix44f, TextureTransforms, [PCGTextureDataInterfaceConstants::MAX_NUM_SRV_BINDINGS])
	SHADER_PARAMETER_ARRAY(FMatrix44f, TextureInverseTransforms, [PCGTextureDataInterfaceConstants::MAX_NUM_SRV_BINDINGS])
	SHADER_PARAMETER(uint32, NumTextureInfos)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FUintVector4>, TextureInfos) // todo_pcg: Could be simple uniform array?
END_SHADER_PARAMETER_STRUCT()

void UPCGTextureDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGTextureDataInterfaceParameters>(UID);
}

TCHAR const* UPCGTextureDataInterface::TemplateFilePath = TEXT("/Plugin/PCG/Private/PCGTextureDataInterface.ush");

TCHAR const* UPCGTextureDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UPCGTextureDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UPCGTextureDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
		{ TEXT("EnableMultipleTextureObjects"), bSingleData ? 0 : 1 },
	};

	FString TemplateFile;
	if (ensure(LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr)))
	{
		OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
	}
}

void UPCGTextureDataInterface::GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const
{
	Super::GetDefines(OutDefinitionSet);

	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_MAX_NUM_SRV_BINDINGS"), FString::FromInt(PCGTextureDataInterfaceConstants::MAX_NUM_SRV_BINDINGS)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_ENABLE_MULTIPLE_TEXTURE_OBJECTS"), FString::FromInt(bSingleData ? 0 : 1)));
}

UComputeDataProvider* UPCGTextureDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGTextureDataProvider>();
}

void UPCGTextureDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGTextureDataInterface* DataInterface = CastChecked<UPCGTextureDataInterface>(InDataInterface);
	bInitializeFromDataCollection = DataInterface->GetInitializeFromDataCollection();
	bSingleData = DataInterface->GetSingleData();
}

FComputeDataProviderRenderProxy* UPCGTextureDataProvider::GetRenderProxy()
{
	return new FPCGTextureDataProviderProxy(this);
}

void UPCGTextureDataProvider::Reset()
{
	Super::Reset();

	BindingInfos.Empty();
	TextureInfos.Empty();
	bInitializeFromDataCollection = false;
	bSingleData = false;
}

bool UPCGTextureDataProvider::PrepareForExecute_GameThread(UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGTextureDataProvider::PrepareForExecute_GameThread);

	check(InBinding);

	if (!Super::PrepareForExecute_GameThread(InBinding))
	{
		return false;
	}

	if (bInitializeFromDataCollection)
	{
		BuildInfosFromDataCollection(InBinding);
	}
	else
	{
		BuildInfosFromDataDescription(InBinding);
	}

	// Validate that all binding infos have the same sampler - otherwise warn that only the first will be used.
	if (!BindingInfos.IsEmpty())
	{
		const EPCGTextureFilter Filter = BindingInfos[0].Filter;

		for (int BindingIndex = 1; BindingIndex < BindingInfos.Num(); ++BindingIndex)
		{
			if (BindingInfos[BindingIndex].Filter != Filter)
			{
				UE_LOGF(LogPCG, Warning, "Encountered multiple unique texture filters on texture data interface: '%ls'. Only the first will be used.", *GetOutputPinLabel().ToString());
			}
		}
	}

	return true;
}

void UPCGTextureDataProvider::BuildInfosFromDataCollection(UPCGDataBinding* InBinding)
{
	check(InBinding);

	// Take any input pin label alias to obtain the data from the input data collection.
	check(!GetDownstreamInputPinLabelAliases().IsEmpty());
	const TArray<FPCGTaggedData> InputTaggedData = InBinding->GetInputDataCollection().GetInputsByPin(GetDownstreamInputPinLabelAliases()[0]);
	const int32 SRVCapacity = bSingleData ? 1 : PCGTextureDataInterfaceConstants::MAX_NUM_SRV_BINDINGS;
	bool bWarnedOverflow = false;

	for (int I = 0; I < InputTaggedData.Num(); ++I)
	{
		const FPCGTaggedData& TaggedData = InputTaggedData[I];
		const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(TaggedData.Data);

		if (!ensure(TextureData))
		{
			UE_LOGF(LogPCG, Error, "Unsupported data type encountered by texture data interface: '%ls'", TaggedData.Data ? *TaggedData.Data->GetName() : TEXT("NULL"));
			continue;
		}

		FPCGTextureBindingInfo BindingInfo(TextureData);
		int32 BindingIndex = BindingInfos.Find(BindingInfo);

		if (BindingIndex == INDEX_NONE)
		{
			if (BindingInfos.Num() < SRVCapacity)
			{
				BindingIndex = BindingInfos.Add(MoveTemp(BindingInfo));
			}
			else
			{
				if (!bWarnedOverflow)
				{
					bWarnedOverflow = true;
					UE_LOGF(LogPCG, Warning, "Texture data interface on pin '%ls' received %d textures but binding capacity is %d; remaining textures ignored.",
						*GetDownstreamInputPinLabelAliases()[0].ToString(),
						InputTaggedData.Num(),
						SRVCapacity);
				}
				continue;
			}
		}

		FPCGTextureInfo& TextureInfo = TextureInfos.Emplace_GetRef();
		TextureInfo.BindingIndex = BindingIndex;
		TextureInfo.SliceIndex = TextureData->GetTextureSlice();
	}
}

void UPCGTextureDataProvider::BuildInfosFromDataDescription(UPCGDataBinding* InBinding)
{
	check(InBinding);

	TSharedPtr<const FPCGDataCollectionDesc> PinDesc = GetPinDescription();
	check(PinDesc);

	bool bWarnedOverflow = false;

	for (const FPCGDataDesc& DataDesc : PinDesc->GetDataDescriptions())
	{
		FPCGTextureBindingInfo BindingInfo(DataDesc);
		int32 BindingIndex = BindingInfos.Find(BindingInfo);

		if (BindingIndex == INDEX_NONE)
		{
			if (BindingInfos.Num() < PCGTextureDataInterfaceConstants::MAX_NUM_UAV_BINDINGS)
			{
				BindingIndex = BindingInfos.Add(MoveTemp(BindingInfo));
			}
			else
			{
				if (!bWarnedOverflow)
				{
					bWarnedOverflow = true;
					UE_LOGF(LogPCG, Warning, "Texture data interface on pin '%ls' received %d texture UAVs but binding capacity is %u; remaining UAVs ignored.",
						*GetOutputPinLabel().ToString(),
						PinDesc->GetDataDescriptions().Num(),
						PCGTextureDataInterfaceConstants::MAX_NUM_UAV_BINDINGS);
				}
				continue;
			}
		}

		FPCGTextureInfo& TextureInfo = TextureInfos.Emplace_GetRef();
		TextureInfo.BindingIndex = BindingIndex;
		TextureInfo.SliceIndex = 0;
	}
}

FPCGTextureDataProviderProxy::FPCGTextureDataProviderProxy(TWeakObjectPtr<UPCGTextureDataProvider> InDataProvider)
{
	check(IsInGameThread());
	check(InDataProvider.IsValid());

	BindingInfos = InDataProvider->GetBindingInfos();
	TextureInfos = InDataProvider->GetTextureInfos();
	ExportMode = InDataProvider->GetExportMode();
	OutputPinLabel = InDataProvider->GetOutputPinLabel();
	OutputPinLabelAlias = InDataProvider->GetOutputPinLabelAlias();
	OriginatingGenerationCount = InDataProvider->GetGenerationCounter();
	bSingleData = InDataProvider->GetSingleData();
	DataProviderWeakPtr_GT = InDataProvider;
	PinDesc = InDataProvider->GetPinDescription();
}

bool FPCGTextureDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	FString* ReasonPtr = nullptr;
#if !NO_LOGGING
	FString Reason;
	ReasonPtr = &Reason;
#endif

	for (const FPCGTextureBindingInfo& BindingInfo : BindingInfos)
	{
		if (!BindingInfo.IsValid(ReasonPtr))
		{
			UE_LOGF(LogPCG, Warning, "FPCGTextureDataProviderProxy invalid due to invalid texture binding.");
			if (ReasonPtr)
			{
				UE_LOGF(LogPCG, Warning, "FPCGTextureDataProviderProxy: %ls.", **ReasonPtr);
			}
			return false;
		}
	}

	for (const FPCGTextureInfo& TextureInfo : TextureInfos)
	{
		if (!BindingInfos.IsValidIndex(TextureInfo.BindingIndex) || TextureInfo.SliceIndex < 0)
		{
			UE_LOGF(LogPCG, Warning, "FPCGTextureDataProviderProxy invalid due to invalid binding indices.");
			return false;
		}
	}

	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

void FPCGTextureDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];

		Parameters.Sampler = (!BindingInfos.IsEmpty() && BindingInfos[0].Filter == EPCGTextureFilter::Point)
			? TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI()
			: TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Parameters.NumTextureInfos = TextureInfos.Num();
		Parameters.TextureInfos = TextureInfosBufferSRV;

		for (int BindingIndex = 0; BindingIndex < PCGTextureDataInterfaceConstants::MAX_NUM_SRV_BINDINGS; ++BindingIndex)
		{
			Parameters.Texture[BindingIndex] = TextureSRV[BindingIndex];
			Parameters.TextureArray[BindingIndex] = TextureArraySRV[BindingIndex];

			if (BindingIndex < BindingInfos.Num())
			{
				const FTransform& Transform = BindingInfos[BindingIndex].Transform;
				Parameters.TextureTransforms[BindingIndex] = FMatrix44f(Transform.ToMatrixWithScale()).GetTransposed();
				Parameters.TextureInverseTransforms[BindingIndex] = !FMath::IsNearlyZero(Transform.GetDeterminant()) ? FMatrix44f(Transform.ToInverseMatrixWithScale()).GetTransposed() : FMatrix44f::Identity;
			}
			else
			{
				Parameters.TextureTransforms[BindingIndex] = FMatrix44f::Identity;
				Parameters.TextureInverseTransforms[BindingIndex] = FMatrix44f::Identity;
			}
		}

		for (int BindingIndex = 0; BindingIndex < PCGTextureDataInterfaceConstants::MAX_NUM_UAV_BINDINGS; ++BindingIndex)
		{
			Parameters.TextureUAV[BindingIndex] = TextureUAV[BindingIndex];
		}
	}
}

void FPCGTextureDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	LLM_SCOPE_BYTAG(PCG);

	CreateDefaultTextures(GraphBuilder);
	CreateTextures(GraphBuilder);
	PackTextureInfos(GraphBuilder);
}

void FPCGTextureDataProviderProxy::CreateDefaultTextures(FRDGBuilder& GraphBuilder)
{
	FRDGTextureRef DummyTextureForSRV = GSystemTextures.GetDefaultTexture2D(GraphBuilder, EPixelFormat::PF_G8, 0.0f);
	FRDGTextureRef DummyTextureArrayForSRV = GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture2DArray, EPixelFormat::PF_G8, 0.0f);

	FRDGTextureRef DummyTextureForUAV = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_G8, FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV),
			TEXT("PCGTextureDI_DummyTextureForUAV"));

	// Always fill every slot - ValidateShaderParameters checks the full C++ struct regardless of
	// which slots the HLSL actually uses (e.g. when PCG_ENABLE_MULTIPLE_TEXTURE_OBJECTS == 0).
	for (int BindingIndex = 0; BindingIndex < PCGTextureDataInterfaceConstants::MAX_NUM_SRV_BINDINGS; ++BindingIndex)
	{
		TextureSRV[BindingIndex] = GraphBuilder.CreateSRV(DummyTextureForSRV);
		TextureArraySRV[BindingIndex] = GraphBuilder.CreateSRV(DummyTextureArrayForSRV);
	}

	for (int BindingIndex = 0; BindingIndex < PCGTextureDataInterfaceConstants::MAX_NUM_UAV_BINDINGS; ++BindingIndex)
	{
		TextureUAV[BindingIndex] = GraphBuilder.CreateUAV(DummyTextureForUAV);
	}
}

void FPCGTextureDataProviderProxy::CreateTextures(FRDGBuilder& GraphBuilder)
{
	FRDGTextureRef ExportableTextures[PCGTextureDataInterfaceConstants::MAX_NUM_UAV_BINDINGS] = {};

	for (int BindingIndex = 0; BindingIndex < BindingInfos.Num(); ++BindingIndex)
	{
		FRDGTextureRef Texture = nullptr;
		bool bCanCreateUAV = false;

		if (BindingInfos[BindingIndex].ResourceType == EPCGTextureResourceType::TextureObject && BindingInfos[BindingIndex].Texture)
		{
			Texture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(BindingInfos[BindingIndex].Texture, TEXT("PCGTextureDI_RenderTarget")));
		}
		else if (BindingInfos[BindingIndex].ResourceType == EPCGTextureResourceType::ExportedTexture && BindingInfos[BindingIndex].ExportedTexture)
		{
			Texture = GraphBuilder.RegisterExternalTexture(BindingInfos[BindingIndex].ExportedTexture);
		}
		else if (BindingIndex < PCGTextureDataInterfaceConstants::MAX_NUM_UAV_BINDINGS)
		{
			Texture = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(
					BindingInfos[BindingIndex].Size,
					BindingInfos[BindingIndex].Format,
					FClearValueBinding::Black,
					ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV),
				TEXT("PCGTextureDI_UAV"));

			bCanCreateUAV = true;
			ExportableTextures[BindingIndex] = Texture;
		}

		if (!Texture)
		{
			continue;
		}

		if (BindingInfos[BindingIndex].Dimension == ETextureDimension::Texture2D)
		{
			TextureSRV[BindingIndex] = GraphBuilder.CreateSRV(Texture);

			if (bCanCreateUAV)
			{
				TextureUAV[BindingIndex] = GraphBuilder.CreateUAV(Texture);
			}
		}
		else if (BindingInfos[BindingIndex].Dimension == ETextureDimension::Texture2DArray)
		{
			TextureArraySRV[BindingIndex] = GraphBuilder.CreateSRV(Texture);
		}
		else
		{
			checkNoEntry();
		}
	}

	if (ExportMode != EPCGExportMode::NoExport)
	{
		TArray<TRefCountPtr<IPooledRenderTarget>> ExportedTextures;

		for (const FPCGTextureInfo& TextureInfo : TextureInfos)
		{
			if (ensure(TextureInfo.BindingIndex >= 0 && TextureInfo.BindingIndex < PCGTextureDataInterfaceConstants::MAX_NUM_UAV_BINDINGS))
			{
				if (FRDGTextureRef ExportableTexture = ExportableTextures[TextureInfo.BindingIndex])
				{
					ExportedTextures.Add(GraphBuilder.ConvertToExternalTexture(ExportableTexture));
				}
			}
		}

		ExportTextureUAVs(ExportedTextures);
	}
}

void FPCGTextureDataProviderProxy::ExportTextureUAVs(const TArray<TRefCountPtr<IPooledRenderTarget>>& ExportedTextures)
{
	LLM_SCOPE_BYTAG(PCG);

	// Export textures and pass a reference back to the main thread where it will be picked up by the compute graph element.
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [ExportedTextures=ExportedTextures, DataProviderWeakPtr=DataProviderWeakPtr_GT, PinDesc=PinDesc, OutputPinLabel=OutputPinLabel, ExportMode=ExportMode, OutputPinLabelAlias=OutputPinLabelAlias, GenerationCount=OriginatingGenerationCount]()
	{
		LLM_SCOPE_BYTAG(PCG);

		// Obtain objects. No ensures added because a graph cancellation could feasibly destroy some or all of these.
		UPCGTextureDataProvider* DataProvider = DataProviderWeakPtr.Get();
		if (!DataProvider)
		{
			UE_LOGF(LogPCG, Error, "Could not resolve UPCGTextureDataProvider object to pass back buffer handle.");
			return;
		}

		if (DataProvider->GetGenerationCounter() != GenerationCount)
		{
			return;
		}

		if (DataProvider->bInitializeFromDataCollection)
		{
			DataProvider->OnDataExported_GameThread().Broadcast();
			return;
		}

		if (!ensure(PinDesc))
		{
			return;
		}

		if (UPCGDataBinding* Binding = DataProvider->GetDataBinding())
		{
			const TArray<FPCGTextureInfo>& TextureInfos_GT = DataProvider->GetTextureInfos();
			const TArray<FPCGTextureBindingInfo>& BindingInfos_GT = DataProvider->GetBindingInfos();

			for (int TextureDataIndex = 0; TextureDataIndex < TextureInfos_GT.Num(); ++TextureDataIndex)
			{
				const FPCGTextureInfo& TextureInfo = TextureInfos_GT[TextureDataIndex];
				check(ExportedTextures.IsValidIndex(TextureDataIndex));
				check(PinDesc->GetDataDescriptions().IsValidIndex(TextureDataIndex))
				check(BindingInfos_GT.IsValidIndex(TextureInfo.BindingIndex));

				const FPCGTextureBindingInfo& BindingInfo = BindingInfos_GT[TextureInfo.BindingIndex];
				const FTransform TextureTransform = BindingInfos_GT[TextureInfo.BindingIndex].Transform;
				const FVector::FReal XSize = 2.0 * TextureTransform.GetScale3D().X;
				const FVector::FReal YSize = 2.0 * TextureTransform.GetScale3D().Y;
				const FVector2D TexelSize = FVector2D(XSize, YSize) / BindingInfo.Size;

				UPCGTextureData* ExportedData = NewObject<UPCGTextureData>();
				ExportedData->TexelSize = TexelSize.GetMin();
				ExportedData->Initialize(ExportedTextures[TextureDataIndex], TextureInfo.SliceIndex, TextureTransform);

				// @todo_pcg: Binding is doing a lot of work. Could store a context handle in the data provider instead?
				Binding->ReceiveDataFromGPU_GameThread(ExportedData, DataProvider->GetProducerSettings(), ExportMode, OutputPinLabel, OutputPinLabelAlias, PinDesc->GetDataDescriptions()[TextureDataIndex]);
			}
		}

		DataProvider->OnDataExported_GameThread().Broadcast();
	});
}

void FPCGTextureDataProviderProxy::PackTextureInfos(FRDGBuilder& GraphBuilder)
{
	// Layout:
	// - TextureInfo0
	//   - uint32 - BindingIndex
	//   - uint32 - SliceIndex (aka TextureIndex if bound to a TextureArray, 0 otherwise.)
	//   - uint32 - Dimension
	//   - uint32 - NumMips
	//   - uint32 - ResolutionX
	//   - uint32 - ResolutionY
	//   - float  - TexelSizeWorldX (cm)
	//   - float  - TexelSizeWorldY (cm)
	// - TextureInfo1
	// - TextureInfo2
	// - ...

	if (!TextureInfos.IsEmpty())
	{
		const FRDGBufferDesc TextureInfosDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FUintVector4), TextureInfos.Num() * 2);

		FRDGBufferRef TextureInfosBuffer = GraphBuilder.CreateBuffer(TextureInfosDesc, TEXT("PCGTextureDI_InfosBuffer"));
		TextureInfosBufferSRV = GraphBuilder.CreateSRV(TextureInfosBuffer);

		TArray<FUintVector4> TextureInfosPackedData;
		TextureInfosPackedData.SetNumUninitialized(TextureInfos.Num() * 2);

		for (int TextureInfoIndex = 0; TextureInfoIndex < TextureInfos.Num(); ++TextureInfoIndex)
		{
			const FPCGTextureInfo& TextureInfo = TextureInfos[TextureInfoIndex];
			const FPCGTextureBindingInfo& BindingInfo = BindingInfos[TextureInfo.BindingIndex];
			const FVector& TextureWorldScale = BindingInfo.Transform.GetScale3D();
			const float TexelWorldSizeX = BindingInfo.Size.X != 0 ? (TextureWorldScale.X * 2.0) / BindingInfo.Size.X : 0.0f;
			const float TexelWorldSizeY = BindingInfo.Size.Y != 0 ? (TextureWorldScale.Y * 2.0) / BindingInfo.Size.Y : 0.0f;

			TextureInfosPackedData[TextureInfoIndex * 2 + 0] = FUintVector4(TextureInfo.BindingIndex, TextureInfo.SliceIndex, static_cast<uint32>(BindingInfo.Dimension), BindingInfo.NumMips);
			TextureInfosPackedData[TextureInfoIndex * 2 + 1] = FUintVector4(BindingInfo.Size.X, BindingInfo.Size.Y, FMath::AsUInt(TexelWorldSizeX), FMath::AsUInt(TexelWorldSizeY));
		}

		GraphBuilder.QueueBufferUpload(TextureInfosBuffer, MakeArrayView(TextureInfosPackedData));
	}
	else
	{
		TextureInfosBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FUintVector4))));
	}
}

#undef LOCTEXT_NAMESPACE
