// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGDataDescription.h"
#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "Compute/DataInterfaces/PCGTextureInfo.h"
#include "Data/PCGTextureData.h"

#include "RHIResources.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "PCGTextureDataInterface.generated.h"

class FPCGTextureDataInterfaceParameters;
class UPCGTexture2DSingleBaseData;

namespace PCGTextureDataInterfaceConstants
{
	/**
	 * The maximum number of texture SRVs and UAVs which can be bound to a PCGTextureDataInterface.
	 * We can't use full bindless on all platforms, so fallback to emulating.
	 * If you change this number, make sure update PCGTextureDataInterface.ush as well.
	 */
	constexpr uint32 MAX_NUM_SRV_BINDINGS = 8;
	constexpr uint32 MAX_NUM_UAV_BINDINGS = 1;
}

/** Data Interface allowing sampling of a texture. */
UCLASS(ClassGroup = (Procedural))
class UPCGTextureDataInterface : public UPCGExportableDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGTexture"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	TCHAR const* GetShaderVirtualPath() const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	void GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	void SetInitializeFromDataCollection(bool bInInitializeFromDataCollection) { bInitializeFromDataCollection = bInInitializeFromDataCollection; }
	bool GetInitializeFromDataCollection() const { return bInitializeFromDataCollection; }

	void SetSingleData(bool bInSingleData) { bSingleData = bInSingleData; }
	bool GetSingleData() const { return bSingleData; }

protected:
	UPROPERTY()
	bool bInitializeFromDataCollection = false;

	/** When single data, the multi-data SRVs are stripped, minimizing number of resource binders and combined samplers. */
	UPROPERTY()
	bool bSingleData = false;

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading a texture. */
UCLASS()
class UPCGTextureDataProvider : public UPCGExportableDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	//~ Begin UPCGComputeDataProvider Interface
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	const TArray<FPCGTextureBindingInfo>& GetBindingInfos() const { return BindingInfos; }
	const TArray<FPCGTextureInfo>& GetTextureInfos() const { return TextureInfos; }
	bool GetSingleData() const { return bSingleData; }

protected:
	void BuildInfosFromDataCollection(UPCGDataBinding* InBinding);
	void BuildInfosFromDataDescription(UPCGDataBinding* InBinding);

public:
	TArray<FPCGTextureBindingInfo> BindingInfos;
	TArray<FPCGTextureInfo> TextureInfos;
	bool bInitializeFromDataCollection = false;
	bool bSingleData = false;
};

class FPCGTextureDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	explicit FPCGTextureDataProviderProxy(TWeakObjectPtr<UPCGTextureDataProvider> InDataProvider);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	//~ End FComputeDataProviderRenderProxy Interface

	void CreateDefaultTextures(FRDGBuilder& GraphBuilder);
	void CreateTextures(FRDGBuilder& GraphBuilder);
	void ExportTextureUAVs(const TArray<TRefCountPtr<IPooledRenderTarget>>& ExportedTextures);
	void PackTextureInfos(FRDGBuilder& GraphBuilder);

protected:
	using FParameters = FPCGTextureDataInterfaceParameters;

	TArray<FPCGTextureBindingInfo> BindingInfos;
	TArray<FPCGTextureInfo> TextureInfos;

	// For export
	EPCGExportMode ExportMode = EPCGExportMode::NoExport;
	TSharedPtr<const FPCGDataCollectionDesc> PinDesc = nullptr;
	FName OutputPinLabel;
	FName OutputPinLabelAlias;

	/** Generation count of the data provider when the proxy was created. */
	uint64 OriginatingGenerationCount = 0;

	bool bSingleData = false;

	// Weak pointer useful for passing back texture handles. Do not access this directly from the render thread.
	TWeakObjectPtr<UPCGTextureDataProvider> DataProviderWeakPtr_GT;

	// Allocated resources. Arrays are always full size; in single-data mode only slot 0 is populated and referenced.
	FRDGTextureSRVRef TextureSRV[PCGTextureDataInterfaceConstants::MAX_NUM_SRV_BINDINGS];
	FRDGTextureSRVRef TextureArraySRV[PCGTextureDataInterfaceConstants::MAX_NUM_SRV_BINDINGS];
	FRDGTextureUAVRef TextureUAV[PCGTextureDataInterfaceConstants::MAX_NUM_UAV_BINDINGS];
	FRDGTextureUAVRef TextureArrayUAV[PCGTextureDataInterfaceConstants::MAX_NUM_UAV_BINDINGS];
	FRDGBufferSRVRef TextureInfosBufferSRV = nullptr;
};
