// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTextureInfo.h"

#include "Compute/PCGDataDescription.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Data/PCGTextureData.h"

#include "RHIDefinitions.h"

FPCGTextureBindingInfo::FPCGTextureBindingInfo(const UPCGTexture2DSingleBaseData* InTextureData)
{
	check(InTextureData);

	ResourceType = InTextureData->GetTextureResourceType();
	Texture = InTextureData->GetTextureRHI();
	ExportedTexture = InTextureData->GetRefCountedTexture();
	Transform = InTextureData->GetTransform();
	Filter = InTextureData->Filter;

	if (Texture.IsValid())
	{
		const FRHITextureDesc& Desc = Texture->GetDesc();
		Size = Desc.Extent;
		ArraySize = Desc.ArraySize;
		Dimension = Desc.Dimension;
		Format = Desc.Format;
		NumMips = Desc.NumMips;
	}
	else
	{
		UE_LOGF(LogPCG, Error, "PCGTextureDataInterface: Texture data '%ls' has an invalid texture.", InTextureData ? *InTextureData->GetName() : TEXT("NULL"));
	}
}

FPCGTextureBindingInfo::FPCGTextureBindingInfo(const UPCGTexture2DArrayData* InTextureData)
{
	check(InTextureData);

	ResourceType = InTextureData->GetResourceType();
	Texture = InTextureData->GetRHI();
	ExportedTexture = InTextureData->GetRefCountedTexture();
	Filter = InTextureData->GetFilter();
	Transform = InTextureData->GetTransform();

	if (Texture.IsValid())
	{
		const FRHITextureDesc& Desc = Texture->GetDesc();
		Size = Desc.Extent;
		ArraySize = Desc.ArraySize;
		Dimension = Desc.Dimension;
		Format = Desc.Format;
		NumMips = Desc.NumMips;
	}
	else
	{
		UE_LOGF(LogPCG, Error, "PCGTexture2DArrayDataInterface: Texture2DArray data '%ls' has an invalid texture.", InTextureData ? *InTextureData->GetName() : TEXT("NULL"));
	}
}

FPCGTextureBindingInfo::FPCGTextureBindingInfo(const FPCGDataDesc& InDataDesc)
{
	Size = FIntPoint(FMath::Max(InDataDesc.GetElementCount().X, 1), FMath::Max(InDataDesc.GetElementCount().Y, 1));
	ArraySize = InDataDesc.GetTextureArraySize();
	Dimension = InDataDesc.GetType().IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId()) ? ETextureDimension::Texture2DArray : ETextureDimension::Texture2D;
	Format = PCGComputeHelpers::GetPixelFormatFromPCGRenderTargetFormat(InDataDesc.GetRenderTargetFormat());
	Filter = InDataDesc.GetTextureFilter();
	Transform = InDataDesc.GetTextureTransform();
	NumMips = InDataDesc.GetTextureNumMips();
}

bool FPCGTextureBindingInfo::operator==(const FPCGTextureBindingInfo& Other) const
{
	return ResourceType == Other.ResourceType
		&& Texture == Other.Texture
		&& ExportedTexture == Other.ExportedTexture
		&& Size == Other.Size
		&& ArraySize == Other.ArraySize
		&& NumMips == Other.NumMips
		&& Dimension == Other.Dimension
		&& Format == Other.Format
		&& Filter == Other.Filter
		&& Transform.Equals(Other.Transform);
}

bool FPCGTextureBindingInfo::IsValid(FString* OutInvalidReason) const
{
	const int32 MaxTextureDimension = GetMax2DTextureDimension();
	const int32 MaxTextureArrayLayers = GetMaxTextureArrayLayers();

	if (ResourceType == EPCGTextureResourceType::Invalid)
	{
		if (OutInvalidReason)
		{
			*OutInvalidReason = TEXT("ResourceType is Invalid");
		}
		return false;
	}

	if (Size.X <= 0 || Size.Y <= 0)
	{
		if (OutInvalidReason)
		{
			*OutInvalidReason = FString::Format(TEXT("Size ({0}) has zero or negative component (binding may not have been populated)"), { Size.ToString() });
		}
		return false;
	}

	if (Size.X > MaxTextureDimension || Size.Y > MaxTextureDimension)
	{
		if (OutInvalidReason)
		{
			*OutInvalidReason = FString::Format(TEXT("Size ({0}) exceeds platform MaxTextureDimension ({1})"), { Size.ToString(), MaxTextureDimension });
		}
		return false;
	}

	if (Dimension == ETextureDimension::Texture2DArray && (ArraySize <= 0 || (int32)ArraySize > MaxTextureArrayLayers))
	{
		if (OutInvalidReason)
		{
			*OutInvalidReason = FString::Format(TEXT("Texture2DArray ArraySize ({0}) out of valid range [1, {1}]"), { ArraySize, MaxTextureArrayLayers });
		}
		return false;
	}

	if (NumMips <= 0 || NumMips > MAX_TEXTURE_MIP_COUNT)
	{
		if (OutInvalidReason)
		{
			*OutInvalidReason = FString::Format(TEXT("NumMips ({0}) out of valid range [1, {1}]"), { NumMips, MAX_TEXTURE_MIP_COUNT });
		}
		return false;
	}

	return true;
}
