// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGTextureData.h"

class UPCGTexture2DSingleBaseData;
class UPCGTexture2DArrayData;
struct FPCGDataDesc;

struct FPCGTextureBindingInfo
{
	FPCGTextureBindingInfo() = default;
	explicit FPCGTextureBindingInfo(const UPCGTexture2DSingleBaseData* InTextureData);
	explicit FPCGTextureBindingInfo(const UPCGTexture2DArrayData* InTextureData);
	FPCGTextureBindingInfo(const FPCGDataDesc& InDataDesc);
	bool operator==(const FPCGTextureBindingInfo& Other) const;
	bool IsValid(FString* OutInvalidReason = nullptr) const;

	EPCGTextureResourceType ResourceType = EPCGTextureResourceType::TextureObject;
	FTextureRHIRef Texture = nullptr;
	TRefCountPtr<IPooledRenderTarget> ExportedTexture = nullptr;
	FIntPoint Size = FIntPoint::ZeroValue;
	uint16 ArraySize = 1;
	uint8 NumMips = 1;
	ETextureDimension Dimension = ETextureDimension::Texture2D;
	EPixelFormat Format = EPixelFormat::PF_Unknown;
	EPCGTextureFilter Filter = EPCGTextureFilter::Bilinear;
	FTransform Transform = FTransform::Identity;
};

struct FPCGTextureInfo
{
	int BindingIndex = 0;
	int SliceIndex = 0;
};
