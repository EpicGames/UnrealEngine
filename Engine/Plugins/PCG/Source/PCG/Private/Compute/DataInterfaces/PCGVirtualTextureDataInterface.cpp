// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGVirtualTextureDataInterface.h"

#include "PCGData.h"
#include "PCGGraphExecutionStateInterface.h"
#include "Compute/PCGDataBinding.h"
#include "Data/PCGVirtualTextureData.h"

#include "GlobalRenderResources.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "VT/RuntimeVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGVirtualTextureDataInterface)

#define LOCTEXT_NAMESPACE "PCGVirtualTextureDataInterface"

static const TCHAR* EnableMultipleVirtualTexturePermutationName = TEXT("ENABLE_MULTIPLE_VIRTUAL_TEXTURES");

void UPCGVirtualTextureDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("SampleVirtualTexture"))
		.AddParam(EShaderFundamentalType::Uint) // InDataIndex
		.AddParam(EShaderFundamentalType::Float, 3) // InWorldPos
		.AddParam(EShaderFundamentalType::Bool, 0, 0, EShaderParamModifier::Out) // bOutInsideVolume
		.AddParam(EShaderFundamentalType::Float, 3, 0, EShaderParamModifier::Out) // OutBaseColor
		.AddParam(EShaderFundamentalType::Float, 0, 0, EShaderParamModifier::Out) // OutSpecular
		.AddParam(EShaderFundamentalType::Float, 0, 0, EShaderParamModifier::Out) // OutRoughness
		.AddParam(EShaderFundamentalType::Float, 0, 0, EShaderParamModifier::Out) // OutWorldHeight
		.AddParam(EShaderFundamentalType::Float, 3, 0, EShaderParamModifier::Out) // OutNormal
		.AddParam(EShaderFundamentalType::Float, 0, 0, EShaderParamModifier::Out) // OutDisplacement
		.AddParam(EShaderFundamentalType::Float, 0, 0, EShaderParamModifier::Out) // OutMask
		.AddParam(EShaderFundamentalType::Float, 4, 0, EShaderParamModifier::Out); // OutMask4
}

BEGIN_SHADER_PARAMETER_STRUCT(FPCGVirtualTextureDataInterfaceParameters, )
	// Per binding info
	SHADER_PARAMETER_TEXTURE_ARRAY(Texture2D<uint4>, PageTable, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS])
	SHADER_PARAMETER_TEXTURE_ARRAY(Texture2D<uint>, PageTableIndirection, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS])
	SHADER_PARAMETER_ARRAY(FUintVector4, PageTableUniforms, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS * 2])
	SHADER_PARAMETER_ARRAY(FUintVector4, PageTableInfo, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS])
	SHADER_PARAMETER_ARRAY(FVector4f, LWCTiles, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS])
	SHADER_PARAMETER_ARRAY(FMatrix44f, WorldToUVTransforms, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS])

	// Per layer info
	SHADER_PARAMETER_SRV_ARRAY(Texture2D, VirtualTexture0, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS])
	SHADER_PARAMETER_SRV_ARRAY(Texture2D, VirtualTexture1, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS])
	SHADER_PARAMETER_SRV_ARRAY(Texture2D, VirtualTexture2, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS])
	SHADER_PARAMETER_ARRAY(FUintVector4, Uniforms, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS * PCGVirtualTextureDIConstants::MAX_NUM_LAYERS])
	SHADER_PARAMETER_ARRAY(FUintVector4, ValidLayerMasks, [PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS])

	// Shared
	SHADER_PARAMETER(uint32, NumVirtualTextures)
	SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
END_SHADER_PARAMETER_STRUCT()

void UPCGVirtualTextureDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPCGVirtualTextureDataInterfaceParameters>(UID);
}

TCHAR const* UPCGVirtualTextureDataInterface::TemplateFilePath = TEXT("/Plugin/PCG/Private/PCGVirtualTextureDataInterface.ush");

TCHAR const* UPCGVirtualTextureDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UPCGVirtualTextureDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
	GetShaderFileHash(TEXT("/Engine/Private/VirtualTextureCommon.ush"), EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
	GetShaderFileHash(TEXT("/Plugin/PCG/Private/PCGVirtualTextureCommon.ush"), EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UPCGVirtualTextureDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	if (ensure(LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr)))
	{
		OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
	}
}

void UPCGVirtualTextureDataInterface::GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const
{
	Super::GetDefines(OutDefinitionSet);

	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_MAX_NUM_BINDINGS"), FString::FromInt(PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS)));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(TEXT("PCG_MAX_NUM_LAYERS"), FString::FromInt(PCGVirtualTextureDIConstants::MAX_NUM_LAYERS)));
}

void UPCGVirtualTextureDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	OutPermutationVector.AddPermutation(EnableMultipleVirtualTexturePermutationName, /*NumValues=*/2);
}

UComputeDataProvider* UPCGVirtualTextureDataInterface::CreateDataProvider() const
{
	return NewObject<UPCGVirtualTextureDataProvider>();
}

void UPCGVirtualTextureDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGVirtualTextureDataProvider::Initialize);

	Super::Initialize(InDataInterface, InBinding, InInputMask, InOutputMask);

	const UPCGVirtualTextureDataInterface* DataInterface = CastChecked<UPCGVirtualTextureDataInterface>(InDataInterface);
	UPCGDataBinding* Binding = CastChecked<UPCGDataBinding>(InBinding);
	const IPCGGraphExecutionSource* Source = Cast<IPCGGraphExecutionSource>(Binding->GetExecutionSource());

	// Take any input pin label alias to obtain the data from the input data collection.
	check(!DataInterface->GetDownstreamInputPinLabelAliases().IsEmpty());
	const TArray<FPCGTaggedData> InputTaggedData = Binding->GetInputDataCollection().GetInputsByPin(DataInterface->GetDownstreamInputPinLabelAliases()[0]);

	// Sampling of virtual textures is not supported outside of runtime generation at this time, because the RuntimeGenScheduler is responsible for priming the virtual textures.
	if (!Source || !Source->GetExecutionState().IsManagedByRuntimeGenSystem())
	{
		UE_LOGF(LogPCG, Error, "Virtual texture data interface on pin '%ls' failed to initialize. Virtual texture sampling is only supported for runtime generation.",
			*GetDownstreamInputPinLabelAliases()[0].ToString());
		return;
	}

	for (const FPCGTaggedData& TaggedData : InputTaggedData)
	{
		const UPCGVirtualTextureData* VirtualTextureData = Cast<UPCGVirtualTextureData>(TaggedData.Data);

		if (!ensure(VirtualTextureData))
		{
			UE_LOGF(LogPCG, Error, "Virtual texture data interface on pin '%ls' receieved unsupported data type: '%ls'",
				*GetDownstreamInputPinLabelAliases()[0].ToString(),
				TaggedData.Data ? *TaggedData.Data->GetName() : TEXT("NULL"));
			continue;
		}

		if (VirtualTextures.Num() < PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS)
		{
			VirtualTextures.Add(VirtualTextureData->GetRuntimeVirtualTexture());
		}
		else
		{
			UE_LOGF(LogPCG, Warning, "Texture data interface on pin '%ls' received too many textures to bind. Only the first %d textures will be bound.",
				*GetDownstreamInputPinLabelAliases()[0].ToString(),
				PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS);
			break;
		}
	}
}

FComputeDataProviderRenderProxy* UPCGVirtualTextureDataProvider::GetRenderProxy()
{
	return new FPCGVirtualTextureDataProviderProxy(VirtualTextures);
}

void UPCGVirtualTextureDataProvider::Reset()
{
	Super::Reset();

	VirtualTextures.Empty();
}

FPCGVirtualTextureDataProviderProxy::FPCGVirtualTextureDataProviderProxy(const TArray<TObjectPtr<const URuntimeVirtualTexture>>& InVirtualTextures)
{
	for (const TObjectPtr<const URuntimeVirtualTexture>& VirtualTexture : InVirtualTextures)
	{
		VirtualTextures.Add(VirtualTexture);
	}
}

void FPCGVirtualTextureDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	for (int32 BindingIndex = 0; BindingIndex < PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS; ++BindingIndex)
	{
		MaterialTypes[BindingIndex] = ERuntimeVirtualTextureMaterialType::Count;
	}

	for (int32 VirtualTextureIndex = 0; VirtualTextureIndex < VirtualTextures.Num(); ++VirtualTextureIndex)
	{
		if (TStrongObjectPtr<const URuntimeVirtualTexture> VirtualTexture = VirtualTextures[VirtualTextureIndex].Pin())
		{
			const ERuntimeVirtualTextureMaterialType MaterialType = VirtualTexture->GetMaterialType();
			MaterialTypes[VirtualTextureIndex] = MaterialType;
			
			PageTables[VirtualTextureIndex].Initialize(VirtualTexture.Get());

			const int32 NumVirtualTextureLayers = FMath::Min(URuntimeVirtualTexture::GetLayerCount(MaterialType), PCGVirtualTextureDIConstants::MAX_NUM_LAYERS);
			const int32 PackedLayerBaseIndex = VirtualTextureIndex * PCGVirtualTextureDIConstants::MAX_NUM_LAYERS;

			for (int32 LayerIndex = 0; LayerIndex < NumVirtualTextureLayers; ++LayerIndex)
			{
				const int32 PackedLayerIndex = PackedLayerBaseIndex + LayerIndex;
				Layers[PackedLayerIndex].Initialize(VirtualTexture.Get(), LayerIndex, VirtualTexture->IsLayerSRGB(LayerIndex));
			}
		}

	}
}

bool FPCGVirtualTextureDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	return InValidationData.ParameterStructSize == sizeof(FParameters);
}

struct FPCGVirtualTextureDataInterfacePermutationIds
{
	uint32 EnableMultipleVirtualTextures = 0;
 
	FPCGVirtualTextureDataInterfacePermutationIds(const FComputeKernelPermutationVector& PermutationVector)
	{
		static FString Name(EnableMultipleVirtualTexturePermutationName);
		static uint32 Hash = GetTypeHash(Name);
		EnableMultipleVirtualTextures = PermutationVector.GetPermutationBits(Name, Hash, /*Value=*/1);
	}
};

void FPCGVirtualTextureDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	FPCGVirtualTextureDataInterfacePermutationIds PermutationIds(InOutPermutationData.PermutationVector);
	for (int32 InvocationIndex = 0; InvocationIndex < InOutPermutationData.NumInvocations; ++InvocationIndex)
	{
		InOutPermutationData.PermutationIds[InvocationIndex] |= (VirtualTextures.Num() > 1 ? PermutationIds.EnableMultipleVirtualTextures : 0);
	}
}

void FPCGVirtualTextureDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVirtualTextures = VirtualTextures.Num();
		Parameters.Sampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		for (int32 BindingIndex = 0; BindingIndex < PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS; ++BindingIndex)
		{
			const uint32 MaterialType = (uint32)MaterialTypes[BindingIndex];
			const PCGVirtualTextureCommon::FVirtualTexturePageTable& PageTable = PageTables[BindingIndex];

			if (PageTable.IsValid())
			{
				Parameters.PageTable[BindingIndex] = PageTable.PageTableRef;
				Parameters.PageTableIndirection[BindingIndex] = PageTable.PageTableIndirectionRef;
				Parameters.PageTableInfo[BindingIndex] = FUintVector4(MaterialType, PageTable.bIsAdaptive ? 1 : 0, PageTable.bSinglePhysicalSpace ? 1 : 0, 0);
				Parameters.PageTableUniforms[BindingIndex * 2 + 0] = PageTable.PageTableUniforms[0];
				Parameters.PageTableUniforms[BindingIndex * 2 + 1] = PageTable.PageTableUniforms[1];

				const FLargeWorldRenderPosition TextureOrigin(PageTable.WorldToUVParameters[0]);
				Parameters.LWCTiles[BindingIndex] = TextureOrigin.GetTile();
				Parameters.WorldToUVTransforms[BindingIndex] = FMatrix44f(
					TextureOrigin.GetOffset(),
					FVector3f((FVector4f)PageTable.WorldToUVParameters[1]),
					FVector3f((FVector4f)PageTable.WorldToUVParameters[2]),
					FVector3f((FVector4f)PageTable.WorldToUVParameters[3]));
			}
			else
			{
				Parameters.PageTable[BindingIndex] = GBlackUintTexture->TextureRHI;
				Parameters.PageTableIndirection[BindingIndex] = GBlackUintTexture->TextureRHI;
				Parameters.PageTableInfo[BindingIndex] = FUintVector4(MaterialType, 0, 0, 0);
				Parameters.PageTableUniforms[BindingIndex * 2 + 0] = FUintVector4::ZeroValue;
				Parameters.PageTableUniforms[BindingIndex * 2 + 1] = FUintVector4::ZeroValue;
				Parameters.LWCTiles[BindingIndex] = FVector3f::ZeroVector;
				Parameters.WorldToUVTransforms[BindingIndex] = FMatrix44f::Identity;
			}

			const int32 PackedLayerBaseIndex = BindingIndex * PCGVirtualTextureDIConstants::MAX_NUM_LAYERS;
			const PCGVirtualTextureCommon::FVirtualTextureLayer& Layer0 = Layers[PackedLayerBaseIndex + 0];
			const PCGVirtualTextureCommon::FVirtualTextureLayer& Layer1 = Layers[PackedLayerBaseIndex + 1];
			const PCGVirtualTextureCommon::FVirtualTextureLayer& Layer2 = Layers[PackedLayerBaseIndex + 2];

			Parameters.ValidLayerMasks[BindingIndex] = FUintVector4::ZeroValue;

			if (Layer0.IsValid())
			{
				Parameters.ValidLayerMasks[BindingIndex].X |= 1;
				Parameters.VirtualTexture0[BindingIndex] = Layer0.TextureSRV;
				Parameters.Uniforms[PackedLayerBaseIndex + 0] = Layer0.TextureUniforms;
			}
			else
			{
				Parameters.VirtualTexture0[BindingIndex] = GBlackTextureWithSRV->ShaderResourceViewRHI;
				Parameters.Uniforms[PackedLayerBaseIndex + 0] = FUintVector4::ZeroValue;
			}

			if (Layer1.IsValid())
			{
				Parameters.ValidLayerMasks[BindingIndex].X |= 2;
				Parameters.VirtualTexture1[BindingIndex] = Layer1.TextureSRV;
				Parameters.Uniforms[PackedLayerBaseIndex + 1] = Layer1.TextureUniforms;
			}
			else
			{
				Parameters.VirtualTexture1[BindingIndex] = GBlackTextureWithSRV->ShaderResourceViewRHI;
				Parameters.Uniforms[PackedLayerBaseIndex + 1] = FUintVector4::ZeroValue;
			}

			if (Layer2.IsValid())
			{
				Parameters.ValidLayerMasks[BindingIndex].X |= 4;
				Parameters.VirtualTexture2[BindingIndex] = Layer2.TextureSRV;
				Parameters.Uniforms[PackedLayerBaseIndex + 2] = Layer2.TextureUniforms;
			}
			else
			{
				Parameters.VirtualTexture2[BindingIndex] = GBlackTextureWithSRV->ShaderResourceViewRHI;
				Parameters.Uniforms[PackedLayerBaseIndex + 2] = FUintVector4::ZeroValue;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
