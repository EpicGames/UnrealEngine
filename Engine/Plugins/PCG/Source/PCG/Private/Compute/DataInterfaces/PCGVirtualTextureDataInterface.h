// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/Data/PCGVirtualTextureCommon.h"
#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "PCGVirtualTextureDataInterface.generated.h"

class FPCGVirtualTextureDataInterfaceParameters;
class URuntimeVirtualTexture;

namespace PCGVirtualTextureDIConstants
{
	/**
	 * The maximum number of virtual textures which can be bound to a PCGVirtualTextureDataInterface.
	 * We can't use full bindless on all platforms, so fallback to emulating.
	 * If you change this number, make sure update PCGVirtualTextureDataInterface.ush as well.
	 */
	constexpr uint32 MAX_NUM_BINDINGS = 4;

	/**
	 * The maximum number of layers in a virtual texture.
	 * If you change this number, make sure update PCGVirtualTextureDataInterface.ush as well.
	 */
	constexpr const int32 MAX_NUM_LAYERS = 3;
}

/** Data Interface allowing sampling of a virtual texture. */
UCLASS(ClassGroup = (Procedural))
class UPCGVirtualTextureDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGVirtualTexture"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	TCHAR const* GetShaderVirtualPath() const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	void GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const override;
	void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading a virtual texture. */
UCLASS()
class UPCGVirtualTextureDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	virtual void Reset() override;
	//~ End UComputeDataProvider Interface

private:
	UPROPERTY()
	TArray<TObjectPtr<const URuntimeVirtualTexture>> VirtualTextures;
};

class FPCGVirtualTextureDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGVirtualTextureDataProviderProxy(const TArray<TObjectPtr<const URuntimeVirtualTexture>>& InVirtualTextures);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherPermutations(FPermutationData& InOutPermutationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGVirtualTextureDataInterfaceParameters;

	TArray<TWeakObjectPtr<const URuntimeVirtualTexture>> VirtualTextures;

	ERuntimeVirtualTextureMaterialType MaterialTypes[PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS];
	PCGVirtualTextureCommon::FVirtualTextureLayer Layers[PCGVirtualTextureDIConstants::MAX_NUM_LAYERS * PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS];
	PCGVirtualTextureCommon::FVirtualTexturePageTable PageTables[PCGVirtualTextureDIConstants::MAX_NUM_BINDINGS];
};
