// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/Elements/PCGComputeGraphElement.h"

#include "PCGComponent.h"
#include "PCGDefaultExecutionSource.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGProfilingLog.h"
#include "Components/PCGProceduralISMComponent.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGComputeKernel.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Data/PCGTextureData.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/DataInterfaces/PCGDataCollectionDataInterface.h"
#include "Elements/PCGStaticMeshSpawnerKernel.h"
#include "Editor/IPCGEditorModule.h"
#include "Graph/PCGGraphCompiler.h"
#include "Graph/PCGGraphExecutor.h"
#include "Helpers/PCGHelpers.h"
#include "Subsystems/IPCGBaseSubsystem.h"

#include "ComputeWorkerInterface.h"
#include "ComputeFramework/ComputeFramework.h"
#include "ComputeFramework/ComputeKernelCompileResult.h"
#include "Engine/World.h"
#include "Logging/LogVerbosity.h"
#include "UObject/GCObject.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGComputeGraphElement)

#define LOCTEXT_NAMESPACE "PCGComputeGraphElement"

namespace PCGComputeGraphElementHelpers
{
	/** Resolve a scene for compute dispatch. Prefers the execution source's world scene, falls back to the editor world scene for standalone graphs. */
	static FSceneInterface* GetSceneForCompute(const FPCGContext* InContext)
	{
		if (!InContext->ExecutionSource.IsValid())
		{
			return nullptr;
		}

		if (UWorld* World = InContext->ExecutionSource->GetExecutionState().GetWorld())
		{
			return World->Scene;
		}

#if WITH_EDITOR
		// Fallback to editor scene for standalone graph execution (UPCGDefaultExecutionSource has no world by design).
		if (GEditor && Cast<UPCGDefaultExecutionSource>(InContext->ExecutionSource.Get()))
		{
			if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
			{
				return EditorWorld->Scene;
			}
		}
#endif

		return nullptr;
	}
}

#if WITH_EDITOR
#define PCG_COMPUTE_GRAPH_BREAKPOINT() \
	if (Context->ComputeGraph && Context->ComputeGraph->bBreakDebugger && Context->IsExecutingGraphInspected()) \
	{ \
		if (const UEnum* EnumPtr = StaticEnum<EPCGComputeGraphExecutionPhase>()) \
		{ \
			UE_LOGF(LogPCG, Log, "BREAKPOINT: FPCGComputeGraphElement: %ls", *EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Context->ExecutionSubPhase)).ToString()); \
		} \
		\
		UE_DEBUG_BREAK(); \
	}

static TAutoConsoleVariable<bool> CVarReadbackTextureDataOnInspect(
	TEXT("pcg.GPU.ReadbackTextureDataOnInspect"),
	true,
	TEXT("Reads texture data back to the CPU when inspected."));

#else // WITH_EDITOR
#define PCG_COMPUTE_GRAPH_BREAKPOINT()
#endif // WITH_EDITOR

static TAutoConsoleVariable<bool> CVarPCGFixProviderGCRaceOnDestroy(
	TEXT("pcg.GPU.FixProviderGCRaceOnDestroy"),
	true,
	TEXT("When enabled, pins UComputeDataProvider objects in GC via an FGCObject while the deferred "
	     "game-thread cleanup lambda is in flight. Prevents a use-after-free when GC runs between "
	     "~FPCGComputeGraphContext (worker thread) and the lambda executing (game thread)."));

FPCGComputeGraphContext::~FPCGComputeGraphContext()
{
	const FPCGStack* StackPtr = GetStack();
	TWeakObjectPtr<const UPCGGraph> GraphWeak = StackPtr ? StackPtr->GetNearestNonInlinedGraphForCurrentFrame() : nullptr;
	TWeakObjectPtr<UPCGDataBinding> DataBindingWeak = MakeWeakObjectPtr(DataBinding);

	// Holds UComputeDataProvider objects alive in GC for the duration of the deferred lambda.
	// Without this, a GC cycle between the destructor (worker thread) and the lambda (game thread)
	// can collect the providers, making Provider.Get() return a dangling pointer.
	struct FProviderGCPinner : public FGCObject
	{
		TArray<TObjectPtr<UComputeDataProvider>> Providers;

		virtual void AddReferencedObjects(FReferenceCollector& Collector) override
		{
			Collector.AddReferencedObjects(Providers);
		}

		virtual FString GetReferencerName() const override
		{
			return TEXT("FPCGComputeGraphContext::FProviderGCPinner");
		}
	};

	TSharedPtr<FProviderGCPinner> GCPinner;
	if (CVarPCGFixProviderGCRaceOnDestroy.GetValueOnAnyThread() && ComputeGraphInstance.IsValid() && !IsInGameThread())
	{
		GCPinner = MakeShared<FProviderGCPinner>();
		GCPinner->Providers = ComputeGraphInstance->GetDataProviders();
	}

	// Can be here on a worker thread, after unpinning the context in one of the async tasks.
	// Will execute immediately if on game thread, otherwise on next game thread frame.
	PCGHelpers::ExecuteOnGameThread(TEXT("~FPCGComputeGraphContext"), [DataBindingWeak, GraphInstance=ComputeGraphInstance, bInstanceInitialized=bComputeGraphInstanceInitialized, GraphWeak, GridSize=GenerationGridSize, GraphIndex=ComputeGraphIndex, GCPinner]
	{
		LLM_SCOPE_BYTAG(PCG);

		if (DataBindingWeak.IsValid())
		{
			DataBindingWeak->ReleaseTransientResources();
		}

		if (GraphInstance.IsValid())
		{
			for (TObjectPtr<UComputeDataProvider>& Provider : GraphInstance->GetDataProviders())
			{
				if (UPCGComputeDataProvider* PCGDataProvider = Cast<UPCGComputeDataProvider>(Provider.Get()))
				{
					PCGDataProvider->ReleaseTransientResources(TEXT("~FPCGComputeGraphContext"));
				}
			}

			if (bInstanceInitialized)
			{
				GraphInstance->ResetDataProviders();

				// Return instance to pool.
				if (ensure(GraphWeak.IsValid()) && GridSize != PCGHiGenGrid::UninitializedGridSize())
				{
					const UPCGGraph::FComputeGraphInstanceKey Key = { GridSize, GraphIndex };
					GraphWeak->ReturnComputeGraphInstanceToPool(Key, GraphInstance);
				}
			}
		}

		// GCPinner goes out of scope here - FGCObject unregisters and providers
		// are released back to GC naturally after this point.
	});
}

bool FPCGComputeGraphContext::HasPendingAsyncOperations() const
{
	return !ProvidersWithBufferExports.IsEmpty();
}

bool FPCGComputeGraphContext::ResolveComputeGraph(const UPCGGraph* InGraph, uint32 GridSize, uint32 InComputeGraphIndex)
{
	if (TSharedPtr<FPCGGraphExecutor> Executor = GetGraphExecutor().Pin())
	{
		ComputeGraph = Executor->GetCompiler()->GetComputeGraph(InGraph, GridSize, InComputeGraphIndex);
	}

	return ComputeGraph != nullptr;
}

void FPCGComputeGraphContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	if (bComputeGraphInstanceInitialized)
	{
		Collector.AddPropertyReferences(FComputeGraphInstance::StaticStruct(), ComputeGraphInstance.Get());
	}

	for (TObjectPtr<UComputeDataProvider>& Provider : ProvidersWithBufferExports)
	{
		Collector.AddReferencedObject(Provider);
	}

	Collector.AddReferencedObject(DataBinding);
	Collector.AddReferencedObject(ComputeGraph);
}

#if WITH_EDITOR
bool FPCGComputeGraphElement::operator==(const FPCGComputeGraphElement& Other) const
{
	return ComputeGraphIndex == Other.ComputeGraphIndex;
}
#endif

bool FPCGComputeGraphElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGComputeGraphElement::ExecuteInternal);
	check(InContext);

	FPCGComputeGraphContext* Context = static_cast<FPCGComputeGraphContext*>(InContext);

	PCG_COMPUTE_GRAPH_BREAKPOINT();

	if (!InContext->ExecutionSource.IsValid())
	{
		UE_LOGF(LogPCG, Warning, "FPCGComputeGraphElement: Execution source lost, element execution halted.");
		return true;
	}

	FSceneInterface* Scene = PCGComputeGraphElementHelpers::GetSceneForCompute(InContext);

	if (!Scene)
	{
		UE_LOGF(LogPCG, Error, "FPCGComputeGraphElement: Could not resolve a scene for compute dispatch.");
		return true;
	}

	auto SleepUntilNextFrame = [Context]()
	{
		Context->bIsPaused = true;
		FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
		{
			if (TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin())
			{
				if (FPCGContext* ContextPtr = SharedHandle->GetContext())
				{
					ContextPtr->bIsPaused = false;
				}
			}
		});
	};

	switch (Context->ExecutionSubPhase)
	{
	case EPCGComputeGraphExecutionPhase::GetComputeGraph:
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetComputeGraph);

		const IPCGGraphExecutionSource* Source = InContext->ExecutionSource.Get();
		const UPCGGraph* TopGraph = Source ? Source->GetExecutionState().GetGraph() : nullptr;
		if (!TopGraph)
		{
			UE_LOGF(LogPCG, Error, "Graph was null, compute graph execution aborted.");
			return true;
		}

		const FPCGStack* StackPtr = Context->GetStack();
		if (!StackPtr)
		{
			UE_LOGF(LogPCG, Error, "Stack information missing from context, compute graph execution aborted.");
			return true;
		}

		// Higen is always disabled within dynamic subgraphs - will retrieve Uninitialized tasks (which are always cooked).
		const UPCGGraph* DynamicSubgraph = StackPtr ? StackPtr->GetNearestDynamicSubgraphForCurrentFrame() : nullptr;

		if (TopGraph->IsHierarchicalGenerationEnabled() && !DynamicSubgraph)
		{
			Context->GenerationGridSize = Source->GetExecutionState().GetGenerationGridSize();
		}
		
		if (!Context->ResolveComputeGraph(DynamicSubgraph ? DynamicSubgraph : TopGraph, Context->GenerationGridSize, ComputeGraphIndex))
		{
#if WITH_EDITOR
			// Normally emitted when a graph compilation is flushed while generation in progress.
			UE_LOGF(LogPCG, Warning, "Failed to obtain compute graph for %ls '%ls', grid size %u, compute graph index %d. Expected if graph compilation cache flushed mid-execution. Compute graph execution aborted.",
				DynamicSubgraph ? TEXT("dynamic subgraph") : TEXT("top graph"), DynamicSubgraph ? *DynamicSubgraph->GetName() : *TopGraph->GetName(), Context->GenerationGridSize, ComputeGraphIndex);
#else
			// Flush unlikely in standalone, so may be that cook failed to produce the compute graph.
			UE_LOGF(LogPCG, Error, "Failed to obtain compute graph for %ls '%ls', grid size %u, compute graph index %d. Either the graph compilation cache was flushed mid-execution, or the cook may have failed to produce the required compute graph. Compute graph execution aborted.",
				DynamicSubgraph ? TEXT("dynamic subgraph") : TEXT("top graph"), DynamicSubgraph ? *DynamicSubgraph->GetName() : *TopGraph->GetName(), Context->GenerationGridSize, ComputeGraphIndex);
#endif

			return true;
		}

		PCG_COMPUTE_GRAPH_BREAKPOINT();

		Context->ComputeGraphIndex = ComputeGraphIndex;

		if (!Context->ComputeGraph->AreGraphSettingsValid(Context))
		{
			return true;
		}

		Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::InitializeDataBindingAndComputeGraph;

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}
	case EPCGComputeGraphExecutionPhase::InitializeDataBindingAndComputeGraph: // Fallthrough
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(InitializeDataBindingAndComputeGraph);
		ensure(!Context->DataBinding);

		UPCGDataBinding* DataBinding = FPCGContext::NewObject_AnyThread<UPCGDataBinding>(Context);
		Context->DataBinding = DataBinding;

		DataBinding->Initialize(Context->ComputeGraph.Get(), Context);

		// Start data binding full initialization task which can run concurrently with initializing data providers.
		UE::Tasks::TTask<void> InitializeBindingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [ContextHandle = Context->GetOrCreateHandle()]()
		{
			LLM_SCOPE_BYTAG(PCG);
			FPCGContext::FSharedContext<FPCGComputeGraphContext> SharedContext(ContextHandle);
			FPCGComputeGraphContext* Context = SharedContext.Get();
			if (!Context)
			{
				return;
			}

			Context->DataBinding->InitializeTables(Context);
		});

		const FPCGStack* StackPtr = Context->GetStack();
		const UPCGGraph* Graph = StackPtr ? StackPtr->GetNearestNonInlinedGraphForCurrentFrame() : nullptr;
		const UPCGGraph::FComputeGraphInstanceKey Key = { Context->GenerationGridSize, ComputeGraphIndex };

		if (!ensure(Graph))
		{
			return true;
		}

		bool bNewInstance = false;
		Context->ComputeGraphInstance = Graph->RetrieveComputeGraphInstanceFromPool(Key, bNewInstance);

		// The data provider initialization must not depend on the data binding being set up, although they still output of the PreInitialize such
		// as the SourceComponent.
		if (bNewInstance)
		{
			Context->ComputeGraphInstance->CreateDataProviders(Context->ComputeGraph.Get(), 0, Context->DataBinding.Get());
		}
		else
		{
			Context->ComputeGraphInstance->InitializeDataProviders(Context->ComputeGraph.Get(), 0, Context->DataBinding.Get());
		}

#if WITH_EDITOR
		Context->ComputeGraphInstance->SetRenderCapturesEnabled(Context->IsExecutingGraphInspected());
#endif

		Context->bComputeGraphInstanceInitialized = true;

		// Register all providers running async operations.
		for (UComputeDataProvider* ComputeDataProvider : Context->ComputeGraphInstance->GetDataProviders())
		{
			UPCGExportableDataProvider* DataProvider = Cast<UPCGExportableDataProvider>(ComputeDataProvider);
			if (!DataProvider)
			{
				continue;
			}

			if (DataProvider->IsExportRequired())
			{
				Context->ProvidersWithBufferExports.Add(DataProvider);

				TWeakObjectPtr<UPCGExportableDataProvider> DataProviderWeak = DataProvider;

				const uint64 OriginatingGenerationCount = DataProvider->GetGenerationCounter();

				DataProvider->OnDataExported_GameThread().AddLambda([ContextHandle = Context->GetOrCreateHandle(), DataProviderWeak, GenerationCount = OriginatingGenerationCount]()
				{
					check(IsInGameThread());

					TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin();
					FPCGComputeGraphContext* ContextPtr = static_cast<FPCGComputeGraphContext*>(SharedHandle ? SharedHandle->GetContext() : nullptr);
					UPCGExportableDataProvider* DataProvider = DataProviderWeak.Get();

					if (!DataProvider || !ContextPtr || GenerationCount != DataProvider->GetGenerationCounter())
					{
						// Safe to just jump out. The GPU buffer is ref counted.
						return;
					}

					ContextPtr->ProvidersWithBufferExports.Remove(DataProvider);

					if (!ContextPtr->HasPendingAsyncOperations())
					{
						ContextPtr->bIsPaused = false;
					}
				});
			}
		}

		InitializeBindingTask.Wait();

		Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::InitializeKernelParams;

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}
	case EPCGComputeGraphExecutionPhase::InitializeKernelParams: // Fallthrough
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(InitializeKernelParams);

		if (UPCGDataBinding* DataBinding = Context->DataBinding.Get(); ensure(DataBinding))
		{
			// @todo_pcg: Move this to a worker thread.
			if (!DataBinding->InitializeKernelParams(Context))
			{
				SleepUntilNextFrame();
				return false;
			}
		}

		Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::PreExecuteReadbacks;

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}
	case EPCGComputeGraphExecutionPhase::PreExecuteReadbacks: // Fallthrough
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PreExecuteReadbacks);

		bool bAllReady = true;

		for (UComputeDataProvider* ComputeDataProvider : Context->ComputeGraphInstance->GetDataProviders())
		{
			if (UPCGComputeDataProvider* DataProvider = Cast<UPCGComputeDataProvider>(ComputeDataProvider))
			{
				if (!DataProvider->PerformPreExecuteReadbacks_GameThread(Context->DataBinding.Get()))
				{
					bAllReady = false;
				}
			}
		}

		if (bAllReady)
		{
			Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::PrimeDataDescriptionsAndValidateData;
		}
		else
		{
			SleepUntilNextFrame();
			return false;
		}

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}
	case EPCGComputeGraphExecutionPhase::PrimeDataDescriptionsAndValidateData: // Fallthrough
	{
		// Note: Priming the data description cache must take place after pre-execute readbacks, as data descriptions
		// may rely on the readback data, e.g. analysis kernels.
		if (!Context->bDataDescrPrimeAndValidateScheduled)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrimeDataDescriptionsAndValidateData);

			Context->bDataDescrPrimeAndValidateScheduled = true;

			// @todo_pcg: In the future perhaps we can just use Context->ScheduleGeneric(), but it's unclear at the moment if there is any guarantee this task
			// would be kicked off this tick. To avoid the unknown, we'll just launch a task directly here, but it should be investigated in the future.
			Context->DataDescrPrimeAndValidateTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [ContextHandle = Context->GetOrCreateHandle()]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(DataDescrPrimeAndValidateTask);
				LLM_SCOPE_BYTAG(PCG);

				FPCGContext::FSharedContext<FPCGComputeGraphContext> SharedContext(ContextHandle);
				FPCGComputeGraphContext* Context = SharedContext.Get();
				if (!Context)
				{
					return;
				}

				Context->DataBinding->PrimeDataDescriptionCache();

				// Graph data validation is currently allowed to depend on data descriptions, hence done serially here.
				Context->bGraphValid = Context->ComputeGraph->IsGraphDataValid(Context->DataBinding.Get(), Context);
			});

			return false;
		}

		if (!ensure(Context->DataDescrPrimeAndValidateTask.IsValid()))
		{
			return true;
		}

		if (!Context->DataDescrPrimeAndValidateTask.IsCompleted())
		{
			SleepUntilNextFrame();
			return false;
		}

		Context->DataBinding->DebugLogDataDescriptions();

		if (!Context->bGraphValid)
		{
			return true;
		}

		Algo::Transform(Context->ComputeGraphInstance->GetDataProviders(), Context->DataProvidersPendingReadyForExecute, [](UComputeDataProvider* InProvider)
		{
			return Cast<UPCGComputeDataProvider>(InProvider);
		});

		Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::PrepareForExecute;

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}
	case EPCGComputeGraphExecutionPhase::PrepareForExecute: // Fallthrough
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareForExecute);

		for (int Index = Context->DataProvidersPendingReadyForExecute.Num() - 1; Index >= 0; --Index)
		{
			UPCGComputeDataProvider* DataProvider = Context->DataProvidersPendingReadyForExecute[Index];

			if (!DataProvider || DataProvider->PrepareForExecute_GameThread(Context->DataBinding.Get()))
			{
				Context->DataProvidersPendingReadyForExecute.RemoveAtSwap(Index);
			}
		}

		if (Context->DataProvidersPendingReadyForExecute.IsEmpty())
		{
			Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::ValidateComputeGraphCompilation;
		}
		else
		{
			SleepUntilNextFrame();
			return false;
		}

#if PCG_PROFILING_ENABLED
		// Compute per-node output memory sizes from the data descriptions before they are flushed.
		ComputeNodeOutputMemorySize(Context);
#endif

		// After prepare for execute flush the data description cache because the cache uses non-negligible memory.
		Context->DataBinding->FlushDataDescriptionCache();

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}
	case EPCGComputeGraphExecutionPhase::ValidateComputeGraphCompilation: // Fallthrough
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ValidateComputeGraphCompilation);

		if (Context->ComputeGraph->HasKernelResourcesPendingShaderCompilation())
		{
			UE_LOGF(LogPCG, Verbose, "Deferring until next frame as the kernel has pending shader compilations.");
			SleepUntilNextFrame();
			return false;
		}
		else if (!Context->ComputeGraph->GetRenderProxy())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Context->ComputeGraph->UpdateResources);

			Context->ComputeGraph->UpdateResources(/*bSync=*/!ComputeFramework::IsDeferredCompilation());

			SleepUntilNextFrame();
			return false;
		}
		else
		{
			// Add any messages that may have occurred during compilation to visual logs.
#if WITH_EDITOR
			LogCompilationMessages(Context);
#endif

			// If there was any error then we should abort.
			using FNodeAndCompileMessages = const TPair<TObjectKey<const UPCGNode>, TArray<FComputeKernelCompileMessage>>;
			for (FNodeAndCompileMessages& NodeAndCompileMessages : Context->ComputeGraph->KernelToCompileMessages)
			{
				for (const FComputeKernelCompileMessage& Message : NodeAndCompileMessages.Get<1>())
				{
					// Some error messages were getting lost, and we were only getting the final 'failed' message. Treat this as failure and report for now.
					// TODO: Revert the 'failed' part once we're happy all relevant issues are bubbling up.
					if (Message.Type == FComputeKernelCompileMessage::EMessageType::Error || Message.Text.Contains(TEXT("failed"), ESearchCase::IgnoreCase))
					{
						return true;
					}
				}
			}
		}

		Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::ScheduleComputeGraph;

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}
	case EPCGComputeGraphExecutionPhase::ScheduleComputeGraph: // Fallthrough
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ScheduleComputeGraph);

#if HAS_GPU_STATS
		// Discard any futures from a previous dispatch round. Old stat objects are deleted by ProcessFrame after all per-queue OnTimingResults calls have fired.
		Context->KernelTimingFutures.Reset();
		const int32 NumKernels = Context->ComputeGraph->GetNumKernels();
		Context->KernelTimingFutures.Reserve(NumKernels);
		TArray<TPromise<FComputeFrameworkKernelTiming>> KernelTimingPromises;
		KernelTimingPromises.Reserve(NumKernels);
		for (int32 KernelIndex = 0; KernelIndex < NumKernels; ++KernelIndex)
		{
			TPromise<FComputeFrameworkKernelTiming> Promise;
			Context->KernelTimingFutures.Add(Promise.GetFuture());
			KernelTimingPromises.Add(MoveTemp(Promise));
		}
#endif // HAS_GPU_STATS

		const bool bGraphEnqueued = Context->ComputeGraphInstance->EnqueueWork(
			Context->ComputeGraph.Get(),
			Scene,
			Context->ComputeGraph->GetRequiredExecutionGroup(),
			FName(*InContext->ExecutionSource->GetExecutionState().GetDebugName()),
			/*InFallbackDelegate=*/FSimpleDelegate::CreateLambda([ContextHandle = Context->GetOrCreateHandle()]()
			{
				// This render thread delegate will be executed if SubmitWork fails at any stage.
				FPCGContext::FSharedContext<FPCGComputeGraphContext> SharedContext(ContextHandle);
				FPCGComputeGraphContext* Context = SharedContext.Get();
				if (!Context)
				{
					return;
				}

				// Wake up so that we can terminate execution of the ComputeGraphElement.
				Context->bIsPaused = false;
				Context->bGraphSubmitFailed = true;
			}),
			/*InOwnerPointer=*/Context->DataBinding // Use data binding as owner UObject as binding is created per-execution, and owner is needed if we abort the work.
#if HAS_GPU_STATS
			, /*InGraphSortPriorityOffset=*/0, MoveTemp(KernelTimingPromises)
#endif
		);

		if (!bGraphEnqueued)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("EnqueueFailed", "Compute graph enqueue failed, check log for errors."));
			ResetAsyncOperations(InContext);
			return true;
		}

#ifdef PCG_GPU_KERNEL_PROFILING
		if (Context->ComputeGraph->ShouldRepeatDispatch())
		{
			UE_LOGF(LogPCG, Log, "Repeating dispatch (graph execution will not complete).");
		}
		else
#endif // PCG_GPU_KERNEL_PROFILING
		{
			Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::WaitForExecutionComplete;
		}

		SleepUntilNextFrame();

		return false;
	}
	case EPCGComputeGraphExecutionPhase::WaitForExecutionComplete:
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WaitForExecutionComplete);

		if (Context->bGraphSubmitFailed)
		{
			UE_LOGF(LogPCG, Warning, "Submit of compute graph '%ls' failed.", *Context->ComputeGraph->GetName());
			return true;
		}

		if (Context->HasPendingAsyncOperations())
		{
			// Still running. Likely we need a frame to pass in order to make progress with readbacks etc.
			SleepUntilNextFrame();

			return false;
		}

		Context->bExecutionSuccess = true;

		Context->DataProvidersPendingPostExecute.Reserve(Context->ComputeGraphInstance->GetNumDataProviders());
		for (UComputeDataProvider* Provider : Context->ComputeGraphInstance->GetDataProviders())
		{
			UPCGComputeDataProvider* PCGProvider = Cast<UPCGComputeDataProvider>(Provider);
			if (PCGProvider)
			{
				Context->DataProvidersPendingPostExecute.Add(PCGProvider);
			}
		}

		Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::PostExecute;

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
	}
	case EPCGComputeGraphExecutionPhase::PostExecute: // Fallthrough
	{
		for (int Index = Context->DataProvidersPendingPostExecute.Num() - 1; Index >= 0; --Index)
		{
			UPCGComputeDataProvider* DataProvider = Context->DataProvidersPendingPostExecute[Index];

			if (!DataProvider || DataProvider->PostExecute(Context->DataBinding.Get()))
			{
				Context->DataProvidersPendingPostExecute.RemoveAtSwap(Index);
			}
		}

		if (!Context->DataProvidersPendingPostExecute.IsEmpty())
		{
			SleepUntilNextFrame();
			return false;
		}

		// Currently we don't output anything if processing any readback data processing failed.
		if (ensure(Context->bExecutionSuccess) && ensure(Context->DataBinding))
		{
			Context->OutputData = Context->DataBinding->GetOutputDataCollection();
		}

#if WITH_EDITOR
		Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::DebugAndInspection;

		if (Context->AsyncState.ShouldStop())
		{
			return false;
		}
#elif HAS_GPU_STATS
		Context->ExecutionSubPhase = Context->KernelTimingFutures.IsEmpty() ? EPCGComputeGraphExecutionPhase::Done : EPCGComputeGraphExecutionPhase::WaitForPerformanceData;
		if (Context->ExecutionSubPhase == EPCGComputeGraphExecutionPhase::Done)
		{
			return true;
		}
		// Falls through the empty DebugAndInspection case body to WaitForPerformanceData.
#else
		Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::Done;
		return true;
#endif
	}
	case EPCGComputeGraphExecutionPhase::DebugAndInspection: // Fallthrough
	{
#if WITH_EDITOR
		CollectDebugDataPrepareActions(Context);

		bool bAllReady = true;

		for (int32 Index = Context->DebugDataPrepareActions.Num() - 1; Index >= 0; --Index)
		{
			FPCGComputeGraphContext::FDebugDataPrepareAction Action = Context->DebugDataPrepareActions[Index];
			const bool bActionIsDone = !Action.IsSet() || Action(Context);

			bAllReady &= bActionIsDone;

			if (bActionIsDone)
			{
				Context->DebugDataPrepareActions.RemoveAtSwap(Index);
			}
		}

		if (!bAllReady)
		{
			SleepUntilNextFrame();
			return false;
		}

		ExecuteDebugDraw(Context);
		StoreDataForInspection(Context);

#if HAS_GPU_STATS
		Context->ExecutionSubPhase = Context->KernelTimingFutures.IsEmpty() ? EPCGComputeGraphExecutionPhase::Done : EPCGComputeGraphExecutionPhase::WaitForPerformanceData;
#else
		Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::Done;
#endif
#endif // WITH_EDITOR
	}
#if HAS_GPU_STATS
	case EPCGComputeGraphExecutionPhase::WaitForPerformanceData:
	{
		bool bAllFuturesReady = true;
		for (const TFuture<FComputeFrameworkKernelTiming>& Future : Context->KernelTimingFutures)
		{
			if (!Future.IsReady())
			{
				bAllFuturesReady = false;
				break;
			}
		}

		if (!bAllFuturesReady)
		{
			SleepUntilNextFrame();
			return false;
		}

		Context->ExecutionSubPhase = EPCGComputeGraphExecutionPhase::Done;
	}
#endif // HAS_GPU_STATS
	}

	return true;
}

void FPCGComputeGraphElement::PostExecuteInternal(FPCGContext* InContext) const
{
#if PCG_PROFILING_ENABLED
	check(InContext);
	FPCGComputeGraphContext* Context = static_cast<FPCGComputeGraphContext*>(InContext);
	const FPCGStack* Stack = Context->GetStack();
	if (!Context->DataBinding || !ensure(Stack))
	{
		return;
	}

	if (Context->bExecutionSuccess)
	{
		IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
		const TArray<TSoftObjectPtr<const UPCGNode>>& KernelToNode = Context->DataBinding->GetComputeGraph()->KernelToNode;

		// Accumulate per-node timers across all kernels. A single PCG node can emit multiple compute kernels; GPU time is summed so the profiler shows the full cost rather than just the first kernel's.
		TMap<const UPCGNode*, PCGUtils::FCallTime> NodeTimers;
		NodeTimers.Reserve(KernelToNode.Num());
		for (int32 KernelIndex = 0; KernelIndex < KernelToNode.Num(); ++KernelIndex)
		{
			const UPCGNode* Node = KernelToNode[KernelIndex].Get();
			if (!Node)
			{
				continue;
			}

			PCGUtils::FCallTime& Timer = NodeTimers.FindOrAdd(Node);

			if (const uint64* MemSize = Context->NodeOutputMemorySizes.Find(TObjectKey<const UPCGNode>(Node)))
			{
				Timer.OutputGPUMemorySize = *MemSize;
			}

#if HAS_GPU_STATS
			if (Context->KernelTimingFutures.IsValidIndex(KernelIndex) && Context->KernelTimingFutures[KernelIndex].IsReady())
			{
				Timer.GPUTime = Timer.GPUTime.Get(0.0) + Context->KernelTimingFutures[KernelIndex].Get().BusyMs / 1000.0;
			}
#endif
		}

		if (ExecutionSource && Stack
#if !WITH_EDITOR
			&& PCGProfilingLog::IsEnabled()
#endif
			)
		{
			for (auto& [Node, Timer] : NodeTimers)
			{
				ExecutionSource->GetExecutionState().GetInspection().NotifyNodeExecuted(Node, Stack, &Timer, /*bUsedCache*/false);
			}
		}
	}
#endif // PCG_PROFILING_ENABLED
}

void FPCGComputeGraphElement::AbortInternal(FPCGContext* InContext) const
{
	if (FPCGComputeGraphContext* Context = static_cast<FPCGComputeGraphContext*>(InContext))
	{
		FSceneInterface* ComputeScene = PCGComputeGraphElementHelpers::GetSceneForCompute(InContext);
		if (Context->ComputeGraphInstance && Context->DataBinding && ComputeScene)
		{
			ComputeFramework::AbortWork(ComputeScene, Context->DataBinding);
		}
	}

	ResetAsyncOperations(InContext);
}

void FPCGComputeGraphElement::ResetAsyncOperations(FPCGContext* InContext) const
{
	check(IsInGameThread());

	if (InContext)
	{
		FPCGComputeGraphContext* Context = static_cast<FPCGComputeGraphContext*>(InContext);
		Context->ProvidersWithBufferExports.Reset();
	}
}

#if WITH_EDITOR
void FPCGComputeGraphElement::CollectDebugDataPrepareActions(FPCGComputeGraphContext* InContext) const
{
	if (!InContext || !InContext->DataBinding)
	{
		return;
	}

	const bool bReadbackTextureDataOnInspect = CVarReadbackTextureDataOnInspect.GetValueOnGameThread();

	auto CollectDebugDataPrepareAction = [InContext, bReadbackTextureDataOnInspect](FPCGDataToDebug& DataToDebug, bool bInIsInspecting)
	{
		if (const UPCGProxyForGPUData* ProxyData = Cast<UPCGProxyForGPUData>(DataToDebug.Data.Get()))
		{
			InContext->DebugDataPrepareActions.Add([&DataToDebug](FPCGComputeGraphContext* InContext)
			{
				const UPCGProxyForGPUData* ProxyData = CastChecked<UPCGProxyForGPUData>(DataToDebug.Data.Get());
				UPCGProxyForGPUData::FReadbackResult Readback = ProxyData->GetCPUData(InContext);
				return Readback.bComplete;
			});
		}
		else if (const UPCGTextureData* TextureData = Cast<UPCGTextureData>(DataToDebug.Data.Get()))
		{
			InContext->DebugDataPrepareActions.Add([&DataToDebug, bInIsInspecting, bReadbackTextureDataOnInspect](FPCGComputeGraphContext* InContext)
			{
				if (bInIsInspecting && !bReadbackTextureDataOnInspect)
				{
					return true;
				}

				// Duplicate and CPU-initialize the debug texture data.
				const UPCGTextureData* TextureData = CastChecked<UPCGTextureData>(DataToDebug.Data.Get());
				TObjectPtr<UPCGTextureData> DuplicateTextureData = Cast<UPCGTextureData>(DataToDebug.DataPendingInit);

				if (!DuplicateTextureData)
				{
					DuplicateTextureData = InContext->NewObject_AnyThread<UPCGTextureData>(InContext, GetTransientPackage());
					DuplicateTextureData->TexelSize = TextureData->TexelSize;
					DataToDebug.DataPendingInit = DuplicateTextureData;
				}

				return DuplicateTextureData->Initialize(TextureData->GetRefCountedTexture(), TextureData->TextureIndex, TextureData->GetTransform());
			});
		}
	};

	for (FPCGDataToDebug& DataToDebug : InContext->DataBinding->GetDataToDebugMutable())
	{
		CollectDebugDataPrepareAction(DataToDebug, /*bIsInspecting=*/false);
	}

	for (FPCGDataToDebug& DataToDebug : InContext->DataBinding->GetDataToInspectMutable())
	{
		CollectDebugDataPrepareAction(DataToDebug, /*bIsInspecting=*/true);
	}
}

void FPCGComputeGraphElement::ExecuteDebugDraw(FPCGComputeGraphContext* InContext) const
{
	if (!InContext || !InContext->DataBinding)
	{
		return;
	}

	const FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetConstPCGDataVisualizationRegistry();

	TArray<FPCGDataToDebug>& AllDataToDebug = InContext->DataBinding->GetDataToDebugMutable();

	for (int Index = AllDataToDebug.Num() - 1; Index >= 0; --Index)
	{
		FPCGDataToDebug& DataToDebug = AllDataToDebug[Index];
		const UPCGData* DataToDisplay = nullptr;

		if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(DataToDebug.Data.Get()))
		{
			UPCGProxyForGPUData::FReadbackResult Readback = Proxy->GetCPUData(InContext);
			ensure(Readback.bComplete);

			DataToDisplay = Readback.TaggedData.Data.Get();
		}
		else if (Cast<UPCGTextureData>(DataToDebug.Data.Get()))
		{
			DataToDisplay = DataToDebug.DataPendingInit;
		}
		else if (Cast<UPCGTexture2DArrayData>(DataToDebug.Data.Get()))
		{
			DataToDisplay = DataToDebug.Data;
		}

		AllDataToDebug.RemoveAtSwap(Index);

		const UPCGSettings* ProducerSettings = DataToDebug.ProducerSettings.Get();

		if (DataToDisplay && ProducerSettings)
		{
			if (const IPCGDataVisualization* DataVis = DataVisRegistry.GetDataVisualization(DataToDisplay->GetClass()))
			{
				DataVis->ExecuteDebugDisplay(InContext, ProducerSettings, DataToDisplay, InContext->GetTypedExecutionTarget<AActor>());
			}
		}
	}
}

void FPCGComputeGraphElement::StoreDataForInspection(FPCGComputeGraphContext* InContext) const
{
	if (!InContext || !InContext->DataBinding || !InContext->ExecutionSource.IsValid())
	{
		return;
	}

	// Collect all data into a collection and store it.
	TMap<const UPCGSettings*, FPCGDataCollection> SettingsToDataCollection;

	for (FPCGDataToDebug& DataToInspect : InContext->DataBinding->GetDataToInspectMutable())
	{
		if (const UPCGSettings* ProducerSettings = DataToInspect.ProducerSettings.Get())
		{
			FPCGDataCollection& DataCollection = SettingsToDataCollection.FindOrAdd(ProducerSettings);

			if (const UPCGProxyForGPUData* ProxyData = Cast<UPCGProxyForGPUData>(DataToInspect.Data.Get()))
			{
				UPCGProxyForGPUData::FReadbackResult Readback = ProxyData->GetCPUData(InContext);
				FPCGTaggedData TaggedData = Readback.TaggedData;
				TaggedData.Pin = DataToInspect.PinLabel;
				TaggedData.Tags.Append(DataToInspect.Tags);
				DataCollection.TaggedData.Add(TaggedData);
			}
			else if (Cast<UPCGTextureData>(DataToInspect.Data.Get()))
			{
				const bool bReadbackTextureDataOnInspect = CVarReadbackTextureDataOnInspect.GetValueOnGameThread();

				FPCGTaggedData TaggedData;
				TaggedData.Data = bReadbackTextureDataOnInspect ? DataToInspect.DataPendingInit : DataToInspect.Data;
				TaggedData.Pin = DataToInspect.PinLabel;
				TaggedData.Tags = DataToInspect.Tags;
				DataCollection.TaggedData.Add(TaggedData);
			}
			else if (Cast<UPCGTexture2DArrayData>(DataToInspect.Data.Get()))
			{
				FPCGTaggedData TaggedData;
				TaggedData.Data = DataToInspect.Data;
				TaggedData.Pin = DataToInspect.PinLabel;
				TaggedData.Tags = DataToInspect.Tags;
				DataCollection.TaggedData.Add(TaggedData);
			}
		}
	}

	for (TPair<const UPCGSettings*, FPCGDataCollection>& SettingsAndData : SettingsToDataCollection)
	{
		// Required by inspection code.
		SettingsAndData.Value.ComputeCrcs(/*bFullDataCrc=*/false);

		// Look up node from compute graph's kernel-to-node mapping.
		const UPCGNode* Node = nullptr;
		if (const UPCGComputeGraph* ComputeGraph = InContext->ComputeGraph.Get())
		{
			for (int32 KernelIndex = 0; KernelIndex < ComputeGraph->GetNumKernels(); ++KernelIndex)
			{
				if (const UPCGComputeKernel* Kernel = Cast<UPCGComputeKernel>(ComputeGraph->GetKernel(KernelIndex)))
				{
					if (Kernel->GetSettings() == SettingsAndData.Key)
					{
						Node = Kernel->GetNode();
						break;
					}
				}
			}
		}

		if (!Node)
		{
			Node = Cast<UPCGNode>(SettingsAndData.Key->GetOuter());
		}

		// TODO: Input data not yet supported.
		InContext->ExecutionSource->GetExecutionState().GetInspection().StoreInspectionData(InContext->GetStack(), Node, /*InTimer=*/nullptr, /*InInputData=*/{}, SettingsAndData.Get<1>(), /*bUsedCache*/false);
	}

	InContext->DataBinding->GetDataToInspectMutable().Empty();
}

void FPCGComputeGraphElement::LogCompilationMessages(FPCGComputeGraphContext* InContext) const
{
	if (InContext->ExecutionSource.IsValid() && InContext->GetStack())
	{
		using FNodeAndCompileMessages = const TPair<TObjectKey<const UPCGNode>, TArray<FComputeKernelCompileMessage>>;
		for (FNodeAndCompileMessages& NodeAndCompileMessages : InContext->ComputeGraph->KernelToCompileMessages)
		{
			for (const FComputeKernelCompileMessage& Message : NodeAndCompileMessages.Get<1>())
			{
				// These messages already go to log. So just pick out the warnings and errors to display on graph. Need to convert
				// message type.
				ELogVerbosity::Type Verbosity = ELogVerbosity::All;
				if (Message.Type == FComputeKernelCompileMessage::EMessageType::Warning)
				{
					Verbosity = ELogVerbosity::Warning;
				}
				else if (Message.Type == FComputeKernelCompileMessage::EMessageType::Error)
				{
					Verbosity = ELogVerbosity::Error;
				}
				else if (Message.Text.Contains(TEXT("failed"), ESearchCase::IgnoreCase))
				{
					// Some error messages were getting lost, and we were only getting the final 'failed' message.
					// Treat this as failure and report for now.
					// TODO: Revert this once we're happy all relevant issues are bubbling up.
					Verbosity = ELogVerbosity::Error;
				}

				if (Verbosity < ELogVerbosity::Log)
				{
					if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
					{
						FPCGStack StackWithNode = *InContext->GetStack();
						StackWithNode.PushFrame(NodeAndCompileMessages.Get<0>().ResolveObjectPtr());

						FText LogText;

						if (Message.Line != INDEX_NONE)
						{
							if (Message.ColumnStart != INDEX_NONE)
							{
								LogText = FText::Format(LOCTEXT("ErrorWithLineColFormat", "[{0},{1}] {2}"), Message.Line, Message.ColumnStart, FText::FromString(Message.Text));
							}
							else
							{
								LogText = FText::Format(LOCTEXT("ErrorWithLineFormat", "[{0}] {1}"), Message.Line, FText::FromString(Message.Text));
							}
						}
						else
						{
							LogText = FText::FromString(Message.Text);
						}

						PCGEditorModule->GetNodeVisualLogsMutable().Log(StackWithNode, Verbosity, LogText);
					}
				}
			}
		}
	}
}
#endif // WITH_EDITOR

#if PCG_PROFILING_ENABLED
void FPCGComputeGraphElement::ComputeNodeOutputMemorySize(FPCGComputeGraphContext* InContext) const
{
	check(IsInGameThread());

	if (!InContext)
	{
		return;
	}

	const UPCGComputeGraph* ComputeGraph = InContext->ComputeGraph;
	if (!ComputeGraph || !InContext->DataBinding)
	{
		return;
	}

	for (int32 KernelIndex = 0; KernelIndex < ComputeGraph->KernelToNode.Num(); ++KernelIndex)
	{
		const UPCGComputeKernel* Kernel = Cast<UPCGComputeKernel>(ComputeGraph->GetKernel(KernelIndex));
		const UPCGNode* Node = ComputeGraph->KernelToNode[KernelIndex].Get();
		if (!Kernel || !Node)
		{
			continue;
		}

		TArray<FPCGPinPropertiesGPU> OutputPins;
		Kernel->GetOutputPins(OutputPins);

		uint64 KernelMemorySize = 0;
		for (const FPCGPinPropertiesGPU& OutputPin : OutputPins)
		{
			if (TSharedPtr<const FPCGDataCollectionDesc> DataDesc = InContext->DataBinding->GetCachedKernelPinDataDesc(Kernel, OutputPin.Label, /*bIsInput=*/false))
			{
				KernelMemorySize += PCGComputeHelpers::ComputeSizeBytes(DataDesc);
			}
		}

		InContext->NodeOutputMemorySizes.FindOrAdd(Node) += KernelMemorySize;
	}
}
#endif // PCG_PROFILING_ENABLED

UPCGComputeGraphSettings::UPCGComputeGraphSettings()
{
#if WITH_EDITOR
	bExposeToLibrary = false;
#endif
}

FPCGElementPtr UPCGComputeGraphSettings::CreateElement() const
{
	return MakeShared<FPCGComputeGraphElement>(ComputeGraphIndex);
}

#undef PCG_COMPUTE_GRAPH_BREAKPOINT
#undef LOCTEXT_NAMESPACE
