// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "ComputeFramework/ComputeGraphInstance.h"

#include "Async/Future.h"
#include "Tasks/Task.h"
#include "UObject/StrongObjectPtr.h"

#include "PCGComputeGraphElement.generated.h"

class UPCGComputeDataProvider;

UENUM()
enum class EPCGComputeGraphExecutionPhase : uint8
{
	None,
	GetComputeGraph,
	InitializeDataBindingAndComputeGraph,
	InitializeKernelParams,
	PreExecuteReadbacks,
	PrimeDataDescriptionsAndValidateData,
	PrepareForExecute,
	ValidateComputeGraphCompilation,
	ScheduleComputeGraph,
	WaitForExecutionComplete,
	PostExecute,
	DebugAndInspection,
	/** Wait for per-kernel GPU timing results to arrive via async callbacks (editor only, requires HAS_GPU_STATS). */
	WaitForPerformanceData,
	Done,
};

struct FPCGComputeGraphContext : public FPCGContext
{
public:
	virtual bool IsComputeContext() const override { return true; }
	virtual ~FPCGComputeGraphContext();

	bool HasPendingAsyncOperations() const;

	/** Resolves and assigns ComputeGraph from the executor's compiler cache. Returns true if the graph was found. */
	bool ResolveComputeGraph(const UPCGGraph* InGraph, uint32 GridSize, uint32 InComputeGraphIndex);

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;

public:
	TObjectPtr<UPCGDataBinding> DataBinding = nullptr;
	TObjectPtr<UPCGComputeGraph> ComputeGraph = nullptr;

	uint32 GenerationGridSize = PCGHiGenGrid::UninitializedGridSize();

	int32 ComputeGraphIndex = INDEX_NONE;

	/** Data providers created from data interfaces and data bindings. */
	TSharedPtr<FComputeGraphInstance> ComputeGraphInstance = nullptr;

	/** Whether the compute graph instance has been initialized (data providers created, etc). */
	bool bComputeGraphInstanceInitialized = false;

	/** Whether the task to prime the data description cache and validate the input data has been scheduled. */
	bool bDataDescrPrimeAndValidateScheduled = false;
	UE::Tasks::TTask<void> DataDescrPrimeAndValidateTask;

	/** Graph was enqueued but was invalid for some reason and the work could not be submitted. */
	bool bGraphSubmitFailed = false;

	/** Graph executed successfully. */
	bool bExecutionSuccess = false;

	/** The data providers which are not yet ready for execution. */
	TArray<UPCGComputeDataProvider*> DataProvidersPendingReadyForExecute;

	TArray<UPCGComputeDataProvider*> DataProvidersPendingPostExecute;

	/** Data providers with buffers that are passed to downstream tasks. The buffer will be created on the render thread, and then passed
	* back to main thread, upon which a reference is taken to the buffer and the provider is removed from this set to signal completion.
	*/
	TSet<TObjectPtr<UComputeDataProvider>> ProvidersWithBufferExports;

	EPCGComputeGraphExecutionPhase ExecutionSubPhase = EPCGComputeGraphExecutionPhase::GetComputeGraph;

	std::atomic<bool> bGraphValid = false;

#if PCG_PROFILING_ENABLED
	/** Accumulated output memory size per GPU node, computed from data descriptions before the cache is flushed. */
	TMap<TObjectKey<const UPCGNode>, uint64> NodeOutputMemorySizes;
#endif

	using FDebugDataPrepareAction = TFunction<bool(FPCGComputeGraphContext*)>;
	TArray<FDebugDataPrepareAction> DebugDataPrepareActions;

#if HAS_GPU_STATS
	/** Per-kernel futures that become ready when the GPU profiler fires the timing callback for each kernel.
	 *  Fulfilled by FComputeFrameworkGPUStat::OnTimingResults (value = busy time in ms) or with 0.0 on abort.
	 *  WaitForPerformanceData polls these until all are ready, then PostExecuteInternal reads the values. */
	TArray<TFuture<FComputeFrameworkKernelTiming>> KernelTimingFutures;
#endif
};

/** Executes a CF graph. Created by the compiler when collapsing GPU nodes rather than by a settings/node. */
class FPCGComputeGraphElement : public IPCGElement
{
public:
	FPCGComputeGraphElement() = default;
	explicit FPCGComputeGraphElement(int32 InComputeGraphIndex)
		: ComputeGraphIndex(InComputeGraphIndex)
	{}

#if WITH_EDITOR
	//~Begin IPCGElement interface
	virtual bool IsComputeGraphElement() const override { return true; }
	//~End IPCGElement interface

	/** Return true if the elements are identical, used for change detection. */
	bool operator==(const FPCGComputeGraphElement& Other) const;
#endif

	// TODO: ComputeGraphIndex could be removed from the element if we properly hook up EPCGElementSource::FromCookedSettings in the graph executor 
	// to manufacture the context's settings from the FPCGGraphTask Settings member. Would also eliminate the need for IsComputeGraphElement().
	int32 ComputeGraphIndex = INDEX_NONE;

protected:
	virtual FPCGContext* CreateContext() override { return new FPCGComputeGraphContext(); }
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual void PostExecuteInternal(FPCGContext* InContext) const override;
	virtual void AbortInternal(FPCGContext* InContext) const override;;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }

	// The calls to initialize the compute graph are not thread safe.
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

	// TODO - need to accumulate dependencies from compute graph nodes.
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

	void ResetAsyncOperations(FPCGContext* InContext) const;

#if WITH_EDITOR
	void CollectDebugDataPrepareActions(FPCGComputeGraphContext* InContext) const;

	// TODO: Debug draw has a special path here because the compute graph element represents a set of original elements which do not themselves
	// execute. Review if this can be reconciled or unified with the normal debug draw path.
	void ExecuteDebugDraw(FPCGComputeGraphContext* InContext) const;

	void StoreDataForInspection(FPCGComputeGraphContext* InContext) const;

	void LogCompilationMessages(FPCGComputeGraphContext* InContext) const;
#endif

#if PCG_PROFILING_ENABLED
	void ComputeNodeOutputMemorySize(FPCGComputeGraphContext* InContext) const;
#endif
};

UCLASS(ClassGroup = (Procedural))
class UPCGComputeGraphSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGComputeGraphSettings();

protected:
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY()
	int32 ComputeGraphIndex = INDEX_NONE;
};
