// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/Elements/PCGCullPointsOutsideActorBoundsKernel.h"

#include "PCGContext.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/DataInterfaces/Elements/PCGCullPointsOutsideActorBoundsDataInterface.h"
#include "Elements/PCGCullPointsOutsideActorBounds.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCullPointsOutsideActorBoundsKernel)

#define LOCTEXT_NAMESPACE "PCGCullPointsOutsideActorBoundsKernel"

TSharedPtr<const FPCGDataCollectionDesc> UPCGCullPointsOutsideActorBoundsKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	// Code assumes single output pin.
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	const FPCGKernelPin SourceKernelPin(GetKernelIndex(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	return InBinding->ComputeKernelPinDataDesc(SourceKernelPin);
}

int UPCGCullPointsOutsideActorBoundsKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const TSharedPtr<const FPCGDataCollectionDesc> OutputPinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultOutputLabel, /*bIsInput=*/false);
	return ensure(OutputPinDesc) ? OutputPinDesc->ComputeTotalElementCount() : 0;
}

#if WITH_EDITOR
void UPCGCullPointsOutsideActorBoundsKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGCullPointsOutsideActorBoundsDataInterface> NodeDI = InOutContext.NewObject_AnyThread<UPCGCullPointsOutsideActorBoundsDataInterface>(InObjectOuter);
	NodeDI->SetProducerKernel(this);

	OutDataInterfaces.Add(NodeDI);
}
#endif

void UPCGCullPointsOutsideActorBoundsKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
}

void UPCGCullPointsOutsideActorBoundsKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
}

#undef LOCTEXT_NAMESPACE
