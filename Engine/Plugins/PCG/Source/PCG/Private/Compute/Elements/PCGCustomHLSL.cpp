// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/Elements/PCGCustomHLSL.h"

#include "PCGContext.h"
#include "PCGComponent.h"
#include "PCGEdge.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "PCGSubsystem.h"
#include "Compute/PCGComputeSource.h"
#include "Compute/PCGKernelHelpers.h"
#include "Compute/Elements/PCGCustomHLSLKernel.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Data/PCGLandscapeData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGTextureData.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Helpers/PCGHelpers.h"

#include "Internationalization/Regex.h"
#include "Containers/StaticArray.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCustomHLSL)

#define LOCTEXT_NAMESPACE "PCGCustomHLSLElement"

namespace PCGHLSLElement
{
	const FString PinDeclTemplateStr = TEXT("{pin}");
}

UPCGCustomHLSLSettings::UPCGCustomHLSLSettings()
{
	bExecuteOnGPU = true;

#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject) && !FPCGContext::IsInitializingSettings())
	{
		if (!UPCGComputeSource::OnModifiedDelegate.IsBoundToObject(this))
		{
			UPCGComputeSource::OnModifiedDelegate.AddUObject(this, &UPCGCustomHLSLSettings::OnComputeSourceModified);
		}
	}
#endif
}

#if WITH_EDITOR
void UPCGCustomHLSLSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (PointCount_DEPRECATED > 0)
	{
		NumElements = PointCount_DEPRECATED;
		PointCount_DEPRECATED = 0;
	}

	if (NumElements2D_DEPRECATED.X > 0 || NumElements2D_DEPRECATED.Y > 0)
	{
		NumElementsX = NumElements2D_DEPRECATED.X;
		NumElementsY = NumElements2D_DEPRECATED.Y;
		NumElements2D_DEPRECATED = FIntPoint(0, 0);
	}

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGCustomHLSLAttributeCreationSettingsToggle)
	{
		// For old nodes: enable the toggle if any output pin already has manually authored attribute keys,
		// so existing settings remain visible after the upgrade. New nodes default to false (hidden).
		bPerPinAttributeCreationSettings = false;
		for (const FPCGPinPropertiesGPU& OutputPin : OutputPins)
		{
			if (OutputPin.PropertiesGPU.CreatedKernelAttributeKeys.Num() > 0)
			{
				bPerPinAttributeCreationSettings = true;
				break;
			}
		}
	}
#endif

	// Note: We update here so that Custom HLSL nodes will have the correct pin settings & declarations on load.
	UpdatePinSettings();
	UpdateAttributeKeys();
	UpdateDeclarations();
}

void UPCGCustomHLSLSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Newly created kernels opt into the strict input/output split. Loaded assets keep the serialized value, which falls
	// back to the CDO default (true) for assets authored before this flag existed. IsNewObjectAndNotDefault returns false
	// during load (RF_NeedLoad still set), so we don't clobber existing serialized state.
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		bUseLegacyDataCollectionAPI = false;
	}

	// Note: We update here so that Custom HLSL nodes will have the correct pin settings & declarations on creation.
	UpdatePinSettings();
	UpdateAttributeKeys();
	UpdateDeclarations();
}

void UPCGCustomHLSLSettings::BeginDestroy()
{
	UPCGComputeSource::OnModifiedDelegate.RemoveAll(this);

	Super::BeginDestroy();
}
#endif

TArray<FPCGPinProperties> UPCGCustomHLSLSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	Algo::Transform(OutputPins, PinProperties, [](const FPCGPinPropertiesGPU& InPropertiesGPU) { return InPropertiesGPU; });
	return PinProperties;
}

#if WITH_EDITOR
void UPCGCustomHLSLSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	// If a pin label or type is about to change, snapshot all input pins to diff against in PostEditChangeProperty.
	if (PropertyAboutToChange
		&& (PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, Label)
			|| PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, AllowedTypes)))
	{
		InputPinsPreEditChange = InputPins;
	}
}

void UPCGCustomHLSLSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName MemberProperty = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	const FName Property = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	// Warn if the user tried to change the type of a pin that is driven by the kernel.
	if (Property == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, AllowedTypes))
	{
		const FPCGDataTypeIdentifier ElementType = GetElementType();
		const bool bIsOutputPin = (MemberProperty == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, OutputPins));
		const bool bInputForced = !bIsOutputPin && IsProcessorKernel() && !InputPins.IsEmpty() && InputPins[0].AllowedTypes != ElementType;
		const bool bOutputForced = bIsOutputPin && (IsGeneratorKernel() || IsProcessorKernel()) && !OutputPins.IsEmpty() && OutputPins[0].AllowedTypes != ElementType;

		if (bInputForced || bOutputForced)
		{
			const FName PinLabel = bIsOutputPin ? OutputPins[0].Label : InputPins[0].Label;
			UE_LOGF(LogPCG, Warning, "Custom HLSL: cannot change pin '%ls' type. Kernel type '%ls' forces this pin to '%ls'. Switch the kernel type to 'Custom' to author primary pin types manually.",
				*PinLabel.ToString(),
				*UEnum::GetValueAsString(KernelType),
				*ElementType.ToString());
		}
	}

	// Apply any pin setup before refreshing the node.
	UpdatePinSettings();
	UpdateAttributeKeys();

	if (MemberProperty == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, OutputPins)
		&& Property == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, OutputPins)
		&& PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
	{
		// Whenever a new output pin is created, we should default initialize 'PinsToInitializeFrom' with the first input pin label (if it exists).
		if (const FPCGPinProperties* FirstInputPin = GetFirstInputPinProperties())
		{
			check(!OutputPins.IsEmpty());

			FPCGPinPropertiesGPU& PinProps = OutputPins.Last();
			PinProps.PropertiesGPU.PinsToInititalizeFrom.Add(FirstInputPin->Label);
		}
	}
	else if (MemberProperty == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, InputPins)
		&& Property == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, AllowedTypes)
		&& InputPinsPreEditChange.Num() == InputPins.Num())
	{
		// When a pin's type changes from non-texture to texture, default bAllowMultipleData to false. General texture
		// pins have no hard limit on data count, so the user can still toggle it back on via the UI if needed.
		for (int32 PinIndex = 0; PinIndex < InputPins.Num(); ++PinIndex)
		{
			const bool bWasTexture = InputPinsPreEditChange[PinIndex].AllowedTypes.IsChildOf(EPCGDataType::BaseTexture);
			const bool bIsTexture = InputPins[PinIndex].AllowedTypes.IsChildOf(EPCGDataType::BaseTexture);
			if (!bWasTexture && bIsTexture)
			{
				InputPins[PinIndex].bAllowMultipleData = false;
			}
		}
	}
	else if (MemberProperty == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, InputPins)
		&& Property == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, Label)
		&& InputPinsPreEditChange.Num() == InputPins.Num())
	{
		// Fix-up output pin initialization references if an input pin label changed.
		for (int32 Index = 0; Index < InputPinsPreEditChange.Num(); ++Index)
		{
			const FName InputLabelBeforeChange = InputPinsPreEditChange[Index].Label;
			const FName InputLabelAfterChange = InputPins[Index].Label;

			if (InputLabelBeforeChange != InputLabelAfterChange)
			{
				for (FPCGPinPropertiesGPU& OutPinProps : OutputPins)
				{
					for (FName& InitPinLabel : OutPinProps.PropertiesGPU.PinsToInititalizeFrom)
					{
						if (InitPinLabel == InputLabelBeforeChange)
						{
							InitPinLabel = InputLabelAfterChange;
						}
					}
				}

				// TODO: Could also find/replace to fix-up the kernel source
			}
		}
	}

	// Sanitize pin labels to ensure they are valid HLSL identifiers.
	if (Property == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, Label))
	{
		auto SanitizePinLabel = [](FPCGPinProperties& PinProps)
		{
			if (!PCGComputeHelpers::IsValidHLSLPinLabel(PinProps.Label))
			{
				const FString OriginalLabel = PinProps.Label.ToString();
				const FString SanitizedLabel = PCGComputeHelpers::SanitizePinLabelForHLSL(PinProps.Label);
				PinProps.Label = FName(SanitizedLabel);

				UE_LOGF(LogPCG, Warning, "Pin label '%ls' contained invalid HLSL characters and was sanitized to '%ls'.", *OriginalLabel, *SanitizedLabel);
			}
		};

		for (FPCGPinProperties& PinProps : InputPins)
		{
			SanitizePinLabel(PinProps);
		}

		for (FPCGPinPropertiesGPU& PinProps : OutputPins)
		{
			SanitizePinLabel(PinProps);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	UpdateDeclarations();
}
#endif

FPCGElementPtr UPCGCustomHLSLSettings::CreateElement() const
{
	return MakeShared<FPCGCustomHLSLElement>();
}

#if WITH_EDITOR
TArray<FPCGPreConfiguredSettingsInfo> UPCGCustomHLSLSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGKernelType>();
}

EPCGChangeType UPCGCustomHLSLSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, ShaderSource)
		|| InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, ShaderFunctions))
	{
		ChangeType |= EPCGChangeType::ShaderSource;
	}

	// Any settings change to this node could change the compute graph.
	ChangeType |= EPCGChangeType::Structural;

	return ChangeType;
}
#endif

bool UPCGCustomHLSLSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	if (!Super::IsPinUsedByNodeExecution(InPin))
	{
		return false;
	}

	if (InPin)
	{
		const FName PinLabel = InPin->Properties.Label;

		// Processor kernels never use the NumElements size override pins.
		if (IsProcessorKernel()
			&& (PinLabel == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElements)
			|| PinLabel == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsX)
			|| PinLabel == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsY)
			|| PinLabel == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsZ)))
		{
			return false;
		}

		// Generator kernels only use size override pins matching their dispatch dimension.
		if (IsGeneratorKernel())
		{
			const EPCGElementDimension Dim = GetElementDimension();

			if (PinLabel == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElements))
			{
				return Dim == EPCGElementDimension::One;
			}
			else if (PinLabel == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsX)
				|| PinLabel == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsY))
			{
				return Dim > EPCGElementDimension::One;
			}
			else if (PinLabel == GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsZ))
			{
				return Dim > EPCGElementDimension::Two;
			}
		}
	}

	return true;
}

void UPCGCustomHLSLSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGKernelType>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			KernelType = EPCGKernelType(PreconfiguredInfo.PreconfiguredIndex);

			// Generators don't utilize the default input pin, so let's not add it by default.
			if (IsGeneratorKernel())
			{
				InputPins.Empty();
			}

#if WITH_EDITOR
			UpdatePinSettings();
#endif

			// Default to initializing the first output pin's from the first input pin's data.
			if (const FPCGPinProperties* FirstInputPin = GetFirstInputPinProperties())
			{
				if (!OutputPins.IsEmpty())
				{
					FPCGPinPropertiesGPU& PinProps = OutputPins.Last();
					PinProps.PropertiesGPU.PinsToInititalizeFrom.Add(FirstInputPin->Label);
				}
			}

#if WITH_EDITOR
			UpdateDeclarations();

			// Broadcast a Structural change here so the graph recompiles with the post-preconfigure pin layout (e.g. empty InputPins for generator kernels).
			OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Structural);
#endif
		}
	}
}

FString UPCGCustomHLSLSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGKernelType>())
	{
		return EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(KernelType)).ToString();
	}

	return FString();
}

const FPCGPinProperties* UPCGCustomHLSLSettings::GetFirstInputPinProperties() const
{
	return !InputPins.IsEmpty() ? &InputPins[0] : nullptr;
}

const FPCGPinPropertiesGPU* UPCGCustomHLSLSettings::GetFirstOutputPinProperties() const
{
	return !OutputPins.IsEmpty() ? &OutputPins[0] : nullptr;
}

EPCGElementDimension UPCGCustomHLSLSettings::GetElementDimension() const
{
	switch (KernelType)
	{
	case EPCGKernelType::PointGenerator:
	case EPCGKernelType::PointProcessor:
	case EPCGKernelType::AttributeSetProcessor:
	case EPCGKernelType::AttributeSetGenerator:
	case EPCGKernelType::Custom:
		return EPCGElementDimension::One;
	case EPCGKernelType::TextureGenerator:
	case EPCGKernelType::TextureProcessor:
		return EPCGElementDimension::Two;
	case EPCGKernelType::TextureArrayGenerator:
	case EPCGKernelType::TextureArrayProcessor:
		return EPCGElementDimension::Three;
	default:
		checkNoEntry();
		return EPCGElementDimension::One;
	}
}

FPCGDataTypeIdentifier UPCGCustomHLSLSettings::GetElementType() const
{
	switch (KernelType)
	{
	case EPCGKernelType::PointGenerator:
	case EPCGKernelType::PointProcessor:
		return FPCGDataTypeInfoPoint::AsId();
	case EPCGKernelType::TextureGenerator:
	case EPCGKernelType::TextureProcessor:
		return FPCGDataTypeInfoTexture2DSingleBase::AsId();
	case EPCGKernelType::TextureArrayGenerator:
	case EPCGKernelType::TextureArrayProcessor:
		return FPCGDataTypeInfoTexture2DArray::AsId();
	case EPCGKernelType::AttributeSetProcessor:
	case EPCGKernelType::AttributeSetGenerator:
		return FPCGDataTypeInfoParam::AsId();
	default:
		// Custom kernel has no primary type and returns None.
		return EPCGDataType::None;
	}
}

#if WITH_EDITOR
FString UPCGCustomHLSLSettings::GetDeclarationsText() const
{
	return InputDeclarations + TEXT("\n\n") + OutputDeclarations + TEXT("\n\n") + HelperDeclarations;
}

FString UPCGCustomHLSLSettings::GetFunctionsText() const
{
	return ShaderFunctions;
}

FString UPCGCustomHLSLSettings::GetSourceText() const
{
	return ShaderSource;
}

void UPCGCustomHLSLSettings::SetFunctionsText(const FString& InText)
{
	const FName PropertyName = GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, ShaderFunctions);
	FProperty* Property = FindFProperty<FProperty>(StaticClass(), PropertyName);
	FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);

	{
		FScopedTransaction Transaction(LOCTEXT("OnSetShaderFunctionsText", "Set Functions Text"));
	
		PreEditChange(Property);
		Modify();
		ShaderFunctions = InText;
		PostEditChangeProperty(PropertyChangedEvent);
	}
	
	OnSettingsChangedDelegate.Broadcast(this, GetChangeTypeForProperty(PropertyName));
}

void UPCGCustomHLSLSettings::SetSourceText(const FString& InText)
{
	const FName PropertyName = GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, ShaderSource);
	FProperty* Property = FindFProperty<FProperty>(StaticClass(), PropertyName);
	FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
	
	{
		FScopedTransaction Transaction(LOCTEXT("OnSetShaderSourceText", "Set Source Text"));

		PreEditChange(Property);
		Modify();
		ShaderSource = InText;
		PostEditChangeProperty(PropertyChangedEvent);
	}

	OnSettingsChangedDelegate.Broadcast(this, GetChangeTypeForProperty(PropertyName));
}

bool UPCGCustomHLSLSettings::IsReadOnly() const
{
	return KernelSourceOverride != nullptr;
}

void UPCGCustomHLSLSettings::SetKernelType(EPCGKernelType InKernelType)
{
	KernelType = InKernelType;

	// Generators don't use the default input pin; clear so UpdatePinSettings recreates it with the correct type.
	if (IsGeneratorKernel())
	{
		InputPins.Empty();
	}

	UpdatePinSettings();
	UpdateDeclarations();
}

void UPCGCustomHLSLSettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);

	CreateParams.NodeInputPinsToWire.Empty(InputPins.Num());
	Algo::Transform(InputPins, CreateParams.NodeInputPinsToWire, [](const FPCGPinProperties& InProps) { return InProps.Label; });

	CreateParams.NodeOutputPinsToWire.Empty(OutputPins.Num());
	Algo::Transform(OutputPins, CreateParams.NodeOutputPinsToWire, [](const FPCGPinProperties& InProps) { return InProps.Label; });

	UPCGCustomHLSLKernel* Kernel = PCGKernelHelpers::CreateKernel<UPCGCustomHLSLKernel>(InOutContext, CreateParams, OutKernels, OutEdges);

	for (const FPCGPinPropertiesGPU& OutputPin : OutputPins)
	{
		if (OutputPin.PropertiesGPU.bInitializeToZero)
		{
			Kernel->SetOutputPinRequiresZeroInitialization(OutputPin.Label);
		}
	}
}

TArray<FPCGDataTypeIdentifier> UPCGCustomHLSLSettings::GetAllowedInputTypes()
{
	return PCGComputeHelpers::GetAllowedInputTypesList();
}

TArray<FPCGDataTypeIdentifier> UPCGCustomHLSLSettings::GetAllowedOutputTypes()
{
	return PCGComputeHelpers::GetAllowedOutputTypesList();
}
#endif // WITH_EDITOR

const FPCGPinPropertiesGPU* UPCGCustomHLSLSettings::GetOutputPinPropertiesGPU(const FName& InPinLabel) const
{
	return OutputPins.FindByPredicate([InPinLabel](const FPCGPinPropertiesGPU& InProperties)
	{
		return InProperties.Label == InPinLabel;
	});
}

#if WITH_EDITOR
void UPCGCustomHLSLSettings::UpdateDeclarations()
{
	// Reference: UOptimusNode_CustomComputeKernel::UpdatePreamble
	UpdateInputDeclarations();
	UpdateOutputDeclarations();
	UpdateHelperDeclarations();

	// TODO: Should data labels be explained/exemplified in the declarations?
}

void UPCGCustomHLSLSettings::UpdateInputDeclarations()
{
	InputDeclarations.Reset();

	// Constants category
	{
		if (IsGeneratorKernel())
		{
			InputDeclarations += TEXT("/*** INPUT CONSTANTS ***/\n\n");

			switch (GetElementDimension())
			{
			case EPCGElementDimension::One:
				InputDeclarations += TEXT("const uint NumElements;\n\n");
				break;
			case EPCGElementDimension::Two:
				InputDeclarations += TEXT("const uint2 NumElements;\n\n");
				break;
			case EPCGElementDimension::Three:
				InputDeclarations += TEXT("const uint3 NumElements;\n\n");
				break;
			case EPCGElementDimension::Four:
				InputDeclarations += TEXT("const uint4 NumElements;\n\n");
				break;
			default:
				checkNoEntry();
			}
		}

		InputDeclarations += TEXT("/*** INPUT PER-THREAD CONSTANTS ***/\n\n");
		InputDeclarations += TEXT("const uint ThreadIndex;\n");

		if (IsProcessorKernel())
		{
			const FPCGPinProperties* InputPin = GetFirstInputPinProperties();
			const FPCGPinPropertiesGPU* OutputPin = GetFirstOutputPinProperties();

			if (InputPin && OutputPin)
			{
				InputDeclarations += FString::Format(TEXT(
					"const uint {0}_DataIndex;\n"
					"const uint {1}_DataIndex;\n"),
					{ InputPin->Label.ToString(), OutputPin->Label.ToString() });
			}
		}
		else if (IsGeneratorKernel())
		{
			if (const FPCGPinPropertiesGPU* PointProcessingOutputPin = GetFirstOutputPinProperties())
			{
				InputDeclarations += FString::Format(
					TEXT("const uint {0}_DataIndex;\n"),
					{ PointProcessingOutputPin->Label.ToString() });
			}
		}

		if (!IsCustomKernel())
		{
			switch (GetElementDimension())
			{
			case EPCGElementDimension::One:
				InputDeclarations += TEXT("const uint ElementIndex;\n");
				break;
			case EPCGElementDimension::Two:
				InputDeclarations += TEXT("const uint2 ElementIndex;\n");
				break;
			case EPCGElementDimension::Three:
				InputDeclarations += TEXT("const uint3 ElementIndex;\n");
				break;
			case EPCGElementDimension::Four:
				InputDeclarations += TEXT("const uint4 ElementIndex;\n");
				break;
			default:
				ensure(false);
				break;
			}
		}

		InputDeclarations += TEXT("\n");
	}

	TArray<FString> DataCollectionDataPins;
	TArray<FString> PointDataPins;
	TArray<FString> LandscapeDataPins;
	TArray<FString> TextureDataPins;
	TArray<FString> TextureArrayDataPins;
	TArray<FString> VirtualTextureDataPins;
	TArray<FString> RawBufferDataPins;
	TArray<FString> StaticMeshDataPins;

	for (const FPCGPinProperties& Pin : InputPinProperties())
	{
		if (PCGComputeHelpers::IsTypeAllowedInDataCollection(Pin.AllowedTypes))
		{
			DataCollectionDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & EPCGDataType::Point))
		{
			PointDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & EPCGDataType::Landscape))
		{
			LandscapeDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & EPCGDataType::BaseTexture))
		{
			TextureDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & FPCGDataTypeInfoTexture2DArray::AsId()))
		{
			TextureArrayDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & EPCGDataType::VirtualTexture))
		{
			VirtualTextureDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & FPCGDataTypeInfoRawBuffer::AsId()))
		{
			RawBufferDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & EPCGDataType::StaticMeshResource))
		{
			StaticMeshDataPins.Add(Pin.Label.ToString());
		}
	}

	if (!DataCollectionDataPins.IsEmpty())
	{
		InputDeclarations += TEXT("/*** INPUT DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = DataCollectionDataPins.Num() > 1;

		if (bMultiPin)
		{
			InputDeclarations += TEXT("// Valid pins: ") + FString::Join(DataCollectionDataPins, TEXT(", ")) + TEXT("\n");
		}

		InputDeclarations += FString::Format(TEXT(
			"uint {0}_GetNumData();\n"
			"uint {0}_GetNumElements();\n"
			"uint {0}_GetNumElements(uint DataIndex);\n"
			"\n"
			"// Valid types: bool, int, float, float2, float3, float4, Rotator (float3), Quat (float4), Transform (float4x4), StringKey (int), Name (int)\n"
			"\n"
			"{type} {0}_Get{type}(uint DataIndex, uint ElementIndex, 'AttributeName');\n"
			"\n"
			"// Example: {0}_GetFloat({0}_DataIndex, ElementIndex, 'MyFloatAttr');\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : DataCollectionDataPins[0] });

		InputDeclarations += TEXT("\n");
	}

	if (!PointDataPins.IsEmpty())
	{
		InputDeclarations += TEXT("/*** INPUT POINT DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = PointDataPins.Num() > 1;

		if (bMultiPin)
		{
			InputDeclarations += TEXT("// Valid pins: ") + FString::Join(PointDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		InputDeclarations += FString::Format(TEXT(
			"float3 {0}_GetPosition(uint DataIndex, uint ElementIndex);\n"
			"float4 {0}_GetRotation(uint DataIndex, uint ElementIndex);\n"
			"float3 {0}_GetScale(uint DataIndex, uint ElementIndex);\n"
			"float3 {0}_GetBoundsMin(uint DataIndex, uint ElementIndex);\n"
			"float3 {0}_GetBoundsMax(uint DataIndex, uint ElementIndex);\n"
			"float4 {0}_GetColor(uint DataIndex, uint ElementIndex);\n"
			"float {0}_GetDensity(uint DataIndex, uint ElementIndex);\n"
			"int {0}_GetSeed(uint DataIndex, uint ElementIndex);\n"
			"float {0}_GetSteepness(uint DataIndex, uint ElementIndex);\n"
			"float4x4 {0}_GetPointTransform(uint DataIndex, uint ElementIndex);\n"
			"bool {0}_IsPointRemoved(uint DataIndex, uint ElementIndex);\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : PointDataPins[0] });

		InputDeclarations += TEXT("\n");
	}

	if (!LandscapeDataPins.IsEmpty())
	{
		InputDeclarations += TEXT("/*** INPUT LANDSCAPE DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = LandscapeDataPins.Num() > 1;

		if (bMultiPin)
		{
			InputDeclarations += TEXT("// Valid pins: ") + FString::Join(LandscapeDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		InputDeclarations += FString::Format(TEXT(
			"float {0}_GetHeight(float3 WorldPos);\n"
			"float3 {0}_GetNormal(float3 WorldPos);\n"
			"float3 {0}_GetBaseColor(float3 WorldPos);\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : LandscapeDataPins[0] });

		InputDeclarations += TEXT("\n");
	}

	if (!TextureDataPins.IsEmpty())
	{
		InputDeclarations += TEXT("/*** INPUT TEXTURE 2D DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = TextureDataPins.Num() > 1;

		if (bMultiPin)
		{
			InputDeclarations += TEXT("// Valid pins: ") + FString::Join(TextureDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		InputDeclarations += FString::Format(TEXT(
			"uint {0}_GetNumData();\n"
			"uint2 {0}_GetNumElements(uint DataIndex);\n"
			"// Computes an unclamped texture coordinate.\n"
			"float2 {0}_GetTexCoords(uint DataIndex, float3 WorldPos);\n"
			"float4 {0}_Sample(uint DataIndex, float2 TexCoords);\n"
			"// Computes sample coordinates of the WorldPos relative to the texture data's transform.\n"
			"float4 {0}_SampleWorldPos(uint DataIndex, float3 WorldPos);\n"
			"float4 {0}_Load(uint DataIndex, uint2 ElementIndex);\n"
			"float4 {0}_Load(uint DataIndex, uint2 ElementIndex, uint MipIndex);\n"
			"float2 {0}_GetTexelSize(uint DataIndex);\n"
			"float2 {0}_GetTexelSizeWorld(uint DataIndex);\n"
			"float4x4 {0}_GetTransform(uint DataIndex);\n"
			"uint {0}_GetNumMips(uint DataIndex);\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : TextureDataPins[0] });

		InputDeclarations += TEXT("\n");
	}

	if (!TextureArrayDataPins.IsEmpty())
	{
		InputDeclarations += TEXT("/*** INPUT TEXTURE 2D ARRAY DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = TextureArrayDataPins.Num() > 1;

		if (bMultiPin)
		{
			InputDeclarations += TEXT("// Valid pins: ") + FString::Join(TextureArrayDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		InputDeclarations += FString::Format(TEXT(
			"uint {0}_GetNumData();\n"
			"// NumElements: Size in XY, ArraySize in Z.\n"
			"uint3 {0}_GetNumElements(uint DataIndex);\n"
			"uint2 {0}_GetResolution(uint DataIndex);\n"
			"uint {0}_GetArraySize(uint DataIndex);\n"
			"// Computes an unclamped texture coordinate.\n"
			"float2 {0}_GetTexCoords(uint DataIndex, float3 WorldPos);\n"
			"float4 {0}_Sample(uint DataIndex, float2 TexCoords, uint InSliceIndex);\n"
			"// Computes sample coordinates of the WorldPos relative to the texture data's transform.\n"
			"float4 {0}_SampleWorldPos(uint DataIndex, float3 WorldPos, uint InSliceIndex);\n"
			"float4 {0}_Load(uint DataIndex, uint3 ElementIndex);\n"
			"float4 {0}_Load(uint DataIndex, uint3 ElementIndex, uint MipIndex);\n"
			"float2 {0}_GetTexelSize(uint DataIndex);\n"
			"float2 {0}_GetTexelSizeWorld(uint DataIndex);\n"
			"float4x4 {0}_GetTransform(uint DataIndex);\n"
			"uint {0}_GetNumMips(uint DataIndex);\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : TextureArrayDataPins[0] });

		InputDeclarations += TEXT("\n");
	}

	if (!VirtualTextureDataPins.IsEmpty())
	{
		InputDeclarations += TEXT("/*** INPUT VIRTUAL TEXTURE DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = VirtualTextureDataPins.Num() > 1;

		if (bMultiPin)
		{
			InputDeclarations += TEXT("// Valid pins: ") + FString::Join(VirtualTextureDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		InputDeclarations += FString::Format(TEXT(
			"// Samples a virtual texture and gets all values that are available. Otherwise returns default values.\n"
			"void {0}_SampleVirtualTexture(\n"
			"	uint InDataIndex,\n"
			"	float3 InWorldPos,\n"
			"	out bool bOutInsideVolume,\n"
			"	out float3 OutBaseColor,\n"
			"	out float OutSpecular,\n"
			"	out float OutRoughness,\n"
			"	out float OutWorldHeight,\n"
			"	out float3 OutNormal,\n"
			"	out float OutDisplacement,\n"
			"	out float OutMask,\n"
			"	out float4 OutMask4);\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : VirtualTextureDataPins[0] });

		InputDeclarations += TEXT("\n");
	}

	if (!RawBufferDataPins.IsEmpty())
	{
		InputDeclarations += TEXT("/*** INPUT BYTE ADDRESS BUFFER DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = RawBufferDataPins.Num() > 1;

		if (bMultiPin)
		{
			InputDeclarations += TEXT("// Valid pins: ") + FString::Join(RawBufferDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		InputDeclarations += FString::Format(TEXT(
			"uint {0}_GetNumElements();\n"
			"uint {0}_Load(uint ElementIndex);\n"
			"uint4 {0}_Load4(uint FirstElementIndex);\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : RawBufferDataPins[0] });

		InputDeclarations += TEXT("\n");
	}

	if (!StaticMeshDataPins.IsEmpty())
	{
		InputDeclarations += TEXT("/*** INPUT STATIC MESH DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = StaticMeshDataPins.Num() > 1;

		if (bMultiPin)
		{
			InputDeclarations += TEXT("// Valid pins: ") + FString::Join(StaticMeshDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		InputDeclarations += FString::Format(TEXT(
			"// Vertex functions\n"
			"int {0}_GetNumVertices(int DataIndex);\n"
			"void {0}_GetVertex(int DataIndex, int VertexIndex, out float3 OutPosition, out float3 OutNormal, out float3 OutTangent, out float3 OutBitangent);\n"
			"float4 {0}_GetVertexColor(int DataIndex, int VertexIndex);\n"
			"float2 {0}_GetVertexUV(int DataIndex, int VertexIndex, int UVSet);\n"
			"\n"
			"// Triangle functions\n"
			"int {0}_GetNumTriangles(int DataIndex);\n"
			"void {0}_GetTriangleIndices(int DataIndex, int TriangleIndex, out int OutIndex0, out int OutIndex1, out int OutIndex2);\n"
			"void {0}_SampleTriangle(int DataIndex, int TriangleIndex, float3 BaryCoords, out float3 OutPosition, out float3 OutNormal, out float3 OutTangent, out float3 OutBitangent);\n"
			"float4 {0}_SampleTriangleColor(int DataIndex, int TriangleIndex, float3 BaryCoords);\n"
			"float2 {0}_SampleTriangleUV(int DataIndex, int TriangleIndex, float3 BaryCoords, int UVSet);\n"
			"\n"
			"// Get bounds extents of the static mesh.\n"
			"float3 {0}_GetMeshBoundsExtents(int DataIndex);\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : StaticMeshDataPins[0] });

		InputDeclarations += TEXT("\n");
	}

	InputDeclarations.TrimStartAndEndInline();
}

void UPCGCustomHLSLSettings::UpdateOutputDeclarations()
{
	OutputDeclarations.Reset();

	TArray<FString> DataCollectionDataPins;
	TArray<FString> PointDataPins;
	TArray<FString> TextureDataPins;
	TArray<FString> TextureArrayDataPins;
	TArray<FString> RawBufferDataPins;

	for (const FPCGPinProperties& Pin : OutputPinProperties())
	{
		if (PCGComputeHelpers::IsTypeAllowedInDataCollection(Pin.AllowedTypes))
		{
			DataCollectionDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & EPCGDataType::Point))
		{
			PointDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & EPCGDataType::BaseTexture))
		{
			TextureDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & FPCGDataTypeInfoTexture2DArray::AsId()))
		{
			TextureArrayDataPins.Add(Pin.Label.ToString());
		}

		if (!!(Pin.AllowedTypes & FPCGDataTypeInfoRawBuffer::AsId()))
		{
			RawBufferDataPins.Add(Pin.Label.ToString());
		}
	}

	if (!DataCollectionDataPins.IsEmpty())
	{
		OutputDeclarations += TEXT("/*** OUTPUT DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = DataCollectionDataPins.Num() > 1;

		if (bMultiPin)
		{
			OutputDeclarations += TEXT("// Valid pins: ") + FString::Join(DataCollectionDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		OutputDeclarations += FString::Format(TEXT(
			"void {0}_GetElementCountMultiplier();\n"
			"void {0}_CopyElementFrom_{input_pin}(uint TargetDataIndex, uint TargetElementIndex, uint SourceDataIndex, uint SourceElementIndex);\n"
			"\n"
			"uint {0}_GetNumData();\n"
			"uint {0}_GetNumElements();\n"
			"uint {0}_GetNumElements(uint DataIndex);\n"
			"\n"
			"// Valid types: bool, int, float, float2, float3, float4, Rotator (float3), Quat (float4), Transform (float4x4), StringKey (int), Name (int)\n"
			"\n"
			"{type} {0}_Get{type}(uint DataIndex, uint ElementIndex, 'AttributeName');\n"
			"\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : DataCollectionDataPins[0] });

		OutputDeclarations += FString::Format(TEXT(
			"// Valid types: bool, int, float, float2, float3, float4, Rotator (float3), Quat (float4), Transform (float4x4), StringKey (int), Name (uint2)\n"
			"// Example: {0}_SetFloat({0}_DataIndex, ElementIndex, 'MyFloatAttr', MyValue);\n"
			"void {0}_Set{type}(uint DataIndex, uint ElementIndex, 'AttributeName', {type} Value);\n"
			"int {0}_AtomicAddInt(uint DataIndex, uint ElementIndex, 'IntAttributeName', int ValueToAdd); // Returns value before it was incremented.\n"
			"uint {0}_AtomicMaxUint(uint DataIndex, uint ElementIndex, 'IntAttributeName', uint ValueToMax); // Returns value before max operation.\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : DataCollectionDataPins[0] });

		OutputDeclarations += TEXT("\n");
	}

	if (!PointDataPins.IsEmpty())
	{
		OutputDeclarations += TEXT("/*** OUTPUT POINT DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = PointDataPins.Num() > 1;

		if (bMultiPin)
		{
			OutputDeclarations += TEXT("// Valid pins: ") + FString::Join(PointDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		OutputDeclarations += FString::Format(TEXT(
			"float3 {0}_GetPosition(uint DataIndex, uint ElementIndex);\n"
			"float4 {0}_GetRotation(uint DataIndex, uint ElementIndex);\n"
			"float3 {0}_GetScale(uint DataIndex, uint ElementIndex);\n"
			"float3 {0}_GetBoundsMin(uint DataIndex, uint ElementIndex);\n"
			"float3 {0}_GetBoundsMax(uint DataIndex, uint ElementIndex);\n"
			"float4 {0}_GetColor(uint DataIndex, uint ElementIndex);\n"
			"float {0}_GetDensity(uint DataIndex, uint ElementIndex);\n"
			"int {0}_GetSeed(uint DataIndex, uint ElementIndex);\n"
			"float {0}_GetSteepness(uint DataIndex, uint ElementIndex);\n"
			"float4x4 {0}_GetPointTransform(uint DataIndex, uint ElementIndex);\n"
			"bool {0}_IsPointRemoved(uint DataIndex, uint ElementIndex);\n"
			"\n"
			"void {0}_InitializePoint(uint DataIndex, uint ElementIndex);\n"
			"bool {0}_RemovePoint(uint DataIndex, uint ElementIndex);\n"
			"\n"
			"void {0}_SetPosition(uint DataIndex, uint ElementIndex, float3 Position);\n"
			"void {0}_SetRotation(uint DataIndex, uint ElementIndex, float4 Rotation);\n"
			"void {0}_SetScale(uint DataIndex, uint ElementIndex, float3 Scale);\n"
			"void {0}_SetBoundsMin(uint DataIndex, uint ElementIndex, float3 BoundsMin);\n"
			"void {0}_SetBoundsMax(uint DataIndex, uint ElementIndex, float3 BoundsMax);\n"
			"void {0}_SetColor(uint DataIndex, uint ElementIndex, float4 Color);\n"
			"void {0}_SetDensity(uint DataIndex, uint ElementIndex, float Density);\n"
			"void {0}_SetSeed(uint DataIndex, uint ElementIndex, int Seed);\n"
			"void {0}_SetSteepness(uint DataIndex, uint ElementIndex, float Steepness);\n"
			"void {0}_SetPointTransform(uint DataIndex, uint ElementIndex, float4x4 Transform);\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : PointDataPins[0] });

		OutputDeclarations += TEXT("\n");
	}

	if (!TextureDataPins.IsEmpty())
	{
		OutputDeclarations += TEXT("/*** OUTPUT TEXTURE 2D DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = TextureDataPins.Num() > 1;

		if (bMultiPin)
		{
			OutputDeclarations += TEXT("// Valid pins: ") + FString::Join(TextureDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		OutputDeclarations += FString::Format(TEXT(
			"uint {0}_GetNumData();\n"
			"uint2 {0}_GetNumElements(uint DataIndex);\n"
			"float2 {0}_GetTexCoords(uint DataIndex, float3 WorldPos);\n"
			"float2 {0}_GetTexelSize(uint DataIndex);\n"
			"float2 {0}_GetTexelSizeWorld(uint DataIndex);\n"
			"uint {0}_GetNumMips(uint DataIndex);\n"
			"float4x4 {0}_GetTransform(uint DataIndex);\n"
			"uint {0}_GetNumMips(uint DataIndex);\n"
			"void {0}_Store(uint DataIndex, uint2 ElementIndex, float4 Value);\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : TextureDataPins[0] });

		OutputDeclarations += TEXT("\n");
	}

	if (!TextureArrayDataPins.IsEmpty())
	{
		OutputDeclarations += TEXT("/*** OUTPUT TEXTURE 2D ARRAY DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = TextureArrayDataPins.Num() > 1;

		if (bMultiPin)
		{
			OutputDeclarations += TEXT("// Valid pins: ") + FString::Join(TextureArrayDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		OutputDeclarations += FString::Format(TEXT(
			"uint {0}_GetNumData();\n"
			"// NumElements: Size in XY, ArraySize in Z.\n"
			"uint3 {0}_GetNumElements(uint DataIndex);\n"
			"uint2 {0}_GetSize(uint DataIndex);\n"
			"uint {0}_GetArraySize(uint DataIndex);\n"
			"float2 {0}_GetTexCoords(uint DataIndex, float3 WorldPos);\n"
			"float2 {0}_GetTexelSize(uint DataIndex);\n"
			"float2 {0}_GetTexelSizeWorld(uint DataIndex);\n"
			"float4x4 {0}_GetTransform(uint DataIndex);\n"
			"uint {0}_GetNumMips(uint DataIndex);\n"
			"void {0}_Store(uint DataIndex, uint3 ElementIndex, float4 Value);\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : TextureArrayDataPins[0] });

		OutputDeclarations += TEXT("\n");
	}

	if (!RawBufferDataPins.IsEmpty())
	{
		OutputDeclarations += TEXT("/*** OUTPUT BYTE ADDRESS BUFFER DATA FUNCTIONS ***/\n\n");

		const bool bMultiPin = RawBufferDataPins.Num() > 1;

		if (bMultiPin)
		{
			OutputDeclarations += TEXT("// Valid pins: ") + FString::Join(RawBufferDataPins, TEXT(", ")) + TEXT("\n\n");
		}

		OutputDeclarations += FString::Format(TEXT(
			"uint {0}_GetNumElements();\n"
			"uint {0}_Load(uint ElementIndex);\n"
			"uint4 {0}_Load4(uint FirstElementIndex);\n"
			"void {0}_Store(uint ElementIndex, uint Value);\n"
			"void {0}_Store4(uint FirstElementIndex, uint4 Value);\n"
			"uint {0}_AtomicAdd(uint ElementIndex, uint ValueToAdd); // Returns value before add.\n"
			"uint {0}_AtomicMax(uint ElementIndex, uint ValueToMax); // Returns value before max operation.\n"),
			{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : RawBufferDataPins[0] });

		OutputDeclarations += TEXT("\n");
	}

	OutputDeclarations.TrimStartAndEndInline();
}

void UPCGCustomHLSLSettings::UpdateHelperDeclarations()
{
	HelperDeclarations.Reset();

	// Helper functions category
	{
		HelperDeclarations += TEXT(
			"/*** HELPER FUNCTIONS ***/\n"
			"\n"
			"int3 GetNumThreads();\n"
			"uint GetThreadCountMultiplier();\n");

		// Get thread data - useful in all kernel types for secondary pins.
		{
			TArray<FString> DataCollectionPinNames;
			TArray<FString> TextureDataPinNames;
			TArray<FString> TextureArrayDataPinNames;

			for (const FPCGPinProperties& Properties : InputPinProperties())
			{
				if (PCGComputeHelpers::IsTypeAllowedInDataCollection(Properties.AllowedTypes))
				{
					DataCollectionPinNames.Add(Properties.Label.ToString());
				}

				if (!!(Properties.AllowedTypes & EPCGDataType::BaseTexture))
				{
					TextureDataPinNames.Add(Properties.Label.ToString());
				}

				if (!!(Properties.AllowedTypes & FPCGDataTypeInfoTexture2DArray::AsId()))
				{
					TextureArrayDataPinNames.Add(Properties.Label.ToString());
				}
			}

			for (const FPCGPinProperties& Properties : OutputPinProperties())
			{
				if (PCGComputeHelpers::IsTypeAllowedInDataCollection(Properties.AllowedTypes))
				{
					DataCollectionPinNames.Add(Properties.Label.ToString());
				}

				if (!!(Properties.AllowedTypes & EPCGDataType::BaseTexture))
				{
					TextureDataPinNames.Add(Properties.Label.ToString());
				}

				if (!!(Properties.AllowedTypes & FPCGDataTypeInfoTexture2DArray::AsId()))
				{
					TextureArrayDataPinNames.Add(Properties.Label.ToString());
				}
			}

			if (!DataCollectionPinNames.IsEmpty())
			{
				HelperDeclarations += TEXT("\n// Returns false if thread has no data to operate on.\n");

				const bool bMultiPin = DataCollectionPinNames.Num() > 1;

				if (bMultiPin)
				{
					HelperDeclarations += TEXT("// Valid pins: ") + FString::Join(DataCollectionPinNames, TEXT(", ")) + TEXT("\n");
				}

				HelperDeclarations += FString::Format(
					TEXT("bool {0}_GetThreadData(uint ThreadIndex, out uint OutDataIndex, out uint OutElementIndex);\n"),
					{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : DataCollectionPinNames[0] });

				HelperDeclarations += TEXT("\n");
			}

			if (!TextureDataPinNames.IsEmpty())
			{
				HelperDeclarations += TEXT("\n// Returns false if thread has no data to operate on.\n");

				const bool bMultiPin = TextureDataPinNames.Num() > 1;

				if (bMultiPin)
				{
					HelperDeclarations += TEXT("// Valid pins: ") + FString::Join(TextureDataPinNames, TEXT(", ")) + TEXT("\n");
				}

				HelperDeclarations += FString::Format(
					TEXT("bool {0}_GetThreadData(uint ThreadIndex, out uint OutDataIndex, out uint2 OutElementIndex);\n"),
					{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : TextureDataPinNames[0] });

				HelperDeclarations += TEXT("\n");
			}

			if (!TextureArrayDataPinNames.IsEmpty())
			{
				HelperDeclarations += TEXT("\n// Returns false if thread has no data to operate on.\n");

				const bool bMultiPin = TextureArrayDataPinNames.Num() > 1;

				if (bMultiPin)
				{
					HelperDeclarations += TEXT("// Valid pins: ") + FString::Join(TextureArrayDataPinNames, TEXT(", ")) + TEXT("\n");
				}

				HelperDeclarations += FString::Format(
					TEXT("bool {0}_GetThreadData(uint ThreadIndex, out uint OutDataIndex, out uint3 OutElementIndex);\n"),
					{ bMultiPin ? PCGHLSLElement::PinDeclTemplateStr : TextureArrayDataPinNames[0] });

				HelperDeclarations += TEXT("\n");
			}
		}

		HelperDeclarations += TEXT(
			"float3 GetComponentBoundsMin(); // World-space\n"
			"float3 GetComponentBoundsMax();\n"
			"uint GetSeed();\n"
			"\n"
			"float FRand(inout uint Seed); // Returns random float between 0 and 1.\n"
			"uint ComputeSeed(uint A, uint B);\n"
			"uint ComputeSeed(uint A, uint B, uint C);\n"
			"uint ComputeSeedFromPosition(float3 Position);\n"
			"\n"
			"// Returns the position of the Nth point in a 2D or 3D grid with the given constraints.\n"
			"float3 CreateGrid2D(uint ElementIndex, uint NumPoints, float3 Min, float3 Max);\n"
			"float3 CreateGrid2D(uint ElementIndex, uint NumPoints, uint NumX, float3 Min, float3 Max);\n"
			"float3 CreateGrid3D(uint ElementIndex, uint NumPoints, float3 Min, float3 Max);\n"
			"float3 CreateGrid3D(uint ElementIndex, uint NumPoints, uint NumX, uint NumY, float3 Min, float3 Max);\n");

		if (bPrintShaderDebugValues)
		{
			HelperDeclarations += FString::Format(TEXT(
				"\n"
				"// Writes floats to the debug buffer array, which will be readback and logged in the console for inspection.\n"
				"void WriteDebugValue(uint Index, float Value); // Index in [0, {0}] (set from 'Debug Buffer Size' property)\n"),
				{ DebugBufferSize - 1 });
		}
	}

	HelperDeclarations.TrimStartAndEndInline();
}

void UPCGCustomHLSLSettings::UpdatePinSettings()
{
	if (IsProcessorKernel() && InputPins.IsEmpty())
	{
		if (FPCGDataTypeIdentifier ElementType = GetElementType(); ensure(ElementType != EPCGDataType::None))
		{
			InputPins.Emplace(PCGPinConstants::DefaultInputLabel, ElementType);
		}
	}

	// Setup input pins.
	for (int PinIndex = 0; PinIndex < InputPins.Num(); ++PinIndex)
	{
		FPCGPinProperties& Properties = InputPins[PinIndex];

		// Type Any is not allowed, default to Point
		if (Properties.AllowedTypes == EPCGDataType::Any)
		{
			Properties.AllowedTypes = EPCGDataType::Point;
		}

		// Allow kernel type to drive the first pin type.
		if (PinIndex == 0 && IsProcessorKernel())
		{
			Properties.AllowedTypes = GetElementType();

			if (IsTextureKernel())
			{
				// Texture kernels are single-data for now due to pressure from resource binding limits and sampler counts.
				Properties.bAllowMultipleData = false;
				Properties.bAllowEditMultipleData = false;
			}
		}

		// Enforce single data on pin types that don't support a dynamic number of data.
		if (Properties.AllowedTypes.IsChildOf(FPCGDataTypeInfoLandscape::AsId())
			|| Properties.AllowedTypes.IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId())
			|| Properties.AllowedTypes.IsChildOf(FPCGDataTypeInfoRawBuffer::AsId()))
		{
			Properties.bAllowMultipleData = false;
			Properties.bAllowEditMultipleData = false;
		}
		else if (!(PinIndex == 0 && IsTextureKernel()))
		{
			// Only re-enable editing here if the kernel-specific block above didn't already lock it.
			Properties.bAllowEditMultipleData = true;
		}

		// TODO: We have work to do to allow dynamic merging of data. Also we will likely inject Gather
		// nodes on the CPU side so that merging is handled CPU side where possible.
		Properties.SetAllowMultipleConnections(false);

		Properties.bAllowEditMultipleConnections = false;
	}

	// Setup output pins.
	for (int PinIndex = 0; PinIndex < OutputPins.Num(); ++PinIndex)
	{
		FPCGPinPropertiesGPU& Properties = OutputPins[PinIndex];

		// Type Any is not allowed, default to Point
		if (Properties.AllowedTypes == EPCGDataType::Any)
		{
			Properties.AllowedTypes = EPCGDataType::Point;
		}

		// Only allow editing the initialization mode if it's not driven by the kernel type.
		bool bInitModeDrivenByKernel = false;
		if (PinIndex == 0)
		{
			if (!IsCustomKernel())
			{
				bInitModeDrivenByKernel = true;

				if (FPCGDataTypeIdentifier ElementType = GetElementType(); ElementType != EPCGDataType::None)
				{
					Properties.AllowedTypes = ElementType;
				}
			}

			Properties.bShowPropertiesGPU = true;
		}

		// Enforce single data on pin types that don't support a dynamic number of data.
		if (Properties.AllowedTypes.IsChildOf(FPCGDataTypeInfoTexture2DSingleBase::AsId()))
		{
			Properties.bAllowMultipleData = false;
			Properties.bAllowEditMultipleData = false;
			Properties.PropertiesGPU.bShowTexturePinSettings = true;

			// Don't show texture array settings for non-array textures.
			Properties.PropertiesGPU.bShowTextureArrayPinSettings = false;
		}
		else if (Properties.AllowedTypes.IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId()))
		{
			Properties.bAllowMultipleData = false;
			Properties.bAllowEditMultipleData = false;
			Properties.PropertiesGPU.bShowTextureArrayPinSettings = true;

			// Also show normal texture settings for texture arrays.
			Properties.PropertiesGPU.bShowTexturePinSettings = true;
		}
		else if (Properties.AllowedTypes.IsChildOf(FPCGDataTypeInfoRawBuffer::AsId()))
		{
			Properties.bAllowMultipleData = false;
			Properties.bAllowEditMultipleData = false;
			Properties.PropertiesGPU.bShowTexturePinSettings = false;
			Properties.PropertiesGPU.bShowTextureArrayPinSettings = false;
		}
		else
		{
			Properties.PropertiesGPU.bShowTexturePinSettings = false;
			Properties.PropertiesGPU.bShowTextureArrayPinSettings = false;
		}

		Properties.PropertiesGPU.bAllowEditInitMode = !bInitModeDrivenByKernel;
		Properties.PropertiesGPU.bMultipleInitPins = Properties.PropertiesGPU.PinsToInititalizeFrom.Num() > 1;

		// Output pins should always allow multiple connections.
		// TODO this could be hoisted up somewhere in the future.
		Properties.bAllowEditMultipleConnections = false;

		Properties.bAllowEditMultipleData = true;
		Properties.PropertiesGPU.bAllowEditDataCount = true;

		// Static initialization to zero is allowed for pins that the kernel is not controlling, and is not currently implemented for textures.
		Properties.PropertiesGPU.bShowInitializeToZeroSetting = !bInitModeDrivenByKernel
			&& (!!(Properties.AllowedTypes & EPCGDataType::Point) || !!(Properties.AllowedTypes & EPCGDataType::Param));

		// Auto-initialize output from input is only relevant for the primary output pin of processors and generators.
		Properties.PropertiesGPU.bShowAutoInitializeOutput = bInitModeDrivenByKernel;

		// Show the legacy "Attributes to Create" array only when the user has explicitly opted in.
		Properties.PropertiesGPU.bShowCreatedAttributesSettings = bPerPinAttributeCreationSettings;
	}
}

void UPCGCustomHLSLSettings::UpdateAttributeKeys()
{
	// Keep identifiers on manually-authored CreatedKernelAttributeKeys in sync with their selectors.
	// These authored entries are still respected (as a base that HLSL inference augments).
	bool bMarkedDirty = false;
	for (FPCGPinPropertiesGPU& OutputPin : OutputPins)
	{
		for (FPCGKernelAttributeKey& AttributeKey : OutputPin.PropertiesGPU.CreatedKernelAttributeKeys)
		{
			if (AttributeKey.UpdateIdentifierFromSelector() && !bMarkedDirty)
			{
				bMarkedDirty = true;
				MarkPackageDirty();
			}
		}
	}
}

void UPCGCustomHLSLSettings::OnComputeSourceModified(const UPCGComputeSource* InModifiedComputeSource)
{
	TArray<UComputeSource*> ComputeSourcesToVisit;
	TSet<UComputeSource*> VisitedComputeSources;

	ComputeSourcesToVisit.Push(KernelSourceOverride);
	ComputeSourcesToVisit.Append(AdditionalSources);

	// Visit the entire network of additional sources to see if our source depends on the modified compute source.
	bool bAnyMatch = false;

	while (!ComputeSourcesToVisit.IsEmpty())
	{
		if (UComputeSource* ComputeSource = ComputeSourcesToVisit.Pop())
		{
			if (ComputeSource == InModifiedComputeSource)
			{
				bAnyMatch = true;
				break;
			}

			VisitedComputeSources.Add(ComputeSource);

			for (UComputeSource* AdditionalSource : ComputeSource->AdditionalSources)
			{
				if (!VisitedComputeSources.Contains(AdditionalSource))
				{
					ComputeSourcesToVisit.Add(AdditionalSource);
				}
			}
		}
	}

	if (bAnyMatch)
	{
		// @todo_pcg: Revisit whether we can remove Structural from this (and other) source modifications.
		OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::ShaderSource | EPCGChangeType::Structural);
	}
}

TArray<FName> UPCGCustomHLSLSettings::GetInputPinNames() const
{
	TArray<FName> PinNames;

	for (const FPCGPinProperties& PinProps : InputPins)
	{
		PinNames.Add(PinProps.Label);
	}

	return PinNames;
}

TArray<FName> UPCGCustomHLSLSettings::GetInputPinNamesAndNone() const
{
	TArray<FName> PinNames = GetInputPinNames();
	PinNames.Insert(NAME_None, 0);

	return PinNames;
}
#endif // WITH_EDITOR

bool FPCGCustomHLSLElement::ExecuteInternal(FPCGContext* Context) const
{
	// This element does not support CPU execution and we are never supposed to land here.
	check(false);
	return true;
}

#undef LOCTEXT_NAMESPACE
