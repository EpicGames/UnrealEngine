// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Compute/IPCGCodeEditorTextProvider.h"

#include "Compute/PCGPinPropertiesGPU.h"

#include "PCGCustomHLSL.generated.h"

class UPCGComputeKernel;
class UPCGComputeSource;
class UPCGNode;
class UPCGPin;

class UComputeSource;

/** Type of kernel allows us to make decisions about execution automatically, streamlining authoring. */
UENUM()
enum class EPCGKernelType : uint8
{
	PointProcessor UMETA(Tooltip = "Kernel executes on each point in first input pin."),
	PointGenerator UMETA(Tooltip = "Kernel executes for fixed number of points, configurable on node."),
	TextureProcessor UMETA(Tooltip = "Kernel executes on each texel in the first input pin."),
	TextureGenerator UMETA(Tooltip = "Kernel executes for each texel in a fixed size texture, configurable on node."),
	Custom UMETA(Tooltip = "Execution thread counts and output buffer sizes configurable on node. All data read/write indices must be manually bounds checked."),
	AttributeSetProcessor UMETA(Tooltip = "Kernel executes on each element of the attribute sets in the first input pin."),
	AttributeSetGenerator UMETA(Tooltip = "Kernel executes on an attribute set with a fixed number of elements, configurable on node."),
	TextureArrayGenerator UMETA(Tooltip = "Kernel executes for each texel-slice in a fixed size Texture2DArray, configurable on node."),
	TextureArrayProcessor UMETA(Tooltip = "Kernel executes on each texel-slice in the first input Texture2DArray pin."),
};

/** Total number of threads that will be dispatched for this kernel. */
UENUM()
enum class EPCGDispatchThreadCount : uint8
{
	FromFirstOutputPin UMETA(Tooltip = "One thread per pin data element."),
	Fixed UMETA(DisplayName = "Fixed Thread Count"),
	FromProductOfInputPins UMETA(Tooltip = "Dispatches a thread per element in the product of one or more pins. So if there are 4 data elements in pin A and 6 data elements in pin B, 24 threads will be dispatched."),
};

/** Produces a HLSL compute shader which will be executed on the GPU. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCustomHLSLSettings
	: public UPCGSettings
	, public IPCGCodeEditorTextProvider
{
	GENERATED_BODY()

public:
	UPCGCustomHLSLSettings();

#if WITH_EDITOR
	//~Begin UObject interface
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void BeginDestroy() override;
	//~End UObject interface
#endif

	//~Begin UPCGSettings interface
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return InputPins; }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual bool IsInputPinRequiredByExecution(const UPCGPin* InPin) const override { return true; }
	virtual bool UseSeed() const override { return true; }
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CustomHLSL")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCustomHLSLElement", "NodeTitle", "Custom HLSL"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGCustomHLSLElement", "NodeTooltip", "Produces a HLSL compute shader which will be executed on the GPU."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::GPU; }
	virtual void CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const override;

	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
#endif

	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;

protected:
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
#if WITH_EDITOR
	//~Begin IPCGCodeEditorTextProvider interface
	FString GetSourceText() const override;
	FString GetDeclarationsText() const override;
	FString GetFunctionsText() const override;
	void SetFunctionsText(const FString& InText) override;
	void SetSourceText(const FString& InText) override;
	bool IsReadOnly() const override;
	//~End IPCGCodeEditorTextProvider interface

	/** Sets the kernel type and updates pin settings and declarations. */
	void SetKernelType(EPCGKernelType InKernelType);

	/** Sets bPerPinAttributeCreationSettings. When false (default), authored CreatedKernelAttributeKeys are ignored and only HLSL inference applies. */
	void SetPerPinAttributeCreationSettings(bool bInValue) { bPerPinAttributeCreationSettings = bInValue; }
#endif

#if WITH_EDITOR
	UFUNCTION()
	static bool SupportsComposition() { return false; }

	UFUNCTION()
	static TArray<FPCGDataTypeIdentifier> GetAllowedInputTypes();

	UFUNCTION()
	static TArray<FPCGDataTypeIdentifier> GetAllowedOutputTypes();
#endif // WITH_EDITOR

protected:
	/** Gets the GPU pin properties for the output pin with the given label. */
	const FPCGPinPropertiesGPU* GetOutputPinPropertiesGPU(const FName& InPinLabel) const;

#if WITH_EDITOR
	void UpdateDeclarations();
	void UpdateInputDeclarations();
	void UpdateOutputDeclarations();
	void UpdateHelperDeclarations();

	/** Enforce required pin settings and set display toggles to drive UI. */
	void UpdatePinSettings();
	void UpdateAttributeKeys();

	/** Called when a compute source is modified to propagate graph refreshes. */
	void OnComputeSourceModified(const UPCGComputeSource* InModifiedComputeSource);

	/** List of all non-advanced input pin names. */
	UFUNCTION()
	TArray<FName> GetInputPinNames() const;

	/** List of all non-advanced input pin names, prepended with 'Name_NONE'. */
	UFUNCTION()
	TArray<FName> GetInputPinNamesAndNone() const;
#endif

	const FPCGPinProperties* GetFirstInputPinProperties() const;
	const FPCGPinPropertiesGPU* GetFirstOutputPinProperties() const;

	/** Will the ThreadCountMultiplier value be applied when calculating the dispatch thread count. */
	bool IsThreadCountMultiplierInUse() const { return KernelType == EPCGKernelType::Custom && DispatchThreadCount != EPCGDispatchThreadCount::Fixed; }

	bool IsProcessorKernel() const { return KernelType == EPCGKernelType::PointProcessor || KernelType == EPCGKernelType::TextureProcessor || KernelType == EPCGKernelType::AttributeSetProcessor || KernelType == EPCGKernelType::TextureArrayProcessor; }
	bool IsGeneratorKernel() const { return KernelType == EPCGKernelType::PointGenerator || KernelType == EPCGKernelType::TextureGenerator || KernelType == EPCGKernelType::AttributeSetGenerator || KernelType == EPCGKernelType::TextureArrayGenerator; }

	/** The dimensionality of the kernels processing. Custom kernel type is 1D. */
	EPCGElementDimension GetElementDimension() const;

	/** Primary element data type of the kernel, if there is one (returns None for Custom kernel type). */
	FPCGDataTypeIdentifier GetElementType() const;

	bool IsTextureKernel() const { return KernelType == EPCGKernelType::TextureGenerator || KernelType == EPCGKernelType::TextureProcessor || KernelType == EPCGKernelType::TextureArrayGenerator || KernelType == EPCGKernelType::TextureArrayProcessor; }
	bool IsPointKernel() const { return GetElementType() == EPCGDataType::Point; }
	bool IsAttributeSetKernel() const { return GetElementType() == EPCGDataType::Param; }
	bool IsCustomKernel() const { return KernelType == EPCGKernelType::Custom; }

	UFUNCTION()
	bool DisplayNumElements1D() const { return IsGeneratorKernel() && GetElementDimension() == EPCGElementDimension::One; }
	UFUNCTION()
	bool DisplayNumElements2D() const { return IsGeneratorKernel() && GetElementDimension() == EPCGElementDimension::Two; }
	UFUNCTION()
	bool DisplayNumElements3D() const { return IsGeneratorKernel() && GetElementDimension() == EPCGElementDimension::Three; }

protected:
	UPROPERTY(EditAnywhere, Category = "Settings")
	EPCGKernelType KernelType = EPCGKernelType::PointProcessor;

public:
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "DisplayNumElements1D()", EditConditionHides, PCG_OverridableCPUAndGPUWithReadback))
	int NumElements = 256;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "DisplayNumElements2D() || DisplayNumElements3D()", EditConditionHides, PCG_OverridableCPUAndGPUWithReadback))
	int NumElementsX = 64;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "DisplayNumElements2D() || DisplayNumElements3D()", EditConditionHides, PCG_OverridableCPUAndGPUWithReadback))
	int NumElementsY = 64;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "DisplayNumElements3D()", EditConditionHides, PCG_OverridableCPUAndGPUWithReadback))
	int NumElementsZ = 1;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "KernelType == EPCGKernelType::TextureGenerator || KernelType == EPCGKernelType::TextureArrayGenerator", EditConditionHides, PCG_OverridableCPUAndGPUWithReadback))
	EPCGRenderTargetFormat TextureFormat = EPCGRenderTargetFormat::RGBA16f;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "KernelType == EPCGKernelType::TextureGenerator || KernelType == EPCGKernelType::TextureArrayGenerator", EditConditionHides, PCG_OverridableCPUAndGPUWithReadback))
	EPCGTextureFilter TextureFilter = EPCGTextureFilter::Bilinear;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "KernelType == EPCGKernelType::TextureGenerator || KernelType == EPCGKernelType::TextureArrayGenerator", EditConditionHides))
	bool bOverrideTextureTransform = false;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "(KernelType == EPCGKernelType::TextureGenerator || KernelType == EPCGKernelType::TextureArrayGenerator) && bOverrideTextureTransform", EditConditionHides))
	FTransform TextureTransform = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = "Settings|Thread Count", meta = (EditCondition = "KernelType == EPCGKernelType::Custom", EditConditionHides))
	EPCGDispatchThreadCount DispatchThreadCount = EPCGDispatchThreadCount::FromFirstOutputPin;

	UPROPERTY(EditAnywhere, Category = "Settings|Thread Count", meta = (EditCondition = "KernelType == EPCGKernelType::Custom && DispatchThreadCount != EPCGDispatchThreadCount::Fixed", EditConditionHides, PCG_OverridableCPUAndGPUWithReadback))
	int ThreadCountMultiplier = 1;

	UPROPERTY(EditAnywhere, Category = "Settings|Thread Count", meta = (EditCondition = "KernelType == EPCGKernelType::Custom && DispatchThreadCount == EPCGDispatchThreadCount::Fixed", EditConditionHides, PCG_OverridableCPUAndGPUWithReadback))
	int FixedThreadCount = 1;

	UPROPERTY(EditAnywhere, DisplayName = "Input Pins", Category = "Settings|Thread Count", meta = (EditCondition = "KernelType == EPCGKernelType::Custom && DispatchThreadCount == EPCGDispatchThreadCount::FromProductOfInputPins", EditConditionHides, GetOptions = "GetInputPinNames"))
	TArray<FName> ThreadCountInputPinLabels;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PCG_TypeFilter = "GetAllowedInputTypes()"))
	TArray<FPCGPinProperties> InputPins = Super::DefaultPointInputPinProperties();

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PCG_TypeFilter = "GetAllowedOutputTypes()"))
	TArray<FPCGPinPropertiesGPU> OutputPins = { FPCGPinPropertiesGPU(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeIdentifier{EPCGDataType::Point}) };

#if WITH_EDITOR
	/** Snapshot of input pins from PreEditChange, used in PostEditPropertyChange to diff label and type changes. */
	TArray<FPCGPinProperties> InputPinsPreEditChange;
#endif

protected:
#if WITH_EDITORONLY_DATA
	/** Override your kernel with a PCG compute source asset. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (AllowedClasses = "/Script/PCG.PCGComputeSource"))
	TObjectPtr<UComputeSource> KernelSourceOverride;

	/** Additional source files to use in your kernel. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (AllowedClasses = "/Script/PCG.PCGComputeSource"))
	TArray<TObjectPtr<UComputeSource>> AdditionalSources;
#endif

	/** Mute uninitialized data errors. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bMuteUnwrittenPinDataErrors = false;

	/** Expose a per-pin "Attributes to Create" array so attributes can be manually specified. Attributes written via Set calls in HLSL (e.g. Out_SetFloat) are now inferred automatically, so this is no longer required. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Settings")
	bool bPerPinAttributeCreationSettings = false;

public:
	bool GetUseLegacyDataCollectionAPI() const { return bUseLegacyDataCollectionAPI; }

protected:
	/** Use the legacy data collection API where every pin direction exposes the full set of getter and setter functions regardless of pin direction. */
	UPROPERTY()
	bool bUseLegacyDataCollectionAPI = true;

#if WITH_EDITORONLY_DATA
	// Shader source and declarations are entirely editor-only, and should never be serialized outside of the editor.

	/** Optional functions that can be called from the source. Intended to be edited using the HLSL Source Editor window. */
	UPROPERTY()
	FString ShaderFunctions = "/** CUSTOM SHADER FUNCTIONS **/\n";

	/** Shader code that forms the body of the kernel. Intended to be edited using the HLSL Source Editor window. */
	UPROPERTY()
	FString ShaderSource;

	/** Inputs data accessors that can be used from the shader code. Intended to be viewed using the HLSL Source Editor window. */
	UPROPERTY(Transient)
	FString InputDeclarations;

	/** Output data accessors that can be used from the shader code. Intended to be viewed using the HLSL Source Editor window. */
	UPROPERTY(Transient)
	FString OutputDeclarations;

	/** Helper data and functions that can be used from the shader code. Intended to be viewed using the HLSL Source Editor window. */
	UPROPERTY(Transient)
	FString HelperDeclarations;

	UPROPERTY()
	int PointCount_DEPRECATED = 0;

	UPROPERTY()
	FIntPoint NumElements2D_DEPRECATED = FIntPoint(0, 0);
#endif

	friend class UPCGCustomHLSLKernel;
	friend struct FPCGCustomHLSLSettingsTestHelper;
};

class FPCGCustomHLSLElement : public IPCGElement
{
protected:
	// This will only be called if the custom HLSL node is not set up correctly (valid nodes are replaced with a compute graph element).
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
