// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/Elements/PCGCustomHLSLKernel.h"

#include "PCGCommon.h"
#include "PCGComputeGraphElement.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGPoint.h"
#include "PCGPointPropertiesTraits.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Compute/DataInterfaces/Elements/PCGCustomHLSLDataInterface.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include "ComputeFramework/ComputeSource.h"
#include "Containers/StaticArray.h"

#if WITH_EDITOR
#include "Compute/PCGHLSLSyntaxTokenizer.h"

#include "Framework/Text/SyntaxTokenizer.h"
#include "Internationalization/Regex.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCustomHLSLKernel)

#define LOCTEXT_NAMESPACE "PCGCustomHLSLKernel"

namespace PCGCustomHLSLKernel
{
	constexpr TCHAR AttributeFunctionGetKeyword[] = { TEXT("Get") };
	constexpr TCHAR AttributeFunctionSetKeyword[] = { TEXT("Set") };
	constexpr TCHAR AttributeFunctionAtomicAddKeyword[] = { TEXT("AtomicAdd") };
	constexpr TCHAR AttributeFunctionAtomicAddType[] = { TEXT("Int") };
	constexpr TCHAR CopyElementFunctionKeyword[] = { TEXT("CopyElementFrom") };
	constexpr TCHAR InitializeFunctionKeyword[] = { TEXT("InitializePoint") };
	constexpr TCHAR StoreFunctionKeyword[] = { TEXT("Store") };
	constexpr TCHAR Store4FunctionKeyword[] = { TEXT("Store4") };
	constexpr TCHAR SetPositionKeyword[] = { TEXT("SetPosition") };
	constexpr TCHAR SetRotationKeyword[] = { TEXT("SetRotation") };
	constexpr TCHAR SetScaleKeyword[] = { TEXT("SetScale") };
	constexpr TCHAR SetBoundsMinKeyword[] = { TEXT("SetBoundsMin") };
	constexpr TCHAR SetBoundsMaxKeyword[] = { TEXT("SetBoundsMax") };
	constexpr TCHAR SetColorKeyword[] = { TEXT("SetColor") };
	constexpr TCHAR SetDensityKeyword[] = { TEXT("SetDensity") };
	constexpr TCHAR SetSteepnessKeyword[] = { TEXT("SetSteepness") };
	constexpr TCHAR SetSeedKeyword[] = { TEXT("SetSeed") };
	constexpr TCHAR SetPointTransformKeyword[] = { TEXT("SetPointTransform") };

	// Per-component bits for PinWrittenTransformSetters - finer-grained than the combined EPCGPointNativeProperties::Transform bit, so smart-init can handle P/R/S independently.
	constexpr int32 TRSPositionBit = 1;
	constexpr int32 TRSRotationBit = 2;
	constexpr int32 TRSScaleBit    = 4;

#if WITH_EDITOR
	// Helpers to format C++ default values as HLSL literals.
	FString ToHLSLFloat(float InValue)    { return FString::Printf(TEXT("%g"), InValue); }
	FString ToHLSLInt(int32 InValue)      { return FString::Printf(TEXT("%d"), InValue); }
	FString ToHLSLVec3(const FVector& InVector)  { return FString::Printf(TEXT("float3(%g, %g, %g)"), InVector.X, InVector.Y, InVector.Z); }
	FString ToHLSLVec4(const FVector4& InVector) { return FString::Printf(TEXT("float4(%g, %g, %g, %g)"), InVector.X, InVector.Y, InVector.Z, InVector.W); }
	FString ToHLSLQuat(const FQuat& InQuat)      { return ToHLSLVec4(FVector4(InQuat.X, InQuat.Y, InQuat.Z, InQuat.W)); }

	/** Appends HLSL to zero-initialize metadata attributes per-element. No-op if AttrKeys is null or empty (no new output-only attributes for this pin). */
	void AddMetadataInitPreamble(FName InPinLabel, const FPCGKernelAttributeKeyList* AttrKeys, FString& OutPreamble)
	{
		if (AttrKeys && !AttrKeys->Keys.IsEmpty())
		{
			const FString PinLabel = InPinLabel.ToString();
			OutPreamble += FString::Format(TEXT(
				"\n"
				"    // Initialize metadata attributes per-element.\n"
				"    {0}_InitializeMetadata({0}_DataIndex, ElementIndex);\n"),
				{ PinLabel });
		}
	}

	/** Point properties excluding Density (which is always non-CVR). Each entry maps an allocation bit to a HLSL stem: the getter is Get{Stem} and the setter is Set{Stem}. */
	struct FPointPropertyEntry
	{
		EPCGPointNativeProperties AllocBit = EPCGPointNativeProperties::None;
		const TCHAR* Stem = nullptr;
	};

	static constexpr FPointPropertyEntry PointPropertiesExceptDensity[] =
	{
		// Position, Rotation, Scale all share the Transform bit.
		{ EPCGPointNativeProperties::Transform, TEXT("Position") },
		{ EPCGPointNativeProperties::Transform, TEXT("Rotation") },
		{ EPCGPointNativeProperties::Transform, TEXT("Scale")    },
		{ EPCGPointNativeProperties::BoundsMin, TEXT("BoundsMin") },
		{ EPCGPointNativeProperties::BoundsMax, TEXT("BoundsMax") },
		{ EPCGPointNativeProperties::Color,     TEXT("Color")     },
		{ EPCGPointNativeProperties::Seed,      TEXT("Seed")      },
		{ EPCGPointNativeProperties::Steepness, TEXT("Steepness") },
	};

	/** Collects one line per CVR property from BuildLine(Stem) and wraps them in an 'if (ElementIndex == 0)' block. No-op if all properties are fully allocated. */
	void AppendCVRBlock(int32 AllocProps, FString& OutCode, TFunctionRef<FString(const TCHAR*)> BuildLine)
	{
		FString Lines;
		for (const FPointPropertyEntry& Entry : PointPropertiesExceptDensity)
		{
			if (!(AllocProps & static_cast<int32>(Entry.AllocBit)))
			{
				Lines += BuildLine(Entry.Stem);
			}
		}

		if (!Lines.IsEmpty())
		{
			OutCode += FString::Format(TEXT(
				"\n"
				"    // Initialize CVR native properties at element 0; one write covers all N elements.\n"
				"    if (ElementIndex == 0)\n"
				"    {\n"
				"{0}"
				"    }\n"),
				{ Lines });
		}
	}

	/** Appends one line per fully-allocated (non-CVR) property from BuildLine(Stem), plus always Density (which is never CVR). */
	void AppendPerElementBlock(int32 AllocProps, FString& OutCode, TFunctionRef<FString(const TCHAR*)> BuildLine)
	{
		FString Lines;
		for (const FPointPropertyEntry& Entry : PointPropertiesExceptDensity)
		{
			if (AllocProps & static_cast<int32>(Entry.AllocBit))
			{
				Lines += BuildLine(Entry.Stem);
			}
		}

		// Density is always non-CVR to support RemovePoint which writes PCG_INVALID_DENSITY per-element.
		Lines += BuildLine(TEXT("Density"));

		OutCode += FString::Format(TEXT("\n{0}"), { Lines });
	}

	/** Appends the point generator preamble: CVR-aware init of all native properties to FPCGPoint defaults. CVR-eligible properties are initialized at element 0 only; allocated properties and Density per-element. */
	void AddPointGeneratorPreamble(FName InPinLabel, int32 AllocProps, FString& OutPreamble)
	{
		const FString PinLabel = InPinLabel.ToString();
		const FPCGPoint DefaultPoint{};

		auto DefaultValue = [&DefaultPoint](const TCHAR* Stem) -> FString
		{
			if (FStringView(Stem) == TEXTVIEW("Position"))
			{
				return ToHLSLVec3(DefaultPoint.Transform.GetTranslation());
			}
			else if (FStringView(Stem) == TEXTVIEW("Rotation"))
			{
				return ToHLSLQuat(DefaultPoint.Transform.GetRotation());
			}
			else if (FStringView(Stem) == TEXTVIEW("Scale"))
			{
				return ToHLSLVec3(DefaultPoint.Transform.GetScale3D());
			}
			else if (FStringView(Stem) == TEXTVIEW("BoundsMin"))
			{
				return ToHLSLVec3(FVector(-50.0f)); // PCG GPU point default
			}
			else if (FStringView(Stem) == TEXTVIEW("BoundsMax"))
			{
				return ToHLSLVec3(FVector(50.0f)); // PCG GPU point default
			}
			else if (FStringView(Stem) == TEXTVIEW("Color"))
			{
				return ToHLSLVec4(DefaultPoint.Color);
			}
			else if (FStringView(Stem) == TEXTVIEW("Seed"))
			{
				return ToHLSLInt(PCGValueConstants::DefaultSeed);
			}
			else if (FStringView(Stem) == TEXTVIEW("Steepness"))
			{
				return ToHLSLFloat(DefaultPoint.Steepness);
			}
			else if (FStringView(Stem) == TEXTVIEW("Density"))
			{
				return ToHLSLFloat(DefaultPoint.Density);
			}
			else
			{
				checkNoEntry();
				return {};
			}
		};

		AppendCVRBlock(AllocProps, OutPreamble, [&PinLabel, &DefaultValue](const TCHAR* Stem) -> FString
		{
			return FString::Format(TEXT("        {0}_Set{1}({0}_DataIndex, 0, {2});\n"), { PinLabel, FString(Stem), DefaultValue(Stem) });
		});

		AppendPerElementBlock(AllocProps, OutPreamble, [&PinLabel, &DefaultValue](const TCHAR* Stem) -> FString
		{
			return FString::Format(TEXT("    {0}_Set{1}({0}_DataIndex, ElementIndex, {2});\n"), { PinLabel, FString(Stem), DefaultValue(Stem) });
		});
	}

	/** Appends the texture generator preamble: zero-initializes the output texel. */
	void AddTextureGeneratorPreamble(FName InPinLabel, FString& OutPreamble)
	{
		const FString PinLabel = InPinLabel.ToString();
		OutPreamble += FString::Format(TEXT(
			"\n"
			"    // Zero-initialize for output pin {0}\n"
			"    {0}_Store({0}_DataIndex, ElementIndex, (float4)0.0f);\n"),
			{ PinLabel });
	}

	/** Appends the point processor preamble: early-out if the input point is removed, then copies all native point properties (CVR-aware) and metadata attributes from input to output.
	 *
	 *  For properties not written by the user (AllocProps bit clear), the copy is guarded by a check against the
	 *  AllocatedNativePropertiesMask uniform - a C++-supplied bitmask of which output properties are actually N-slot
	 *  at runtime (set from the output pin's data description at dispatch time):
	 *      if (ElementIndex == 0 || (Out_GetAllocatedNativePropertiesMask() & <EPCGPointNativeProperties bit>))
	 *          Out_SetColor(Out_DataIndex, ElementIndex, In_GetColor(In_DataIndex, ElementIndex));
	 *  CVR output (bit clear): only thread 0 copies - preserving single-write CVR performance.
	 *  N-slot output (bit set, e.g. inherited from per-element input data): all threads copy their element.
	 *
	 *  Properties already in AllocProps (written by user's HLSL) go straight to AppendPerElementBlock: the output
	 *  buffer for those is always N-slot so no guard is needed. */
	void InitializeOutputDataPoint(FName InInputLabel, FName InOutputLabel, int32 AllocProps, FString& OutPreamble)
	{
		const FString InputLabel = InInputLabel.ToString();
		const FString OutputLabel = InOutputLabel.ToString();

		// CVR-eligible properties: guard each copy with the C++-supplied AllocatedNativePropertiesMask uniform.
		FString CVRLines;
		for (const FPointPropertyEntry& Entry : PointPropertiesExceptDensity)
		{
			if (!(AllocProps & static_cast<int32>(Entry.AllocBit)))
			{
				CVRLines += FString::Format(
					TEXT("    if (ElementIndex == 0 || ({0}_GetAllocatedNativePropertiesMask() & {2}u))\n")
					TEXT("    {\n")
					TEXT("        {0}_Set{3}({0}_DataIndex, ElementIndex, {1}_Get{3}({1}_DataIndex, ElementIndex));\n")
					TEXT("    }\n"),
					{ OutputLabel, InputLabel, FString::FromInt(static_cast<int32>(Entry.AllocBit)), FString(Entry.Stem) });
			}
		}

		if (!CVRLines.IsEmpty())
		{
			OutPreamble += FString::Format(TEXT("\n    // Copy unwritten native properties; mask from C++ preserves CVR when output is 1-slot, copies per-element when N-slot.\n{0}"), { CVRLines });
		}

		AppendPerElementBlock(AllocProps, OutPreamble, [&OutputLabel, &InputLabel](const TCHAR* Stem) -> FString
		{
			return FString::Format(TEXT("    {0}_Set{2}({0}_DataIndex, ElementIndex, {1}_Get{2}({1}_DataIndex, ElementIndex));\n"), { OutputLabel, InputLabel, FString(Stem) });
		});

		// Copy metadata attributes from input. Existing attrs are copied; new output-only attrs are zeroed. Must be done as dynamic loop as we don't know in advance which metadata attributes are present.
		OutPreamble += FString::Format(TEXT(
			"\n"
			"    // Copy metadata attributes from input.\n"
			"    PCG_COPY_METADATA_ATTRIBUTES_TO_OUTPUT({0}, {1}, {0}_DataIndex, ElementIndex, {1}_DataIndex, ElementIndex);\n"),
			{ OutputLabel, InputLabel });
	}

	/** Appends the attribute set processor preamble: copies all attributes to the output. */
	void InitializeOutputDataAttributeSet(FName InInputLabel, FName InOutputLabel, FString& OutPreamble)
	{
		const FString InputLabel = InInputLabel.ToString();
		const FString OutputLabel = InOutputLabel.ToString();
		// todo_pcg: pass in IDs of attributes that are actually present?
		OutPreamble += FString::Format(TEXT(
			"\n"
			"    // Processor always initializes outputs by copying input data elements.\n"
			"    PCG_COPY_ALL_ATTRIBUTES_TO_OUTPUT({1}, {0}, {1}_DataIndex, ElementIndex, {0}_DataIndex, ElementIndex);\n"),
			{ InputLabel, OutputLabel });
	}

	/** Appends the texture processor preamble: copies the input texel to the output. */
	void InitializeOutputDataTexture(FName InInputLabel, FName InOutputLabel, FString& OutPreamble)
	{
		const FString InputLabel = InInputLabel.ToString();
		const FString OutputLabel = InOutputLabel.ToString();
		OutPreamble += FString::Format(TEXT(
			"\n"
			"    // Processor always initializes outputs by copying the inputs.\n"
			"    {1}_Store({1}_DataIndex, ElementIndex, {0}_Load({0}_DataIndex, ElementIndex));\n"),
			{ InputLabel, OutputLabel });
	}

	enum class EParseState : uint8
	{
		None,
		LookingForDoubleQuotedString,
		LookingForSingleQuotedString,
		LookingForSingleLineComment,
		LookingForMultiLineComment,
	};

	FString GetDataTypeString(const FPCGDataTypeIdentifier& Type)
	{
		return Type.ToString();
	}

	// Reference implementation in FHLSLSyntaxHighlighterMarshaller::ProcessTokenizedLine()
	void ProcessTokenizedLine(const FString& InSourceString, const ISyntaxTokenizer::FTokenizedLine& InTokenizedLine, EParseState& InOutParseState, TArray<FPCGCustomHLSLParsedSource::FToken>& OutTokens)
	{
		for (const ISyntaxTokenizer::FToken& Token : InTokenizedLine.Tokens)
		{
			FPCGCustomHLSLParsedSource::FToken& Run = OutTokens.Emplace_GetRef();
			Run.Range = Token.Range;
	
			const FString TokenText = InSourceString.Mid(Token.Range.BeginIndex, Token.Range.Len());
			const bool bIsWhitespace = TokenText.TrimEnd().IsEmpty();

			if (!bIsWhitespace)
			{
				bool bHasMatchedSyntax = false;
				if (Token.Type == ISyntaxTokenizer::ETokenType::Syntax)
				{
					if (InOutParseState == EParseState::None && TokenText == TEXT("\""))
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::DoubleQuotedString;
						InOutParseState = EParseState::LookingForDoubleQuotedString;
						bHasMatchedSyntax = true;
					}
					else if(InOutParseState == EParseState::LookingForDoubleQuotedString && TokenText == TEXT("\""))
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::Normal;
						InOutParseState = EParseState::None;
					}
					else if(InOutParseState == EParseState::None && TokenText == TEXT("\'"))
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::SingleQuotedString;
						InOutParseState = EParseState::LookingForSingleQuotedString;
						bHasMatchedSyntax = true;
					}
					else if(InOutParseState == EParseState::LookingForSingleQuotedString && TokenText == TEXT("\'"))
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::Normal;
						InOutParseState = EParseState::None;
					}
					else if(InOutParseState == EParseState::None && TokenText.StartsWith(TEXT("#")))
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::PreProcessorKeyword;
						InOutParseState = EParseState::None;
					}
					else if(InOutParseState == EParseState::None && TokenText == TEXT("//"))
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::Comment;
						InOutParseState = EParseState::LookingForSingleLineComment;
					}
					else if(InOutParseState == EParseState::None && TokenText == TEXT("/*"))
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::Comment;
						InOutParseState = EParseState::LookingForMultiLineComment;
					}
					else if(InOutParseState == EParseState::LookingForMultiLineComment && TokenText == TEXT("*/"))
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::Comment;
						InOutParseState = EParseState::None;
					}
					else if(InOutParseState == EParseState::None && TChar<TCHAR>::IsIdentifier(TokenText[0]))
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::Keyword;
						InOutParseState = EParseState::None;
					}
					else if(InOutParseState == EParseState::None && !TChar<TCHAR>::IsIdentifier(TokenText[0]))
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::Operator;
						InOutParseState = EParseState::None;
					}
				}
				
				// It's possible that we fail to match a syntax token if we're in a state where it isn't parsed
				// In this case, we treat it as a literal token
				if (Token.Type == ISyntaxTokenizer::ETokenType::Literal || !bHasMatchedSyntax)
				{
					if(InOutParseState == EParseState::LookingForDoubleQuotedString)
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::DoubleQuotedString;
					}
					else if(InOutParseState == EParseState::LookingForSingleQuotedString)
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::SingleQuotedString;
					}
					else if(InOutParseState == EParseState::LookingForSingleLineComment)
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::Comment;
					}
					else if(InOutParseState == EParseState::LookingForMultiLineComment)
					{
						Run.Type = FPCGCustomHLSLParsedSource::ETokenType::Comment;
					}
				}
			}
			else
			{
				Run.Type = FPCGCustomHLSLParsedSource::ETokenType::Whitespace;
			}
		}
	
		if (InOutParseState != EParseState::LookingForMultiLineComment)
		{
			InOutParseState = EParseState::None;
		}
	}

	void ParseTokens(const FString& InSourceString, const TArray<ISyntaxTokenizer::FTokenizedLine>& InTokenizedLines, TArray<FPCGCustomHLSLParsedSource::FToken>& OutTokens)
	{
		EParseState ParseState = EParseState::None;

		for(const ISyntaxTokenizer::FTokenizedLine& TokenizedLine : InTokenizedLines)
		{
			ProcessTokenizedLine(InSourceString, TokenizedLine, ParseState, OutTokens);
		}
	}

	/**
	 * Returns true if all meaningful tokens in the range represent a constant expression
	 * (numeric literals, hex literals, HLSL type constructors, and operators only -- no user-defined identifiers).
	 * Whitespace and block/line comments are treated as transparent and do not affect the result.
	 */
	bool IsConstantValueExpression(const FString& Source, TArrayView<const FPCGCustomHLSLParsedSource::FToken> Tokens)
	{
		using ETokenType = FPCGCustomHLSLParsedSource::ETokenType;
		static const TSet<FString> HLSLTypeConstructors = {
			TEXT("float"),  TEXT("float1"),  TEXT("float2"),  TEXT("float3"),  TEXT("float4"),
			TEXT("half"),   TEXT("half1"),   TEXT("half2"),   TEXT("half3"),   TEXT("half4"),
			TEXT("int"),    TEXT("int1"),    TEXT("int2"),    TEXT("int3"),    TEXT("int4"),
			TEXT("uint"),   TEXT("uint1"),   TEXT("uint2"),   TEXT("uint3"),   TEXT("uint4"),
			TEXT("double"), TEXT("bool"),
		};

		bool bFoundNonWhitespace = false;

		// Tracks whether the previous non-whitespace, non-comment token was exactly "0".
		// Hex literals (e.g. 0x2A) tokenize as two consecutive tokens: "0" then "x2A". When
		// this flag is set and the current token matches x[0-9a-fA-F]+, it is accepted as the
		// second half of a hex literal. Whitespace between the two tokens resets this flag
		// (since "0 x2A" is not valid HLSL), but comments do not (they are transparent).
		bool bPrevWasZero = false;

		for (const FPCGCustomHLSLParsedSource::FToken& Token : Tokens)
		{
			switch (Token.Type)
			{
			case ETokenType::Whitespace:
				bPrevWasZero = false; // "0 x2A" is not a hex literal.
				break;
			case ETokenType::Comment:
				break; // Semantically transparent: do not update bPrevWasZero.
			case ETokenType::Operator:
				bFoundNonWhitespace = true;
				bPrevWasZero = false;
				break;
			case ETokenType::Keyword:
			{
				bFoundNonWhitespace = true;
				bPrevWasZero = false;

				const FString Str = Source.Mid(Token.Range.BeginIndex, Token.Range.Len());
				if (!HLSLTypeConstructors.Contains(Str))
				{
					return false;
				}

				break;
			}
			case ETokenType::Normal:
			{
				const FString Str = Source.Mid(Token.Range.BeginIndex, Token.Range.Len());
				if (Str == TEXT(","))
				{
					bPrevWasZero = false;
					break;
				}

				bFoundNonWhitespace = true;

				// Allow tokens that start with a digit or '.', or single-character HLSL numeric literal suffixes (f/h for float/half, u/l for int/long). Without this, '1.0f' tokenizes as
				// '1.0' + 'f' and the 'f' token would incorrectly fail the constant check.
				const bool bIsNumericStart = !Str.IsEmpty() && (FChar::IsDigit(Str[0]) || Str[0] == TEXT('.'));
				const bool bIsLiteralSuffix = Str.Len() == 1 && (Str[0] == TEXT('f') || Str[0] == TEXT('F') || Str[0] == TEXT('h') || Str[0] == TEXT('H') || Str[0] == TEXT('u') || Str[0] == TEXT('U') || Str[0] == TEXT('l') || Str[0] == TEXT('L'));

				// Hex literal continuation: "0" followed immediately by "x[0-9a-fA-F]+" (both produced
				// as separate tokens since the tokenizer splits at alpha/digit boundaries).
				bool bIsHexContinuation = false;
				if (bPrevWasZero && Str.Len() >= 2 && (Str[0] == TEXT('x') || Str[0] == TEXT('X')))
				{
					bIsHexContinuation = true;
					for (int32 i = 1; i < Str.Len() && bIsHexContinuation; ++i)
					{
						const TCHAR c = Str[i];
						bIsHexContinuation = FChar::IsDigit(c) || (c >= TEXT('a') && c <= TEXT('f')) || (c >= TEXT('A') && c <= TEXT('F'));
					}
				}

				if (!bIsNumericStart && !bIsLiteralSuffix && !bIsHexContinuation)
				{
					return false;
				}

				bPrevWasZero = (Str == TEXT("0"));
				break;
			}
			default:
				return false;
			}
		}

		return bFoundNonWhitespace;
	}
#endif
}

#if WITH_EDITOR
void UPCGCustomHLSLKernel::InitializeInternal()
{
	Super::InitializeInternal();

	InitEntryPoint();

	// Must run before PopulateAttributeKeys to populate ParsedSources.
	ParseShaderSource();

	// Combine CreatedKernelAttributeKeys + inferred HLSL keys into PinCreatedAttributeKeys.
	PopulateAttributeKeys();
}
#endif // WITH_EDITOR

bool UPCGCustomHLSLKernel::IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCustomHLSLKernel::IsKernelDataValid);

	if (!Super::IsKernelDataValid(InDataBinding, InContext))
	{
		return false;
	}

	if (InDataBinding)
	{
		FText* ErrorTextPtr = nullptr;
#if PCG_KERNEL_LOGGING_ENABLED
		FText ErrorText;
		ErrorTextPtr = &ErrorText;
#endif

		if (!AreAttributesValid(InDataBinding, InContext, ErrorTextPtr))
		{
			if (ErrorTextPtr)
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), *ErrorTextPtr);
			}
			return false;
		}
	}

	return true;
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGCustomHLSLKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);
	
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());

	const FPCGPinPropertiesGPU* OutputPinProperties = CustomHLSLSettings->OutputPins.FindByPredicate([InOutputPinLabel](const FPCGPinPropertiesGPU& InProps) { return InProps.Label == InOutputPinLabel; });
	if (!OutputPinProperties)
	{
		return nullptr;
	}

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = nullptr;
	const FName FirstOutputPinLabel = CustomHLSLSettings->OutputPins[0].Label;

	// The primary output pin follows any rules prescribed by kernel type.
	if (InOutputPinLabel == FirstOutputPinLabel && CustomHLSLSettings->IsProcessorKernel())
	{
		if (const FPCGPinProperties* FirstInputPinProps = GetFirstInputPin())
		{
			const FPCGKernelPin FirstKernelPin(GetKernelIndex(), FirstInputPinProps->Label, /*bIsInput=*/true);
			const TSharedPtr<const FPCGDataCollectionDesc> InputDataDesc = InBinding->ComputeKernelPinDataDesc(FirstKernelPin);

			if (ensure(InputDataDesc))
			{
				OutDataDesc = FPCGDataCollectionDesc::MakeSharedFrom(InputDataDesc);
			}
			else
			{
				UE_LOGF(LogPCG, Warning, "Kernel pin not registered in compute graph. May be due to unsupported pin type. KernelIndex=%d, PinLabel='%ls', Input=%d",
					FirstKernelPin.KernelIndex,
					*FirstKernelPin.PinLabel.ToString(),
					FirstKernelPin.bIsInput);

				return nullptr;
			}
		}
	}
	else if (InOutputPinLabel == FirstOutputPinLabel && CustomHLSLSettings->IsGeneratorKernel() && CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::One)
	{
		const FPCGKernelParams* KernelParams = InBinding->GetCachedKernelParams(this);

		if (!ensure(KernelParams))
		{
			return nullptr;
		}

		OutDataDesc = FPCGDataCollectionDesc::MakeShared();

		// Generators always produce a single data with known element count.
		OutDataDesc->GetDataDescriptionsMutable().Emplace(CustomHLSLSettings->GetElementType(), KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElements)));
	}
	else if (InOutputPinLabel == FirstOutputPinLabel && (CustomHLSLSettings->KernelType == EPCGKernelType::TextureGenerator || CustomHLSLSettings->KernelType == EPCGKernelType::TextureArrayGenerator))
	{
		const FPCGKernelParams* KernelParams = InBinding->GetCachedKernelParams(this);

		if (!ensure(KernelParams))
		{
			return nullptr;
		}

		OutDataDesc = FPCGDataCollectionDesc::MakeShared();

		const bool bIsArray = CustomHLSLSettings->KernelType == EPCGKernelType::TextureArrayGenerator;
		const int NumElementsX = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsX));
		const int NumElementsY = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsY));
		const int NumElementsZ = bIsArray ? KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsZ)) : 0;
		const EPCGRenderTargetFormat TextureFormat = KernelParams->GetValueEnum<EPCGRenderTargetFormat>(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, TextureFormat));
		const EPCGTextureFilter TextureFilter = KernelParams->GetValueEnum<EPCGTextureFilter>(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, TextureFilter));

		// Generators always produce a single texture (or texture array) data with known size.
		const FPCGDataTypeIdentifier DataType = bIsArray ? FPCGDataTypeInfoTexture2DArray::AsId() : FPCGDataTypeIdentifier(EPCGDataType::BaseTexture);
		OutDataDesc->GetDataDescriptionsMutable().Emplace(DataType, FIntVector4(NumElementsX, NumElementsY, NumElementsZ, 0));
		OutDataDesc->SetRenderTargetFormatForAllData(TextureFormat);
		OutDataDesc->SetTextureFilterForAllData(TextureFilter);

		// For arrays, ElementCount.Z sets the array size.
		if (bIsArray)
		{
			OutDataDesc->SetTextureArraySizeForAllData(NumElementsZ);
		}

		if (CustomHLSLSettings->bOverrideTextureTransform)
		{
			OutDataDesc->SetTextureTransformForAllData(CustomHLSLSettings->TextureTransform);
		}
		else
		{
			TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
			if (FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr)
			{
				OutDataDesc->SetTextureTransformForAllData(UPCGTexture2DBaseData::ComputeTransform(Context->ExecutionSource.Get()));
			}
		}
	}
	else
	{
		OutDataDesc = FPCGDataCollectionDesc::MakeShared();

		ComputeDataDescFromPinProperties(*OutputPinProperties, MakeArrayView(CustomHLSLSettings->InputPins), InBinding, OutDataDesc);
	}

	if (!ensure(OutDataDesc))
	{
		return nullptr;
	}

	// Add attributes created for this output pin. PinCreatedAttributeKeys is pre-computed during editor init
	// (in PopulateAttributeKeys) from both manually-authored CreatedKernelAttributeKeys and HLSL Set/AtomicAdd calls.
	if (const FPCGKernelAttributeKeyList* CreatedKeyList = PinCreatedAttributeKeys.Find(InOutputPinLabel))
	{
		for (const FPCGKernelAttributeKey& CreatedKey : CreatedKeyList->Keys)
		{
			OutDataDesc->AddAttributeToAllData(CreatedKey, InBinding);
		}
	}

	// Try to propagate string keys across node. Not trivial because there could be one or more string key attributes on input pins and on output pins,
	// and it is in general hard to determine from source which string keys from input are being written to outputs. Try first collecting all string keys
	// from matching attribute names (across all input pins), and then fall back to collecting keys from all string key attributes across all inputs.
	bool bOutputHasStringKeys = false;

	for (const FPCGDataDesc& DataDesc : OutDataDesc->GetDataDescriptions())
	{
		auto HasStringKey = [](const FPCGKernelAttributeDesc& InAttributeDesc)
		{
			return InAttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::StringKey || InAttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::Name;
		};

		if (DataDesc.GetAttributeDescriptions().FindByPredicate(HasStringKey))
		{
			bOutputHasStringKeys = true;
			break;
		}
	}

	if (bOutputHasStringKeys)
	{
		TArray<const TSharedPtr<const FPCGDataCollectionDesc>> RelevantInputDataDescs;

		// Collect descriptions of input data items that have string key attributes.
		for(const FPCGPinProperties& PinProps : CustomHLSLSettings->InputPins)
		{
			const FPCGKernelPin InputKernelPin(GetKernelIndex(), PinProps.Label, /*bIsInput=*/true);
			const TSharedPtr<const FPCGDataCollectionDesc> InputPinDesc = InBinding->ComputeKernelPinDataDesc(InputKernelPin);

			if (!ensure(InputPinDesc))
			{
				continue;
			}

			bool bFoundStringKeyAttribute = false;

			for (FPCGDataDesc& DataDesc : OutDataDesc->GetDataDescriptionsMutable())
			{
				for (FPCGKernelAttributeDesc& AttributeDesc : DataDesc.GetAttributeDescriptionsMutable())
				{
					bFoundStringKeyAttribute |= (AttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::StringKey || AttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::Name);
				}
			}

			if (bFoundStringKeyAttribute)
			{
				RelevantInputDataDescs.Add(InputPinDesc);
			}
		}

		if (!RelevantInputDataDescs.IsEmpty())
		{
			for (FPCGDataDesc& DataDesc : OutDataDesc->GetDataDescriptionsMutable())
			{
				for (FPCGKernelAttributeDesc& AttributeDesc : DataDesc.GetAttributeDescriptionsMutable())
				{
					if (AttributeDesc.GetAttributeKey().GetType() != EPCGKernelAttributeType::StringKey && AttributeDesc.GetAttributeKey().GetType() != EPCGKernelAttributeType::Name)
					{
						continue;
					}

					bool bFoundMatchingAttribute = false;

					for (const TSharedPtr<const FPCGDataCollectionDesc>& InputPinDataDesc : RelevantInputDataDescs)
					{
						// Try to find string keys for matching attributes on inputs. E.g. if we are processing an output attribute named 'MeshPath',
						// look at data on all input pins for an attribute named MeshPath and assume we could use any of its values - copy the string keys.
						for (const FPCGDataDesc& InputDataDesc : InputPinDataDesc->GetDataDescriptions())
						{
							for (const FPCGKernelAttributeDesc& InputAttributeDesc : InputDataDesc.GetAttributeDescriptions())
							{
								const bool bIsString = InputAttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::StringKey || InputAttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::Name;
								if (bIsString && InputAttributeDesc.GetAttributeKey().GetIdentifier() == AttributeDesc.GetAttributeKey().GetIdentifier())
								{
									AttributeDesc.AddUniqueStringKeys(InputAttributeDesc.GetUniqueStringKeys());
									bFoundMatchingAttribute = true;
									break;
								}
							}
						}
					}

					if (!bFoundMatchingAttribute)
					{
						// We didn't find an exact attribute. Fall back to finding any and all string keys. This is concerning and perhaps we can
						// have additional hinting mechanisms in the kernel source or in the node UI.
						for (const TSharedPtr<const FPCGDataCollectionDesc>& InputPinDataDesc : RelevantInputDataDescs)
						{
							// Try to find string keys for matching attributes on inputs. E.g. if we are processing an output attribute named 'MeshPath',
							// look at data on all input pins for an attribute named MeshPath and assume we could use any of its values - copy the string keys.
							for (const FPCGDataDesc& InputDataDesc : InputPinDataDesc->GetDataDescriptions())
							{
								for (const FPCGKernelAttributeDesc& InputAttributeDesc : InputDataDesc.GetAttributeDescriptions())
								{
									if (InputAttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::StringKey || InputAttributeDesc.GetAttributeKey().GetType() == EPCGKernelAttributeType::Name)
									{
										AttributeDesc.AddUniqueStringKeys(InputAttributeDesc.GetUniqueStringKeys());
									}
								}
							}
						}
					}
				}
			}
		}
		else
		{
			// If there were no string keys found on any input pin then we are in a bad place. String values cannot be built on the GPU, they must
			// come in through an input.
			UE_LOGF(LogPCG, Warning, "No incoming attributes to obtain string keys from.");
		}
	}

	// Apply constant value range (CVR) compression.
	{
		int32 FullyAllocatedProps = 0;
		int32 CVRProperties = 0;

		// Properties with non-constant or multiple setter calls are always N-slot: each element may receive a different value.
		if (const int32* AllocatedProperties = PinToPropertiesWithSetters.Find(OutputPinProperties->Label))
		{
			FullyAllocatedProps |= *AllocatedProperties;
		}

		// Properties with a single unconditional constant-expression setter without auto-init: CVR (1 slot), since the constant value is the same for every element.
		// It's possible the set is done conditionally and doesn't apply to all elements, but if we're not initializing the output, there's no point allocating the other unwritten elements.
		// N-slot when auto-init, because the per-element auto-init pass means the output buffer must support N writes even if the setter is constant.
		if (const int32* ConstantOnlyProps = PinToPropertiesWithSingleConstantSetters.Find(OutputPinProperties->Label))
		{
			if (!OutputPinProperties->PropertiesGPU.bAutoInitializeOutput)
			{
				CVRProperties |= *ConstantOnlyProps;
			}
			else
			{
				FullyAllocatedProps |= *ConstantOnlyProps;
			}
		}

		// In a Processor without auto-init, properties that are never written by the kernel are the same for all elements: use CVR (1 slot).
		// Not applicable when auto-init is on (outer condition), since then all elements are written. Density excluded (sentinel-encoded).
		if (CustomHLSLSettings->IsProcessorKernel() && !OutputPinProperties->PropertiesGPU.bAutoInitializeOutput)
		{
			CVRProperties |= ~(PinToPropertiesWithSetters.FindRef(OutputPinProperties->Label) | PinToPropertiesWithSingleConstantSetters.FindRef(OutputPinProperties->Label));
		}

		if (FullyAllocatedProps != 0)
		{
			OutDataDesc->AllocatePropertiesForAllData(static_cast<EPCGPointNativeProperties>(FullyAllocatedProps));
		}
		if (CVRProperties != 0)
		{
			OutDataDesc->DeallocatePropertiesForAllData(static_cast<EPCGPointNativeProperties>(CVRProperties));
		}
	}

	return OutDataDesc;
}

#if WITH_EDITOR
FString UPCGCustomHLSLKernel::GetCookedSource(FPCGGPUCompilationContext& InOutContext) const
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());
	FString ShaderPathName = CustomHLSLSettings->GetPathName();
	PCGComputeHelpers::ConvertObjectPathToShaderFilePath(ShaderPathName);

	const FString Source = ProcessShaderSource(InOutContext, ParsedSources[0]);
	const FString AdditionalSources = ProcessAdditionalShaderSources(InOutContext);
	//const bool bHasKernelKeyword = Source.Contains(TEXT("KERNEL"), ESearchCase::CaseSensitive);

	const FIntVector GroupSize(PCGComputeConstants::THREAD_GROUP_SIZE, 1, 1);
	const FString KernelFunc = FString::Printf(
		TEXT("[numthreads(%d, %d, %d)]\nvoid %s(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)"),
		GroupSize.X, GroupSize.Y, GroupSize.Z, *GetEntryPoint());

	const FString UnWrappedDispatchThreadId = FString::Printf(
		TEXT("GetUnWrappedDispatchThreadId(GroupId, GroupIndex, %d)"),
		GroupSize.X * GroupSize.Y * GroupSize.Z
	);

	// Per-kernel-type preamble. Set up shader inputs and initialize output data.
	FString KernelSpecificPreamble = TEXT("    // Kernel preamble\n");

	auto AddThreadInfoForPin = [&KernelSpecificPreamble](const FPCGPinProperties* InPin)
	{
		check(InPin);

		KernelSpecificPreamble += FString::Format(TEXT(
			"    uint {0}_DataIndex;\n"
			"    if (!{0}_GetThreadData(ThreadIndex, {0}_DataIndex, ElementIndex)) return;\n"),
			{ InPin->Label.ToString() });
	};

	// Expand code below and update this ensure when functionality comes online.
	ensure(CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::One
		|| CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::Two
		|| CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::Three);

	if (CustomHLSLSettings->IsProcessorKernel())
	{
		const FPCGPinProperties* InputPin = GetFirstInputPin();
		const FPCGPinPropertiesGPU* OutputPin = GetFirstOutputPin();

		if (InputPin && OutputPin)
		{
			if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::One)
			{
				KernelSpecificPreamble += TEXT("    uint ElementIndex; // Assumption - element index identical in input and output data for processor.\n");
			}
			else if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::Two)
			{
				KernelSpecificPreamble += TEXT("    uint2 ElementIndex; // Assumption - element index identical in input and output data.\n");
			}
			else if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::Three)
			{
				KernelSpecificPreamble += TEXT("    uint3 ElementIndex; // Assumption - element index identical in input and output data.\n");
			}

			AddThreadInfoForPin(InputPin);
			AddThreadInfoForPin(OutputPin);

			if (CustomHLSLSettings->IsPointKernel())
			{
				KernelSpecificPreamble += FString::Format(TEXT(
					"    if ({0}_IsPointRemoved({0}_DataIndex, ElementIndex))\n"
					"    {\n"
					"        {1}_RemovePoint({1}_DataIndex, ElementIndex);\n"
					"        return;\n"
					"    }\n"),
					{ InputPin->Label.ToString(), OutputPin->Label.ToString() });

				if (OutputPin->PropertiesGPU.bAutoInitializeOutput)
				{
					PCGCustomHLSLKernel::InitializeOutputDataPoint(InputPin->Label, OutputPin->Label, PinToPropertiesWithSetters.FindRef(OutputPin->Label), KernelSpecificPreamble);
				}
			}
			else if (CustomHLSLSettings->IsAttributeSetKernel() && OutputPin->PropertiesGPU.bAutoInitializeOutput)
			{
				PCGCustomHLSLKernel::InitializeOutputDataAttributeSet(InputPin->Label, OutputPin->Label, KernelSpecificPreamble);
			}
			else if (CustomHLSLSettings->IsTextureKernel() && OutputPin->PropertiesGPU.bAutoInitializeOutput)
			{
				PCGCustomHLSLKernel::InitializeOutputDataTexture(InputPin->Label, OutputPin->Label, KernelSpecificPreamble);
			}
		}
	}
	else if (CustomHLSLSettings->IsGeneratorKernel())
	{
		if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::One)
		{
			KernelSpecificPreamble += TEXT("    const uint NumElements = NumElements_GetOverridableValue();\n");
		}
		else if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::Two)
		{
			KernelSpecificPreamble += TEXT("    const uint2 NumElements = uint2(NumElementsX_GetOverridableValue(), NumElementsY_GetOverridableValue());\n");
		}
		else if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::Three)
		{
			KernelSpecificPreamble += TEXT("    const uint3 NumElements = uint3(NumElementsX_GetOverridableValue(), NumElementsY_GetOverridableValue(), NumElementsZ_GetOverridableValue());\n");
		}

		if (CustomHLSLSettings->IsPointKernel())
		{
			KernelSpecificPreamble += TEXT(
				"    // NumPoints is deprecated.\n"
				"    const uint NumPoints = NumElements;\n");
		}

		if (const FPCGPinPropertiesGPU* OutputPin = GetFirstOutputPin())
		{
			if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::One)
			{
				KernelSpecificPreamble += TEXT("    uint ElementIndex; // Assumption - element index identical in input and output data.\n");
			}
			else if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::Two)
			{
				KernelSpecificPreamble += TEXT("    uint2 ElementIndex; // Assumption - element index identical in input and output data.\n");
			}
			else if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::Three)
			{
				KernelSpecificPreamble += TEXT("    uint3 ElementIndex; // Assumption - element index identical in input and output data.\n");
			}

			AddThreadInfoForPin(OutputPin);

			if (CustomHLSLSettings->IsPointKernel() && OutputPin->PropertiesGPU.bAutoInitializeOutput)
			{
				// Include constant-only properties in AllocProps so they are treated as fully allocated in the preamble.
				// With bAutoInitializeOutput=true a constant-expression setter still forces N-slot (not CVR), so its buffer has
				// N slots. Other TRS components sharing the same allocation bit (e.g. Rotation and Scale when Position has a
				// constant setter) must also be written per-element, not just at element 0.
				const int32 AllocProps = PinToPropertiesWithSetters.FindRef(OutputPin->Label) | PinToPropertiesWithSingleConstantSetters.FindRef(OutputPin->Label);
				PCGCustomHLSLKernel::AddPointGeneratorPreamble(OutputPin->Label, AllocProps, KernelSpecificPreamble);
			}
			else if (CustomHLSLSettings->IsTextureKernel() && OutputPin->PropertiesGPU.bAutoInitializeOutput)
			{
				PCGCustomHLSLKernel::AddTextureGeneratorPreamble(OutputPin->Label, KernelSpecificPreamble);
			}

			if (OutputPin->PropertiesGPU.bAutoInitializeOutput)
			{
				const FPCGKernelAttributeKeyList* AttrKeys = PinCreatedAttributeKeys.Find(OutputPin->Label);
				if (AttrKeys && !AttrKeys->Keys.IsEmpty())
				{
					PCGCustomHLSLKernel::AddMetadataInitPreamble(OutputPin->Label, AttrKeys, KernelSpecificPreamble);
				}
			}
		}
	}

	FString Result;

	// Note, it would be preferable to have the AdditionalSources included via the kernel CreateAdditionalSources(), but when the HLSL is composed, those additional sources are
	// placed above the data interfaces, so any additional sources would be unable to utilize functions provided by the data interfaces. Therefore we just inject them by hand here.

	// TODO: Support KERNEL keyword in shader source. Could be handy for external source assets and breaking kernels into sections to
	// support pin/attribute declarations, etc.
	/*if (bHasKernelKeyword)
	{
		Source.ReplaceInline(TEXT("KERNEL"), TEXT("void __kernel_func(uint ThreadIndex)"), ESearchCase::CaseSensitive);

		Result = FString::Printf(TEXT(
			"#line 0 \"%s\"\n" // ShaderPathName
			"%s\n" // AdditionalSources
			"%s\n" // Source
			"%s { __kernel_func(%s); }\n"), // KernelFunc, UnWrappedDispatchThreadId
			*ShaderPathName, *AdditionalSources, *Source, *KernelFunc, *UnWrappedDispatchThreadId);
	}
	else*/
	{
		Result = FString::Printf(TEXT(
			"%s\n\n" // AdditionalSources
			"%s\n" // KernelFunc
			"{\n"
			"	const uint ThreadIndex = %s;\n" // UnWrappedDispatchThreadId
			"	if (ThreadIndex >= GetNumThreads().x) return;\n"
			"%s\n" // KernelSpecificPreamble
			"#line 0 \"%s\"\n" // ShaderPathName
			"%s\n" // Source
			"}\n"),
			*AdditionalSources, *KernelFunc, *UnWrappedDispatchThreadId, *KernelSpecificPreamble, *ShaderPathName, *Source);
	}

	return Result;
}

void UPCGCustomHLSLKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGCustomHLSLDataInterface> NodeDI = InOutContext.NewObject_AnyThread<UPCGCustomHLSLDataInterface>(InObjectOuter);
	NodeDI->SetProducerKernel(this);

	OutDataInterfaces.Add(NodeDI);
}
#endif // WITH_EDITOR

int UPCGCustomHLSLKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());
	int ThreadCount = 0;

	const FPCGKernelParams* KernelParams = InBinding->GetCachedKernelParams(this);

	if (!ensure(KernelParams))
	{
		return ThreadCount;
	}

	if (CustomHLSLSettings->IsGeneratorKernel())
	{
		// Generators have fixed thread count.
		if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::One)
		{
			ThreadCount = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElements));
		}
		else if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::Two)
		{
			const int NumElementsX = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsX));
			const int NumElementsY = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsY));

			// Texture generator has fixed thread count.
			ThreadCount = NumElementsX * NumElementsY;
		}
		else if (CustomHLSLSettings->GetElementDimension() == EPCGElementDimension::Three)
		{
			const int NumElementsX = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsX));
			const int NumElementsY = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsY));
			const int NumElementsZ = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsZ));

			// Texture array generator has fixed thread count.
			ThreadCount = NumElementsX * NumElementsY * NumElementsZ;
		}
		else
		{
			// Needs implementation.
			checkNoEntry();
		}
	}
	else if (CustomHLSLSettings->IsProcessorKernel())
	{
		// Processing volume depends on data arriving on primary pin.
		if (const FPCGPinProperties* InputPin = GetFirstInputPin())
		{
			ThreadCount = GetElementCountForInputPin(*InputPin, InBinding);
		}
	}
	else if (CustomHLSLSettings->KernelType == EPCGKernelType::Custom)
	{
		if (CustomHLSLSettings->DispatchThreadCount == EPCGDispatchThreadCount::FromFirstOutputPin)
		{
			if (const FPCGPinPropertiesGPU* OutputPin = CustomHLSLSettings->OutputPins.IsEmpty() ? nullptr : &CustomHLSLSettings->OutputPins[0])
			{
				const TSharedPtr<const FPCGDataCollectionDesc> Desc = InBinding->GetCachedKernelPinDataDesc(this, OutputPin->Label, /*bIsInput=*/false);
				ThreadCount = Desc ? Desc->ComputeTotalElementCount() : 0;
			}
		}
		else if (CustomHLSLSettings->DispatchThreadCount == EPCGDispatchThreadCount::FromProductOfInputPins)
		{
			for (const FName& PinLabel : CustomHLSLSettings->ThreadCountInputPinLabels)
			{
				if (const FPCGPinProperties* InputPin = CustomHLSLSettings->InputPins.FindByPredicate([PinLabel](const FPCGPinProperties& InProps) { return InProps.Label == PinLabel; }))
				{
					ThreadCount = FMath::Max(ThreadCount, 1) * GetElementCountForInputPin(*InputPin, InBinding);
				}
			}
		}
		else if (CustomHLSLSettings->DispatchThreadCount == EPCGDispatchThreadCount::Fixed)
		{
			ThreadCount = KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, FixedThreadCount));
		}
	}
	else
	{
		checkNoEntry();
	}

	if (IsThreadCountMultiplierInUse())
	{
		ThreadCount *= KernelParams->GetValueInt(GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, ThreadCountMultiplier));
	}

	return ThreadCount;
}

void UPCGCustomHLSLKernel::GetDataLabels(FName InPinLabel, TArray<FString>& OutDataLabels) const
{
	if (const FPCGDataLabels* DataLabels = PinDataLabels.PinToDataLabels.Find(InPinLabel))
	{
		OutDataLabels = DataLabels->Labels;
	}
}

void UPCGCustomHLSLKernel::GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const
{
	OutKeys.Append(KernelAttributeKeys);
}

uint32 UPCGCustomHLSLKernel::GetThreadCountMultiplier() const
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());
	return CustomHLSLSettings->ThreadCountMultiplier;
}

uint32 UPCGCustomHLSLKernel::GetElementCountMultiplier(FName InOutputPinLabel) const
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());

	if (const FPCGPinPropertiesGPU* PinProps = CustomHLSLSettings->OutputPins.FindByPredicate([InOutputPinLabel](const FPCGPinPropertiesGPU& InProps) { return InProps.Label == InOutputPinLabel; }))
	{
		return PinProps->GetElementCountMultiplier();
	}
	else
	{
		ensure(false);
		return 1u;
	}
}

void UPCGCustomHLSLKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());
	OutPins = CustomHLSLSettings->InputPins;
}

void UPCGCustomHLSLKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());
	OutPins = CustomHLSLSettings->OutputPins;
}

#if WITH_EDITOR
void UPCGCustomHLSLKernel::InitEntryPoint()
{
	const UPCGSettings* Settings = GetSettings();
	check(Settings);

	// Record the node title - used to name the kernel and shows up in profiling.
	const UPCGNode* Node = GetNode();
	if (Node && Node->HasAuthoredTitle())
	{
		EntryPoint = Node->GetAuthoredTitleName().ToString();
	}
	else
	{
		EntryPoint = Settings->GetDefaultNodeTitle().ToString();
	}

	// We append the PCG kernel name here because of a weird issue on Mac where if the last word of the name of a
	// shader parameter matches the kernel name, the value for the parameter is always zero for unknown reasons
	// (e.g. parameter named "XXXWeight" & kernel also named "Weight"). Adding this bit of "random" string after
	// the user provided kernel name should greatly reduce the chance of things like that happening. We don't include
	// the full fname with number as this will change across executions and cause DDC misses.
	EntryPoint += TEXT("_") + GetFName().GetPlainNameString();

	// Sanitize to a valid HLSL identifier [a-zA-Z_][a-zA-Z0-9_]*.
	// Explicit ASCII allowlist rather than a blocklist: non-ASCII characters (e.g. from a localized
	// display name or a non-Latin authored node title) are replaced with '_' instead of reaching the
	// shader compiler and causing a build failure.
	for (TCHAR& Char : EntryPoint)
	{
		const bool bValidIdentifierChar =
			(Char >= TCHAR('a') && Char <= TCHAR('z')) ||
			(Char >= TCHAR('A') && Char <= TCHAR('Z')) ||
			(Char >= TCHAR('0') && Char <= TCHAR('9')) ||
			(Char == TCHAR('_'));
		if (!bValidIdentifierChar)
		{
			Char = TCHAR('_');
		}
	}

	// HLSL identifiers cannot start with a digit.
	if (!EntryPoint.IsEmpty() && EntryPoint[0] >= TCHAR('0') && EntryPoint[0] <= TCHAR('9'))
	{
		EntryPoint.InsertAt(0, TCHAR('_'));
	}
}

void UPCGCustomHLSLKernel::PopulateAttributeKeys()
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());

	PinCreatedAttributeKeys.Reset();

	// Pass 1: seed from manually-authored CreatedKernelAttributeKeys on each output pin, but only when the user
	// has explicitly enabled per-pin attribute creation settings. When disabled, only HLSL inference is used.
	if (CustomHLSLSettings->bPerPinAttributeCreationSettings)
	{
		for (const FPCGPinPropertiesGPU& OutputPin : CustomHLSLSettings->OutputPins)
		{
			for (const FPCGKernelAttributeKey& AuthoredKey : OutputPin.PropertiesGPU.CreatedKernelAttributeKeys)
			{
				if (AuthoredKey.IsValid())
				{
					KernelAttributeKeys.AddUnique(AuthoredKey);
					PinCreatedAttributeKeys.FindOrAdd(OutputPin.Label).Keys.AddUnique(AuthoredKey);
				}
			}
		}
	}

	// Pass 2: augment with attributes inferred from HLSL Set/AtomicAdd calls in ParsedSources.
	// ParseShaderSource() already added these to KernelAttributeKeys, but we also need them in PinCreatedAttributeKeys.
	for (const FPCGCustomHLSLParsedSource& ParsedSource : ParsedSources)
	{
		for (const FPCGParsedAttributeFunction& AttrFunc : ParsedSource.AttributeFunctions)
		{
			const bool bIsSet = (AttrFunc.FunctionName == PCGCustomHLSLKernel::AttributeFunctionSetKeyword);
			const bool bIsAtomicAdd = (AttrFunc.FunctionName == PCGCustomHLSLKernel::AttributeFunctionAtomicAddKeyword);
			if (!bIsSet && !bIsAtomicAdd)
			{
				continue;
			}

			const FPCGKernelAttributeKey InferredKey(FPCGAttributePropertySelector::CreateSelectorFromString(AttrFunc.AttributeName), static_cast<EPCGKernelAttributeType>(AttrFunc.AttributeType));
			if (!InferredKey.IsValid())
			{
				continue;
			}

			const FName PinLabel(*AttrFunc.PinLabel);
			TArray<FPCGKernelAttributeKey>& PinKeys = PinCreatedAttributeKeys.FindOrAdd(PinLabel).Keys;
			if (PinKeys.Contains(InferredKey))
			{
				continue;
			}

			PinKeys.Add(InferredKey);
		}
	}
}

void UPCGCustomHLSLKernel::ParseShaderSource()
{
	KernelAttributeKeys.Reset();
	PinToPropertiesWithSetters.Empty();
	PinToPropertiesWithSingleConstantSetters.Empty();
	PinWrittenTransformSetters.Empty();

	CreateParsedSources();

	static const TArray<FString> AttributeTypeStrings =
	{
		TEXT("Bool"),
		TEXT("Int"),
		TEXT("Uint"),
		TEXT("Float"),
		TEXT("Float2"),
		TEXT("Float3"),
		TEXT("Float4"),
		TEXT("Rotator"),
		TEXT("Quat"),
		TEXT("Transform"),
		TEXT("StringKey"),
		TEXT("Name"),
	};

	// Collect additional keywords, such as function getters and setters.
	TArray<FString> AdditionalKeywords;
	TArray<FString> AttributeKeywords;
	TArray<FString> CopyElementKeywords;
	TArray<FString> InitializeKeywords;
	TArray<FString> PointPropertySetterKeywords;

	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());

	for (const FPCGPinProperties& PinProps : CustomHLSLSettings->InputPins)
	{
		if (!!(PinProps.AllowedTypes & (EPCGDataType::Point | EPCGDataType::Param)))
		{
			for (const FString& AttributeTypeString : AttributeTypeStrings)
			{
				AttributeKeywords.Add(PinProps.Label.ToString() + "_" + PCGCustomHLSLKernel::AttributeFunctionGetKeyword + AttributeTypeString);
			}
		}
	}

	for (const FPCGPinProperties& PinProps : CustomHLSLSettings->OutputPins)
	{
		const FString PinStr = PinProps.Label.ToString();

		if (!!(PinProps.AllowedTypes & (EPCGDataType::Point | EPCGDataType::Param)))
		{
			for (const FString& AttributeTypeString : AttributeTypeStrings)
			{
				AttributeKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::AttributeFunctionSetKeyword + AttributeTypeString);
			}

			AttributeKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::AttributeFunctionAtomicAddKeyword + PCGCustomHLSLKernel::AttributeFunctionAtomicAddType);

			for (const FPCGPinProperties& InputPinProps : CustomHLSLSettings->InputPins)
			{
				if (InputPinProps.AllowedTypes == PinProps.AllowedTypes)
				{
					CopyElementKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::CopyElementFunctionKeyword + TEXT("_") + InputPinProps.Label.ToString());
				}
			}

			InitializeKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::InitializeFunctionKeyword);
		}
		else if (!!(PinProps.AllowedTypes & EPCGDataType::BaseTexture) || PinProps.AllowedTypes.IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId()))
		{
			InitializeKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::StoreFunctionKeyword);
		}
		else if (PinProps.AllowedTypes.IsChildOf(FPCGDataTypeInfoRawBuffer::AsId()))
		{
			InitializeKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::StoreFunctionKeyword);
			InitializeKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::Store4FunctionKeyword);
		}

		if (!!(PinProps.AllowedTypes & EPCGDataType::Point))
		{
			PointPropertySetterKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::SetPositionKeyword);
			PointPropertySetterKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::SetRotationKeyword);
			PointPropertySetterKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::SetScaleKeyword);
			PointPropertySetterKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::SetBoundsMinKeyword);
			PointPropertySetterKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::SetBoundsMaxKeyword);
			PointPropertySetterKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::SetColorKeyword);
			PointPropertySetterKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::SetDensityKeyword);
			PointPropertySetterKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::SetSteepnessKeyword);
			PointPropertySetterKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::SetSeedKeyword);
			PointPropertySetterKeywords.Add(PinStr + TEXT("_") + PCGCustomHLSLKernel::SetPointTransformKeyword);
		}
	}

	AdditionalKeywords.Append(AttributeKeywords);
	AdditionalKeywords.Append(CopyElementKeywords);
	AdditionalKeywords.Append(InitializeKeywords);
	AdditionalKeywords.Append(PointPropertySetterKeywords);

	FPCGSyntaxTokenizerParams TokenizerParams;
	TokenizerParams.AdditionalKeywords = MoveTemp(AdditionalKeywords);

	TSharedPtr<ISyntaxTokenizer> Tokenizer = MakeShared<FPCGHLSLSyntaxTokenizer>(TokenizerParams);
	check(Tokenizer.IsValid());

	auto ParseHelper = [&](FPCGCustomHLSLParsedSource& InOutParsedSource)
	{
		TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines;
		Tokenizer->Process(TokenizedLines, InOutParsedSource.Source);
		PCGCustomHLSLKernel::ParseTokens(InOutParsedSource.Source, TokenizedLines, InOutParsedSource.Tokens);

		// Information about an attribute function looking for a match.
		struct FAttributeFunctionMatch
		{
			FString PinStr;
			FString FuncStr;
			FString TypeStr;
			FString NameStr;
			int32 MatchBeginning = INDEX_NONE;
			int32 EncounteredCommas = 0;
			const int32 RequiredCommas = 2;

			void Reset()
			{
				PinStr = "";
				FuncStr = "";
				TypeStr = "";
				NameStr = "";
				MatchBeginning = INDEX_NONE;
				EncounteredCommas = 0;
			}
		};

		FAttributeFunctionMatch AttributeFunctionMatch;
		bool bLookingForAttributeFunctionMatch = false;

		FString SingleQuoteString = "";
		bool bLookingForSingleQuoteMatch = false;

		struct FPropertySetterState
		{
			FString PinStr;
			EPCGPointNativeProperties PropertyFlag = EPCGPointNativeProperties::None;
			int32 TRSBit = 0; // Fine-grained TRS component bit (TRSPositionBit etc.), or 0 for non-TRS properties.
			int32 ParenDepth = 0;
			int32 ArgCommaCount = 0;
			bool bCollectingValueArg = false;
			int32 ValueArgStartTokenIndex = INDEX_NONE;
		};
		FPropertySetterState PropertySetterState;
		bool bLookingForPropertySetterValueArg = false;

		// Tracks which fine-grained TRS sub-components (TRSPositionBit etc.) have been written with constant-only expressions per pin. Used to distinguish "Position then Rotation" (different sub-setters, both CVR-eligible)
		// from "Position then Position" (same sub-setter written twice, must demote to non-CVR).
		TMap<FName, int32> PinConstantTRSSetters;

		auto AddCompletedAttributeFunction = [&KernelAttributeKeys=KernelAttributeKeys, &AttributeFunctions=InOutParsedSource.AttributeFunctions](const FAttributeFunctionMatch& InAttributeFunction)
		{
			if (InAttributeFunction.EncounteredCommas != InAttributeFunction.RequiredCommas)
			{
				return;
			}

			// @todo_pcg: Validate NameStr in [a-zA-Z0-9 -_\/] ?
			const FString& PinStr = InAttributeFunction.PinStr;
			const FString& FuncStr = InAttributeFunction.FuncStr;
			const FString& TypeStr = InAttributeFunction.TypeStr;
			const FString& NameStr = InAttributeFunction.NameStr;
			const FString UsageString = PinStr + TEXT("_") + FuncStr + TypeStr;

			if (PinStr.IsEmpty() || FuncStr.IsEmpty() || TypeStr.IsEmpty() || NameStr.IsEmpty())
			{
				UE_LOGF(LogPCG, Error, "Invalid attribute usage in shader source: '%ls' on attribute name '%ls'.", *UsageString, *NameStr);
				return;
			}

			const UEnum* AttributeTypeEnum = StaticEnum<EPCGKernelAttributeType>();
			check(AttributeTypeEnum);

			const int64 AttributeType = AttributeTypeEnum->GetValueByName(FName(*TypeStr));

			if (AttributeType == INDEX_NONE)
			{
				UE_LOGF(LogPCG, Error, "Invalid attribute type in shader source: '%ls' on attribute name '%ls'.", *UsageString, *NameStr);
				return;
			}

			// Add the attribute if it hasn't already been referenced.
			const FPCGKernelAttributeKey Key(FPCGAttributePropertySelector::CreateSelectorFromString(NameStr), static_cast<EPCGKernelAttributeType>(AttributeType));

			if (Key.IsValid())
			{
				KernelAttributeKeys.AddUnique(Key);
				AttributeFunctions.Emplace(PinStr, FuncStr, AttributeType, NameStr, InAttributeFunction.MatchBeginning);
			}
		};

		const int32 NumTokens = InOutParsedSource.Tokens.Num();

		for (int32 TokenIndex = 0; TokenIndex < NumTokens; ++TokenIndex)
		{
			const FPCGCustomHLSLParsedSource::FToken& Token = InOutParsedSource.Tokens[TokenIndex];
			const FString TokenString = InOutParsedSource.Source.Mid(Token.Range.BeginIndex, Token.Range.Len());

			// Track paren depth and collect the value argument for property setter calls.
			if (bLookingForPropertySetterValueArg)
			{
				if (Token.Type == FPCGCustomHLSLParsedSource::ETokenType::Operator)
				{
					if (TokenString == TEXT("("))
					{
						++PropertySetterState.ParenDepth;
					}
					else if (TokenString == TEXT(")"))
					{
						if (PropertySetterState.ParenDepth == 1)
						{
							// Closing paren of the setter call - evaluate the collected value tokens.
							if (PropertySetterState.bCollectingValueArg && PropertySetterState.ValueArgStartTokenIndex != INDEX_NONE)
							{
								const int32 ValueTokenCount = TokenIndex - PropertySetterState.ValueArgStartTokenIndex;
								TArrayView<const FPCGCustomHLSLParsedSource::FToken> ValueTokens(InOutParsedSource.Tokens.GetData() + PropertySetterState.ValueArgStartTokenIndex, ValueTokenCount);
								const bool bIsConstant = PCGCustomHLSLKernel::IsConstantValueExpression(InOutParsedSource.Source, ValueTokens);
								const FName PinName(PropertySetterState.PinStr);
								const int32 PropertyBit = static_cast<int32>(PropertySetterState.PropertyFlag);
								const int32 TRSBit = PropertySetterState.TRSBit;

								if (bIsConstant)
								{
									// Only track as constant if no non-constant setter has been seen for this property.
									if (!(PinToPropertiesWithSetters.FindRef(PinName) & PropertyBit))
									{
										// For TRS sub-setters (Position/Rotation/Scale), use a fine-grained bit to detect
										// duplicate writes. Position then Rotation both share Transform's PropertyBit, but
										// are different setters - both can remain CVR. Position then Position again must demote.
										// For non-TRS properties, the coarse PropertyBit is sufficient.
										const bool bAlreadySeen = (TRSBit != 0) ? (PinConstantTRSSetters.FindRef(PinName) & TRSBit) != 0 : (PinToPropertiesWithSingleConstantSetters.FindRef(PinName) & PropertyBit) != 0;

										if (bAlreadySeen)
										{
											// Second constant write to the same setter - values may differ per element.
											// Promote to non-constant so the buffer gets N slots.
											PinToPropertiesWithSingleConstantSetters.FindOrAdd(PinName) &= ~PropertyBit;
											PinToPropertiesWithSetters.FindOrAdd(PinName) |= PropertyBit;
										}
										else
										{
											PinToPropertiesWithSingleConstantSetters.FindOrAdd(PinName) |= PropertyBit;
											if (TRSBit != 0)
											{
												PinConstantTRSSetters.FindOrAdd(PinName) |= TRSBit;
											}
										}
									}
								}
								else
								{
									PinToPropertiesWithSetters.FindOrAdd(PinName) |= PropertyBit;
									// Non-constant (or conditional-constant) setter wins - remove from constant-only if previously marked.
									if (int32* ConstantProps = PinToPropertiesWithSingleConstantSetters.Find(PinName))
									{
										*ConstantProps &= ~PropertyBit;
									}
									if (TRSBit != 0)
									{
										if (int32* TRSSetters = PinConstantTRSSetters.Find(PinName))
										{
											*TRSSetters &= ~TRSBit;
										}
									}
								}
							}

							bLookingForPropertySetterValueArg = false;
						}
						else
						{
							--PropertySetterState.ParenDepth;
						}
					}
				}
				else if (Token.Type == FPCGCustomHLSLParsedSource::ETokenType::Normal && TokenString == TEXT(",") && PropertySetterState.ParenDepth == 1)
				{
					++PropertySetterState.ArgCommaCount;

					if (PropertySetterState.ArgCommaCount == 2)
					{
						// Third argument is the value - start collecting from the next token.
						PropertySetterState.bCollectingValueArg = true;
						PropertySetterState.ValueArgStartTokenIndex = TokenIndex + 1;
					}
				}

				// Do NOT continue here - attribute function matching (keyword detection, comma counting, single-quote parsing) must also run for tokens inside the setter's value argument.
				// Tokens inside the setter's value argument must reach the attribute-function path to handle nested attribute getters like:
				//   Out_SetScale(..., CalculateScale(LayerData_GetFloat3(0u, 0u, 'Scale')));
				// The depth-1 comma guard on ArgCommaCount above prevents double-counting.
			}

			if (Token.Type == FPCGCustomHLSLParsedSource::ETokenType::Keyword)
			{
				if (AttributeKeywords.Contains(TokenString))
				{
					AttributeFunctionMatch.Reset();

					// Use the last underscore as a delimiter in case the pin name contains underscores.
					int32 DelimiterIndex = INDEX_NONE;
					TokenString.FindLastChar('_', DelimiterIndex);

					AttributeFunctionMatch.PinStr = TokenString.Left(DelimiterIndex);

					int32 TypeIndex = 0;
					for (; TypeIndex < AttributeTypeStrings.Num(); ++TypeIndex)
					{
						if (TokenString.EndsWith(AttributeTypeStrings[TypeIndex]))
						{
							break;
						}
					}

					if (TypeIndex == AttributeTypeStrings.Num())
					{
						UE_LOGF(LogPCG, Error, "Invalid attribute type in shader source: '%ls'.", *TokenString);
						continue;
					}

					AttributeFunctionMatch.TypeStr = AttributeTypeStrings[TypeIndex];
					AttributeFunctionMatch.FuncStr = TokenString.Mid(DelimiterIndex + 1, Token.Range.Len() - AttributeFunctionMatch.TypeStr.Len() - DelimiterIndex - 1);
					AttributeFunctionMatch.MatchBeginning = Token.Range.BeginIndex;
					bLookingForAttributeFunctionMatch = true;

					if (AttributeFunctionMatch.FuncStr == PCGCustomHLSLKernel::AttributeFunctionSetKeyword || AttributeFunctionMatch.FuncStr == PCGCustomHLSLKernel::AttributeFunctionAtomicAddKeyword)
					{
						InOutParsedSource.InitializedOutputPins.Add(AttributeFunctionMatch.PinStr);
					}
				}
				else if (CopyElementKeywords.Contains(TokenString))
				{
					// Format: TargetPin_CopyElementFrom_SourcePin. Find the known middle keyword to split.
					const FString CopyKeyword = FString(TEXT("_")) + PCGCustomHLSLKernel::CopyElementFunctionKeyword + TEXT("_");
					const int32 CopyKeywordIndex = TokenString.Find(CopyKeyword);
					FString TargetPin = TokenString.Left(CopyKeywordIndex);
					FString SourcePin = TokenString.RightChop(CopyKeywordIndex + CopyKeyword.Len());

					InOutParsedSource.InitializedOutputPins.Add(TargetPin);
					InOutParsedSource.CopyElementFunctions.Emplace(MoveTemp(SourcePin), MoveTemp(TargetPin));
				}
				else if (InitializeKeywords.Contains(TokenString))
				{
					// Use the last underscore as a delimiter in case the pin name contains underscores.
					int32 DelimiterIndex = INDEX_NONE;
					TokenString.FindLastChar(TEXT('_'), DelimiterIndex);
					FString PinStr = TokenString.Left(DelimiterIndex);

					InOutParsedSource.InitializedOutputPins.Add(MoveTemp(PinStr));
				}
				else if (PointPropertySetterKeywords.Contains(TokenString))
				{
					// Use the last underscore as a delimiter in case the pin name contains underscores.
					int32 DelimiterIndex = INDEX_NONE;
					TokenString.FindLastChar(TEXT('_'), DelimiterIndex);
					const FString PinStr = TokenString.Left(DelimiterIndex);
					const FString PropertyStr = TokenString.RightChop(DelimiterIndex + 1);

					// Map the property keyword to its allocation flag.
					EPCGPointNativeProperties PropertyFlag = EPCGPointNativeProperties::None;
					if (PropertyStr == PCGCustomHLSLKernel::SetPositionKeyword
						|| PropertyStr == PCGCustomHLSLKernel::SetRotationKeyword
						|| PropertyStr == PCGCustomHLSLKernel::SetScaleKeyword
						|| PropertyStr == PCGCustomHLSLKernel::SetPointTransformKeyword)
					{
						PropertyFlag = EPCGPointNativeProperties::Transform;

						// Record which specific TRS component was called for fine-grained smart init.
						// SetPointTransform marks all three. Separate from the coarse Transform bit used for allocation.
						int32 TRSBits = 0;
						if (PropertyStr == PCGCustomHLSLKernel::SetPositionKeyword)
						{
							TRSBits = PCGCustomHLSLKernel::TRSPositionBit;
						}
						else if (PropertyStr == PCGCustomHLSLKernel::SetRotationKeyword)
						{
							TRSBits = PCGCustomHLSLKernel::TRSRotationBit;
						}
						else if (PropertyStr == PCGCustomHLSLKernel::SetScaleKeyword)
						{
							TRSBits = PCGCustomHLSLKernel::TRSScaleBit;
						}
						else // SetPointTransform
						{
							TRSBits = PCGCustomHLSLKernel::TRSPositionBit | PCGCustomHLSLKernel::TRSRotationBit | PCGCustomHLSLKernel::TRSScaleBit;
						}

						PinWrittenTransformSetters.FindOrAdd(FName(PinStr)) |= TRSBits;
						PropertySetterState.TRSBit = TRSBits;
					}
					else if (PropertyStr == PCGCustomHLSLKernel::SetBoundsMinKeyword)
					{
						PropertyFlag = EPCGPointNativeProperties::BoundsMin;
					}
					else if (PropertyStr == PCGCustomHLSLKernel::SetBoundsMaxKeyword)
					{
						PropertyFlag = EPCGPointNativeProperties::BoundsMax;
					}
					else if (PropertyStr == PCGCustomHLSLKernel::SetColorKeyword)
					{
						PropertyFlag = EPCGPointNativeProperties::Color;
					}
					else if (PropertyStr == PCGCustomHLSLKernel::SetDensityKeyword)
					{
						PropertyFlag = EPCGPointNativeProperties::Density;
					}
					else if (PropertyStr == PCGCustomHLSLKernel::SetSteepnessKeyword)
					{
						PropertyFlag = EPCGPointNativeProperties::Steepness;
					}
					else if (PropertyStr == PCGCustomHLSLKernel::SetSeedKeyword)
					{
						PropertyFlag = EPCGPointNativeProperties::Seed;
					}
					else
					{
						ensure(false);
					}

					// Density always requires full allocation - it encodes the removed-point sentinel value. For all other properties, defer the allocation decision: only allocate if the value
					// argument is not a constant expression (detected by scanning tokens up to the closing paren).
					if (PropertyFlag == EPCGPointNativeProperties::Density)
					{
						PinToPropertiesWithSetters.FindOrAdd(FName(PinStr)) |= static_cast<int32>(PropertyFlag);
					}
					else if (PropertyFlag != EPCGPointNativeProperties::None)
					{
						PropertySetterState.PinStr = PinStr;
						PropertySetterState.PropertyFlag = PropertyFlag;
						// TRSBit is set above for TRS setters; reset to 0 here so non-TRS setters don't inherit a stale value.
						if (PropertyFlag != EPCGPointNativeProperties::Transform)
						{
							PropertySetterState.TRSBit = 0;
						}
						PropertySetterState.ParenDepth = 0;
						PropertySetterState.ArgCommaCount = 0;
						PropertySetterState.bCollectingValueArg = false;
						PropertySetterState.ValueArgStartTokenIndex = INDEX_NONE;
						bLookingForPropertySetterValueArg = true;
					}

					InOutParsedSource.InitializedOutputPins.Add(PinStr);
				}
			}
			else if (Token.Type == FPCGCustomHLSLParsedSource::ETokenType::Normal)
			{
				if (bLookingForSingleQuoteMatch && TokenString == "\'")
				{
					if (bLookingForAttributeFunctionMatch && AttributeFunctionMatch.EncounteredCommas == AttributeFunctionMatch.RequiredCommas)
					{
						AttributeFunctionMatch.NameStr = SingleQuoteString;
						AddCompletedAttributeFunction(AttributeFunctionMatch);
						bLookingForAttributeFunctionMatch = false;
					}

					SingleQuoteString = "";
					bLookingForSingleQuoteMatch = false;
				}
				else if (bLookingForAttributeFunctionMatch && TokenString == ",")
				{
					++AttributeFunctionMatch.EncounteredCommas;

					if (AttributeFunctionMatch.EncounteredCommas > AttributeFunctionMatch.RequiredCommas)
					{
						AttributeFunctionMatch.Reset();
						bLookingForAttributeFunctionMatch = false;
					}
				}
			}
			else if (Token.Type == FPCGCustomHLSLParsedSource::ETokenType::SingleQuotedString)
			{
				if (!bLookingForSingleQuoteMatch)
				{
					bLookingForSingleQuoteMatch = true;

					// Chop the leading single quote.
					SingleQuoteString = TokenString.RightChop(1);
				}
				else
				{
					SingleQuoteString += TokenString;
				}
			}
		}

		// @todo_pcg: Maybe this should also be parsed instead of regex'd, but it's not as trivial to create tokens to detect the data labels pattern.
		CollectDataLabels(InOutParsedSource);
	};

	for (FPCGCustomHLSLParsedSource& ParsedSource : ParsedSources)
	{
		ParseHelper(ParsedSource);
	}
}

void UPCGCustomHLSLKernel::CreateParsedSources()
{
	TArray<TObjectPtr<UComputeSource>> AdditionalSources;
	TSet<TObjectPtr<UComputeSource>> VisitedAdditionalSources;

	auto TraverseAdditionalSources = [&AdditionalSources, &VisitedAdditionalSources](TObjectPtr<UComputeSource> AdditionalSource, auto&& RecursiveCall)
	{
		if (!AdditionalSource || VisitedAdditionalSources.Contains(AdditionalSource))
		{
			return;
		}

		VisitedAdditionalSources.Add(AdditionalSource);

		// We do a postfix traversal of the nested additional sources because we need them to be pasted higher in the resulting HLSL, since presumably a source depends on its additional sources.
		for (TObjectPtr<UComputeSource> NestedSource : AdditionalSource->AdditionalSources)
		{
			RecursiveCall(NestedSource, RecursiveCall);
		}

		AdditionalSources.Add(AdditionalSource);
	};

	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());
	TArray<TObjectPtr<UComputeSource>> AdditionalSourcesToProcess = CustomHLSLSettings->AdditionalSources;

	if (TObjectPtr<UComputeSource> KernelSourceOverride = CustomHLSLSettings->KernelSourceOverride)
	{
		ParsedSources.Emplace(KernelSourceOverride->GetSource());
		VisitedAdditionalSources.Add(KernelSourceOverride);
		AdditionalSourcesToProcess.Append(KernelSourceOverride->AdditionalSources);
	}
	else
	{
		ParsedSources.Emplace(CustomHLSLSettings->ShaderSource);
		ParsedSources.Emplace(CustomHLSLSettings->ShaderFunctions);
	}
 
	for (TObjectPtr<UComputeSource> RootAdditionalSource : AdditionalSourcesToProcess)
	{
		TraverseAdditionalSources(RootAdditionalSource, TraverseAdditionalSources);
	}

	// Now that the additional sources are in post-fix order, we can begin to parse them.
	for (TObjectPtr<UComputeSource> AdditionalSource : AdditionalSources)
	{
		check(AdditionalSource);
		ParsedSources.Emplace(AdditionalSource->GetSource());
	}
}

void UPCGCustomHLSLKernel::CollectDataLabels(const FPCGCustomHLSLParsedSource& InParsedSource)
{
	auto CollectDataLabelsForPin = [&InParsedSource=InParsedSource, &PinDataLabels=PinDataLabels](FName InPinLabel)
	{
		FPCGDataLabels& DataLabels = PinDataLabels.PinToDataLabels.FindOrAdd(InPinLabel);

		// Matches against {PinName}_AnyFunction('{DataLabel}'...
		const FString Pattern = FString::Format(TEXT("{0}_.*?[\\s]*?\\([\\s]*?'([a-zA-Z0-9_].*?)'"), { InPinLabel.ToString() });

		// First capture: Data label (supports a - z, A - Z, 0 - 9, and underscores).
		constexpr int LabelCaptureGroup = 1;

		FRegexMatcher ModuleMatcher(FRegexPattern(Pattern), InParsedSource.Source);
		while (ModuleMatcher.FindNext())
		{
			DataLabels.Labels.AddUnique(ModuleMatcher.GetCaptureGroup(LabelCaptureGroup));
		}
	};

	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());

	for (const FPCGPinProperties& PinProps : CustomHLSLSettings->InputPins)
	{
		CollectDataLabelsForPin(PinProps.Label);
	}

	for (const FPCGPinProperties& PinProps : CustomHLSLSettings->OutputPins)
	{
		CollectDataLabelsForPin(PinProps.Label);
	}
}

bool UPCGCustomHLSLKernel::PerformStaticValidation()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCustomHLSLKernel::PerformStaticValidation);

	if (!Super::PerformStaticValidation())
	{
		return false;
	}

	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());

	if (CustomHLSLSettings->OutputPins.IsEmpty())
	{
#if PCG_KERNEL_LOGGING_ENABLED
		AddStaticLogEntry(LOCTEXT("NoOutputs", "Custom HLSL nodes must have at least one output."), EPCGKernelLogVerbosity::Error);
#endif
		return false;
	}

	auto CheckPinLabel = [this, Settings=CustomHLSLSettings](FName PinLabel)
	{
		if (PinLabel == NAME_None)
		{
#if PCG_KERNEL_LOGGING_ENABLED
			AddStaticLogEntry(LOCTEXT("InvalidPinLabelNone", "Pin label 'None' is not a valid pin label."), EPCGKernelLogVerbosity::Error);
#endif
			return false;
		}

		bool bFoundPinLabel = false;

		auto IsAlreadyFound = [this, Settings, PinLabel, &bFoundPinLabel](const FPCGPinProperties PinProps)
		{
			if (PinProps.Label == PinLabel)
			{
				if (bFoundPinLabel)
				{
#if PCG_KERNEL_LOGGING_ENABLED
					AddStaticLogEntry(FText::Format(LOCTEXT("DuplicatedPinLabels", "Duplicate pin label '{0}', all labels must be unique."), FText::FromName(PinLabel)), EPCGKernelLogVerbosity::Error);
#endif
					return true;
				}

				bFoundPinLabel = true;
			}

			return false;
		};

		for (const FPCGPinProperties& PinProps : Settings->InputPins)
		{
			if (IsAlreadyFound(PinProps))
			{
				return false;
			}
		}

		for (const FPCGPinProperties& PinProps : Settings->OutputPins)
		{
			if (IsAlreadyFound(PinProps))
			{
				return false;
			}
		}

		return true;
	};

	// Validate input pins
	bool bIsFirstInputPin = true;
	for (const FPCGPinProperties& Properties : CustomHLSLSettings->InputPins)
	{
		if (!CheckPinLabel(Properties.Label))
		{
			return false;
		}

		if (bIsFirstInputPin && CustomHLSLSettings->KernelType == EPCGKernelType::PointProcessor)
		{
			if (Properties.AllowedTypes != EPCGDataType::Point)
			{
#if PCG_KERNEL_LOGGING_ENABLED
				AddStaticLogEntry(FText::Format(
					LOCTEXT("InvalidNonPointPrimaryInput", "'Point Processor' nodes require primary input pin to be of type 'Point', but found '{0}'."),
					FText::FromString(PCGCustomHLSLKernel::GetDataTypeString(Properties.AllowedTypes))),
					EPCGKernelLogVerbosity::Error);
#endif
				return false;
			}
		}
		else if (bIsFirstInputPin && (CustomHLSLSettings->KernelType == EPCGKernelType::TextureProcessor || CustomHLSLSettings->KernelType == EPCGKernelType::TextureArrayProcessor))
		{
			if (!Properties.AllowedTypes.IsChildOf(FPCGDataTypeInfoTexture2DBase::AsId()))
			{
#if PCG_KERNEL_LOGGING_ENABLED
				AddStaticLogEntry(FText::Format(
					LOCTEXT("InvalidNonTexturePrimaryInput", "Texture processor nodes require primary input pin to be a 2D texture type, but found '{0}'."),
					FText::FromString(PCGCustomHLSLKernel::GetDataTypeString(Properties.AllowedTypes))),
					EPCGKernelLogVerbosity::Error);
#endif
				return false;
			}
		}
		else if (bIsFirstInputPin && CustomHLSLSettings->KernelType == EPCGKernelType::AttributeSetProcessor)
		{
			if (!(Properties.AllowedTypes & EPCGDataType::Param))
			{
#if PCG_KERNEL_LOGGING_ENABLED
				AddStaticLogEntry(FText::Format(
					LOCTEXT("InvalidNonAttributeSetPrimaryInput", "'Attribute Set Processor' nodes require primary input pin to be of type 'Attribute Set', but found '{0}'."),
					FText::FromString(PCGCustomHLSLKernel::GetDataTypeString(Properties.AllowedTypes))),
					EPCGKernelLogVerbosity::Error);
#endif
				return false;
			}
		}

		if (!PCGComputeHelpers::IsTypeAllowedAsInput(Properties.AllowedTypes))
		{
#if PCG_KERNEL_LOGGING_ENABLED
			AddStaticLogEntry(FText::Format(
				LOCTEXT("InvalidInputType", "Unsupported input type '{0}', found on pin '{1}'."),
				FText::FromString(PCGCustomHLSLKernel::GetDataTypeString(Properties.AllowedTypes)),
				FText::FromName(Properties.Label)),
				EPCGKernelLogVerbosity::Error);
#endif
			return false;
		}

		bIsFirstInputPin = false;
	}

	// Validate output pins
	bool bIsFirstOutputPin = true;
	for (const FPCGPinPropertiesGPU& Properties : CustomHLSLSettings->OutputPins)
	{
		if (!CheckPinLabel(Properties.Label))
		{
			return false;
		}

		const bool bPinIsDefinedByKernel = bIsFirstOutputPin && CustomHLSLSettings->KernelType != EPCGKernelType::Custom;

		if (bIsFirstOutputPin && CustomHLSLSettings->IsPointKernel())
		{
			if (Properties.AllowedTypes != EPCGDataType::Point)
			{
#if PCG_KERNEL_LOGGING_ENABLED
				AddStaticLogEntry(FText::Format(
					LOCTEXT("InvalidNonPointPrimaryOutput", "'Point Processor' and 'Point Generator' nodes require primary output pin to be of type 'Point', but found '{0}'."),
					FText::FromString(PCGCustomHLSLKernel::GetDataTypeString(Properties.AllowedTypes))),
					EPCGKernelLogVerbosity::Error);
#endif
				return false;
			}
		}
		else if (bIsFirstOutputPin && CustomHLSLSettings->IsTextureKernel())
		{
			if (!Properties.AllowedTypes.IsChildOf(FPCGDataTypeInfoTexture2DBase::AsId()))
			{
#if PCG_KERNEL_LOGGING_ENABLED
				AddStaticLogEntry(FText::Format(
					LOCTEXT("InvalidNonTexturePrimaryOutput", "Texture kernel nodes require primary output pin to be a 2D texture type, but found '{0}'."),
					FText::FromString(PCGCustomHLSLKernel::GetDataTypeString(Properties.AllowedTypes))),
					EPCGKernelLogVerbosity::Error);
#endif
				return false;
			}
		}
		else if (bIsFirstOutputPin && CustomHLSLSettings->IsAttributeSetKernel())
		{
			if (!(Properties.AllowedTypes & EPCGDataType::Param))
			{
#if PCG_KERNEL_LOGGING_ENABLED
				AddStaticLogEntry(FText::Format(
					LOCTEXT("InvalidNonAttributeSetPrimaryOutput", "'Attribute Set Processor' and 'Attribute Set Generator' nodes require primary output pin to be of type 'Attribute Set', but found '{0}'."),
					FText::FromString(PCGCustomHLSLKernel::GetDataTypeString(Properties.AllowedTypes))),
					EPCGKernelLogVerbosity::Error);
#endif
				return false;
			}
		}

		if (!PCGComputeHelpers::IsTypeAllowedAsOutput(Properties.AllowedTypes))
		{
#if PCG_KERNEL_LOGGING_ENABLED
			AddStaticLogEntry(FText::Format(
				LOCTEXT("InvalidOutputType", "Unsupported output type '{0}', found on pin '{1}'."),
				FText::FromString(PCGCustomHLSLKernel::GetDataTypeString(Properties.AllowedTypes)),
				FText::FromName(Properties.Label)),
				EPCGKernelLogVerbosity::Error);
#endif
			return false;
		}

		if (!bPinIsDefinedByKernel)
		{
			const FPCGPinPropertiesGPUStruct& Props = Properties.PropertiesGPU;

			if (Props.InitializationMode == EPCGPinInitMode::FromInputPins)
			{
				if (Props.PinsToInititalizeFrom.IsEmpty())
				{
#if PCG_KERNEL_LOGGING_ENABLED
					AddStaticLogEntry(FText::Format(
						LOCTEXT("InitFromEmptyPins", "Output pin '{0}' tried to initialize from input pins, but no pins were specified."),
						FText::FromName(Properties.Label)),
						EPCGKernelLogVerbosity::Error);
#endif
					return false;
				}

				for (const FName InitPinName : Props.PinsToInititalizeFrom)
				{
					const FPCGPinProperties* InitPinProps = CustomHLSLSettings->InputPins.FindByPredicate([InitPinName](const FPCGPinProperties& InPinProps)
					{
						return InPinProps.Label == InitPinName;
					});

					if (InitPinProps)
					{
						if (!PCGComputeHelpers::IsTypeAllowedAsOutput(InitPinProps->AllowedTypes))
						{
#if PCG_KERNEL_LOGGING_ENABLED
							AddStaticLogEntry(FText::Format(
								LOCTEXT("InitFromInvalidPinType", "Output pin '{0}' tried to initialize from input pin '{1}', but pin '{1}' has an invalid type."),
								FText::FromName(Properties.Label),
								FText::FromName(InitPinName)),
								EPCGKernelLogVerbosity::Error);
#endif
							return false;
						}
					}
					else
					{
#if PCG_KERNEL_LOGGING_ENABLED
						AddStaticLogEntry(FText::Format(
							LOCTEXT("InitFromNonExistentPin", "Output pin '{0}' tried to initialize from non-existent input pin '{1}'."),
							FText::FromName(Properties.Label),
							FText::FromName(InitPinName)),
							EPCGKernelLogVerbosity::Error);
#endif
						return false;
					}
				}

				// TODO: Could do validation on data multiplicity for Pairwise, checking that data counts are 1 or N, but maybe that should be a runtime error instead.
			}

			const bool bUsingFixedDataCount = Props.InitializationMode == EPCGPinInitMode::Custom || Props.DataCountMode == EPCGDataCountMode::Fixed;

			if (bUsingFixedDataCount)
			{
				if (Props.DataCount < 1)
				{
#if PCG_KERNEL_LOGGING_ENABLED
					AddStaticLogEntry(FText::Format(
						LOCTEXT("InvalidDataCount", "Invalid fixed data count {0} on output pin '{1}'. Must be greater than 0."),
						FText::AsNumber(Props.DataCount),
						FText::FromName(Properties.Label)),
						EPCGKernelLogVerbosity::Error);
#endif
					return false;
				}
			}

			const bool bUsingFixedElemCount = Props.InitializationMode == EPCGPinInitMode::Custom || Props.ElementCountMode == EPCGElementCountMode::Fixed;

			if (bUsingFixedElemCount)
			{
				if (Props.ElementCount < 1)
				{
#if PCG_KERNEL_LOGGING_ENABLED
					AddStaticLogEntry(FText::Format(
						LOCTEXT("InvalidElementCount", "Invalid fixed num elements {0} on output pin '{1}'. Must be greater than 0."),
						FText::AsNumber(Props.ElementCount),
						FText::FromName(Properties.Label)),
						EPCGKernelLogVerbosity::Error);
#endif
					return false;
				}

				if (Props.NumElements2D.GetMin() < 1)
				{
#if PCG_KERNEL_LOGGING_ENABLED
					AddStaticLogEntry(FText::Format(
						LOCTEXT("InvalidElementCount2D", "Invalid fixed num elements ({0}, {1}) on output pin '{2}'. Must be greater than 0."),
						FText::AsNumber(Props.NumElements2D.X),
						FText::AsNumber(Props.NumElements2D.Y),
						FText::FromName(Properties.Label)),
						EPCGKernelLogVerbosity::Error);
#endif
					return false;
				}
			}

			if (Props.ElementCountMultiplier < 1)
			{
#if PCG_KERNEL_LOGGING_ENABLED
				AddStaticLogEntry(FText::Format(
					LOCTEXT("InvalidElementCountMultiplier", "Invalid element count multiplier {0} on output pin '{1}'. Must be greater than 0."),
					FText::AsNumber(Props.ElementCountMultiplier),
					FText::FromName(Properties.Label)),
					EPCGKernelLogVerbosity::Error);
#endif
				return false;
			}
		}

		bIsFirstOutputPin = false;
	}

	if (CustomHLSLSettings->KernelType == EPCGKernelType::Custom && CustomHLSLSettings->DispatchThreadCount == EPCGDispatchThreadCount::FromProductOfInputPins)
	{
		if (CustomHLSLSettings->ThreadCountInputPinLabels.IsEmpty())
		{
#if PCG_KERNEL_LOGGING_ENABLED
			AddStaticLogEntry(LOCTEXT("MissingThreadCountPins", "Dispatch thread count is based on input pins but no labels have been set in Input Pins array."), EPCGKernelLogVerbosity::Error);
#endif
			return false;
		}

		for (const FName& Label : CustomHLSLSettings->ThreadCountInputPinLabels)
		{
			if (!CustomHLSLSettings->InputPins.FindByPredicate([Label](const FPCGPinProperties& InProps) { return InProps.Label == Label; }))
			{
#if PCG_KERNEL_LOGGING_ENABLED
				AddStaticLogEntry(FText::Format(LOCTEXT("MissingThreadCountPin", "Invalid pin specified in Input Pins array: '{0}'."), FText::FromName(Label)), EPCGKernelLogVerbosity::Error);
#endif
				return false;
			}
		}
	}

	if (IsThreadCountMultiplierInUse())
	{
		if (CustomHLSLSettings->ThreadCountMultiplier < 1)
		{
#if PCG_KERNEL_LOGGING_ENABLED
			AddStaticLogEntry(FText::Format(LOCTEXT("InvalidThreadCountMultiplier", "Thread Count Multiplier has invalid value ({0}). Must be greater than 0."), CustomHLSLSettings->ThreadCountMultiplier), EPCGKernelLogVerbosity::Error);
#endif
			return false;
		}
	}

	for (const FPCGKernelAttributeKey& AttributeKey : KernelAttributeKeys)
	{
		if (AttributeKey.GetType() == EPCGKernelAttributeType::Invalid)
		{
			const UEnum* AttributeTypeEnum = StaticEnum<EPCGKernelAttributeType>();
			check(AttributeTypeEnum);

#if PCG_KERNEL_LOGGING_ENABLED
			AddStaticLogEntry(FText::Format(
				LOCTEXT("InvalidAttributeTypeNone", "Attribute '{0}' has invalid GPU attribute type '{1}', check the 'Attributes to Create' array on your pins."),
				FText::FromName(AttributeKey.GetIdentifier().Name),
				FText::FromString(AttributeTypeEnum->GetNameStringByValue(static_cast<int64>(AttributeKey.GetType())))),
				EPCGKernelLogVerbosity::Error);
#endif
			return false;
		}
	}

	if (!ValidateShaderSource())
	{
		return false;
	}

	return true;
}

bool UPCGCustomHLSLKernel::ValidateShaderSource()
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());

	if (!CustomHLSLSettings->bMuteUnwrittenPinDataErrors && !AreAllOutputPinsWritten())
	{
		return false;
	}

	// @todo_pcg: Validation of parsed attribute functions could be done here instead of during parsing?

	return true;
}

FString UPCGCustomHLSLKernel::ProcessShaderSource(FPCGGPUCompilationContext& InOutContext, const FPCGCustomHLSLParsedSource& InParsedSource) const
{
	FString OutShaderSource = InParsedSource.Source;

	const FPCGKernelAttributeTable* StaticAttributeTable = InOutContext.GetStaticAttributeTable();

	if (!ensure(StaticAttributeTable))
	{
		return OutShaderSource;
	}

	// Replacement relies on precomputed indices into the source strings, therefore the replacement must take place
	// before any other modifications. Otherwise, the indices will be incorrect and the source will become gibberish.
	using FReplacement = TTuple</*ReplacementString=*/FString, /*ReplaceStartIndex=*/int32, /*ReplaceEndIndex=*/int32>;
	TArray<FReplacement> Replacements;

	const UEnum* AttributeTypeEnum = StaticEnum<EPCGKernelAttributeType>();
	check(AttributeTypeEnum);

	// We inject attribute IDs directly into the source. This is most efficient and saves us passing them into the kernel. However
	// the trade off is that a shader with a shared source will generate multiple variants if used in different compute graphs with
	// different attribute IDs.
	for (const FPCGParsedAttributeFunction& ParsedFunction : InParsedSource.AttributeFunctions)
	{
		const int64 AttrType = ParsedFunction.AttributeType;

		if (!ensure(AttrType != INDEX_NONE))
		{
			break;
		}

		const FPCGKernelAttributeKey AttributeKey(FPCGAttributePropertySelector::CreateSelectorFromString(*ParsedFunction.AttributeName), static_cast<EPCGKernelAttributeType>(AttrType));
		const FString SourceDefinition = FString::Format(TEXT("'{0}'"), { ParsedFunction.AttributeName });

		const int32 AttributeIndex = StaticAttributeTable->GetAttributeId(AttributeKey);
		const int32 ReplaceStartIndex = OutShaderSource.Find(SourceDefinition, ESearchCase::CaseSensitive, ESearchDir::FromStart, ParsedFunction.MatchBeginning);
		const int32 ReplaceEndIndex = ReplaceStartIndex + SourceDefinition.Len();

		Replacements.Emplace(FString::FromInt(AttributeIndex), ReplaceStartIndex, ReplaceEndIndex);
	}

	// We inject data IDs directly into the source. They will get remapped to data indices using a label resolver data interface.
	for (const TPair<FName, FPCGDataLabels>& Pair : PinDataLabels.PinToDataLabels)
	{
		const FName PinLabel = Pair.Key;
		const TArray<FString>& DataLabels = Pair.Value.Labels;

		for (int DataId = 0; DataId < DataLabels.Num(); ++DataId)
		{
			const FString& DataLabel = DataLabels[DataId];
			const FString ReplacementStr = FString::Format(TEXT("{0}_GetDataIndexFromIdInternal(/*DataId=*/{1}u)"),
			{
				PCGComputeHelpers::GetDataLabelResolverName(PinLabel),
				FString::FromInt(DataId)
			});

			// Matches against {PinName}_AnyFunction('{DataLabel}'...
			// First capture group is the data label, so that we can find & replace it by index.
			const FString Pattern = FString::Format(TEXT("{0}_.*?[\\s]*?\\([\\s]*?('{1}')"), { PinLabel.ToString(), DataLabel });
			FRegexMatcher ModuleMatcher(FRegexPattern(Pattern), OutShaderSource);

			while (ModuleMatcher.FindNext())
			{
				const int32 DataLabelCharIndexStart = ModuleMatcher.GetCaptureGroupBeginning(1);
				const int32 DataLabelCharIndexEnd = ModuleMatcher.GetCaptureGroupEnding(1);

				Replacements.Emplace(ReplacementStr, DataLabelCharIndexStart, DataLabelCharIndexEnd);
			}
		}
	}

	// Sort the replacements by replacement index and apply them to the source in reverse order.
	// Note: Assumes that two replacements do not overlap.
	Algo::Sort(Replacements, [](const FReplacement& A, const FReplacement& B) { return A.Get<1>() < B.Get<1>(); });

	for (int I = Replacements.Num() - 1; I >= 0; --I)
	{
		const FString& ReplacementString = Replacements[I].Get<0>();
		const int32 ReplaceStartIndex = Replacements[I].Get<1>();
		const int32 ReplaceEndIndex = Replacements[I].Get<2>();

		OutShaderSource = OutShaderSource.Left(ReplaceStartIndex) + ReplacementString + OutShaderSource.RightChop(ReplaceEndIndex);
	}

	// Remove old-school stuff.
	OutShaderSource.ReplaceInline(TEXT("\r"), TEXT(""));

	// @todo_pcg: Replace using token ranges instead of find/replace, similar to what we do with the parsed attribute functions.
	// Replace function calls like Out_CopyElementFrom_In(...) with macro PCG_COPY_ALL_ATTRIBUTES_TO_OUTPUT(Out, In, ...).
	for (const FPCGParsedCopyElementFunction& ParsedFunction : InParsedSource.CopyElementFunctions)
	{
		OutShaderSource.ReplaceInline(
			*FString::Format(TEXT("{2}_{0}_{1}("), { PCGCustomHLSLKernel::CopyElementFunctionKeyword, ParsedFunction.SourcePin, ParsedFunction.TargetPin }),
			*FString::Format(TEXT("PCG_COPY_ALL_ATTRIBUTES_TO_OUTPUT({1}, {0}, "), { ParsedFunction.SourcePin, ParsedFunction.TargetPin })
		);
	}

	return OutShaderSource;
}

FString UPCGCustomHLSLKernel::ProcessAdditionalShaderSources(FPCGGPUCompilationContext& InOutContext) const
{
	// @todo_pcg: We should pivot to a stringbuilder here for perf.
	FString OutShaderSource;

	// The first parsed source is reserved for the kernel source.
	for (int SourceIndex = 1; SourceIndex < ParsedSources.Num(); ++SourceIndex)
	{
		OutShaderSource += ProcessShaderSource(InOutContext, ParsedSources[SourceIndex]) + "\n\n";
	}

	return OutShaderSource;
}

bool UPCGCustomHLSLKernel::AreAllOutputPinsWritten()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCustomHLSLKernel::AreAllOutputPinsWritten);

	auto IsPinInitializedBySource = [](const FPCGCustomHLSLParsedSource& InParsedSource, const FString& PinStr)
	{
		return InParsedSource.InitializedOutputPins.Contains(PinStr);
	};

	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());

	// Processor/Generator kernels initialize the first output pin data automatically.
	const bool bSkipFirstPin = CustomHLSLSettings->IsProcessorKernel() || CustomHLSLSettings->IsGeneratorKernel();

	for (int32 I = 0; I < CustomHLSLSettings->OutputPins.Num(); ++I)
	{
		if (I == 0 && bSkipFirstPin)
		{
			continue;
		}

		const FPCGPinPropertiesGPU& PinProps = CustomHLSLSettings->OutputPins[I];
		const FString PinStr = PinProps.Label.ToString();
		bool bInitializedByAnySource = false;

		for (const FPCGCustomHLSLParsedSource& ParsedSource : ParsedSources)
		{
			if (IsPinInitializedBySource(ParsedSource, PinStr))
			{
				bInitializedByAnySource = true;
				break;
			}
		}

		if (!bInitializedByAnySource)
		{
#if PCG_KERNEL_LOGGING_ENABLED
			AddStaticLogEntry(FText::Format(
				LOCTEXT("PinMayNotBeWritten", "Data on pin '{0}' may be uninitialized. Add code to write to this data, or mute this error in the node settings."),
				{ FText::FromString(PinStr) }),
				EPCGKernelLogVerbosity::Error);
#endif
			return false;
		}
	}

	return true;
}
#endif // WITH_EDITOR

bool UPCGCustomHLSLKernel::IsThreadCountMultiplierInUse() const
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());
	return CustomHLSLSettings->KernelType == EPCGKernelType::Custom && CustomHLSLSettings->DispatchThreadCount != EPCGDispatchThreadCount::Fixed;
}

bool UPCGCustomHLSLKernel::AreAttributesValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext, FText* OutErrorText) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCustomHLSLKernel::AreAttributesValid);

	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());

	if (InDataBinding)
	{
		TMap<FName, const TSharedPtr<const FPCGDataCollectionDesc>> InputPinDescs;
		TMap<FName, const TSharedPtr<const FPCGDataCollectionDesc>> OutputPinDescs;

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCustomHLSLKernel::AreAttributesValid::GetCachedPinDataDescs);

			for (const FPCGPinProperties& Pin : CustomHLSLSettings->InputPins)
			{
				if (const TSharedPtr<const FPCGDataCollectionDesc> DataDesc = InDataBinding->GetCachedKernelPinDataDesc(this, Pin.Label, /*bIsInputPin=*/true))
				{
					InputPinDescs.Add(Pin.Label, DataDesc);
				}
			}

			for (const FPCGPinProperties& Pin : CustomHLSLSettings->OutputPins)
			{
				if (const TSharedPtr<const FPCGDataCollectionDesc> DataDesc = InDataBinding->GetCachedKernelPinDataDesc(this, Pin.Label, /*bIsInputPin=*/false))
				{
					OutputPinDescs.Add(Pin.Label, DataDesc);
				}
			}
		}

		auto ValidateParsedAttributeFunctions = [&InputPinDescs, &OutputPinDescs, OutErrorText](const TArray<FPCGParsedAttributeFunction>& ParsedAttributeFunctions)
		{
			for (const auto& ParsedFunction : ParsedAttributeFunctions)
			{
				const UEnum* AttributeTypeEnum = StaticEnum<EPCGKernelAttributeType>();
				check(AttributeTypeEnum);

				const FString& PinLabelStr = ParsedFunction.PinLabel;
				const FString& FunctionName = ParsedFunction.FunctionName;
				const FString& AttributeName = ParsedFunction.AttributeName;
				const FString TypeStr = AttributeTypeEnum->GetNameStringByValue(ParsedFunction.AttributeType);

				const FName PinLabel = FName(*PinLabelStr);
				TSharedPtr<const FPCGDataCollectionDesc> PinDesc = nullptr;

				auto ConstructFunctionText = [&ParsedFunction, &TypeStr]()
				{
					return FText::FromString(ParsedFunction.PinLabel + TEXT("_") + ParsedFunction.FunctionName + TypeStr);
				};

				if (FunctionName == PCGCustomHLSLKernel::AttributeFunctionSetKeyword || FunctionName == PCGCustomHLSLKernel::AttributeFunctionAtomicAddKeyword)
				{
					if (const TSharedPtr<const FPCGDataCollectionDesc>* PinDescPtr = OutputPinDescs.Find(PinLabel))
					{
						PinDesc = *PinDescPtr;
					}

					if (!PinDesc && InputPinDescs.Find(PinLabel))
					{
#if PCG_KERNEL_LOGGING_ENABLED
						if (OutErrorText)
						{
							*OutErrorText = FText::Format(
								LOCTEXT("InvalidSetAttributeUsage", "Tried to call attribute function '{0}' on read-only input pin '{1}'."),
								ConstructFunctionText(),
								FText::FromName(PinLabel));
						}
#endif

						return false;
					}
				}
				else if (ensure(FunctionName == PCGCustomHLSLKernel::AttributeFunctionGetKeyword))
				{
					if (const TSharedPtr<const FPCGDataCollectionDesc>* PinDescPtr = InputPinDescs.Find(PinLabel))
					{
						PinDesc = *PinDescPtr;
					}
				}

				if (!PinDesc)
				{
#if PCG_KERNEL_LOGGING_ENABLED
					if (OutErrorText)
					{
						*OutErrorText = FText::Format(
							LOCTEXT("InvalidAttributePinName", "Tried to call attribute function '{0}' on non-existent pin '{1}'."),
							ConstructFunctionText(),
							FText::FromName(PinLabel));
					}
#endif

					return false;
				}


				const int64 AttrType = AttributeTypeEnum->GetValueByName(FName(*TypeStr));

				if (AttrType == INDEX_NONE)
				{
#if PCG_KERNEL_LOGGING_ENABLED
					if (OutErrorText)
					{
						*OutErrorText = FText::Format(
							LOCTEXT("InvalidAttributePinType", "Tried to call attribute function '{0}' on non-existent type '{1}'."),
							ConstructFunctionText(),
							FText::FromString(TypeStr));
					}
#endif

					return false;
				}

				const FPCGKernelAttributeDesc* AttrDesc = nullptr;
				const FPCGKernelAttributeKey AttrKey(FPCGAttributePropertySelector::CreateSelectorFromString(AttributeName), static_cast<EPCGKernelAttributeType>(AttrType));
				bool bFoundMatchingAttributeName = false;

				// Verify that the attribute exists on at least one data in pin data collection.
				for (const FPCGDataDesc& DataDesc : PinDesc->GetDataDescriptions())
				{
					AttrDesc = DataDesc.GetAttributeDescriptions().FindByPredicate([&AttrKey, &bFoundMatchingAttributeName](const FPCGKernelAttributeDesc& Desc)
					{
						const bool bAttributeNameMatches = Desc.GetAttributeKey().GetIdentifier() == AttrKey.GetIdentifier();
						bFoundMatchingAttributeName |= bAttributeNameMatches;

						return bAttributeNameMatches && Desc.GetAttributeKey().GetType() == AttrKey.GetType();
					});

					if (AttrDesc)
					{
						break;
					}
				}

				if (!AttrDesc && !PinDesc->GetDataDescriptions().IsEmpty())
				{
#if PCG_KERNEL_LOGGING_ENABLED
					if (OutErrorText)
					{
						if (bFoundMatchingAttributeName)
						{
							*OutErrorText = FText::Format(
								LOCTEXT("InvalidAttributeType", "Tried to call attribute function '{0}' on attribute '{1}' which is not of type '{2}'."),
								ConstructFunctionText(),
								FText::FromString(AttributeName),
								FText::FromString(TypeStr));
						}
						else
						{
							*OutErrorText = FText::Format(
								LOCTEXT("InvalidAttributeDNE", "Tried to call attribute function '{0}' on attribute '{1}' which does not exist."),
								ConstructFunctionText(),
								FText::FromString(AttributeName),
								FText::FromString(TypeStr));
						}
					}
#endif
					return false;
				}
			}

			return true;
		};

		for (const FPCGCustomHLSLParsedSource& ParsedSource : ParsedSources)
		{
			if (!ValidateParsedAttributeFunctions(ParsedSource.AttributeFunctions))
			{
				return false;
			}
		}
	}

	return true;
}

const FPCGPinProperties* UPCGCustomHLSLKernel::GetFirstInputPin() const
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());
	return !CustomHLSLSettings->InputPins.IsEmpty() ? &CustomHLSLSettings->InputPins[0] : nullptr;
}

const FPCGPinPropertiesGPU* UPCGCustomHLSLKernel::GetFirstOutputPin() const
{
	const UPCGCustomHLSLSettings* CustomHLSLSettings = CastChecked<UPCGCustomHLSLSettings>(GetSettings());
	return !CustomHLSLSettings->OutputPins.IsEmpty() ? &CustomHLSLSettings->OutputPins[0] : nullptr;
}

int UPCGCustomHLSLKernel::GetElementCountForInputPin(const FPCGPinProperties& InInputPinProps, const UPCGDataBinding* InBinding) const
{
	check(InBinding);

	const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InBinding->GetCachedKernelPinDataDesc(this, InInputPinProps.Label, /*bIsInputPin=*/true);

	return InputDesc ? InputDesc->ComputeTotalElementCount() : 0;
}

#undef LOCTEXT_NAMESPACE
