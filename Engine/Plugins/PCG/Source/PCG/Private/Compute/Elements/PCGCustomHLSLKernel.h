// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGComputeKernel.h"

#include "Compute/PCGPinPropertiesGPU.h"

#include "PCGCustomHLSLKernel.generated.h"

/** Helper struct for serializing per-pin created attribute key lists. */
USTRUCT()
struct FPCGKernelAttributeKeyList
{
    GENERATED_BODY()

public:
    UPROPERTY()
    TArray<FPCGKernelAttributeKey> Keys;
};

USTRUCT()
struct FPCGParsedAttributeFunction
{
	GENERATED_BODY()

public:
	FPCGParsedAttributeFunction() = default;
	FPCGParsedAttributeFunction(const FString& InPinLabel, const FString& InFunctionName, int64 InAttributeType, const FString& InAttributeName, int32 InMatchBeginning)
		: PinLabel(InPinLabel)
		, FunctionName(InFunctionName)
		, AttributeType(InAttributeType)
		, AttributeName(InAttributeName)
		, MatchBeginning(InMatchBeginning)
	{}

	UPROPERTY()
	FString PinLabel;

	UPROPERTY()
	FString FunctionName;

	UPROPERTY()
	int64 AttributeType = INDEX_NONE;

	UPROPERTY()
	FString AttributeName;

	UPROPERTY()
	int32 MatchBeginning = INDEX_NONE;
};

USTRUCT()
struct FPCGParsedCopyElementFunction
{
	GENERATED_BODY()

public:
	FPCGParsedCopyElementFunction() = default;
	FPCGParsedCopyElementFunction(const FString& InSourcePin, const FString& InTargetPin)
		: SourcePin(InSourcePin)
		, TargetPin(InTargetPin)
	{}

	UPROPERTY()
	FString SourcePin;

	UPROPERTY()
	FString TargetPin;
};

// @todo_pcg: It would be ideal someday for parsed source to be editor only. Unfortunately not possible right now because we need to validate the parsed attribute functions during execution.
/** Holds the results of parsing for some HLSL source. */
USTRUCT()
struct FPCGCustomHLSLParsedSource
{
	GENERATED_BODY()

	FPCGCustomHLSLParsedSource() = default;

#if WITH_EDITOR
	FPCGCustomHLSLParsedSource(const FString& InSource)
		: Source(InSource)
	{}
#endif

	UPROPERTY()
	TArray<FPCGParsedAttributeFunction> AttributeFunctions;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FPCGParsedCopyElementFunction> CopyElementFunctions;

	enum class ETokenType : uint8
	{
		Normal,
		Keyword,
		PreProcessorKeyword,
		Operator,
		DoubleQuotedString,
		SingleQuotedString,
		Comment,
		Whitespace
	};

	struct FToken
	{
		ETokenType Type = ETokenType::Normal;
		FTextRange Range;
	};

	FString Source;
	TArray<FToken> Tokens;

	/** Pins identified as being written to. Used to validate that output pins are initialized in some way. */
	TArray<FString> InitializedOutputPins;
#endif
};

UCLASS()
class UPCGCustomHLSLKernel : public UPCGComputeKernel
{
	GENERATED_BODY()

	// ~Begin UPCGComputeKernel interface
public:
	virtual bool IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const override;
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
#if WITH_EDITOR
	virtual FString GetCookedSource(FPCGGPUCompilationContext& InOutContext) const override;
	virtual FString GetEntryPoint() const override { return EntryPoint; }
	virtual void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
#endif
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const override;
	virtual void GetDataLabels(FName InPinLabel, TArray<FString>& OutDataLabels) const override;
	virtual void GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const override;
	virtual uint32 GetThreadCountMultiplier() const override;
	virtual uint32 GetElementCountMultiplier(FName InOutputPinLabel) const override;
	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;

	void SetOutputPinRequiresZeroInitialization(FName InOutputPinLabel) { PinsRequiringZeroInitialization.AddUnique(InOutputPinLabel); }
	virtual bool DoesOutputPinRequireZeroInitialization(FName InOutputPinLabel) const override { return PinsRequiringZeroInitialization.Contains(InOutputPinLabel); }

protected:
#if WITH_EDITOR
	virtual void InitializeInternal() override;
	virtual bool PerformStaticValidation() override;
	// ~End UPCGComputeKernel interface

	void InitEntryPoint();
	void PopulateAttributeKeys();
	void ParseShaderSource();
	void CreateParsedSources();
	void CollectDataLabels(const FPCGCustomHLSLParsedSource& InParsedSource);
	bool ValidateShaderSource();
	bool AreAllOutputPinsWritten();

	/** Apply any code gen/transformations/expansions to the shader source. */
	virtual FString ProcessShaderSource(FPCGGPUCompilationContext& InOutContext, const FPCGCustomHLSLParsedSource& InParsedSource) const;
	virtual FString ProcessAdditionalShaderSources(FPCGGPUCompilationContext& InOutContext) const;
#endif

	/** Will the ThreadCountMultiplier value be applied when calculating the dispatch thread count. */
	bool IsThreadCountMultiplierInUse() const;
	bool AreAttributesValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext, FText* OutErrorText) const;

	const FPCGPinProperties* GetFirstInputPin() const;
	const FPCGPinPropertiesGPU* GetFirstOutputPin() const;

	int GetElementCountForInputPin(const FPCGPinProperties& InInputPinProps, const UPCGDataBinding* InBinding) const;

public:
#if WITH_EDITORONLY_DATA
	/** The name of the main function in the shader. Generated from the node title. */
	UPROPERTY()
	FString EntryPoint;
#endif

protected:
	UPROPERTY()
	TArray<FPCGCustomHLSLParsedSource> ParsedSources;

	UPROPERTY()
	TArray<FPCGKernelAttributeKey> KernelAttributeKeys;

	UPROPERTY()
	FPCGPinDataLabels PinDataLabels;

	/** Per-output-pin created attribute keys, combining manually authored CreatedKernelAttributeKeys and
	 *  keys inferred from HLSL Set/AtomicAdd calls. Populated during editor init, serialized for runtime use. */
	UPROPERTY()
	TMap<FName, FPCGKernelAttributeKeyList> PinCreatedAttributeKeys;

	/** Properties that need to be fully allocated per pin. Note, we use an integer instead of EPCGPointNativeProperties directly because bitflag enums don't serialize correctly. */
	UPROPERTY()
	TMap<FName, /*EPCGPointNativeProperties*/int32> PinToPropertiesWithSetters;

	/** Properties written exclusively with constant literal expressions per pin. These are stripped from any inherited input allocation so the output uses a single-slot constant value range (CVR) instead of a
	 *  per-element buffer. Disjoint with PinToPropertiesWithSetters: a non-constant setter always wins. */
	UPROPERTY()
	TMap<FName, /*EPCGPointNativeProperties*/int32> PinToPropertiesWithSingleConstantSetters;

	/** Per-output-pin fine-grained TRS setter tracking for smart init. Bit 0 = SetPosition was called, Bit 1 = SetRotation was called, Bit 2 = SetScale was called. SetPointTransform sets all three bits.
	 *  Separate from PinToPropertiesWithSetters which uses the combined Transform bit; allows smart-init to handle e.g. rotation/scale independently when only position is written. */
	UPROPERTY()
	TMap<FName, int32> PinWrittenTransformSetters;

	/** Pins for which the output buffer should be statically initialized to 0 before execution. */
	UPROPERTY()
	TArray<FName> PinsRequiringZeroInitialization;
};
