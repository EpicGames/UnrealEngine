// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/Elements/PCGNormalToDensityKernel.h"

#include "PCGContext.h"
#include "Compute/PCGDataBinding.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Compute/DataInterfaces/Elements/PCGNormalToDensityDataInterface.h"

#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGNormalToDensityKernel)

#define LOCTEXT_NAMESPACE "PCGTemplateKernel"

TSharedPtr<const FPCGDataCollectionDesc> UPCGNormalToDensityKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	// Code assumes single output pin.
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	const FPCGKernelPin SourceKernelPin(GetKernelIndex(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	const TSharedPtr<const FPCGDataCollectionDesc> InDataDesc = InBinding->ComputeKernelPinDataDesc(SourceKernelPin);

	if (!ensure(InDataDesc))
	{
		return nullptr;
	}

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeSharedFrom(InDataDesc);

	// Allocate density property if the transform is also allocated. Otherwise the densities will all be the same value and full-width allocation is unnecessary.
	for (FPCGDataDesc& DataDesc : OutDataDesc->GetDataDescriptionsMutable())
	{
		if (!!(DataDesc.GetAllocatedProperties() & EPCGPointNativeProperties::Transform))
		{
			DataDesc.AllocateProperties(EPCGPointNativeProperties::Density);
		}
	}

	return OutDataDesc;
}

int UPCGNormalToDensityKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const TSharedPtr<const FPCGDataCollectionDesc> OutputPinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultOutputLabel, /*bIsInput=*/false);
	return ensure(OutputPinDesc) ? OutputPinDesc->ComputeTotalElementCount() : 0;
}

#if WITH_EDITOR
void UPCGNormalToDensityKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGNormalToDensityDataInterface> NodeDI = InOutContext.NewObject_AnyThread<UPCGNormalToDensityDataInterface>(InObjectOuter);
	NodeDI->SetProducerKernel(this);

	OutDataInterfaces.Add(NodeDI);
}
#endif

void UPCGNormalToDensityKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
}

void UPCGNormalToDensityKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
}

#undef LOCTEXT_NAMESPACE
