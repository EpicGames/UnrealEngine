// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/Elements/PCGTransformPointsKernel.h"

#include "PCGContext.h"
#include "Compute/PCGDataBinding.h"
#include "Elements/PCGTransformPoints.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Compute/DataInterfaces/Elements/PCGTransformPointsDataInterface.h"

#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGTransformPointsKernel)

#define LOCTEXT_NAMESPACE "PCGTransformPointsKernel"

TSharedPtr<const FPCGDataCollectionDesc> UPCGTransformPointsKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	// Code assumes single output pin.
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	const FPCGKernelPin SourceKernelPin(GetKernelIndex(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	const TSharedPtr<const FPCGDataCollectionDesc> InDataDesc = InBinding->ComputeKernelPinDataDesc(SourceKernelPin);

	if (!ensure(InDataDesc))
	{
		return nullptr;
	}

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeSharedFrom(InDataDesc);
	
	const UPCGTransformPointsSettings* Settings = CastChecked<UPCGTransformPointsSettings>(GetSettings());
	if (!ensure(Settings))
	{
		return nullptr;
	}

	// Allocate properties if needed
	for (FPCGDataDesc& DataDesc : OutDataDesc->GetDataDescriptionsMutable())
	{
		DataDesc.AllocateProperties(Settings->GetPropertiesToAllocate());
	}

	return OutDataDesc;
}

int UPCGTransformPointsKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const TSharedPtr<const FPCGDataCollectionDesc> OutputPinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultOutputLabel, /*bIsInput=*/false);
	return ensure(OutputPinDesc) ? OutputPinDesc->ComputeTotalElementCount() : 0;
}

#if WITH_EDITOR
void UPCGTransformPointsKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGTransformPointsDataInterface> NodeDI = InOutContext.NewObject_AnyThread<UPCGTransformPointsDataInterface>(InObjectOuter);
	NodeDI->SetProducerKernel(this);

	OutDataInterfaces.Add(NodeDI);
}
#endif

void UPCGTransformPointsKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
}

void UPCGTransformPointsKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
}

#undef LOCTEXT_NAMESPACE
