// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGComputeCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGSubsystem.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Compute/DataInterfaces/PCGDataCollectionDataInterface.h"
#include "Compute/DataInterfaces/PCGLandscapeDataInterface.h"
#include "Compute/DataInterfaces/PCGRawBufferDataInterface.h"
#include "Compute/DataInterfaces/PCGStaticMeshDataInterface.h"
#include "Compute/DataInterfaces/PCGTexture2DArrayDataInterface.h"
#include "Compute/DataInterfaces/PCGTextureDataInterface.h"
#include "Compute/DataInterfaces/PCGVirtualTextureDataInterface.h"
#include "Compute/Packing/PCGDataCollectionPacking.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGTextureData.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Editor/IPCGEditorModule.h"

#include "DynamicRHI.h"
#include "PixelFormat.h"
#include "RHIStats.h"
#include "RenderResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGComputeCommon)

namespace PCGComputeConstants
{
	constexpr TCHAR DataLabelTagPrefix[] = { TEXT("PCG_DATA_LABEL") };
}

namespace PCGComputeHelpers
{
	static TAutoConsoleVariable<float> CVarMaxGPUBufferSizeProportion(
		TEXT("pcg.GraphExecution.GPU.MaxBufferSize"),
		0.5f,
		TEXT("Maximum GPU buffer size as proportion of total available graphics memory."));

	FIntVector4 GetElementCount(const UPCGData* InData)
	{
		FIntVector4 ElementCount = FIntVector4::ZeroValue;

		if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(InData))
		{
			ElementCount.X = PointData->GetNumPoints();
		}
		else if (const UPCGParamData* ParamData = Cast<UPCGParamData>(InData))
		{
			if (const UPCGMetadata* Metadata = ParamData->ConstMetadata())
			{
				ElementCount.X = Metadata->GetItemCountForChild();
			}
		}
		else if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(InData))
		{
			ElementCount = Proxy->GetElementCount();
		}
		else if (const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(InData))
		{
			const FIntPoint TextureSize = TextureData->GetResolution();
			ElementCount.X = TextureSize.X;
			ElementCount.Y = TextureSize.Y;
		}
		else if (const UPCGTexture2DArrayData* Texture2DArrayData = Cast<UPCGTexture2DArrayData>(InData))
		{
			const FIntPoint TextureResolution = Texture2DArrayData->GetResolution();
			ElementCount.X = TextureResolution.X;
			ElementCount.Y = TextureResolution.Y;
			ElementCount.Z = Texture2DArrayData->GetArraySize();
		}
		else if (const UPCGRawBufferData* RawBufferData = Cast<UPCGRawBufferData>(InData))
		{
			ElementCount.X = RawBufferData->GetNumUint32s();
		}

		return ElementCount;
	}

	EPCGElementDimension GetElementDimension(const UPCGData* InData)
	{
		return InData ? GetElementDimension(InData->GetDataTypeId()) : EPCGElementDimension::One;
	}

	EPCGElementDimension GetElementDimension(const FPCGDataTypeIdentifier& InDataType)
	{
		if (InDataType.IsChildOf(EPCGDataType::BaseTexture))
		{
			return EPCGElementDimension::Two;
		}
		else if (InDataType.IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId()))
		{
			return EPCGElementDimension::Three;
		}

		return EPCGElementDimension::One;
	}

	int32 GetMaxOutputDataCount(const FPCGDataTypeIdentifier& InDataType)
	{
		if (InDataType.IsChildOf(EPCGDataType::BaseTexture) || InDataType.IsChildOf(FPCGDataTypeInfoRawBuffer::AsId()) || InDataType.IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId()))
		{
			// These data types can only emit a single data.
			return 1;
		}
		else
		{
			// No limit.
			return INT32_MAX;
		}
	}

	const TArray<FPCGDataTypeIdentifier>& GetAllowedInputTypesList()
	{
		static TArray<FPCGDataTypeIdentifier> AllowedTypes =
		{
			EPCGDataType::Point,
			EPCGDataType::Param,
			EPCGDataType::Landscape,
			EPCGDataType::BaseTexture,
			EPCGDataType::VirtualTexture,
			EPCGDataType::StaticMeshResource,
			EPCGDataType::ProxyForGPU,
			FPCGDataTypeInfoRawBuffer::AsId(),
			FPCGDataTypeInfoTexture2DArray::AsId()
		};

		return AllowedTypes;
	}

	const TArray<FPCGDataTypeIdentifier>& GetAllowedOutputTypesList()
	{
		static TArray<FPCGDataTypeIdentifier> AllowedTypes =
		{
			EPCGDataType::Point,
			EPCGDataType::Param,
			EPCGDataType::ProxyForGPU,
			EPCGDataType::BaseTexture,
			FPCGDataTypeInfoRawBuffer::AsId(),
			FPCGDataTypeInfoTexture2DArray::AsId()
		};

		return AllowedTypes;
	}

	inline const FPCGDataTypeIdentifier& GetAllowedInputTypes()
	{
		static FPCGDataTypeIdentifier Result = FPCGDataTypeIdentifier::Construct(GetAllowedInputTypesList());
		return Result;
	}

	inline const FPCGDataTypeIdentifier& GetAllowedOutputTypes()
	{
		static FPCGDataTypeIdentifier Result = FPCGDataTypeIdentifier::Construct(GetAllowedOutputTypesList());
		return Result;
	}

	/** PCG data types supported in GPU data collections. */
	inline const FPCGDataTypeIdentifier& GetAllowedDataCollectionTypes()
	{
		// Intentionally excludes raw buffer data which cannot be mixed with other data types like point.
		static FPCGDataTypeIdentifier Result{ EPCGDataType::Point | EPCGDataType::Param | EPCGDataType::ProxyForGPU };
		return Result;
	}

	bool IsTypeAllowedAsInput(const FPCGDataTypeIdentifier& Type)
	{
		return GetAllowedInputTypes().Intersects(Type);
	}

	bool IsTypeAllowedAsOutput(const FPCGDataTypeIdentifier& Type)
	{
		return GetAllowedOutputTypes().Intersects(Type);
	}

	bool IsTypeAllowedInDataCollection(const FPCGDataTypeIdentifier& Type)
	{
		return GetAllowedDataCollectionTypes().Intersects(Type);
	}

	bool ShouldImportAttributesFromData(const UPCGData* InData)
	{
		// We only read and expose attributes to compute from the following types. Other types are supported but we don't
		// register/upload their metadata attributes automatically.
		return InData && (InData->IsA<UPCGParamData>() || InData->IsA<UPCGBasePointData>());
	}

#if PCG_KERNEL_LOGGING_ENABLED
	void LogKernelWarning(const FPCGContext* Context, const UPCGSettings* Settings, const FText& InText)
	{
#if WITH_EDITOR
		if (Context && Settings)
		{
			if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
			{
				const FPCGStack* Stack = Context->GetStack();
				FPCGStack StackWithNode = Stack ? *Stack : FPCGStack();
				StackWithNode.PushFrame(Settings->GetOuter());

				PCGEditorModule->GetNodeVisualLogsMutable().Log(StackWithNode, ELogVerbosity::Warning, InText);
			}
		}
#endif
		PCGE_LOG_C(Warning, LogOnly, Context, InText);
	}

	void LogKernelError(const FPCGContext* Context, const UPCGSettings* Settings, const FText& InText)
	{
#if WITH_EDITOR
		if (Context && Settings)
		{
			if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
			{
				FPCGStack StackWithNode = Context->GetStack() ? *Context->GetStack() : FPCGStack();
				StackWithNode.PushFrame(Settings->GetOuter());

				PCGEditorModule->GetNodeVisualLogsMutable().Log(StackWithNode, ELogVerbosity::Error, InText);
			}
		}
#endif
		PCGE_LOG_C(Error, LogOnly, Context, InText);
	}
#endif

	bool IsBufferSizeTooLarge(uint64 InBufferSizeBytes, bool bInLogError)
	{
		FTextureMemoryStats TextureMemStats;
		RHIGetTextureMemoryStats(TextureMemStats);

		// If buffer size exceeds a proportion of total graphics memory, then it is deemed too large. Using this as a heuristic as there
		// is no RHI API to obtain available graphics memory outside of D3D12.
		const uint64 GPUBudgetBytes = static_cast<uint64>(TextureMemStats.TotalGraphicsMemory) * static_cast<double>(CVarMaxGPUBufferSizeProportion.GetValueOnAnyThread());

		// Buffer size also cannot exceed the max size of a uint32, otherwise the size will get truncated and other systems will break.
		// TODO: This limits the maximum number of points to around 46 million. Support uint64 max instead of uint32 to get up to 2 billion points.
		const uint64 MaxBufferSize = TNumericLimits<uint32>::Max();
		const uint64 BudgetBytes = FMath::Min(GPUBudgetBytes, MaxBufferSize);

		const bool bBufferTooLarge = TextureMemStats.TotalGraphicsMemory > 0 && InBufferSizeBytes > BudgetBytes;

		if (bBufferTooLarge && bInLogError)
		{
			UE_LOGF(LogPCG, Error, "Attempted to allocate a GPU buffer of size %llu bytes which is larger than the safety threshold (%llu bytes). Compute graph execution aborted.",
				InBufferSizeBytes,
				BudgetBytes);
		}

		return bBufferTooLarge;
	}

	bool CanPackIntoDataCollection(const FPCGDataCollectionDesc& InDataCollectionDesc)
	{
		TConstArrayView<FPCGDataDesc> DataDescs = InDataCollectionDesc.GetDataDescriptions();
		if (DataDescs.IsEmpty())
		{
			return true;
		}

		const FPCGDataTypeIdentifier& FirstType = DataDescs[0].GetType();
		if (!IsTypeAllowedInDataCollection(FirstType))
		{
			return false;
		}

		for (int32 i = 1; i < DataDescs.Num(); ++i)
		{
			if (!DataDescs[i].GetType().IsSameType(FirstType))
			{
				return false;
			}
		}

		return true;
	}

	uint64 ComputeSizeBytes(TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc)
	{
		if (!InDataCollectionDesc)
		{
			return 0;
		}

		if (CanPackIntoDataCollection(*InDataCollectionDesc))
		{
			return PCGDataCollectionPackingHelpers::ComputePackedSizeBytes(InDataCollectionDesc);
		}

		// Non-data-collection types that can be written from GPU nodes: currently texture render targets or raw uint32 buffers.
		uint64 TotalSizeBytes = 0;
		for (const FPCGDataDesc& DataDesc : InDataCollectionDesc->GetDataDescriptions())
		{
			const FPCGDataTypeIdentifier& Type = DataDesc.GetType();
			const FIntVector4 ElementCount = DataDesc.GetElementCount();

			if (Type.IsChildOf(FPCGDataTypeInfoTexture2DBase::AsId()))
			{
				// Render target: sum all mip levels. For 2D textures ElementDimension is Two; for 2D array it is Three with Z = layer count.
				const int64 Layers = (DataDesc.GetElementDimension() == EPCGElementDimension::Three) ? FMath::Max(ElementCount.Z, 1) : 1;
				const EPixelFormat PixelFormat = GetPixelFormatFromPCGRenderTargetFormat(DataDesc.GetRenderTargetFormat());
				const int64 BytesPerTexel = GPixelFormats[PixelFormat].BlockBytes;
				int64 MipWidth = FMath::Max(ElementCount.X, 1);
				int64 MipHeight = FMath::Max(ElementCount.Y, 1);
				for (int32 Mip = 0; Mip < DataDesc.GetTextureNumMips(); ++Mip)
				{
					TotalSizeBytes += static_cast<uint64>(MipWidth * MipHeight * Layers * BytesPerTexel);
					MipWidth  = FMath::Max(MipWidth  / 2, 1LL);
					MipHeight = FMath::Max(MipHeight / 2, 1LL);
				}
			}
			else if (Type.IsChildOf(FPCGDataTypeInfoRawBuffer::AsId()))
			{
				// Raw buffer: element count is the number of uint32 values.
				TotalSizeBytes += static_cast<uint64>(FMath::Max(ElementCount.X, 0)) * sizeof(uint32);
			}
		}

		return TotalSizeBytes;
	}

	int32 GetAttributeIdFromMetadataAttributeIndex(int32 InAttributeIndex)
	{
		return InAttributeIndex >= 0 ? (InAttributeIndex + PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS) : INDEX_NONE;
	}

	int32 GetMetadataAttributeIndexFromAttributeId(int32 InAttributeId)
	{
		return InAttributeId >= PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS ? (InAttributeId - PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS) : INDEX_NONE;
	}

	FString GetPrefixedDataLabel(const FString& InLabel)
	{
		FString PrefixedLabel;
		GetPrefixedDataLabel(InLabel, PrefixedLabel);
		return PrefixedLabel;
	}

	void GetPrefixedDataLabel(const FString& InLabel, FString& OutPrefixedLabel)
	{
		OutPrefixedLabel = FString::Format(TEXT("{0}:{1}"), { PCGComputeConstants::DataLabelTagPrefix, InLabel });
	}

	FString GetDataLabelResolverName(FName InPinLabel)
	{
		return FString::Format(TEXT("{0}_DataResolver"), { *InPinLabel.ToString() });
	}

	EPixelFormat GetPixelFormatFromPCGRenderTargetFormat(EPCGRenderTargetFormat InRenderTargetFormat)
	{
		switch (InRenderTargetFormat)
		{
		// 8 bit
		case EPCGRenderTargetFormat::R8:      return PF_G8;
		case EPCGRenderTargetFormat::RG8:     return PF_R8G8;
		case EPCGRenderTargetFormat::RGBA8:   return PF_B8G8R8A8;

		// 16 bit
		case EPCGRenderTargetFormat::R16f:    return PF_R16F;
		case EPCGRenderTargetFormat::RG16f:   return PF_G16R16F;
		case EPCGRenderTargetFormat::RGBA16f: return PF_FloatRGBA;

		// 32 bit
		case EPCGRenderTargetFormat::R32f:    return PF_R32_FLOAT;
		case EPCGRenderTargetFormat::RG32f:   return PF_G32R32F;
		case EPCGRenderTargetFormat::RGBA32f: return PF_A32B32G32R32F;
		}

		UE_LOGF(LogPCG, Verbose, "No mapping from EPCGRenderTargetFormat to EPixelFormat is possible for entry %u, falling back to EPixelFormat::FloatRGBA.", (uint32)InRenderTargetFormat);

		// Default to FloatRGBA as it is a safe value that won't crash the renderer.
		return PF_FloatRGBA;
	}

	EPCGRenderTargetFormat GetPCGRenderTargetFormatFromPixelFormat(EPixelFormat InPixelFormat)
	{
		switch (InPixelFormat)
		{
		// 8 bit
		case EPixelFormat::PF_R8:             // fallthrough
		case EPixelFormat::PF_G8:             return EPCGRenderTargetFormat::R8;
		case EPixelFormat::PF_R8G8:           return EPCGRenderTargetFormat::RG8;
		case EPixelFormat::PF_B8G8R8A8:       // fallthrough
		case EPixelFormat::PF_R8G8B8A8:       return EPCGRenderTargetFormat::RGBA8;

		// 16 bit
		case EPixelFormat::PF_R16F:           // fallthrough
		case EPixelFormat::PF_G16:            return EPCGRenderTargetFormat::R16f;
		case EPixelFormat::PF_G16R16F:        return EPCGRenderTargetFormat::RG16f;
		case EPixelFormat::PF_FloatRGBA:      // fallthrough
		case EPixelFormat::PF_FloatRGB:       // fallthrough
		case EPixelFormat::PF_FloatR11G11B10: return EPCGRenderTargetFormat::RGBA16f;

		// 32 bit
		case EPixelFormat::PF_R32_FLOAT:      return EPCGRenderTargetFormat::R32f;
		case EPixelFormat::PF_G32R32F:        return EPCGRenderTargetFormat::RG32f;
		case EPixelFormat::PF_A32B32G32R32F:  // fallthrough
		case EPixelFormat::PF_R32G32B32F:     return EPCGRenderTargetFormat::RGBA32f;
		}

		const FPixelFormatInfo& FormatInfo = GPixelFormats[InPixelFormat];

		if (IsFloatFormat(InPixelFormat) || IsHDR(InPixelFormat))
		{
			switch (FormatInfo.NumComponents)
			{
			case 1: return EPCGRenderTargetFormat::R16f;
			case 2: return EPCGRenderTargetFormat::RG16f;
			case 3: // fallthrough
			case 4: return EPCGRenderTargetFormat::RGBA16f;
			}
		}
		else
		{
			switch (FormatInfo.NumComponents)
			{
			case 1: return EPCGRenderTargetFormat::R8;
			case 2: return EPCGRenderTargetFormat::RG8;
			case 3: // fallthrough
			case 4: return EPCGRenderTargetFormat::RGBA8;
			}
		}

		UE_LOGF(LogPCG, Verbose, "No mapping from EPixelFormat to EPCGRenderTargetFormat is possible for %ls, falling back to EPCGRenderTargetFormat::RGBA16f.", *UEnum::GetValueAsString<EPixelFormat>(InPixelFormat));

		return EPCGRenderTargetFormat::RGBA16f;
	}

#if WITH_EDITOR
	void ConvertObjectPathToShaderFilePath(FString& InOutPath)
	{
		// Shader compiler recognizes "/Engine/Generated/..." path as special. 
		// It doesn't validate file suffix etc.
		InOutPath = FString::Printf(TEXT("/Engine/Generated/UObject%s.ush"), *InOutPath);
		// Shader compilation result parsing will break if it finds ':' where it doesn't expect.
		InOutPath.ReplaceCharInline(TEXT(':'), TEXT('@'));
	}

	UPCGComputeDataInterface* CreateOutputPinDataInterface(const FCreateDataInterfaceParams& InParams)
	{
		check(InParams.Context);
		check(InParams.PinProperties);
		check(InParams.ObjectOuter);

		FPCGGPUCompilationContext& Context = *InParams.Context;
		const FPCGPinProperties& PinProperties = *InParams.PinProperties;

		UPCGComputeDataInterface* DataInterface = nullptr;

		if (PCGComputeHelpers::IsTypeAllowedInDataCollection(PinProperties.AllowedTypes))
		{
			UPCGDataCollectionDataInterface* DataInterfacePCGData = Context.NewObject_AnyThread<UPCGDataCollectionDataInterface>(InParams.ObjectOuter);

			// If data comes from a CPU task, upload it to the GPU.
			ensureMsgf(!InParams.bRequiresExport || !InParams.bProducedByCPU, TEXT("Download from GPU only relevant for data produced on GPU."));

			DataInterfacePCGData->SetRequiresExport(InParams.bRequiresExport);

			DataInterfacePCGData->SetElementCountMultiplier(InParams.ProducerKernel ? InParams.ProducerKernel->GetElementCountMultiplier(PinProperties.Label) : 1);
			DataInterfacePCGData->SetRequiresZeroInitialization(InParams.ProducerKernel && InParams.ProducerKernel->DoesOutputPinRequireZeroInitialization(PinProperties.Label));

			DataInterface = DataInterfacePCGData;
		}
		else if (PinProperties.AllowedTypes == EPCGDataType::VirtualTexture)
		{
			DataInterface = Context.NewObject_AnyThread<UPCGVirtualTextureDataInterface>(InParams.ObjectOuter);
		}
		else if (PinProperties.AllowedTypes.Intersects(EPCGDataType::BaseTexture))
		{
			UPCGTextureDataInterface* TextureDataInterface = Context.NewObject_AnyThread<UPCGTextureDataInterface>(InParams.ObjectOuter);
			TextureDataInterface->SetRequiresExport(InParams.bRequiresExport);
			TextureDataInterface->SetInitializeFromDataCollection(InParams.bProducedByCPU);
			TextureDataInterface->SetSingleData(!PinProperties.bAllowMultipleData);

			DataInterface = TextureDataInterface;
		}
		else if (PinProperties.AllowedTypes.IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId()))
		{
			UPCGTexture2DArrayDataInterface* TextureDataInterface = Context.NewObject_AnyThread<UPCGTexture2DArrayDataInterface>(InParams.ObjectOuter);
			TextureDataInterface->SetRequiresExport(InParams.bRequiresExport);
			TextureDataInterface->SetInitializeFromDataCollection(InParams.bProducedByCPU);

			DataInterface = TextureDataInterface;
		}
		else if (PinProperties.AllowedTypes == EPCGDataType::Landscape)
		{
			DataInterface = Context.NewObject_AnyThread<UPCGLandscapeDataInterface>(InParams.ObjectOuter);
		}
		else if (PinProperties.AllowedTypes == EPCGDataType::StaticMeshResource)
		{
			DataInterface = Context.NewObject_AnyThread<UPCGStaticMeshDataInterface>(InParams.ObjectOuter);
		}
		else if (PinProperties.AllowedTypes == FPCGDataTypeInfoRawBuffer::AsId())
		{
			UPCGRawBufferDataInterface* DataInterfaceRawBuffer = Context.NewObject_AnyThread<UPCGRawBufferDataInterface>(InParams.ObjectOuter);

			DataInterfaceRawBuffer->SetRequiresExport(InParams.bRequiresExport);
			DataInterfaceRawBuffer->SetRequiresZeroInitialization(InParams.ProducerKernel && InParams.ProducerKernel->DoesOutputPinRequireZeroInitialization(PinProperties.Label));

			// todo_pcg: We should support SetElementCountMultiplier also.

			DataInterface = DataInterfaceRawBuffer;
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Unsupported connected upstream pin '%ls' on node '%ls' with type %ls. Consider adding a conversion to a supported type such as Point.",
				*PinProperties.Label.ToString(),
				InParams.NodeForDebug ? *InParams.NodeForDebug->GetNodeTitle(EPCGNodeTitleType::ListView).ToString() : TEXT("MISSING"),
				*PinProperties.AllowedTypes.ToString()
			);
		}

		return DataInterface;
	}

	static void NotifyHelper(UPCGDataBinding* InBinding, const UPCGComputeKernel* InKernel, const UPCGSettings* InSettings, const TFunction<void(FPCGContext*, const UPCGNode*)>& InLambda)
	{
		if (!ensure(InBinding))
		{
			return;
		}

		TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
		if (FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr)
		{
			const UPCGNode* IndicatorNode = nullptr;

			if (InKernel)
			{
				IndicatorNode = InBinding->GetComputeGraph() ? InBinding->GetComputeGraph()->GetKernelNode(InKernel) : nullptr;
			}
			else if (InSettings)
			{
				IndicatorNode = Cast<UPCGNode>(InSettings->GetOuter());
			}

			if (IndicatorNode)
			{
				InLambda(Context, IndicatorNode);
			}
		}
	}

	void NotifyGPUToCPUReadback(UPCGDataBinding* InBinding, const UPCGComputeKernel* InKernel, const UPCGSettings* InSettings)
	{
		NotifyHelper(InBinding, InKernel, InSettings, [](FPCGContext* InContext, const UPCGNode* InNode)
		{
			InContext->ExecutionSource->GetExecutionState().GetInspection().NotifyGPUToCPUReadback(InNode, InContext->GetStack());
		});
	}

	void NotifyCPUToGPUUpload(UPCGDataBinding* InBinding, const UPCGComputeKernel* InKernel, const UPCGSettings* InSettings)
	{
		NotifyHelper(InBinding, InKernel, InSettings, [](FPCGContext* InContext, const UPCGNode* InNode)
		{
			InContext->ExecutionSource->GetExecutionState().GetInspection().NotifyCPUToGPUUpload(InNode, InContext->GetStack());
		});
	}

	FString SanitizePinLabelForHLSL(FName InLabel)
	{
		FString Result = InLabel.ToString();

		// Replace invalid characters in-place.
		for (TCHAR& Ch : Result)
		{
			if (!FChar::IsAlpha(Ch) && !FChar::IsDigit(Ch) && Ch != TEXT('_'))
			{
				Ch = TEXT('_');
			}
		}

		// Trim leading non-alpha characters — our code generation and parsing requires identifiers that start with a letter.
		int32 FirstAlpha = 0;
		while (FirstAlpha < Result.Len() && !FChar::IsAlpha(Result[FirstAlpha]))
		{
			++FirstAlpha;
		}

		if (FirstAlpha > 0)
		{
			Result.RightChopInline(FirstAlpha);
		}

		return Result;
	}

	bool IsValidHLSLPinLabel(FName InLabel)
	{
		return SanitizePinLabelForHLSL(InLabel) == InLabel.ToString();
	}

	namespace SRVUAVTemplate
	{
		// Suffixes used for {NameSuffix} template arg in ExpandShaderTemplateForSRVUAV.
		static const TCHAR* SRVNameSuffix = TEXT("PCG_SRV");
		static const TCHAR* UAVNameSuffix = TEXT("PCG_UAV");

		// Suffixes used for {BufferSuffix} template arg in ExpandShaderTemplateForSRVUAV.
		static const TCHAR* SRVBufferSuffix = TEXT("SRV");
		static const TCHAR* UAVBufferSuffix = TEXT("UAV");

		struct FVariant
		{
			const TCHAR* NameSuffix = nullptr;
			const TCHAR* BufferSuffix = nullptr;
		};

		// Walks the template, processing each <Begin>...<End> region in turn. Each region is emitted once per entry in InVariants, with {NameSuffix} / {BufferSuffix}
		// substituted accordingly. Content outside regions is appended unchanged. Multiple regions per template are supported.
		static FString ExpandDirective(const FString& InTemplate, const FString& InBeginMarker, const FString& InEndMarker, TArrayView<const FVariant> InVariants)
		{
			auto FormatVariant = [](const FString& InRegion, const TCHAR* InNameSuffix, const TCHAR* InBufferSuffix) -> FString
			{
				TMap<FString, FStringFormatArg> Args;
				Args.Add(TEXT("NameSuffix"),   FStringFormatArg(FString(InNameSuffix)));
				Args.Add(TEXT("BufferSuffix"), FStringFormatArg(FString(InBufferSuffix)));
				return FString::Format(*InRegion, Args);
			};

			FString Result;
			int32 SearchStart = 0;

			while (true)
			{
				const int32 BeginIdx = InTemplate.Find(InBeginMarker, ESearchCase::CaseSensitive, ESearchDir::FromStart, SearchStart);
				if (BeginIdx == INDEX_NONE)
				{
					// No more regions. Append the rest of the template and exit.
					Result += InTemplate.Mid(SearchStart);
					break;
				}

				const int32 EndIdx = InTemplate.Find(InEndMarker, ESearchCase::CaseSensitive, ESearchDir::FromStart, BeginIdx + InBeginMarker.Len());
				if (!ensureMsgf(EndIdx != INDEX_NONE,
					TEXT("Shader template has %s without matching %s; emitting remainder unchanged."), *InBeginMarker, *InEndMarker))
				{
					Result += InTemplate.Mid(SearchStart);
					break;
				}

				// Append text before the Begin marker (single-emission).
				Result += InTemplate.Mid(SearchStart, BeginIdx - SearchStart);

				// Extract and emit the region between markers once per variant.
				const int32 RegionStart = BeginIdx + InBeginMarker.Len();
				const FString Region = InTemplate.Mid(RegionStart, EndIdx - RegionStart);
				for (const FVariant& Variant : InVariants)
				{
					Result += FormatVariant(Region, Variant.NameSuffix, Variant.BufferSuffix);
				}

				// Advance past the End marker and look for more regions.
				SearchStart = EndIdx + InEndMarker.Len();
			}

			return Result;
		}
	}

	FString ExpandShaderTemplateForSRVUAV(const FString& InTemplate)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGComputeHelpers::ExpandShaderTemplateForSRVUAV);

		using namespace SRVUAVTemplate;

		// Pass 1: {{BeginDuplicateForSRVUAV}}...{{EndDuplicateForSRVUAV}} -> emit each region twice (SRV then UAV).
		const FVariant DuplicateVariants[] =
		{
			{ SRVNameSuffix, SRVBufferSuffix },
			{ UAVNameSuffix, UAVBufferSuffix },
		};
		const FString AfterDuplicate = ExpandDirective(InTemplate, TEXT("{{BeginDuplicateForSRVUAV}}"), TEXT("{{EndDuplicateForSRVUAV}}"), DuplicateVariants);

		// Pass 2: {{BeginUAVOnly}}...{{EndUAVOnly}} -> emit each region once with UAV suffixes substituted. Lets UAV-only function bodies (setters, atomics)
		// reference duplicated readers via {NameSuffix} and the buffer via {BufferSuffix}, so the literal "PCG_UAV" / "UAV" tokens stay out of .ush sources.
		const FVariant UAVOnlyVariants[] =
		{
			{ UAVNameSuffix, UAVBufferSuffix },
		};
		return ExpandDirective(AfterDuplicate, TEXT("{{BeginUAVOnly}}"), TEXT("{{EndUAVOnly}}"), UAVOnlyVariants);
	}

	FString GetSRVFunction(const TCHAR* InBaseName)
	{
		return FString::Printf(TEXT("%s_%s"), InBaseName, SRVUAVTemplate::SRVNameSuffix);
	}

	FString GetUAVFunction(const TCHAR* InBaseName)
	{
		return FString::Printf(TEXT("%s_%s"), InBaseName, SRVUAVTemplate::UAVNameSuffix);
	}

	void StripSRVUAVSuffix(FString& InOutName)
	{
		InOutName.RemoveFromEnd(FString::Printf(TEXT("_%s"), SRVUAVTemplate::SRVNameSuffix));
		InOutName.RemoveFromEnd(FString::Printf(TEXT("_%s"), SRVUAVTemplate::UAVNameSuffix));
	}
#endif
}

namespace PCGComputeDummies
{
	class FPCGEmptyBufferSRV : public FRenderResource
	{
	public:
		FPCGEmptyBufferSRV(EPixelFormat InPixelFormat, const FString& InDebugName)
			: PixelFormat(InPixelFormat)
			, DebugName(InDebugName)
		{}

		EPixelFormat PixelFormat;
		FString DebugName;
		FBufferRHIRef Buffer = nullptr;
		FShaderResourceViewRHIRef BufferSRV = nullptr;
	
		virtual void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			// Create a buffer with one element.
			const uint32 NumBytes = GPixelFormats[PixelFormat].BlockBytes;

			FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(*DebugName, NumBytes)
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Static)
				.DetermineInitialState();

			Buffer = RHICmdList.CreateBuffer(CreateDesc.SetInitActionZeroData());
			BufferSRV = RHICmdList.CreateShaderResourceView(
				Buffer,
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PixelFormat));
		}
	
		virtual void ReleaseRHI() override
		{
			BufferSRV.SafeRelease();
			Buffer.SafeRelease();
		}
	};
	
	FRHIShaderResourceView* GetDummyFloatBuffer()
	{
		static TGlobalResource<FPCGEmptyBufferSRV> DummyFloatBuffer(PF_R32_FLOAT, TEXT("PCGDummyFloat"));
		return DummyFloatBuffer.BufferSRV;
	}
	
	FRHIShaderResourceView* GetDummyFloat2Buffer()
	{
		static TGlobalResource<FPCGEmptyBufferSRV> DummyFloat2Buffer(PF_G32R32F, TEXT("PCGDummyFloat2"));
		return DummyFloat2Buffer.BufferSRV;
	}
	
	FRHIShaderResourceView* GetDummyFloat4Buffer()
	{
		static TGlobalResource<FPCGEmptyBufferSRV> DummyFloat4Buffer(PF_A32B32G32R32F, TEXT("PCGDummyFloat4"));
		return DummyFloat4Buffer.BufferSRV;
	}
}

uint32 GetTypeHash(const FPCGPinReference& In)
{
	return HashCombine(/*GetTypeHash(In.TaskId),*/ PointerHash(In.Kernel), GetTypeHash(In.Label));
}

uint32 GetTypeHash(const FPCGKernelPin& In)
{
	return HashCombine(GetTypeHash(In.KernelIndex), GetTypeHash(In.PinLabel), GetTypeHash(In.bIsInput));
}

