// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGComputeGraph.h"

#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "Compute/PCGCompilerDiagnostic.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/Packing/PCGDataCollectionPacking.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Data/PCGTextureData.h"
#include "Metadata/PCGMetadata.h"

#include "ComputeWorkerInterface.h"
#include "RenderGraphResources.h"
#include "ComputeFramework/ComputeKernelCompileResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGComputeGraph)

#define PCG_DATA_DESCRIPTION_LOGGING 0

static FPCGCompilerDiagnostic ProcessCompilationMessage(const FComputeKernelCompileMessage& InMessage)
{
	FPCGCompilerDiagnostic Diagnostic;

	if (InMessage.Type == FComputeKernelCompileMessage::EMessageType::Error)
	{
		Diagnostic.Level = EPCGDiagnosticLevel::Error;
	}
	else if (InMessage.Type == FComputeKernelCompileMessage::EMessageType::Warning)
	{
		Diagnostic.Level = EPCGDiagnosticLevel::Warning;
	}
	else if (InMessage.Type == FComputeKernelCompileMessage::EMessageType::Info)
	{
		Diagnostic.Level = EPCGDiagnosticLevel::Info;
	}

	Diagnostic.Line = InMessage.Line;
	Diagnostic.ColumnStart = InMessage.ColumnStart;
	Diagnostic.ColumnEnd = InMessage.ColumnEnd;

	FString Message;
	if (!InMessage.VirtualFilePath.IsEmpty())
	{
		Message = InMessage.VirtualFilePath;
		if (InMessage.Line != -1)
		{
			if (InMessage.ColumnStart == InMessage.ColumnEnd)
			{
				Message += FString::Printf(TEXT(" (%d,%d)"), InMessage.Line, InMessage.ColumnStart);
			}
			else
			{
				Message += FString::Printf(TEXT(" (%d,%d-%d)"), InMessage.Line, InMessage.ColumnStart, InMessage.ColumnEnd);
			}
		}
		Message += TEXT(": ");
	}
	Message += InMessage.Text;
	Diagnostic.Message = FText::FromString(Message);

	return Diagnostic;
}

void UPCGComputeGraph::OnKernelCompilationComplete(int32 InKernelIndex, FComputeKernelCompileResults const& InCompileResults)
{
	const UPCGNode* Node = (KernelToNode.IsValidIndex(InKernelIndex) && KernelToNode[InKernelIndex].IsValid()) ? KernelToNode[InKernelIndex].Get() : nullptr;

	if (Node)
	{
		KernelToCompileMessages.FindOrAdd(KernelToNode[InKernelIndex].Get()) = InCompileResults.Messages;

		UPCGGraph* Graph = Cast<UPCGGraph>(Node->GetOuter());
		if (Graph)
		{
			FPCGCompilerDiagnostics Diagnostics;
			Diagnostics.Diagnostics.Reserve(InCompileResults.Messages.Num());

			Algo::Transform(InCompileResults.Messages, Diagnostics.Diagnostics, [](const FComputeKernelCompileMessage& InMessage)
			{
				return ProcessCompilationMessage(InMessage);
			});

#if WITH_EDITOR
			Graph->OnNodeSourceCompiledDelegate.Broadcast(Node, Diagnostics);
#endif
		}
	}
	else
	{
		// We may in general have kernels with no corresponding node.
		UE_LOGF(LogPCG, Verbose, "Compilation message ignored for kernel index %d which has no associated node.", InKernelIndex);
	}
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGComputeGraph::ComputeKernelBindingDataDesc(int32 InBindingIndex, UPCGDataBinding* InBinding) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGComputeGraph::ComputeKernelBindingDataDesc);

	const FComputeGraphEdge& GraphEdge = GraphEdges[InBindingIndex];

#if PCG_DATA_DESCRIPTION_LOGGING
	UE_LOGF(LogPCG, Warning, "Computing data desc for kernel %d, DI '%ls', edge index %d (function '%ls'), %ls",
		GraphEdge.KernelIndex,
		*DataInterfaces[GraphEdge.DataInterfaceIndex]->GetName(),
		InBindingIndex,
		*GraphEdge.BindingFunctionNameOverride,
		GraphEdge.bKernelInput ? TEXT("INPUT") : TEXT("OUTPUT"));
#endif

	// Find out if this data is coming from CPU (either from CPU nodes or from a separate compute graph).
	if (const FName* VirtualLabel = CPUDataBindingToVirtualPinLabel.Find(InBindingIndex))
	{
		// Create description from compute graph element input data.
		return ComputeExternalPinDesc(*VirtualLabel, InBinding);
	}
	else if (GraphEdge.bKernelInput)
	{
		const int32* UpstreamBindingIndex = DownstreamToUpstreamBinding.Find(InBindingIndex);
		return ensure(UpstreamBindingIndex) ? InBinding->ComputeKernelBindingDataDesc(*UpstreamBindingIndex) : nullptr;
	}
	else
	{
		// Compute description of data from output binding. Data doesn't come from CPU but via a kernel. Consult the kernel for its data description.
		check(KernelInvocations.IsValidIndex(GraphEdge.KernelIndex));

		const UPCGComputeKernel* Kernel = CastChecked<UPCGComputeKernel>(KernelInvocations[GraphEdge.KernelIndex]);
		const FName* OutputPinLabel = KernelBindingToPinLabel.Find(InBindingIndex);

		return ensure(OutputPinLabel) ? Kernel->ComputeOutputBindingDataDesc(*OutputPinLabel, InBinding) : nullptr;
	}
}

int32 UPCGComputeGraph::GetBindingIndex(const FPCGKernelPin& InKernelPin) const
{
	// Bindings in compute graphs roughly map to a single API on a Data Interface, like GetNumData(). There can be multiple bindings per PCG edge.
	// We choose the first binding index to represent the kernel pin.
	if (const int32* BindingIndex = KernelPinToFirstBinding.Find(InKernelPin))
	{
		return *BindingIndex;
	}
	else
	{
		UE_LOGF(LogPCG, Warning, "Kernel pin not registered in compute graph. May be due to unsupported pin type. KernelIndex=%d, PinLabel='%ls', Input=%d",
			InKernelPin.KernelIndex,
			*InKernelPin.PinLabel.ToString(),
			InKernelPin.bIsInput);

		return INDEX_NONE;
	}
}

void UPCGComputeGraph::GetKernelPins(TArray<FPCGKernelPin>& OutKernelPins) const
{
	KernelPinToFirstBinding.GenerateKeyArray(OutKernelPins);
}

int32 UPCGComputeGraph::GetNumKernels() const
{
	return KernelInvocations.Num();
}

const UComputeKernel* UPCGComputeGraph::GetKernel(int32 InKernelIndex) const
{
	return KernelInvocations.IsValidIndex(InKernelIndex) ? KernelInvocations[InKernelIndex] : nullptr;
}

bool UPCGComputeGraph::AreGraphSettingsValid(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGComputeGraph::AreGraphSettingsValid);

	bool bAllKernelsValid = true;

	// Run validation on all kernels.
	for (const UComputeKernel* Kernel : KernelInvocations)
	{
		bAllKernelsValid &= CastChecked<UPCGComputeKernel>(Kernel)->AreKernelSettingsValid(InContext);
	}

	return bAllKernelsValid;
}

bool UPCGComputeGraph::IsGraphDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGComputeGraph::IsGraphDataValid);

	bool bAllKernelsValid = true;

	// Run validation on all kernels.
	for (const UComputeKernel* Kernel : KernelInvocations)
	{
		bAllKernelsValid &= CastChecked<UPCGComputeKernel>(Kernel)->IsKernelDataValid(InDataBinding, InContext);
	}

	return bAllKernelsValid;
}

void UPCGComputeGraph::DebugLogDataDescriptions(const UPCGDataBinding* InBinding) const
{
#if PCG_KERNEL_LOGGING_ENABLED
	const UEnum* PCGKernelAttributeTypeEnum = StaticEnum<EPCGKernelAttributeType>();
	check(PCGKernelAttributeTypeEnum);

	auto LogDataDescription = [this, PCGKernelAttributeTypeEnum, InBinding](const TSharedPtr<const FPCGDataCollectionDesc> InDataDesc, const FPCGKernelPin& InKernelPin)
	{
		if (!InDataDesc)
		{
			return;
		}

		for (int32 DataIndex = 0; DataIndex < InDataDesc->GetDataDescriptions().Num(); ++DataIndex)
		{
			const FPCGDataDesc& DataDesc = InDataDesc->GetDataDescriptions()[DataIndex];
			const FPCGDataTypeIdentifier& Type = DataDesc.GetType();

			UE_LOGF(LogPCG, Display, "\t\tData Index %d", DataIndex);
			UE_LOGF(LogPCG, Display, "\t\t\tType: %ls", *Type.ToString());
			UE_LOGF(LogPCG, Display, "\t\t\tNum Elements: %ls (%ls)", *DataDesc.GetElementCount().ToString(), *UEnum::GetValueAsString(DataDesc.GetElementDimension()));

			if (Type.IsChildOf(FPCGDataTypeInfoTexture2DSingleBase::AsId()) || Type.IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId()))
			{
				UE_LOGF(LogPCG, Display, "\t\t\tFormat: %ls", *UEnum::GetValueAsString(DataDesc.GetRenderTargetFormat()));
				UE_LOGF(LogPCG, Display, "\t\t\tTransform: %ls", *DataDesc.GetTextureTransform().ToString());
				UE_LOGF(LogPCG, Display, "\t\t\tFilter: %ls", *UEnum::GetValueAsString(DataDesc.GetTextureFilter()));
				UE_LOGF(LogPCG, Display, "\t\t\tNumMips: %d", DataDesc.GetTextureNumMips());
			}
			else
			{
				UE_LOGF(LogPCG, Display, "\t\t\tAttributes (%d)", DataDesc.GetAttributeDescriptions().Num());

				TSharedPtr<const FPCGProxyForGPUDataCollection> UpstreamDataCollectionInfo;

				if (const FName* VirtualLabel = GetVirtualPinLabelFromKernelInputPin(InKernelPin))
				{
					TArray<FPCGTaggedData> PinInputs = InBinding->GetInputDataCollection().GetInputsByPin(*VirtualLabel);

					if (ensure(PinInputs.IsValidIndex(DataIndex)))
					{
						if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(PinInputs[DataIndex].Data))
						{
							UpstreamDataCollectionInfo = Proxy->GetGPUInfo();
						}
					}
				}

				for (const FPCGKernelAttributeDesc& AttributeDesc : DataDesc.GetAttributeDescriptions())
				{
					const bool bAllocated = DataDesc.IsAttributeAllocated(AttributeDesc);

					// If the data comes from an upstream / source buffer then look up its info.
					FString UniqueStringKeys;
					for (int32 GraphStringKey : AttributeDesc.GetUniqueStringKeys())
					{
						UniqueStringKeys += TEXT(", ");
						UniqueStringKeys += FString::Format(TEXT("{0}"), { GraphStringKey });

						if (const int32* UpstreamStringKey = UpstreamDataCollectionInfo ? InBinding->GetGraphToSourceBufferStringKey().Find({ UpstreamDataCollectionInfo->GetBuffer(), GraphStringKey }) : nullptr)
						{
							UniqueStringKeys += FString::Format(TEXT(" ({0})"), { *UpstreamStringKey });
						}
					}

					// If attribute comes from an upstream GPU buffer, display the attribute ID in the upstream buffer.
					int32 SourceBufferAttributeId = INDEX_NONE;

					if (AttributeDesc.GetAttributeId() >= PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS && UpstreamDataCollectionInfo)
					{
						const int32 GraphAttributeIndex = PCGComputeHelpers::GetMetadataAttributeIndexFromAttributeId(AttributeDesc.GetAttributeId());
						const int32* FoundIndex = InBinding->GetGraphAttributeToSourceBufferAttributeIndex().Find({ GraphAttributeIndex, UpstreamDataCollectionInfo->GetBuffer() });

						SourceBufferAttributeId = PCGComputeHelpers::GetAttributeIdFromMetadataAttributeIndex(FoundIndex ? *FoundIndex : GraphAttributeIndex);
					}

					if (SourceBufferAttributeId == INDEX_NONE)
					{
						UE_LOGF(LogPCG, Display, "\t\t\t\tID: %d\t\tName: %ls\t\tType: %ls\t\tAllocated: %ls\t\tUniqueStringKeys%ls",
							AttributeDesc.GetAttributeId(),
							*AttributeDesc.GetAttributeKey().GetIdentifier().ToString(),
							*PCGKernelAttributeTypeEnum->GetDisplayNameTextByValue(static_cast<int64>(AttributeDesc.GetAttributeKey().GetType())).ToString(),
							bAllocated ? TEXT("true") : TEXT("false"),
							UniqueStringKeys.IsEmpty() ? TEXT(": ") : *UniqueStringKeys);
					}
					else
					{
						UE_LOGF(LogPCG, Display, "\t\t\t\tID: %d\t\tName: %ls\t\tType: %ls\t\tAllocated: %ls\t\tUniqueStringKeys%ls\t\tSource Buffer ID: %d",
							AttributeDesc.GetAttributeId(),
							*AttributeDesc.GetAttributeKey().GetIdentifier().ToString(),
							*PCGKernelAttributeTypeEnum->GetDisplayNameTextByValue(static_cast<int64>(AttributeDesc.GetAttributeKey().GetType())).ToString(),
							bAllocated ? TEXT("true") : TEXT("false"),
							UniqueStringKeys.IsEmpty() ? TEXT(": ") : *UniqueStringKeys,
							SourceBufferAttributeId);
					}
				}

				UE_LOGF(LogPCG, Display, "\t\t\tTags (%d)", DataDesc.GetTagStringKeys().Num());

				FString TagStringKeys;
				for (int32 TagStringKey : DataDesc.GetTagStringKeys())
				{
					TagStringKeys += TEXT(", ");
					TagStringKeys += FString::Format(TEXT("{0}"), { TagStringKey });
				}

				UE_LOGF(LogPCG, Display, "\t\t\tTag String Keys%ls", *TagStringKeys);
			}
		}
	};

	for (int32 KernelIndex = 0; KernelIndex < KernelInvocations.Num(); ++KernelIndex)
	{
		const UPCGComputeKernel* PCGKernel = Cast<UPCGComputeKernel>(KernelInvocations[KernelIndex]);

		if (PCGKernel && PCGKernel->GetLogDataDescriptions())
		{
			const UPCGSettings* Settings = PCGKernel->GetSettings();

			UE_LOGF(LogPCG, Display, "Settings: %ls", Settings ? *Settings->GetName() : TEXT("MISSINGSETTINGS"));
			UE_LOGF(LogPCG, Display, "Kernel: %ls", *PCGKernel->GetName());

			TArray<FPCGPinProperties> InputPins;
			PCGKernel->GetInputPinsAndOverridablePins(InputPins);
			for (const FPCGPinProperties& Pin : InputPins)
			{
				if (Pin.Label != NAME_None)
				{
					if (const TSharedPtr<const FPCGDataCollectionDesc> DataDesc = InBinding->GetCachedKernelPinDataDesc(PCGKernel, Pin.Label, true))
					{
						UE_LOGF(LogPCG, Display, "\tInput Pin: %ls (%d data)", *Pin.Label.ToString(), DataDesc->GetDataDescriptions().Num());

						LogDataDescription(DataDesc, FPCGKernelPin(KernelIndex, Pin.Label, true));
					}
				}
			}

			TArray<FPCGPinPropertiesGPU> OutputPins;
			PCGKernel->GetOutputPins(OutputPins);
			for (const FPCGPinPropertiesGPU& Pin : OutputPins)
			{
				if (Pin.Label != NAME_None)
				{
					if (const TSharedPtr<const FPCGDataCollectionDesc> DataDesc = InBinding->GetCachedKernelPinDataDesc(PCGKernel, Pin.Label, false))
					{
						UE_LOGF(LogPCG, Display, "\tOutput Pin: %ls (%d data)", *Pin.Label.ToString(), DataDesc->GetDataDescriptions().Num());

						LogDataDescription(DataDesc, FPCGKernelPin(KernelIndex, Pin.Label, false));
					}
				}
			}
		}
	}
#endif
}

const UPCGNode* UPCGComputeGraph::GetKernelNode(const UComputeKernel* InKernel) const
{
	const int FoundIndex = KernelInvocations.IndexOfByPredicate([InKernel](const TObjectPtr<UComputeKernel>& Kernel) { return Kernel.Get() == InKernel; });
	return (FoundIndex != INDEX_NONE) ? KernelToNode[FoundIndex].Get() : nullptr;
}

#ifdef PCG_GPU_KERNEL_PROFILING
bool UPCGComputeGraph::ShouldRepeatDispatch() const
{
	for (const UComputeKernel* Kernel : KernelInvocations)
	{
		const UPCGComputeKernel* PCGKernel = Cast<UPCGComputeKernel>(Kernel);
		if (PCGKernel && PCGKernel->ShouldRepeatDispatch())
		{
			return true;
		}
	}

	return false;
}
#endif // PCG_GPU_KERNEL_PROFILING

FName UPCGComputeGraph::GetRequiredExecutionGroup() const
{
	for (const TObjectPtr<UComputeKernel>& Kernel : KernelInvocations)
	{
		if (const UPCGComputeKernel* PCGKernel = Cast<UPCGComputeKernel>(Kernel))
		{
			if (PCGKernel->RequiresPostTLASBuildExecutionGroup())
			{
				return ComputeTaskExecutionGroup::PostTLASBuild;
			}
		}
	}

	return ComputeTaskExecutionGroup::EndOfFrameUpdate;
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGComputeGraph::ComputeExternalPinDesc(FName InVirtualLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeShared();

	const TArray<FPCGTaggedData> TaggedDatas = InBinding->GetInputDataCollection().GetInputsByPin(InVirtualLabel);

	for (const FPCGTaggedData& TaggedData : TaggedDatas)
	{
		if (!TaggedData.Data || !PCGComputeHelpers::IsTypeAllowedAsInput(TaggedData.Data->GetDataTypeId()))
		{
			continue;
		}

		FPCGDataDesc DataDesc;
		if (ensure(ComputeTaggedDataPinDesc(TaggedData, InBinding, DataDesc)))
		{
			OutDataDesc->GetDataDescriptionsMutable().Add(MoveTemp(DataDesc));
		}
	}

	return OutDataDesc;
}

bool UPCGComputeGraph::ComputeTaggedDataPinDesc(const FPCGTaggedData& InTaggedData, const UPCGDataBinding* InBinding, FPCGDataDesc& OutDescription) const
{
	if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(InTaggedData.Data))
	{
		if (!Proxy->GetDescription(OutDescription))
		{
			return false;
		}

		if (TSharedPtr<const FPCGProxyForGPUDataCollection> DataGPU = Proxy->GetGPUInfo())
		{
			// Remap attribute IDs and string keys.
			for (FPCGKernelAttributeDesc& AttrDesc : OutDescription.GetAttributeDescriptionsMutable())
			{
				const int32 Index = PCGComputeHelpers::GetMetadataAttributeIndexFromAttributeId(AttrDesc.GetAttributeId());
				if (Index == INDEX_NONE)
				{
					continue;
				}

				// Map from the attribute IDs in the buffer / upstream compute graph to attribute IDs usable in the current graph.
				if (const int32* AttributeIndex = InBinding->GetSourceBufferAttributeToGraphAttributeIndex().Find(UPCGDataBinding::FSourceBufferAndAttributeIndex(DataGPU, Index)))
				{
					const int32 NewId = PCGComputeHelpers::GetAttributeIdFromMetadataAttributeIndex(*AttributeIndex);
					//UE_LOGF(LogPCG, Warning, "Remapped attribute '%ls' ID: %d -> %d", *AttrDesc.Name.ToString(), AttrDesc.Index, NewId);
					AttrDesc.SetAttributeId(NewId);
				}

				// Map from the attribute IDs in the buffer / upstream compute graph to attribute IDs usable in the current graph.
				if (!InBinding->GetSourceBufferToGraphStringKey().IsEmpty())
				{
					TArray<int32> GraphStringKeys;
					GraphStringKeys.Reserve(AttrDesc.GetUniqueStringKeys().Num());
					bool bStringKeyRemapped = false;

					for (int32 SourceStringKey : AttrDesc.GetUniqueStringKeys())
					{
						if (const int32* GraphStringKey = InBinding->GetSourceBufferToGraphStringKey().Find({ DataGPU->GetBuffer(), SourceStringKey }))
						{
							//UE_LOGF(LogPCG, Warning, "Remapped string key: %d -> %d", SourceStringKey, *GraphStringKey);
							GraphStringKeys.AddUnique(*GraphStringKey);
							bStringKeyRemapped = true;
						}
						else
						{
							GraphStringKeys.AddUnique(SourceStringKey);
						}
					}

					if (bStringKeyRemapped)
					{
						AttrDesc.SetStringKeys(GraphStringKeys);
					}
				}
			}
		}

		for (const FString& Tag : InTaggedData.Tags)
		{
			const int32 Key = InBinding->GetStringTable().IndexOfByKey(Tag);
			if (ensure(Key != INDEX_NONE))
			{
				OutDescription.AddUniqueTagStringKey(Key);
			}
		}
	}
	else
	{
		OutDescription = FPCGDataDesc(InTaggedData, InBinding);
	}

	return true;
}

#undef PCG_DATA_DESCRIPTION_LOGGING 
