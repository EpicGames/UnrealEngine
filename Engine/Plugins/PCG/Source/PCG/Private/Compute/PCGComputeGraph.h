// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGDataDescription.h"

#include "ComputeFramework/ComputeGraph.h"
#include "ComputeFramework/ComputeKernel.h"
#include "ComputeFramework/ComputeKernelCompileResult.h"

#include "UObject/ObjectKey.h"

#include "PCGComputeGraph.generated.h"

class UPCGComputeKernel;
class UPCGDataBinding;
class UPCGNode;
class UPCGPin;
class UPCGStaticMeshSpawnerKernel;

UCLASS()
class UPCGComputeGraph : public UComputeGraph
{
	GENERATED_BODY()

public:
	//~Begin UComputeGraph interface
	void OnKernelCompilationComplete(int32 InKernelIndex, FComputeKernelCompileResults const& InCompileResults) override;
	//~End UComputeGraph interface

	/** Get the attribute table collated statically during compilation. */
	const FPCGKernelAttributeTable& GetStaticAttributeTable() const { return StaticAttributeTable; }

	/** Get the data labels collated statically during compilation. */
	const TMap</*KernelIndex*/int32, FPCGPinDataLabels>& GetStaticDataLabelsTable() const { return StaticDataLabelsTable; }

	/** Get the strings collated statically during compilation. */
	const TArray<FString>& GetStaticStringTable() const { return StringTable; }

	/**
	 * Computes a description of the data produced/consumed by a binding in the graph.
	 *
	 * In compute graph, a binding is a single function that is exposed to a kernel.
	 * In PCG, a pin is a data interface (multiple incident edges on an input pin will be merged using a DI).
	 *
	 * A pin will have many bindings, but we arbitrarily use the first binding index to uniquely represent a pin.
	 */
	TSharedPtr<const FPCGDataCollectionDesc> ComputeKernelBindingDataDesc(int32 InBindingIndex, UPCGDataBinding* InBinding) const;

	/** Get the first binding index, which uniquely identifies a kernel pin to the compute graph. */
	int32 GetBindingIndex(const FPCGKernelPin& InKernelPin) const;

	/** Get the virtual label associated with a kernel pin. This only exists if the kernel pin is an input pin on the border of the compute graph. */
	const FName* GetVirtualPinLabelFromKernelInputPin(const FPCGKernelPin& InKernelPin) const { return KernelPinToVirtualPinLabel.Find(InKernelPin); }

	/** Get the PCG data type associated with the virtual input pin label. */
	const FPCGDataTypeIdentifier* GetVirtualInputPinType(FName InVirtualInputPinLabel) const { return VirtualInputPinToType.Find(InVirtualInputPinLabel); }

	/** Populates an array with all of the kernel pins in the compute graph. */
	void GetKernelPins(TArray<FPCGKernelPin>& OutKernelPins) const;

	int32 GetNumKernels() const;
	const UComputeKernel* GetKernel(int32 InKernelIndex) const;

	/** Validates and logs information about the graph known at compile time. */
	bool AreGraphSettingsValid(FPCGContext* InContext) const;

	/** Validates and logs information about incoming data to the compute graph. */
	bool IsGraphDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext = nullptr) const;

	/** Logs detailed data descriptions for all the kernels in the compute graph. */
	void DebugLogDataDescriptions(const UPCGDataBinding* InBinding) const;

	const UPCGNode* GetKernelNode(const UComputeKernel* InKernel) const;

#ifdef PCG_GPU_KERNEL_PROFILING
	/** Whether to repeatedly dispatch every frame. Helpful for profiling. */
	bool ShouldRepeatDispatch() const;
#endif

	/** Returns the execution group required by this compute graph based on its kernels' requirements. */
	FName GetRequiredExecutionGroup() const;

private:
	/** Compute the data description on a pin external to the compute graph. */
	TSharedPtr<const FPCGDataCollectionDesc> ComputeExternalPinDesc(FName InVirtualLabel, UPCGDataBinding* InBinding) const;

	/** Compute the data description of a tagged data. */
	bool ComputeTaggedDataPinDesc(const FPCGTaggedData& InTaggedData, const UPCGDataBinding* InBinding, FPCGDataDesc& OutDescription) const;

public:
	TMap<TObjectKey<const UPCGNode>, TArray<FComputeKernelCompileMessage>> KernelToCompileMessages;
	
	/** Node corresponding to each kernel, useful for compilation feedback. */
	UPROPERTY()
	TArray<TSoftObjectPtr<const UPCGNode>> KernelToNode;

	/** List of settings for all nodes that spawn static meshes, so we can do required primitive & DI setup when compute graph element executes. */
	UPROPERTY()
	TArray<TObjectPtr<const UPCGStaticMeshSpawnerKernel>> StaticMeshSpawners;

	UPROPERTY()
	bool bLogDataDescriptions = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	bool bBreakDebugger = false;
#endif

protected:
	UPROPERTY()
	TMap</*BindingIndex*/int32, /*PinLabel*/FName> KernelBindingToPinLabel;

	UPROPERTY()
	TMap</*DownstreamBindingIndex*/int32, /*UpstreamBindingIndex*/int32> DownstreamToUpstreamBinding;

	/** Global attribute information collated during compilation. */
	UPROPERTY()
	FPCGKernelAttributeTable StaticAttributeTable;

	/** Global data label information collated during compilation. */
	UPROPERTY()
	TMap</*KernelIndex*/int32, FPCGPinDataLabels> StaticDataLabelsTable;

	/** String table collated during compilation. */
	UPROPERTY()
	TArray<FString> StringTable;

	/** Table to look up a kernel pin's first binding index. */
	UPROPERTY()
	TMap<FPCGKernelPin, /*FirstBindingIndex*/int32> KernelPinToFirstBinding;

	/** Binding index to virtual label for bindings that are that receive data from external nodes (executing on the CPU or in separate compute graphs). */
	UPROPERTY()
	TMap</*BindingIndex*/int32, /*VirtualPinLabel*/FName> CPUDataBindingToVirtualPinLabel;

	/** Kernel input pin to virtual pin label, populated for pins that receive data from external output pins. */
	UPROPERTY()
	TMap<FPCGKernelPin, /*VirtualPinLabel*/FName> KernelPinToVirtualPinLabel;

	/** Maps virtual input pins to their PCG data type. */
	UPROPERTY()
	TMap</*VirtualPinLabel*/FName, FPCGDataTypeIdentifier> VirtualInputPinToType;

	friend class FPCGGraphCompilerGPU;
};
