// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGComputeKernel.h"

#include "PCGContext.h"
#include "PCGEdge.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "Compute/PCGComputeSource.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/Elements/PCGComputeGraphElement.h"
#include "Compute/Packing/PCGDataCollectionPacking.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "ShaderCompilerCore.h"
#include "Async/Async.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGComputeKernel)

#define LOCTEXT_NAMESPACE "PCGComputeKernel"

namespace PCGComputeKernel
{
	static TAutoConsoleVariable<bool> CVarWarnOnOverridePinUsage(
		TEXT("pcg.Graph.GPU.WarnOnOverridePinUsage"),
		true,
		TEXT("Enables warnings when unsupported parameters are overidden on GPU nodes."));

	static const TStaticArray<FSoftObjectPath, 3> DefaultAdditionalSourcePaths =
	{
		FSoftObjectPath(TEXT("/Script/PCG.PCGComputeSource'/PCG/ComputeSources/PCGCS_ShaderUtils.PCGCS_ShaderUtils'")),
		FSoftObjectPath(TEXT("/Script/PCG.PCGComputeSource'/PCG/ComputeSources/PCGCS_ShaderUtilsInternal.PCGCS_ShaderUtilsInternal'")),

		// Note: PCGDataCollectionDataInterface.ush depends on the quaternion helpers, therefore all kernels also depend on the quaternion helpers.
		// @todo_pcg: In the future quaternion compute source could be opt-in if the kernel does not manipulate point/attribute data.
		FSoftObjectPath(TEXT("/Script/PCG.PCGComputeSource'/PCG/ComputeSources/PCGCS_Quaternion.PCGCS_Quaternion'"))
	};

	TConstArrayView<FSoftObjectPath> GetDefaultAdditionalSourcePaths()
	{
		return MakeArrayView(DefaultAdditionalSourcePaths);
	}
}

#if WITH_EDITOR
void UPCGComputeKernel::Initialize(const FPCGComputeKernelParams& InParams)
{
	ResolvedSettings = InParams.Settings;
	SettingsPath = ResolvedSettings;

	ResolvedNode = InParams.Node;
	NodePath = ResolvedNode;

	if (InParams.bLogDescriptions)
	{
		PCGKernelFlags |= EPCGComputeKernelFlags::LogDataDescriptions;
	}

	if (InParams.bRequiresOverridableParams && ResolvedSettings)
	{
		// Copy any GPU supported overridable params from the settings.
		for (const FPCGSettingsOverridableParam& OverridableParam : ResolvedSettings->OverridableParams())
		{
			if (OverridableParam.bSupportsGPU)
			{
				FPCGKernelOverridableParam& KernelOverridableParam = CachedOverridableParams.Emplace_GetRef();
				KernelOverridableParam.Label = OverridableParam.Label;
				KernelOverridableParam.PropertiesNames = OverridableParam.PropertiesNames;
				KernelOverridableParam.bRequiresGPUReadback = OverridableParam.bRequiresGPUReadback;
				KernelOverridableParam.bIsPropertyOverriddenByPin = ResolvedSettings->IsPropertyOverriddenByPin(OverridableParam.PropertiesNames);
				KernelOverridableParam.UnderlyingType = OverridableParam.UnderlyingType;
			}
		}
	}

	bInitialized = true;

	InitializeInternal();

	if (!PerformStaticValidation())
	{
		PCGKernelFlags |= EPCGComputeKernelFlags::HasStaticValidationErrors;
	}
}
#endif

const UPCGSettings* UPCGComputeKernel::GetSettings() const
{
	if (!ResolvedSettings)
	{
		FGCScopeGuard Guard;
		ResolvedSettings = SettingsPath.Get();
	}

	return ResolvedSettings;
}

const UPCGNode* UPCGComputeKernel::GetNode() const
{
	if (!ResolvedNode)
	{
		FGCScopeGuard Guard;
		ResolvedNode = NodePath.Get();
	}

	return ResolvedNode;
}

bool UPCGComputeKernel::RequiresPostTLASBuildExecutionGroup() const
{
	return false;
}

void UPCGComputeKernel::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UPCGComputeKernel* This = CastChecked<UPCGComputeKernel>(InThis);
	Collector.AddReferencedObject(This->ResolvedSettings);
	Collector.AddReferencedObject(This->ResolvedNode);
}

bool UPCGComputeKernel::AreKernelSettingsValid(FPCGContext* InContext) const
{
	if (!bInitialized)
	{
		PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), LOCTEXT("UninitializeKernel", "Kernel was not initialized during compilation. Make sure to call Initialize() when creating your kernels."));
		return false;
	}

#if PCG_KERNEL_LOGGING_ENABLED
	for (const FPCGKernelLogEntry& StaticLogEntry : StaticLogEntries)
	{
		if (StaticLogEntry.Verbosity == EPCGKernelLogVerbosity::Error)
		{
			PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), StaticLogEntry.Message);
		}
		else if (StaticLogEntry.Verbosity == EPCGKernelLogVerbosity::Warning)
		{
			PCG_KERNEL_VALIDATION_WARN(InContext, GetSettings(), StaticLogEntry.Message);
		}
		else
		{
			ensure(false);
		}
	}
#endif

	return !HasStaticValidationErrors();
}

bool UPCGComputeKernel::IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGComputeKernel::IsKernelDataValid);

	if (!InDataBinding)
	{
		return true;
	}

	const UPCGSettings* Settings = CastChecked<UPCGSettings>(GetSettings());
	const UPCGNode* Node = GetNode();

	if (InDataBinding && Node)
	{
		TArray<FPCGPinPropertiesGPU> OutputPins;
		GetOutputPins(OutputPins);

		for (const FPCGPinPropertiesGPU& Pin : OutputPins)
		{
			// Pairwise requires all data to be N:N, 1:N, or N:1.
			if (Pin.PropertiesGPU.InitializationMode == EPCGPinInitMode::FromInputPins
				&& Pin.PropertiesGPU.DataCountMode == EPCGDataCountMode::FromInputData
				&& Pin.PropertiesGPU.DataMultiplicity == EPCGDataMultiplicity::Pairwise)
			{
				uint32 NumData = -1;

				for (FName InitPin : Pin.PropertiesGPU.PinsToInititalizeFrom)
				{
					// This is a defensive condition for legacy data to avoid a long chain of ensures:
					// It is possible for the InitPins to be out of date with which pins actually exist on the node. This could cause a cache miss
					// when looking up the pin data description, so only consider InitPins which actually exist.
					if (Node->GetInputPin(InitPin))
					{
						if (const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InDataBinding->GetCachedKernelPinDataDesc(this, InitPin, /*bIsInputPin=*/true))
						{
							const uint32 InputNumData = InputDesc->GetDataDescriptions().Num();

							// Only test pins that have a non-trivial number of data.
							if (InputNumData > 1)
							{
								if (NumData == -1)
								{
									NumData = InputNumData;
								}
								else if (InputNumData != NumData)
								{
									PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), FText::Format(
										LOCTEXT("NumDataMismatch", "Number of data found on input pin '{0}' ({1}), mismatches with number of data found on another input pin ({2}). Pairwise data multiplicity only supports N:N, 1:N and N:1 operation."),
										FText::FromName(InitPin),
										InputNumData,
										NumData));

									return false;
								}
							}
							else if (InputNumData == 0)
							{
								// No validation is necessary if any pin has 0 data. There are no pairs to validate.
								break;
							}
						}
					}
				}
			}
		}
	}

	return true;
}

// Deprecated (5.8)
bool UPCGComputeKernel::IsKernelDataValid(FPCGContext* InContext) const
{
	const UPCGDataBinding* DataBinding = nullptr;
	if (InContext && InContext->IsComputeContext())
	{
		DataBinding = static_cast<FPCGComputeGraphContext*>(InContext)->DataBinding.Get();
	}

	return IsKernelDataValid(DataBinding, InContext);
}

#if WITH_EDITOR
FString UPCGComputeKernel::GetCookedSource(FPCGGPUCompilationContext& InOutContext) const
{
	FString SourceFile;
	ensure(LoadShaderSourceFile(GetSourceFilePath(), EShaderPlatform::SP_PCD3D_SM5, &SourceFile, nullptr));
	return SourceFile;
}

void UPCGComputeKernel::GatherAdditionalSources(TArray<TObjectPtr<UComputeSource>>& OutAdditionalSources) const
{
	for (const FSoftObjectPath& AdditionalSourcePath : PCGComputeKernel::DefaultAdditionalSourcePaths)
	{
		UPCGComputeSource* AdditionalSource = Cast<UPCGComputeSource>(AdditionalSourcePath.ResolveObject());
		
		if (ensure(AdditionalSource))
		{
			OutAdditionalSources.Add(AdditionalSource);
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Default additional compute source '%ls' could not be resolved for kernel '%ls'.", *AdditionalSourcePath.ToString(), *GetName());
		}
	}
}

void UPCGComputeKernel::CreateKernelParamsDataInterface(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& InOutDataInterfaces) const
{
	if (CachedOverridableParams.IsEmpty())
	{
		return;
	}

	// Check if a subclass already added a KernelParamsDataInterface (or a derived class).
	const bool bAlreadyHasKernelParamsDI = InOutDataInterfaces.ContainsByPredicate([](const TObjectPtr<UComputeDataInterface>& DataInterface)
	{
		return DataInterface && DataInterface->IsA<UPCGKernelParamsDataInterface>();
	});

	if (!bAlreadyHasKernelParamsDI)
	{
		TObjectPtr<UPCGKernelParamsDataInterface> KernelParamsDI = InOutContext.NewObject_AnyThread<UPCGKernelParamsDataInterface>(InObjectOuter);
		KernelParamsDI->SetProducerKernel(this);
		InOutDataInterfaces.Add(KernelParamsDI);
	}
}

bool UPCGComputeKernel::SplitGraphAtInput(FName InPinLabel) const
{
	for (const FPCGKernelOverridableParam& OverridableParam : CachedOverridableParams)
	{
		if (OverridableParam.bRequiresGPUReadback && OverridableParam.Label == InPinLabel)
		{
			return true;
		}
	}

	return false;
}

UPCGComputeDataInterface* UPCGComputeKernel::CreateOutputPinDataInterface(const PCGComputeHelpers::FCreateDataInterfaceParams& InParams) const
{
	return PCGComputeHelpers::CreateOutputPinDataInterface(InParams);
}
#endif

void UPCGComputeKernel::ComputeDataDescFromPinProperties(const FPCGPinPropertiesGPU& OutputPinProps, const TArrayView<const FPCGPinProperties>& InInputPinProps, UPCGDataBinding* InBinding, TSharedPtr<FPCGDataCollectionDesc> OutPinDesc) const
{
	check(InBinding);

	const FPCGPinPropertiesGPUStruct& Props = OutputPinProps.PropertiesGPU;

	const int32 MaximumOutputDataCount = PCGComputeHelpers::GetMaxOutputDataCount(OutputPinProps.AllowedTypes);

	if (Props.InitializationMode == EPCGPinInitMode::FromInputPins)
	{
		TArray<FName> InitPins;
		TArray<TSharedPtr<const FPCGDataCollectionDesc>> InputDescs;

		for (FName PinToInitFrom : Props.PinsToInititalizeFrom)
		{
			if (const FPCGPinProperties* InputPinProps = InInputPinProps.FindByPredicate([PinToInitFrom](const FPCGPinProperties& InProps) { return InProps.Label == PinToInitFrom; }))
			{
				InitPins.Emplace(PinToInitFrom);

				const FPCGKernelPin KernelPin(KernelIndex, PinToInitFrom, /*bIsInput=*/true);
				TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InBinding->ComputeKernelPinDataDesc(KernelPin);

				InputDescs.Add(InputDesc);
			}
		}

		const int NumInitPins = InitPins.Num();
		check(NumInitPins == InputDescs.Num());

		// Copies unique (non-reserved) attribute descriptions from 'InDataDesc' to 'OutDataDesc'.
		auto AddAttributesFromData = [](const FPCGDataDesc& InDataDesc, FPCGDataDesc& OutDataDesc)
		{
			TArray<FPCGKernelAttributeDesc>& OutAttributeDescs = OutDataDesc.GetAttributeDescriptionsMutable();

			for (const FPCGKernelAttributeDesc& InAttrDesc : InDataDesc.GetAttributeDescriptions())
			{
				if (InAttrDesc.GetAttributeId() >= PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS
					&& !OutAttributeDescs.ContainsByPredicate([InAttrDesc](const FPCGKernelAttributeDesc& OutAttrDesc)
						{
							// Note: We shouldn't need to check for uniqueness of attr Index, since attributes should all have unique index via
							// the GlobalAttributeLookupTable
							return InAttrDesc.GetAttributeKey().GetIdentifier() == OutAttrDesc.GetAttributeKey().GetIdentifier();
						}))
				{
					OutAttributeDescs.Emplace(InAttrDesc);
				}
			}
		};

		// Combines the data for index i of each pin into one data. Creates exactly 'MaxDataCount' datas.
		auto AddDataPairwise = [&](int MaxDataCount)
		{
			for (int DataIndex = 0; DataIndex < MaxDataCount; ++DataIndex)
			{
				FIntVector4 InitialNumElements = FIntVector4::ZeroValue; // Total number of elements computed for this data index.

				// If the element count is a product, we have to start at 1 instead of 0 or we'll end up with 0!
				if (Props.ElementMultiplicity == EPCGElementMultiplicity::Product && NumInitPins > 0)
				{
					const EPCGElementDimension ElementDimension = PCGComputeHelpers::GetElementDimension(OutputPinProps.AllowedTypes);

					if (ElementDimension == EPCGElementDimension::One)
					{
						InitialNumElements.X = 1;
					}
					else if (ElementDimension == EPCGElementDimension::Two)
					{
						InitialNumElements.X = 1;
						InitialNumElements.Y = 1;
					}
					else if (ElementDimension == EPCGElementDimension::Three)
					{
						InitialNumElements.X = 1;
						InitialNumElements.Y = 1;
						InitialNumElements.Z = 1;
					}
					else if (ElementDimension == EPCGElementDimension::Four)
					{
						InitialNumElements.X = 1;
						InitialNumElements.Y = 1;
						InitialNumElements.Z = 1;
						InitialNumElements.W = 1;
					}
				}

				FPCGDataDesc& DataDesc = OutPinDesc->GetDataDescriptionsMutable().Emplace_GetRef(OutputPinProps.AllowedTypes, InitialNumElements);

				// For each data index, loop over all the pins and create the uber-data.
				for (int InputPinIndex = 0; InputPinIndex < NumInitPins; ++InputPinIndex)
				{
					const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InputDescs[InputPinIndex];

					int ClampedDataIndex = DataIndex;

					// If this pin does not have the same number of data, clamp it to the last data.
					if (DataIndex >= InputDesc->GetDataDescriptions().Num())
					{
						ClampedDataIndex = InputDesc->GetDataDescriptions().Num() - 1;
					}

					const FPCGDataDesc& InputDataDesc = InputDesc->GetDataDescriptions().IsValidIndex(ClampedDataIndex) ? InputDesc->GetDataDescriptions()[ClampedDataIndex] : FPCGDataDesc(FPCGDataTypeIdentifier{EPCGDataType::Any}, 0);
					const EPCGElementDimension SourceDimension = InputDataDesc.GetElementDimension();

					if (Props.ElementCountMode == EPCGElementCountMode::FromInputData)
					{
						DataDesc.CombineElementCount(InputDataDesc, Props.ElementMultiplicity);
					}
					else if (Props.ElementCountMode == EPCGElementCountMode::Fixed)
					{
						const EPCGElementDimension ElementDimension = PCGComputeHelpers::GetElementDimension(OutputPinProps.AllowedTypes);

						if (ElementDimension == EPCGElementDimension::One)
						{
							DataDesc.AddElementCount(Props.ElementCount);
						}
						else if (ElementDimension == EPCGElementDimension::Two)
						{
							DataDesc.AddElementCount(Props.NumElements2D);
						}
						else
						{
							// Not supported yet.
							checkNoEntry();
						}
					}
					else
					{
						checkNoEntry();
					}

					if (Props.AttributeInheritanceMode == EPCGAttributeInheritanceMode::CopyAttributeSetup)
					{
						AddAttributesFromData(InputDataDesc, DataDesc);
					}

					// Add unique tags from this data.
					for (const int32 TagStringKey : InputDataDesc.GetTagStringKeys())
					{
						DataDesc.AddUniqueTagStringKey(TagStringKey);
					}

					DataDesc.AllocateProperties(InputDataDesc.GetAllocatedProperties());
				}
			}
		};

		if (Props.DataCountMode == EPCGDataCountMode::FromInputData)
		{
			// If this is the only input pin, we can just copy it.
			if (NumInitPins == 1)
			{
				for (const FPCGDataDesc& InputDataDesc : InputDescs[0]->GetDataDescriptions())
				{
					FPCGDataDesc& DataDesc = OutPinDesc->GetDataDescriptionsMutable().Emplace_GetRef(OutputPinProps.AllowedTypes, /*InitialElementCount=*/0);
					DataDesc.CombineElementCount(InputDataDesc, EPCGElementMultiplicity::Sum);

					if (Props.AttributeInheritanceMode == EPCGAttributeInheritanceMode::CopyAttributeSetup)
					{
						AddAttributesFromData(InputDataDesc, DataDesc);
					}

					DataDesc.SetTagStringKeys(InputDataDesc.GetTagStringKeys());
					DataDesc.AllocateProperties(InputDataDesc.GetAllocatedProperties());

					if (OutPinDesc->GetDataDescriptions().Num() >= MaximumOutputDataCount)
					{
						break;
					}
				}
			}
			// Take pairs of datas, where the pairs are given by each data of each pin to each data of every other pin.
			else if (Props.DataMultiplicity == EPCGDataMultiplicity::CartesianProduct)
			{
				bool bDone = false;

				for (int InputPinIndex = 0; InputPinIndex < NumInitPins && !bDone; ++InputPinIndex)
				{
					const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InputDescs[InputPinIndex];

					for (int OtherInputPinIndex = InputPinIndex + 1; OtherInputPinIndex < NumInitPins && !bDone; ++OtherInputPinIndex)
					{
						const TSharedPtr<const FPCGDataCollectionDesc> OtherInputDesc = InputDescs[OtherInputPinIndex];

						for (const FPCGDataDesc& InputDataDesc : InputDesc->GetDataDescriptions())
						{
							for (const FPCGDataDesc& OtherInputDataDesc : OtherInputDesc->GetDataDescriptions())
							{
								FPCGDataDesc& DataDesc = OutPinDesc->GetDataDescriptionsMutable().Emplace_GetRef(OutputPinProps.AllowedTypes, /*InitialElementCount=*/0);

								if (Props.ElementCountMode == EPCGElementCountMode::FromInputData)
								{
									DataDesc.CombineElementCount(FPCGDataDesc::CombineElementCount(InputDataDesc, OtherInputDataDesc, Props.ElementMultiplicity), EPCGElementMultiplicity::Sum);
								}
								else if (Props.ElementCountMode == EPCGElementCountMode::Fixed)
								{
									const EPCGElementDimension ElementDimension = PCGComputeHelpers::GetElementDimension(OutputPinProps.AllowedTypes);

									if (ElementDimension == EPCGElementDimension::One)
									{
										DataDesc.AddElementCount(Props.ElementCount);
									}
									else if (ElementDimension == EPCGElementDimension::Two)
									{
										DataDesc.AddElementCount(Props.NumElements2D);
									}
									else
									{
										// Not supported yet.
										checkNoEntry();
									}
								}
								else
								{
									checkNoEntry();
								}

								if (Props.AttributeInheritanceMode == EPCGAttributeInheritanceMode::CopyAttributeSetup)
								{
									AddAttributesFromData(InputDataDesc, DataDesc);
									AddAttributesFromData(OtherInputDataDesc, DataDesc);
								}

								// Add unique tags from both input data.
								for (const int32 TagStringKey : InputDataDesc.GetTagStringKeys())
								{
									DataDesc.AddUniqueTagStringKey(TagStringKey);
								}

								for (const int32 TagStringKey : OtherInputDataDesc.GetTagStringKeys())
								{
									DataDesc.AddUniqueTagStringKey(TagStringKey);
								}

								DataDesc.AllocateProperties(InputDataDesc.GetAllocatedProperties() | OtherInputDataDesc.GetAllocatedProperties());

								bDone = OutPinDesc->GetDataDescriptions().Num() >= MaximumOutputDataCount;
								if (bDone)
								{
									break;
								}
							}

							if (bDone)
							{
								break;
							}
						}
					}
				}
			}
			// Combine elements for each set of datas, where the sets are given by the Nth datas on each pin (or the first data if there is only one data).
			else if (Props.DataMultiplicity == EPCGDataMultiplicity::Pairwise)
			{
				int MaxDataCount = 0;

				// Find the maximum number of data among the init pins. Note, they should all be the same number of data, or only one data.
				for (int I = 0; I < NumInitPins; ++I)
				{
					const int DataCount = InputDescs[I]->GetDataDescriptions().Num();

					// If any init pins have no data, then the pairing fails and no data are produced.
					if (DataCount == 0)
					{
						MaxDataCount = 0;
						break;
					}

					MaxDataCount = FMath::Max(MaxDataCount, DataCount);
				}

				AddDataPairwise(FMath::Min(MaxDataCount, MaximumOutputDataCount));
			}
			else
			{
				checkNoEntry();
			}
		}
		else if (Props.DataCountMode == EPCGDataCountMode::Fixed)
		{
			AddDataPairwise(FMath::Min(Props.DataCount, MaximumOutputDataCount));
		}
		else
		{
			checkNoEntry();
		}

		// Apply element count multiplier.
		const uint32 Multiplier = OutputPinProps.GetElementCountMultiplier();
		for (FPCGDataDesc& Desc : OutPinDesc->GetDataDescriptionsMutable())
		{
			Desc.ScaleElementCount(Multiplier);
		}

		if (OutputPinProps.AllowedTypes.IsChildOf(EPCGDataType::BaseTexture) || OutputPinProps.AllowedTypes.IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId()))
		{
			// This code assumes texture output pins are single data.
			ensure(MaximumOutputDataCount == 1);
			const FPCGDataDesc* InputDesc = (!InputDescs.IsEmpty() && InputDescs[0] && !InputDescs[0]->GetDataDescriptions().IsEmpty()) ? &InputDescs[0]->GetDataDescriptions()[0] : nullptr;

			if (OutputPinProps.PropertiesGPU.TextureFormatInheritanceMode == EPCGPropertyInheritanceMode::FromInput && InputDesc)
			{
				OutPinDesc->SetRenderTargetFormatForAllData(InputDesc->GetRenderTargetFormat());
			}
			else
			{
				OutPinDesc->SetRenderTargetFormatForAllData(OutputPinProps.PropertiesGPU.TextureFormat);
			}

			if (OutputPinProps.PropertiesGPU.TextureFilterInheritanceMode == EPCGPropertyInheritanceMode::FromInput && InputDesc)
			{
				OutPinDesc->SetTextureFilterForAllData(InputDesc->GetTextureFilter());
			}
			else
			{
				OutPinDesc->SetTextureFilterForAllData(OutputPinProps.PropertiesGPU.TextureFilter);
			}

			if (OutputPinProps.PropertiesGPU.TextureArraySizeInheritanceMode == EPCGPropertyInheritanceMode::FromInput && InputDesc)
			{
				OutPinDesc->SetTextureArraySizeForAllData(InputDesc->GetTextureArraySize());
			}
			else
			{
				OutPinDesc->SetTextureArraySizeForAllData(OutputPinProps.PropertiesGPU.TextureArraySize);
			}

			if (OutputPinProps.PropertiesGPU.TextureTransformInheritanceMode == EPCGPropertyInheritanceMode::FromInput && InputDesc)
			{
				OutPinDesc->SetTextureTransformForAllData(InputDesc->GetTextureTransform());
			}
			else if (!OutputPinProps.PropertiesGPU.bInitializeTextureTransformFromGenerationVolume && OutputPinProps.PropertiesGPU.TextureTransformInheritanceMode == EPCGPropertyInheritanceMode::Custom)
			{
				OutPinDesc->SetTextureTransformForAllData(OutputPinProps.PropertiesGPU.TextureTransform);
			}
			else
			{
				TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
				if (FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr)
				{
					OutPinDesc->SetTextureTransformForAllData(UPCGTexture2DBaseData::ComputeTransform(Context->ExecutionSource.Get()));
				}
			}
		}
	}
	else if (Props.InitializationMode == EPCGPinInitMode::Custom)
	{
		TArray<FPCGDataDesc>& DataDescriptions = OutPinDesc->GetDataDescriptionsMutable();

		const int32 DataCount = FMath::Min(Props.DataCount, MaximumOutputDataCount);

		for (int I = 0; I < DataCount; ++I)
		{
			FPCGDataDesc* DataDesc = nullptr;

			const EPCGElementDimension ElementDimension = PCGComputeHelpers::GetElementDimension(OutputPinProps.AllowedTypes);

			if (ElementDimension == EPCGElementDimension::One)
			{
				DataDesc = &OutPinDesc->GetDataDescriptionsMutable().Emplace_GetRef(OutputPinProps.AllowedTypes, Props.ElementCount);
			}
			else if (ElementDimension == EPCGElementDimension::Two)
			{
				DataDesc = &OutPinDesc->GetDataDescriptionsMutable().Emplace_GetRef(OutputPinProps.AllowedTypes, Props.NumElements2D);
			}
			else if (ElementDimension == EPCGElementDimension::Three && OutputPinProps.AllowedTypes.IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId()))
			{
				DataDesc = &OutPinDesc->GetDataDescriptionsMutable().Emplace_GetRef(OutputPinProps.AllowedTypes, FIntVector3(Props.NumElements2D.X, Props.NumElements2D.Y, Props.TextureArraySize));
			}
			else
			{
				// Not supported yet.
				checkNoEntry();
			}
		}

		if (OutputPinProps.AllowedTypes.IsChildOf(EPCGDataType::BaseTexture) || OutputPinProps.AllowedTypes.IsChildOf(FPCGDataTypeInfoTexture2DArray::AsId()))
		{
			OutPinDesc->SetRenderTargetFormatForAllData(OutputPinProps.PropertiesGPU.TextureFormat);
			OutPinDesc->SetTextureFilterForAllData(OutputPinProps.PropertiesGPU.TextureFilter);
			OutPinDesc->SetTextureArraySizeForAllData(OutputPinProps.PropertiesGPU.TextureArraySize);

			if (OutputPinProps.PropertiesGPU.bInitializeTextureTransformFromGenerationVolume)
			{
				TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
				if (FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr)
				{
					OutPinDesc->SetTextureTransformForAllData(UPCGTexture2DBaseData::ComputeTransform(Context->ExecutionSource.Get()));
				}
			}
			else
			{
				OutPinDesc->SetTextureTransformForAllData(OutputPinProps.PropertiesGPU.TextureTransform);
			}
		}
	}
	else
	{
		checkNoEntry();
	}
}

void UPCGComputeKernel::GetInputPinsAndOverridablePins(TArray<FPCGPinProperties>& OutPins) const
{
	GetInputPins(OutPins);
	GetOverridableParamPins(OutPins);
}

#if WITH_EDITOR
bool UPCGComputeKernel::PerformStaticValidation()
{
	return ValidatePCGNode(StaticLogEntries);
}

bool UPCGComputeKernel::ValidatePCGNode(TArray<FPCGKernelLogEntry>& OutLogEntries) const
{
	const UPCGSettings* Settings = GetSettings();
	if (!Settings)
	{
		return true;
	}

#if PCG_KERNEL_LOGGING_ENABLED
	if (PCGComputeKernel::CVarWarnOnOverridePinUsage.GetValueOnAnyThread())
	{
		// Note: Properties which must be known at compile time (e.g. UPCGCopyPointsSettings::bMatchBasedOnAttribute), are not supported.
		// They would require compiling two versions of the kernel graph and branching dynamically at execution time.
		// Note, nothing currently validates that params of this nature are not overridden, so implement new overrides with caution.

		const UPCGNode* Node = GetNode();
		const UPCGPin* Pin = Node ? Node->GetInputPin(PCGPinConstants::DefaultParamsLabel) : nullptr;

		if (Pin && Pin->IsConnected())
		{
			OutLogEntries.Emplace(FText::Format(
				LOCTEXT("InvalidOverridesPin", "Unsupported use of pin '{0}' on node '{1}'. Use a direct override pin instead."),
				FText::FromName(PCGPinConstants::DefaultParamsLabel),
				Node->GetNodeTitle(EPCGNodeTitleType::ListView)),
				EPCGKernelLogVerbosity::Warning);
		}

		for (const FPCGSettingsOverridableParam& OverridableParam : Settings->OverridableParams())
		{
			if (!OverridableParam.bSupportsGPU && Settings->IsPropertyOverriddenByPin(OverridableParam.PropertiesNames))
			{
				OutLogEntries.Emplace(FText::Format(
					LOCTEXT("UnsupportedOverrideError", "Tried to override parameter '{0}', but this parameter does not support overrides when executing on the GPU at this time."),
					FText::FromName(OverridableParam.Label)),
					EPCGKernelLogVerbosity::Warning);
			}
		}
	}
#endif

	// Validate types of incident edges to make sure we catch invalid cases like Spatial -> Point.
	return AreInputEdgesValid(OutLogEntries);
}

bool UPCGComputeKernel::AreInputEdgesValid(TArray<FPCGKernelLogEntry>& OutLogEntries) const
{
	bool bAllEdgesValid = true;

	if (const UPCGNode* Node = GetNode())
	{
		for (const UPCGPin* InputPin : Node->GetInputPins())
		{
			if (!InputPin)
			{
				continue;
			}

			for (const UPCGEdge* InputEdge : InputPin->Edges)
			{
				const UPCGPin* UpstreamPin = InputEdge ? InputEdge->GetOtherPin(InputPin) : nullptr;
				if (UpstreamPin && !InputPin->IsCompatible(UpstreamPin))
				{
#if PCG_KERNEL_LOGGING_ENABLED
					OutLogEntries.Emplace(FText::Format(
						LOCTEXT("InvalidInputPinEdge", "Unsupported connected upstream pin '{0}' on node '{1}' with type {2}. Recreate the edge to add required conversion nodes."),
						FText::FromName(UpstreamPin->Properties.Label),
						Node->GetNodeTitle(EPCGNodeTitleType::ListView),
						FText::FromString(UpstreamPin->Properties.AllowedTypes.ToString())),
						EPCGKernelLogVerbosity::Error);
#endif

					bAllEdgesValid = false;
				}
			}
		}
	}

	return bAllEdgesValid;
}
#endif

void UPCGComputeKernel::GetOverridableParamPins(TArray<FPCGPinProperties>& OutPins) const
{
	for (const FPCGKernelOverridableParam& OverridableParam : CachedOverridableParams)
	{
		if (OverridableParam.bIsPropertyOverriddenByPin)
		{
			FPCGPinProperties& ParamPin = OutPins.Emplace_GetRef(OverridableParam.Label, EPCGDataType::Param, /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false);
			ParamPin.SetOverrideOrUserParamPin();
		}
	}
}

#undef LOCTEXT_NAMESPACE
