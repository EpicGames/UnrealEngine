// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/ComputeKernelSource.h"

#include "PCGComputeKernelSource.generated.h"

UCLASS(MinimalAPI)
class UPCGComputeKernelSource : public UComputeKernelSource
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	//~ Begin UComputeKernelSource Interface.
	FString GetSource() const override
	{
		return Source;
	}
	//~ End UComputeKernelSource Interface.

	void SetSource(FString const& InSource)
	{
		Source = InSource;
	}
#endif

protected:
#if WITH_EDITOR
	// Transient string used only during compilation.
	FString Source;
#endif
};
