// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGComputeSource.h"

#include "Compute/PCGComputeCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGComputeSource)

#if WITH_EDITOR
FOnPCGComputeSourceModified UPCGComputeSource::OnModifiedDelegate;

void UPCGComputeSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnModifiedDelegate.Broadcast(this);
}

void UPCGComputeSource::PostEditUndo()
{
	Super::PostEditUndo();

	OnModifiedDelegate.Broadcast(this);
}
#endif

FString UPCGComputeSource::GetSource() const
{
#if WITH_EDITOR
	return Source;
#else
	ensure(false);
	return "";
#endif
}

FString UPCGComputeSource::GetVirtualPath() const
{
#if WITH_EDITOR
	FString ShaderPathName = GetPathName();
	PCGComputeHelpers::ConvertObjectPathToShaderFilePath(ShaderPathName);
	return ShaderPathName;
#else
	ensure(false);
	return "";
#endif
}

#if WITH_EDITOR
void UPCGComputeSource::SetSourceText(const FString& InText)
{
	if (!Source.Equals(InText, ESearchCase::CaseSensitive))
	{
		Modify();

		Source = InText;

		OnModifiedDelegate.Broadcast(this);
	}
}
#endif
