// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGDataBinding.h"

#include "PCGContext.h"
#include "PCGNode.h"
#include "PCGParamData.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/Packing/PCGDataCollectionPacking.h"
#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "RenderGraphResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataBinding)

void UPCGDataBinding::Initialize(const UPCGComputeGraph* InComputeGraph, FPCGContext* InContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataBinding::Initialize);
	check(InComputeGraph);
	check(InContext);

	Graph = InComputeGraph;

	ContextHandle = InContext->GetOrCreateHandle();

	// Add compatible data objects from input collection.
	InputDataCollection.TaggedData.Reserve(InContext->InputData.TaggedData.Num());
	for (const FPCGTaggedData& InputData : InContext->InputData.TaggedData)
	{
		if (!InputData.Data)
		{
			continue;
		}
		
		if (!PCGComputeHelpers::IsTypeAllowedAsInput(FPCGDataTypeIdentifier{InputData.Data->GetDataTypeId()}))
		{
			UE_LOGF(LogPCG, Warning, "Stripped input data '%ls' that is not currently supported by GPU execution.", *InputData.Data->GetName());
			continue;
		}

		const FPCGDataTypeIdentifier* VirtualInputPinType = Graph->GetVirtualInputPinType(InputData.Pin);
		const FPCGDataTypeIdentifier InputDataType = InputData.Data->GetUnderlyingDataTypeId();

		if (!VirtualInputPinType || !InputDataType.IsChildOf(*VirtualInputPinType))
		{
			UE_LOGF(LogPCG, Warning, "Stripped input data '%ls' that is not the correct type for pin '%ls'.", *InputData.Data->GetName(), *InputData.Pin.ToString());
			continue;
		}

		InputDataCollection.TaggedData.Add(InputData);
	}
}

void UPCGDataBinding::InitializeTables(FPCGContext* InContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataBinding::InitializeTables);
	check(InContext);

	AttributeTable = Graph->GetStaticAttributeTable();

	// String table always contains empty string in index 0 (and string key attributes are 0-initialized), and then any strings known statically at compilation time.
	StringTable = { FString() };
	StringTable.Append(Graph->GetStaticStringTable());

	// Augment static tables with data incoming from CPU.
	AddInputDataAttributesToTable();
	AddInputDataStringsToTable();
	AddInputDataTagsToTable();

	bTablesInitialized = true;
}

void UPCGDataBinding::ReleaseTransientResources()
{
	KernelParamsCache.Reset();
	SourceBufferAttributeToGraphAttributeIndex.Empty();
	GraphAttributeToSourceBufferAttributeIndex.Empty();
	SourceBufferToGraphStringKey.Empty();
	GraphToSourceBufferStringKey.Empty();
	InputDataCollection.TaggedData.Empty();
	OutputDataCollection.TaggedData.Empty();
	MeshSpawnersToPrimitives.Empty();
	CompletedMeshSpawners.Empty();
	DataToDebug.Empty();
	DataToInspect.Empty();
	ContextHandle.Reset();
	DataDescriptionCache.Empty();
	AttributeTable.Empty();
	StringTable.Empty();
}

void UPCGDataBinding::PrimeDataDescriptionCache()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataBinding::PrimeDataDescriptionCache);

	if (bIsDataDescriptionCachePrimed || !DataDescriptionCache.IsEmpty())
	{
		ensureMsgf(false, TEXT("Attempted to prime the data description cache, but it was already populated."));
		return;
	}

	if (ensure(Graph))
	{
		TArray<FPCGKernelPin> KernelPins;
		Graph->GetKernelPins(KernelPins);

		// Compute and cache a description for every kernel pin in the graph (including both input and output pins).
		// Note: It is fine for some kernel pins to fail to compute a description, e.g. DataLabelResolver pins, as they will never need a data description.
		// @todo_pcg: Could add a ParallelFor if this becomes a bottle neck.
		for (const FPCGKernelPin& KernelPin : KernelPins)
		{
			ComputeKernelPinDataDesc(KernelPin);
		}
	}

	bIsDataDescriptionCachePrimed = true;
}

void UPCGDataBinding::FlushDataDescriptionCache()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataBinding::FlushDataDescriptionCache);
	DataDescriptionCache.Empty();
	bIsDataDescriptionCachePrimed = false;
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGDataBinding::ComputeKernelPinDataDesc(const FPCGKernelPin& InKernelPin)
{
	if (!ensure(Graph))
	{
		return nullptr;
	}

	const int32 GraphBindingIndex = Graph->GetBindingIndex(InKernelPin);

	if (GraphBindingIndex == -1)
	{
		UE_LOGF(LogPCG, Error, "Failed to compute data description for kernel pin '%ls'.", *InKernelPin.PinLabel.ToString());
		return nullptr;
	}

	return ComputeKernelBindingDataDesc(GraphBindingIndex);
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGDataBinding::ComputeKernelBindingDataDesc(int32 InGraphBindingIndex)
{
	ensure(bTablesInitialized);

	if (!ensure(Graph))
	{
		return nullptr;
	}

	// Try to look up the data description if it's been computed previously.
	if (const TSharedPtr<const FPCGDataCollectionDesc>* FoundDesc = DataDescriptionCache.Find(InGraphBindingIndex))
	{
		return *FoundDesc;
	}

	ensureMsgf(!bIsDataDescriptionCachePrimed, TEXT("Data description cache miss, investigate why the data description was not cached for this pin during priming."));

	// If the description wasn't found in the cache, compute it now.
	const TSharedPtr<const FPCGDataCollectionDesc> ComputedDesc = Graph->ComputeKernelBindingDataDesc(InGraphBindingIndex, this);
	DataDescriptionCache.Add(InGraphBindingIndex, ComputedDesc);

	return ComputedDesc;
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGDataBinding::GetCachedKernelPinDataDesc(const UPCGComputeKernel* InKernel, FName InPinLabel, bool bIsInput) const
{
	if (!ensure(Graph) || !ensure(InKernel))
	{
		return nullptr;
	}

	const int32 GraphBindingIndex = Graph->GetBindingIndex(FPCGKernelPin(InKernel->GetKernelIndex(), InPinLabel, bIsInput));

	if (GraphBindingIndex != INDEX_NONE)
	{
		return GetCachedKernelPinDataDesc(GraphBindingIndex);
	}
	else
	{
		UE_LOGF(LogPCG, Error, "No binding index found for %s pin '%ls' of kernel '%ls'.",
			bIsInput ? "input" : "output",
			*InPinLabel.ToString(),
			*InKernel->GetName());

		return nullptr;
	}
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGDataBinding::GetCachedKernelPinDataDesc(int32 InGraphBindingIndex) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataBinding::GetCachedKernelPinDataDesc);

	if (!ensure(bIsDataDescriptionCachePrimed))
	{
		UE_LOGF(LogPCG, Error, "Tried to get a kernel pin data description before the data description cache was primed.");
		return nullptr;
	}

	const TSharedPtr<const FPCGDataCollectionDesc>* FoundDescription = DataDescriptionCache.Find(InGraphBindingIndex);

	if (!FoundDescription)
	{
		// Should not happen - all data descriptions should have been computed at this point.
		UE_LOGF(LogPCG, Error, "Cache miss while retrieving kernel pin data description for binding index (%d).", InGraphBindingIndex);
		return nullptr;
	}

	return *FoundDescription;
}

void UPCGDataBinding::ReceiveDataFromGPU_GameThread(
	UPCGData* InData,
	const UPCGSettings* InProducerSettings,
	EPCGExportMode InExportMode,
	FName InPinLabel,
	FName InPinLabelAlias,
	const FPCGDataDesc& InDataDesc)
{
	check(IsInGameThread());

	TSet<FString> Tags;
	for (int32 TagStringKey : InDataDesc.GetTagStringKeys())
	{
		if (GetStringTable().IsValidIndex(TagStringKey) && TagStringKey > 0)
		{
			Tags.Add(GetStringTable()[TagStringKey]);
		}
	}

	// When deprecated ReceiveDataFromGPU_GameThread functions removed, inline here.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ReceiveDataFromGPU_GameThread(InData, InProducerSettings, InExportMode, InPinLabel, InPinLabelAlias, Tags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const TMap<UPCGDataBinding::FSourceBufferAndAttributeIndex, int32>& UPCGDataBinding::GetSourceBufferAttributeToGraphAttributeIndex() const
{
	return SourceBufferAttributeToGraphAttributeIndex;
}

const TMap<UPCGDataBinding::FGraphAttributeIndexAndSourceBuffer, int32>& UPCGDataBinding::GetGraphAttributeToSourceBufferAttributeIndex() const
{
	return GraphAttributeToSourceBufferAttributeIndex;
}

const TMap<UPCGDataBinding::FSourceBufferAndStringKey, int32>& UPCGDataBinding::GetSourceBufferToGraphStringKey() const
{
	return SourceBufferToGraphStringKey;
}

const TMap<UPCGDataBinding::FSourceBufferAndStringKey, int32>& UPCGDataBinding::GetGraphToSourceBufferStringKey() const
{
	return GraphToSourceBufferStringKey;
}

IPCGGraphExecutionSource* UPCGDataBinding::GetExecutionSource() const
{
	if (TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin())
	{
		if (FPCGContext* ContextPtr = SharedHandle->GetContext())
		{
			return ContextPtr->ExecutionSource.Get();
		}
	}

	return nullptr;
}

int32 UPCGDataBinding::GetFirstInputDataIndex(const UPCGComputeKernel* InKernel, FName InPinLabel) const
{
	int32 FirstDataIndex = INDEX_NONE;

	if (const FName* VirtualLabel = Graph->GetVirtualPinLabelFromKernelInputPin(FPCGKernelPin(InKernel->GetKernelIndex(), InPinLabel, /*bIsInput=*/true)))
	{
		FirstDataIndex = InputDataCollection.TaggedData.IndexOfByPredicate([VirtualLabel](const FPCGTaggedData& InData)
		{
			return InData.Pin == *VirtualLabel;
		});
	}

	return FirstDataIndex;
}

TArray<int32> UPCGDataBinding::GetPinInputDataIndices(const UPCGComputeKernel* InKernel, FName InPinLabel) const
{
	TArray<int32> PinDataIndices;

	if (const FName* VirtualLabel = Graph->GetVirtualPinLabelFromKernelInputPin(FPCGKernelPin(InKernel->GetKernelIndex(), InPinLabel, /*bIsInput=*/true)))
	{
		PinDataIndices.Reserve(InputDataCollection.TaggedData.Num());

		for (int32 DataIndex = 0; DataIndex < InputDataCollection.TaggedData.Num(); ++DataIndex)
		{
			if (InputDataCollection.TaggedData[DataIndex].Pin == *VirtualLabel)
			{
				PinDataIndices.Add(DataIndex);
			}
		}
	}

	return PinDataIndices;
}

bool UPCGDataBinding::ReadbackInputDataToCPU(int32 InInputDataIndex)
{
	const UPCGProxyForGPUData* DataWithGPUSupport = InputDataCollection.TaggedData.IsValidIndex(InInputDataIndex) ? Cast<UPCGProxyForGPUData>(InputDataCollection.TaggedData[InInputDataIndex].Data) : nullptr;

	if (!DataWithGPUSupport)
	{
		// No work to do, signal completion.
		return true;
	}

	TSharedPtr<FPCGContextHandle> Context = ContextHandle.Pin();

	UPCGProxyForGPUData::FReadbackResult Result = DataWithGPUSupport->GetCPUData(Context ? Context->GetContext() : nullptr);

	if (!Result.bComplete)
	{
		// Read back pending - wait for it to complete.
		return false;
	}

	if (ensure(Result.TaggedData.Data))
	{
		InputDataCollection.TaggedData[InInputDataIndex].Data = Result.TaggedData.Data;
		InputDataCollection.TaggedData[InInputDataIndex].Tags = MoveTemp(Result.TaggedData.Tags);
	}

	return true;
}

bool UPCGDataBinding::InitializeKernelParams(FPCGContext* InContext)
{
	return KernelParamsCache.Initialize(InContext, this);
}

const FPCGKernelParams* UPCGDataBinding::GetCachedKernelParams(const UPCGComputeKernel* InKernel) const
{
	return InKernel ? GetCachedKernelParams(InKernel->GetKernelIndex()) : nullptr;
}

const FPCGKernelParams* UPCGDataBinding::GetCachedKernelParams(int32 InKernelIndex) const
{
	return KernelParamsCache.GetCachedKernelParams(InKernelIndex);
}

TArray<FPCGDataToDebug>& UPCGDataBinding::GetDataToDebugMutable()
{
	return DataToDebug;
}

TArray<FPCGDataToDebug>& UPCGDataBinding::GetDataToInspectMutable()
{
	return DataToInspect;
}

void UPCGDataBinding::AddCompletedMeshSpawnerKernel(TObjectPtr<const UComputeKernel> InCompletedMeshSpawnerKernel)
{
	CompletedMeshSpawners.Add(InCompletedMeshSpawnerKernel);
}

bool UPCGDataBinding::IsMeshSpawnerKernelComplete(TObjectPtr<const UComputeKernel> InMeshSpawnerKernel) const
{
	return CompletedMeshSpawners.Contains(InMeshSpawnerKernel);
}

void UPCGDataBinding::AddMeshSpawnerPrimitives(TObjectPtr<const UComputeKernel> InMeshSpawnerKernel, const FPCGSpawnerPrimitives& Primitives)
{
	MeshSpawnersToPrimitives.Add(InMeshSpawnerKernel, Primitives);
}

void UPCGDataBinding::AddMeshSpawnerPrimitives(TObjectPtr<const UComputeKernel> InMeshSpawnerKernel, FPCGSpawnerPrimitives&& Primitives)
{
	MeshSpawnersToPrimitives.Add(InMeshSpawnerKernel, MoveTempIfPossible(Primitives));
}

FPCGSpawnerPrimitives& UPCGDataBinding::FindOrAddMeshSpawnerPrimitives(TObjectPtr<const UComputeKernel> InMeshSpawnerKernel)
{
	return MeshSpawnersToPrimitives.FindOrAdd(InMeshSpawnerKernel);
}

FPCGSpawnerPrimitives* UPCGDataBinding::FindMeshSpawnerPrimitives(TObjectPtr<const UComputeKernel> InMeshSpawnerKernel)
{
	return MeshSpawnersToPrimitives.Find(InMeshSpawnerKernel);
}

const UPCGComputeGraph* UPCGDataBinding::GetComputeGraph() const
{
	return Graph;
}

const FPCGDataCollection& UPCGDataBinding::GetInputDataCollection() const
{
	return InputDataCollection;
}

FPCGDataCollection& UPCGDataBinding::GetInputDataCollectionMutable()
{
	return InputDataCollection;
}

const FPCGDataCollection& UPCGDataBinding::GetOutputDataCollection() const
{
	return OutputDataCollection;
}

FIntVector UPCGDataBinding::GetMaxNumCullingCells(int32 InCullingCellExtent) const
{
	if (InCullingCellExtent <= 0)
	{
		return FIntVector(1, 1, 1);
	}

	const IPCGGraphExecutionSource* ExecSource = GetExecutionSource();
	if (!ExecSource)
	{
		return FIntVector(1, 1, 1);
	}

	const FBox Bounds = ExecSource->GetExecutionState().GetBounds();
	const double CellExtent = static_cast<double>(InCullingCellExtent);
	const FIntVector LowerLeft(FMath::Floor(Bounds.Min.X / CellExtent), FMath::Floor(Bounds.Min.Y / CellExtent), FMath::Floor(Bounds.Min.Z / CellExtent));
	const FIntVector UpperRight(FMath::Floor(Bounds.Max.X / CellExtent), FMath::Floor(Bounds.Max.Y / CellExtent), FMath::Floor(Bounds.Max.Z / CellExtent));
	return UpperRight - LowerLeft + FIntVector(1, 1, 1);
}

void UPCGDataBinding::AddInputDataAttributesToTable()
{
	if (InputDataCollection.TaggedData.IsEmpty())
	{
		return;
	}

	AttributeTable.Reserve(16); // Heuristic guess at a reasonable upper bound for many cases.

	FPCGKernelAttributeTable::FAttributeKeyToIndexMap AttributeToGraphAttributeIndex = AttributeTable.GetAttributeKeyToIndex();

	for (const FPCGTaggedData& Data : InputDataCollection.TaggedData)
	{
		if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(Data.Data))
		{
			TSharedPtr<const FPCGProxyForGPUDataCollection> DataCollectionInfo = Proxy->GetGPUInfo();

			FPCGDataDesc SourceDesc;
			if (DataCollectionInfo && Proxy->GetDescription(SourceDesc))
			{
				for (const FPCGKernelAttributeDesc& SourceAttrDesc : SourceDesc.GetAttributeDescriptions())
				{
					// Convert from absolute ID to index into metadata attributes.
					const int32 SourceAttributeTableIndex = PCGComputeHelpers::GetMetadataAttributeIndexFromAttributeId(SourceAttrDesc.GetAttributeId());
					if (SourceAttributeTableIndex == INDEX_NONE)
					{
						// Invalid index means the attribute is a property rather than metadata.
						continue;
					}

					int32 GraphAttributeIndex = INDEX_NONE;
					if (int32* FoundGraphAttributeIndex = AttributeToGraphAttributeIndex.Find(SourceAttrDesc.GetAttributeKey()))
					{
						GraphAttributeIndex = *FoundGraphAttributeIndex;
					}
					else
					{
						GraphAttributeIndex = AttributeTable.AddAttributeAssumeUnique(SourceAttrDesc.GetAttributeKey());
						AttributeToGraphAttributeIndex.Add(SourceAttrDesc.GetAttributeKey(), GraphAttributeIndex);
					}

					if (GraphAttributeIndex == INDEX_NONE)
					{
						UE_LOGF(LogPCG, Error, "Attribute table exceeded maximum size (%d), use the 'Dump Data Descriptions' setting on the GPU node(s) to list attributes that are present.", PCGDataCollectionPackingConstants::MAX_NUM_CUSTOM_ATTRS);
						break;
					}

					if (GraphAttributeIndex != SourceAttributeTableIndex)
					{
						SourceBufferAttributeToGraphAttributeIndex.Add({ DataCollectionInfo, SourceAttributeTableIndex }, GraphAttributeIndex);
						GraphAttributeToSourceBufferAttributeIndex.Add({ GraphAttributeIndex, DataCollectionInfo->GetBuffer() }, SourceAttributeTableIndex);
					}
				}
			}
		}
		else if (const UPCGMetadata* Metadata = (Data.Data && PCGComputeHelpers::ShouldImportAttributesFromData(Data.Data)) ? Data.Data->ConstMetadata() : nullptr)
		{
			const FPCGMetadataDomainID MetadataDefaultDomainID = Metadata->GetConstDefaultMetadataDomain()->GetDomainID();
			
			TArray<FPCGAttributeIdentifier> AttributeIdentifiers;
			TArray<EPCGMetadataTypes> AttributeTypes;
			Metadata->GetAllAttributes(AttributeIdentifiers, AttributeTypes);

			// @todo_pcg: Attributes on other domains than the default are ignored at the moment, until we have a better way of representing
			// different domains in the GPU header.
			// It means those are lost.

			for (int Index = 0; Index < AttributeIdentifiers.Num(); ++Index)
			{
				FPCGAttributeIdentifier AttributeIdentifier = AttributeIdentifiers[Index];
				if (AttributeIdentifier.MetadataDomain != PCGMetadataDomainID::Default && AttributeIdentifier.MetadataDomain != MetadataDefaultDomainID)
				{
					continue;
				}

				// If the domain is the default domain, force it to the default identifier.
				if (AttributeIdentifier.MetadataDomain == MetadataDefaultDomainID)
				{
					AttributeIdentifier.MetadataDomain = PCGMetadataDomainID::Default;
				}

				const EPCGKernelAttributeType AttributeType = PCGDataDescriptionHelpers::GetAttributeTypeFromMetadataType(AttributeTypes[Index]);
				const FPCGKernelAttributeKey Key(AttributeIdentifier, AttributeType);

				if (!AttributeToGraphAttributeIndex.Contains(Key))
				{
					const int32 GraphAttributeIndex = AttributeTable.AddAttributeAssumeUnique(Key);

					if (GraphAttributeIndex != INDEX_NONE)
					{
						AttributeToGraphAttributeIndex.Add(Key, GraphAttributeIndex);
					}
					else
					{
						UE_LOGF(LogPCG, Error, "Attribute table exceeded maximum size (%d), use the 'Dump Data Descriptions' setting on the GPU node(s) to list attributes that are present.", PCGDataCollectionPackingConstants::MAX_NUM_CUSTOM_ATTRS);
						break;
					}
				}
			}
		}
	}

	AttributeTable.Shrink();

	ensure(AttributeTable.Num() <= PCGDataCollectionPackingConstants::MAX_NUM_CUSTOM_ATTRS);
}

void UPCGDataBinding::AddInputDataStringsToTable()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDataBinding::AddInputDataStringsToTable);
	check(Graph);

	// Often many data items come from a small number of buffers, so make sure we only grab strings from each buffer once.
	TArray<TSharedPtr<const FPCGProxyForGPUDataCollection>> ProcessedBuffers;

	for (const FPCGTaggedData& Data : InputDataCollection.TaggedData)
	{
		// GPU proxies hold a pointer to GPU buffer info which contains the string table.
		if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(Data.Data))
		{
			if (TSharedPtr<const FPCGProxyForGPUDataCollection> GPUInfo = Proxy->GetGPUInfo())
			{
				if (ProcessedBuffers.Contains(GPUInfo))
				{
					continue;
				}

				ProcessedBuffers.Add(GPUInfo);

				// Add the string table from the upstream compute graph to the current graph string table (skipping 0 - null/empty string). There may
				// be multiple upstream compute graphs with conflicting string key values, so we record a map from the current graph string key to the
				// original source string key and use this map when reading string keys from source buffer.
				for (int SourceBufferStringKey = 1; SourceBufferStringKey < GPUInfo->GetStringTable().Num(); ++SourceBufferStringKey)
				{
					const int GraphStringKey = StringTable.AddUnique(GPUInfo->GetStringTable()[SourceBufferStringKey]);

					if (GraphStringKey != SourceBufferStringKey)
					{
						SourceBufferToGraphStringKey.Add({ GPUInfo->GetBuffer(), SourceBufferStringKey}, GraphStringKey);
						GraphToSourceBufferStringKey.Add({ GPUInfo->GetBuffer(), GraphStringKey }, SourceBufferStringKey);
					}
				}
			}
		}
		// Non-GPU-proxy: collect strings from metadata.
		else
		{
			const UPCGMetadata* Metadata = Data.Data ? Data.Data->ConstMetadata() : nullptr;
			if (!Metadata)
			{
				continue;
			}
			
			const FPCGMetadataDomainID MetadataDefaultDomainID = Metadata->GetConstDefaultMetadataDomain()->GetDomainID();

			TArray<FPCGAttributeIdentifier> AttributeIdentifiers;
			TArray<EPCGMetadataTypes> AttributeTypes;
			Metadata->GetAllAttributes(AttributeIdentifiers, AttributeTypes);

			// Cache the keys to a given domain, so we don't recreate them
			TMap<FPCGMetadataDomainID, TUniquePtr<const IPCGAttributeAccessorKeys>> AllKeys;

			for (int AttributeIndex = 0; AttributeIndex < AttributeIdentifiers.Num(); ++AttributeIndex)
			{
				// @todo_pcg: Attributes on other domains than the default are ignored at the moment, until we have a better way of representing
				// different domains in the GPU header.
				// It means those are lost.
				FPCGAttributeIdentifier AttributeIdentifier = AttributeIdentifiers[AttributeIndex];
				if (AttributeIdentifier.MetadataDomain != PCGMetadataDomainID::Default && AttributeIdentifier.MetadataDomain != MetadataDefaultDomainID)
				{
					continue;
				}

				// If the domain is the default domain, force it to the default identifier.
				if (AttributeIdentifier.MetadataDomain == MetadataDefaultDomainID)
				{
					AttributeIdentifier.MetadataDomain = PCGMetadataDomainID::Default;
				}
				
				const EPCGKernelAttributeType AttributeType = PCGDataDescriptionHelpers::GetAttributeTypeFromMetadataType(AttributeTypes[AttributeIndex]);

				if (AttributeType == EPCGKernelAttributeType::StringKey || AttributeType == EPCGKernelAttributeType::Name)
				{
					const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomain(AttributeIdentifier.MetadataDomain);
					const FPCGMetadataAttributeBase* AttributeBase = MetadataDomain->GetConstAttribute(AttributeIdentifier.Name);
					check(MetadataDomain && AttributeBase);

					const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(AttributeBase, MetadataDomain);
					TUniquePtr<const IPCGAttributeAccessorKeys>& Keys = AllKeys.FindOrAdd(AttributeIdentifier.MetadataDomain);
					if (!Keys.IsValid())
					{
						FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttributeIdentifier.Name);
						Data.Data->SetDomainFromDomainID(AttributeIdentifier.MetadataDomain, Selector);
						Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Data.Data, Selector);
					}

					check(Accessor && Keys);

					PCGMetadataElementCommon::ApplyOnAccessor<FString>(*Keys, *Accessor, [&StringTable = StringTable](FString&& InValue, int32)
					{
						StringTable.AddUnique(std::move(InValue));
					}, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
				}
			}
		}
	}
}

void UPCGDataBinding::AddInputDataTagsToTable()
{
	for (const FPCGTaggedData& Data : InputDataCollection.TaggedData)
	{
		for (const FString& Tag : Data.Tags)
		{
			StringTable.AddUnique(Tag);
		}
	}
}

void UPCGDataBinding::DebugLogDataDescriptions() const
{
#if PCG_KERNEL_LOGGING_ENABLED
	ensure(IsDataDescriptionCachePrimed());

	if (Graph && Graph->bLogDataDescriptions)
	{
		UE_LOGF(LogPCG, Display, "\n### METADATA ATTRIBUTE TABLE ###");
		AttributeTable.DebugLog();

		UE_LOGF(LogPCG, Display, "\n### STRING TABLE ###");
		for (int32 i = 0; i < StringTable.Num(); ++i)
		{
			UE_LOGF(LogPCG, Display, "\t%d: %ls", i, *StringTable[i]);
		}

		UE_LOGF(LogPCG, Display, "\n### INPUT PIN DATA DESCRIPTIONS ###");
		Graph->DebugLogDataDescriptions(this);
	}
#endif
}

// Deprecated
void UPCGDataBinding::ReceiveDataFromGPU_GameThread(UPCGData* InData, const UPCGSettings* InProducerSettings, EPCGExportMode InExportMode, FName InPinLabel, FName InPinLabelAlias)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ReceiveDataFromGPU_GameThread(InData, InProducerSettings, InExportMode, InPinLabel, InPinLabelAlias, /*AdditionalTags*/TSet<FString>());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

// Deprecated
void UPCGDataBinding::ReceiveDataFromGPU_GameThread(UPCGData* InData, const UPCGSettings* InProducerSettings, EPCGExportMode InExportMode, FName InPinLabel, FName InPinLabelAlias, const TSet<FString>& AdditionalTags)
{
	if (!!(InExportMode & EPCGExportMode::ComputeGraphOutput))
	{
		FPCGTaggedData& TaggedData = OutputDataCollection.TaggedData.Emplace_GetRef();
		TaggedData.Pin = InPinLabelAlias;
		TaggedData.Data = InData;
		TaggedData.Tags = AdditionalTags;
	}

	auto SetupDataToDebug = [&](FPCGDataToDebug& Data)
	{
		Data.Data = InData;
		Data.ProducerSettings = InProducerSettings;
		Data.PinLabel = InPinLabel;
		Data.PinLabelAlias = InPinLabelAlias;
		Data.Tags = AdditionalTags;
	};

	if (!!(InExportMode & EPCGExportMode::DebugVisualization))
	{
		SetupDataToDebug(DataToDebug.Emplace_GetRef());
	}

	if (!!(InExportMode & EPCGExportMode::Inspection))
	{
		SetupDataToDebug(DataToInspect.Emplace_GetRef());
	}
}
