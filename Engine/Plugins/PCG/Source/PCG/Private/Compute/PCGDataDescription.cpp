// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGDataDescription.h"

#include "Compute/PCGDataBinding.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/Packing/PCGDataCollectionPacking.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataDescription)

#define LOCTEXT_NAMESPACE "PCGDataDescription"

namespace PCGDataDescriptionConstants
{
	const static FPCGKernelAttributeDesc PointPropertyDescs[PCGDataCollectionPackingConstants::NUM_POINT_PROPERTIES] =
	{
		FPCGKernelAttributeDesc(PCGDataCollectionPackingConstants::POINT_POSITION_ATTRIBUTE_ID,   EPCGKernelAttributeType::Float3, TEXT("$Position")),
		FPCGKernelAttributeDesc(PCGDataCollectionPackingConstants::POINT_ROTATION_ATTRIBUTE_ID,   EPCGKernelAttributeType::Quat,   TEXT("$Rotation")),
		FPCGKernelAttributeDesc(PCGDataCollectionPackingConstants::POINT_SCALE_ATTRIBUTE_ID,      EPCGKernelAttributeType::Float3, TEXT("$Scale")),
		FPCGKernelAttributeDesc(PCGDataCollectionPackingConstants::POINT_BOUNDS_MIN_ATTRIBUTE_ID, EPCGKernelAttributeType::Float3, TEXT("$BoundsMin")),
		FPCGKernelAttributeDesc(PCGDataCollectionPackingConstants::POINT_BOUNDS_MAX_ATTRIBUTE_ID, EPCGKernelAttributeType::Float3, TEXT("$BoundsMax")),
		FPCGKernelAttributeDesc(PCGDataCollectionPackingConstants::POINT_COLOR_ATTRIBUTE_ID,      EPCGKernelAttributeType::Float4, TEXT("$Color")),
		FPCGKernelAttributeDesc(PCGDataCollectionPackingConstants::POINT_DENSITY_ATTRIBUTE_ID,    EPCGKernelAttributeType::Float,  TEXT("$Density")),
		FPCGKernelAttributeDesc(PCGDataCollectionPackingConstants::POINT_SEED_ATTRIBUTE_ID,       EPCGKernelAttributeType::Int,    TEXT("$Seed")),
		FPCGKernelAttributeDesc(PCGDataCollectionPackingConstants::POINT_STEEPNESS_ATTRIBUTE_ID,  EPCGKernelAttributeType::Float,  TEXT("$Steepness"))
	};
}

namespace PCGDataDescriptionHelpers
{
	EPCGKernelAttributeType GetAttributeTypeFromMetadataType(EPCGMetadataTypes MetadataType)
	{
		switch (MetadataType)
		{
		case EPCGMetadataTypes::Boolean:
			return EPCGKernelAttributeType::Bool;
		case EPCGMetadataTypes::Float:
		case EPCGMetadataTypes::Double:
			return EPCGKernelAttributeType::Float;
		case EPCGMetadataTypes::Integer32:
		case EPCGMetadataTypes::Integer64:
			return EPCGKernelAttributeType::Int;
		case EPCGMetadataTypes::Vector2:
			return EPCGKernelAttributeType::Float2;
		case EPCGMetadataTypes::Vector:
			return EPCGKernelAttributeType::Float3;
		case EPCGMetadataTypes::Rotator:
			return EPCGKernelAttributeType::Rotator;
		case EPCGMetadataTypes::Vector4:
			return EPCGKernelAttributeType::Float4;
		case EPCGMetadataTypes::Quaternion:
			return EPCGKernelAttributeType::Quat;
		case EPCGMetadataTypes::Transform:
			return EPCGKernelAttributeType::Transform;
		case EPCGMetadataTypes::SoftObjectPath: // TODO: This collapses all StringKey types into String attributes, meaning we'll lose the original CPU type when doing readback.
		case EPCGMetadataTypes::SoftClassPath:
		case EPCGMetadataTypes::String:
			return EPCGKernelAttributeType::StringKey;
		case EPCGMetadataTypes::Name:
			return EPCGKernelAttributeType::Name;
		default:
			return EPCGKernelAttributeType::Invalid;
		}
	}

	int GetAttributeTypeStrideBytes(EPCGKernelAttributeType Type)
	{
		switch (Type)
		{
		case EPCGKernelAttributeType::Bool:
		case EPCGKernelAttributeType::Int:
		case EPCGKernelAttributeType::Float:
		case EPCGKernelAttributeType::StringKey:
		case EPCGKernelAttributeType::Name:
			return 4;
		case EPCGKernelAttributeType::Float2:
			return 8;
		case EPCGKernelAttributeType::Float3:
		case EPCGKernelAttributeType::Rotator:
			return 12;
		case EPCGKernelAttributeType::Float4:
		case EPCGKernelAttributeType::Quat:
			return 16;
		case EPCGKernelAttributeType::Transform:
			return 64;
		default:
			checkNoEntry();
			return 0;
		}
	}
}

FPCGKernelAttributeKey::FPCGKernelAttributeKey(const FPCGAttributePropertySelector& InSelector, EPCGKernelAttributeType InType)
	: Type(InType)
{
	SetSelector(InSelector);
}

void FPCGKernelAttributeKey::SetSelector(const FPCGAttributePropertySelector& InSelector)
{
	Name.ImportFromOtherSelector(InSelector);
	UpdateIdentifierFromSelector();
}

bool FPCGKernelAttributeKey::UpdateIdentifierFromSelector()
{
	// @todo_pcg: When we support more domains, we'll need to have a way to convert it to a proper identifier.
	// For now, only support default metadata domain, and mark the identifier invalid if it is anything else.
	if (Name.GetSelection() != EPCGAttributePropertySelection::Attribute)
	{
		UE_LOGF(LogPCG, Error, "While updating FPCGKernelAttributeKey with %ls, the selector was not targeting an attribute. Discarded.", *Name.ToString());
		Identifier.MetadataDomain = PCGMetadataDomainID::Invalid;
		return true;
	}

	const FName DomainName = Name.GetDomainName();
	if (DomainName != NAME_None)// && DomainName != PCGDataConstants::DataDomainName)
	{
		//UE_LOGF(LogPCG, Error, "While updating FPCGKernelAttributeKey with %ls, the selector was targeting a domain that is not the @Data domain which is unsupported at the moment. Discarded.", *Name.ToString());
		UE_LOGF(LogPCG, Error, "While updating FPCGKernelAttributeKey with %ls, the selector was targeting a domain which is unsupported at the moment. Discarded.", *Name.ToString());
		Identifier.MetadataDomain = PCGMetadataDomainID::Invalid;
		return true;
	}

	FPCGAttributeIdentifier NewIdentifier{Name.GetAttributeName(), DomainName == PCGDataConstants::DataDomainName ? PCGMetadataDomainID::Data : PCGMetadataDomainID::Default};
	bool bHasChanged = NewIdentifier != Identifier;
	if (bHasChanged)
	{
		Identifier = std::move(NewIdentifier);
	}
	
	return bHasChanged;
}

bool FPCGKernelAttributeKey::IsValid() const
{
	return Identifier.MetadataDomain.IsValid();
}

bool FPCGKernelAttributeKey::operator==(const FPCGKernelAttributeKey& Other) const
{
	return Type == Other.Type && Identifier == Other.Identifier;
}

uint32 GetTypeHash(const FPCGKernelAttributeKey& In)
{
	return HashCombine(GetTypeHash(In.Type), GetTypeHash(In.Identifier));
}

int32 FPCGKernelAttributeTable::GetAttributeId(const FPCGKernelAttributeKey& InAttribute) const
{
	const int TableIndex = AttributeTable.IndexOfByKey(InAttribute);
	if (TableIndex >= 0)
	{
		return PCGComputeHelpers::GetAttributeIdFromMetadataAttributeIndex(TableIndex);
	}
	else
	{
		// Attribute with the given name and type is not present. If this is unexpected, check this attribute table for attributes
		// in case the supplied name or type is wrong, and ensure any attribute created by a kernel is declared using GetKernelAttributeKeys().
		return INDEX_NONE;
	}
}

int32 FPCGKernelAttributeTable::GetAttributeId(FPCGAttributeIdentifier InIdentifier, EPCGKernelAttributeType InType) const
{
	return GetAttributeId(FPCGKernelAttributeKey(InIdentifier, InType));
}

int32 FPCGKernelAttributeTable::AddAttribute(const FPCGKernelAttributeKey& Key)
{
	int32 Index = AttributeTable.Find(Key);

	if (Index == INDEX_NONE && AttributeTable.Num() < PCGDataCollectionPackingConstants::MAX_NUM_CUSTOM_ATTRS)
	{
		Index = AttributeTable.Num();
		AttributeTable.Add(Key);
	}

	return Index;
}

int32 FPCGKernelAttributeTable::AddAttribute(FPCGAttributeIdentifier InIdentifier, EPCGKernelAttributeType InType)
{
	return AddAttribute(FPCGKernelAttributeKey(InIdentifier, InType));
}

int32 FPCGKernelAttributeTable::AddAttributeAssumeUnique(const FPCGKernelAttributeKey& Key)
{
	if (AttributeTable.Num() < PCGDataCollectionPackingConstants::MAX_NUM_CUSTOM_ATTRS)
	{
		return AttributeTable.Add(Key);
	}
	else
	{
		return INDEX_NONE;
	}
}

FPCGKernelAttributeTable::FAttributeKeyToIndexMap FPCGKernelAttributeTable::GetAttributeKeyToIndex() const
{
	FAttributeKeyToIndexMap AttributeToIndex;
	AttributeToIndex.Reserve(AttributeTable.Num());

	for (int Index = 0; Index < AttributeTable.Num(); ++Index)
	{
		AttributeToIndex.Add(AttributeTable[Index], Index);
	}

	return AttributeToIndex;
}

#if PCG_KERNEL_LOGGING_ENABLED
void FPCGKernelAttributeTable::DebugLog() const
{
	const UEnum* PCGKernelAttributeTypeEnum = StaticEnum<EPCGKernelAttributeType>();

	for (const FPCGKernelAttributeKey& Attribute : AttributeTable)
	{
		UE_LOGF(LogPCG, Display, "\tName: %ls\t\tID: %d\t\tType: %ls",
			*Attribute.GetIdentifier().ToString(),
			GetAttributeId(Attribute),
			*PCGKernelAttributeTypeEnum->GetDisplayNameTextByValue(static_cast<int64>(Attribute.GetType())).ToString());
	}
}
#endif

void FPCGKernelAttributeDesc::SetAttributeId(int32 InAttributeId)
{
	AttributeId = InAttributeId;
}

void FPCGKernelAttributeDesc::AddUniqueStringKeys(const TArray<int32>& InOtherStringKeys)
{
	for (int32 OtherStringKey : InOtherStringKeys)
	{
		UniqueStringKeys.AddUnique(OtherStringKey);
	}
}

void FPCGKernelAttributeDesc::SetStringKeys(const TConstArrayView<int32>& InStringKeys)
{
	UniqueStringKeys = InStringKeys;
}

bool FPCGKernelAttributeDesc::operator==(const FPCGKernelAttributeDesc& Other) const
{
	return AttributeId == Other.AttributeId && AttributeKey == Other.AttributeKey;
}

FPCGDataDesc::FPCGDataDesc(FPCGDataTypeIdentifier InType, int32 InElementCount)
	: FPCGDataDesc(InType, FIntVector4(InElementCount, 0, 0, 0))
{
}

FPCGDataDesc::FPCGDataDesc(FPCGDataTypeIdentifier InType, FIntPoint InElementCount)
	: FPCGDataDesc(InType, FIntVector4(InElementCount.X, InElementCount.Y, 0, 0))
{
}

FPCGDataDesc::FPCGDataDesc(FPCGDataTypeIdentifier InType, FIntVector3 InElementCount)
	: FPCGDataDesc(InType, FIntVector4(InElementCount.X, InElementCount.Y, InElementCount.Z, 0))
{
}

FPCGDataDesc::FPCGDataDesc(FPCGDataTypeIdentifier InType, FIntVector4 InElementCount)
	: Type(InType)
	, ElementCount(InElementCount)
{
	ElementDimension = PCGComputeHelpers::GetElementDimension(InType);

	InitializeAttributeDescs(/*InData=*/nullptr, /*InBinding*/nullptr);
}

FPCGDataDesc::FPCGDataDesc(const FPCGTaggedData& InTaggedData, const UPCGDataBinding* InBinding)
{
	check(InTaggedData.Data);

	Type = InTaggedData.Data->GetDataTypeId();
	ElementCount = PCGComputeHelpers::GetElementCount(InTaggedData.Data);
	ElementDimension = PCGComputeHelpers::GetElementDimension(InTaggedData.Data);

	// @todo_pcg: Implement a way for a UPCGData to generate its data description.
	if (const UPCGTexture2DBaseData* TextureData = Cast<UPCGTexture2DBaseData>(InTaggedData.Data))
	{
		SetTextureArraySize(TextureData->GetArraySize());
		SetRenderTargetFormat(PCGComputeHelpers::GetPCGRenderTargetFormatFromPixelFormat(TextureData->GetFormat()));
		SetTextureFilter(TextureData->GetFilter());
		SetTextureTransform(TextureData->GetTransform());
	}

	TagStringKeys.Reserve(InTaggedData.Tags.Num());

	for (const FString& Tag : InTaggedData.Tags)
	{
		const int Index = InBinding->GetStringTable().IndexOfByKey(Tag);
		if (Index != INDEX_NONE)
		{
			TagStringKeys.Add(Index);
		}
	}

	InitializeAttributeDescs(InTaggedData.Data, InBinding);
}

void FPCGDataDesc::DeallocateProperties(EPCGPointNativeProperties InProperties)
{
	// Never deallocate Density - it is used to remove points.
	AllocatedPointProperties &= ~(InProperties & ~EPCGPointNativeProperties::Density);
}

bool FPCGDataDesc::HasElementsMetadataDomainAttributes() const
{
	return AttributeDescs.ContainsByPredicate([](const FPCGKernelAttributeDesc& AttributeDesc)
	{
		return AttributeDesc.GetAttributeId() >= PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS && AttributeDesc.GetAttributeKey().GetIdentifier().MetadataDomain != PCGMetadataDomainID::Data;
	});
}

bool FPCGDataDesc::ContainsAttribute(FPCGAttributeIdentifier InAttributeIdentifier) const
{
	return AttributeDescs.ContainsByPredicate([InAttributeIdentifier](const FPCGKernelAttributeDesc& AttributeDesc)
	{
		return AttributeDesc.GetAttributeKey().GetIdentifier() == InAttributeIdentifier;
	});
}

bool FPCGDataDesc::ContainsAttribute(FPCGAttributeIdentifier InAttributeIdentifier, EPCGKernelAttributeType InAttributeType) const
{
	return AttributeDescs.ContainsByPredicate([InAttributeKey = FPCGKernelAttributeKey(InAttributeIdentifier, InAttributeType)](const FPCGKernelAttributeDesc& AttributeDesc)
	{
		return AttributeDesc.GetAttributeKey() == InAttributeKey;
	});
}

void FPCGDataDesc::AddAttribute(FPCGKernelAttributeKey InAttribute, const UPCGDataBinding* InBinding, const TArray<int32>* InOptionalUniqueStringKeys)
{
	check(InBinding);

	const int32 AttributeId = InBinding->GetAttributeId(InAttribute);

	if (AttributeId == INDEX_NONE)
	{
		return;
	}

	// Remove existing attributes if they collide with the new attribute (that is, if they have the same Identifier). The new attribute will stomp them,
	// which is consistent with the general behavior on CPU. The node is authoritative on its attributes types.
	// @todo_pcg: We should probably encapsulate the AttributeDescs, and they can only be added through AddAttribute() to ensure consistent behavior (w.r.t. stomping existing attributes).
	for (int AttributeIndex = AttributeDescs.Num() - 1; AttributeIndex >= 0; --AttributeIndex)
	{
		FPCGKernelAttributeDesc& ExistingAttributeDesc = AttributeDescs[AttributeIndex];

		if (ExistingAttributeDesc.GetAttributeKey().GetIdentifier() == InAttribute.GetIdentifier())
		{
			AttributeDescs.RemoveAtSwap(AttributeIndex);
		}
	}

	AttributeDescs.Emplace(AttributeId, InAttribute.GetType(), InAttribute.GetIdentifier(), InOptionalUniqueStringKeys);
}

FIntVector4 FPCGDataDesc::GetElementCountForAttribute(const FPCGKernelAttributeDesc& AttributeDesc) const
{
	// A buffer that is not fully allocated uses only a single value to represent all elements.
	// Data domain only has a single element.
	if (!IsAttributeAllocated(AttributeDesc) || AttributeDesc.GetAttributeKey().GetIdentifier().MetadataDomain == PCGMetadataDomainID::Data)
	{
		return FIntVector4(1, 0, 0, 0);
	}
	else
	{
		return ElementCount;
	}
}

int32 FPCGDataDesc::ComputeTotalElementCount() const
{
	switch (ElementDimension)
	{
	default: // Fallthrough
	case EPCGElementDimension::One:
		return ElementCount.X;
	case EPCGElementDimension::Two:
		return ElementCount.X * ElementCount.Y;
	case EPCGElementDimension::Three:
		return ElementCount.X * ElementCount.Y * ElementCount.Z;
	case EPCGElementDimension::Four:
		return ElementCount.X * ElementCount.Y * ElementCount.Z * ElementCount.W;
	}
}

void FPCGDataDesc::AddElementCount(int32 InElementCountToAdd)
{
	ensure(ElementDimension == EPCGElementDimension::One);
	ElementCount.X += InElementCountToAdd;
}

void FPCGDataDesc::AddElementCount(FIntPoint InElementCountToAdd)
{
	ensure(ElementDimension == EPCGElementDimension::Two);
	ElementCount.X += InElementCountToAdd.X;
	ElementCount.Y += InElementCountToAdd.Y;
}

void FPCGDataDesc::AddElementCount(FIntVector3 InElementCountToAdd)
{
	ensure(ElementDimension == EPCGElementDimension::Three);
	ElementCount.X += InElementCountToAdd.X;
	ElementCount.Y += InElementCountToAdd.Y;
	ElementCount.Z += InElementCountToAdd.Z;
}

void FPCGDataDesc::AddElementCount(FIntVector4 InElementCountToAdd)
{
	ensure(ElementDimension == EPCGElementDimension::Four);
	ElementCount += InElementCountToAdd;
}

void FPCGDataDesc::ScaleElementCount(int32 InMultiplier)
{
	ElementCount *= InMultiplier;
}

FPCGDataDesc& FPCGDataDesc::CombineElementCount(const FPCGDataDesc& Other, EPCGElementMultiplicity Multiplicity)
{
	if (Multiplicity == EPCGElementMultiplicity::Sum)
	{
		if (ElementDimension == EPCGElementDimension::One)
		{
			// Special case for 1D processing, sum by collapsing the other element count into a single value.
			ElementCount.X += Other.ComputeTotalElementCount();
		}
		else if (ElementDimension == EPCGElementDimension::Two)
		{
			if (Other.ElementDimension == EPCGElementDimension::One)
			{
				ElementCount.X += Other.ElementCount.X;
				ElementCount.Y += Other.ElementCount.X;
			}
			else
			{
				ElementCount += Other.ElementCount;
				ElementCount.Z = 0;
				ElementCount.W = 0;
			}
		}
		else if (ElementDimension == EPCGElementDimension::Three)
		{
			if (Other.ElementDimension == EPCGElementDimension::One)
			{
				ElementCount.X += Other.ElementCount.X;
				ElementCount.Y += Other.ElementCount.X;
				ElementCount.Z += Other.ElementCount.X;
			}
			else
			{
				ElementCount += Other.ElementCount;
				ElementCount.W = 0;
			}
		}
		else if (ElementDimension == EPCGElementDimension::Four)
		{
			if (Other.ElementDimension == EPCGElementDimension::One)
			{
				ElementCount.X += Other.ElementCount.X;
				ElementCount.Y += Other.ElementCount.X;
				ElementCount.Z += Other.ElementCount.X;
				ElementCount.W += Other.ElementCount.X;
			}
			else
			{
				ElementCount += Other.ElementCount;
			}
		}
	}
	else
	{
		if (ElementDimension == EPCGElementDimension::One)
		{
			// Special case for 1D processing, scale by collapsing the other element count into a single value.
			ElementCount.X *= Other.ComputeTotalElementCount();
		}
		else if (ElementDimension == EPCGElementDimension::Two)
		{
			if (Other.ElementDimension == EPCGElementDimension::One)
			{
				ElementCount *= Other.ElementCount.X;
			}
			else
			{
				ElementCount *= Other.ElementCount;
			}
		}
		else if (ElementDimension == EPCGElementDimension::Three)
		{
			if (Other.ElementDimension == EPCGElementDimension::One)
			{
				ElementCount *= Other.ElementCount.X;
			}
			else
			{
				ElementCount *= Other.ElementCount;
			}
		}
		else if (ElementDimension == EPCGElementDimension::Four)
		{
			if (Other.ElementDimension == EPCGElementDimension::One)
			{
				ElementCount *= Other.ElementCount.X;
			}
			else
			{
				ElementCount *= Other.ElementCount;
			}
		}
	}

	return *this;
}

FPCGDataDesc FPCGDataDesc::CombineElementCount(const FPCGDataDesc& A, const FPCGDataDesc& B, EPCGElementMultiplicity Multiplicity)
{
	FPCGDataDesc Desc = A;
	Desc.CombineElementCount(B, Multiplicity);
	return Desc;
}

bool FPCGDataDesc::IsAttributeAllocated(int32 InAttributeId) const
{
	// @todo_pcg: Only point properties are supported for single value ranges. All other attributes are fully allocated.
	if (Type == FPCGDataTypeInfoPoint::AsId() && InAttributeId < PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS)
	{
		switch (InAttributeId)
		{
		case PCGDataCollectionPackingConstants::POINT_POSITION_ATTRIBUTE_ID:
			return !!(AllocatedPointProperties & EPCGPointNativeProperties::Transform);
		case PCGDataCollectionPackingConstants::POINT_ROTATION_ATTRIBUTE_ID:
			return !!(AllocatedPointProperties & EPCGPointNativeProperties::Transform);
		case PCGDataCollectionPackingConstants::POINT_SCALE_ATTRIBUTE_ID:
			return !!(AllocatedPointProperties & EPCGPointNativeProperties::Transform);
		case PCGDataCollectionPackingConstants::POINT_BOUNDS_MIN_ATTRIBUTE_ID:
			return !!(AllocatedPointProperties & EPCGPointNativeProperties::BoundsMin);
		case PCGDataCollectionPackingConstants::POINT_BOUNDS_MAX_ATTRIBUTE_ID:
			return !!(AllocatedPointProperties & EPCGPointNativeProperties::BoundsMax);
		case PCGDataCollectionPackingConstants::POINT_COLOR_ATTRIBUTE_ID:
			return !!(AllocatedPointProperties & EPCGPointNativeProperties::Color);
		case PCGDataCollectionPackingConstants::POINT_SEED_ATTRIBUTE_ID:
			return !!(AllocatedPointProperties & EPCGPointNativeProperties::Seed);
		case PCGDataCollectionPackingConstants::POINT_STEEPNESS_ATTRIBUTE_ID:
			return !!(AllocatedPointProperties & EPCGPointNativeProperties::Steepness);
		// Note: Because density is used to represent invalid points, it must always be allocated.
		case PCGDataCollectionPackingConstants::POINT_DENSITY_ATTRIBUTE_ID:
			return true;
		default:
			return true;
		}
	}

	return true;
}

TArrayView<int32> FPCGDataDesc::GetTagStringKeysMutable()
{
	return MakeArrayView(TagStringKeys);
}

void FPCGDataDesc::InitializeAttributeDescs(const UPCGData* InData, const UPCGDataBinding* InBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataDesc::InitializeAttributeDescs);

	if (Type == FPCGDataTypeInfoPoint::AsId())
	{
		AttributeDescs.Append(PCGDataDescriptionConstants::PointPropertyDescs, PCGDataCollectionPackingConstants::NUM_POINT_PROPERTIES);

		if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(InData))
		{
			AllocatedPointProperties = PointData->GetAllocatedProperties();
		}
	}
	else { /* TODO: More types! */ }

	const UPCGMetadata* Metadata = (InData && PCGComputeHelpers::ShouldImportAttributesFromData(InData)) ? InData->ConstMetadata() : nullptr;

	if (InBinding && Metadata)
	{
		const FPCGMetadataDomainID MetadataDefaultDomainID = Metadata->GetConstDefaultMetadataDomain()->GetDomainID();
		
		TArray<FPCGAttributeIdentifier> AttributeIdentifiers;
		TArray<EPCGMetadataTypes> AttributeTypes;
		Metadata->GetAllAttributes(AttributeIdentifiers, AttributeTypes);

		// Cache the keys to a given domain, so we don't recreate them
		TMap<FPCGMetadataDomainID, TUniquePtr<const IPCGAttributeAccessorKeys>> AllKeys;

		for (int CustomAttributeIndex = 0; CustomAttributeIndex < AttributeIdentifiers.Num(); ++CustomAttributeIndex)
		{
			// @todo_pcg: Attributes on other domains than the default are ignored at the moment, until we have a better way of representing
			// different domains in the GPU header.
			// It means those are lost.
			FPCGAttributeIdentifier AttributeIdentifier = AttributeIdentifiers[CustomAttributeIndex];
			if (AttributeIdentifier.MetadataDomain != PCGMetadataDomainID::Default && AttributeIdentifier.MetadataDomain != MetadataDefaultDomainID)
			{
				continue;
			}

			// If the domain is the default domain, force it to the default identifier.
			if (AttributeIdentifier.MetadataDomain == MetadataDefaultDomainID)
			{
				AttributeIdentifier.MetadataDomain = PCGMetadataDomainID::Default;
			}
			
			const EPCGKernelAttributeType AttributeType = PCGDataDescriptionHelpers::GetAttributeTypeFromMetadataType(AttributeTypes[CustomAttributeIndex]);

			if (AttributeType == EPCGKernelAttributeType::Invalid)
			{
				const UEnum* EnumClass = StaticEnum<EPCGMetadataTypes>();
				check(EnumClass);

				FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttributeIdentifier.Name);
				InData->SetDomainFromDomainID(AttributeIdentifier.MetadataDomain, Selector);

				UE_LOGF(LogPCG, Warning, "Skipping attribute '%ls'. '%ls' type attributes are not supported on GPU.",
					*Selector.ToString(),
					*EnumClass->GetNameStringByValue(static_cast<int64>(AttributeTypes[CustomAttributeIndex])));

				continue;
			}

			// Ignore excess attributes.
			if (CustomAttributeIndex >= PCGDataCollectionPackingConstants::MAX_NUM_CUSTOM_ATTRS)
			{
				// TODO: Would be nice to include the pin label for debug purposes
				UE_LOGF(LogPCG, Warning, "Attempted to exceed max number of custom attributes (%d). Additional attributes will be ignored.", PCGDataCollectionPackingConstants::MAX_NUM_CUSTOM_ATTRS);
				break;
			}

			TArray<int32> UniqueStringKeys;

			if (AttributeType == EPCGKernelAttributeType::StringKey || AttributeType == EPCGKernelAttributeType::Name)
			{
				const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomain(AttributeIdentifier.MetadataDomain);
				const FPCGMetadataAttributeBase* AttributeBase = MetadataDomain->GetConstAttribute(AttributeIdentifier.Name);
				check(MetadataDomain && AttributeBase);

				const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(AttributeBase, MetadataDomain);
				TUniquePtr<const IPCGAttributeAccessorKeys>& Keys = AllKeys.FindOrAdd(AttributeIdentifier.MetadataDomain);
				if (!Keys.IsValid())
				{
					FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttributeIdentifier.Name);
					InData->SetDomainFromDomainID(AttributeIdentifier.MetadataDomain, Selector);
					Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InData, Selector);
				}

				check(Accessor && Keys);

				PCGMetadataElementCommon::ApplyOnAccessor<FString>(*Keys, *Accessor, [&UniqueStringKeys, &StringTable = InBinding->GetStringTable()](const FString& InValue, int32)
				{
					const int StringTableIndex = StringTable.IndexOfByKey(InValue);
					if (StringTableIndex != INDEX_NONE)
					{
						UniqueStringKeys.AddUnique(StringTableIndex);
					}
				});
			}

			const int32 AttributeId = InBinding->GetAttributeId(AttributeIdentifier, AttributeType);
			ensureMsgf(AttributeId != INDEX_NONE, TEXT("Attribute '%s' type %d was missing from attribute table."), *AttributeIdentifier.Name.ToString(), (int32)AttributeType);

			if (AttributeId != INDEX_NONE)
			{
				AttributeDescs.Emplace(AttributeId, AttributeType, AttributeIdentifier, &UniqueStringKeys);
			}
		}
	}
}

TSharedPtr<FPCGDataCollectionDesc> FPCGDataCollectionDesc::MakeShared()
{
	return MakeShareable(new FPCGDataCollectionDesc());
}

TSharedPtr<FPCGDataCollectionDesc> FPCGDataCollectionDesc::MakeSharedFrom(const TSharedPtr<const FPCGDataCollectionDesc> InOtherDataDesc)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataCollectionDesc::MakeSharedFrom);

	TSharedPtr<FPCGDataCollectionDesc> NewDesc = MakeShareable(new FPCGDataCollectionDesc());

	if (InOtherDataDesc)
	{
		NewDesc->DataDescs = InOtherDataDesc->DataDescs;
	}

	return NewDesc;
}

uint32 FPCGDataCollectionDesc::ComputeTotalElementCount() const
{
	uint32 ProcessingElementCount = 0;

	for (const FPCGDataDesc& DataDesc : DataDescs)
	{
		ProcessingElementCount += DataDesc.ComputeTotalElementCount();
	}

	return ProcessingElementCount;
}

bool FPCGDataCollectionDesc::GetAttributeDesc(FPCGAttributeIdentifier InAttributeIdentifier, FPCGKernelAttributeDesc& OutAttributeDesc, bool& bOutConflictingTypesFound, bool& bOutPresentOnAllData) const
{
	// Will be set to the type of the first attribute across all the data which has a matching name, and then is used to detect if subsequent
	// attributes are found with conflicting type.
	EPCGKernelAttributeType FoundType = EPCGKernelAttributeType::Invalid;
	bOutPresentOnAllData = false;

	bool bFoundOnAllData = true;

	for (const FPCGDataDesc& DataDesc : DataDescs)
	{
		bool bFoundOnData = false;

		for (const FPCGKernelAttributeDesc& AttributeDesc : DataDesc.GetAttributeDescriptions())
		{
			if (AttributeDesc.GetAttributeKey().GetIdentifier() == InAttributeIdentifier && ensure(AttributeDesc.GetAttributeKey().GetType() != EPCGKernelAttributeType::Invalid))
			{
				bFoundOnData = true;

				if (FoundType == EPCGKernelAttributeType::Invalid)
				{
					// Take the first matching attribute.
					OutAttributeDesc = AttributeDesc;
					FoundType = AttributeDesc.GetAttributeKey().GetType();
				}
				else if (FoundType != AttributeDesc.GetAttributeKey().GetType())
				{
					// Signal conflict found.
					bOutConflictingTypesFound = true;

					// Can stop iterating once we find a conflict. Signal that a matching attribute was found (despite the conflict).
					return true;
				}
			}
		}

		bFoundOnAllData &= bFoundOnData;
	}

	// Signal no conflict found.
	bOutConflictingTypesFound = false;

	bOutPresentOnAllData = bFoundOnAllData;

	// Signal attribute found or not.
	return FoundType != EPCGKernelAttributeType::Invalid;
}

bool FPCGDataCollectionDesc::ContainsAttributeOnAnyData(FPCGAttributeIdentifier InAttributeIdentifier) const
{
	return DataDescs.ContainsByPredicate([InAttributeIdentifier](const FPCGDataDesc& DataDesc)
	{
		return DataDesc.ContainsAttribute(InAttributeIdentifier);
	});
}

void FPCGDataCollectionDesc::AddAttributeToAllData(FPCGKernelAttributeKey InAttribute, const UPCGDataBinding* InBinding, const TArray<int32>* InOptionalUniqueStringKeys)
{
	for (FPCGDataDesc& DataDesc : DataDescs)
	{
		DataDesc.AddAttribute(InAttribute, InBinding, InOptionalUniqueStringKeys);
	}
}

void FPCGDataCollectionDesc::AllocatePropertiesForAllData(EPCGPointNativeProperties InProperties)
{
	for (FPCGDataDesc& DataDesc : DataDescs)
	{
		DataDesc.AllocateProperties(InProperties);
	}
}

void FPCGDataCollectionDesc::DeallocatePropertiesForAllData(EPCGPointNativeProperties InProperties)
{
	for (FPCGDataDesc& DataDesc : DataDescs)
	{
		DataDesc.DeallocateProperties(InProperties);
	}
}

EPCGPointNativeProperties FPCGDataCollectionDesc::GetAllocatedPropertiesUnion() const
{
	EPCGPointNativeProperties Result = EPCGPointNativeProperties::None;
	for (const FPCGDataDesc& DataDesc : DataDescs)
	{
		Result |= DataDesc.GetAllocatedProperties();
	}

	return Result;
}

void FPCGDataCollectionDesc::GetUniqueStringKeyValues(int32 InAttributeId, TArray<int32>& OutUniqueStringKeys) const
{
	for (const FPCGDataDesc& DataDesc : DataDescs)
	{
		for (const FPCGKernelAttributeDesc& AttributeDesc : DataDesc.GetAttributeDescriptions())
		{
			if (AttributeDesc.GetAttributeId() == InAttributeId)
			{
				for (int32 StringKey : AttributeDesc.GetUniqueStringKeys())
				{
					OutUniqueStringKeys.AddUnique(StringKey);
				}

				break;
			}
		}
	}
}

int FPCGDataCollectionDesc::GetNumStringKeyValues(int32 InAttributeId) const
{
	TArray<int32> UniqueStringKeys;
	GetUniqueStringKeyValues(InAttributeId, UniqueStringKeys);

	return UniqueStringKeys.Num();
}

void FPCGDataCollectionDesc::SetRenderTargetFormatForAllData(EPCGRenderTargetFormat InRenderTargetFormat)
{
	for (FPCGDataDesc& DataDesc : DataDescs)
	{
		DataDesc.SetRenderTargetFormat(InRenderTargetFormat);
	}
}

void FPCGDataCollectionDesc::SetTextureFilterForAllData(EPCGTextureFilter InTextureFilter)
{
	for (FPCGDataDesc& DataDesc : DataDescs)
	{
		DataDesc.SetTextureFilter(InTextureFilter);
	}
}

void FPCGDataCollectionDesc::SetTextureArraySizeForAllData(uint16 InTextureArraySize)
{
	for (FPCGDataDesc& DataDesc : DataDescs)
	{
		DataDesc.SetTextureArraySize(InTextureArraySize);
	}
}

void FPCGDataCollectionDesc::SetTextureTransformForAllData(const FTransform& InTransform)
{
	for (FPCGDataDesc& DataDesc : DataDescs)
	{
		DataDesc.SetTextureTransform(InTransform);
	}
}

#undef LOCTEXT_NAMESPACE
