// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGDataUploadAdaptor.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGKernelHelpers.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/Elements/PCGComputeGraphElement.h"
#include "Compute/Packing/PCGDataCollectionPacking.h"

#include "RenderCaptureInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "SystemTextures.h"

FPCGDataUploadAdaptor::FPCGDataUploadAdaptor(UPCGDataBinding* InDataBinding, const TSharedPtr<const FPCGDataCollectionDesc> InTargetDataCollectionDesc, FName InInputPinLabel)
{
	check(IsInGameThread());
	check(InTargetDataCollectionDesc);

	TargetDataCollectionDesc = InTargetDataCollectionDesc;
	InputPinLabel = InInputPinLabel;
	DataBinding = InDataBinding;

	int NumDataForThisPin = 0;
	int FirstDataForThisPin = INDEX_NONE;

	// Are all our data items coming direct from GPU - and importantly are a complete, single data collection?
	bool bMultipleBuffersEncountered = false;
	TSharedPtr<const FPCGProxyForGPUDataCollection> UniqueDataCollection;
	for (int InputDataIndex = 0; InputDataIndex < DataBinding->GetInputDataCollection().TaggedData.Num(); ++InputDataIndex)
	{
		const FName DataPin = DataBinding->GetInputDataCollection().TaggedData[InputDataIndex].Pin;

		if (InInputPinLabel != DataPin)
		{
			continue;
		}

		++NumDataForThisPin;

		if (FirstDataForThisPin == INDEX_NONE)
		{
			FirstDataForThisPin = InputDataIndex;
		}

		const UPCGProxyForGPUData* DataGPU = Cast<UPCGProxyForGPUData>(DataBinding->GetInputDataCollection().TaggedData[InputDataIndex].Data);
		if (!DataGPU)
		{
			// Encountered CPU data, no reuse for now.
			// TODO: Support multiple input buffers and only upload required data.
			UniqueDataCollection.Reset();
			break;
		}

		TSharedPtr<const FPCGProxyForGPUDataCollection> InputDataCollection = DataGPU->GetInputDataCollectionInfo();
		if (!InputDataCollection)
		{
			UE_LOGF(LogPCG, Error, "No reuse: Missing data collection buffer!");

			UniqueDataCollection.Reset();
			break;
		}

		if (!UniqueDataCollection)
		{
			UniqueDataCollection = InputDataCollection;
		}
		else if (UniqueDataCollection != InputDataCollection || DataGPU->GetDataIndexInCollection() != (InputDataIndex - FirstDataForThisPin))
		{
			// Multiple input buffers or out of order data items, no reuse for now.
			// TODO: support indirection of data items.
			UniqueDataCollection.Reset();
			break;
		}
	}

	if (UniqueDataCollection && UniqueDataCollection->GetDescription() && UniqueDataCollection->GetDescription()->GetDataDescriptions().Num() != NumDataForThisPin)
	{
		// We're not using all the data items from the buffer, no reuse for now.
		// TODO: Support indirection of data items.
		UniqueDataCollection.Reset();
	}

	if (UniqueDataCollection && UniqueDataCollection->GetDescription())
	{
		ExternalBufferForReuse = UniqueDataCollection->GetBuffer();
		ExternalBufferSizeBytes = UniqueDataCollection->GetBufferSizeBytes();

		ensureAlways(ExternalBufferMaxStringKeyValue == INDEX_NONE);

		for (const FPCGDataDesc& DataDesc : UniqueDataCollection->GetDescription()->GetDataDescriptions())
		{
			for (const FPCGKernelAttributeDesc& AttrDesc : DataDesc.GetAttributeDescriptions())
			{
				for (int32 StringKeyValue : AttrDesc.GetUniqueStringKeys())
				{
					ExternalBufferMaxStringKeyValue = FMath::Max(ExternalBufferMaxStringKeyValue, StringKeyValue);
				}
			}
		}
	}
}

/** Do any preparation work such as data readbacks. Returns true when preparation is complete. */
bool FPCGDataUploadAdaptor::PrepareData_GameThread(const UPCGComputeKernel* InProducerKernel, const UPCGSettings* InProducerSettings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataUploadAdaptor::PrepareData_GameThread);

	UPCGDataBinding* Binding = DataBinding.Get();
	if (!ensure(Binding))
	{
		return true;
	}

	check(IsInGameThread());

	if (ExternalBufferForReuse)
	{
		ensureAlways(GraphToSourceBufferAttributeIndex.IsEmpty());
		const int32 NumGraphAttributes = Binding->GetAttributeTableSize();
		GraphToSourceBufferAttributeIndex.SetNumUninitialized(NumGraphAttributes);

		// Now compute the mapping for any metadata attributes.
		for (int32 GraphAttributeIndex = 0; GraphAttributeIndex < NumGraphAttributes; ++GraphAttributeIndex)
		{
			const int32* SourceBufferAttributeIndex = Binding->GetGraphAttributeToSourceBufferAttributeIndex().Find({ GraphAttributeIndex, ExternalBufferForReuse });

			GraphToSourceBufferAttributeIndex[GraphAttributeIndex] = SourceBufferAttributeIndex ? *SourceBufferAttributeIndex : GraphAttributeIndex;
		}

		ensureAlways(BufferToGraphStringKey.IsEmpty());

		if (ExternalBufferMaxStringKeyValue != INDEX_NONE)
		{
			const int32 NumSourceStringKeys = ExternalBufferMaxStringKeyValue + 1;
			BufferToGraphStringKey.SetNumUninitialized(NumSourceStringKeys);

			// Now compute the mapping for any string keys.
			BufferToGraphStringKey[0] = 0;
			for (int32 SourceBufferStringKey = 1; SourceBufferStringKey < NumSourceStringKeys; ++SourceBufferStringKey)
			{
				const int32* GraphStringKeyIndex = Binding->GetSourceBufferToGraphStringKey().Find({ ExternalBufferForReuse, SourceBufferStringKey });

				BufferToGraphStringKey[SourceBufferStringKey] = GraphStringKeyIndex ? *GraphStringKeyIndex : SourceBufferStringKey;
			}
		}
	}
	else
	{
		if (!PackDataCollectionTask.IsValid())
		{
			const uint64 BufferSize = PCGDataCollectionPackingHelpers::ComputePackedSizeBytes(TargetDataCollectionDesc);
			if (PCGComputeHelpers::IsBufferSizeTooLarge(BufferSize))
			{
				return true;
			}

			bool bAllDataReady = true;

			for (FPCGTaggedData& TaggedData : Binding->GetInputDataCollectionMutable().TaggedData)
			{
				if (TaggedData.Pin != InputPinLabel)
				{
					continue;
				}

				if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(TaggedData.Data))
				{
#if WITH_EDITOR
					PCGComputeHelpers::NotifyGPUToCPUReadback(Binding, InProducerKernel, InProducerSettings);
#endif

					UPCGProxyForGPUData::FReadbackResult Result = Proxy->GetCPUData(/*InContext=*/nullptr);

					if (!Result.bComplete)
					{
						bAllDataReady = false;
					}
					else
					{
						TaggedData.Data = Result.TaggedData.Data;
						TaggedData.Tags = MoveTemp(Result.TaggedData.Tags);
					}
				}
			}

			if (!bAllDataReady)
			{
				return false;
			}

			// Kick off async task to pack data collection as this can be a heavy operation.
			PackDataCollectionTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [ContextHandle=Binding->GetContextHandle(), ThisWeak=AsWeak()]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(PackDataCollectionTask);
				LLM_SCOPE_BYTAG(PCG);

				FPCGContext::FSharedContext<FPCGComputeGraphContext> SharedContext(ContextHandle);
				FPCGComputeGraphContext* Context = SharedContext.Get();
				if (!Context)
				{
					return;
				}

				TSharedPtr<FPCGDataUploadAdaptor> This = ThisWeak.Pin();
				if (!This)
				{
					return;
				}

				PCGDataCollectionPackingHelpers::PackDataCollection(Context->DataBinding->GetInputDataCollection(), This->TargetDataCollectionDesc, This->InputPinLabel, Context->DataBinding, This->PackedDataCollection);
			});

			if (!ensure(PackDataCollectionTask.IsValid()))
			{
				return true;
			}

#if WITH_EDITOR
			PCGComputeHelpers::NotifyCPUToGPUUpload(Binding, InProducerKernel, InProducerSettings);
#endif
		}

		return PackDataCollectionTask.IsCompleted();
	}

	return true;
}

FRDGBufferSRVRef FPCGDataUploadAdaptor::GetAttributeRemapBufferSRV(FRDGBuilder& InGraphBuilder, int32& OutFirstRemappedAttributeId)
{
	// Should only be called if UsesAttributeIdRemap() returns true and ExternalBufferForReuse is set.
	check(ExternalBufferForReuse);

	// Mapping created for all non-system attributes (system attributes always have fixed IDs).
	OutFirstRemappedAttributeId = PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS;

	FRDGBufferRef Buffer = nullptr;

	if (!GraphToSourceBufferAttributeIndex.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(GraphToSourceBufferAttributeIndex.GetTypeSize(), GraphToSourceBufferAttributeIndex.Num());
		Buffer = InGraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGDataUploadAdaptor_GraphToBufferAttributeId"));

		InGraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(GraphToSourceBufferAttributeIndex));
	}
	else
	{
		Buffer = GSystemTextures.GetDefaultStructuredBuffer(InGraphBuilder, sizeof(uint32));
	}

	return InGraphBuilder.CreateSRV(Buffer);
}

FRDGBufferSRVRef FPCGDataUploadAdaptor::GetBufferToGraphStringKeySRV(FRDGBuilder& InGraphBuilder, int32& OutNumRemappedStringKeys)
{
	// Should only be called if UsesStringKeyRemap() returns true and ExternalBufferForReuse is set.
	check(ExternalBufferForReuse);

	OutNumRemappedStringKeys = BufferToGraphStringKey.Num();

	FRDGBufferRef Buffer = nullptr;

	if (!BufferToGraphStringKey.IsEmpty())
	{
		const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(BufferToGraphStringKey.GetTypeSize(), BufferToGraphStringKey.Num());
		Buffer = InGraphBuilder.CreateBuffer(BufferDesc, TEXT("PCGDataUploadAdaptor_BufferToGraphStringKey"));

		InGraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(BufferToGraphStringKey));
	}
	else
	{
		Buffer = GSystemTextures.GetDefaultStructuredBuffer(InGraphBuilder, sizeof(uint32));
	}

	return InGraphBuilder.CreateSRV(Buffer);
}

/** Create buffer with the element counts of each data. Stored in a buffer because we do not constrain the max data count. */
FRDGBufferSRVRef FPCGDataUploadAdaptor::GetDataElementCountsBufferSRV(FRDGBuilder& InGraphBuilder)
{
	TArray<uint32> DataElementCounts;
	DataElementCounts.Reserve(TargetDataCollectionDesc->GetDataDescriptions().Num());

	for (const FPCGDataDesc& DataDesc : TargetDataCollectionDesc->GetDataDescriptions())
	{
		ensure(DataDesc.GetElementDimension() == EPCGElementDimension::One);
		DataElementCounts.Add(DataDesc.GetElementCount().X);
	}

	if (DataElementCounts.IsEmpty())
	{
		const FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), DataElementCounts.Num());
		FRDGBufferRef NullBuffer = InGraphBuilder.CreateBuffer(Desc, TEXT("EmptyDataElementCounts"));

		uint32 Zero = 0;
		InGraphBuilder.QueueBufferUpload(NullBuffer, &Zero, sizeof(Zero));

		return InGraphBuilder.CreateSRV(NullBuffer);
	}
	else
	{
		const FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), DataElementCounts.Num());

		FRDGBufferRef Buffer = InGraphBuilder.CreateBuffer(Desc, TEXT("PCGDataCollectionUpload_DataElementCounts"));

		InGraphBuilder.QueueBufferUpload(Buffer, MakeArrayView(DataElementCounts));

		return InGraphBuilder.CreateSRV(Buffer);
	}
}

/** Gets the buffer that can then be used to read the data in kernels. */
FRDGBufferRef FPCGDataUploadAdaptor::GetBuffer_RenderThread(FRDGBuilder& InGraphBuilder, EPCGExportMode InExportMode)
{
	RDG_EVENT_SCOPE(InGraphBuilder, "FPCGDataUploadAdaptor::GetBuffer_RenderThread");

	if (!ExternalBufferForReuse)
	{
		// Not reusing an existing buffer, create a new buffer and pack the input data collection.
		FRDGBufferDesc Desc = FRDGBufferDesc::CreateByteAddressDesc(sizeof(uint32) * PackedDataCollection.Num());
		if (InExportMode != EPCGExportMode::NoExport)
		{
			// We don't know for sure whether buffer will be read back or not, so need to flag the possibility if the buffer will be passed downstream.
			Desc.Usage |= BUF_SourceCopy;
		}

		FRDGBufferRef Buffer = InGraphBuilder.CreateBuffer(Desc, TEXT("PCGDataCollectionUpload"));

		InGraphBuilder.QueueBufferUpload(Buffer, PackedDataCollection.GetData(), PackedDataCollection.Num() * PackedDataCollection.GetTypeSize());

		return Buffer;
	}
	else
	{
		ensure(PackedDataCollection.IsEmpty());

		return InGraphBuilder.RegisterExternalBuffer(ExternalBufferForReuse);
	}
}
