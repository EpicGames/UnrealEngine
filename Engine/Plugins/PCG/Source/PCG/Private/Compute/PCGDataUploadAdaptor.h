// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGDataDescription.h"

#include "CoreMinimal.h"
#include "RendererInterface.h"

class FRDGBuilder;
class UPCGComputeKernel;
class UPCGDataBinding;
class UPCGSettings;
enum class EPCGExportMode : uint8;

/** A helper to ingest incoming data objects and pass the required data to the data interface. */
class FPCGDataUploadAdaptor : public TSharedFromThis<FPCGDataUploadAdaptor, ESPMode::ThreadSafe>
{
public:
	FPCGDataUploadAdaptor(UPCGDataBinding* InDataBinding, const TSharedPtr<const FPCGDataCollectionDesc> InTargetDataCollectionDesc, FName InInputPinLabel);

	/** Do any preparation work such as data readbacks. Returns true when preparation is complete. */
	bool PrepareData_GameThread(const UPCGComputeKernel* InProducerKernel, const UPCGSettings* InProducerSettings);

	FRDGBufferSRVRef GetAttributeRemapBufferSRV(FRDGBuilder& InGraphBuilder, int32& OutFirstRemappedAttributeId);

	FRDGBufferSRVRef GetBufferToGraphStringKeySRV(FRDGBuilder& InGraphBuilder, int32& OutNumRemappedStringKeys);

	/** Create buffer with the element counts of each data. Stored in a buffer because we do not constrain the max data count. */
	FRDGBufferSRVRef GetDataElementCountsBufferSRV(FRDGBuilder& InGraphBuilder);

	/** Gets the buffer that can then be used to read the data in kernels. */
	FRDGBufferRef GetBuffer_RenderThread(FRDGBuilder& InGraphBuilder, EPCGExportMode InExportMode);

	bool UsesAttributeIdRemap() const { return !!ExternalBufferForReuse; }

	bool UsesStringKeyRemap() const { return ExternalBufferMaxStringKeyValue != INDEX_NONE; }

	bool IsUploadingFromCPU() const { return !ExternalBufferForReuse; }

private:
	TRefCountPtr<FRDGPooledBuffer> ExternalBufferForReuse;

	int32 ExternalBufferMaxStringKeyValue = INDEX_NONE;

	TWeakObjectPtr<UPCGDataBinding> DataBinding;

	TSharedPtr<const FPCGDataCollectionDesc> TargetDataCollectionDesc;

	/** If data cannot be reused from GPU, will be packed into this buffer prior to upload. */
	TArray<uint32> PackedDataCollection;

	/** Map from (this) compute graph attribute ID to the attribute ID used in the upstream compute graph that produced this buffer. */
	TArray<int32> GraphToSourceBufferAttributeIndex;
	
	/** Map from string key values buffer from upstream compute graph to corresponding string key values in this graph. */
	TArray<int32> BufferToGraphStringKey;

	FName InputPinLabel;

	uint32 ExternalBufferSizeBytes = 0;

	UE::Tasks::TTask<void> PackDataCollectionTask;
};
