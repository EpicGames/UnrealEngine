// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGHLSLSyntaxTokenizer.h"

#if WITH_EDITOR

FPCGHLSLSyntaxTokenizer::FPCGHLSLSyntaxTokenizer(const FPCGSyntaxTokenizerParams& InParams)
	: FHlslSyntaxTokenizer()
{
	Keywords.Append(InParams.AdditionalKeywords);
}

#endif // WITH_EDITOR
