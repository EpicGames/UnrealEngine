// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGKernelOverridableParamTypeInfo.h"

FShaderValueTypeHandle FPCGKernelOverridableParamTypeInfo::GetShaderValueType(bool bForShaderParamStruct) const
{
	// Bool is illegal in shader parameter structs (triggers a fatal error in FShaderParametersMetadata).
	// Hand-written structs always use SHADER_PARAMETER(uint32, bMyBool). Remap Bool -> Uint to match.
	const EShaderFundamentalType Type = (bForShaderParamStruct && FundamentalType == EShaderFundamentalType::Bool) ? EShaderFundamentalType::Uint : FundamentalType;

	check(NumRows >= 1 && NumRows <= 4 && NumCols >= 1 && NumCols <= 4);

	if (NumCols > 1)
	{
		return FShaderValueType::Get(Type, NumRows, NumCols);
	}
	else if (NumRows > 1)
	{
		return FShaderValueType::Get(Type, NumRows);
	}
	else
	{
		return FShaderValueType::Get(Type);
	}
}

const FPCGKernelOverridableParamTypeInfo& FPCGKernelOverridableParamTypeInfo::Get(EPCGMetadataTypes InType)
{
#if WITH_EDITOR
	#define PCG_SHADER_PARAM_TYPE_INFO(HlslType, Accessor, FundametalType, Rows, Cols) { HlslType, Accessor, FundametalType, Rows, Cols }
#else
	#define PCG_SHADER_PARAM_TYPE_INFO(HlslType, Accessor, FundametalType, Rows, Cols) { FundametalType, Rows, Cols }
#endif

	static const FPCGKernelOverridableParamTypeInfo BoolInfo      = PCG_SHADER_PARAM_TYPE_INFO(TEXT("bool"),     TEXT("Bool"),      EShaderFundamentalType::Bool,  1, 1);
	static const FPCGKernelOverridableParamTypeInfo UintInfo      = PCG_SHADER_PARAM_TYPE_INFO(TEXT("uint"),     TEXT("Uint"),      EShaderFundamentalType::Uint,  1, 1);
	static const FPCGKernelOverridableParamTypeInfo IntInfo       = PCG_SHADER_PARAM_TYPE_INFO(TEXT("int"),      TEXT("Int"),       EShaderFundamentalType::Int,   1, 1);
	static const FPCGKernelOverridableParamTypeInfo FloatInfo     = PCG_SHADER_PARAM_TYPE_INFO(TEXT("float"),    TEXT("Float"),     EShaderFundamentalType::Float, 1, 1);
	static const FPCGKernelOverridableParamTypeInfo Float2Info    = PCG_SHADER_PARAM_TYPE_INFO(TEXT("float2"),   TEXT("Float2"),    EShaderFundamentalType::Float, 2, 1);
	static const FPCGKernelOverridableParamTypeInfo Float3Info    = PCG_SHADER_PARAM_TYPE_INFO(TEXT("float3"),   TEXT("Float3"),    EShaderFundamentalType::Float, 3, 1);
	static const FPCGKernelOverridableParamTypeInfo Float4Info    = PCG_SHADER_PARAM_TYPE_INFO(TEXT("float4"),   TEXT("Float4"),    EShaderFundamentalType::Float, 4, 1);
	static const FPCGKernelOverridableParamTypeInfo RotatorInfo   = PCG_SHADER_PARAM_TYPE_INFO(TEXT("float3"),   TEXT("Rotator"),   EShaderFundamentalType::Float, 3, 1);
	static const FPCGKernelOverridableParamTypeInfo QuatInfo      = PCG_SHADER_PARAM_TYPE_INFO(TEXT("float4"),   TEXT("Quat"),      EShaderFundamentalType::Float, 4, 1);
	static const FPCGKernelOverridableParamTypeInfo TransformInfo = PCG_SHADER_PARAM_TYPE_INFO(TEXT("float4x4"), TEXT("Transform"), EShaderFundamentalType::Float, 4, 4);

#undef PCG_SHADER_PARAM_TYPE_INFO

	switch (InType)
	{
	case EPCGMetadataTypes::Boolean:
		return BoolInfo;
	case EPCGMetadataTypes::Name:
	case EPCGMetadataTypes::Enum:
		return UintInfo;
	case EPCGMetadataTypes::Integer32:
	case EPCGMetadataTypes::Integer64: // Note: truncated to 32-bit in HLSL.
		return IntInfo;
	case EPCGMetadataTypes::Float:
	case EPCGMetadataTypes::Double: // Note: reduced to 32-bit float in HLSL.
		return FloatInfo;
	case EPCGMetadataTypes::Vector2:
		return Float2Info;
	case EPCGMetadataTypes::Vector:
		return Float3Info;
	case EPCGMetadataTypes::Vector4:
		return Float4Info;
	case EPCGMetadataTypes::Rotator:
		return RotatorInfo;
	case EPCGMetadataTypes::Quaternion:
		return QuatInfo;
	case EPCGMetadataTypes::Transform:
		return TransformInfo;
	default:
		ensureMsgf(false, TEXT("Unsupported EPCGMetadataTypes '%s' in GPU override type mapping."), *StaticEnum<EPCGMetadataTypes>()->GetNameStringByValue(static_cast<int64>(InType)));
		return UintInfo;
	}
}
