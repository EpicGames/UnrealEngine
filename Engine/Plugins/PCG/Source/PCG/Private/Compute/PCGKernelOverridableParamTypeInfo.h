// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataCommon.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"

/** Maps EPCGMetadataTypes to shader type info for GPU override codegen and runtime metadata building. */
struct FPCGKernelOverridableParamTypeInfo
{
#if WITH_EDITOR
	/** HLSL type name (e.g. "uint", "float3", "float4x4"). Editor-only, used for HLSL source generation. */
	const TCHAR* HLSLType = nullptr;

	/** Suffix for the GetFirstAttributeAs*() HLSL accessor (e.g. "Uint", "Float3", "Transform"). Editor-only, used for HLSL source generation. */
	const TCHAR* AccessorSuffix = nullptr;
#endif

	/** Shader fundamental type (Float, Uint, Int, etc.). */
	EShaderFundamentalType FundamentalType = EShaderFundamentalType::None;

	/** Number of rows. */
	int32 NumRows = 0;

	/** Number of columns. */
	int32 NumCols = 0;

	/** Returns the appropriate FShaderValueType. If bForShaderParamStruct is true, remaps Bool to Uint (SHADER_PARAMETER(bool) is illegal in shader parameter structs). */
	FShaderValueTypeHandle GetShaderValueType(bool bForShaderParamStruct = false) const;

	static const FPCGKernelOverridableParamTypeInfo& Get(EPCGMetadataTypes InType);
};
