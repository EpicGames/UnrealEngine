// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGKernelParamsCache.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGSettings.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#define LOCTEXT_NAMESPACE "PCGKernelParamsCache"

namespace PCGKernelParamsHelpers
{
	void OverrideKernelParamValue(FPCGContext* InContext, const UPCGSettings* InSettings, FInstancedPropertyBag& InOutKernelParams, const UStruct* KernelParamsStructType, FName InParamName, const FProperty* OverrideProperty, void* OverrideValuePtr, const UPCGData* InData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OverrideKernelParamValue);
		check(KernelParamsStructType);
		check(OverrideProperty);
		check(OverrideValuePtr);

		const UPCGParamData* OverrideData = Cast<UPCGParamData>(InData);

		if (!OverrideData)
		{
			PCG_KERNEL_VALIDATION_ERR(InContext, InSettings, FText::Format(LOCTEXT("NoOverrideData", "Override pin '{0}' has no data, we can't override it."), FText::FromName(InParamName)));
			return;
		}

		const UPCGMetadata* Metadata = InData ? InData->ConstMetadata() : nullptr;
		const FPCGMetadataAttributeBase* AttributeBase = Metadata ? Metadata->GetConstAttribute(Metadata->GetLatestAttributeNameOrNone()) : nullptr;

		if (!AttributeBase)
		{
			PCG_KERNEL_VALIDATION_ERR(InContext, InSettings, FText::Format(LOCTEXT("NoOverrideAttribute", "Data on override pin '{0}' has no attributes, we can't override it."), FText::FromName(InParamName)));
			return;
		}

		auto LogUnsupportedAttributeTypeError = [InContext, InSettings, InParamName, OverrideProperty, AttributeBase]()
		{
			PCG_KERNEL_VALIDATION_ERR(
				InContext,
				InSettings,
				FText::Format(LOCTEXT("UnsupportedAttributeType", "Tried to override pin '{0}', but attribute of type '{1}' does not support conversion to expected type '{2}'."),
					FText::FromName(InParamName),
					StaticEnum<EPCGMetadataTypes>()->GetDisplayNameTextByValue((int64)AttributeBase->GetTypeId()),
					FText::FromString(OverrideProperty->GetClass()->GetName())));
		};

		if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(OverrideProperty))
		{
			if (NumericProperty->IsFloatingPoint())
			{
				double Value = 0.0f;

				switch (AttributeBase->GetTypeId())
				{
				case PCG::Private::MetadataTypes<float>::Id:
					Value = static_cast<const FPCGMetadataAttribute<float>*>(AttributeBase)->GetValueFromItemKey(PCGFirstEntryKey);
					break;
				case PCG::Private::MetadataTypes<double>::Id:
					Value = static_cast<const FPCGMetadataAttribute<double>*>(AttributeBase)->GetValueFromItemKey(PCGFirstEntryKey);
					break;
				default:
					LogUnsupportedAttributeTypeError();
					return;
				}

				NumericProperty->SetFloatingPointPropertyValue(OverrideValuePtr, Value);
			}
			else
			{
				int64 Value = 0;

				switch (AttributeBase->GetTypeId())
				{
				case PCG::Private::MetadataTypes<int32>::Id:
					Value = static_cast<const FPCGMetadataAttribute<int32>*>(AttributeBase)->GetValueFromItemKey(PCGFirstEntryKey);
					break;
				case PCG::Private::MetadataTypes<int64>::Id:
					Value = static_cast<const FPCGMetadataAttribute<int64>*>(AttributeBase)->GetValueFromItemKey(PCGFirstEntryKey);
					break;
				default:
					LogUnsupportedAttributeTypeError();
					return;
				}

				NumericProperty->SetIntPropertyValue(OverrideValuePtr, Value);
			}
		}
		else if (const FEnumProperty* EnumPropety = CastField<FEnumProperty>(OverrideProperty))
		{
			const FNumericProperty* UnderlyingNumericProperty = EnumPropety->GetUnderlyingProperty();

			if (!ensure(UnderlyingNumericProperty))
			{
				PCG_KERNEL_VALIDATION_ERR(
					InContext,
					InSettings,
					FText::Format(LOCTEXT("MissingUnderlyingNumericProperty", "Tried to override pin '{0}', but enum property '{1}' does not have a valid underlying numeric property."),
						FText::FromName(InParamName),
						FText::FromString(OverrideProperty->GetClass()->GetName())));
				return;
			}

			int64 Value = 0;

			switch (AttributeBase->GetTypeId())
			{
			case PCG::Private::MetadataTypes<int32>::Id:
				Value = static_cast<const FPCGMetadataAttribute<int32>*>(AttributeBase)->GetValueFromItemKey(PCGFirstEntryKey);
				break;
			case PCG::Private::MetadataTypes<int64>::Id:
				Value = static_cast<const FPCGMetadataAttribute<int64>*>(AttributeBase)->GetValueFromItemKey(PCGFirstEntryKey);
				break;
			default:
				LogUnsupportedAttributeTypeError();
				return;
			}

			UnderlyingNumericProperty->SetIntPropertyValue(OverrideValuePtr, Value);
		}
		else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(OverrideProperty))
		{
			bool Value = false;

			switch (AttributeBase->GetTypeId())
			{
			case PCG::Private::MetadataTypes<int32>::Id:
				Value = static_cast<const FPCGMetadataAttribute<int32>*>(AttributeBase)->GetValueFromItemKey(PCGFirstEntryKey) != 0;
				break;
			case PCG::Private::MetadataTypes<int64>::Id:
				Value = static_cast<const FPCGMetadataAttribute<int64>*>(AttributeBase)->GetValueFromItemKey(PCGFirstEntryKey) != 0;
				break;
			case PCG::Private::MetadataTypes<bool>::Id:
				Value = static_cast<const FPCGMetadataAttribute<bool>*>(AttributeBase)->GetValueFromItemKey(PCGFirstEntryKey);
				break;
			default:
				LogUnsupportedAttributeTypeError();
				return;
			}

			BoolProperty->SetPropertyValue(OverrideValuePtr, Value);
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(OverrideProperty))
		{
			const UScriptStruct* Struct = StructProperty->Struct;

			if (!ensure(Struct))
			{
				return;
			}

			bool bHandled = false;

#define PCG_OVERRIDE_STRUCT_TYPE(Type) \
			if (Struct == TBaseStructure<Type>::Get()) \
			{ \
				if (AttributeBase->GetTypeId() == PCG::Private::MetadataTypes<Type>::Id) \
				{ \
					*static_cast<Type*>(OverrideValuePtr) = static_cast<const FPCGMetadataAttribute<Type>*>(AttributeBase)->GetValueFromItemKey(PCGFirstEntryKey); \
					bHandled = true; \
				} \
				else \
				{ \
					LogUnsupportedAttributeTypeError(); \
					return; \
				} \
			}

			PCG_OVERRIDE_STRUCT_TYPE(FVector2D)
			else PCG_OVERRIDE_STRUCT_TYPE(FVector)
			else PCG_OVERRIDE_STRUCT_TYPE(FVector4)
			else PCG_OVERRIDE_STRUCT_TYPE(FQuat)
			else PCG_OVERRIDE_STRUCT_TYPE(FRotator)
			else PCG_OVERRIDE_STRUCT_TYPE(FTransform)

#undef PCG_OVERRIDE_STRUCT_TYPE

			if (!bHandled)
			{
				PCG_KERNEL_VALIDATION_ERR(
					InContext,
					InSettings,
					FText::Format(LOCTEXT("UnsupportedStructType", "Tried to override pin '{0}', but struct type '{1}' is not supported."),
						FText::FromName(InParamName),
						FText::FromString(Struct->GetName())));
			}
		}
		else
		{
			PCG_KERNEL_VALIDATION_ERR(
				InContext,
				InSettings,
				FText::Format(LOCTEXT("UnsupportedPropertyType", "Tried to override pin '{0}', but property of type '{1}' is not supported."),
					FText::FromName(InParamName),
					FText::FromString(OverrideProperty->GetClass()->GetName())));
		}
	}
}

bool FPCGKernelParams::GetValueBool(FName InParamName) const
{
	const FBoolProperty* Property = nullptr;
	const void* ValuePtr = nullptr;

	if (const FPropertyAndAddress* PropertyAndAddress = OverriddenProperties.Find(InParamName))
	{
		Property = CastField<FBoolProperty>(PropertyAndAddress->Key);
		ValuePtr = PropertyAndAddress->Value;
	}
	else if (const UPCGSettings* Settings = WeakSettings.Get())
	{
		Property = CastField<FBoolProperty>(Settings->GetClass()->FindPropertyByName(InParamName));
		ValuePtr = Property ? Property->ContainerPtrToValuePtr<void>(Settings) : nullptr;
	}

	return ensure(Property && ValuePtr) ? Property->GetPropertyValue(ValuePtr) : false;
}

int FPCGKernelParams::GetValueInt(FName InParamName) const
{
	// Use FNumericProperty so int8/16/32/64 (and unsigned variants) all work.
	const FNumericProperty* Property = nullptr;
	const void* ValuePtr = nullptr;

	if (const FPropertyAndAddress* PropertyAndAddress = OverriddenProperties.Find(InParamName))
	{
		Property = CastField<FNumericProperty>(PropertyAndAddress->Key);
		ValuePtr = PropertyAndAddress->Value;
	}
	else if (const UPCGSettings* Settings = WeakSettings.Get())
	{
		Property = CastField<FNumericProperty>(Settings->GetClass()->FindPropertyByName(InParamName));
		ValuePtr = Property ? Property->ContainerPtrToValuePtr<void>(Settings) : nullptr;
	}

	return ensure(Property && ValuePtr && Property->IsInteger()) ? static_cast<int>(Property->GetSignedIntPropertyValue(ValuePtr)) : 0;
}

FName FPCGKernelParams::GetValueName(FName InParamName) const
{
	const FNameProperty* Property = nullptr;
	const void* ValuePtr = nullptr;

	if (const FPropertyAndAddress* PropertyAndAddress = OverriddenProperties.Find(InParamName))
	{
		Property = CastField<FNameProperty>(PropertyAndAddress->Key);
		ValuePtr = PropertyAndAddress->Value;
	}
	else if (const UPCGSettings* Settings = WeakSettings.Get())
	{
		Property = CastField<FNameProperty>(Settings->GetClass()->FindPropertyByName(InParamName));
		ValuePtr = Property ? Property->ContainerPtrToValuePtr<void>(Settings) : nullptr;
	}

	return ensure(Property && ValuePtr) ? Property->GetPropertyValue(ValuePtr) : NAME_None;
}

float FPCGKernelParams::GetValueFloat(FName InParamName) const
{
	return static_cast<float>(GetValueDouble(InParamName));
}

double FPCGKernelParams::GetValueDouble(FName InParamName) const
{
	// Use FNumericProperty so both FFloatProperty and FDoubleProperty work.
	const FNumericProperty* Property = nullptr;
	const void* ValuePtr = nullptr;

	if (const FPropertyAndAddress* PropertyAndAddress = OverriddenProperties.Find(InParamName))
	{
		Property = CastField<FNumericProperty>(PropertyAndAddress->Key);
		ValuePtr = PropertyAndAddress->Value;
	}
	else if (const UPCGSettings* Settings = WeakSettings.Get())
	{
		Property = CastField<FNumericProperty>(Settings->GetClass()->FindPropertyByName(InParamName));
		ValuePtr = Property ? Property->ContainerPtrToValuePtr<void>(Settings) : nullptr;
	}

	return ensure(Property && ValuePtr && Property->IsFloatingPoint()) ? Property->GetFloatingPointPropertyValue(ValuePtr) : 0.0;
}

FVector2D FPCGKernelParams::GetValueVector2D(FName InParamName) const
{
	return GetStructValue<FVector2D>(InParamName);
}

FVector FPCGKernelParams::GetValueVector(FName InParamName) const
{
	return GetStructValue<FVector>(InParamName);
}

FVector4 FPCGKernelParams::GetValueVector4(FName InParamName) const
{
	return GetStructValue<FVector4>(InParamName);
}

FQuat FPCGKernelParams::GetValueQuat(FName InParamName) const
{
	return GetStructValue<FQuat>(InParamName);
}

FRotator FPCGKernelParams::GetValueRotator(FName InParamName) const
{
	return GetStructValue<FRotator>(InParamName);
}

FTransform FPCGKernelParams::GetValueTransform(FName InParamName) const
{
	return GetStructValue<FTransform>(InParamName);
}

uint32 FPCGKernelParams::GetValueEnumAsUint(FName InParamName) const
{
	const FEnumProperty* Property = nullptr;
	const void* ValuePtr = nullptr;

	if (const FPropertyAndAddress* PropertyAndAddress = OverriddenProperties.Find(InParamName))
	{
		Property = CastField<FEnumProperty>(PropertyAndAddress->Key);
		ValuePtr = PropertyAndAddress->Value;
	}
	else if (const UPCGSettings* Settings = WeakSettings.Get())
	{
		Property = CastField<FEnumProperty>(Settings->GetClass()->FindPropertyByName(InParamName));
		ValuePtr = Property ? Property->ContainerPtrToValuePtr<void>(Settings) : nullptr;
	}

	const FNumericProperty* NumericProperty = Property ? Property->GetUnderlyingProperty() : nullptr;
	return static_cast<uint32>(ensure(NumericProperty && ValuePtr) ? NumericProperty->GetUnsignedIntPropertyValue(ValuePtr) : 0);
}

bool FPCGKernelParamsCache::Initialize(FPCGContext* InContext, UPCGDataBinding* InDataBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGKernelParamsCache::Initialize);

	if (!ensure(InDataBinding))
	{
		return true;
	}

	CacheOverridableParamDataIndices(InContext, InDataBinding);
	
	if (!ReadbackOverridableParamData(InDataBinding))
	{
		return false;
	}

	CreateAndCacheKernelParams(InContext, InDataBinding);

	return true;
}

const FPCGKernelParams* FPCGKernelParamsCache::GetCachedKernelParams(int32 InKernelIndex) const
{
	const FPCGKernelParams* FoundKernelParams = KernelParams.Find(InKernelIndex);
	return FoundKernelParams ? FoundKernelParams : nullptr;
}

void FPCGKernelParamsCache::Reset()
{
	KernelParams.Empty();
	OverridableParamDataIndices.Empty();
}

void FPCGKernelParamsCache::CacheOverridableParamDataIndices(FPCGContext* InContext, const UPCGDataBinding* InDataBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGKernelParamsCache::CacheOverridableParamDataIndices);
	check(InDataBinding);

	if (OverridableParamDataIndices.IsEmpty())
	{
		const UPCGComputeGraph* ComputeGraph = InDataBinding->GetComputeGraph();

		if (!ensure(ComputeGraph))
		{
			return;
		}

		for (int32 KernelIndex = 0; KernelIndex < ComputeGraph->GetNumKernels(); ++KernelIndex)
		{
			const UPCGComputeKernel* Kernel = Cast<UPCGComputeKernel>(ComputeGraph->GetKernel(KernelIndex));

			if (ensure(Kernel))
			{
				for (const FPCGKernelOverridableParam& OverridableParam : Kernel->GetCachedOverridableParams())
				{
					if (OverridableParam.bIsPropertyOverriddenByPin && OverridableParam.bRequiresGPUReadback)
					{
						const int32 OverridableParamDataIndex = InDataBinding->GetFirstInputDataIndex(Kernel, OverridableParam.Label);

						if (OverridableParamDataIndex != INDEX_NONE)
						{
							TArray<TPair<FPCGKernelOverridableParam, int32>>& OverridableParamDataIndicesArray = OverridableParamDataIndices.FindOrAdd(KernelIndex);
							OverridableParamDataIndicesArray.Emplace(OverridableParam, OverridableParamDataIndex);
						}
						else
						{
							PCG_KERNEL_VALIDATION_WARN(InContext, Kernel->GetSettings(), FText::Format(LOCTEXT("NoOverrideDataFound", "No data found on connected override pin '{0}'."), FText::FromName(OverridableParam.Label)));
						}
					}
				}
			}
		}
	}
}

bool FPCGKernelParamsCache::ReadbackOverridableParamData(UPCGDataBinding* InDataBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGKernelParamsCache::ReadbackOverridableParamData);
	check(InDataBinding);

	// Readback any overridable params that require readback.
	bool bAllReadBack = true;

	for (const TPair<int32, TArray<TPair<FPCGKernelOverridableParam, int32>>>& OverridableParamDataIndicesArray : OverridableParamDataIndices)
	{
		for (const TPair<FPCGKernelOverridableParam, int32>& OverridableParam : OverridableParamDataIndicesArray.Value)
		{
			// Readback data - poll until readback complete.
			// @todo_pcg: There could be a hot path for reading back overridable param values, since they are a single-attribute, single-entry attribute set of a known type.
			if (!InDataBinding->ReadbackInputDataToCPU(OverridableParam.Value))
			{
				bAllReadBack = false;
			}
		}

		if (!bAllReadBack)
		{
			return false;
		}
	}

	return true;
}

void FPCGKernelParamsCache::CreateAndCacheKernelParams(FPCGContext* InContext, const UPCGDataBinding* InDataBinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGKernelParamsCache::CreateAndCacheKernelParams);
	check(InDataBinding);

	const UPCGComputeGraph* ComputeGraph = InDataBinding->GetComputeGraph();

	if (!ensure(ComputeGraph))
	{
		return;
	}

	for (int32 KernelIndex = 0; KernelIndex < ComputeGraph->GetNumKernels(); ++KernelIndex)
	{
		const UPCGComputeKernel* Kernel = Cast<UPCGComputeKernel>(ComputeGraph->GetKernel(KernelIndex));
		const UPCGSettings* Settings = ensure(Kernel) ? Kernel->GetSettings() : nullptr;

		if (!Settings)
		{
			continue;
		}

		FPCGKernelParams NewKernelParams;
		NewKernelParams.WeakSettings = Settings;

		// Apply any overrides that exist.
		if (const TArray<TPair<FPCGKernelOverridableParam, int32>>* OverridableParams = OverridableParamDataIndices.Find(KernelIndex))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(OverrideKernelParamValues);

			NewKernelParams.OverriddenValues = Settings->CreateEmptyPropertyBagInstance();

			const UStruct* KernelParamsStructType = NewKernelParams.OverriddenValues.GetPropertyBagStruct();

			if (!ensure(KernelParamsStructType))
			{
				continue;
			}

			for (const TPair<FPCGKernelOverridableParam, int32>& OverridableParamAndDataIndex : *OverridableParams)
			{
				const FPCGKernelOverridableParam& OverridableParam = OverridableParamAndDataIndex.Key;
				const UPCGData* Data = InDataBinding->GetInputDataCollection().TaggedData[OverridableParamAndDataIndex.Value].Data;
				const FProperty* OverrideProperty = KernelParamsStructType ? KernelParamsStructType->FindPropertyByName(OverridableParam.Label) : nullptr;

				if (!OverrideProperty)
				{
					PCG_KERNEL_VALIDATION_ERR(InContext, Settings, FText::Format(LOCTEXT("OverridePropertyNotFound", "Overridden property '{0}' was not found in container '{1}'."), FText::FromName(OverridableParam.Label), FText::FromString(KernelParamsStructType->GetName())));
					continue;
				}

				void* OverrideValuePtr = OverrideProperty->ContainerPtrToValuePtr<void>(NewKernelParams.OverriddenValues.GetMutableValue().GetMemory());

				if (!ensure(OverrideValuePtr))
				{
					continue;
				}

				PCGKernelParamsHelpers::OverrideKernelParamValue(InContext, Settings, NewKernelParams.OverriddenValues, KernelParamsStructType, OverridableParam.Label, OverrideProperty, OverrideValuePtr, Data);
				NewKernelParams.OverriddenProperties.FindOrAdd(OverridableParam.Label) = { OverrideProperty, OverrideValuePtr };
			}
		}

		KernelParams.FindOrAdd(KernelIndex) = MoveTemp(NewKernelParams);
	}
}

#undef LOCTEXT_NAMESPACE
