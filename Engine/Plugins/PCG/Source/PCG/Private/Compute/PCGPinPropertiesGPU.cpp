// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PCGPinPropertiesGPU.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPinPropertiesGPU)

uint32 FPCGPinPropertiesGPU::GetElementCountMultiplier() const
{
	if (PropertiesGPU.ElementCountMode != EPCGElementCountMode::Fixed)
	{
		return static_cast<uint32>(FMath::Max(PropertiesGPU.ElementCountMultiplier, 1));
	}
	else
	{
		return 1u;
	}
}

#if WITH_EDITOR
bool FPCGPinPropertiesGPU::CanEditChange(const FEditPropertyChain& PropertyChain) const
{
	if (FProperty* Property = PropertyChain.GetActiveNode()->GetValue())
	{
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, bAllowMultipleData))
		{
			return bAllowEditMultipleData;
		}
	}

	return true;
}
#endif // WITH_EDITOR
