// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/Packing/PCGDataCollectionPacking.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGDataDescription.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGAsync.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Async/ParallelFor.h"

namespace PCGDataCollectionPackingHelpers
{
	uint32 GetElementDataStartAddressUints(const uint32* InPackedDataCollection, uint32 InDataIndex, uint32 InAttributeId)
	{
		uint32 ReadAddressBytes = PCGDataCollectionPackingConstants::DATA_COLLECTION_HEADER_SIZE_BYTES + InDataIndex * PCGDataCollectionPackingConstants::DATA_HEADER_SIZE_BYTES;
		ReadAddressBytes += /*TypeId*/4 + /*Attribute Count*/4 + /*Element Count*/4;
	
		ReadAddressBytes += InAttributeId * PCGDataCollectionPackingConstants::ATTRIBUTE_HEADER_SIZE_BYTES;
		ReadAddressBytes += /*PackedIdAndStride*/4;
	
		return InPackedDataCollection[ReadAddressBytes >> 2] >> 2;
	}

	bool PackAttributeHelper(const FPCGMetadataAttributeBase* InAttributeBase, const FPCGKernelAttributeDesc& InAttributeDesc, PCGMetadataEntryKey InEntryKey, const TArray<FString>& InStringTable, TArray<uint32>& OutPackedDataCollection, uint32& InOutAddressUints)
	{
		check(InAttributeBase);

		const PCGMetadataValueKey ValueKey = InAttributeBase->GetValueKey(InEntryKey);
		const int16 TypeId = InAttributeBase->GetTypeId();
		const int StrideBytes = PCGDataDescriptionHelpers::GetAttributeTypeStrideBytes(InAttributeDesc.GetAttributeKey().GetType());

		switch (TypeId)
		{
		case PCG::Private::MetadataTypes<bool>::Id:
		{
			const FPCGMetadataAttribute<bool>* Attribute = static_cast<const FPCGMetadataAttribute<bool>*>(InAttributeBase);
			const bool Value = Attribute->GetValue(ValueKey);
			check(StrideBytes == 4);
			OutPackedDataCollection[InOutAddressUints++] = Value;
			break;
		}
		case PCG::Private::MetadataTypes<float>::Id:
		{
			const FPCGMetadataAttribute<float>* Attribute = static_cast<const FPCGMetadataAttribute<float>*>(InAttributeBase);
			const float Value = Attribute->GetValue(ValueKey);
			check(StrideBytes == 4);
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(Value);
			break;
		}
		case PCG::Private::MetadataTypes<double>::Id:
		{
			const FPCGMetadataAttribute<double>* Attribute = static_cast<const FPCGMetadataAttribute<double>*>(InAttributeBase);
			const double Value = Attribute->GetValue(ValueKey);
			check(StrideBytes == 4);
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value));
			break;
		}
		case PCG::Private::MetadataTypes<int32>::Id:
		{
			const FPCGMetadataAttribute<int32>* Attribute = static_cast<const FPCGMetadataAttribute<int32>*>(InAttributeBase);
			const int32 Value = Attribute->GetValue(ValueKey);
			check(StrideBytes == 4);
			OutPackedDataCollection[InOutAddressUints++] = Value;
			break;
		}
		case PCG::Private::MetadataTypes<int64>::Id:
		{
			const FPCGMetadataAttribute<int64>* Attribute = static_cast<const FPCGMetadataAttribute<int64>*>(InAttributeBase);
			const int64 Value = Attribute->GetValue(ValueKey);
			check(StrideBytes == 4);
			OutPackedDataCollection[InOutAddressUints++] = Value;
			break;
		}
		case PCG::Private::MetadataTypes<FVector2D>::Id:
		{
			const FPCGMetadataAttribute<FVector2D>* Attribute = static_cast<const FPCGMetadataAttribute<FVector2D>*>(InAttributeBase);
			const FVector2D Value = Attribute->GetValue(ValueKey);
			check(StrideBytes == 8);
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.X));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.Y));
			break;
		}
		case PCG::Private::MetadataTypes<FRotator>::Id:
		{
			const FPCGMetadataAttribute<FRotator>* Attribute = static_cast<const FPCGMetadataAttribute<FRotator>*>(InAttributeBase);
			const FRotator Value = Attribute->GetValue(ValueKey);
			check(StrideBytes == 12);
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.Pitch));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.Yaw));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.Roll));
			break;
		}
		case PCG::Private::MetadataTypes<FVector>::Id:
		{
			const FPCGMetadataAttribute<FVector>* Attribute = static_cast<const FPCGMetadataAttribute<FVector>*>(InAttributeBase);
			const FVector Value = Attribute->GetValue(ValueKey);
			check(StrideBytes == 12);
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.X));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.Y));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.Z));
			break;
		}
		case PCG::Private::MetadataTypes<FVector4>::Id:
		{
			const FPCGMetadataAttribute<FVector4>* Attribute = static_cast<const FPCGMetadataAttribute<FVector4>*>(InAttributeBase);
			const FVector4 Value = Attribute->GetValue(ValueKey);
			check(StrideBytes == 16);
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.X));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.Y));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.Z));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.W));
			break;
		}
		case PCG::Private::MetadataTypes<FQuat>::Id:
		{
			const FPCGMetadataAttribute<FQuat>* Attribute = static_cast<const FPCGMetadataAttribute<FQuat>*>(InAttributeBase);
			const FQuat Value = Attribute->GetValue(ValueKey);
			check(StrideBytes == 16);
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.X));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.Y));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.Z));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Value.W));
			break;
		}
		case PCG::Private::MetadataTypes<FTransform>::Id:
		{
			const FPCGMetadataAttribute<FTransform>* Attribute = static_cast<const FPCGMetadataAttribute<FTransform>*>(InAttributeBase);
			const FTransform Transform = Attribute->GetValue(ValueKey);

			const bool bIsRotationNormalized = Transform.IsRotationNormalized();
			if (!bIsRotationNormalized)
			{
				UE_LOGF(LogPCG, Error, "Tried to pack attribute '%ls' of type Transform for GPU data collection, but the transform's rotation is not normalized (%f, %f, %f, %f). Using identity instead.",
					*InAttributeBase->Name.ToString(), Transform.GetRotation().X, Transform.GetRotation().Y, Transform.GetRotation().Z, Transform.GetRotation().W);
			}

			// Note: ToMatrixWithScale() crashes if the transform is not normalized.
			const FMatrix Matrix = bIsRotationNormalized ? Transform.ToMatrixWithScale() : FMatrix::Identity;

			check(StrideBytes == 64);
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[0][0]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[1][0]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[2][0]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[3][0]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[0][1]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[1][1]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[2][1]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[3][1]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[0][2]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[1][2]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[2][2]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[3][2]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[0][3]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[1][3]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[2][3]));
			OutPackedDataCollection[InOutAddressUints++] = FMath::AsUInt(static_cast<float>(Matrix.M[3][3]));

			break;
		}
		case PCG::Private::MetadataTypes<FString>::Id:
		{
			// String stored as an integer for reading/writing in kernel, and accompanying string table in data description.
			const FPCGMetadataAttribute<FString>* Attribute = static_cast<const FPCGMetadataAttribute<FString>*>(InAttributeBase);
			const int32 Value = InStringTable.IndexOfByKey(Attribute->GetValue(ValueKey));
			check(StrideBytes == 4);
			OutPackedDataCollection[InOutAddressUints++] = Value;
			break;
		}
		case PCG::Private::MetadataTypes<FSoftObjectPath>::Id:
		{
			// SOP path string stored as an integer for reading/writing in kernel, and accompanying string table in data description.
			const FPCGMetadataAttribute<FSoftObjectPath>* Attribute = static_cast<const FPCGMetadataAttribute<FSoftObjectPath>*>(InAttributeBase);
			const int32 Value = InStringTable.IndexOfByKey(Attribute->GetValue(ValueKey).ToString());
			check(StrideBytes == 4);
			OutPackedDataCollection[InOutAddressUints++] = Value;
			break;
		}
		case PCG::Private::MetadataTypes<FSoftClassPath>::Id:
		{
			// SCP path string stored as an integer for reading/writing in kernel, and accompanying string table in data description.
			const FPCGMetadataAttribute<FSoftClassPath>* Attribute = static_cast<const FPCGMetadataAttribute<FSoftClassPath>*>(InAttributeBase);
			const int32 Value = InStringTable.IndexOfByKey(Attribute->GetValue(ValueKey).ToString());
			check(StrideBytes == 4);
			OutPackedDataCollection[InOutAddressUints++] = Value;
			break;
		}
		case PCG::Private::MetadataTypes<FName>::Id:
		{
			// FNames are currently stored in string table so use same logic as string.
			const FPCGMetadataAttribute<FName>* Attribute = static_cast<const FPCGMetadataAttribute<FName>*>(InAttributeBase);
			const int32 Value = InStringTable.IndexOfByKey(Attribute->GetValue(ValueKey).ToString());
			check(StrideBytes == 4);
			OutPackedDataCollection[InOutAddressUints++] = Value;
			break;
		}
		default:
			return false;
		}

		return true;
	}

	FPCGMetadataAttributeBase* CreateAttributeFromAttributeDesc(UPCGMetadata* Metadata, const FPCGKernelAttributeDesc& AttributeDesc)
	{
		check(Metadata);

		const FPCGKernelAttributeKey& AttributeKey = AttributeDesc.GetAttributeKey();
		FPCGMetadataDomain* MetadataDomain = Metadata->GetMetadataDomain(AttributeKey.GetIdentifier().MetadataDomain);

		if (!MetadataDomain)
		{
			return nullptr;
		}

		const FName AttributeName = AttributeKey.GetIdentifier().Name;
		
		switch (AttributeKey.GetType())
		{
		case EPCGKernelAttributeType::Bool:
		{
			return MetadataDomain->FindOrCreateAttribute<bool>(AttributeName);
		}
		case EPCGKernelAttributeType::Int:
		{
			return MetadataDomain->FindOrCreateAttribute<int>(AttributeName);
		}
		case EPCGKernelAttributeType::Float:
		{
			return MetadataDomain->FindOrCreateAttribute<float>(AttributeName);
		}
		case EPCGKernelAttributeType::Float2:
		{
			return MetadataDomain->FindOrCreateAttribute<FVector2D>(AttributeName);
		}
		case EPCGKernelAttributeType::Float3:
		{
			return MetadataDomain->FindOrCreateAttribute<FVector>(AttributeName);
		}
		case EPCGKernelAttributeType::Float4:
		{
			return MetadataDomain->FindOrCreateAttribute<FVector4>(AttributeName);
		}
		case EPCGKernelAttributeType::Rotator:
		{
			return MetadataDomain->FindOrCreateAttribute<FRotator>(AttributeName);
		}
		case EPCGKernelAttributeType::Quat:
		{
			return MetadataDomain->FindOrCreateAttribute<FQuat>(AttributeName);
		}
		case EPCGKernelAttributeType::Transform:
		{
			return MetadataDomain->FindOrCreateAttribute<FTransform>(AttributeName);
		}
		case EPCGKernelAttributeType::StringKey:
		{
			return MetadataDomain->FindOrCreateAttribute<FString>(AttributeName);
		}
		case EPCGKernelAttributeType::Name:
		{
			return MetadataDomain->FindOrCreateAttribute<FName>(AttributeName);
		}
		default:
			return nullptr;
		}
	}

	bool UnpackAttributeHelper(FPCGContext* InContext, const void* InPackedData, const FPCGKernelAttributeDesc& InAttributeDesc, const TArray<FString>& InStringTable, uint32 InAddressUints, uint32 InNumElements, UPCGData* OutData)
	{
		check(InPackedData);
		check(OutData);

		const float* DataAsFloat = static_cast<const float*>(InPackedData);
		const int32* DataAsInt = static_cast<const int32*>(InPackedData);

		FPCGAttributePropertyOutputSelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyOutputSelector>(InAttributeDesc.GetAttributeKey().GetIdentifier().Name);
		OutData->SetDomainFromDomainID(InAttributeDesc.GetAttributeKey().GetIdentifier().MetadataDomain, Selector);

		switch (InAttributeDesc.GetAttributeKey().GetType())
		{
		case EPCGKernelAttributeType::Bool:
		{
			TArray<bool> Values;
			Values.SetNumUninitialized(InNumElements);

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex;
				Values[ElementIndex] = static_cast<bool>(DataAsFloat[PackedElementIndex]);
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<bool>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		case EPCGKernelAttributeType::Int:
		{
			TArray<int> Values;
			Values.SetNumUninitialized(InNumElements);

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex;
				Values[ElementIndex] = DataAsInt[PackedElementIndex];
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<int>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		case EPCGKernelAttributeType::Float:
		{
			TArray<float> Values;
			Values.SetNumUninitialized(InNumElements);

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex;
				Values[ElementIndex] = DataAsFloat[PackedElementIndex];
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<float>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		case EPCGKernelAttributeType::Float2:
		{
			TArray<FVector2D> Values;
			Values.SetNumUninitialized(InNumElements);

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex * 2;
				Values[ElementIndex].X = DataAsFloat[PackedElementIndex + 0];
				Values[ElementIndex].Y = DataAsFloat[PackedElementIndex + 1];
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<FVector2D>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		case EPCGKernelAttributeType::Float3:
		{
			TArray<FVector> Values;
			Values.SetNumUninitialized(InNumElements);

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex * 3;
				Values[ElementIndex].X = DataAsFloat[PackedElementIndex + 0];
				Values[ElementIndex].Y = DataAsFloat[PackedElementIndex + 1];
				Values[ElementIndex].Z = DataAsFloat[PackedElementIndex + 2];
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<FVector>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		case EPCGKernelAttributeType::Float4:
		{
			TArray<FVector4> Values;
			Values.SetNumUninitialized(InNumElements);

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex * 4;
				Values[ElementIndex].X = DataAsFloat[PackedElementIndex + 0];
				Values[ElementIndex].Y = DataAsFloat[PackedElementIndex + 1];
				Values[ElementIndex].Z = DataAsFloat[PackedElementIndex + 2];
				Values[ElementIndex].W = DataAsFloat[PackedElementIndex + 3];
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<FVector4>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		case EPCGKernelAttributeType::Rotator:
		{
			TArray<FRotator> Values;
			Values.SetNumUninitialized(InNumElements);

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex * 3;
				Values[ElementIndex].Pitch = DataAsFloat[PackedElementIndex + 0];
				Values[ElementIndex].Yaw = DataAsFloat[PackedElementIndex + 1];
				Values[ElementIndex].Roll = DataAsFloat[PackedElementIndex + 2];
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<FRotator>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		case EPCGKernelAttributeType::Quat:
		{
			TArray<FQuat> Values;
			Values.SetNumUninitialized(InNumElements);

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex * 4;
				Values[ElementIndex].X = DataAsFloat[PackedElementIndex + 0];
				Values[ElementIndex].Y = DataAsFloat[PackedElementIndex + 1];
				Values[ElementIndex].Z = DataAsFloat[PackedElementIndex + 2];
				Values[ElementIndex].W = DataAsFloat[PackedElementIndex + 3];
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<FQuat>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		case EPCGKernelAttributeType::Transform:
		{
			TArray<FTransform> Values;
			Values.SetNumUninitialized(InNumElements);

			FMatrix Matrix;

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex * 16;

				Matrix.M[0][0] = DataAsFloat[PackedElementIndex + 0];
				Matrix.M[1][0] = DataAsFloat[PackedElementIndex + 1];
				Matrix.M[2][0] = DataAsFloat[PackedElementIndex + 2];
				Matrix.M[3][0] = DataAsFloat[PackedElementIndex + 3];
				Matrix.M[0][1] = DataAsFloat[PackedElementIndex + 4];
				Matrix.M[1][1] = DataAsFloat[PackedElementIndex + 5];
				Matrix.M[2][1] = DataAsFloat[PackedElementIndex + 6];
				Matrix.M[3][1] = DataAsFloat[PackedElementIndex + 7];
				Matrix.M[0][2] = DataAsFloat[PackedElementIndex + 8];
				Matrix.M[1][2] = DataAsFloat[PackedElementIndex + 9];
				Matrix.M[2][2] = DataAsFloat[PackedElementIndex + 10];
				Matrix.M[3][2] = DataAsFloat[PackedElementIndex + 11];
				Matrix.M[0][3] = DataAsFloat[PackedElementIndex + 12];
				Matrix.M[1][3] = DataAsFloat[PackedElementIndex + 13];
				Matrix.M[2][3] = DataAsFloat[PackedElementIndex + 14];
				Matrix.M[3][3] = DataAsFloat[PackedElementIndex + 15];

				new (&Values[ElementIndex]) FTransform(Matrix);
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<FTransform>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		case EPCGKernelAttributeType::StringKey:
		{
			check(!InStringTable.IsEmpty());

			TArray<FString> Values;
			Values.Reserve(InNumElements);

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex;
				const int32 StringKey = InStringTable.IsValidIndex(DataAsInt[PackedElementIndex]) ? DataAsInt[PackedElementIndex] : 0;
				Values.Add(InStringTable[StringKey]);
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<FString>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		case EPCGKernelAttributeType::Name:
		{
			check(!InStringTable.IsEmpty());

			TArray<FName> Values;
			Values.SetNumUninitialized(InNumElements);

			for (uint32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
			{
				const uint32 PackedElementIndex = InAddressUints + ElementIndex;

				// FNames currently stored in string table.
				const int32 StringKey = InStringTable.IsValidIndex(DataAsInt[PackedElementIndex]) ? DataAsInt[PackedElementIndex] : 0;
				Values[ElementIndex] = *InStringTable[StringKey];
			}

			ensure(PCGAttributeAccessorHelpers::WriteAllValues<FName>(OutData, Selector, Values, /*SourceSelector=*/nullptr, InContext));
			break;
		}
		default:
			return false;
		}
		return true;
	}

	uint64 ComputePackedSizeBytes(const FPCGDataDesc& InDataDesc)
	{
		if (!ensure(InDataDesc.GetElementDimension() == EPCGElementDimension::One))
		{
			return 0u;
		}

		const FPCGDataTypeIdentifier& Type = InDataDesc.GetType();

		if (!PCGComputeHelpers::IsTypeAllowedInDataCollection(Type))
		{
			return 0u;
		}

		uint64 DataSizeBytes = PCGDataCollectionPackingConstants::DATA_HEADER_SIZE_BYTES;

		for (const FPCGKernelAttributeDesc& AttributeDesc : InDataDesc.GetAttributeDescriptions())
		{
			const uint64 NumValues = static_cast<uint64>(InDataDesc.GetElementCountForAttribute(AttributeDesc).X);
			DataSizeBytes += static_cast<uint64>(PCGDataDescriptionHelpers::GetAttributeTypeStrideBytes(AttributeDesc.GetAttributeKey().GetType())) * NumValues;
		}

		return DataSizeBytes;
	}

	uint32 ComputePackedHeaderSizeBytes(TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc)
	{
		return InDataCollectionDesc ? PCGDataCollectionPackingConstants::DATA_COLLECTION_HEADER_SIZE_BYTES + PCGDataCollectionPackingConstants::DATA_HEADER_SIZE_BYTES * InDataCollectionDesc->GetDataDescriptions().Num() : 0;
	}
	
	uint64 ComputePackedSizeBytes(TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDataCollectionPacking::ComputePackedSizeBytes);

		if (!InDataCollectionDesc)
		{
			return 0;
		}
	
		bool bRequiresHeader = false;
		uint64 TotalDataSizeBytes = 0;

		const TConstArrayView<FPCGDataDesc> DataDescs = InDataCollectionDesc->GetDataDescriptions();
	
		for (const FPCGDataDesc& DataDesc : DataDescs)
		{
			bRequiresHeader |= PCGComputeHelpers::IsTypeAllowedInDataCollection(DataDesc.GetType());
			TotalDataSizeBytes += ComputePackedSizeBytes(DataDesc);
		}
	
		// Assumption: If no data items then pack as empty data collection, so include header.
		if (bRequiresHeader || DataDescs.IsEmpty())
		{
			TotalDataSizeBytes += ComputePackedHeaderSizeBytes(InDataCollectionDesc);
		}
	
		return TotalDataSizeBytes;
	}
	
	void WriteHeader(TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc, TArray<uint32>& OutPackedDataCollectionHeader)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDataCollectionPacking::WriteHeader);

		if (!InDataCollectionDesc)
		{
			return;
		}

		const TConstArrayView<FPCGDataDesc> DataDescs = InDataCollectionDesc->GetDataDescriptions();
	
		const uint32 HeaderSizeBytes = ComputePackedHeaderSizeBytes(InDataCollectionDesc);
		const uint32 HeaderSizeUints = HeaderSizeBytes >> 2;
	
		if (OutPackedDataCollectionHeader.Num() < static_cast<int32>(HeaderSizeUints))
		{
			OutPackedDataCollectionHeader.SetNumUninitialized(HeaderSizeUints);
		}
	
		// Zero-initialize header portion. We detect absent attributes using 0s.
		for (uint32 Index = 0; Index < HeaderSizeUints; ++Index)
		{
			OutPackedDataCollectionHeader[Index] = 0;
		}
	
		uint32 WriteAddressUints = 0;
	
		// Start at end of header
		uint32 DataStartAddressBytes = HeaderSizeBytes;
	
		// Num data
		OutPackedDataCollectionHeader[WriteAddressUints++] = DataDescs.Num();
	
		for (int32 DataIndex = 0; DataIndex < DataDescs.Num(); ++DataIndex)
		{
			const FPCGDataDesc& DataDesc = DataDescs[DataIndex];
			ensure(DataDesc.GetElementDimension() == EPCGElementDimension::One);
	
			// Data i: type ID
			if (DataDesc.GetType() == EPCGDataType::Param)
			{
				OutPackedDataCollectionHeader[WriteAddressUints++] = PCGDataCollectionPackingConstants::PARAM_DATA_TYPE_ID;
			}
			else
			{
				ensure(DataDesc.GetType() == EPCGDataType::Point);
				OutPackedDataCollectionHeader[WriteAddressUints++] = PCGDataCollectionPackingConstants::POINT_DATA_TYPE_ID;
			}
	
			// Data i: attribute count (including intrinsic point properties)
			OutPackedDataCollectionHeader[WriteAddressUints++] = DataDesc.GetAttributeDescriptions().Num();
	
			// Data i: element count
			OutPackedDataCollectionHeader[WriteAddressUints++] = DataDesc.GetElementCount().X;
	
			const uint32 DataAttributesHeaderStartAddressBytes = WriteAddressUints << 2;
	
			for (const FPCGKernelAttributeDesc& AttributeDesc : DataDesc.GetAttributeDescriptions())
			{
				const int AttributeStride = PCGDataDescriptionHelpers::GetAttributeTypeStrideBytes(AttributeDesc.GetAttributeKey().GetType());
				const int AttributeElementsCount = DataDesc.GetElementCountForAttribute(AttributeDesc).X;
	
				// Scatter from attributes that are present into header which has slots for all possible attributes.
				WriteAddressUints = (AttributeDesc.GetAttributeId() * PCGDataCollectionPackingConstants::ATTRIBUTE_HEADER_SIZE_BYTES + DataAttributesHeaderStartAddressBytes) >> 2;
	
				// Data i element j: packed attribute info and data address.
				const uint32 PackedAttributeInfo = (DataDesc.IsAttributeAllocated(AttributeDesc) ? PCGDataCollectionPackingConstants::AttributeAllocatedMask : 0) + (AttributeDesc.GetAttributeId() << 8) + AttributeStride;
				OutPackedDataCollectionHeader[WriteAddressUints++] = PackedAttributeInfo;
				OutPackedDataCollectionHeader[WriteAddressUints++] = DataStartAddressBytes;
	
				// Move the DataStartAddress to the beginning of the next attribute by skipping all the bytes of the current attribute.
				DataStartAddressBytes += AttributeElementsCount * AttributeStride;
			}
	
			// After scattering in attribute headers, fast forward to end of section.
			WriteAddressUints = (PCGDataCollectionPackingConstants::MAX_NUM_ATTRS * PCGDataCollectionPackingConstants::ATTRIBUTE_HEADER_SIZE_BYTES + DataAttributesHeaderStartAddressBytes) >> 2;
		}
	
		check(WriteAddressUints * 4 == HeaderSizeBytes);
	}
	
	void PackDataCollection(const FPCGDataCollection& InDataCollection, TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc, FName InPin, const UPCGDataBinding* InDataBinding, TArray<uint32>& OutPackedDataCollection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDataCollectionPacking::PackDataCollection);

		if (!ensure(InDataCollectionDesc))
		{
			return;
		}

		const TConstArrayView<FPCGDataDesc> DataDescs = InDataCollectionDesc->GetDataDescriptions();
		const TArray<FPCGTaggedData> InputData = InDataCollection.GetInputsByPin(InPin);
	
		for (const FPCGTaggedData& TaggedData : InputData)
		{
			if (TaggedData.Data && !PCGComputeHelpers::IsTypeAllowedInDataCollection(TaggedData.Data->GetDataType()))
			{
				UE_LOGF(LogPCG, Error,
					"PackDataCollection: Encountered data '%ls' which does not support packing into data collections. " "Ensure the correct data interface has been created for the pin ('%ls'). Packing will fail.",
					*TaggedData.Data->GetName(), *InPin.ToString());
				return;
			}
		}
	
		const uint32 PackedDataCollectionSizeBytes = ComputePackedSizeBytes(InDataCollectionDesc);
		check(PackedDataCollectionSizeBytes >= 0);
	
		// Uninitialized is fine, all data is initialized explicitly.
		OutPackedDataCollection.SetNumUninitialized(PackedDataCollectionSizeBytes >> 2);
	
	#if !UE_BUILD_SHIPPING
		if (PCGSystemSwitches::CVarFuzzGPUMemory.GetValueOnAnyThread())
		{
			FRandomStream Rand(GFrameCounter);
			for (uint32& Value : OutPackedDataCollection)
			{
				Value = Rand.GetUnsignedInt();
			}
		}
	#endif // !UE_BUILD_SHIPPING
	
		// Data addresses are written to the header and will be used during packing below.
		WriteHeader(InDataCollectionDesc, OutPackedDataCollection);
	
		for (int32 DataIndex = 0; DataIndex < InputData.Num(); ++DataIndex)
		{
			const FPCGDataDesc& DataDesc = DataDescs[DataIndex];
			ensure(DataDesc.GetElementDimension() == EPCGElementDimension::One);
	
			if (Cast<UPCGProxyForGPUData>(InputData[DataIndex].Data))
			{
				UE_LOGF(LogPCG, Error, "Attempted to pack a data that is not resident on the CPU. Uploaded data will be uninitialized.");
				continue;
			}
	
			// No work to do if there are no elements to process.
			if (DataDesc.GetElementCount().X <= 0)
			{
				continue;
			}
	
			const UPCGMetadata* Metadata = InputData[DataIndex].Data ? InputData[DataIndex].Data->ConstMetadata() : nullptr;
			if (!ensure(Metadata))
			{
				continue;
			}
	
			if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(InputData[DataIndex].Data))
			{
				const uint32 NumElements = PointData->GetNumPoints();
	
				if (NumElements == 0)
				{
					continue;
				}
	
				for (const FPCGKernelAttributeDesc& AttributeDesc : DataDesc.GetAttributeDescriptions())
				{
					const uint32 AttributeId = AttributeDesc.GetAttributeId();
	
					const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomain(AttributeDesc.GetAttributeKey().GetIdentifier().MetadataDomain);
					const FPCGMetadataAttributeBase* AttributeBase = (AttributeId >= PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS && MetadataDomain) ? MetadataDomain->GetConstAttribute(AttributeDesc.GetAttributeKey().GetIdentifier().Name) : nullptr;
	
					uint32 AddressUints = GetElementDataStartAddressUints(OutPackedDataCollection.GetData(), DataIndex, AttributeId);
	
					if (AttributeId < PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS)
					{
						const uint32 NumElementsForAttribute = DataDesc.IsAttributeAllocated(AttributeDesc) ? NumElements : 1u;
	
						// Point property.
						switch (AttributeId)
						{
						case PCGDataCollectionPackingConstants::POINT_POSITION_ATTRIBUTE_ID:
						{
							const TConstPCGValueRange<FTransform> ReadValueRange = PointData->GetConstTransformValueRange();
	
							for (uint32 ElementIndex = 0; ElementIndex < NumElementsForAttribute; ++ElementIndex)
							{
								const FVector Position = ReadValueRange[ElementIndex].GetLocation();
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Position.X));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Position.Y));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Position.Z));
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_ROTATION_ATTRIBUTE_ID:
						{
							const TConstPCGValueRange<FTransform> ReadValueRange = PointData->GetConstTransformValueRange();
	
							for (uint32 ElementIndex = 0; ElementIndex < NumElementsForAttribute; ++ElementIndex)
							{
								const FQuat Rotation = ReadValueRange[ElementIndex].GetRotation();
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Rotation.X));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Rotation.Y));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Rotation.Z));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Rotation.W));
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_SCALE_ATTRIBUTE_ID:
						{
							const TConstPCGValueRange<FTransform> ReadValueRange = PointData->GetConstTransformValueRange();
	
							for (uint32 ElementIndex = 0; ElementIndex < NumElementsForAttribute; ++ElementIndex)
							{
								const FVector Scale = ReadValueRange[ElementIndex].GetScale3D();
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Scale.X));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Scale.Y));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Scale.Z));
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_BOUNDS_MIN_ATTRIBUTE_ID:
						{
							const TConstPCGValueRange<FVector> ReadValueRange = PointData->GetConstBoundsMinValueRange();
	
							for (uint32 ElementIndex = 0; ElementIndex < NumElementsForAttribute; ++ElementIndex)
							{
								const FVector& BoundsMin = ReadValueRange[ElementIndex];
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(BoundsMin.X));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(BoundsMin.Y));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(BoundsMin.Z));
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_BOUNDS_MAX_ATTRIBUTE_ID:
						{
							const TConstPCGValueRange<FVector> ReadValueRange = PointData->GetConstBoundsMaxValueRange();
	
							for (uint32 ElementIndex = 0; ElementIndex < NumElementsForAttribute; ++ElementIndex)
							{
								const FVector& BoundsMax = ReadValueRange[ElementIndex];
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(BoundsMax.X));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(BoundsMax.Y));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(BoundsMax.Z));
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_COLOR_ATTRIBUTE_ID:
						{
							const TConstPCGValueRange<FVector4> ReadValueRange = PointData->GetConstColorValueRange();
	
							for (uint32 ElementIndex = 0; ElementIndex < NumElementsForAttribute; ++ElementIndex)
							{
								const FVector4& Color = ReadValueRange[ElementIndex];
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Color.X));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Color.Y));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Color.Z));
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(static_cast<float>(Color.W));
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_DENSITY_ATTRIBUTE_ID:
						{
							const TConstPCGValueRange<float> ReadValueRange = PointData->GetConstDensityValueRange();
	
							for (uint32 ElementIndex = 0; ElementIndex < NumElementsForAttribute; ++ElementIndex)
							{
								const float Density = ReadValueRange[ElementIndex];
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(Density);
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_SEED_ATTRIBUTE_ID:
						{
							const TConstPCGValueRange<int32> ReadValueRange = PointData->GetConstSeedValueRange();
	
							for (uint32 ElementIndex = 0; ElementIndex < NumElementsForAttribute; ++ElementIndex)
							{
								const int Seed = ReadValueRange[ElementIndex];
								OutPackedDataCollection[AddressUints++] = Seed;
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_STEEPNESS_ATTRIBUTE_ID:
						{
							const TConstPCGValueRange<float> ReadValueRange = PointData->GetConstSteepnessValueRange();
	
							for (uint32 ElementIndex = 0; ElementIndex < NumElementsForAttribute; ++ElementIndex)
							{
								const float Steepness = ReadValueRange[ElementIndex];
								OutPackedDataCollection[AddressUints++] = FMath::AsUInt(Steepness);
							}
							break;
						}
						default:
							checkNoEntry();
							break;
						}
					}
					else
					{
						// Pack attribute. Validate first element only for perf.
						if (AttributeBase->GetMetadataDomain()->GetDomainID() == PCGMetadataDomainID::Data)
						{
							ensure(PackAttributeHelper(AttributeBase, AttributeDesc, /*InEntryKey*/0, InDataBinding->GetStringTable(), OutPackedDataCollection, AddressUints));
						}
						else
						{
							const TConstPCGValueRange<int64> ReadValueRange = PointData->GetConstMetadataEntryValueRange();
	
							ensure(PackAttributeHelper(AttributeBase, AttributeDesc, ReadValueRange[0], InDataBinding->GetStringTable(), OutPackedDataCollection, AddressUints));
							for (uint32 ElementIndex = 1; ElementIndex < NumElements; ++ElementIndex)
							{
								PackAttributeHelper(AttributeBase, AttributeDesc, ReadValueRange[ElementIndex], InDataBinding->GetStringTable(), OutPackedDataCollection, AddressUints);
							}
						}
					}
				}
			}
			else if (const UPCGParamData* ParamData = Cast<UPCGParamData>(InputData[DataIndex].Data))
			{
				for (const FPCGKernelAttributeDesc& AttributeDesc : DataDesc.GetAttributeDescriptions())
				{
					const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomain(AttributeDesc.GetAttributeKey().GetIdentifier().MetadataDomain);
					const FPCGMetadataAttributeBase* AttributeBase = MetadataDomain ? MetadataDomain->GetConstAttribute(AttributeDesc.GetAttributeKey().GetIdentifier().Name) : nullptr;
					if (!AttributeBase)
					{
						continue;
					}
	
					uint32 AddressUints = GetElementDataStartAddressUints(OutPackedDataCollection.GetData(), DataIndex, AttributeDesc.GetAttributeId());
	
					// Pack attribute. Validate first element only for perf.
					ensure(PackAttributeHelper(AttributeBase, AttributeDesc, /*InEntryKey*/0, InDataBinding->GetStringTable(), OutPackedDataCollection, AddressUints));
					for (int32 ElementIndex = 1; ElementIndex < DataDesc.GetElementCountForAttribute(AttributeDesc).X; ++ElementIndex)
					{
						PackAttributeHelper(AttributeBase, AttributeDesc, /*InEntryKey*/ElementIndex, InDataBinding->GetStringTable(), OutPackedDataCollection, AddressUints);
					}
				}
			}
			else { /* TODO: Support additional data types. */ }
		}
	}
	
	EPCGUnpackDataCollectionResult UnpackDataCollection(FPCGContext* InContext, TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc, const TArray<uint8>& InPackedData, FName InPin, const TArray<FString>& InStringTable, FPCGDataCollection& OutDataCollection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDataCollectionPacking::UnpackDataCollection);
	
		if (InPackedData.IsEmpty())
		{
			ensureMsgf(false, TEXT("Tried to unpack a GPU data collection, but the readback buffer was empty."));
			return EPCGUnpackDataCollectionResult::NoData;
		}

		if (!ensure(InDataCollectionDesc))
		{
			return EPCGUnpackDataCollectionResult::NoData;
		}

		const TConstArrayView<FPCGDataDesc> DataDescs = InDataCollectionDesc->GetDataDescriptions();
	
		const float* DataAsFloat = reinterpret_cast<const float*>(InPackedData.GetData());
		const uint32* DataAsUint = reinterpret_cast<const uint32*>(InPackedData.GetData());
		const int32* DataAsInt = reinterpret_cast<const int32*>(InPackedData.GetData());
	
		const uint32 NumData = DataAsUint[0];
	
		if (NumData != DataDescs.Num())
		{
			return EPCGUnpackDataCollectionResult::DataMismatch;
		}
	
		TArray<FPCGTaggedData>& OutData = OutDataCollection.TaggedData;
		TStaticArray<FPCGMetadataAttributeBase*, PCGDataCollectionPackingConstants::MAX_NUM_ATTRS> MetadataAttributes;
	
		for (uint32 DataIndex = 0; DataIndex < NumData; ++DataIndex)
		{
			const FPCGDataDesc& DataDesc = DataDescs[DataIndex];
			ensure(DataDesc.GetElementDimension() == EPCGElementDimension::One);
	
			const uint32 DataHeaderAddress = (PCGDataCollectionPackingConstants::DATA_COLLECTION_HEADER_SIZE_BYTES + PCGDataCollectionPackingConstants::DATA_HEADER_SIZE_BYTES * DataIndex) / sizeof(uint32);
			const uint32 TypeId =        DataAsUint[DataHeaderAddress + 0];
			const uint32 NumAttributes = DataAsUint[DataHeaderAddress + 1];
			
			uint32 NumElements = DataAsUint[DataHeaderAddress + 2];
			// Use tighter/more refined element count if we have one:
			if (DataDesc.GetElementCount().X >= 0)
			{
				NumElements = FMath::Min(NumElements, static_cast<uint32>(DataDesc.GetElementCount().X));
			}
	
			const TConstArrayView<FPCGKernelAttributeDesc> AttributeDescs = DataDesc.GetAttributeDescriptions();
			check(NumAttributes == AttributeDescs.Num());
			check(AttributeDescs.Num() <= PCGDataCollectionPackingConstants::MAX_NUM_ATTRS);
	
			if (TypeId == PCGDataCollectionPackingConstants::POINT_DATA_TYPE_ID)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UnpackPointDataItem);
	
				UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(InContext);
				UPCGMetadata* Metadata = OutPointData->MutableMetadata();
				OutPointData->SetNumPoints(NumElements, /*bInitializeValues=*/false);
	
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(InitializeOutput);
	
					// We only need to add the entry keys if there are actually attributes to unpack.
					if (DataDesc.HasElementsMetadataDomainAttributes())
					{
						TPCGValueRange<int64> MetadataEntryRange = OutPointData->GetMetadataEntryValueRange(/*bAllocate=*/true);
						TArray<PCGMetadataEntryKey*> ParentEntryKeys;
						ParentEntryKeys.Reserve(NumElements);
	
						for (uint32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
						{
							PCGMetadataEntryKey& EntryKey = MetadataEntryRange[ElementIndex];
							EntryKey = PCGInvalidEntryKey;
							ParentEntryKeys.Add(&EntryKey);
						}
	
						Metadata->AddEntriesInPlace(ParentEntryKeys);
					}
					else if(OutPointData->IsA<UPCGPointData>())
					{
						// Special case so that we can use ParallelFor (UPCGPointArrayData doesn't need this)
						TPCGValueRange<int64> MetadataEntryRange = OutPointData->GetMetadataEntryValueRange(/*bAllocate=*/true);
						ParallelFor(NumElements, [&MetadataEntryRange](int32 ElementIndex)
						{
							MetadataEntryRange[ElementIndex] = -1;
						});
					}
					else
					{
						OutPointData->SetMetadataEntry(-1);
					}
				}
	
				FPCGTaggedData& OutTaggedData = OutData.Emplace_GetRef();
				OutTaggedData.Data = OutPointData;
				OutTaggedData.Pin = InPin;
	
				for (const FPCGKernelAttributeDesc& AttributeDesc : AttributeDescs)
				{
					if (AttributeDesc.GetAttributeId() >= PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS)
					{
						MetadataAttributes[AttributeDesc.GetAttributeId()] = CreateAttributeFromAttributeDesc(Metadata, AttributeDesc);
					}
				}
	
				// No work to do if there are no elements to process.
				if (NumElements == 0)
				{
					continue;
				}
	
				FTransform DefaultTransform = FTransform::Identity;
				bool bTransformIsAllocated = false;
	
				// Loop over attributes.
				for (const FPCGKernelAttributeDesc& AttributeDesc : AttributeDescs)
				{
					const uint32 AttributeId = AttributeDesc.GetAttributeId();
					const uint32 AttributeStrideUints = PCGDataDescriptionHelpers::GetAttributeTypeStrideBytes(AttributeDesc.GetAttributeKey().GetType()) >> 2;
					const uint32 AddressUints = GetElementDataStartAddressUints(DataAsUint, DataIndex, AttributeId);
	
					if (AttributeId < PCGDataCollectionPackingConstants::NUM_RESERVED_ATTRS)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UnpackPointProperty);
	
						const bool bAllocated = DataDesc.IsAttributeAllocated(AttributeDesc);
	
						// We tried hoisting this decision to a lambda but it didn't appear to help.
						switch (AttributeId)
						{
						case PCGDataCollectionPackingConstants::POINT_POSITION_ATTRIBUTE_ID:
						{
							bTransformIsAllocated = bAllocated;
	
							if (bAllocated)
							{
								TPCGValueRange<FTransform> TransformRange = OutPointData->GetTransformValueRange(/*bAllocate=*/true);
								ParallelFor(NumElements, [DataAsFloat, AddressUints, &TransformRange](int32 ElementIndex)
								{
									const FVector Location = FVector(
										DataAsFloat[AddressUints + ElementIndex * 3 + 0],
										DataAsFloat[AddressUints + ElementIndex * 3 + 1],
										DataAsFloat[AddressUints + ElementIndex * 3 + 2]);
	
									TransformRange[ElementIndex].SetLocation(Location);
								});
							}
							else
							{
								const FVector Location = FVector(
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 0],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 1],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 2]);
	
								DefaultTransform.SetLocation(Location);
							}
	
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_ROTATION_ATTRIBUTE_ID:
						{
							if (bAllocated)
							{
								TPCGValueRange<FTransform> TransformRange = OutPointData->GetTransformValueRange(/*bAllocate=*/true);
								ParallelFor(NumElements, [DataAsFloat, AddressUints, &TransformRange](int32 ElementIndex)
								{
									const FQuat Rotation = FQuat(
										DataAsFloat[AddressUints + ElementIndex * 4 + 0],
										DataAsFloat[AddressUints + ElementIndex * 4 + 1],
										DataAsFloat[AddressUints + ElementIndex * 4 + 2],
										DataAsFloat[AddressUints + ElementIndex * 4 + 3]);
	
									// Normalize here with default tolerance (zero quat will return identity).
									TransformRange[ElementIndex].SetRotation(Rotation.GetNormalized());
								});
							}
							else
							{
								const FQuat Rotation = FQuat(
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 4 + 0],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 4 + 1],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 4 + 2],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 4 + 3]);
	
								DefaultTransform.SetRotation(Rotation);
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_SCALE_ATTRIBUTE_ID:
						{
							if (bAllocated)
							{
								TPCGValueRange<FTransform> TransformRange = OutPointData->GetTransformValueRange(/*bAllocate=*/true);
								ParallelFor(NumElements, [DataAsFloat, AddressUints, &TransformRange](int32 ElementIndex)
								{
									const FVector Scale = FVector(
										DataAsFloat[AddressUints + ElementIndex * 3 + 0],
										DataAsFloat[AddressUints + ElementIndex * 3 + 1],
										DataAsFloat[AddressUints + ElementIndex * 3 + 2]);
	
									TransformRange[ElementIndex].SetScale3D(Scale);
								});
							}
							else
							{
								const FVector Scale = FVector(
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 0],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 1],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 2]);
	
								DefaultTransform.SetScale3D(Scale);
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_BOUNDS_MIN_ATTRIBUTE_ID:
						{
							if (bAllocated)
							{
								TPCGValueRange<FVector> BoundsMinRange = OutPointData->GetBoundsMinValueRange(/*bAllocate=*/true);
								ParallelFor(NumElements, [DataAsFloat, AddressUints, &BoundsMinRange](int32 ElementIndex)
								{
									const FVector BoundsMin = FVector(
										DataAsFloat[AddressUints + ElementIndex * 3 + 0],
										DataAsFloat[AddressUints + ElementIndex * 3 + 1],
										DataAsFloat[AddressUints + ElementIndex * 3 + 2]);
	
									BoundsMinRange[ElementIndex] = BoundsMin;
								});
							}
							else
							{
								const FVector BoundsMin = FVector(
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 0],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 1],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 2]);
	
								OutPointData->SetBoundsMin(BoundsMin);
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_BOUNDS_MAX_ATTRIBUTE_ID:
						{
							if (bAllocated)
							{
								TPCGValueRange<FVector> BoundsMaxRange = OutPointData->GetBoundsMaxValueRange(/*bAllocate=*/true);
								ParallelFor(NumElements, [DataAsFloat, AddressUints, &BoundsMaxRange](int32 ElementIndex)
								{
									const FVector BoundsMax = FVector(
										DataAsFloat[AddressUints + ElementIndex * 3 + 0],
										DataAsFloat[AddressUints + ElementIndex * 3 + 1],
										DataAsFloat[AddressUints + ElementIndex * 3 + 2]);
	
									BoundsMaxRange[ElementIndex] = BoundsMax;
								});
							}
							else
							{
								const FVector BoundsMax = FVector(
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 0],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 1],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 3 + 2]);
	
								OutPointData->SetBoundsMax(BoundsMax);
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_COLOR_ATTRIBUTE_ID:
						{
							if (bAllocated)
							{
								TPCGValueRange<FVector4> ColorRange = OutPointData->GetColorValueRange(/*bAllocate=*/true);
								ParallelFor(NumElements, [DataAsFloat, AddressUints, &ColorRange](int32 ElementIndex)
								{
									const FVector4 Color = FVector4(
										DataAsFloat[AddressUints + ElementIndex * 4 + 0],
										DataAsFloat[AddressUints + ElementIndex * 4 + 1],
										DataAsFloat[AddressUints + ElementIndex * 4 + 2],
										DataAsFloat[AddressUints + ElementIndex * 4 + 3]);
	
									ColorRange[ElementIndex] = Color;
								});
							}
							else
							{
								const FVector4 Color = FVector4(
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 4 + 0],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 4 + 1],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 4 + 2],
									DataAsFloat[AddressUints + /*ElementIndex=*/0 * 4 + 3]);
	
								OutPointData->SetColor(Color);
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_DENSITY_ATTRIBUTE_ID:
						{
							if (bAllocated)
							{
								TPCGValueRange<float> DensityRange = OutPointData->GetDensityValueRange(/*bAllocate=*/true);
								ParallelFor(NumElements, [DataAsFloat, AddressUints, &DensityRange](int32 ElementIndex)
								{
									DensityRange[ElementIndex] = DataAsFloat[AddressUints + ElementIndex];
								});
							}
							else
							{
								OutPointData->SetDensity(DataAsFloat[AddressUints]);
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_SEED_ATTRIBUTE_ID:
						{
							if (bAllocated)
							{
								TPCGValueRange<int32> SeedRange = OutPointData->GetSeedValueRange(/*bAllocate=*/true);
								ParallelFor(NumElements, [DataAsInt, AddressUints, &SeedRange](int32 ElementIndex)
								{
									SeedRange[ElementIndex] = DataAsInt[AddressUints + ElementIndex];
								});
							}
							else
							{
								OutPointData->SetSeed(DataAsInt[AddressUints]);
							}
							break;
						}
						case PCGDataCollectionPackingConstants::POINT_STEEPNESS_ATTRIBUTE_ID:
						{
							if (bAllocated)
							{
								TPCGValueRange<float> SteepnessRange = OutPointData->GetSteepnessValueRange(/*bAllocate=*/true);
								ParallelFor(NumElements, [DataAsFloat, AddressUints, &SteepnessRange](int32 ElementIndex)
								{
									SteepnessRange[ElementIndex] = DataAsFloat[AddressUints + ElementIndex];
								});
							}
							else
							{
								OutPointData->SetSteepness(DataAsFloat[AddressUints]);
							}
							break;
						}
						default:
							checkNoEntry();
							break;
						}
					}
					else
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UnpackAttribute);
	
						if (FPCGMetadataAttributeBase* AttributeBase = MetadataAttributes[AttributeDesc.GetAttributeId()])
						{
							ensure(UnpackAttributeHelper(InContext, InPackedData.GetData(), AttributeDesc, InStringTable, AddressUints, DataDesc.GetElementCountForAttribute(AttributeDesc).X, OutPointData));
						}
					}
				}
	
				if (!bTransformIsAllocated)
				{
					OutPointData->SetTransform(DefaultTransform);
				}
	
				// TODO: It may be more efficient to create a mapping from input point index to final output point index and do everything in one pass.
				auto ProcessRangeFunc = [OutPointData](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
				{
					int32 NumWritten = 0;
	
					const FConstPCGPointValueRanges InRanges(OutPointData);
					FPCGPointValueRanges OutRanges(OutPointData, /*bAllocate=*/false);
	
					for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
					{
						if (InRanges.DensityRange[ReadIndex] == PCGDataCollectionPackingConstants::INVALID_DENSITY)
						{
							continue;
						}
	
						const int32 WriteIndex = StartWriteIndex + NumWritten;
						OutRanges.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);
						++NumWritten;
					}
	
					return NumWritten;
				};
	
				auto MoveDataRangeFunc = [OutPointData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
				{
					OutPointData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
				};
	
				auto FinishedFunc = [OutPointData](int32 NumWritten)
				{
					OutPointData->SetNumPoints(NumWritten);
				};
	
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(DiscardInvalidPoints);
					FPCGAsyncState* AsyncState = InContext ? &InContext->AsyncState : nullptr;
					FPCGAsync::AsyncProcessingRangeEx(
						AsyncState, 
						OutPointData->GetNumPoints(),
						[]{},
						ProcessRangeFunc,
						MoveDataRangeFunc,
						FinishedFunc,
						/*bEnableTimeSlicing=*/false);
				}
			}
			else if (TypeId == PCGDataCollectionPackingConstants::PARAM_DATA_TYPE_ID)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UnpackParamDataItem);
	
				UPCGParamData* OutParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InContext);
				UPCGMetadata* Metadata = OutParamData->MutableMetadata();
	
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(InitializeOutput);
	
					TArray<TTuple</*EntryKey=*/int64, /*ParentEntryKey=*/int64>> AllMetadataEntries;
					AllMetadataEntries.SetNumUninitialized(NumElements);
	
					ParallelFor(NumElements, [&](int32 ElementIndex)
					{
						//LLM_SCOPE_BYTAG(PCG); // AddEntryPlaceholder() does not seem to allocate but consider chunking this loop and enabling this LLM tag if this changes.
						AllMetadataEntries[ElementIndex] = MakeTuple(Metadata->AddEntryPlaceholder(), PCGInvalidEntryKey);
					});
	
					Metadata->AddDelayedEntries(AllMetadataEntries);
				}
	
				FPCGTaggedData& OutTaggedData = OutData.Emplace_GetRef();
				OutTaggedData.Data = OutParamData;
				OutTaggedData.Pin = InPin;
	
				for (const FPCGKernelAttributeDesc& AttributeDesc : AttributeDescs)
				{
					MetadataAttributes[AttributeDesc.GetAttributeId()] = CreateAttributeFromAttributeDesc(Metadata, AttributeDesc);
				}
	
				// No work to do if there are no elements to process.
				if (NumElements == 0)
				{
					continue;
				}
	
				// Loop over attributes.
				for (const FPCGKernelAttributeDesc& AttributeDesc : AttributeDescs)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UnpackAttribute);
	
					if (FPCGMetadataAttributeBase* AttributeBase = MetadataAttributes[AttributeDesc.GetAttributeId()])
					{
						const uint32 AddressUints = GetElementDataStartAddressUints(DataAsUint, DataIndex, AttributeDesc.GetAttributeId());
						ensure(UnpackAttributeHelper(InContext, InPackedData.GetData(), AttributeDesc, InStringTable, AddressUints, NumElements, OutParamData));
					}
				}
			}
			else { /* TODO: Support additional data types. */ }
		}
	
		return EPCGUnpackDataCollectionResult::Success;
	}
}
