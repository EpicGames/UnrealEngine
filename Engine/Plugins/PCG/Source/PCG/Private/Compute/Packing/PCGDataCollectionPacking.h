// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UPCGDataBinding;
struct FPCGContext;
struct FPCGDataCollection;
struct FPCGDataCollectionDesc;
struct FPCGDataDesc;

enum class EPCGUnpackDataCollectionResult
{
	Success,
	DataMismatch,
	NoData
};

namespace PCGDataCollectionPackingConstants
{
	constexpr int32 MAX_NUM_ATTRS = 128;
	constexpr int32 NUM_RESERVED_ATTRS = 32; // Reserved for point properties, spline accessors, etc.
	constexpr int32 MAX_NUM_CUSTOM_ATTRS = MAX_NUM_ATTRS - NUM_RESERVED_ATTRS; // Reserved for custom attributes

	constexpr int32 DATA_COLLECTION_HEADER_SIZE_BYTES = 4; // 4 bytes for NumData
	constexpr int32 DATA_HEADER_PREAMBLE_SIZE_BYTES = 12; // 4 bytes for DataType, 4 bytes for NumAttrs, 4 bytes for NumElements
	constexpr int32 ATTRIBUTE_HEADER_SIZE_BYTES = 8; // 4 bytes for PackedIdAndStride, 4 bytes for data start address
	constexpr int32 DATA_HEADER_SIZE_BYTES = DATA_HEADER_PREAMBLE_SIZE_BYTES + MAX_NUM_ATTRS * ATTRIBUTE_HEADER_SIZE_BYTES;

	constexpr int32 POINT_DATA_TYPE_ID = 0;
	constexpr int32 PARAM_DATA_TYPE_ID = 1;

	constexpr int32 NUM_POINT_PROPERTIES = 9;
	constexpr int32 POINT_POSITION_ATTRIBUTE_ID = 0;
	constexpr int32 POINT_ROTATION_ATTRIBUTE_ID = 1;
	constexpr int32 POINT_SCALE_ATTRIBUTE_ID = 2;
	constexpr int32 POINT_BOUNDS_MIN_ATTRIBUTE_ID = 3;
	constexpr int32 POINT_BOUNDS_MAX_ATTRIBUTE_ID = 4;
	constexpr int32 POINT_COLOR_ATTRIBUTE_ID = 5;
	constexpr int32 POINT_DENSITY_ATTRIBUTE_ID = 6;
	constexpr int32 POINT_SEED_ATTRIBUTE_ID = 7;
	constexpr int32 POINT_STEEPNESS_ATTRIBUTE_ID = 8;

	// PackedAttributeInfo bitmasks
	constexpr uint32 AttributeStrideMask = 0xFF;
	constexpr uint32 AttributeAllocatedMask = 1 << 31;

	// Used to represent invalid/removed points. We use a value slightly less than max float,
	// as not all platforms support float infinity in shaders.
	constexpr float INVALID_DENSITY = 3.402823e+38f;
}

namespace PCGDataCollectionPackingHelpers
{
	/** Computes the size (in bytes) of the data description after packing. */
	uint64 ComputePackedSizeBytes(const FPCGDataDesc& InDataDesc);

	/** Computes the size (in bytes) of the header portion of the packed data collection buffer. */
	uint32 ComputePackedHeaderSizeBytes(TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc);

	/** Computes the size (in bytes) of the data collection after packing. */
	uint64 ComputePackedSizeBytes(TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc);

	void WriteHeader(TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc, TArray<uint32>& OutPackedDataCollectionHeader);

	/** Pack a data collection into the GPU data format. DataDescs defines which attributes are packed. */
	void PackDataCollection(const FPCGDataCollection& InDataCollection, TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc, FName InPin, const UPCGDataBinding* InDataBinding, TArray<uint32>& OutPackedDataCollection);

	/** Unpack a buffer of 8-bit uints to a data collection. */
	EPCGUnpackDataCollectionResult UnpackDataCollection(FPCGContext* InContext, TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc, const TArray<uint8>& InPackedData, FName InPin, const TArray<FString>& InStringTable, FPCGDataCollection& OutDataCollection);
}
