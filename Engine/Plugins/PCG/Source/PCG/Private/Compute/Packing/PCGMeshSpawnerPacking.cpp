// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/Packing/PCGMeshSpawnerPacking.h"

#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGDataDescription.h"
#include "Metadata/PCGAttributePropertySelector.h"

#define LOCTEXT_NAMESPACE "PCGMeshSpawnerPacking"

namespace PCGMeshSpawnerPackingHelpers
{
	void ComputeCustomFloatPacking(
		FPCGContext* InContext,
		const UPCGSettings* InSettings,
		TConstArrayView<FPCGAttributePropertyInputSelector> InAttributeSelectors,
		const TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDescription,
		uint32& OutCustomFloatCount,
		TArray<FUint32Vector4>& OutAttributeIdOffsetStrides)
	{
		check(InDataCollectionDescription);

		uint32 OffsetFloats = 0;

		for (const FPCGAttributePropertyInputSelector& AttributeSelector : InAttributeSelectors)
		{
			const FName AttributeName = AttributeSelector.GetAttributeName();

			// We need to do a lookup here as the user does not provide the full attribute description, only the attribute name. So we see what we can
			// find in the input data similar to the CPU SM Spawner.
			FPCGKernelAttributeDesc AttributeDesc;
			bool bConflictingTypesInData = false;
			bool bPresentOnAllData = false;
			InDataCollectionDescription->GetAttributeDesc(AttributeName, AttributeDesc, bConflictingTypesInData, bPresentOnAllData);

			if (bConflictingTypesInData)
			{
				// TODO: in future we could partition the execution - run it once per attribute type in input data.
				PCG_KERNEL_VALIDATION_ERR(InContext, InSettings,
					FText::Format(LOCTEXT("AttributePackingTypeConflict", "Attribute '{0}' encountered with multiple different types in input data, custom float packing failed."), FText::FromName(AttributeName)));

				OutAttributeIdOffsetStrides.Empty();
				OutCustomFloatCount = 0;

				return;
			}

			const uint32 StrideFloats = PCGDataDescriptionHelpers::GetAttributeTypeStrideBytes(AttributeDesc.GetAttributeKey().GetType()) / sizeof(float);

			OutAttributeIdOffsetStrides.Emplace(static_cast<uint32>(AttributeDesc.GetAttributeId()), OffsetFloats, StrideFloats, /*Unused*/0);

			OffsetFloats += StrideFloats;
		}

		OutCustomFloatCount = OffsetFloats;
	}
}

#undef LOCTEXT_NAMESPACE
