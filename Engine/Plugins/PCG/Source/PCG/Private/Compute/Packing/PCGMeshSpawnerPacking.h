// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UPCGSettings;
struct FPCGAttributePropertyInputSelector;
struct FPCGContext;
struct FPCGDataCollectionDesc;

namespace PCGMeshSpawnerPackingHelpers
{
	/** Compute how attributes can be packed to custom floats. */
	void ComputeCustomFloatPacking(
		FPCGContext* InContext,
		const UPCGSettings* InSettings,
		TConstArrayView<FPCGAttributePropertyInputSelector> InAttributeSelectors,
		const TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDescription,
		uint32& OutCustomFloatCount,
		TArray<FUint32Vector4>& OutAttributeIdOffsetStrides);
}
