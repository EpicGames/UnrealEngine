// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PrimitiveFactories/IPCGRuntimePrimitiveFactory.h"

int32 IPCGRuntimePrimitiveFactory::GetNumInstances(int32 InPrimitiveIndex, int32 InCellID) const
{
	// Fallback used if this new GetNumInstances function is not implemented.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetNumInstances(InPrimitiveIndex);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool IPCGRuntimePrimitiveFactory::IsAnyRenderStateDirty() const
{
	ensureMsgf(false, TEXT("IsAnyRenderStateDirty() should be overridden by classes implementing the IPCGRuntimePrimitiveFactory interface."));
	return false;
}
