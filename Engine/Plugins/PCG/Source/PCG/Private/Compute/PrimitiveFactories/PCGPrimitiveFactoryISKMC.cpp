// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PrimitiveFactories/PCGPrimitiveFactoryISKMC.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGManagedResource.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Compute/PCGComputeCommon.h"
#include "Helpers/PCGActorHelpers.h"

#include "PrimitiveSceneInfo.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/SkinnedAsset.h"

FPrimitiveSceneProxy* FPCGPrimitiveFactoryISKMC::GetSceneProxy(int32 InPrimitiveIndex) const
{
	if (ensure(Components.IsValidIndex(InPrimitiveIndex)))
	{
		return Components[InPrimitiveIndex].IsValid() ? Components[InPrimitiveIndex]->GetSceneProxy() : nullptr;
	}
	else
	{
		return nullptr;
	}
}

int32 FPCGPrimitiveFactoryISKMC::GetNumInstances(int32 InPrimitiveIndex, int32 InCellID) const
{
	return NumInstances.IsValidIndex(InPrimitiveIndex) ? NumInstances[InPrimitiveIndex] : 0;
}

int32 FPCGPrimitiveFactoryISKMC::GetNumInstancesTotal(int32 InPrimitiveIndex) const
{
	return NumInstances.IsValidIndex(InPrimitiveIndex) ? NumInstances[InPrimitiveIndex] : 0;
}

bool FPCGPrimitiveFactoryISKMC::IsRenderStateCreated() const
{
	for (int32 Index = 0; Index < GetNumPrimitives(); ++Index)
	{
		FPrimitiveSceneProxy* SceneProxy = GetSceneProxy(Index);
		if (!SceneProxy || !SceneProxy->GetPrimitiveSceneInfo() || SceneProxy->GetPrimitiveSceneInfo()->GetInstanceSceneDataOffset() == -1)
		{
			return false;
		}
	}

	return true;
}

bool FPCGPrimitiveFactoryISKMC::IsAnyRenderStateDirty() const
{
	for (const TWeakObjectPtr<UPrimitiveComponent>& WeakComp : Components)
	{
		const UPrimitiveComponent* Comp = WeakComp.Get();
		if (Comp && Comp->IsRenderStateDirty())
		{
			return true;
		}
	}

	return false;
}

void FPCGPrimitiveFactoryISKMC::Initialize(FParameters&& InParameters)
{
	Descriptors = InParameters.Descriptors;
	TargetActor = InParameters.TargetActor;
}

bool FPCGPrimitiveFactoryISKMC::Create(FPCGContext* InContext)
{
	UPCGComponent* SourceComponent = Cast<UPCGComponent>(InContext->ExecutionSource.Get());
	check(SourceComponent);

	FPCGSkinnedMeshComponentBuilderParams Params;
	Params.NumCustomDataFloats = 0; // This is the num custom floats allocated in CPU memory.

	// If the root actor we're binding to is movable, then the component should be movable by default
	if (USceneComponent* SceneComponent = TargetActor->GetRootComponent())
	{
		Params.Descriptor.Mobility = SceneComponent->Mobility;
	}

	const int32 NumToCreate = FMath::Min(Descriptors.Num(), PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER);

	bool bAnyCreated = false;

	while (NumPrimitivesProcessed < NumToCreate)
	{
		const FPCGSoftSkinnedMeshComponentDescriptor& Desc = Descriptors[NumPrimitivesProcessed];

		Params.Descriptor = Desc;

		// TODO - not sure this is a good managed resource object to use, could make procedural variant?
		UPCGManagedISKMComponent* ManagedComponent = UPCGActorHelpers::GetOrCreateManagedABMC(TargetActor, SourceComponent, Params, InContext);

		if (ManagedComponent)
		{
			Components.Add(ManagedComponent->GetComponent());
			NumInstances.Add(Desc.NumInstancesGPUOnly);
			MeshBounds.Add(Desc.SkinnedAsset->GetBounds().GetBox());
		}

		++NumPrimitivesProcessed;

		if (NumPrimitivesProcessed < NumToCreate && InContext->AsyncState.ShouldStop())
		{
			return false;
		}
	}

	return NumPrimitivesProcessed == NumToCreate;
}

FBox FPCGPrimitiveFactoryISKMC::GetMeshBounds(int32 InPrimitiveIndex) const
{
	return ensure(MeshBounds.IsValidIndex(InPrimitiveIndex)) ? MeshBounds[InPrimitiveIndex] : FBox();
}
