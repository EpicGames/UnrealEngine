// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PrimitiveFactories/IPCGRuntimePrimitiveFactory.h"

#include "UObject/WeakObjectPtr.h"

class AActor;
class UPrimitiveComponent;
struct FPCGContext;
struct FPCGCrc;
struct FPCGSoftSkinnedMeshComponentDescriptor;

class FPCGPrimitiveFactoryISKMC : public IPCGRuntimePrimitiveFactory
{
public:
	//~ Begin IPCGRuntimePrimitiveFactory interface
	virtual int32 GetNumPrimitives() const override { return Components.Num(); }
	virtual FPrimitiveSceneProxy* GetSceneProxy(int32 InPrimitiveIndex) const override;
	virtual bool IsPrimitiveValid(int32 InPrimitiveIndex) const override { return Components.IsValidIndex(InPrimitiveIndex) && Components[InPrimitiveIndex].IsValid(); }
	virtual int32 GetNumInstances(int32 InPrimitiveIndex, int32 InCellID) const override;
	virtual int32 GetNumInstancesTotal(int32 InPrimitiveIndex) const override;
	virtual bool IsRenderStateCreated() const override;
	virtual bool IsAnyRenderStateDirty() const override;
	//~ End IPCGRuntimePrimitiveFactory interface

	struct FParameters
	{
		TArray<FPCGSoftSkinnedMeshComponentDescriptor> Descriptors;
		AActor* TargetActor = nullptr;
	};

	virtual void Initialize(FParameters&& InParameters);
	virtual bool Create(FPCGContext* InContext);
	virtual FBox GetMeshBounds(int32 InPrimitiveIndex) const;

protected:
	TArray<FPCGSoftSkinnedMeshComponentDescriptor> Descriptors;
	AActor* TargetActor = nullptr;

	int32 NumPrimitivesProcessed = 0;

	TArray<TWeakObjectPtr<UPrimitiveComponent>> Components;
	TArray<int32> NumInstances;
	TArray<FBox> MeshBounds;
};
