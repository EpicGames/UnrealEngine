// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PrimitiveFactories/PCGPrimitiveFactoryISMBase.h"

namespace PCGPrimitiveFactoryHelpers
{
	static TFunction<TSharedPtr<IPCGPrimitiveFactoryISMBase>()> FastGeoPrimitiveFactoryGetter;

	TSharedPtr<IPCGPrimitiveFactoryISMBase> GetFastGeoPrimitiveFactory()
	{
		return FastGeoPrimitiveFactoryGetter ? FastGeoPrimitiveFactoryGetter() : nullptr;
	}

	namespace Private
	{
		void SetupFastGeoPrimitiveFactory(TFunction<TSharedPtr<IPCGPrimitiveFactoryISMBase>()>&& Getter)
		{
			ensure(!FastGeoPrimitiveFactoryGetter); // Trivial set and reset patterns expected.
			FastGeoPrimitiveFactoryGetter = MoveTemp(Getter);
		}

		void ResetFastGeoPrimitiveFactory()
		{
			ensure(FastGeoPrimitiveFactoryGetter); // Trivial set and reset patterns expected.
			FastGeoPrimitiveFactoryGetter.Reset();
		}
	}
}

void IPCGPrimitiveFactoryISMBase::PopulateHashes(const FPCGPrimitiveInfo& InPrimitiveInfo, const FBox& InExecutionBounds, TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem>& OutHashes, FBox& OutBounds) const
{
	if (InPrimitiveInfo.InstanceRanges.IsEmpty() || InPrimitiveInfo.CullingCellExtent == 0.0)
	{
		return;
	}

	OutHashes.Reserve(InPrimitiveInfo.InstanceRanges.Num());

	// Minus 1 because CalcLevel returns the level that fully contains this extent, and will return the next level up if a power of 2 cell size is given (which is always
	// the case for us).
	const int32 Level = RenderingSpatialHash::CalcLevel(InPrimitiveInfo.CullingCellExtent - 1.0f);
	RenderingSpatialHash::FLocation64 BaseLocation = RenderingSpatialHash::ToCellLoc(Level, InExecutionBounds.Min);

	for (int32 Index = 0; Index < InPrimitiveInfo.InstanceRanges.Num(); ++Index)
	{
		const int32 NumInstances = InPrimitiveInfo.InstanceRanges[Index].GetNumInstances();
		if (NumInstances > 0)
		{
			FInstanceSceneDataBuffers::FCompressedSpatialHashItem& Hash = OutHashes.Emplace_GetRef();
			Hash.NumInstances = NumInstances;
			Hash.Location = BaseLocation;

			const uint32 SubdivIndex = InPrimitiveInfo.InstanceRanges[Index].GetCellID();
			check(InPrimitiveInfo.NumCullingCells.X > 0 && InPrimitiveInfo.NumCullingCells.Y > 0 && InPrimitiveInfo.NumCullingCells.Z > 0);
			Hash.Location.Coord.X += SubdivIndex % InPrimitiveInfo.NumCullingCells.X;
			Hash.Location.Coord.Y += (SubdivIndex / InPrimitiveInfo.NumCullingCells.X) % InPrimitiveInfo.NumCullingCells.Y;
			Hash.Location.Coord.Z += SubdivIndex / (InPrimitiveInfo.NumCullingCells.X * InPrimitiveInfo.NumCullingCells.Y);

			const FVector3d CellBottomCorner = RenderingSpatialHash::CalcWorldPosition(Hash.Location);
			OutBounds += FBox(CellBottomCorner, CellBottomCorner + FVector3d(InPrimitiveInfo.CullingCellExtent));

			if (InPrimitiveInfo.CullingCellWorldBounds.IsValidIndex(Index))
			{
				const FBox& WorldBounds = InPrimitiveInfo.CullingCellWorldBounds[Index];
				if (WorldBounds.IsValid)
				{
					// Renderer expects ExplicitBounds relative to the cell center (it does CullCenter += ExplicitBounds.GetCenter()).
					const FVector3d CellCenter = RenderingSpatialHash::CalcWorldCellCenter(Hash.Location);
					Hash.ExplicitBounds = FRenderBounds(FBox(WorldBounds.Min - CellCenter, WorldBounds.Max - CellCenter));
				}
			}
		}
	}
}
