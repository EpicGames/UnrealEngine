// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PrimitiveFactories/PCGPrimitiveFactoryPISMC.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "Components/PCGProceduralISMComponent.h"
#include "Compute/PCGComputeCommon.h"

#include "PrimitiveSceneInfo.h"
#include "Engine/StaticMesh.h"
#include "Components/PrimitiveComponent.h"

FPrimitiveSceneProxy* FPCGPrimitiveFactoryPISMC::GetSceneProxy(int32 InPrimitiveIndex) const
{
	if (ensure(Components.IsValidIndex(InPrimitiveIndex)))
	{
		return Components[InPrimitiveIndex].IsValid() ? Components[InPrimitiveIndex]->GetSceneProxy() : nullptr;
	}
	else
	{
		return nullptr;
	}
}

int32 FPCGPrimitiveFactoryPISMC::GetNumInstances(int32 InPrimitiveIndex, int32 InCellID) const
{
	if (ensure(PrimitiveInfos.IsValidIndex(InPrimitiveIndex)))
	{
		for (const FPCGInstanceRange& InstanceRange : PrimitiveInfos[InPrimitiveIndex].InstanceRanges)
		{
			if (InstanceRange.GetCellID() == InCellID)
			{
				return InstanceRange.GetNumInstances();
			}
		}
	}

	return 0;
}

int32 FPCGPrimitiveFactoryPISMC::GetNumInstancesTotal(int32 InPrimitiveIndex) const
{
	int32 NumInstances = 0;

	if (ensure(PrimitiveInfos.IsValidIndex(InPrimitiveIndex)))
	{
		for (const FPCGInstanceRange& PrimitiveInfo : PrimitiveInfos[InPrimitiveIndex].InstanceRanges)
		{
			NumInstances += PrimitiveInfo.GetNumInstances();
		}
	}

	return NumInstances;
}

bool FPCGPrimitiveFactoryPISMC::IsRenderStateCreated() const
{
	for (int32 Index = 0; Index < GetNumPrimitives(); ++Index)
	{
		FPrimitiveSceneProxy* SceneProxy = GetSceneProxy(Index);
		if (!SceneProxy || !SceneProxy->GetPrimitiveSceneInfo() || SceneProxy->GetPrimitiveSceneInfo()->GetInstanceSceneDataOffset() == -1)
		{
			return false;
		}
	}

	return true;
}

bool FPCGPrimitiveFactoryPISMC::IsAnyRenderStateDirty() const
{
	for (const TWeakObjectPtr<UPrimitiveComponent>& WeakComp : Components)
	{
		const UPrimitiveComponent* Comp = WeakComp.Get();
		if (Comp && Comp->IsRenderStateDirty())
		{
			return true;
		}
	}

	return false;
}

void FPCGPrimitiveFactoryPISMC::Initialize(FParameters&& InParameters)
{
	PrimitiveInfos = MoveTemp(InParameters.PrimitiveInfos);
	CustomPrimitiveData = MoveTemp(InParameters.CustomPrimitiveData);
	TargetActor = InParameters.TargetActor;
}

bool FPCGPrimitiveFactoryPISMC::Create(FPCGContext* InContext)
{
	UPCGComponent* SourceComponent = Cast<UPCGComponent>(InContext->ExecutionSource.Get());
	if (!SourceComponent)
	{
		if (InContext->ExecutionSource.IsValid())
		{
			UE_LOGF(LogPCG, Error, "FPCGPrimitiveFactoryPISMC: This primitive factory currently requires a PCG component execution source.");
		}

		return true;
	}

	const int32 NumToCreate = FMath::Min(PrimitiveInfos.Num(), PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER);

	while (NumPrimitivesProcessed < NumToCreate)
	{
		FPCGPrimitiveInfo& PrimitiveInfo = PrimitiveInfos[NumPrimitivesProcessed];
		FPCGProceduralISMComponentDescriptor& Desc = PrimitiveInfo.Descriptor;

		TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem> Hashes;
		FBox BoundsOfAllCells(EForceInit::ForceInit);
		
		PopulateHashes(PrimitiveInfo, SourceComponent->GetExecutionState().GetBounds(), Hashes, BoundsOfAllCells);
		
		if (BoundsOfAllCells.IsValid)
		{
			Desc.WorldBounds = Desc.WorldBounds.Overlap(BoundsOfAllCells);
		}

		const int32 PrimNumInstances = Desc.NumInstances;
		FBox BoundingBox = Desc.StaticMesh.IsValid() ? Desc.StaticMesh->GetBoundingBox() : FBox{};

		FPCGProceduralISMCBuilderParameters Params =
		{
			.Descriptor = MoveTemp(Desc),
			.CustomPrimitiveData = CustomPrimitiveData.GetViewForIndex(NumPrimitivesProcessed),
			.bAllowDescriptorChanges = false,
		};
		
		UPCGManagedProceduralISMComponent* ManagedComponent = PCGManagedProceduralISMComponent::GetOrCreateManagedProceduralISMC(TargetActor, SourceComponent, /*InSettingsUID=*/0, MoveTemp(Params));

		if (ManagedComponent)
		{
			UPCGProceduralISMComponent* PISMC = ManagedComponent->GetComponent();
			PISMC->SetSpatialHashes(MoveTemp(Hashes));

			Components.Add(PISMC);
			MeshBounds.Add(MoveTemp(BoundingBox));
		}

		++NumPrimitivesProcessed;

		if (NumPrimitivesProcessed < NumToCreate && InContext->AsyncState.ShouldStop())
		{
			return false;
		}
	}

	return NumPrimitivesProcessed == NumToCreate;
}

FBox FPCGPrimitiveFactoryPISMC::GetMeshBounds(int32 InPrimitiveIndex) const
{
	return ensure(MeshBounds.IsValidIndex(InPrimitiveIndex)) ? MeshBounds[InPrimitiveIndex] : FBox();
}
