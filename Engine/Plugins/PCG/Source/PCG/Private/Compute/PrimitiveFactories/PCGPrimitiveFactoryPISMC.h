// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PrimitiveFactories/PCGPrimitiveFactoryISMBase.h"

#include "UObject/WeakObjectPtr.h"

class AActor;
class UPrimitiveComponent;
struct FPCGContext;
struct FPCGProceduralISMComponentDescriptor;

class FPCGPrimitiveFactoryPISMC : public IPCGPrimitiveFactoryISMBase
{
public:
	//~ Begin IPCGRuntimePrimitiveFactory interface
	virtual int32 GetNumPrimitives() const override { return Components.Num(); }
	virtual FPrimitiveSceneProxy* GetSceneProxy(int32 InPrimitiveIndex) const override;
	virtual bool IsPrimitiveValid(int32 InPrimitiveIndex) const override { return Components.IsValidIndex(InPrimitiveIndex) && Components[InPrimitiveIndex].IsValid(); }
	virtual int32 GetNumInstances(int32 InPrimitiveIndex, int32 InCellID) const override;
	virtual int32 GetNumInstancesTotal(int32 InPrimitiveIndex) const override;
	virtual bool IsRenderStateCreated() const override;
	virtual bool IsAnyRenderStateDirty() const override;
	//~ End IPCGRuntimePrimitiveFactory interface

	//~ Begin IPCGPrimitiveFactoryISMBase interface
	virtual void Initialize(FParameters&& InParameters) override;
	virtual bool Create(FPCGContext* InContext) override;
	virtual FBox GetMeshBounds(int32 InPrimitiveIndex) const override;
	//~ End IPCGPrimitiveFactoryISMBase interface

protected:
	TArray<FPCGPrimitiveInfo> PrimitiveInfos;
	FPCGPackedCustomData CustomPrimitiveData;
	AActor* TargetActor = nullptr;

	int32 NumPrimitivesProcessed = 0;

	TArray<TWeakObjectPtr<UPrimitiveComponent>> Components;
	TArray<FBox> MeshBounds;
};
