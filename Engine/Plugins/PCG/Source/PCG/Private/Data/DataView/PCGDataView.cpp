// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/DataView/PCGDataView.h"

#include "Data/DataView/PCGDataViewInterface.h"
#include "Data/DataView/PCGDataViewNativePropertySelectors.h"

namespace PCGDataView::Constants
{
	const FGuid FDataViewVersion::GUID = FGuid(0x04E74488, 0x4BAC8717, 0xBBB18694, 0x39F8F3CF);
	FCustomVersionRegistration GRegisterPCGDataViewCustomVersion(FDataViewVersion::GUID, FDataViewVersion::LatestVersion, FDataViewVersion::FriendlyName);
}

bool FPCGDataView::IsValid() const
{
	return ViewedData && (Selection.bAllAttributes || !Selection.Attributes.IsEmpty());
}

TArray<FPCGMetadataDomainID> FPCGDataView::GetAllMetadataDomainIDs() const
{
	if (const UPCGData* Data = Cast<UPCGData>(ViewedData))
	{
		if (Selection.bLimitByDomain)
		{
			const FPCGAttributePropertySelector DummySelector = FPCGAttributePropertySelector::CreateAttributeSelector(FName{}, Selection.Domain);
			FPCGMetadataDomainID DomainID = Data->GetMetadataDomainIDFromSelector(DummySelector);
			if (Data->IsSupportedMetadataDomainID(DomainID))
			{
				return {DomainID};
			}
		}
		else if (Selection.bAllAttributes)
		{
			return Data->GetAllSupportedMetadataDomainIDs();
		}
		else
		{
			TArray<FPCGMetadataDomainID> SupportedDomains;
			for (const FPCGAttributePropertySelector& Selector : Selection.Attributes)
			{
				if (FPCGMetadataDomainID DomainID = Data->GetMetadataDomainIDFromSelector(Selector); DomainID.IsValid())
				{
					// Translate default domain ID to actual domain ID.
					SupportedDomains.AddUnique(DomainID.IsDefault() ? Data->GetDefaultMetadataDomainID() : DomainID);
				}
			}

			return SupportedDomains;
		}
	}

	return {};
}

TArray<FPCGAttributePropertySelector> FPCGDataView::GetResolvedSelection() const
{
	if (Selection.bAllAttributes)
	{
		TArray<FPCGAttributePropertySelector> SelectedAttributes;

		if (!Selection.bIgnoreProperties)
		{
			// Collect all properties defined by the data type
			const FPCGDataViewRegistry& DataViewRegistry = FPCGModule::GetConstPCGDataViewRegistry();
			if (const IPCGDataViewPropertySelector* PropertySelection = DataViewRegistry.GetPropertySelector(*this))
			{
				SelectedAttributes.Append(PropertySelection->GetSelection(*this));
			}
		}

		TOptional<FPCGMetadataDomainID> OptionalDomain;

		if (Selection.bLimitByDomain)
		{
			const FPCGAttributePropertySelector DummySelector = FPCGAttributePropertySelector::CreateAttributeSelector(FName{}, Selection.Domain);
			OptionalDomain = ViewedData->GetMetadataDomainIDFromSelector(DummySelector);

			SelectedAttributes.RemoveAll([this, &OptionalDomain](const FPCGAttributePropertySelector& InSelector)
			{
				const FPCGMetadataDomainID Domain = ViewedData->GetMetadataDomainIDFromSelector(InSelector);
				return !OptionalDomain || Domain != OptionalDomain.GetValue();
			});
		}

		PCGDataView::Helpers::AppendAllAttributeSelectors(ViewedData, SelectedAttributes, OptionalDomain);

		return SelectedAttributes;
	}

	return Selection.Attributes;
}
