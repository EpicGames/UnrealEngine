// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/DataView/PCGDataViewCSVConverter.h"

#include "Data/DataView/PCGDataView.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#define LOCTEXT_NAMESPACE "PCGDataViewCSVConverter"

namespace PCGDataViewCSVConverter::Helpers
{
	TValueOrError<FString, FText> DataViewToString(const FPCGDataView& InDataView, const FInstancedStruct& Parameters)
	{
		if (!InDataView.IsValid() || !Parameters.IsValid())
		{
			return MakeError(LOCTEXT("SerializeStructInvalidData", "Couldn't serialize to target struct: Data View or Parameters were invalid."));
		}

		if (!ensure(Parameters.GetScriptStruct() == FPCGDataViewCSVParameters::StaticStruct()))
		{
			return MakeError(LOCTEXT("ParamStructTypesMismatch", "Struct types between parameters and FPCGDataViewCSVParameters."));
		}

		const TArray<FPCGAttributePropertySelector> Selection = InDataView.GetResolvedSelection();
		TStringBuilder<128> StringBuilder;

		const FPCGDataViewCSVParameters& CSVParams = Parameters.Get<FPCGDataViewCSVParameters>();
		if (!Selection.IsEmpty())
		{
			const UPCGData* ViewedData = InDataView.ViewedData;

			if (!ensure(ViewedData))
			{
				return MakeError(LOCTEXT("InvalidViewedData", "Viewed data was invalid"));
			}

			auto AddAttributeToCSV = [&StringBuilder, &CSVParams, ViewedData](const FPCGAttributePropertyInputSelector& InSelector)
			{
				if (CSVParams.Output == EPCGDataViewCSVOutput::Name || CSVParams.Output == EPCGDataViewCSVOutput::Both)
				{
					StringBuilder += InSelector.ToString(CSVParams.bOmitDomain);
				}

				if (CSVParams.Output == EPCGDataViewCSVOutput::Both)
				{
					StringBuilder += CSVParams.NameValueSeparator;
				}

				if (CSVParams.Output == EPCGDataViewCSVOutput::Value || CSVParams.Output == EPCGDataViewCSVOutput::Both)
				{
					TArray<FString> Values;
					PCGAttributeAccessorHelpers::ExtractAllValues(ViewedData, InSelector, Values, nullptr, EPCGAttributeAccessorFlags::AllowBroadcast);
					StringBuilder.Join(Values, *CSVParams.ValueDelimiter);
				}

				StringBuilder += CSVParams.AttributeDelimiter;
			};

			FPCGMetadataDomainID DefaultDomainID = InDataView.ViewedData->GetDefaultMetadataDomainID();
			for (FPCGMetadataDomainID SupportedDomainID : ViewedData->GetAllSupportedMetadataDomainIDs())
			{
				const bool bIsDefaultDomain = DefaultDomainID == SupportedDomainID;
				const TArray<FPCGAttributePropertySelector> DomainAttributes = Selection.FilterByPredicate([ViewedData, &SupportedDomainID, bIsDefaultDomain](const FPCGAttributePropertySelector& Selector)
				{
					const FPCGMetadataDomainID DomainID = ViewedData->GetMetadataDomainIDFromSelector(Selector);
					return (DomainID == SupportedDomainID) || (bIsDefaultDomain && DomainID == PCGMetadataDomainID::Default);
				});

				for (const FPCGAttributePropertySelector& DomainSelector : DomainAttributes)
				{
					const FPCGAttributePropertyInputSelector Selector = FPCGAttributePropertyInputSelector::CreateFromOtherSelector<FPCGAttributePropertyInputSelector>(DomainSelector);
					AddAttributeToCSV(Selector);
				}
			}
		}

		// Remove the extra delimiter, which should always exist
		if (StringBuilder.Len() > 0 && !CSVParams.AttributeDelimiter.IsEmpty())
		{
			StringBuilder.RemoveSuffix(CSVParams.AttributeDelimiter.Len());
		}

		return MakeValue(StringBuilder.ToString());
	}
} // namespace PCGDataViewCSVConverter::Helpers

TValueOrError<FInstancedStruct, FText> UPCGDataViewCSVConverter::SerializeToTargetStruct(const FPCGDataView& InDataView, const FInstancedStruct& Parameters) const
{
	TValueOrError<FString, FText> Result = PCGDataViewCSVConverter::Helpers::DataViewToString(InDataView, Parameters);
	if (Result.HasError())
	{
		return MakeError(Result.StealError());
	}

	FInstancedStruct InstancedStruct;
	InstancedStruct.InitializeAs<FPCGDataViewCSVOutput>(Result.StealValue());
	return MakeValue(MoveTemp(InstancedStruct));
}

TValueOrError<FString, FText> UPCGDataViewCSVConverter::SerializeToString(const FPCGDataView& InDataView, const FInstancedStruct& Parameters) const
{
	return PCGDataViewCSVConverter::Helpers::DataViewToString(InDataView, Parameters);
}

TValueOrError<UPCGData*, FText> UPCGDataViewCSVConverter::ConstructDataFromTarget(const FInstancedStruct& InTargetStruct, UObject* Outer) const
{
	/** @todo_pcg: Construct a PCG Data from a CSV string. Revisit serialization options. */
	return MakeError(LOCTEXT("DataConstructionNotYetImplemented", "Data construction from CSV not yet implemented."));
}

#undef LOCTEXT_NAMESPACE
