// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/DataView/PCGDataViewData.h"

#include "PCGParamData.h"
#include "Data/PCGSplineData.h"
#include "Elements/PCGConvertToDataView.h"

#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataViewData)

#define LOCTEXT_NAMESPACE "PCGDataViewData"

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoDataView, UPCGDataViewData)

bool FPCGDataTypeInfoDataView::SupportsConversionFrom(const FPCGDataTypeIdentifier& InputType, const FPCGDataTypeIdentifier& ThisType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const
{
	// @todo_pcg: Start with supporting Spatial and Param, but can be expanded to any PCG Data type.
	if (InputType.IsChildOf(FPCGDataTypeInfoSpatial::AsId()) ||
		InputType.IsSameType(FPCGDataTypeInfoParam::AsId()))
	{
		if (OptionalOutConversionSettings)
		{
			*OptionalOutConversionSettings = UPCGConvertToDataViewSettings::StaticClass();
		}

		return true;
	}
	else
	{
		if (OptionalOutCompatibilityMessage)
		{
			*OptionalOutCompatibilityMessage = FText::Format(LOCTEXT("TypeCompatibilityNotSupported", "The '{0}' type is not yet supported by Data Views."), InputType.ToDisplayText());
		}

		return false;
	}
}

void UPCGDataViewData::Initialize(FPCGDataView&& InDataView)
{
	DataView = InDataView;
}

const FPCGDataView& UPCGDataViewData::GetDataView() const
{
	return DataView;
}

FPCGMetadataDomainID UPCGDataViewData::GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const
{
	if (!ensure(DataView.ViewedData))
	{
		return {};
	}

	return DataView.ViewedData->GetMetadataDomainIDFromSelector(InSelector);
}

FPCGMetadataDomainID UPCGDataViewData::GetDefaultMetadataDomainID() const
{
	if (!ensure(DataView.ViewedData))
	{
		return PCGMetadataDomainID::Default;
	}

	TArray<FPCGMetadataDomainID> SupportedDomainIDs = GetAllSupportedMetadataDomainIDs();
	if (SupportedDomainIDs.IsEmpty())
	{
		return {};
	}

	const FPCGMetadataDomainID DefaultDomainID = DataView.ViewedData->GetDefaultMetadataDomainID();

	FPCGMetadataDomainID DomainID{};
	DomainID = SupportedDomainIDs.Contains(DefaultDomainID) ? DefaultDomainID : SupportedDomainIDs[0];

	return DomainID;
}

TArray<FPCGMetadataDomainID> UPCGDataViewData::GetAllSupportedMetadataDomainIDs() const
{
	if (!ensure(DataView.ViewedData))
	{
		return {};
	}

	if (DataView.Selection.bAllAttributes)
	{
		return DataView.ViewedData->GetAllSupportedMetadataDomainIDs();
	}
	else
	{
		TArray<FPCGMetadataDomainID> SupportedDomains;
		for (const FPCGAttributePropertySelector& Selector : DataView.Selection.Attributes)
		{
			if (FPCGMetadataDomainID DomainID = DataView.ViewedData->GetMetadataDomainIDFromSelector(Selector); DomainID.IsValid())
			{
				SupportedDomains.AddUnique(DomainID.IsDefault() ? DataView.ViewedData->GetDefaultMetadataDomainID() : DomainID);
			}
		}

		return SupportedDomains;
	}
}

const UPCGMetadata* UPCGDataViewData::ConstMetadata() const
{
	if (!ensure(DataView.ViewedData))
	{
		return nullptr;
	}

	return DataView.ViewedData->ConstMetadata();
}

FPCGCrc UPCGDataViewData::ComputeCrc(bool bFullDataCrc) const
{
	FArchiveCrc32 Ar;

	AddToCrc(Ar, bFullDataCrc);

	if (DataView.ViewedData)
	{
		Ar << DataView.ViewedData->GetOrComputeCrc(bFullDataCrc);
	}

	return FPCGCrc(Ar.GetCrc());
}

void UPCGDataViewData::AddToCrc(FArchiveCrc32& Ar, const bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	if (!bFullDataCrc)
	{
		// Fallback to UID
		AddUIDToCrc(Ar);
		return;
	}

	uint8 AllAttributesByte = DataView.Selection.bAllAttributes ? 1 : 0;
	Ar << AllAttributesByte;

	for (FPCGAttributePropertySelector Selector : DataView.Selection.Attributes)
	{
		Ar << Selector;
	}

	// @todo_pcg: Add bLimitByDomain, Domain, and bIgnoreProperties when they are live.
}

TArray<FPCGAttributePropertySelector> UPCGDataViewData::GetSelectedAttributes() const
{
	if (!ensure(DataView.ViewedData))
	{
		return {};
	}

	return DataView.GetResolvedSelection();
}

#undef LOCTEXT_NAMESPACE
