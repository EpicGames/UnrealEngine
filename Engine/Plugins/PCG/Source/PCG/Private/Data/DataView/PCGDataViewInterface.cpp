// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/DataView/PCGDataViewInterface.h"

#include "Misc/FileHelper.h"

#define LOCTEXT_NAMESPACE "PCGDataViewInterface"

TValueOrError<void, FText> UPCGDataViewConverterBase::SerializeToFile(const FPCGDataView& InDataView, const FInstancedStruct& Parameters, const FString& FilePath) const
{
	TValueOrError<FString, FText> Result = SerializeToString(InDataView, Parameters);

	if (Result.HasError())
	{
		return MakeError(Result.StealError());
	}

	if (!FFileHelper::SaveStringToFile(Result.GetValue(), *FilePath))
	{
		FText ErrorText = FText::Format(LOCTEXT("InvalidFilePath", "Could not create a valid file with path: {0}"), FText::FromStringView(FilePath));
		return MakeError(MoveTemp(ErrorText));
	}

	return MakeValue();
}

#undef LOCTEXT_NAMESPACE
