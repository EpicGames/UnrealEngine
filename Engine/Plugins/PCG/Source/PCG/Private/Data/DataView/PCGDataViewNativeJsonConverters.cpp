// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/DataView/PCGDataViewNativeJsonConverters.h"

#include "Data/DataView/PCGDataView.h"
#include "Data/DataView/PCGDataViewNativePropertySelectors.h"

#define LOCTEXT_NAMESPACE "PCGDataViewNativeJsonConverters"

TValueOrError<FInstancedStruct, FText> UPCGDataViewJsonConverter::SerializeToTargetStruct(const FPCGDataView& InDataView, const FInstancedStruct& Parameters) const
{
	using namespace PCG::IO;
	if (!InDataView.IsValid() || !Parameters.IsValid())
	{
		return MakeError(LOCTEXT("SerializeStructInvalidData", "Couldn't serialize to target struct: Data View or Parameters were invalid."));
	}

	if (!ensure(Parameters.GetScriptStruct() == FPCGDataViewJsonParameters::StaticStruct()))
	{
		return MakeError(LOCTEXT("ParametersTypeMismatch", "Mismatch between Parameters type and FPCGDataViewJsonParameters."));
	}

	TSharedPtr<FJsonObject> RootJsonObject = MakeShared<FJsonObject>();

	// Overridable
	BuildJsonHeader(InDataView, RootJsonObject);

	const TArray<FPCGAttributePropertySelector> Selection = InDataView.GetResolvedSelection();
	if (!Selection.IsEmpty())
	{
		const UPCGData* ViewedData = InDataView.ViewedData;

		if (!ensure(ViewedData))
		{
			return MakeError(LOCTEXT("InvalidViewedData", "Viewed data was invalid"));
		}

		const FPCGDataViewJsonParameters& JsonParameters = Parameters.Get<FPCGDataViewJsonParameters>();

		const FPCGMetadataDomainID DefaultDomainID = ViewedData->GetDefaultMetadataDomainID();
		auto GetDomainAttributes = [&Selection, ViewedData, DefaultDomainID, Layout = JsonParameters.AttributeLayout, &RootJsonObject](FPCGMetadataDomainID DomainID)
		{
			const bool bIsDefaultDomain = (DefaultDomainID == DomainID);
			const TArray<FPCGAttributePropertySelector> DomainAttributes = Selection.FilterByPredicate([ViewedData, &DomainID, bIsDefaultDomain, &RootJsonObject](const FPCGAttributePropertySelector& Selector)
			{
				const FPCGMetadataDomainID SelectorDomainID = ViewedData->GetMetadataDomainIDFromSelector(Selector);
				return (SelectorDomainID == DomainID) || (bIsDefaultDomain && SelectorDomainID.IsDefault());
			});

			if (DomainAttributes.IsEmpty())
			{
				return;
			}

			if (Layout == EPCGDataViewAttributeLayout::ByElement)
			{
				Json::Helpers::AppendSelectionByElement(RootJsonObject, ViewedData, DomainAttributes);
			}
			else if (Layout == EPCGDataViewAttributeLayout::ByAttribute)
			{
				Json::Helpers::AppendSelectionByAttribute(RootJsonObject, ViewedData, DomainAttributes);
			}
		};

		for (const FPCGMetadataDomainID DomainID : ViewedData->GetAllSupportedMetadataDomainIDs())
		{
			GetDomainAttributes(DomainID);
		}
	}

	FInstancedStruct InstancedStruct;
	InstancedStruct.InitializeAs<FPCGDataViewJsonOutput>(MoveTemp(RootJsonObject));
	return MakeValue(InstancedStruct);
}

TValueOrError<FString, FText> UPCGDataViewJsonConverter::SerializeToString(const FPCGDataView& InDataView, const FInstancedStruct& Parameters) const
{
	TValueOrError<FInstancedStruct, FText> Result = SerializeToTargetStruct(InDataView, Parameters);
	if (Result.HasError())
	{
		return MakeError(Result.StealError());
	}

	const FInstancedStruct DataStruct = Result.GetValue();
	if (!ensure(DataStruct.IsValid() && DataStruct.GetScriptStruct() == FPCGDataViewJsonOutput::StaticStruct()))
	{
		return MakeError(LOCTEXT("DataStructTypesMismatch", "Struct types between result and FPCGDataViewJsonOutput."));
	}

	check(Parameters.IsValid()); // already validated in SerializeToTargetStruct call.
	const FPCGDataViewJsonOutput& JsonData = DataStruct.Get<FPCGDataViewJsonOutput>();
	const FPCGDataViewJsonParameters& JsonParams = Parameters.Get<FPCGDataViewJsonParameters>();
	return MakeValue(PCG::IO::Json::Helpers::ToJsonString(JsonData.JsonObject, JsonParams.bPrettyJson));
}

TValueOrError<UPCGData*, FText> UPCGDataViewJsonConverter::ConstructDataFromTarget(const FInstancedStruct& InTargetStruct, UObject* Outer) const
{
	/** @todo_pcg: Construct a PCG Data from a Json object. Revisit serialization options.
	 * Property Bag Option: Serialize the bag description first as a schema and the payload as a Json string
	 * Use the bag description to construct the property bag, and then de-serialize Json->UObject on the payload string
	 */
	return MakeError(LOCTEXT("DataConstructionNotYetImplemented", "Data construction from Json not yet implemented."));
}

void UPCGDataViewJsonConverter::BuildJsonHeader(const FPCGDataView& InDataView, TSharedPtr<FJsonObject>& InOutJsonObject) const
{
	using namespace PCG::IO;
	// Append data header. Add it directly to the root object
	Json::Helpers::AppendVersionToHeader(InOutJsonObject, PCGDataView::Constants::FDataViewVersion::LatestVersion);
}

#undef LOCTEXT_NAMESPACE
