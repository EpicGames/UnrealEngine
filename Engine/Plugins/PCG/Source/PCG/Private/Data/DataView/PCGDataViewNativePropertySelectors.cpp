// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/DataView/PCGDataViewNativePropertySelectors.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGSplineStruct.h"
#include "Data/DataView/PCGDataView.h"

namespace PCGDataView
{
	namespace Constants
	{
		/** @todo_pcg: It would be ideal if each data could handle this individually, but must also account for
		 * the distinction of only adding non-overlapping properties if there are redundancies. For now, pick
		 * by hand.
		 */
		// A selection of all PCG Point properties that avoid redundancies, i.e. $Transform and $Position.
		static constexpr EPCGPointProperties AllPointProperties[]
		{
			EPCGPointProperties::Transform,
			EPCGPointProperties::Density,
			EPCGPointProperties::BoundsMin,
			EPCGPointProperties::BoundsMax,
			EPCGPointProperties::Color,
			EPCGPointProperties::Steepness,
			EPCGPointProperties::Seed
		};

		// A selection of all Spline properties.
		static constexpr EPCGSplineStructProperties AllSplineProperties[]
		{
			EPCGSplineStructProperties::Transform,
			EPCGSplineStructProperties::ArriveTangent,
			EPCGSplineStructProperties::LeaveTangent,
			EPCGSplineStructProperties::InterpType,
			EPCGSplineStructProperties::LocalTransform
		};
	} // namespace Constants

	namespace Helpers
	{
		void AppendAllAttributeSelectors(const UPCGData* InData, TArray<FPCGAttributePropertySelector>& InOutSelectors, TOptional<FPCGMetadataDomainID> OptionalDomain)
		{
			check(InData);
			check(InData->ConstMetadata());

			TArray<FPCGAttributeIdentifier> AttributeIDs;
			TArray<EPCGMetadataTypes> AttributeTypes;
			InData->ConstMetadata()->GetAllAttributes(AttributeIDs, AttributeTypes);

			for (const FPCGAttributeIdentifier& AttributeID : AttributeIDs)
			{
				FPCGMetadataDomainID MetadataDomain = AttributeID.MetadataDomain.IsDefault() ? InData->GetDefaultMetadataDomainID() : AttributeID.MetadataDomain;

				// Add the selectors if no optional domain is provided, or if provided, it matches the attribute's domain.
				if (!OptionalDomain.IsSet() || OptionalDomain.GetValue() == MetadataDomain)
				{
					FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(AttributeID.Name);
					InData->SetDomainFromDomainID(MetadataDomain, Selector);
					check(!MetadataDomain.IsDefault())

					InOutSelectors.Add(Selector);
				}
			}
		}

		void AppendAllPointPropertySelectors(TArray<FPCGAttributePropertySelector>& InOutSelectors)
		{
			for (const EPCGPointProperties Property : Constants::AllPointProperties)
			{
				InOutSelectors.Add(FPCGAttributePropertySelector::CreatePointPropertySelector(Property, PCGPointDataConstants::ElementsDomainName));
			}
		}

		void AppendAllSplinePropertySelectors(TArray<FPCGAttributePropertySelector>& InOutSelectors)
		{
			if (const UEnum* SplinePropertiesEnum = StaticEnum<EPCGSplineStructProperties>())
			{
				for (const EPCGSplineStructProperties Attribute : Constants::AllSplineProperties)
				{
					const FName DisplayName = FName(SplinePropertiesEnum->GetNameStringByValue(static_cast<int32>(Attribute)));
					InOutSelectors.Add(FPCGAttributePropertySelector::CreatePropertySelector(DisplayName, PCGSplineData::ControlPointDomainName));
				}
			}
		}
	} // namespace Helpers
} // namespace PCGDataView

TArray<FPCGAttributePropertySelector> FPCGDataViewBasePointDataPropertySelector::GetSelection(const FPCGDataView& InDataView) const
{
	TArray<FPCGAttributePropertySelector> Selectors;
	PCGDataView::Helpers::AppendAllPointPropertySelectors(Selectors);
	return Selectors;
}

TArray<FPCGAttributePropertySelector> FPCGDataViewSplinePropertySelector::GetSelection(const FPCGDataView& InDataView) const
{
	TArray<FPCGAttributePropertySelector> Selectors;
	PCGDataView::Helpers::AppendAllSplinePropertySelectors(Selectors);
	return Selectors;
}
