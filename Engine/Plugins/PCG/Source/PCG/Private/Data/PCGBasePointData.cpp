// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGBasePointData.h"

#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGConvertToAttributeSet.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGPointHelpers.h"
#include "Helpers/PCGTagHelpers.h"
#include "Metadata/PCGMetadataAccessor.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include "GameFramework/Actor.h"
#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBasePointData)

#define LOCTEXT_NAMESPACE "PCGBasePointData"

static TAutoConsoleVariable<bool> CVarCacheFullPointDataCrc(
	TEXT("pcg.Cache.FullPointDataCrc"),
	true,
	TEXT("Enable fine-grained CRC of point data for change tracking on elements that request it, rather than using data UID."));

namespace PCGPointDataConstants
{
	const FName ActorReferenceAttribute = TEXT("ActorReference");
	const FName ElementsDomainName = "Points";
}

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoPoint, UPCGBasePointData)

namespace PCGPointHelpers
{
	bool GetDistanceRatios(const FTransform& InTransform, const FVector& InBoundsMin, const FVector& InBoundsMax, const float InSteepness, const FVector& InPosition, FVector& OutRatios)
	{
		FVector LocalPosition = InTransform.InverseTransformPosition(InPosition);
		LocalPosition -= (InBoundsMax + InBoundsMin) / 2;
		LocalPosition /= PCGPointHelpers::GetExtents(InBoundsMin, InBoundsMax);

		// ]-2+s, 2-s] is the valid range of values
		const FVector::FReal LowerBound = InSteepness - 2;
		const FVector::FReal HigherBound = 2 - InSteepness;

		if (LocalPosition.X <= LowerBound || LocalPosition.X > HigherBound ||
			LocalPosition.Y <= LowerBound || LocalPosition.Y > HigherBound ||
			LocalPosition.Z <= LowerBound || LocalPosition.Z > HigherBound)
		{
			return false;
		}

		// [-s, +s] is the range where the density is 1 on that axis
		const FVector::FReal XDist = FMath::Max(0, FMath::Abs(LocalPosition.X) - InSteepness);
		const FVector::FReal YDist = FMath::Max(0, FMath::Abs(LocalPosition.Y) - InSteepness);
		const FVector::FReal ZDist = FMath::Max(0, FMath::Abs(LocalPosition.Z) - InSteepness);

		const FVector::FReal DistanceScale = FMath::Max(2 - 2 * InSteepness, KINDA_SMALL_NUMBER);

		OutRatios.X = XDist / DistanceScale;
		OutRatios.Y = YDist / DistanceScale;
		OutRatios.Z = ZDist / DistanceScale;
		return true;
	}

	FVector::FReal ManhattanDensity(const FTransform& InTransform, const FVector& InBoundsMin, const FVector& InBoundsMax, const float InSteepness, const float InDensity, const FVector& InPosition)
	{
		FVector Ratios;
		if (GetDistanceRatios(InTransform, InBoundsMin, InBoundsMax, InSteepness, InPosition, Ratios))
		{
			return InDensity * (1 - Ratios.X) * (1 - Ratios.Y) * (1 - Ratios.Z);
		}
		else
		{
			return 0;
		}
	}

	FVector::FReal InverseEuclidianDistance(const FTransform& InTransform, const FVector& InBoundsMin, const FVector& InBoundsMax, const float InSteepness, const FVector& InPosition)
	{
		FVector Ratios;
		if (GetDistanceRatios(InTransform, InBoundsMin, InBoundsMax, InSteepness, InPosition, Ratios))
		{
			return 1.0 - Ratios.Length();
		}
		else
		{
			return 0;
		}
	}

	/** Computes reasonable overlap ratio for point, 1d, 2d and volume overlaps, to be used as weights.
	* Note that this assumes that either data set is homogeneous in its points dimension (either 0d, 1d, 2d, 3d)
	* Otherwise there will be some artifacts from our assumption here (namely using a 1.0 value for the additional coordinates).
	*/
	FVector::FReal ComputeOverlapRatio(const FBox& Numerator, const FBox& Denominator)
	{
		const FVector NumeratorExtent = Numerator.GetExtent();
		const FVector DenominatorExtent = Denominator.GetExtent();

		return (FVector::FReal)((DenominatorExtent.X > 0 ? NumeratorExtent.X / DenominatorExtent.X : 1.0) *
			(DenominatorExtent.Y > 0 ? NumeratorExtent.Y / DenominatorExtent.Y : 1.0) *
			(DenominatorExtent.Z > 0 ? NumeratorExtent.Z / DenominatorExtent.Z : 1.0));
	}

	FVector::FReal VolumeOverlap(const FTransform& InTransform, const FVector& InBoundsMin, const FVector& InBoundsMax, const float InSteepness, const FBox& InBounds, const FMatrix& InInverseTransform)
	{
		// This is similar in idea to SAT considering we have two boxes - since we will test all 6 axes.
		// However, there is some uncertainty due to rotation, and using the overlap value as-is is an overestimation, which might not be critical in this case
		// TODO: investigate if we should do a 8-pt test instead (would be more precise, but significantly more costly).
		// Implementation note: we are using FMatrix here because we want to support non-uniform scales
		const FBox PointBounds = PCGPointHelpers::GetLocalDensityBounds(InSteepness, InBoundsMin, InBoundsMax);

		FMatrix PointTransformToInTransform = InTransform.ToMatrixWithScale() * InInverseTransform;
		const FBox PointBoundsTransformed = PointBounds.TransformBy(PointTransformToInTransform);

		const FBox FirstOverlap = InBounds.Overlap(PointBoundsTransformed);
		if (!FirstOverlap.IsValid)
		{
			return 0;
		}

		FMatrix InTransformToPointTransform = PointTransformToInTransform.Inverse();
		const FBox InBoundsTransformed = InBounds.TransformBy(InTransformToPointTransform);

		const FBox SecondOverlap = InBoundsTransformed.Overlap(PointBounds);
		if (!SecondOverlap.IsValid)
		{
			return 0;
		}

		return FMath::Min(ComputeOverlapRatio(FirstOverlap, InBounds), ComputeOverlapRatio(SecondOverlap, InBoundsTransformed));
	}

	/** Helper function for additive blending of quaternions (copied from ControlRig) */
	FQuat AddQuatWithWeight(const FQuat& Q, const FQuat& V, float Weight)
	{
		FQuat BlendQuat = V * Weight;

		if ((Q | BlendQuat) >= 0.0f)
			return Q + BlendQuat;
		else
			return Q - BlendQuat;
	}
}

namespace PCGBasePointData
{
	TUniquePtr<IPCGAttributeAccessor> CreatePropertyAccessor(UPCGBasePointData* PointData, EPCGPointNativeProperties NativeProperty, bool bQuiet)
	{
		switch (NativeProperty)
		{
		case EPCGPointNativeProperties::Transform:
			return MakeUnique<FPCGNativePointPropertyEnumAccessor<EPCGPointNativeProperties::Transform>>(PointData);
		case EPCGPointNativeProperties::Density:
			return MakeUnique<FPCGNativePointPropertyEnumAccessor<EPCGPointNativeProperties::Density>>(PointData);
		case EPCGPointNativeProperties::Steepness:
			return MakeUnique<FPCGNativePointPropertyEnumAccessor<EPCGPointNativeProperties::Steepness>>(PointData);
		case EPCGPointNativeProperties::BoundsMin:
			return MakeUnique<FPCGNativePointPropertyEnumAccessor<EPCGPointNativeProperties::BoundsMin>>(PointData);
		case EPCGPointNativeProperties::BoundsMax:
			return MakeUnique<FPCGNativePointPropertyEnumAccessor<EPCGPointNativeProperties::BoundsMax>>(PointData);
		case EPCGPointNativeProperties::Color:
			return MakeUnique<FPCGNativePointPropertyEnumAccessor<EPCGPointNativeProperties::Color>>(PointData);
		case EPCGPointNativeProperties::Seed:
			return MakeUnique<FPCGNativePointPropertyEnumAccessor<EPCGPointNativeProperties::Seed>>(PointData);
		case EPCGPointNativeProperties::MetadataEntry:
			return MakeUnique<FPCGNativePointPropertyEnumAccessor<EPCGPointNativeProperties::MetadataEntry>>(PointData);
		default:
			UE_LOGF(LogPCG, Error, "EPCGPointNativeProperty value '%d' does not exist.", EnumToUnderlyingType(NativeProperty));
			return TUniquePtr<IPCGAttributeAccessor>();
		}
	}

	TUniquePtr<IPCGAttributeAccessor> CreateConstPropertyAccessor(const UPCGBasePointData* PointData, EPCGPointNativeProperties NativeProperty, bool bQuiet)
	{
		switch (NativeProperty)
		{
		case EPCGPointNativeProperties::Transform:
			return MakeUnique<FPCGNativePointPropertyEnumConstAccessor<EPCGPointNativeProperties::Transform>>(PointData);
		case EPCGPointNativeProperties::Density:
			return MakeUnique<FPCGNativePointPropertyEnumConstAccessor<EPCGPointNativeProperties::Density>>(PointData);
		case EPCGPointNativeProperties::Steepness:
			return MakeUnique<FPCGNativePointPropertyEnumConstAccessor<EPCGPointNativeProperties::Steepness>>(PointData);
		case EPCGPointNativeProperties::BoundsMin:
			return MakeUnique<FPCGNativePointPropertyEnumConstAccessor<EPCGPointNativeProperties::BoundsMin>>(PointData);
		case EPCGPointNativeProperties::BoundsMax:
			return MakeUnique<FPCGNativePointPropertyEnumConstAccessor<EPCGPointNativeProperties::BoundsMax>>(PointData);
		case EPCGPointNativeProperties::Color:
			return MakeUnique<FPCGNativePointPropertyEnumConstAccessor<EPCGPointNativeProperties::Color>>(PointData);
		case EPCGPointNativeProperties::Seed:
			return MakeUnique<FPCGNativePointPropertyEnumConstAccessor<EPCGPointNativeProperties::Seed>>(PointData);
		case EPCGPointNativeProperties::MetadataEntry:
			return MakeUnique<FPCGNativePointPropertyEnumConstAccessor<EPCGPointNativeProperties::MetadataEntry>>(PointData);
		default:
			UE_LOGF(LogPCG, Error, "EPCGPointNativeProperty value '%d' does not exist.", EnumToUnderlyingType(NativeProperty));
			return TUniquePtr<IPCGAttributeAccessor>();
		}
	}

	TUniquePtr<IPCGAttributeAccessor> CreateCustomPropertyAccessor(UPCGBasePointData* PointData, FName Name)
	{
		if (Name == PCGPointCustomPropertyNames::ExtentsName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FVector, FPCGPointBoundsMin::ValueRange, FPCGPointBoundsMax::ValueRange>>(
				PointData,
				[](int32 Index, FVector& OutValue, const FPCGPointBoundsMin::ValueRange& BoundsMin, const FPCGPointBoundsMax::ValueRange& BoundsMax)
				{
					OutValue = PCGPointHelpers::GetExtents(BoundsMin[Index], BoundsMax[Index]);
					return true;
				},
				[](int32 Index, const FVector& InValue, FPCGPointBoundsMin::ValueRange BoundsMin, FPCGPointBoundsMax::ValueRange BoundsMax)
				{
					PCGPointHelpers::SetExtents(InValue, BoundsMin[Index], BoundsMax[Index]);
					return true;
				},
				PointData->GetBoundsMinValueRange(),
				PointData->GetBoundsMaxValueRange()
			);
		}
		else if (Name == PCGPointCustomPropertyNames::LocalCenterName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FVector, FPCGPointBoundsMin::ValueRange, FPCGPointBoundsMax::ValueRange>>(
				PointData,
				[](int32 Index, FVector& OutValue, const FPCGPointBoundsMin::ValueRange& BoundsMin, const FPCGPointBoundsMax::ValueRange& BoundsMax)
				{
					OutValue = PCGPointHelpers::GetLocalCenter(BoundsMin[Index], BoundsMax[Index]);
					return true;
				},
				[](int32 Index, const FVector& InValue, FPCGPointBoundsMin::ValueRange BoundsMin, FPCGPointBoundsMax::ValueRange BoundsMax)
				{
					PCGPointHelpers::SetLocalCenter(InValue, BoundsMin[Index], BoundsMax[Index]);
					return true;
				},
				PointData->GetBoundsMinValueRange(),
				PointData->GetBoundsMaxValueRange()
			);
		}
		else if (Name == PCGPointCustomPropertyNames::PositionName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FVector, FPCGPointTransform::ValueRange>>(
				PointData,
				[](int32 Index, FVector& OutValue, const FPCGPointTransform::ValueRange& Transform)
				{
					OutValue = Transform[Index].GetLocation();
					return true;
				},
				[](int32 Index, const FVector& InValue, FPCGPointTransform::ValueRange Transform)
				{
					Transform[Index].SetLocation(InValue);
					return true;
				},
				PointData->GetTransformValueRange()
			);
		}
		else if (Name == PCGPointCustomPropertyNames::RotationName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FQuat, FPCGPointTransform::ValueRange>>(
				PointData,
				[](int32 Index, FQuat& OutValue, const FPCGPointTransform::ValueRange& Transform)
				{
					OutValue = Transform[Index].GetRotation();
					return true;
				},
				[](int32 Index, const FQuat& InValue, FPCGPointTransform::ValueRange Transform)
				{
					Transform[Index].SetRotation(InValue);
					return true;
				},
				PointData->GetTransformValueRange()
			);
		}
		else if (Name == PCGPointCustomPropertyNames::ScaleName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FVector, FPCGPointTransform::ValueRange>>(
				PointData,
				[](int32 Index, FVector& OutValue, const FPCGPointTransform::ValueRange& Transform)
				{
					OutValue = Transform[Index].GetScale3D();
					return true;
				},
				[](int32 Index, const FVector& InValue, FPCGPointTransform::ValueRange Transform)
				{
					Transform[Index].SetScale3D(InValue);
					return true;
				},
				PointData->GetTransformValueRange()
			);
		}

		return TUniquePtr<IPCGAttributeAccessor>();
	}

	TUniquePtr<IPCGAttributeAccessor> CreateConstCustomPropertyAccessor(const UPCGBasePointData* PointData, FName Name)
	{
		if (Name == PCGPointCustomPropertyNames::ExtentsName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FVector, FPCGPointBoundsMin::ConstValueRange, FPCGPointBoundsMax::ConstValueRange>>(
				PointData,
				[](int32 Index, FVector& OutValue, const FPCGPointBoundsMin::ConstValueRange& BoundsMin, const FPCGPointBoundsMax::ConstValueRange& BoundsMax)
				{
					OutValue = PCGPointHelpers::GetExtents(BoundsMin[Index], BoundsMax[Index]);
					return true;
				},
				PointData->GetConstBoundsMinValueRange(),
				PointData->GetConstBoundsMaxValueRange()
			);
		}
		else if (Name == PCGPointCustomPropertyNames::LocalCenterName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FVector, FPCGPointBoundsMin::ConstValueRange, FPCGPointBoundsMax::ConstValueRange>>(
				PointData,
				[](int32 Index, FVector& OutValue, const FPCGPointBoundsMin::ConstValueRange& BoundsMin, const FPCGPointBoundsMax::ConstValueRange& BoundsMax)
				{
					OutValue = PCGPointHelpers::GetLocalCenter(BoundsMin[Index], BoundsMax[Index]);
					return true;
				},
				PointData->GetConstBoundsMinValueRange(),
				PointData->GetConstBoundsMaxValueRange()
			);
		}
		else if (Name == PCGPointCustomPropertyNames::PositionName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FVector, FPCGPointTransform::ConstValueRange>>(
				PointData,
				[](int32 Index, FVector& OutValue, const FPCGPointTransform::ConstValueRange& Transform)
				{
					OutValue = Transform[Index].GetLocation();
					return true;
				},
				PointData->GetConstTransformValueRange()
			);
		}
		else if (Name == PCGPointCustomPropertyNames::RotationName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FQuat, FPCGPointTransform::ConstValueRange>>(
				PointData,
				[](int32 Index, FQuat& OutValue, const FPCGPointTransform::ConstValueRange& Transform)
				{
					OutValue = Transform[Index].GetRotation();
					return true;
				},
				PointData->GetConstTransformValueRange()
			);
		}
		else if (Name == PCGPointCustomPropertyNames::ScaleName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FVector, FPCGPointTransform::ConstValueRange>>(
				PointData,
				[](int32 Index, FVector& OutValue, const FPCGPointTransform::ConstValueRange& Transform)
				{
					OutValue = Transform[Index].GetScale3D();
					return true;
				},
				PointData->GetConstTransformValueRange()
			);
		}
		else if (Name == PCGPointCustomPropertyNames::LocalSizeName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FVector, FPCGPointBoundsMin::ConstValueRange, FPCGPointBoundsMax::ConstValueRange>>(
				PointData,
				[](int32 Index, FVector& OutValue, const FPCGPointBoundsMin::ConstValueRange& BoundsMin, const FPCGPointBoundsMax::ConstValueRange& BoundsMax)
				{
					OutValue = PCGPointHelpers::GetLocalSize(BoundsMin[Index], BoundsMax[Index]);
					return true;
				},
				PointData->GetConstBoundsMinValueRange(),
				PointData->GetConstBoundsMaxValueRange()
			);
		}
		else if (Name == PCGPointCustomPropertyNames::ScaledLocalSizeName)
		{
			return MakeUnique<FPCGCustomPointPropertyAccessor<FVector, FPCGPointTransform::ConstValueRange, FPCGPointBoundsMin::ConstValueRange, FPCGPointBoundsMax::ConstValueRange>>(
				PointData,
				[](int32 Index, FVector& OutValue, const FPCGPointTransform::ConstValueRange& Transform, const FPCGPointBoundsMin::ConstValueRange& BoundsMin, const FPCGPointBoundsMax::ConstValueRange& BoundsMax)
				{
					OutValue = PCGPointHelpers::GetScaledLocalSize(Transform[Index], BoundsMin[Index], BoundsMax[Index]);
					return true;
				},
				PointData->GetConstTransformValueRange(),
				PointData->GetConstBoundsMinValueRange(),
				PointData->GetConstBoundsMaxValueRange()
			);
		}

		return TUniquePtr<IPCGAttributeAccessor>();
	}

	TUniquePtr<IPCGAttributeAccessor> CreateStaticAccessor(const UPCGBasePointData* PointData, const FPCGAttributePropertySelector& InSelector, bool bQuiet, bool bConst)
	{
		TUniquePtr<IPCGAttributeAccessor> Accessor;

		if (InSelector.GetSelection() == EPCGAttributePropertySelection::Property)
		{
			const FName PropertyName = InSelector.GetPropertyName();

			if (const int64 EnumValue = StaticEnum<EPCGPointNativeProperties>()->GetValueByName(PropertyName); EnumValue != INDEX_NONE)
			{
				const EPCGPointNativeProperties NativeProperty = (EPCGPointNativeProperties)EnumValue;
				if (bConst)
				{
					Accessor = CreateConstPropertyAccessor(PointData, NativeProperty, bQuiet);
				}
				else
				{
					Accessor = CreatePropertyAccessor(const_cast<UPCGBasePointData*>(PointData), NativeProperty, bQuiet);
				}
			}
			else if (PCGPointCustomPropertyNames::IsCustomPropertyName(PropertyName))
			{
				if (bConst)
				{
					Accessor = CreateConstCustomPropertyAccessor(PointData, PropertyName);
				}
				else
				{
					Accessor = CreateCustomPropertyAccessor(const_cast<UPCGBasePointData*>(PointData), PropertyName);
				}
			}
		}
		else if (InSelector.GetSelection() == EPCGAttributePropertySelection::Attribute && !bConst)
		{
			// Let parent factory create the accessor but allocate the metadata entry memory
			const_cast<UPCGBasePointData*>(PointData)->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
		}

		return Accessor;
	}

	template<EPCGPointNativeProperties PointProperty>
	bool IsValidRange(const FPCGPointInputRange& Range, const typename TPCGPointNativeProperty<PointProperty>::ConstValueRange& ValueRange)
	{
		return Range.RangeSize > 0 && ValueRange.IsValidIndex(Range.RangeStartIndex) && ValueRange.IsValidIndex(Range.RangeStartIndex + Range.RangeSize - 1);
	}

	template<EPCGPointNativeProperties PointProperty>
	TArray<typename TPCGPointNativeProperty<PointProperty>::Type> GetInputRangeValues(const FPCGPointInputRange& Range)
	{
		if (!Range.PointData)
		{
			UE_LOGF(LogPCG, Error, "GetInputRangeValues '%ls' called with invalid point data.", *StaticEnum<EPCGPointNativeProperties>()->GetNameStringByValue((int64)PointProperty));
			return {};
		}

		const typename TPCGPointNativeProperty<PointProperty>::ConstValueRange ValueRange = Range.PointData->GetConstValueRange<PointProperty>();

		if (!PCGBasePointData::IsValidRange<PointProperty>(Range, ValueRange))
		{
			UE_LOGF(LogPCG, Error, "GetInputRangeValues '%ls' called with input range [%d, %d[ when valid range is [0, %d].", *StaticEnum<EPCGPointNativeProperties>()->GetNameStringByValue((int64)PointProperty), Range.RangeStartIndex, Range.RangeSize, Range.PointData->GetNumPoints()-1);
			return {};
		}

		TArray<typename TPCGPointNativeProperty<PointProperty>::Type> Values;
		Values.Reserve(Range.RangeSize);

		for (int32 Index = 0; Index < Range.RangeSize; ++Index)
		{
			Values.Add(ValueRange[Range.RangeStartIndex + Index]);
		}

		return Values;
	}

	template<EPCGPointNativeProperties PointProperty>
	bool IsValidRange(const FPCGPointOutputRange& Range, const typename TPCGPointNativeProperty<PointProperty>::ValueRange& ValueRange)
	{
		return Range.RangeSize > 0 && ValueRange.IsValidIndex(Range.RangeStartIndex) && ValueRange.IsValidIndex(Range.RangeStartIndex + Range.RangeSize - 1);
	}

	template<EPCGPointNativeProperties PointProperty>
	void SetOutputRangeValues(FPCGPointOutputRange& Range, const TArray<typename TPCGPointNativeProperty<PointProperty>::Type>& Values)
	{
		if (!IsValid(Range.PointData))
		{
			UE_LOGF(LogPCG, Error, "SetOutputRangeValues '%ls' called with invalid point data.", *StaticEnum<EPCGPointNativeProperties>()->GetNameStringByValue((int64)PointProperty));
			return;
		}

		if (!EnumHasAnyFlags(Range.PointData->GetAllocatedProperties(/*bWithInheritance=*/false), PointProperty))
		{
			UE_LOGF(LogPCG, Error, "SetOutputRangeValues '%ls' called on non-allocated property.", *StaticEnum<EPCGPointNativeProperties>()->GetNameStringByValue((int64)PointProperty));
			return;
		}

		typename TPCGPointNativeProperty<PointProperty>::ValueRange ValueRange = Range.PointData->GetValueRange<PointProperty>(/*bAllocate=*/false);

		if (!PCGBasePointData::IsValidRange<PointProperty>(Range, ValueRange))
		{
			UE_LOGF(LogPCG, Error, "SetOutputRangeValues '%ls' called with output range [%d, %d[ when valid range is [0, %d].", *StaticEnum<EPCGPointNativeProperties>()->GetNameStringByValue((int64)PointProperty), Range.RangeStartIndex, Range.RangeSize, Range.PointData->GetNumPoints()-1);
			return;
		}

		if (Values.Num() > Range.RangeSize)
		{
			UE_LOGF(LogPCG, Error, "SetOutputRangeValues '%ls' called with invalid values array size '%d' when max array size '%d'.", *StaticEnum<EPCGPointNativeProperties>()->GetNameStringByValue((int64)PointProperty), Values.Num(), Range.RangeSize);
			return;
		}

		for (int32 Index = 0; Index < Values.Num(); ++Index)
		{
			ValueRange[Range.RangeStartIndex + Index] = Values[Index];
		}
	}
}

bool FPCGDataTypeInfoPoint::SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const
{
	// Point can convert to param
	if (OutputType.IsSameType(EPCGDataType::Param))
	{
		if (OptionalOutConversionSettings)
		{
			*OptionalOutConversionSettings = UPCGConvertToAttributeSetSettings::StaticClass();
		}

		return true;
	}
	else
	{
		return FPCGDataTypeInfoConcrete::SupportsConversionTo(ThisType, OutputType, OptionalOutConversionSettings, OptionalOutCompatibilityMessage);
	}
}

UPCGBasePointData::UPCGBasePointData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	check(Metadata);
	Metadata->SetupDomain(PCGMetadataDomainID::Elements, /*bIsDefault=*/true);

	// Default to Position as the "last attribute" on creation.
	FPCGAttributePropertySelector DefaultSelector;
	DefaultSelector.SetPointProperty(EPCGPointProperties::Position);
	UPCGSpatialData::SetLastSelector(DefaultSelector);
}

FPCGAttributeAccessorMethods UPCGBasePointData::GetPointAccessorMethods()
{
	auto CreateAccessorFunc = [](UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet)
	{
		return PCGBasePointData::CreateStaticAccessor(CastChecked<UPCGBasePointData>(InData), InSelector, bQuiet, false);
	};

	auto CreateConstAccessorFunc = [](const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<const IPCGAttributeAccessor>
	{
		return PCGBasePointData::CreateStaticAccessor(CastChecked<UPCGBasePointData>(InData), InSelector, bQuiet, true);
	};

	auto CreateAccessorKeysFunc = [](UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<IPCGAttributeAccessorKeys>
	{
		const FPCGMetadataDomainID DomainID = InData->GetMetadataDomainIDFromSelector(InSelector);
		if (DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements)
		{
			UPCGBasePointData* PointData = CastChecked<UPCGBasePointData>(InData);

			// If we know the keys are used with a selector of an attribute, allocate the entries.
			return MakeUnique<FPCGAttributeAccessorKeysPointIndices>(PointData, /*bAllocateMetadataEntries=*/InSelector.GetSelection() == EPCGAttributePropertySelection::Attribute);
		}
		else
		{
			return {};
		}
	};

	auto CreateConstAccessorKeysFunc = [](const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<const IPCGAttributeAccessorKeys>
	{
		const FPCGMetadataDomainID DomainID = InData->GetMetadataDomainIDFromSelector(InSelector);
		if (DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements)
		{
			const UPCGBasePointData* PointData = CastChecked<UPCGBasePointData>(InData);
			return MakeUnique<const FPCGAttributeAccessorKeysPointIndices>(PointData);
		}
		else
		{
			return {};
		}
	};

	FPCGAttributeAccessorMethods Methods
	{
		.CreateAccessorFunc = CreateAccessorFunc,
		.CreateConstAccessorFunc = CreateConstAccessorFunc,
		.CreateAccessorKeysFunc = CreateAccessorKeysFunc,
		.CreateConstAccessorKeysFunc = CreateConstAccessorKeysFunc
	};

#if WITH_EDITOR
	Methods.FillSelectorMenuEntryFromEnum<EPCGPointProperties>({LOCTEXT("PointSelectorMenuEntry", "Point")});
#endif // WITH_EDITOR

	return Methods;
}

FBox UPCGBasePointData::GetBounds() const
{
	RecomputeBoundsIfNeeded();

	return Bounds;
}

void UPCGBasePointData::CopyPropertiesTo(UPCGBasePointData* To, int32 ReadStartIndex, int32 WriteStartIndex, int32 Count, EPCGPointNativeProperties Properties) const
{
	if (EPCGPointNativeProperties::None == Properties || Count <= 0)
	{
		return;
	}

	const TConstPCGValueRange<FTransform> FromTransformRange = GetConstTransformValueRange();
	const TConstPCGValueRange<float> FromSteepnessRange = GetConstSteepnessValueRange();
	const TConstPCGValueRange<float> FromDensityRange = GetConstDensityValueRange();
	const TConstPCGValueRange<FVector> FromBoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> FromBoundsMaxRange = GetConstBoundsMaxValueRange();
	const TConstPCGValueRange<FVector4> FromColorRange = GetConstColorValueRange();
	const TConstPCGValueRange<int64> FromMetadataEntryRange = GetConstMetadataEntryValueRange();
	const TConstPCGValueRange<int32> FromSeedRange = GetConstSeedValueRange();

	To->AllocateProperties(Properties);

	TPCGValueRange<FTransform> ToTransformRange = To->GetTransformValueRange(/*bAllocate=*/false);
	TPCGValueRange<float> ToSteepnessRange = To->GetSteepnessValueRange(/*bAllocate=*/false);
	TPCGValueRange<float> ToDensityRange = To->GetDensityValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector> ToBoundsMinRange = To->GetBoundsMinValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector> ToBoundsMaxRange = To->GetBoundsMaxValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector4> ToColorRange = To->GetColorValueRange(/*bAllocate=*/false);
	TPCGValueRange<int64> ToMetadataEntryRange = To->GetMetadataEntryValueRange(/*bAllocate=*/false);
	TPCGValueRange<int32> ToSeedRange = To->GetSeedValueRange(/*bAllocate=*/false);

	check(ReadStartIndex + Count <= GetNumPoints() && WriteStartIndex + Count <= To->GetNumPoints());

	if (EPCGPointNativeProperties::All == Properties)
	{
		for (int32 Index = 0; Index < Count; ++Index)
		{
			const int32 ReadIndex = ReadStartIndex + Index;
			const int32 WriteIndex = WriteStartIndex + Index;

			ToTransformRange[WriteIndex] = FromTransformRange[ReadIndex];
			ToDensityRange[WriteIndex] = FromDensityRange[ReadIndex];
			ToBoundsMinRange[WriteIndex] = FromBoundsMinRange[ReadIndex];
			ToBoundsMaxRange[WriteIndex] = FromBoundsMaxRange[ReadIndex];
			ToColorRange[WriteIndex] = FromColorRange[ReadIndex];
			ToSteepnessRange[WriteIndex] = FromSteepnessRange[ReadIndex];
			ToSeedRange[WriteIndex] = FromSeedRange[ReadIndex];
			ToMetadataEntryRange[WriteIndex] = FromMetadataEntryRange[ReadIndex];
		}
	}
	else
	{
		auto CopyRangeIf = [ReadStartIndex, WriteStartIndex, Count, Properties]<typename T>(TPCGValueRange<T>& ToRange, const TConstPCGValueRange<T>& FromRange, EPCGPointNativeProperties Property)
		{
			if (EnumHasAllFlags(Properties, Property))
			{
				for (int32 Index = 0; Index < Count; ++Index)
				{
					const int32 ReadIndex = ReadStartIndex + Index;
					const int32 WriteIndex = WriteStartIndex + Index;

					ToRange[WriteIndex] = FromRange[ReadIndex];
				}
			}
		};

		CopyRangeIf(ToTransformRange, FromTransformRange, EPCGPointNativeProperties::Transform);
		CopyRangeIf(ToDensityRange, FromDensityRange, EPCGPointNativeProperties::Density);
		CopyRangeIf(ToBoundsMinRange, FromBoundsMinRange, EPCGPointNativeProperties::BoundsMin);
		CopyRangeIf(ToBoundsMaxRange, FromBoundsMaxRange, EPCGPointNativeProperties::BoundsMax);
		CopyRangeIf(ToColorRange, FromColorRange, EPCGPointNativeProperties::Color);
		CopyRangeIf(ToSteepnessRange, FromSteepnessRange, EPCGPointNativeProperties::Steepness);
		CopyRangeIf(ToSeedRange, FromSeedRange, EPCGPointNativeProperties::Seed);
		CopyRangeIf(ToMetadataEntryRange, FromMetadataEntryRange, EPCGPointNativeProperties::MetadataEntry);
	}

	To->DirtyCache();
}

void UPCGBasePointData::CopyPropertiesTo(UPCGBasePointData* To, const TArrayView<const int32>& ReadIndices, const TArrayView<const int32>& WriteIndices, EPCGPointNativeProperties Properties) const
{
	if (EPCGPointNativeProperties::None == Properties || ReadIndices.Num() == 0 || ReadIndices.Num() != WriteIndices.Num())
	{
		check(ReadIndices.Num() == WriteIndices.Num());
		return;
	}

	const int32 Count = ReadIndices.Num();

	const TConstPCGValueRange<FTransform> FromTransformRange = GetConstTransformValueRange();
	const TConstPCGValueRange<float> FromSteepnessRange = GetConstSteepnessValueRange();
	const TConstPCGValueRange<float> FromDensityRange = GetConstDensityValueRange();
	const TConstPCGValueRange<FVector> FromBoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> FromBoundsMaxRange = GetConstBoundsMaxValueRange();
	const TConstPCGValueRange<FVector4> FromColorRange = GetConstColorValueRange();
	const TConstPCGValueRange<int64> FromMetadataEntryRange = GetConstMetadataEntryValueRange();
	const TConstPCGValueRange<int32> FromSeedRange = GetConstSeedValueRange();

	To->AllocateProperties(Properties);

	TPCGValueRange<FTransform> ToTransformRange = To->GetTransformValueRange(/*bAllocate=*/false);
	TPCGValueRange<float> ToSteepnessRange = To->GetSteepnessValueRange(/*bAllocate=*/false);
	TPCGValueRange<float> ToDensityRange = To->GetDensityValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector> ToBoundsMinRange = To->GetBoundsMinValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector> ToBoundsMaxRange = To->GetBoundsMaxValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector4> ToColorRange = To->GetColorValueRange(/*bAllocate=*/false);
	TPCGValueRange<int64> ToMetadataEntryRange = To->GetMetadataEntryValueRange(/*bAllocate=*/false);
	TPCGValueRange<int32> ToSeedRange = To->GetSeedValueRange(/*bAllocate=*/false);

	auto CopyRangeIf = [&ReadIndices, &WriteIndices, Properties, Count]<typename T>(TPCGValueRange<T>& ToRange, const TConstPCGValueRange<T>& FromRange, EPCGPointNativeProperties Property)
	{
		if (EnumHasAllFlags(Properties, Property))
		{
			for (int32 Index = 0; Index < Count; ++Index)
			{
				const int32& ReadIndex = ReadIndices[Index];
				const int32& WriteIndex = WriteIndices[Index];

				ToRange[WriteIndex] = FromRange[ReadIndex];
			}
		}
	};

	CopyRangeIf(ToTransformRange, FromTransformRange, EPCGPointNativeProperties::Transform);
	CopyRangeIf(ToDensityRange, FromDensityRange, EPCGPointNativeProperties::Density);
	CopyRangeIf(ToBoundsMinRange, FromBoundsMinRange, EPCGPointNativeProperties::BoundsMin);
	CopyRangeIf(ToBoundsMaxRange, FromBoundsMaxRange, EPCGPointNativeProperties::BoundsMax);
	CopyRangeIf(ToColorRange, FromColorRange, EPCGPointNativeProperties::Color);
	CopyRangeIf(ToSteepnessRange, FromSteepnessRange, EPCGPointNativeProperties::Steepness);
	CopyRangeIf(ToSeedRange, FromSeedRange, EPCGPointNativeProperties::Seed);
	CopyRangeIf(ToMetadataEntryRange, FromMetadataEntryRange, EPCGPointNativeProperties::MetadataEntry);

	To->DirtyCache();
}

void UPCGBasePointData::CopyPointsTo(UPCGBasePointData* To, int32 ReadStartIndex, int32 WriteStartIndex, int32 Count) const
{
	CopyPropertiesTo(To, ReadStartIndex, WriteStartIndex, Count, EPCGPointNativeProperties::All);
}

void UPCGBasePointData::CopyPointsTo(UPCGBasePointData* To, const TArrayView<const int32>& ReadIndices, const TArrayView<const int32>& WriteIndices) const
{
	CopyPropertiesTo(To, ReadIndices, WriteIndices, EPCGPointNativeProperties::All);
}

void UPCGBasePointData::BP_SetPointsFrom(const UPCGBasePointData* InData, const TArray<int32>& InDataIndices)
{
	return SetPoints(InData, this, InDataIndices, false);
}

void UPCGBasePointData::SetPointsFrom(const UPCGBasePointData* InData, const TArrayView<const int32>& InDataIndices)
{
	return SetPoints(InData, this, InDataIndices, false);
}

void UPCGBasePointData::SetPoints(const UPCGBasePointData* From, UPCGBasePointData* To, const TArrayView<const int32>& InDataIndices, bool bCopyAll)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGBasePointData::SetPoints);
	check(From);

	const int32 NumPoints = bCopyAll ? From->GetNumPoints() : InDataIndices.Num();
	To->SetNumPoints(NumPoints, /*bInitializeValues=*/false);
	To->AllocateProperties(From->GetAllocatedProperties());

	if (NumPoints == 0)
	{
		To->DirtyCache();
		return;
	}

	const TConstPCGValueRange<FTransform> FromTransformRange = From->GetConstTransformValueRange();
	const TConstPCGValueRange<float> FromSteepnessRange = From->GetConstSteepnessValueRange();
	const TConstPCGValueRange<float> FromDensityRange = From->GetConstDensityValueRange();
	const TConstPCGValueRange<FVector> FromBoundsMinRange = From->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> FromBoundsMaxRange = From->GetConstBoundsMaxValueRange();
	const TConstPCGValueRange<FVector4> FromColorRange = From->GetConstColorValueRange();
	const TConstPCGValueRange<int64> FromMetadataEntryRange = From->GetConstMetadataEntryValueRange();
	const TConstPCGValueRange<int32> FromSeedRange = From->GetConstSeedValueRange();

	TPCGValueRange<FTransform> ToTransformRange = To->GetTransformValueRange(/*bAllocate=*/false);
	TPCGValueRange<float> ToSteepnessRange = To->GetSteepnessValueRange(/*bAllocate=*/false);
	TPCGValueRange<float> ToDensityRange = To->GetDensityValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector> ToBoundsMinRange = To->GetBoundsMinValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector> ToBoundsMaxRange = To->GetBoundsMaxValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector4> ToColorRange = To->GetColorValueRange(/*bAllocate=*/false);
	TPCGValueRange<int64> ToMetadataEntryRange = To->GetMetadataEntryValueRange(/*bAllocate=*/false);
	TPCGValueRange<int32> ToSeedRange = To->GetSeedValueRange(/*bAllocate=*/false);

	for (int32 Index = 0; Index < NumPoints; ++Index)
	{
		const int32 PointIndex = bCopyAll ? Index : InDataIndices[Index];

		ToTransformRange[Index] = FromTransformRange[PointIndex];
		ToSteepnessRange[Index] = FromSteepnessRange[PointIndex];
		ToDensityRange[Index] = FromDensityRange[PointIndex];
		ToBoundsMinRange[Index] = FromBoundsMinRange[PointIndex];
		ToBoundsMaxRange[Index] = FromBoundsMaxRange[PointIndex];
		ToColorRange[Index] = FromColorRange[PointIndex];
		ToMetadataEntryRange[Index] = FromMetadataEntryRange[PointIndex];
		ToSeedRange[Index] = FromSeedRange[PointIndex];
	}

	To->DirtyCache();
}

void UPCGBasePointData::RecomputeBounds() const
{
	PCG::TScopeLock Lock(CachedDataLock);
	if (!bBoundsAreDirty)
	{
		return;
	}

	const TConstPCGValueRange<FTransform> TransformRange = GetConstTransformValueRange();
	const TConstPCGValueRange<float> SteepnessRange = GetConstSteepnessValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

	FBox NewBounds(EForceInit::ForceInit);
	for (int32 PointIndex = 0; PointIndex < GetNumPoints(); ++PointIndex)
	{
		FBoxSphereBounds PointBounds = PCGPointHelpers::GetDensityBounds(TransformRange[PointIndex], SteepnessRange[PointIndex], BoundsMinRange[PointIndex], BoundsMaxRange[PointIndex]);
		NewBounds += FBox::BuildAABB(PointBounds.Origin, PointBounds.BoxExtent);
	}

	Bounds = NewBounds;
	bBoundsAreDirty = false;
}

const PCGPointOctree::FPointOctree& UPCGBasePointData::GetPointOctree() const
{
	RebuildOctreeIfNeeded();
	
	check(PCGPointOctree);
	return *PCGPointOctree;
}

void UPCGBasePointData::RebuildOctree() const
{
	PCG::TScopeLock Lock(CachedDataLock);
	if (!bOctreeIsDirty && PCGPointOctree.IsValid())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGBasePointData::RebuildOctree)
	
	FBox PointBounds = GetBounds();
	const bool bHasInvalidBounds = PointBounds.ContainsNaN();

	if (bHasInvalidBounds)
	{
		UE_LOGF(LogPCG, Error, "Invalid bounds in the point data makes it impossible to create a valid Octree.");
		PointBounds = FBox(EForceInit::ForceInit);
	}

	PCGPointOctree::FPointOctree NewOctree(PointBounds.GetCenter(), PointBounds.GetExtent().Length());

	if (!bHasInvalidBounds)
	{
		const TConstPCGValueRange<FTransform> TransformRange = GetConstTransformValueRange();
		const TConstPCGValueRange<float> SteepnessRange = GetConstSteepnessValueRange();
		const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
		const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

		for (int32 PointIndex = 0; PointIndex < GetNumPoints(); ++PointIndex)
		{
			NewOctree.AddElement(PCGPointOctree::FPointRef(PointIndex, PCGPointHelpers::GetDensityBounds(TransformRange[PointIndex], SteepnessRange[PointIndex], BoundsMinRange[PointIndex], BoundsMaxRange[PointIndex])));
		}
	}

	PCGPointOctree = MakeUnique<PCGPointOctree::FPointOctree>(MoveTemp(NewOctree));
	bOctreeIsDirty = false;
}

void UPCGBasePointData::SetTransform(const FPCGPointTransform::Type& InTransform)
{
	SetPointPropertyValues<FPCGPointTransform::EnumValue>(InTransform);
}

void UPCGBasePointData::SetDensity(FPCGPointDensity::Type InDensity)
{
	SetPointPropertyValues<FPCGPointDensity::EnumValue>(InDensity);
}

void UPCGBasePointData::SetBoundsMin(const FPCGPointBoundsMin::Type& InBoundsMin)
{
	SetPointPropertyValues<FPCGPointBoundsMin::EnumValue>(InBoundsMin);
}

void UPCGBasePointData::SetBoundsMax(const FPCGPointBoundsMax::Type& InBoundsMax)
{
	SetPointPropertyValues<FPCGPointBoundsMax::EnumValue>(InBoundsMax);
}

void UPCGBasePointData::SetColor(const FPCGPointColor::Type& InColor)
{
	SetPointPropertyValues<FPCGPointColor::EnumValue>(InColor);
}

void UPCGBasePointData::SetSteepness(FPCGPointSteepness::Type InSteepness)
{
	SetPointPropertyValues<FPCGPointSteepness::EnumValue>(InSteepness);
}

void UPCGBasePointData::SetSeed(FPCGPointSeed::Type InSeed)
{
	SetPointPropertyValues<FPCGPointSeed::EnumValue>(InSeed);
}

void UPCGBasePointData::SetMetadataEntry(FPCGPointMetadataEntry::Type InMetadataEntry)
{
	SetPointPropertyValues<FPCGPointMetadataEntry::EnumValue>(InMetadataEntry);
}

void UPCGBasePointData::SetExtents(const FVector& InExtents)
{
	// Allocate if needed
	{
		const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
		const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

		check(BoundsMinRange.Num() == BoundsMaxRange.Num());

		if (BoundsMinRange.ViewNum() != BoundsMaxRange.ViewNum())
		{
			AllocateProperties(EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax);
		}
	}

	// If Allocation needed it was already done
	TPCGValueRange<FVector> BoundsMinRange = GetBoundsMinValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector> BoundsMaxRange = GetBoundsMaxValueRange(/*bAllocate=*/false);

	for (int32 PointIndex = 0; PointIndex < BoundsMinRange.ViewNum(); ++PointIndex)
	{
		PCGPointHelpers::SetExtents(InExtents, BoundsMinRange[PointIndex], BoundsMaxRange[PointIndex]);
	}
}

void UPCGBasePointData::SetLocalCenter(const FVector& InLocalCenter)
{
	// Allocate if needed
	{
		const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
		const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

		check(BoundsMinRange.Num() == BoundsMaxRange.Num());

		if (BoundsMinRange.ViewNum() != BoundsMaxRange.ViewNum())
		{
			AllocateProperties(EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax);
		}
	}

	// If Allocation needed it was already done
	TPCGValueRange<FVector> BoundsMinRange = GetBoundsMinValueRange(/*bAllocate=*/false);
	TPCGValueRange<FVector> BoundsMaxRange = GetBoundsMaxValueRange(/*bAllocate=*/false);

	for (int32 PointIndex = 0; PointIndex < BoundsMinRange.ViewNum(); ++PointIndex)
	{
		PCGPointHelpers::SetLocalCenter(InLocalCenter, BoundsMinRange[PointIndex], BoundsMaxRange[PointIndex]);
	}
}

const FPCGPointTransform::Type& UPCGBasePointData::GetTransform(int32 InPointIndex) const
{
	return GetPointPropertyValue<FPCGPointTransform::EnumValue>(InPointIndex);
}

const FPCGPointDensity::Type& UPCGBasePointData::GetDensity(int32 InPointIndex) const
{
	return GetPointPropertyValue<FPCGPointDensity::EnumValue>(InPointIndex);
}

const FPCGPointBoundsMin::Type& UPCGBasePointData::GetBoundsMin(int32 InPointIndex) const
{
	return GetPointPropertyValue<FPCGPointBoundsMin::EnumValue>(InPointIndex);
}

const FPCGPointBoundsMax::Type& UPCGBasePointData::GetBoundsMax(int32 InPointIndex) const
{
	return GetPointPropertyValue<FPCGPointBoundsMax::EnumValue>(InPointIndex);
}

const FPCGPointColor::Type& UPCGBasePointData::GetColor(int32 InPointIndex) const
{
	return GetPointPropertyValue<FPCGPointColor::EnumValue>(InPointIndex);
}

const FPCGPointSteepness::Type& UPCGBasePointData::GetSteepness(int32 InPointIndex) const
{
	return GetPointPropertyValue<FPCGPointSteepness::EnumValue>(InPointIndex);
}

const FPCGPointSeed::Type& UPCGBasePointData::GetSeed(int32 InPointIndex) const
{
	return GetPointPropertyValue<FPCGPointSeed::EnumValue>(InPointIndex);
}

const FPCGPointMetadataEntry::Type& UPCGBasePointData::GetMetadataEntry(int32 InPointIndex) const
{
	return GetPointPropertyValue<FPCGPointMetadataEntry::EnumValue>(InPointIndex);
}

FBoxSphereBounds UPCGBasePointData::GetDensityBounds(int32 InPointIndex) const
{
	const TConstPCGValueRange<FTransform> TransformRange = GetConstTransformValueRange();
	const TConstPCGValueRange<float> SteepnessRange = GetConstSteepnessValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

	return PCGPointHelpers::GetDensityBounds(TransformRange[InPointIndex], SteepnessRange[InPointIndex], BoundsMinRange[InPointIndex], BoundsMaxRange[InPointIndex]);
}

FBox UPCGBasePointData::GetLocalDensityBounds(int32 InPointIndex) const
{
	const TConstPCGValueRange<float> SteepnessRange = GetConstSteepnessValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

	return PCGPointHelpers::GetLocalDensityBounds(SteepnessRange[InPointIndex], BoundsMinRange[InPointIndex], BoundsMaxRange[InPointIndex]);
}

FBox UPCGBasePointData::GetLocalBounds(int32 InPointIndex) const
{
	const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

	return PCGPointHelpers::GetLocalBounds(BoundsMinRange[InPointIndex], BoundsMaxRange[InPointIndex]);
}

FVector UPCGBasePointData::GetLocalCenter(int32 InPointIndex) const
{
	const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

	return PCGPointHelpers::GetLocalCenter(BoundsMinRange[InPointIndex], BoundsMaxRange[InPointIndex]);
}

FVector UPCGBasePointData::GetExtents(int32 InPointIndex) const
{
	const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

	return PCGPointHelpers::GetExtents(BoundsMinRange[InPointIndex], BoundsMaxRange[InPointIndex]);
}

FVector UPCGBasePointData::GetScaledExtents(int32 InPointIndex) const
{
	const TConstPCGValueRange<FTransform> TransformRange = GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

	return PCGPointHelpers::GetScaledExtents(TransformRange[InPointIndex], BoundsMinRange[InPointIndex], BoundsMaxRange[InPointIndex]);
}

FVector UPCGBasePointData::GetLocalSize(int32 InPointIndex) const
{
	const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

	return PCGPointHelpers::GetLocalSize(BoundsMinRange[InPointIndex], BoundsMaxRange[InPointIndex]);
}

FVector UPCGBasePointData::GetScaledLocalSize(int32 InPointIndex) const
{
	const TConstPCGValueRange<FTransform> TransformRange = GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();

	return PCGPointHelpers::GetScaledLocalSize(TransformRange[InPointIndex], BoundsMinRange[InPointIndex], BoundsMaxRange[InPointIndex]);
}

void UPCGBasePointData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	const SIZE_T OctreeSize = PCGPointOctree.IsValid() ? sizeof(*PCGPointOctree) + PCGPointOctree->GetSizeBytes() : 0;

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(OctreeSize + sizeof(Bounds));
}

void UPCGBasePointData::InitializeFromActor(AActor* InActor, bool* bOutOptionalSanitizedTagAttributeName)
{
	check(InActor);
	check(Metadata && Metadata->GetAttributeCount() == 0);

	AddSinglePointFromActor(InActor, bOutOptionalSanitizedTagAttributeName);
}

void UPCGBasePointData::AddSinglePointFromActor(AActor* InActor, bool* bOutOptionalSanitizedTagAttributeName)
{
	check(InActor);

	const int32 PointIndex = GetNumPoints();
	SetNumPoints(PointIndex + 1);

	EPCGPointNativeProperties PropertiesToAllocate = EPCGPointNativeProperties::None;
	
	// Values to assign
	const float PointSteepness = 1.0f;
	const FTransform PointTransform = InActor->GetActorTransform();
	const FVector& Position = PointTransform.GetLocation();
	const int32 PointSeed = PCGHelpers::ComputeSeed((int)Position.X, (int)Position.Y, (int)Position.Z);
	const FBox LocalBounds = PCGHelpers::GetActorLocalBounds(InActor);
	const FVector PointBoundsMin = LocalBounds.Min;
	const FVector PointBoundsMax = LocalBounds.Max;
	const int64 PointMetadataEntry = Metadata->AddEntry();

	// SteepnessRange - Initialize and Allocate if needed 
	const TConstPCGValueRange<float> ConstSteepnessRange = GetConstSteepnessValueRange();
	TOptional<const float> SteepnessSingleValue = ConstSteepnessRange.GetSingleValue();
	const bool bAllocateSteepness = SteepnessSingleValue.IsSet() && SteepnessSingleValue.GetValue() != PointSteepness;
	TPCGValueRange<float> SteepnessRange = GetSteepnessValueRange(bAllocateSteepness);
	
	// TransformRange - Initialize and Allocate if needed 
	const TConstPCGValueRange<FTransform> ConstTransformRange = GetConstTransformValueRange();
	TOptional<const FTransform> TransformSingleValue = ConstTransformRange.GetSingleValue();
	const bool bAllocateTransform = TransformSingleValue.IsSet() && !TransformSingleValue.GetValue().Equals(PointTransform);
	TPCGValueRange<FTransform> TransformRange = GetTransformValueRange(bAllocateTransform);

	// SeedRange - Initialize and Allocate if needed 
	const TConstPCGValueRange<int32> ConstSeedRange = GetConstSeedValueRange();
	TOptional<const int32> SeedSingleValue = ConstSeedRange.GetSingleValue();
	const bool bAllocateSeed = SeedSingleValue.IsSet() && SeedSingleValue.GetValue() != PointSeed;
	TPCGValueRange<int32> SeedRange = GetSeedValueRange(bAllocateSeed);

	// BoundsMinRange - Initialize and Allocate if needed 
	const TConstPCGValueRange<FVector> ConstBoundsMinRange = GetConstBoundsMinValueRange();
	TOptional<const FVector> BoundsMinSingleValue = ConstBoundsMinRange.GetSingleValue();
	const bool bAllocateBoundsMin = BoundsMinSingleValue.IsSet() && BoundsMinSingleValue.GetValue() != PointBoundsMin;
	TPCGValueRange<FVector> BoundsMinRange = GetBoundsMinValueRange(bAllocateBoundsMin);
		
	// BoundsMaxRange - Initialize and Allocate if needed 
	const TConstPCGValueRange<FVector> ConstBoundsMaxRange = GetConstBoundsMaxValueRange();
	TOptional<const FVector> BoundsMaxSingleValue = ConstBoundsMaxRange.GetSingleValue();
	const bool bAllocateBoundsMax = BoundsMaxSingleValue.IsSet() && BoundsMaxSingleValue.GetValue() != PointBoundsMax;
	TPCGValueRange<FVector> BoundsMaxRange = GetBoundsMaxValueRange(bAllocateBoundsMax);

	// MetadataEntryRange - Initialize and Allocate if needed 
	const TConstPCGValueRange<int64> ConstMetadataEntryRange = GetConstMetadataEntryValueRange();
	TOptional<const int64> MetadataEntrySingleValue = ConstMetadataEntryRange.GetSingleValue();
	const bool bAllocateMetadataEntry = MetadataEntrySingleValue.IsSet() && MetadataEntrySingleValue.GetValue() != PointMetadataEntry;
	TPCGValueRange<int64> MetadataEntryRange = GetMetadataEntryValueRange(bAllocateMetadataEntry);

	// Assign values
	SteepnessRange[PointIndex] = PointSteepness;
	TransformRange[PointIndex] = PointTransform;
	SeedRange[PointIndex] = PointSeed;
	BoundsMinRange[PointIndex] = PointBoundsMin;
	BoundsMaxRange[PointIndex] = PointBoundsMax;
	MetadataEntryRange[PointIndex] = PointMetadataEntry;

	FPCGMetadataAttribute<FSoftObjectPath>* ActorReferenceAttribute = Metadata->FindOrCreateAttribute(PCGPointDataConstants::ActorReferenceAttribute, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/false);
	if (ActorReferenceAttribute)
	{
		ActorReferenceAttribute->SetValue(PointMetadataEntry, FSoftObjectPath(InActor));
	}

	bool bSanitizedAttributeNames = false;

	// Parse tags as well
	for (FName Tag : InActor->Tags)
	{
		PCG::Private::FParseTagResult TagData(Tag);
		if (PCG::Private::SetAttributeFromTag(TagData, Metadata, PointMetadataEntry, PCG::Private::ESetAttributeFromTagFlags::CreateAttribute))
		{
			bSanitizedAttributeNames |= TagData.HasBeenSanitized();
		}
	}

	if (bOutOptionalSanitizedTagAttributeName)
	{
		*bOutOptionalSanitizedTagAttributeName = bSanitizedAttributeNames;
	}
}

bool UPCGBasePointData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// Run a projection but don't change the point transform. There is a large overlap in code/functionality so this shares one code path.
	FPCGProjectionParams Params{};
	Params.bProjectPositions = Params.bProjectRotations = Params.bProjectScales = false;
	Params.ColorBlendMode = EPCGProjectionColorBlendMode::SourceValue;

	// The ProjectPoint implementation in this class returns true if the query point is overlapping the point data, which is what SamplePoint should return, so forward the return value.
	return ProjectPoint(InTransform, InBounds, Params, OutPoint, OutMetadata);
}

bool UPCGBasePointData::ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	return ProjectPoint(InTransform, InBounds, InParams, OutPoint, OutMetadata, true);
}

FPCGMetadataDomainID UPCGBasePointData::GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const
{
	const FName DomainName = InSelector.GetDomainName();

	if (DomainName == PCGPointDataConstants::ElementsDomainName)
	{
		return PCGMetadataDomainID::Elements;
	}
	else
	{
		return Super::GetMetadataDomainIDFromSelector(InSelector);
	}
}

bool UPCGBasePointData::SetDomainFromDomainID(const FPCGMetadataDomainID& InDomainID, FPCGAttributePropertySelector& InOutSelector) const
{
	if (InDomainID == PCGMetadataDomainID::Elements)
	{
		InOutSelector.SetDomainName(PCGPointDataConstants::ElementsDomainName, /*bResetExtraNames=*/false);
		return true;
	}
	else
	{
		return Super::SetDomainFromDomainID(InDomainID, InOutSelector);
	}
}

bool UPCGBasePointData::ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata, bool bUseBounds) const
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(UPCGBasePointData::ProjectPoint);
	RebuildOctreeIfNeeded();
	check(PCGPointOctree.IsValid());

	TArray<TPair<int32, FVector::FReal>, TInlineAllocator<4>> Contributions;
	const bool bSampleInVolume = (InBounds.GetExtent() != FVector::ZeroVector);

	const TConstPCGValueRange<FTransform> TransformRange = GetConstTransformValueRange();
	const TConstPCGValueRange<float> SteepnessRange = GetConstSteepnessValueRange();
	const TConstPCGValueRange<float> DensityRange = GetConstDensityValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = GetConstBoundsMaxValueRange();
	const TConstPCGValueRange<FVector4> ColorRange = GetConstColorValueRange();
	const TConstPCGValueRange<int64> MetadataEntryRange = GetConstMetadataEntryValueRange();

	if (!bSampleInVolume)
	{
		const FVector InPosition = InTransform.GetLocation();
		PCGPointOctree->FindElementsWithBoundsTest(FBoxCenterAndExtent(InPosition, FVector::Zero()), [this, &InPosition, &Contributions, &TransformRange, &BoundsMinRange, &BoundsMaxRange, &SteepnessRange](const PCGPointOctree::FPointRef& InPointRef)
		{
			Contributions.Emplace(InPointRef.Index, PCGPointHelpers::InverseEuclidianDistance(TransformRange[InPointRef.Index], BoundsMinRange[InPointRef.Index], BoundsMaxRange[InPointRef.Index], SteepnessRange[InPointRef.Index], InPosition));
		});
	}
	else
	{
		FBox TransformedBounds = InBounds.TransformBy(InTransform);
		FMatrix InTransformInverseMatrix = InTransform.ToMatrixWithScale().Inverse();

		PCGPointOctree->FindElementsWithBoundsTest(FBoxCenterAndExtent(TransformedBounds.GetCenter(), TransformedBounds.GetExtent()), 
			[this, bUseBounds, &InBounds, &InTransformInverseMatrix, &Contributions, &TransformRange, &BoundsMinRange, &BoundsMaxRange, &SteepnessRange](const PCGPointOctree::FPointRef& InPointRef)
			{
				const FVector::FReal Contribution = bUseBounds ? PCGPointHelpers::VolumeOverlap(TransformRange[InPointRef.Index], BoundsMinRange[InPointRef.Index], BoundsMaxRange[InPointRef.Index], SteepnessRange[InPointRef.Index], InBounds, InTransformInverseMatrix) : 1.0;
				if (Contribution > 0)
				{
					Contributions.Emplace(InPointRef.Index, Contribution);
				}
			});
	}

	FVector::FReal SumContributions = 0;
	FVector::FReal MaxContribution = 0;
	int32 MaxContributor = INDEX_NONE;

	for (const TPair<int32, FVector::FReal>& Contribution : Contributions)
	{
		SumContributions += Contribution.Value;

		if (Contribution.Value > MaxContribution)
		{
			MaxContribution = Contribution.Value;
			MaxContributor = Contribution.Key;
		}
	}

	if (SumContributions <= 0)
	{
		return false;
	}

	// Rationale: 
	// When doing volume-to-volume intersection, we want the final density to reflect the amount of overlap
	// if any - hence the volume overlap computation before.
	// But, considering that some points may/will overlap (incl. due to steepness), we want to make sure we do not
	// sum up to more than the total volume. 
	// Note that this might create some artifacts on the edges in some instances, but we will revisit this once we have a
	// better and sufficiently efficient solution.
	const FVector::FReal DensityNormalizationFactor = ((SumContributions > 1.0) ? (1.0 / SumContributions) : 1.0);

	TArray<TPair<PCGMetadataEntryKey, float>, TInlineAllocator<4>> ContributionsForMetadata;

	// Computed weighted average of spatial properties
	FVector WeightedPosition = FVector::ZeroVector;
	FQuat WeightedQuat = FQuat(0.0, 0.0, 0.0, 0.0);
	FVector WeightedScale = FVector::ZeroVector;
	FVector::FReal WeightedDensity = 0;
	FVector WeightedBoundsMin = FVector::ZeroVector;
	FVector WeightedBoundsMax = FVector::ZeroVector;
	FVector4 WeightedColor = FVector4::Zero();
	float WeightedSteepness = 0;

	TArray<int64> MetadataEntries;

	for (const TPair<int32, FVector::FReal>& Contribution : Contributions)
	{
		const int32 SourcePointIndex = Contribution.Key;
		const FVector::FReal Weight = Contribution.Value / SumContributions;

		const FTransform& SourcePointTransform = TransformRange[SourcePointIndex];
		const float& SourcePointSteepness = SteepnessRange[SourcePointIndex];
		const float& SourcePointDensity = DensityRange[SourcePointIndex];
		const FVector& SourcePointBoundsMin = BoundsMinRange[SourcePointIndex];
		const FVector& SourcePointBoundsMax = BoundsMaxRange[SourcePointIndex];
		const FVector4& SourcePointColor = ColorRange[SourcePointIndex];
		const int64& SourcePointMetadataEntry = MetadataEntryRange[SourcePointIndex];

		WeightedPosition += SourcePointTransform.GetLocation() * Weight;
		WeightedQuat = PCGPointHelpers::AddQuatWithWeight(WeightedQuat, SourcePointTransform.GetRotation(), Weight);
		WeightedScale += SourcePointTransform.GetScale3D() * Weight;

		if (!bSampleInVolume)
		{
			WeightedDensity += PCGPointHelpers::ManhattanDensity(SourcePointTransform, SourcePointBoundsMin, SourcePointBoundsMax, SourcePointSteepness, SourcePointDensity, InTransform.GetLocation());
		}
		else
		{
			WeightedDensity += SourcePointDensity * (bUseBounds ? (Contribution.Value * DensityNormalizationFactor) : Weight);
		}

		WeightedBoundsMin += SourcePointBoundsMin * Weight;
		WeightedBoundsMax += SourcePointBoundsMax * Weight;
		WeightedColor += SourcePointColor * Weight;
		WeightedSteepness += SourcePointSteepness * Weight;

		ContributionsForMetadata.Emplace(SourcePointMetadataEntry, static_cast<float>(Weight));
	}

	// Finally, apply changes to point, based on the projection settings
	if (InParams.bProjectPositions)
	{
		OutPoint.Transform.SetLocation(bSampleInVolume ? WeightedPosition : InTransform.GetLocation());
	}
	else
	{
		OutPoint.Transform.SetLocation(InTransform.GetLocation());
	}

	if (InParams.bProjectRotations)
	{
		WeightedQuat.Normalize();
		OutPoint.Transform.SetRotation(WeightedQuat);
	}
	else
	{
		OutPoint.Transform.SetRotation(InTransform.GetRotation());
	}

	if (InParams.bProjectScales)
	{
		OutPoint.Transform.SetScale3D(WeightedScale);
	}
	else
	{
		OutPoint.Transform.SetScale3D(InTransform.GetScale3D());
	}

	OutPoint.Density = static_cast<float>(WeightedDensity);
	OutPoint.BoundsMin = WeightedBoundsMin;
	OutPoint.BoundsMax = WeightedBoundsMax;
	OutPoint.Color = WeightedColor;
	OutPoint.Steepness = WeightedSteepness;

	// Can only compute the metadata if the output metadata supports elements.
	if (FPCGMetadataDomain* OutDomain = OutMetadata ? OutMetadata->GetMetadataDomain(PCGMetadataDomainID::Elements) : nullptr)
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(UPCGBasePointData::SamplePoint::SetupMetadata);
		// Initialize metadata entry for this temporary point
		const FPCGMetadataDomain* ThisDomain = ConstMetadata() ? ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements) : nullptr;
		OutPoint.MetadataEntry = OutDomain->HasParent(ThisDomain) ? OutDomain->AddEntry(MetadataEntryRange[MaxContributor]) : OutDomain->AddEntry();
		
		if (ContributionsForMetadata.Num() > 1)
		{
			OutDomain->ComputeWeightedAttribute(OutPoint.MetadataEntry, MakeArrayView(ContributionsForMetadata), ThisDomain);
		}
	}
	else
	{
		OutPoint.MetadataEntry = PCGInvalidEntryKey;
	}

	return true;
}

void UPCGBasePointData::Flatten()
{
	if (!Metadata)
	{
		return;
	}
		
	// If there is no more attributes, reset all keys from points to invalid
	if (Metadata->GetAttributeCount() == 0)
	{
		TConstPCGValueRange<int64> ConstMetadataEntryRange = GetConstMetadataEntryValueRange();
		TOptional<const int64> MetadataEntrySingleValue = ConstMetadataEntryRange.GetSingleValue();
		
		// Range contains multiple values or the only value in range isn't default
		if (!MetadataEntrySingleValue.IsSet() || MetadataEntrySingleValue.GetValue() != PCGInvalidEntryKey)
		{
			Modify();
		}

		// Release metadata memory if needed
		FreeProperties(EPCGPointNativeProperties::MetadataEntry);
		
		// Set all values to invalid
		TPCGValueRange<int64> MetadataEntryRange = GetMetadataEntryValueRange(/*bAllocate=*/false);
		for (int32 Index = 0; Index < MetadataEntryRange.ViewNum(); ++Index)
		{
			MetadataEntryRange[Index] = PCGInvalidEntryKey;
		}
		
		return;
	}

	// Gather all the keys that are not invalid
	const TConstPCGValueRange<int64> ConstMetadataEntryRange = GetConstMetadataEntryValueRange();
	TArray<PCGMetadataEntryKey> EntryKeys;
	EntryKeys.Reserve(GetNumPoints());
	for (const int64& MetadataEntry : ConstMetadataEntryRange)
	{
		if (MetadataEntry != PCGInvalidEntryKey)
		{
			EntryKeys.Add(MetadataEntry);
		}
	}

	// Then flatten and compress the Metadata for all valid entry keys. Return true if something changed.
	// For the data domain, it will do a normal flatten.
	if (Metadata->FlattenAndCompress({{PCGMetadataDomainID::Elements, EntryKeys}}))
	{
		Modify();
				
		// Go over all the points and assign all a new entry key for all points that has a valid entry key in the first place.
		TPCGValueRange<int64> MetadataEntryRange = GetMetadataEntryValueRange();
		PCGMetadataEntryKey CurrentEntryKey = 0;
		for (int64& MetadataEntry : MetadataEntryRange)
		{
			if (MetadataEntry != PCGInvalidEntryKey)
			{
				MetadataEntry = CurrentEntryKey++;
			}
		}
	}
}

void UPCGBasePointData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// The code below has non-trivial cost, and can be disabled from console.
	if (!bFullDataCrc || !CVarCacheFullPointDataCrc.GetValueOnAnyThread())
	{
		// Fallback to UID
		AddUIDToCrc(Ar);
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGBasePointData::AddToCrc);

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	int32 NumPoints = GetNumPoints();
	if (NumPoints == 0)
	{
		return;
	}

	Ar << NumPoints;

	// Crc point data.
	{
		auto CrcRange = [] <class T> (FArchiveCrc32 & Ar, const TConstPCGValueRange<T>&ValueRange)
		{
			for (int32 Index = 0; Index < ValueRange.ViewNum(); ++Index)
			{
				Ar << const_cast<T&>(ValueRange[Index]);
			}
		};

		UPCGBasePointData* NonConstThis = const_cast<UPCGBasePointData*>(this);

		// Skip Metadata entry keys
		CrcRange(Ar, NonConstThis->GetConstTransformValueRange());
		CrcRange(Ar, NonConstThis->GetConstDensityValueRange());
		CrcRange(Ar, NonConstThis->GetConstBoundsMinValueRange());
		CrcRange(Ar, NonConstThis->GetConstBoundsMaxValueRange());
		CrcRange(Ar, NonConstThis->GetConstSteepnessValueRange());
		CrcRange(Ar, NonConstThis->GetConstSeedValueRange());
		CrcRange(Ar, NonConstThis->GetConstColorValueRange());
	}

	// Crc metadata.
	if (const UPCGMetadata* PCGMetadata = ConstMetadata())
	{
		PCGMetadata->AddToCrc(Ar, bFullDataCrc);
	}
}

TArray<FTransform> UPCGBasePointData::GetTransformValuesFromRange(const FPCGPointInputRange& InputRange)
{
	return PCGBasePointData::GetInputRangeValues<EPCGPointNativeProperties::Transform>(InputRange);
}

TArray<float> UPCGBasePointData::GetDensityValuesFromRange(const FPCGPointInputRange& InputRange)
{
	return PCGBasePointData::GetInputRangeValues<EPCGPointNativeProperties::Density>(InputRange);
}

TArray<FVector> UPCGBasePointData::GetBoundsMinValuesFromRange(const FPCGPointInputRange& InputRange)
{
	return PCGBasePointData::GetInputRangeValues<EPCGPointNativeProperties::BoundsMin>(InputRange);
}

TArray<FVector> UPCGBasePointData::GetBoundsMaxValuesFromRange(const FPCGPointInputRange& InputRange)
{
	return PCGBasePointData::GetInputRangeValues<EPCGPointNativeProperties::BoundsMax>(InputRange);
}

TArray<FVector4> UPCGBasePointData::GetColorValuesFromRange(const FPCGPointInputRange& InputRange)
{
	return PCGBasePointData::GetInputRangeValues<EPCGPointNativeProperties::Color>(InputRange);
}

TArray<float> UPCGBasePointData::GetSteepnessValuesFromRange(const FPCGPointInputRange& InputRange)
{
	return PCGBasePointData::GetInputRangeValues<EPCGPointNativeProperties::Steepness>(InputRange);
}

TArray<int32> UPCGBasePointData::GetSeedValuesFromRange(const FPCGPointInputRange& InputRange)
{
	return PCGBasePointData::GetInputRangeValues<EPCGPointNativeProperties::Seed>(InputRange);
}

TArray<int64> UPCGBasePointData::GetMetadataEntryValuesFromRange(const FPCGPointInputRange& InputRange)
{
	return PCGBasePointData::GetInputRangeValues<EPCGPointNativeProperties::MetadataEntry>(InputRange);
}

void UPCGBasePointData::SetTransformValuesOnRange(FPCGPointOutputRange& OutputRange, const TArray<FTransform>& Values)
{
	PCGBasePointData::SetOutputRangeValues<EPCGPointNativeProperties::Transform>(OutputRange, Values);
}

void UPCGBasePointData::SetDensityValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<float>& Values)
{
	PCGBasePointData::SetOutputRangeValues<EPCGPointNativeProperties::Density>(OutputRange, Values);
}

void UPCGBasePointData::SetBoundsMinValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<FVector>& Values)
{
	PCGBasePointData::SetOutputRangeValues<EPCGPointNativeProperties::BoundsMin>(OutputRange, Values);
}

void UPCGBasePointData::SetBoundsMaxValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<FVector>& Values)
{
	PCGBasePointData::SetOutputRangeValues<EPCGPointNativeProperties::BoundsMax>(OutputRange, Values);
}

void UPCGBasePointData::SetColorValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<FVector4>& Values)
{
	PCGBasePointData::SetOutputRangeValues<EPCGPointNativeProperties::Color>(OutputRange, Values);
}

void UPCGBasePointData::SetSteepnessValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<float>& Values)
{
	PCGBasePointData::SetOutputRangeValues<EPCGPointNativeProperties::Steepness>(OutputRange, Values);
}

void UPCGBasePointData::SetSeedValuesOnRange(FPCGPointOutputRange& OutputRange, const TArray<int32>& Values)
{
	PCGBasePointData::SetOutputRangeValues<EPCGPointNativeProperties::Seed>(OutputRange, Values);
}

void UPCGBasePointData::SetMetadataEntryValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<int64>& Values)
{
	PCGBasePointData::SetOutputRangeValues<EPCGPointNativeProperties::MetadataEntry>(OutputRange, Values);
}

FPCGPoint UPCGBasePointData::GetPointFromRange(const FPCGPointInputRange& InputRange, int32 Index)
{
	if (InputRange.RangeStartIndex < 0)
	{
		UE_LOGF(LogPCG, Error, "GetPointFromRange called with invalid InputRange RangeStartIndex '%d'.", InputRange.RangeStartIndex);
		return FPCGPoint();
	}

	if (Index < 0 || Index >= InputRange.RangeSize)
	{
		UE_LOGF(LogPCG, Error, "GetPointFromRange called with invalid Index '%d' when valid InputRange is [0, %d].", Index, InputRange.RangeSize - 1);
		return FPCGPoint();
	}

	if (!InputRange.PointData)
	{
		UE_LOGF(LogPCG, Error, "GetPointFromRange called with invalid InputRange PointData.");
		return FPCGPoint();
	}

	const int32 FinalIndex = InputRange.RangeStartIndex + Index;

	if (FinalIndex < 0 || FinalIndex >= InputRange.PointData->GetNumPoints())
	{
		UE_LOGF(LogPCG, Error, "GetPointFromRange called with invalid index '%d' when valid range is [0, %d].", FinalIndex, InputRange.PointData->GetNumPoints()-1);
		return FPCGPoint();
	}

	const FConstPCGPointValueRanges PointRanges(InputRange.PointData);
	return PointRanges.GetPoint(FinalIndex);
}

void UPCGBasePointData::SetPointOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, int32 Index, const FPCGPoint& Point)
{
	if (OutputRange.RangeStartIndex < 0)
	{
		UE_LOGF(LogPCG, Error, "SetPointOnRange called with invalid OutputRange RangeStartIndex '%d'.", OutputRange.RangeStartIndex);
		return;
	}

	if (Index < 0 || Index >= OutputRange.RangeSize)
	{
		UE_LOGF(LogPCG, Error, "SetPointOnRange called with invalid Index '%d' when valid OutputRange is [0, %d].", Index, OutputRange.RangeSize-1);
		return;
	}

	if (!OutputRange.PointData)
	{
		UE_LOGF(LogPCG, Error, "SetPointOnRange called with invalid OutputRange PointData.");
		return;
	}

	const int32 FinalIndex = OutputRange.RangeStartIndex + Index;

	if (FinalIndex < 0 || FinalIndex >= OutputRange.PointData->GetNumPoints())
	{
		UE_LOGF(LogPCG, Error, "SetPointOnRange called with invalid index '%d' when valid range is [0, %d].", FinalIndex, OutputRange.PointData->GetNumPoints()-1);
		return;
	}

	FPCGPointValueRanges PointRanges(OutputRange.PointData, /*bAllocate=*/false);
	PointRanges.SetFromPoint(FinalIndex, Point);
}

#undef LOCTEXT_NAMESPACE
