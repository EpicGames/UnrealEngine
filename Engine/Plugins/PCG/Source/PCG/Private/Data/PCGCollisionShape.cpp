// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGCollisionShape.h"
#include "Utils/PCGLogErrors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCollisionShape)

#define LOCTEXT_NAMESPACE "PCGCollisionShape"

FPCGCollisionShape::FPCGCollisionShape(const FCollisionShape InShape, const FPCGContext* InOptionalContext)
{
	switch (InShape.ShapeType)
	{
		case ECollisionShape::Line:
			break;
		case ECollisionShape::Box:
			ShapeType = EPCGCollisionShapeType::Box;
			BoxHalfExtent = InShape.GetBox();
			break;
		case ECollisionShape::Sphere:
			ShapeType = EPCGCollisionShapeType::Sphere;
			SphereRadius = InShape.GetSphereRadius();
			break;
		case ECollisionShape::Capsule:
			ShapeType = EPCGCollisionShapeType::Capsule;
			CapsuleRadius = InShape.GetCapsuleRadius();
			CapsuleHalfHeight = InShape.GetCapsuleHalfHeight();
			break;
		default:
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidFCollisionShape", "Invalid FCollisionShape shape type."), InOptionalContext);
	}
}

FCollisionShape FPCGCollisionShape::ToCollisionShape(const FPCGContext* InOptionalContext) const
{
	FCollisionShape Shape;

	// Recreated the SetShape function to use delineated extents from the settings.
	switch (ShapeType)
	{
		case EPCGCollisionShapeType::Line:
			break;
		case EPCGCollisionShapeType::Box:
			Shape.SetBox(FVector3f(BoxHalfExtent));
			break;
		case EPCGCollisionShapeType::Sphere:
			Shape.SetSphere(SphereRadius);
			break;
		case EPCGCollisionShapeType::Capsule:
			Shape.SetCapsule(CapsuleRadius, CapsuleHalfHeight);
			break;
		default:
			PCGLog::LogErrorOnGraph(
					FText::Format(
							LOCTEXT("InvalidEPCGCollisionShape", "Invalid EPCGCollisionShapeType '{0}'."),
							StaticEnum<EPCGCollisionShapeType>()
								? StaticEnum<EPCGCollisionShapeType>()->GetDisplayNameTextByIndex(static_cast<int64>(ShapeType))
								: LOCTEXT("UnknownEnum", "Unknown")),
					InOptionalContext);
			break;
	}

	return Shape;
}

#undef LOCTEXT_NAMESPACE
