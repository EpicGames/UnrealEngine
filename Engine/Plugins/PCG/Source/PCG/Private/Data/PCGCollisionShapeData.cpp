// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGCollisionShapeData.h"

#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGVolumeSampler.h"

#include "Chaos/GeometryQueries.h"
#include "Components/BoxComponent.h"
#include "Components/CapsuleComponent.h"
#include "Components/ShapeComponent.h"
#include "Components/SphereComponent.h"
#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCollisionShapeData)

void UPCGCollisionShapeData::Initialize(UShapeComponent* InComponent)
{
	check(InComponent && IsSupported(InComponent));
	Shape = InComponent->GetCollisionShape();
	Transform = InComponent->GetComponentTransform();

	//Note: Shape is pre-scaled
	ShapeAdapter = MakeUnique<FPhysicsShapeAdapter>(Transform.GetRotation(), Shape);

	CachedBounds = InComponent->Bounds.GetBox();
	CachedStrictBounds = CachedBounds;
}

bool UPCGCollisionShapeData::IsSupported(UShapeComponent* InComponent)
{
	if (!InComponent)
	{
		return false;
	}

	if(InComponent->IsA<UBoxComponent>() || InComponent->IsA<UCapsuleComponent>() || InComponent->IsA<USphereComponent>())
	{
		return true;
	}

	return false;
}

void UPCGCollisionShapeData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// Implementation note: no metadata at this point yet.

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	Ar << const_cast<FTransform&>(Transform);

	// Shape - only Crc data that is used.
	uint32 ShapeType = static_cast<uint32>(Shape.ShapeType);
	Ar << ShapeType;

	if (Shape.IsSphere())
	{
		Ar << const_cast<float&>(Shape.Sphere.Radius);
	}
	else if (Shape.IsCapsule())
	{
		Ar << const_cast<float&>(Shape.Capsule.Radius);
		Ar << const_cast<float&>(Shape.Capsule.HalfHeight);
	}
	else
	{
		// All other cases - just serialize all three floats
		Ar << const_cast<float&>(Shape.Box.HalfExtentX);
		Ar << const_cast<float&>(Shape.Box.HalfExtentY);
		Ar << const_cast<float&>(Shape.Box.HalfExtentZ);
	}
}

bool UPCGCollisionShapeData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	FCollisionShape CollisionShape;
	CollisionShape.SetBox(FVector3f(InBounds.GetExtent() * InTransform.GetScale3D())); // make sure to prescale
	FPhysicsShapeAdapter PointAdapter(InTransform.GetRotation(), CollisionShape);

	if (Chaos::Utilities::CastHelper(PointAdapter.GetGeometry(), PointAdapter.GetGeomPose(InTransform.GetTranslation()), [this](const auto& Downcast, const auto& FullGeomTransform) { return Chaos::OverlapQuery(ShapeAdapter->GetGeometry(), ShapeAdapter->GetGeomPose(Transform.GetTranslation()), Downcast, FullGeomTransform, /*Thickness=*/0); }))
	{
		new(&OutPoint) FPCGPoint(InTransform, /*Density=*/1.0f, /*Seed=*/0);
		OutPoint.SetLocalBounds(InBounds);
		return true;
	}
	else
	{
		return false;
	}
}

const UPCGPointData* UPCGCollisionShapeData::CreatePointData(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCollisionShapeData::CreatePointData);

	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGCollisionShapeData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCollisionShapeData::CreatePointArrayData);

	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGCollisionShapeData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	const PCGVolumeSampler::FVolumeSamplerParams SamplerParams;

	const UPCGBasePointData* Data = PCGVolumeSampler::SampleVolume(Context, PointDataClass, SamplerParams, this);

	if (Data)
	{
		UE_LOGF(LogPCG, Verbose, "Shape extracted %d points", Data->GetNumPoints());
	}

	return Data;
}

UPCGSpatialData* UPCGCollisionShapeData::CopyInternal(FPCGContext* Context) const
{
	UPCGCollisionShapeData* NewShapeData = FPCGContext::NewObject_AnyThread<UPCGCollisionShapeData>(Context);

	NewShapeData->Transform = Transform;
	NewShapeData->Shape = Shape;
	NewShapeData->ShapeAdapter = MakeUnique<FPhysicsShapeAdapter>(NewShapeData->Transform.GetRotation(), NewShapeData->Shape);
	NewShapeData->CachedBounds = CachedBounds;
	NewShapeData->CachedStrictBounds = CachedStrictBounds;

	return NewShapeData;
}
