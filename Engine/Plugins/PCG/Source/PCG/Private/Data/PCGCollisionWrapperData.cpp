// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGCollisionWrapperData.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"

#include "Components/StaticMeshComponent.h"
#include "Chaos/GeometryQueries.h"
#include "Engine/StaticMesh.h"
#include "Physics/PhysicsFiltering.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/BodySetup.h"
#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCollisionWrapperData)

FPCGCollisionWrapper::~FPCGCollisionWrapper()
{
	Uninitialize();
}

void FPCGCollisionWrapper::Uninitialize()
{
	// Implementation note: we do the full uninitialize even if we think we're not initialized due to the default move operators
	for (FBodyInstance* BodyInstance : BodyInstances)
	{
		if (BodyInstance && BodyInstance->IsValidBodyInstance())
		{
			BodyInstance->TermBody();
		}

		delete BodyInstance;
	}

	BodyInstances.Reset();
	IndexToBodyInstance.Reset();

	bInitialized = false;
}

bool FPCGCollisionWrapper::Prepare(const IPCGAttributeAccessor* Accessor, const IPCGAttributeAccessorKeys* Keys, TArray<FSoftObjectPath>& MeshPathsToLoad)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCollisionWrapper::Prepare);
	MeshPathsToLoad.Reset();

	if (!ensure(!bInitialized))
	{
		return true;
	}

	IndexToBodyInstance.Reset();

	if (!Accessor || !Keys)
	{
		return false;
	}

	auto GatherMeshPathsAndCreateMap = [this, &MeshPathsToLoad](const TArrayView<FSoftObjectPath>& Meshes, int Start, int Range)
	{
		for (const FSoftObjectPath& Mesh : Meshes)
		{
			IndexToBodyInstance.Add(Mesh.IsNull() ? INDEX_NONE : MeshPathsToLoad.AddUnique(Mesh));
		}
	};

	// Implementation note: if this fails, then the mesh paths to load will also be empty, which will behave like we're expecting it to.
	PCGMetadataElementCommon::ApplyOnAccessorRange<FSoftObjectPath>(*Keys, *Accessor, GatherMeshPathsAndCreateMap, EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible);
	return true;
}

void FPCGCollisionWrapper::CreateBodyInstances(const TArray<FSoftObjectPath>& MeshPaths)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCollisionWrapper::CreateBodyInstances);
	if (!ensure(!bInitialized))
	{
		return;
	}

	check(BodyInstances.IsEmpty());
	BodyInstances.Reserve(MeshPaths.Num());

	for (const FSoftObjectPath& MeshPath : MeshPaths)
	{
		FBodyInstance* BodyInstance = nullptr;

		TSoftObjectPtr<UStaticMesh> Mesh(MeshPath);
		UStaticMesh* MeshPtr = nullptr;
	
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCollisionWrapper::CreateBodyInstances::LoadMesh);
			MeshPtr = Mesh.LoadSynchronous();
		}

		if (MeshPtr && MeshPtr->GetBodySetup() && MeshPtr->GetBodySetup()->AggGeom.GetElementCount() > 0)
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCollisionWrapper::CreateBodyInstances::CreateBodyInstance);
			BodyInstance = new FBodyInstance();
			BodyInstance->bAutoWeld = false;
			BodyInstance->bSimulatePhysics = false;
			BodyInstance->InitBody(MeshPtr->GetBodySetup(), FTransform::Identity, nullptr, nullptr);

			if (!BodyInstance->IsValidBodyInstance() || !ensure(BodyInstance->GetBodySetup()))
			{
				// Some part of the initialization process failed.
				delete BodyInstance;
				BodyInstance = nullptr;
			}
		}

		BodyInstances.Add(BodyInstance);
	}

	bInitialized = !MeshPaths.IsEmpty();
}

bool FPCGCollisionWrapper::Initialize(const IPCGAttributeAccessor* Accessor, const IPCGAttributeAccessorKeys* Keys)
{
	if (!ensure(!bInitialized))
	{
		return true;
	}

	TArray<FSoftObjectPath> MeshPathsToLoad;
	if (!Prepare(Accessor, Keys, MeshPathsToLoad))
	{
		return false;
	}

	CreateBodyInstances(MeshPathsToLoad);
	return true;
}

FBodyInstance* FPCGCollisionWrapper::GetBodyInstance(int32 EntryIndex) const
{
	return ((bInitialized && IndexToBodyInstance[EntryIndex] != INDEX_NONE) ? BodyInstances[IndexToBodyInstance[EntryIndex]] : nullptr);
}

void FPCGCollisionWrapper::GetShapeArray(int32 EntryIndex, EPCGCollisionQueryFlag QueryFlag, PhysicsInterfaceTypes::FInlineShapeArray& OutShapeArray) const
{
	GetShapeArray(GetBodyInstance(EntryIndex), QueryFlag, OutShapeArray);
}

bool FPCGCollisionWrapper::GetShapeArray(FBodyInstance* Body, EPCGCollisionQueryFlag QueryFlag, PhysicsInterfaceTypes::FInlineShapeArray& OutShapeArray)
{
	OutShapeArray.Reset();

	if (!Body)
	{
		return true;
	}

	PhysicsInterfaceTypes::FInlineShapeArray AllShapes;
	FillInlineShapeArray_AssumesLocked(AllShapes, Body->GetPhysicsActor());

	for (auto& Shape : AllShapes)
	{
		if (!Shape.Shape)
		{
			continue;
		}

		const Chaos::Filter::FShapeFilterData& ShapeFilter = Shape.Shape->GetShapeFilterData();
		const bool bShapeIsComplex = ShapeFilter.HasFlag(Chaos::EFilterFlags::ComplexCollision);
		const bool bShapeIsSimple = ShapeFilter.HasFlag(Chaos::EFilterFlags::SimpleCollision);

		if ((bShapeIsSimple && (QueryFlag == EPCGCollisionQueryFlag::Simple || QueryFlag == EPCGCollisionQueryFlag::SimpleFirst)) ||
			(bShapeIsComplex && (QueryFlag == EPCGCollisionQueryFlag::Complex || QueryFlag == EPCGCollisionQueryFlag::ComplexFirst)))
		{
			OutShapeArray.Add(Shape);
		}
	}

	if (!OutShapeArray.IsEmpty() || QueryFlag == EPCGCollisionQueryFlag::Simple || QueryFlag == EPCGCollisionQueryFlag::Complex)
	{
		return true;
	}

	// Otherwise, we can take the remainder of the shapes, which means we can take everything
	OutShapeArray = AllShapes;
	return false;
}

bool UPCGCollisionWrapperData::Initialize(const UPCGBasePointData* InPointData, const FPCGAttributePropertyInputSelector& InCollisionSelector, EPCGCollisionQueryFlag InCollisionQueryFlag)
{
	TArray<FSoftObjectPath> MeshesToLoad;
	if (PreInitializeAndGatherMeshesEx(InPointData, InCollisionSelector, InCollisionQueryFlag, MeshesToLoad))
	{
		FinalizeInitializationEx(MeshesToLoad);
		return true;
	}
	else
	{
		return false;
	}
}

bool UPCGCollisionWrapperData::PreInitializeAndGatherMeshesEx(const UPCGBasePointData* InPointData, const FPCGAttributePropertyInputSelector& InCollisionSelector, EPCGCollisionQueryFlag InCollisionQueryFlag, TArray<FSoftObjectPath>& OutMeshesToLoad)
{
	CollisionWrapper.Uninitialize();

	check(InPointData);
	PointData = InPointData;
#if WITH_EDITOR
	RawPointData = InPointData;
#endif

	CollisionSelector = InCollisionSelector;
	CollisionQueryFlag = InCollisionQueryFlag;

	// Go through the point data on the attribute we'll use for the shapes
	// StaticMesh -> see what we're doing in UInstanceStaticMeshComponent::InitInstanceBody, or UStaticMeshComponent::UpdateCollisionFromStaticMesh
	FPCGAttributePropertyInputSelector InputSelector;
	InputSelector = InCollisionSelector.CopyAndFixLast(InPointData);

	TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InPointData, InputSelector);
	TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InPointData, InputSelector);

	if (!InputAccessor.IsValid() || !InputKeys.IsValid())
	{
		return false;
	}

	return CollisionWrapper.Prepare(InputAccessor.Get(), InputKeys.Get(), OutMeshesToLoad);
}

void UPCGCollisionWrapperData::FinalizeInitializationEx(const TArray<FSoftObjectPath>& InMeshPaths)
{
	const bool bWasInitialized = CollisionWrapper.bInitialized;

	CollisionWrapper.CreateBodyInstances(InMeshPaths);

	// Finally, cache the shapes we'll be interested in according to the collision query flag
	if (!bWasInitialized && CollisionWrapper.bInitialized)
	{
		CachedShapes.Reset();
		CachedShapes.Reserve(CollisionWrapper.BodyInstances.Num());

		for (FBodyInstance* BodyInstance : CollisionWrapper.BodyInstances)
		{
			PhysicsInterfaceTypes::FInlineShapeArray& Shapes = CachedShapes.Emplace_GetRef();
			if (BodyInstance)
			{
				CollisionWrapper.GetShapeArray(BodyInstance, CollisionQueryFlag, Shapes);
			}
		}
	}
}

const PhysicsInterfaceTypes::FInlineShapeArray& UPCGCollisionWrapperData::GetCachedShapes(int32 EntryIndex) const
{
	static PhysicsInterfaceTypes::FInlineShapeArray EmptyArray;
	if (!CollisionWrapper.bInitialized || CollisionWrapper.IndexToBodyInstance[EntryIndex] == INDEX_NONE)
	{
		return EmptyArray;
	}
	else
	{
		return CachedShapes[CollisionWrapper.IndexToBodyInstance[EntryIndex]];
	}
}

FPCGCrc UPCGCollisionWrapperData::ComputeCrc(bool bFullDataCrc) const
{
	FArchiveCrc32 Ar;

	AddToCrc(Ar, bFullDataCrc);
	if (GetPointData())
	{
		GetPointData()->AddToCrc(Ar, bFullDataCrc);
	}

	return FPCGCrc(Ar.GetCrc());
}

void UPCGCollisionWrapperData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// Implementation note: no metadata at this point yet

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	CollisionSelector.AddToCrc(Ar);

	uint32 CollisionFlag = static_cast<int32>(CollisionQueryFlag);
	Ar << CollisionFlag;
}

void UPCGCollisionWrapperData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	if (GetPointData())
	{
		const_cast<UPCGBasePointData*>(GetPointData())->GetResourceSizeEx(CumulativeResourceSize);
	}
}

FBox UPCGCollisionWrapperData::GetBounds() const
{
	return GetPointData()->GetBounds();
}

FBox UPCGCollisionWrapperData::GetStrictBounds() const
{
	return GetPointData()->GetStrictBounds();
}

const UPCGPointData* UPCGCollisionWrapperData::ToPointData(FPCGContext* Context, const FBox& InBounds) const
{
	return GetPointData()->ToPointData(Context, InBounds);
}

const UPCGPointArrayData* UPCGCollisionWrapperData::ToPointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	return GetPointData()->ToPointArrayData(Context, InBounds);
}

bool UPCGCollisionWrapperData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	if(!CollisionWrapper.bInitialized)
	{
		return GetPointData()->SamplePoint(InTransform, InBounds, OutPoint, OutMetadata);
	}

	// Find all points matching this point in the point data octree
	// For all touching points, check against the actual physics shape we have for that point
	// E.g. get the point collision attribute value, match to our internal list, get the shape (might require loading), then test against the matching shapes
	// note that we need to place the stuff in the right referential too, since our shapes will all be at the origin
	const PCGPointOctree::FPointOctree& Octree = GetPointData()->GetPointOctree();
	const TConstPCGValueRange<FTransform> TransformRange = GetPointData()->GetConstTransformValueRange();
	const TConstPCGValueRange<float> DensityRange = GetPointData()->GetConstDensityValueRange();

	float Density = 0;
	FBox TransformedBounds = InBounds.TransformBy(InTransform);

	Octree.FindElementsWithBoundsTest(FBoxCenterAndExtent(TransformedBounds.GetCenter(), TransformedBounds.GetExtent()), [this, &TransformRange, &DensityRange, &InTransform, &InBounds, &Density](const PCGPointOctree::FPointRef& InPointRef)
	{
		const float PointDensity = DensityRange[InPointRef.Index];

		if (PointDensity < Density)
		{
			return;
		}

		bool bHasOverlap = false;

		FTransform RelativeTransform = InTransform.GetRelativeTransform(TransformRange[InPointRef.Index]);
		FCollisionShape CollisionShape;
		CollisionShape.SetBox(FVector3f(InBounds.GetExtent() * RelativeTransform.GetScale3D()));
		FPhysicsShapeAdapter CollAdapter(RelativeTransform.GetRotation(), CollisionShape);
		const FPhysicsGeometry& Geom = CollAdapter.GetGeometry();
		FTransform GeomTransform = CollAdapter.GetGeomPose(RelativeTransform.GetLocation());

		const int32 EntryIndex = InPointRef.Index;
		const PhysicsInterfaceTypes::FInlineShapeArray& Shapes = GetCachedShapes(EntryIndex);

		if (!Shapes.IsEmpty())
		{
			for (const FPhysicsShapeHandle& Shape : Shapes)
			{
				if (Chaos::Utilities::CastHelper(Geom, GeomTransform, [&Shape](const auto& Downcast, const auto& FullGeomTransform) { return Chaos::OverlapQuery(*Shape.Shape->GetGeometry(), FTransform::Identity, Downcast, FullGeomTransform); }))
				{
					bHasOverlap = true;
					break;
				}
			}
		}
		else
		{
			FCollisionShape ThisSimpleShape;
			ThisSimpleShape.SetBox(FVector3f(InPointRef.Bounds.BoxExtent));
			FPhysicsShapeAdapter ThisAdapter(FQuat::Identity, ThisSimpleShape);

			bHasOverlap = Chaos::Utilities::CastHelper(Geom, GeomTransform, [&ThisAdapter](const auto& Downcast, const auto& FullGeomTransform) { return Chaos::OverlapQuery(ThisAdapter.GetGeometry(), FTransform::Identity, Downcast, FullGeomTransform, /*Thickness=*/0); });
		}

		if (bHasOverlap)
		{
			Density = PointDensity;
		}
	});

	if (Density > 0)
	{
		OutPoint = FPCGPoint();
		OutPoint.Transform = InTransform;
		OutPoint.SetLocalBounds(InBounds);
		OutPoint.Density = Density;

		return true;
	}
	else
	{
		return false;
	}
}

UPCGSpatialData* UPCGCollisionWrapperData::CopyInternal(FPCGContext* Context) const
{
	UPCGCollisionWrapperData* NewCollisionWrapperData = FPCGContext::NewObject_AnyThread<UPCGCollisionWrapperData>(Context);

	// Implementation note:
	// Since the point data contributes to the CRC, it must be set prior to it's normal initialization, which would have been in the Initialize call.
	NewCollisionWrapperData->PointData = PointData;
#if WITH_EDITOR
	NewCollisionWrapperData->RawPointData = RawPointData;
#endif

	// Initializing physics currently requires being on the GT.
	// In this case, we'll queue a GT task to do the initialization, which will be done by the end of the caller task here.
	// @todo_pcg: in future releases (5.7+) we might want to share the wrapper object, which would be simpler & more efficient
	// assuming we can't change it in any way or fashion.
	auto FinishWrapperDataInitialization = [WeakCollisionWrapperData = TWeakObjectPtr<UPCGCollisionWrapperData>(NewCollisionWrapperData), WeakPointData = TWeakObjectPtr<const UPCGBasePointData>(PointData), CollisionSelector = this->CollisionSelector, CollisionQueryFlag = this->CollisionQueryFlag](FPCGContext* InContext)
	{
		if (UPCGCollisionWrapperData* CollisionWrapperData = WeakCollisionWrapperData.Get())
		{
			CollisionWrapperData->Initialize(WeakPointData.Get(), CollisionSelector, CollisionQueryFlag);
		}

		return true;
	};

	if (IsInGameThread())
	{
		FinishWrapperDataInitialization(Context);
	}
	else
	{
		FPCGScheduleGenericParams Params(FinishWrapperDataInitialization, Context->ExecutionSource.Get());

		// Physics setup must be run on the GT.
		Params.bCanExecuteOnlyOnMainThread = true;

		FPCGTaskId MainThreadTaskId = Context->ScheduleGeneric(Params);

		if (MainThreadTaskId != InvalidPCGTaskId)
		{
			Context->DynamicDependencies.Add(MainThreadTaskId);
		}
	}

	return NewCollisionWrapperData;
}
