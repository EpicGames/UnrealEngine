// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGDataPtrWrapper.h"
#include "PCGData.h"
#include "PCGModule.h"

#include "Containers/Ticker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataPtrWrapper)

FPCGDataPtrWrapper::~FPCGDataPtrWrapper()
{
	DecRefCount();
}

FPCGDataPtrWrapper::FPCGDataPtrWrapper(const FPCGDataPtrWrapper& InWrapper)
{
	Data = InWrapper.Data;
	WeakData = InWrapper.Data;
	IncRefCount();
}

FPCGDataPtrWrapper::FPCGDataPtrWrapper(FPCGDataPtrWrapper&& InWrapper)
{
	Data = InWrapper.Data;
	WeakData = InWrapper.WeakData;
	InWrapper.Data = nullptr;
	InWrapper.WeakData = nullptr;
}

FPCGDataPtrWrapper& FPCGDataPtrWrapper::operator=(const FPCGDataPtrWrapper& InWrapper)
{
	TWeakObjectPtr<const UPCGData> PreviousWeakData = WeakData;

	Data = InWrapper.Data;
	WeakData = InWrapper.WeakData;

	IncRefCount();
	DecRefCount(PreviousWeakData);
	return *this;
}

FPCGDataPtrWrapper& FPCGDataPtrWrapper::operator=(FPCGDataPtrWrapper&& InWrapper)
{
	TWeakObjectPtr<const UPCGData> PreviousWeakData = WeakData;

	Data = InWrapper.Data;
	WeakData = InWrapper.WeakData;

	InWrapper.Data = nullptr;
	InWrapper.WeakData = nullptr;

	DecRefCount(PreviousWeakData);
	return *this;
}

FPCGDataPtrWrapper::FPCGDataPtrWrapper(const UPCGData* InData)
{
	Data = InData;
	WeakData = InData;

	IncRefCount();
}

FPCGDataPtrWrapper& FPCGDataPtrWrapper::operator=(const UPCGData* InData)
{
	TWeakObjectPtr<const UPCGData> PreviousWeakData = WeakData;

	Data = InData;
	WeakData = InData;

	IncRefCount();
	DecRefCount(PreviousWeakData);
	return *this;
}

void FPCGDataPtrWrapper::IncRefCount(const UPCGData* InData)
{
	if (InData)
	{
		InData->IncCollectionRefCount();
	}
}

void FPCGDataPtrWrapper::IncRefCount() 
{ 
	IncRefCount(Data); 
}

void FPCGDataPtrWrapper::DecRefCount(const TWeakObjectPtr<const UPCGData>& InWeakData)
{
	bool bOutPinValid;
	if (TStrongObjectPtr<const UPCGData> InData = InWeakData.TryPin(bOutPinValid))
	{
		InData->DecCollectionRefCount();
	}
	else if(!bOutPinValid)
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [InWeakData]()
		{
			LLM_SCOPE_BYTAG(PCG);

			if (const UPCGData* InData = InWeakData.Get())
			{
				InData->DecCollectionRefCount();
			}
		});
	}
}

void FPCGDataPtrWrapper::DecRefCount() 
{ 
	DecRefCount(WeakData);
}

bool FPCGDataPtrWrapper::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYTAG(PCG);

	TWeakObjectPtr<const UPCGData> WeakDataBeforeSerialization = WeakData;
	Ar << Data;

	if (Ar.IsLoading())
	{
		WeakData = Data;
		IncRefCount(Data);
		DecRefCount(WeakDataBeforeSerialization);
	}

	return true;
}

bool FPCGDataPtrWrapper::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	TWeakObjectPtr<const UPCGData> WeakDataBeforeImport = WeakData;
	StaticStruct()->ImportText(Buffer, this, Parent, PortFlags, ErrorText, StaticStruct()->GetName(), false);

	WeakData = Data;
	IncRefCount(Data);
	DecRefCount(WeakDataBeforeImport);

	return true;
}

bool FPCGDataPtrWrapper::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().GetName() == NAME_ObjectProperty) // "Data"
	{
		TObjectPtr<const UPCGData> DataPtr;
		Slot << DataPtr;
		Data = DataPtr;
		WeakData = DataPtr;
		IncRefCount();
		return true;
	}
	else
	{
		return false;
	}
}
