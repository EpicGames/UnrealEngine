// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGDifferenceData.h"

#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGSpatialDataTpl.h"
#include "Data/PCGUnionData.h"
#include "Elements/PCGExecuteBlueprint.h"
#include "Helpers/PCGAsync.h"

#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDifferenceData)

namespace PCGDifferenceDataUtils
{
	EPCGUnionDensityFunction ToUnionDensityFunction(EPCGDifferenceDensityFunction InDensityFunction)
	{
		if (InDensityFunction == EPCGDifferenceDensityFunction::ClampedSubstraction)
		{
			return EPCGUnionDensityFunction::ClampedAddition;
		}
		else if (InDensityFunction == EPCGDifferenceDensityFunction::Binary)
		{
			return EPCGUnionDensityFunction::Binary;
		}
		else
		{
			return EPCGUnionDensityFunction::Maximum;
		}
	}
}

void UPCGDifferenceData::Initialize(const UPCGSpatialData* InData)
{
	check(InData);
	Source = InData;
	TargetActor = InData->TargetActor;

#if WITH_EDITOR
	RawPointerSource = Source;
#endif
}

void UPCGDifferenceData::K2_AddDifference(const UPCGSpatialData* InDifference)
{
	return AddDifference(UPCGBlueprintBaseElement::ResolveContext(), InDifference);
}

void UPCGDifferenceData::AddDifference(FPCGContext* InContext, const UPCGSpatialData* InDifference)
{
	check(InDifference);

	// In the eventuality that the difference has no overlap with the source, then we can drop it directly
	if (!GetBounds().Intersect(InDifference->GetBounds()))
	{
		return;
	}

	// First difference element we'll keep as is, but subsequent ones will be pushed into a union
	if (!Difference)
	{
		Difference = InDifference;

#if WITH_EDITOR
		RawPointerDifference = InDifference;
#endif
	}
	else
	{
		if (!DifferencesUnion)
		{
			DifferencesUnion = FPCGContext::NewObject_AnyThread<UPCGUnionData>(InContext);
			DifferencesUnion->AddData(Difference);
			DifferencesUnion->SetDensityFunction(PCGDifferenceDataUtils::ToUnionDensityFunction(DensityFunction));
			Difference = DifferencesUnion;

#if WITH_EDITOR
			RawPointerDifference = Difference;
			RawPointerDifferencesUnion = DifferencesUnion;
#endif
		}

		check(Difference == DifferencesUnion);
		DifferencesUnion->AddData(InDifference);
	}
}

void UPCGDifferenceData::SetDensityFunction(EPCGDifferenceDensityFunction InDensityFunction)
{
	DensityFunction = InDensityFunction;

	if (GetDifferencesUnion())
	{
		GetDifferencesUnion()->SetDensityFunction(PCGDifferenceDataUtils::ToUnionDensityFunction(DensityFunction));
	}
}

#if WITH_EDITOR
void UPCGDifferenceData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGDifferenceData, DensityFunction))
	{
		SetDensityFunction(DensityFunction);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPCGDifferenceData::PostLoad()
{
	Super::PostLoad();

	RawPointerSource = Source;
	RawPointerDifference = Difference;
	RawPointerDifferencesUnion = DifferencesUnion;
}
#endif

void UPCGDifferenceData::VisitDataNetwork(TFunctionRef<void(const UPCGData*)> Action) const
{
	if (GetSource())
	{
		GetSource()->VisitDataNetwork(Action);
	}

	if (GetDifference())
	{
		GetDifference()->VisitDataNetwork(Action);
	}
}

FPCGCrc UPCGDifferenceData::ComputeCrc(bool bFullDataCrc) const
{
	FArchiveCrc32 Ar;

	AddToCrc(Ar, bFullDataCrc);

	// Chain together CRCs of operands
	check(GetSource());
	uint32 SourceCrc = GetSource()->GetOrComputeCrc(bFullDataCrc).GetValue();
	Ar << SourceCrc;

	if (GetDifference())
	{
		uint32 DifferenceCrc = GetDifference()->GetOrComputeCrc(bFullDataCrc).GetValue();
		Ar << DifferenceCrc;
	}
	
	return FPCGCrc(Ar.GetCrc());
}

void UPCGDifferenceData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// Implementation note: no metadata in composite data at this point.
	// @todo_pcg: need metadata serialization now

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	uint32 DiffMetadata = bDiffMetadata ? 1 : 0;
	Ar << DiffMetadata;

	uint32 DensityFunctionValue = static_cast<uint32>(DensityFunction);
	Ar << DensityFunctionValue;
}

int UPCGDifferenceData::GetDimension() const
{
	return GetSource()->GetDimension();
}

FBox UPCGDifferenceData::GetBounds() const
{
	return GetSource()->GetBounds();
}

FBox UPCGDifferenceData::GetStrictBounds() const
{
	return GetDifference() ? FBox(EForceInit::ForceInit) : GetSource()->GetStrictBounds();
}

bool UPCGDifferenceData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDifferenceData::SamplePoint);
	check(GetSource());

	FPCGPoint PointFromSource;
	if(!GetSource()->SamplePoint(InTransform, InBounds, PointFromSource, OutMetadata))
	{
		return false;
	}

	OutPoint = PointFromSource;

	FPCGPoint PointFromDiff;
	// Important note: here we will not use the point we got from the source, otherwise we are introducing severe bias
	if (GetDifference() && GetDifference()->SamplePoint(InTransform, InBounds, PointFromDiff, (bDiffMetadata ? OutMetadata : nullptr)))
	{
		const bool bBinaryDensity = (DensityFunction == EPCGDifferenceDensityFunction::Binary);
		
		// Apply difference
		OutPoint.Density = bBinaryDensity ? 0 : FMath::Max(0, PointFromSource.Density - PointFromDiff.Density);
		// Color?
		if (bDiffMetadata && OutMetadata && OutPoint.Density > 0 && PointFromDiff.MetadataEntry != PCGInvalidEntryKey)
		{
			// Safe to also cache GetSource()->Metadata ? I'm not sure it is, but if it is it could also benefit UnionData which sometimes accesses input metadata, and also intersection data
			OutMetadata->MergePointAttributesSubset(PointFromSource, OutMetadata, GetSource()->Metadata, PointFromDiff, OutMetadata, GetDifference()->Metadata, OutPoint, EPCGMetadataOp::Sub);
		}

		return OutPoint.Density > 0;
	}
	else
	{
		return true;
	}
}

bool UPCGDifferenceData::HasNonTrivialTransform() const
{
	check(GetSource());
	return GetSource()->HasNonTrivialTransform();
}

const UPCGPointData* UPCGDifferenceData::CreatePointData(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDifferenceData::CreatePointData);
	
	const UPCGBasePointData* SourcePointData = GetSource()->ToPointData(Context);

	return CastChecked<UPCGPointData>(CreateBasePointData(Context, SourcePointData, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGDifferenceData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDifferenceData::CreatePointArrayData);

	const UPCGBasePointData* SourcePointData = GetSource()->ToPointArrayData(Context);

	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, SourcePointData, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGDifferenceData::CreateBasePointData(FPCGContext* Context, const UPCGBasePointData* SourcePointData, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	if (!SourcePointData)
	{
		UE_LOGF(LogPCG, Error, "Difference unable to get source points");
		return SourcePointData;
	}

	const UPCGSpatialData* DifferenceData = GetDifference();

	if (!DifferenceData)
	{
		UE_LOGF(LogPCG, Verbose, "Difference is trivial");
		return SourcePointData;
	}
		
	UPCGBasePointData* TargetPointData = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
	const UPCGMetadata* SourceMetadata = SourcePointData->ConstMetadata();

	FPCGInitializeFromDataParams InitializeFromDataParams(this);
	InitializeFromDataParams.bInheritSpatialData = false;
	// Since we have collapsed the source data, we need to inherit from this one.
	InitializeFromDataParams.SourceOverride = SourcePointData;
	TargetPointData->InitializeFromDataWithParams(InitializeFromDataParams);

	UPCGMetadata* OutMetadata = TargetPointData->Metadata;

	UPCGMetadata* TempDiffMetadata = nullptr;
	if (bDiffMetadata && OutMetadata)
	{
		TempDiffMetadata = FPCGContext::NewObject_AnyThread<UPCGMetadata>(Context);
		DifferenceData->InitializeTargetMetadata(FPCGInitializeFromDataParams{DifferenceData}, TempDiffMetadata);
	}

	constexpr int ChunkSize = FPCGSpatialDataProcessing::DefaultSamplePointsChunkSize;

	auto ChunkSamplePoints = [this, SourceMetadata, TempDiffMetadata, OutMetadata](const TArrayView<TPair<FTransform, FBox>>& Samples, const UPCGBasePointData* SourcePointData, int32 SourceReadIndex, UPCGBasePointData* TargetPointData, int32 TargetWriteIndex)
	{
		int32 NumWritten = 0;

		FConstPCGPointValueRanges SourceRanges(SourcePointData);
		FPCGPointValueRanges TargetRanges(TargetPointData, /*bAllocate=*/false);

		const int32 NumPoints = Samples.Num();

		TArray<FPCGPoint, TInlineAllocator<ChunkSize>> PointsFromDiff;
		PointsFromDiff.SetNum(NumPoints);

		check(GetDifference());
		GetDifference()->SamplePoints(Samples, PointsFromDiff, TempDiffMetadata);

		struct FKeptPoint
		{
			int32 Index = 0;
			float Density = 0.f;
		};

		TArray<FKeptPoint, TInlineAllocator<ChunkSize>> KeptPoints;
		TArray<int32, TInlineAllocator<ChunkSize>> RejectedPoints;

		const bool bBinaryDensity = (DensityFunction == EPCGDifferenceDensityFunction::Binary);

		for (int PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			const int32 SourceIndex = SourceReadIndex + PointIndex;
			
			const FPCGPoint& PointFromDiff = PointsFromDiff[PointIndex];

			const float Density = (bBinaryDensity && PointFromDiff.Density > 0) ? 0.0f : SourceRanges.DensityRange[SourceIndex] - PointFromDiff.Density;

			if (Density > 0)
			{
				KeptPoints.Add({ PointIndex, Density });
			}
			else if (bKeepZeroDensityPoints)
			{
				RejectedPoints.Add(PointIndex);
			}
		}

		if (KeptPoints.Num() > 0)
		{
			for (const FKeptPoint& KeptPoint : KeptPoints)
			{
				const int32 WriteIndex = TargetWriteIndex + NumWritten;
				const int32 ReadIndex = SourceReadIndex + KeptPoint.Index;

				const FPCGPoint& PointFromDiff = PointsFromDiff[KeptPoint.Index];
								
				TargetRanges.SetFromValueRanges(WriteIndex, SourceRanges, ReadIndex);
				TargetRanges.DensityRange[WriteIndex] = KeptPoint.Density;
				
				if (TempDiffMetadata && PointFromDiff.MetadataEntry != PCGInvalidEntryKey)
				{
					OutMetadata->MergeAttributesSubset(SourceRanges.MetadataEntryRange[ReadIndex], SourceMetadata, SourceMetadata, PointFromDiff.MetadataEntry, TempDiffMetadata, TempDiffMetadata, TargetRanges.MetadataEntryRange[WriteIndex], EPCGMetadataOp::Sub);
				}
				
				++NumWritten;
			}
		}

		if (RejectedPoints.Num() > 0)
		{
			for (int32 RejectedIndex : RejectedPoints)
			{
				const int32 WriteIndex = TargetWriteIndex + NumWritten;
				const int32 ReadIndex = SourceReadIndex + RejectedIndex;

				TargetRanges.SetFromValueRanges(WriteIndex, SourceRanges, ReadIndex);
				TargetRanges.DensityRange[WriteIndex] = 0.f;
				
				++NumWritten;
			}
		}

		return NumWritten;
	};

	const EPCGPointNativeProperties PropertiesToAllocate = SourcePointData->GetAllocatedProperties() | EPCGPointNativeProperties::Density | EPCGPointNativeProperties::MetadataEntry;

	FPCGSpatialDataProcessing::SampleBasedRangeProcessing<ChunkSize>(Context ? &Context->AsyncState : nullptr, ChunkSamplePoints, SourcePointData, TargetPointData, PropertiesToAllocate);

	UE_LOGF(LogPCG, Verbose, "Difference generated %d points from %d source points", TargetPointData->GetNumPoints(), SourcePointData->GetNumPoints());
	return TargetPointData;
}

UPCGSpatialData* UPCGDifferenceData::CopyInternal(FPCGContext* Context) const
{
	UPCGDifferenceData* NewDifferenceData = FPCGContext::NewObject_AnyThread<UPCGDifferenceData>(Context);

	NewDifferenceData->Source = Source;
	NewDifferenceData->Difference = Difference;
	NewDifferenceData->DensityFunction = DensityFunction;
	if (DifferencesUnion)
	{
		NewDifferenceData->DifferencesUnion = static_cast<UPCGUnionData*>(DifferencesUnion->DuplicateData(Context));

#if WITH_EDITOR
		NewDifferenceData->RawPointerDifferencesUnion = NewDifferenceData->DifferencesUnion;
#endif
	}

#if WITH_EDITOR
	NewDifferenceData->RawPointerSource = NewDifferenceData->Source;
	NewDifferenceData->RawPointerDifference = NewDifferenceData->Difference;
#endif

	return NewDifferenceData;
}

void UPCGDifferenceData::InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const
{
	check(InParams.bInheritMetadata);
	check(MetadataToInitialize);

	// Duplicate data case, call the spatial base method
	if (InParams.bIsDuplicatingData)
	{
		UPCGSpatialData::InitializeTargetMetadata(InParams, MetadataToInitialize);
		return;
	}

	// In the case of the difference, we initialize the data from the source, not the difference itself. The difference will add its own
	// attributes afterward.
	if (Source)
	{
		FPCGInitializeFromDataParams CopyParams = InParams;
		CopyParams.SourceOverride = nullptr;

		CopyParams.Source = InParams.SourceOverride ? InParams.SourceOverride : Source;
		CopyParams.Source->InitializeTargetMetadata(CopyParams, MetadataToInitialize);
	}

	MetadataToInitialize->AddAttributes(Metadata);
}
