// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGDynamicMeshData.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGVolumeSampler.h"

#include "UDynamicMesh.h"
#include "Components/DynamicMeshComponent.h"
#include "Helpers/PCGHelpers.h"
#include "Serialization/ArchiveCrc32.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDynamicMeshData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoDynamicMesh, UPCGDynamicMeshData)

UPCGDynamicMeshData::UPCGDynamicMeshData(const FObjectInitializer& ObjectInitializer)
	: UPCGSpatialData(ObjectInitializer)
{
	DynamicMesh = ObjectInitializer.CreateDefaultSubobject<UDynamicMesh>(this, TEXT("DynamicMesh"));
}

void UPCGDynamicMeshData::Initialize(UDynamicMesh* InMesh, bool bCanTakeOwnership, const TArray<UMaterialInterface*>& InMaterials)
{
	check(DynamicMesh);

	if (!InMesh)
	{
		UE_LOGF(LogPCG, Error, "Dynamic Mesh initialization failed because InMesh is null.");
		return;
	}

	if (bCanTakeOwnership)
	{
		Initialize(std::move(InMesh->GetMeshRef()), InMaterials);
	}
	else
	{
		// Make a copy
		Initialize(UE::Geometry::FDynamicMesh3(InMesh->GetMeshRef()), InMaterials);
	}
}

void UPCGDynamicMeshData::Initialize(UE::Geometry::FDynamicMesh3&& InMesh, const TArray<UMaterialInterface*>& InMaterials)
{
	check(DynamicMesh);
	DynamicMesh->SetMesh(std::move(InMesh));
	ResetBounds();
	
	bDynamicMeshOctreeIsDirty = true;
	
	SetMaterials(InMaterials);
}

void UPCGDynamicMeshData::ResetBounds() const
{
	check(DynamicMesh);
	CachedBounds = static_cast<FBox>(DynamicMesh->GetMeshRef().GetBounds());
	bDynamicMeshBoundsAreDirty = false;
}

FBox UPCGDynamicMeshData::GetBounds() const
{
	if (bDynamicMeshBoundsAreDirty)
	{
		PCG::TScopeLock Lock(DynamicMeshBoundsLock);
		if (bDynamicMeshBoundsAreDirty)
		{
			ResetBounds();
		}
	}

	return CachedBounds;
}

void UPCGDynamicMeshData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	if (bFullDataCrc)
	{
		if (Metadata)
		{
			Metadata->Serialize(Ar);
		}

		FString ClassName = StaticClass()->GetPathName();
		Ar << ClassName;

		Ar << DynamicMesh->GetMeshRef();
	}
	else
	{
		AddUIDToCrc(Ar);
	}
}

bool UPCGDynamicMeshData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	if (!DynamicMesh)
	{
		return false;
	}
	
	const FVector BoxCenter = InTransform.TransformPosition(InBounds.GetCenter());
	const FVector HalfScaledExtents(InBounds.GetExtent() * InTransform.GetScale3D());
	const UE::Geometry::FAxisAlignedBox3d ScaledBounds(BoxCenter - HalfScaledExtents, BoxCenter + HalfScaledExtents);
	
	if (!ScaledBounds.Intersects(GetBounds()))
	{
		return false;
	}

	const UE::Geometry::FDynamicMeshOctree3& Octree = GetDynamicMeshOctree();
	const UE::Geometry::FDynamicMesh3& MeshRef = DynamicMesh->GetMeshRef();
	

	// TODO: Awful way to do this, need to implement a triangle AABB box intersection if we want more precision. But not even sure that is the right approach using the Octree
	auto TriangleIntersectionTest = [this, &ScaledBounds, MeshRef](int32 TriangleID) -> bool
	{
		return ScaledBounds.Intersects(MeshRef.GetTriBounds(TriangleID));
	};

	const bool bIntersect = Octree.ParallelOverlapAnyQuery(ScaledBounds, TriangleIntersectionTest) != INDEX_NONE;

	if (bIntersect)
	{
		OutPoint.Transform = InTransform;
		OutPoint.SetLocalBounds(InBounds);
		OutPoint.Density = 1.0f;
		return true;
	}
	else
	{
		return false;
	}
}

const UE::Geometry::FDynamicMeshOctree3& UPCGDynamicMeshData::GetDynamicMeshOctree() const
{
	if (DynamicMesh && bDynamicMeshOctreeIsDirty)
	{
		PCG::TScopeLock Lock(DynamicMeshOctreeLock);

		if (bDynamicMeshOctreeIsDirty)
		{
			if (DynamicMeshOctree.Mesh != DynamicMesh->GetMeshPtr())
			{
				// Was never initialized, or with another mesh, reset it
				if (DynamicMeshOctree.Mesh)
				{
					DynamicMeshOctree = UE::Geometry::FDynamicMeshOctree3{};
				}
				
				DynamicMeshOctree.Initialize(&DynamicMesh->GetMeshRef());
			}
			else
			{
				// We can re-insert all triangles to avoid recomputing the whole octree
				DynamicMeshOctree.ReinsertTriangles(DynamicMesh->GetMeshRef().TriangleIndicesItr());
			}
			
			bDynamicMeshOctreeIsDirty = false;
		}
	}

	return DynamicMeshOctree;
}

const UPCGBasePointData* UPCGDynamicMeshData::ToBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDynamicMeshData::CreatePointData);

	check(DynamicMesh);

	TArray<FVector> Vertices;

	DynamicMesh->ProcessMesh([&Vertices](const FDynamicMesh3& ReadMesh)
		{
			Vertices.Reserve(ReadMesh.VertexCount());
			for (int32 vid : ReadMesh.VertexIndicesItr())
			{
				Vertices.Add(ReadMesh.GetVertex(vid));
			}
		});

	UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
	Data->SetNumPoints(Vertices.Num());
	Data->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed);
	Data->MutableMetadata()->Initialize(ConstMetadata());

	int32 NumWritten = 0;
	FPCGPointValueRanges OutRanges(Data, /*bAllocate=*/false);

	for (const FVector& Vertex : Vertices)
	{
		if (InBounds.IsValid && !InBounds.IsInside(Vertex))
		{
			continue;
		}
		
		OutRanges.TransformRange[NumWritten].SetLocation(Vertex);
		OutRanges.SeedRange[NumWritten] = PCGHelpers::ComputeSeedFromPosition(Vertex);

		++NumWritten;
	}

	Data->SetNumPoints(NumWritten);

	UE_LOGF(LogPCG, Verbose, "Dynamic Mesh extracted %d points", NumWritten);

	return Data;
}

const UPCGPointArrayData* UPCGDynamicMeshData::ToPointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointArrayData>(ToBasePointData(Context, InBounds, UPCGPointArrayData::StaticClass()));
}

const UPCGPointData* UPCGDynamicMeshData::ToPointData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointData>(ToBasePointData(Context, InBounds, UPCGPointData::StaticClass()));
}

void UPCGDynamicMeshData::SetMaterials(const TArray<UMaterialInterface*>& InMaterials)
{
	Materials = InMaterials;
}

void UPCGDynamicMeshData::SetMaterials(TArray<UMaterialInterface*>&& InMaterials)
{
	Materials = MoveTemp(InMaterials);
}

void UPCGDynamicMeshData::InitializeDynamicMeshComponentFromData(UDynamicMeshComponent* InComponent) const
{
	check(InComponent);

	InComponent->SetMesh(UE::Geometry::FDynamicMesh3(DynamicMesh->GetMeshRef()));
	for (int32 i = 0; i < Materials.Num(); ++i)
	{
		InComponent->SetMaterial(i, Materials[i]);
	}
}

UPCGSpatialData* UPCGDynamicMeshData::CopyInternal(FPCGContext* Context) const
{
	UPCGDynamicMeshData* NewDynamicMeshData = FPCGContext::NewObject_AnyThread<UPCGDynamicMeshData>(Context);
	
	NewDynamicMeshData->Initialize(DynamicMesh, /*bCanTakeOwnership=*/false);
	NewDynamicMeshData->Materials = Materials;

	return NewDynamicMeshData;
}
