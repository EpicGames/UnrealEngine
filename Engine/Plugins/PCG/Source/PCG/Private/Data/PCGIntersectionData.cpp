// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGIntersectionData.h"

#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGSpatialDataTpl.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"

#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGIntersectionData)

namespace PCGIntersectionDataMaths
{
	float ComputeDensity(float InDensityA, float InDensityB, EPCGIntersectionDensityFunction InDensityFunction)
	{
		if (InDensityFunction == EPCGIntersectionDensityFunction::Minimum)
		{
			return FMath::Min(InDensityA, InDensityB);
		}
		else // default: Multiply
		{
			return InDensityA * InDensityB;
		}
	}
}

void UPCGIntersectionData::Initialize(const UPCGSpatialData* InA, const UPCGSpatialData* InB)
{
	check(InA && InB);
	A = InA;
	B = InB;
	TargetActor = A->TargetActor;

#if WITH_EDITOR
	RawPointerA = A;
	RawPointerB = B;
#endif

	CachedBounds = PCGHelpers::OverlapBounds(GetA()->GetBounds(), GetB()->GetBounds());
	CachedStrictBounds = PCGHelpers::OverlapBounds(GetA()->GetStrictBounds(), GetB()->GetStrictBounds());
}

#if WITH_EDITOR
void UPCGIntersectionData::PostLoad()
{
	Super::PostLoad();

	RawPointerA = A;
	RawPointerB = B;
}
#endif

void UPCGIntersectionData::VisitDataNetwork(TFunctionRef<void(const UPCGData*)> Action) const
{
	check(GetA() && GetB());
	GetA()->VisitDataNetwork(Action);
	GetB()->VisitDataNetwork(Action);
}

FPCGCrc UPCGIntersectionData::ComputeCrc(bool bFullDataCrc) const
{
	FArchiveCrc32 Ar;

	AddToCrc(Ar, bFullDataCrc);

	// Chain together CRCs of operands
	check(GetA() && GetB());
	uint32 CrcA = GetA()->GetOrComputeCrc(bFullDataCrc).GetValue();
	uint32 CrcB = GetB()->GetOrComputeCrc(bFullDataCrc).GetValue();
	Ar << CrcA;
	Ar << CrcB;
	
	return FPCGCrc(Ar.GetCrc());
}

void UPCGIntersectionData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// Implementation note: no metadata in composite data at this point.
	// @todo_pcg: need metadata serialization now

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	uint32 DensityFunctionValue = static_cast<uint32>(DensityFunction);
	Ar << DensityFunctionValue;
}

int UPCGIntersectionData::GetDimension() const
{
	check(GetA() && GetB());
	return FMath::Min(GetA()->GetDimension(), GetB()->GetDimension());
}

FBox UPCGIntersectionData::GetBounds() const
{
	check(GetA() && GetB());
	return CachedBounds;
}

FBox UPCGIntersectionData::GetStrictBounds() const
{
	check(GetA() && GetB());
	return CachedStrictBounds;
}

bool UPCGIntersectionData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(UPCGIntersectionData::SamplePoint);
	check(GetA() && GetB());
	const UPCGSpatialData* X = (GetA()->HasNonTrivialTransform() || !GetB()->HasNonTrivialTransform()) ? GetA() : GetB();
	const UPCGSpatialData* Y = (X == GetA()) ? GetB() : GetA();

	FPCGPoint PointFromX;
	if(!X->SamplePoint(InTransform, InBounds, PointFromX, OutMetadata))
	{
		return false;
	}

	FPCGPoint PointFromY;
	if(!Y->SamplePoint(PointFromX.Transform, InBounds, PointFromY, OutMetadata))
	{
		return false;
	}

	// Merge points into a single point
	OutPoint = PointFromY;
	OutPoint.Density = PCGIntersectionDataMaths::ComputeDensity(PointFromX.Density, PointFromY.Density, DensityFunction);
	OutPoint.Color = PointFromX.Color * PointFromY.Color;

	if (OutMetadata)
	{
		if (PointFromX.MetadataEntry != PCGInvalidEntryKey && PointFromY.MetadataEntry != PCGInvalidEntryKey)
		{
			OutMetadata->MergePointAttributesSubset(PointFromX, OutMetadata, X->Metadata, PointFromY, OutMetadata, Y->Metadata, OutPoint, EPCGMetadataOp::Min);
		}
		else if (PointFromX.MetadataEntry != PCGInvalidEntryKey)
		{
			OutPoint.MetadataEntry = PointFromX.MetadataEntry;
		}
		else
		{
			OutPoint.MetadataEntry = PointFromY.MetadataEntry;
		}
	}

	return true;
}

bool UPCGIntersectionData::HasNonTrivialTransform() const
{
	check(GetA() && GetB());
	return GetA()->HasNonTrivialTransform() || GetB()->HasNonTrivialTransform();
}

const UPCGSpatialData* UPCGIntersectionData::FindFirstConcreteShapeFromNetwork() const
{
	check(GetA() && GetB());

	if (const UPCGSpatialData* CandidateA = GetA()->FindFirstConcreteShapeFromNetwork())
	{
		return CandidateA;
	}

	return GetB()->FindFirstConcreteShapeFromNetwork();
}

const UPCGPointData* UPCGIntersectionData::CreatePointData(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGIntersectionData::CreatePointData);
	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGIntersectionData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGIntersectionData::CreatePointArrayData);
	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

UPCGBasePointData* UPCGIntersectionData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	const UPCGSpatialData* DataA = GetA();
	const UPCGSpatialData* DataB = GetB();
	check(DataA && DataB);
	// TODO: this is a placeholder;
	// Here we will get the point data from the lower-dimensionality data
	// and then cull out any of the points that are outside the bounds of the other
	if (DataA->GetDimension() <= DataB->GetDimension())
	{
		return CreateAndFilterPointData(Context, DataA, DataB, PointDataClass);
	}
	else
	{
		return CreateAndFilterPointData(Context, DataB, DataA, PointDataClass);
	}
}

UPCGBasePointData* UPCGIntersectionData::CreateAndFilterPointData(FPCGContext* Context, const UPCGSpatialData* X, const UPCGSpatialData* Y, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGIntersectionData::CreateAndFilterPointData);
	check(X && Y);
	check(X->GetDimension() <= Y->GetDimension());

	const UPCGBasePointData* SourcePointData = X->ToBasePointData(Context, CachedBounds);

	if (!SourcePointData)
	{
		UE_LOGF(LogPCG, Error, "Intersection unable to get source points");
		return nullptr;
	}
	
	const UPCGMetadata* SourceMetadata = SourcePointData->ConstMetadata();
	const UPCGMetadata* ThisMetadata = ConstMetadata();

	UPCGBasePointData* OutputData = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
	UPCGMetadata* OutputMetadata = OutputData->MutableMetadata();

	FPCGInitializeFromDataParams InitializeFromDataParams(this);
	InitializeFromDataParams.bInheritSpatialData = false;
	// Since we have collapsed the source data, we need to inherit from this one.
	InitializeFromDataParams.SourceOverride = SourcePointData;
	OutputData->InitializeFromDataWithParams(InitializeFromDataParams);

	UPCGMetadata* TempYMetadata = nullptr;
	if (Y && Y->ConstMetadata())
	{
		TempYMetadata = FPCGContext::NewObject_AnyThread<UPCGMetadata>(Context);
		Y->InitializeTargetMetadata(FPCGInitializeFromDataParams{Y}, TempYMetadata);
	}

	const bool bPointDataHasCommonAttributes = (SourceMetadata && TempYMetadata && SourceMetadata->HasCommonAttributes(TempYMetadata));

	constexpr int ChunkSize = FPCGSpatialDataProcessing::DefaultSamplePointsChunkSize;

	auto ChunkSamplePoints = [this, SourceMetadata, Y, TempYMetadata, bPointDataHasCommonAttributes](const TArrayView<TPair<FTransform, FBox>>& Samples, const UPCGBasePointData* SourcePointData, int32 SourceReadIndex, UPCGBasePointData* TargetPointData, int32 TargetWriteIndex)
	{
		int32 NumWritten = 0;

		const FConstPCGPointValueRanges SourceRanges(SourcePointData);
		FPCGPointValueRanges TargetRanges(TargetPointData, /*bAllocate=*/false);
				
		const int NumPoints = Samples.Num();

		TArray<FPCGPoint, TInlineAllocator<ChunkSize>> PointsFromY;
		PointsFromY.SetNum(NumPoints);

		Y->SamplePoints(Samples, PointsFromY, TempYMetadata);

		TArray<int32, TInlineAllocator<ChunkSize>> KeptPoints;
		TArray<int32, TInlineAllocator<ChunkSize>> RejectedPoints;

		// Filter points based on output density
		for (int PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			const int32 SourceIndex = SourceReadIndex + PointIndex;

			FPCGPoint& PointFromY = PointsFromY[PointIndex];

			if (PointFromY.Density > 0)
			{
				KeptPoints.Add(PointIndex); // note: not the sampled point
			}
			else if (bKeepZeroDensityPoints)
			{
				RejectedPoints.Add(PointIndex); // note: not the sampled point
			}
		}

		if (KeptPoints.Num() > 0)
		{
			for (int32 KeptIndex : KeptPoints)
			{
				const int32 WriteIndex = TargetWriteIndex + NumWritten;
				const int32 ReadIndex = SourceReadIndex + KeptIndex;

				FPCGPoint& PointFromY = PointsFromY[KeptIndex];

				TargetRanges.SetFromValueRanges(WriteIndex, SourceRanges, ReadIndex);

				TargetRanges.DensityRange[WriteIndex] = PCGIntersectionDataMaths::ComputeDensity(SourceRanges.DensityRange[ReadIndex], PointFromY.Density, DensityFunction);
				TargetRanges.ColorRange[WriteIndex] = SourceRanges.ColorRange[ReadIndex] * PointFromY.Color;
				
				// TODO: create an array-based MergePointsAttributeSubset..
				// If either the point from Y has metadata or the merge would be a non-trivial value, then perform the full merge
				UPCGMetadata* TargetMetadata = TargetPointData->MutableMetadata();
				if (TargetMetadata && (bPointDataHasCommonAttributes || PointFromY.MetadataEntry != PCGInvalidEntryKey))
				{
					TargetMetadata->MergeAttributesSubset(SourceRanges.MetadataEntryRange[ReadIndex], SourceMetadata, SourceMetadata, PointFromY.MetadataEntry, TempYMetadata, TempYMetadata, TargetRanges.MetadataEntryRange[WriteIndex], EPCGMetadataOp::Min);
				}

				++NumWritten;
			}
		}

		if (RejectedPoints.Num() > 0)
		{
			for (int32 RejectedIndex : RejectedPoints)
			{
				const int32 WriteIndex = TargetWriteIndex + NumWritten;
				const int32 ReadIndex = SourceReadIndex + RejectedIndex;

				TargetRanges.SetFromValueRanges(WriteIndex, SourceRanges, ReadIndex);
				
				TargetRanges.DensityRange[WriteIndex] = 0.f;
				
				++NumWritten;
			}
		}

		return NumWritten;
	};

	const EPCGPointNativeProperties PropertiesToAllocate = SourcePointData->GetAllocatedProperties() | EPCGPointNativeProperties::Density | EPCGPointNativeProperties::Color | EPCGPointNativeProperties::MetadataEntry;

	FPCGSpatialDataProcessing::SampleBasedRangeProcessing<ChunkSize>(Context ? &Context->AsyncState : nullptr, ChunkSamplePoints, SourcePointData, OutputData, PropertiesToAllocate);

	UE_LOGF(LogPCG, Verbose, "Intersection generated %d points from %d source points", OutputData->GetNumPoints(), SourcePointData->GetNumPoints());

	return OutputData;
}

UPCGSpatialData* UPCGIntersectionData::CopyInternal(FPCGContext* Context) const
{
	UPCGIntersectionData* NewIntersectionData = FPCGContext::NewObject_AnyThread<UPCGIntersectionData>(Context);

	NewIntersectionData->DensityFunction = DensityFunction;
	NewIntersectionData->A = A;
	NewIntersectionData->B = B;
	NewIntersectionData->CachedBounds = CachedBounds;
	NewIntersectionData->CachedStrictBounds = CachedStrictBounds;

#if WITH_EDITOR
	NewIntersectionData->RawPointerA = RawPointerA;
	NewIntersectionData->RawPointerB = RawPointerB;
#endif

	return NewIntersectionData;
}

void UPCGIntersectionData::InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const
{
	check(InParams.bInheritMetadata);
	check(MetadataToInitialize);

	// Duplicate data case, call the spatial base method
	if (InParams.bIsDuplicatingData)
	{
		UPCGSpatialData::InitializeTargetMetadata(InParams, MetadataToInitialize);
		return;
	}

	check(GetA());
	check(GetB());

	// In the case of the intersection, we initialize the data from either A or B, depending on the dimension, not the intersection itself. The intersection will add its own
	// attributes afterward.
	FPCGInitializeFromDataParams ParamsCopy = InParams;
	ParamsCopy.SourceOverride = nullptr;

	const UPCGSpatialData* SourceData = (GetA()->GetDimension() <= GetB()->GetDimension()) ? GetA() : GetB();
	const UPCGSpatialData* OtherData = SourceData == GetA() ? GetB() : GetA();

	ParamsCopy.Source = InParams.SourceOverride ? InParams.SourceOverride.Get() : SourceData;
	ParamsCopy.Source->InitializeTargetMetadata(ParamsCopy, MetadataToInitialize);

	ParamsCopy.Source = OtherData;
	ParamsCopy.Source->InitializeTargetMetadata(ParamsCopy, MetadataToInitialize);

	MetadataToInitialize->AddAttributes(Metadata);
}
