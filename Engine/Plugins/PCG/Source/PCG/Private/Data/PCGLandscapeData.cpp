// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGLandscapeData.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGSubsystem.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialDataTpl.h"
#include "Data/PCGSurfaceData.h"
#include "Grid/PCGLandscapeCache.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGWorldQueryHelpers.h"
#include "Utils/PCGLogErrors.h"

#include "ChaosInterfaceWrapperCore.h"
#include "Landscape.h"
#include "LandscapeInfo.h"
#include "LandscapeProxy.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Chaos/PhysicsObjectCollisionInterface.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGLandscapeData)

namespace PCGLandscapeDataConstants
{
	const FName ComponentXAttribute = TEXT("ComponentX");
	const FName ComponentYAttribute = TEXT("ComponentY");
}

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoLandscape, UPCGLandscapeData)

void UPCGLandscapeData::Initialize(const TArray<TWeakObjectPtr<ALandscapeProxy>>& InLandscapes, const FBox& InBounds, const FPCGLandscapeDataProps& InDataProps)
{
	TSet<ALandscapeProxy*> LandscapesToIgnore;

	for (TWeakObjectPtr<ALandscapeProxy> InLandscape : InLandscapes)
	{
		if (InLandscape.IsValid() && InLandscape->GetLandscapeActor() != InLandscape.Get())
		{
			LandscapesToIgnore.Add(InLandscape->GetLandscapeActor());
		}
	}

	TArray<TWeakObjectPtr<ALandscapeProxy>> FilteredLandscapes;
	for (TWeakObjectPtr<ALandscapeProxy> InLandscape : InLandscapes)
	{
		if (InLandscape.IsValid() && !LandscapesToIgnore.Contains(InLandscape.Get()))
		{
			FilteredLandscapes.Add(InLandscape);
		}
	}

	for (TWeakObjectPtr<ALandscapeProxy> Landscape : FilteredLandscapes)
	{
		check(Landscape.IsValid());
		Landscapes.Emplace(Landscape.Get());

		// Build landscape info list
		LandscapeInfos.AddUnique(Landscape->GetLandscapeInfo());
		BoundsToLandscapeInfos.Emplace(PCGHelpers::GetLandscapeBounds(Landscape.Get()), Landscape->GetLandscapeInfo());
	}

	check(!Landscapes.IsEmpty());

	ALandscapeProxy* FirstLandscape = Landscapes[0].Get();
	check(FirstLandscape);

	Bounds = InBounds;
	DataProps = InDataProps;

	Transform = FirstLandscape->GetActorTransform();

	// Store cache pointer for easier access
	UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(FirstLandscape->GetWorld());
	LandscapeCache = PCGSubsystem ? PCGSubsystem->GetLandscapeCache() : nullptr;
}

void UPCGLandscapeData::PostLoad()
{
	Super::PostLoad();

	ALandscapeProxy* FirstLandscape = nullptr;

	for (TSoftObjectPtr<ALandscapeProxy> Landscape : Landscapes)
	{
		Landscape.LoadSynchronous();
		if (Landscape.Get())
		{
			LandscapeInfos.AddUnique(Landscape->GetLandscapeInfo());
			BoundsToLandscapeInfos.Emplace(PCGHelpers::GetLandscapeBounds(Landscape.Get()), Landscape->GetLandscapeInfo());

			if (!FirstLandscape)
			{
				FirstLandscape = Landscape.Get();
			}
		}
		else
		{
			UE_LOGF(LogPCG, Warning, "Was unable to load landscape in landscape data");
		}
	}

	UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(FirstLandscape ? FirstLandscape->GetWorld() : nullptr);
	LandscapeCache = PCGSubsystem ? PCGSubsystem->GetLandscapeCache() : nullptr;

#if WITH_EDITOR
	if (bHeightOnly_DEPRECATED)
	{
		DataProps.bGetHeightOnly = bHeightOnly_DEPRECATED;
		bHeightOnly_DEPRECATED = false;
	}

	if (!bUseMetadata_DEPRECATED)
	{
		DataProps.bGetLayerWeights = bUseMetadata_DEPRECATED;
		bUseMetadata_DEPRECATED = true;
	}
#endif
}

void UPCGLandscapeData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(Ar);
}

FBox UPCGLandscapeData::GetBounds() const
{
	return Bounds;
}

FBox UPCGLandscapeData::GetStrictBounds() const
{
	// TODO: if the landscape contains holes, then the strict bounds
	// should be empty
	return Bounds;
}

bool UPCGLandscapeData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	const ULandscapeInfo* LandscapeInfo = GetLandscapeInfo(InTransform.GetLocation());
	if (!LandscapeInfo || !LandscapeInfo->GetLandscapeProxy())
	{
		return false;
	}

	const FTransform LandscapeTransform = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld();

	// Box in local space -> box in world space -> box in landscape space
	const FTransform BoundsTransformInLanscapeSpace = InTransform.GetRelativeTransform(LandscapeTransform);
	FBox BoundsInLanscapeSpace = InBounds.TransformBy(BoundsTransformInLanscapeSpace);

	// Gather all landscape heightfield components we need to test
	const int ComponentMapKeyMinX = FMath::FloorToInt(BoundsInLanscapeSpace.Min.X / LandscapeInfo->ComponentSizeQuads);
	const int ComponentMapKeyMaxX = FMath::FloorToInt(BoundsInLanscapeSpace.Max.X / LandscapeInfo->ComponentSizeQuads);
	const int ComponentMapKeyMinY = FMath::FloorToInt(BoundsInLanscapeSpace.Min.Y / LandscapeInfo->ComponentSizeQuads);
	const int ComponentMapKeyMaxY = FMath::FloorToInt(BoundsInLanscapeSpace.Max.Y / LandscapeInfo->ComponentSizeQuads);

	TArray<ULandscapeHeightfieldCollisionComponent*, TInlineAllocator<1>> LandscapeCollisionComponents;

	for (int X = ComponentMapKeyMinX; X <= ComponentMapKeyMaxX; ++X)
	{
		for (int Y = ComponentMapKeyMinY; Y <= ComponentMapKeyMaxY; ++Y)
		{
			ULandscapeHeightfieldCollisionComponent* CollisionComponent = LandscapeInfo->XYtoCollisionComponentMap.FindRef(FIntPoint(X, Y));

			if (CollisionComponent)
			{
				LandscapeCollisionComponents.AddUnique(CollisionComponent);
			}
		}
	}

	FCollisionShape CollisionShape;
	if (!LandscapeCollisionComponents.IsEmpty())
	{
		CollisionShape.SetBox(FVector3f(InBounds.GetExtent() * InTransform.GetScale3D())); 
	}

	// Test collision against all gathered collision components
	for (ULandscapeHeightfieldCollisionComponent* Component : LandscapeCollisionComponents)
	{
		check(Component);
		TArray<FOverlapResult> OutOverlap;
		if (Component->OverlapComponentWithResult(InTransform.GetLocation(), InTransform.GetRotation(), CollisionShape, OutOverlap))
		{
			new(&OutPoint) FPCGPoint(InTransform, /*Density=*/1.0f, /*Seed=*/0);
			OutPoint.SetLocalBounds(InBounds);
			return true;
		}
	}

	return false;
}

void UPCGLandscapeData::SamplePoints(const TArrayView<const TPair<FTransform, FBox>>& Samples, const TArrayView<FPCGPoint>& OutPoints, UPCGMetadata* OutMetadata) const
{
	// Implementation note:
	// We will first build a list of all relevant landscsape collision components and the samples to test against them
	constexpr int32 ChunkSize = FPCGSpatialDataProcessing::DefaultSamplePointsChunkSize;
	TMap<ULandscapeHeightfieldCollisionComponent*, TArray<int, TInlineAllocator<ChunkSize>>> LandscapeCollisionComponentsToSamples;
	TMap<const ULandscapeInfo*, FTransform> LandscapeTransformsMap;

	for (int SampleIndex = 0; SampleIndex < Samples.Num(); ++SampleIndex)
	{
		const TPair<FTransform, FBox>& Sample = Samples[SampleIndex];
		const ULandscapeInfo* LandscapeInfo = GetLandscapeInfo(Sample.Key.GetLocation());

		// Implementation note: we reset the density here to simplify the early return cases + the points that wouldn't be kept at the end of the process
		OutPoints[SampleIndex].Density = 0;

		if (!LandscapeInfo || !LandscapeInfo->GetLandscapeProxy())
		{
			continue;
		}

		if (!LandscapeTransformsMap.Contains(LandscapeInfo))
		{
			LandscapeTransformsMap.Add(LandscapeInfo, LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld());
		}

		const FTransform& LandscapeTransform = LandscapeTransformsMap[LandscapeInfo];

		// Transform Box in local space -> box in world space -> box in landscape space
		const FTransform BoundsTransformInLanscapeSpace = Sample.Key.GetRelativeTransform(LandscapeTransform);
		FBox BoundsInLanscapeSpace = Sample.Value.TransformBy(BoundsTransformInLanscapeSpace);

		// The landscape is transformed so that its coordinates are [0, ComponentSizeQuads], so we'll compute our min/max bounds here in landscape local space down below
		// Gather all landscape heightfield components we need to test
		const int ComponentMapKeyMinX = FMath::FloorToInt(BoundsInLanscapeSpace.Min.X / LandscapeInfo->ComponentSizeQuads);
		const int ComponentMapKeyMaxX = FMath::FloorToInt(BoundsInLanscapeSpace.Max.X / LandscapeInfo->ComponentSizeQuads);
		const int ComponentMapKeyMinY = FMath::FloorToInt(BoundsInLanscapeSpace.Min.Y / LandscapeInfo->ComponentSizeQuads);
		const int ComponentMapKeyMaxY = FMath::FloorToInt(BoundsInLanscapeSpace.Max.Y / LandscapeInfo->ComponentSizeQuads);

		for (int X = ComponentMapKeyMinX; X <= ComponentMapKeyMaxX; ++X)
		{
			for (int Y = ComponentMapKeyMinY; Y <= ComponentMapKeyMaxY; ++Y)
			{
				if (ULandscapeHeightfieldCollisionComponent* CollisionComponent = LandscapeInfo->XYtoCollisionComponentMap.FindRef(FIntPoint(X, Y)))
				{
					LandscapeCollisionComponentsToSamples.FindOrAdd(CollisionComponent).Add(SampleIndex);
				}
			}
		}
	}

	if (LandscapeCollisionComponentsToSamples.IsEmpty())
	{
		return;
	}

	TArray<FCollisionShape, TInlineAllocator<ChunkSize>> CollisionShapes;
	TArray<FPhysicsShapeAdapter_Chaos, TInlineAllocator<ChunkSize>> CollisionShapeAdapters;
	TBitArray<TInlineAllocator<ChunkSize>> KeptSamples(false, Samples.Num());
	CollisionShapes.Reserve(Samples.Num());

	for (const TPair<FTransform, FBox>& Sample : Samples)
	{
		FCollisionShape& CollisionShape = CollisionShapes.Emplace_GetRef();
		CollisionShape.SetBox(FVector3f(Sample.Value.GetExtent() * Sample.Key.GetScale3D()));
		CollisionShapeAdapters.Emplace(Sample.Key.GetRotation(), CollisionShape);
	}

	// For each landscape collision component, lock, test all points, repeat.
	TArray<ChaosInterface::FOverlapHit> OverlapHits;
	for (const auto& ComponentToSamples : LandscapeCollisionComponentsToSamples)
	{
		ULandscapeHeightfieldCollisionComponent* Component = ComponentToSamples.Key;
		const TArray<int, TInlineAllocator<ChunkSize>>& SampleIndices = ComponentToSamples.Value;

		// Implementation note: this is an exploded version of OverlapComponentWithResult so we lock only once per chunk
		// TODO: Replace this by the proper API call once it is available
		TArray<Chaos::FPhysicsObjectHandle> Objects = Component->GetAllPhysicsObjects();
		FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(Objects);
		Objects = Objects.FilterByPredicate(
			[&Interface](Chaos::FPhysicsObjectHandle Handle)
			{
				return !Interface->AreAllDisabled({ &Handle, 1 });
			}
		);

		Chaos::FPhysicsObjectCollisionInterface_External CollisionInterface{ Interface.GetInterface() };

		for (int ShapeIndex : SampleIndices)
		{
			if (KeptSamples[ShapeIndex])
			{
				continue;
			}

			const FPhysicsGeometry& Geometry = CollisionShapeAdapters[ShapeIndex].GetGeometry();
			const FTransform& SampleTransform = Samples[ShapeIndex].Key;

			if (CollisionInterface.ShapeOverlap(Objects, Geometry, { SampleTransform.GetRotation(), SampleTransform.GetLocation() }, OverlapHits))
			{
				if (!OverlapHits.IsEmpty())
				{
					KeptSamples[ShapeIndex] = true;
				}

				OverlapHits.Reset();
			}
		}
	}

	// Finally, write back the data to the OutPoints
	for (int SampleIndex = 0; SampleIndex < Samples.Num(); ++SampleIndex)
	{
		FPCGPoint& OutPoint = OutPoints[SampleIndex];
		if (KeptSamples[SampleIndex])
		{
			new(&OutPoint) FPCGPoint(Samples[SampleIndex].Key, /*Density=*/1.0f, /*Seed=*/0);
			OutPoint.SetLocalBounds(Samples[SampleIndex].Value);
		}
	}
}

bool UPCGLandscapeData::ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	if (!LandscapeCache)
	{
		return false;
	}

	const ULandscapeInfo* LandscapeInfo = GetLandscapeInfo(InTransform.GetLocation());
	ALandscapeProxy* LandscapeProxy = LandscapeInfo ? LandscapeInfo->GetLandscapeProxy() : nullptr;
	if (!LandscapeProxy)
	{
		return false;
	}

	const FTransform LandscapeTransform = LandscapeProxy->LandscapeActorToWorld();

	// TODO: compute full transform when we want to support bounds
	const FVector LocalPoint = LandscapeTransform.InverseTransformPosition(InTransform.GetLocation());
	const FIntPoint ComponentMapKey(FMath::FloorToInt(LocalPoint.X / LandscapeInfo->ComponentSizeQuads), FMath::FloorToInt(LocalPoint.Y / LandscapeInfo->ComponentSizeQuads));

	TSharedPtr<const FPCGLandscapeCacheEntry> LandscapeCacheEntry = LandscapeCache->GetCacheEntry(LandscapeInfo, ComponentMapKey);

	if (!LandscapeCacheEntry)
	{
		return false;
	}

	const FVector2D ComponentLocalPoint(LocalPoint.X - ComponentMapKey.X * LandscapeInfo->ComponentSizeQuads, LocalPoint.Y - ComponentMapKey.Y * LandscapeInfo->ComponentSizeQuads);

	if (DataProps.bGetHeightOnly)
	{
		LandscapeCacheEntry->GetInterpolatedPointHeightOnly(ComponentLocalPoint, OutPoint, DataProps.bGetLayerWeights ? OutMetadata : nullptr);
	}
	else
	{
		LandscapeCacheEntry->GetInterpolatedPoint(ComponentLocalPoint, OutPoint, DataProps.bGetLayerWeights ? OutMetadata : nullptr);
	}

	// Not needed anymore
	LandscapeCacheEntry.Reset();

	ULandscapeHeightfieldCollisionComponent* LandscapeCollisionComponent = LandscapeInfo->XYtoCollisionComponentMap.FindRef(ComponentMapKey);

	if (DataProps.bGetActorReference && OutMetadata && LandscapeCollisionComponent)
	{
		if (FPCGMetadataAttribute<FSoftObjectPath>* ActorReferenceAttribute = OutMetadata->FindOrCreateAttribute<FSoftObjectPath>(PCGPointDataConstants::ActorReferenceAttribute))
		{
			// Landscape code seems to indicate the XYtoComponentMap can be sometimes invalid, so rely on the collision map instead
			OutMetadata->InitializeOnSet(OutPoint.MetadataEntry);
			ActorReferenceAttribute->SetValue(OutPoint.MetadataEntry, FSoftObjectPath(LandscapeCollisionComponent->GetOwner()));
		}
	}

	if (DataProps.bGetPhysicalMaterial && OutMetadata && LandscapeCollisionComponent)
	{
		if (FPCGMetadataAttribute<FSoftObjectPath>* PhysicalMaterialAttribute = OutMetadata->FindOrCreateAttribute<FSoftObjectPath>(PCGWorldQueryConstants::PhysicalMaterialReferenceAttribute))
		{
			if(UPhysicalMaterial* PhysicalMaterial = LandscapeCollisionComponent->GetPhysicalMaterial(static_cast<float>(ComponentLocalPoint.X), static_cast<float>(ComponentLocalPoint.Y), EHeightfieldSource::Complex))
			{
				OutMetadata->InitializeOnSet(OutPoint.MetadataEntry);
				PhysicalMaterialAttribute->SetValue(OutPoint.MetadataEntry, FSoftObjectPath(PhysicalMaterial));
			}
		}
	}

	if (DataProps.bGetComponentCoordinates && OutMetadata)
	{
		if(FPCGMetadataAttribute<int32>* ComponentXAttribute = OutMetadata->FindOrCreateAttribute<int32>(PCGLandscapeDataConstants::ComponentXAttribute))
		{
			OutMetadata->InitializeOnSet(OutPoint.MetadataEntry);
			ComponentXAttribute->SetValue(OutPoint.MetadataEntry, ComponentMapKey.X);
		}

		if (FPCGMetadataAttribute<int32>* ComponentYAttribute = OutMetadata->FindOrCreateAttribute<int32>(PCGLandscapeDataConstants::ComponentYAttribute))
		{
			OutMetadata->InitializeOnSet(OutPoint.MetadataEntry);
			ComponentYAttribute->SetValue(OutPoint.MetadataEntry, ComponentMapKey.Y);
		}
	}

	// Respect projection settings
	if (!InParams.bProjectPositions)
	{
		OutPoint.Transform.SetLocation(InTransform.GetLocation());
	}
		
	if (!InParams.bProjectRotations)
	{
		OutPoint.Transform.SetRotation(InTransform.GetRotation());
	}
	else
	{
		// Take landscape transform, but respect initial point yaw (don't spin points around Z axis).
		FVector RotVector = InTransform.GetRotation().ToRotationVector();
		RotVector.X = RotVector.Y = 0;
		OutPoint.Transform.SetRotation(OutPoint.Transform.GetRotation() * FQuat::MakeFromRotationVector(RotVector));
	}

	if (!InParams.bProjectScales)
	{
		OutPoint.Transform.SetScale3D(InTransform.GetScale3D());
	}

	return true;
}

const UPCGPointData* UPCGLandscapeData::CreatePointData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGLandscapeData::CreatePointData);

	return CastChecked<UPCGPointData>(CreateBasePointData(Context, InBounds, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGLandscapeData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGLandscapeData::CreatePointArrayData);

	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, InBounds, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGLandscapeData::CreateBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	if (LandscapeInfos.IsEmpty() || !LandscapeCache)
	{
		PCGLog::LogErrorOnGraph(NSLOCTEXT("PCGLandscapeData", "RequiredDataNotInitialized", "PCG Landscape cache or Landscape info are not initialized"), Context);
		return nullptr;
	}

	UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);

	FPCGInitializeFromDataParams InitializeFromDataParams(this);
	InitializeFromDataParams.bInheritSpatialData = false;
	Data->InitializeFromDataWithParams(InitializeFromDataParams);

	// @todo_pcg: try to optimize allocation of BoundsMin/BoundsMax as it currently depends on the Landscape components and should always be the same.
	EPCGPointNativeProperties PropertiesToAllocate = EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax;
	
	FBox EffectiveBounds = Bounds;
	if (InBounds.IsValid)
	{
		EffectiveBounds = Bounds.Overlap(InBounds);
	}

	// Early out
	if (!EffectiveBounds.IsValid)
	{
		return Data;
	}

	// @todo_pcg: for now this method does not write out all the selected attributes (see UseMetaData())
	UPCGMetadata* OutMetadata = DataProps.bGetLayerWeights ? Data->Metadata.Get() : nullptr;
	if (OutMetadata)
	{
		PropertiesToAllocate |= EPCGPointNativeProperties::MetadataEntry;
	}
		
	int32 NumPoints = 0;

	// Most proxies we gathered will have the same landscape info, we shouldn't loop multiple times
	// on them, unless we add the box filtering - but even then, depending on the transform we could have overlaps
	for (ULandscapeInfo* LandscapeInfo : LandscapeInfos)
	{
		ALandscapeProxy* LandscapeProxy = LandscapeInfo ? LandscapeInfo->GetLandscapeProxy() : nullptr;
		if (!LandscapeProxy)
		{
			continue;
		}

		const FTransform LandscapeTransform = LandscapeProxy->LandscapeActorToWorld();
		const int32 ComponentSizeQuads = LandscapeInfo->ComponentSizeQuads;

		// TODO: add offset to nearest edge, will have an impact if the grid size doesn't match the landscape size
		const FVector MinPt = LandscapeTransform.InverseTransformPosition(EffectiveBounds.Min);
		const FVector MaxPt = LandscapeTransform.InverseTransformPosition(EffectiveBounds.Max);

		// Note: the MaxX/Y here are inclusive, hence the floor & the +1 in the sizes
		const int32 MinX = FMath::CeilToInt(MinPt.X);
		const int32 MaxX = FMath::FloorToInt(MaxPt.X);
		const int32 MinY = FMath::CeilToInt(MinPt.Y);
		const int32 MaxY = FMath::FloorToInt(MaxPt.Y);

		//Early out if the bounds do not overlap any landscape vertices
		if (MaxX < MinX || MaxY < MinY)
		{
			continue;
		}

		const int64 PointCountUpperBound = (1 + MaxX - MinX) * (1 + MaxY - MinY);
		const int32 PointsBeforeNum = NumPoints;
		if (PointCountUpperBound > 0)
		{
			Data->SetNumPoints(PointsBeforeNum + PointCountUpperBound);
			Data->AllocateProperties(PropertiesToAllocate);
		}

		FPCGPointValueRanges OutRanges(Data, /*bAllocate=*/false);

		const int32 MinComponentX = MinX / ComponentSizeQuads;
		const int32 MaxComponentX = MaxX / ComponentSizeQuads;
		const int32 MinComponentY = MinY / ComponentSizeQuads;
		const int32 MaxComponentY = MaxY / ComponentSizeQuads;
		const FGuid LandscapeGuid = LandscapeProxy->GetOriginalLandscapeGuid();

		for (int32 ComponentX = MinComponentX; ComponentX <= MaxComponentX; ++ComponentX)
		{
			for (int32 ComponentY = MinComponentY; ComponentY <= MaxComponentY; ++ComponentY)
			{
				FIntPoint ComponentMapKey(ComponentX, ComponentY);

				const TSharedPtr<const FPCGLandscapeCacheEntry> LandscapeCacheEntry = LandscapeCache->GetCacheEntry(LandscapeInfo, ComponentMapKey);

				if (!LandscapeCacheEntry)
				{
					continue;
				}

				// Rebase our bounds in the component referential
				const int32 LocalMinX = FMath::Clamp(MinX - ComponentMapKey.X * ComponentSizeQuads, 0, ComponentSizeQuads - 1);
				const int32 LocalMaxX = FMath::Clamp(MaxX - ComponentMapKey.X * ComponentSizeQuads, 0, ComponentSizeQuads - 1);

				const int32 LocalMinY = FMath::Clamp(MinY - ComponentMapKey.Y * ComponentSizeQuads, 0, ComponentSizeQuads - 1);
				const int32 LocalMaxY = FMath::Clamp(MaxY - ComponentMapKey.Y * ComponentSizeQuads, 0, ComponentSizeQuads - 1);

				// We can't really copy data from the component points wholesale because the component points have an additional boundary point.
				// TODO: consider optimizing this, though it will impact the Sample then
				for (int32 LocalX = LocalMinX; LocalX <= LocalMaxX; ++LocalX)
				{
					for (int32 LocalY = LocalMinY; LocalY <= LocalMaxY; ++LocalY)
					{
						const int32 PointIndex = LocalX + LocalY * (ComponentSizeQuads + 1);

						FPCGPoint Point;
						if (DataProps.bGetHeightOnly)
						{
							LandscapeCacheEntry->GetPointHeightOnly(PointIndex, Point);
						}
						else
						{
							LandscapeCacheEntry->GetPoint(PointIndex, Point, OutMetadata);

							if (OutMetadata)
							{
								OutRanges.MetadataEntryRange[NumPoints] = Point.MetadataEntry;
							}
						}

						OutRanges.TransformRange[NumPoints] = Point.Transform;
						OutRanges.SeedRange[NumPoints] = Point.Seed;
						OutRanges.BoundsMinRange[NumPoints] = Point.BoundsMin;
						OutRanges.BoundsMaxRange[NumPoints] = Point.BoundsMax;
						++NumPoints;
					}
				}
			}
		}

		check(NumPoints - PointsBeforeNum <= PointCountUpperBound);
		UE_LOGF(LogPCG, Verbose, "Landscape %ls extracted %d of %lld potential points", *LandscapeProxy->GetFName().ToString(), NumPoints - PointsBeforeNum, PointCountUpperBound);
	}

	return Data;
}

bool UPCGLandscapeData::UseMetadata() const
{
	return DataProps.bGetActorReference || DataProps.bGetComponentCoordinates || DataProps.bGetLayerWeights || DataProps.bGetPhysicalMaterial;
}

const ULandscapeInfo* UPCGLandscapeData::GetLandscapeInfo(const FVector& InPosition) const
{
	check(!LandscapeInfos.IsEmpty());

	// Early out
	if (LandscapeInfos.Num() == 1)
	{
		return LandscapeInfos[0];
	}

	// As discussed in the header, this loop here is the reason why we do not really support overlapping landscapes.
	// TODO: we could maybe improve on this if we find the "nearest" landscape on a Z perspective, but this might still lead to issues
	for (const TPair<FBox, ULandscapeInfo*>& LandscapeInfoPair : BoundsToLandscapeInfos)
	{
		if (PCGHelpers::IsInsideBoundsXY(LandscapeInfoPair.Key, InPosition))
		{
			return LandscapeInfoPair.Value;
		}
	}

	return nullptr;
}

UPCGSpatialData* UPCGLandscapeData::CopyInternal(FPCGContext* Context) const
{
	UPCGLandscapeData* NewLandscapeData = FPCGContext::NewObject_AnyThread<UPCGLandscapeData>(Context);

	CopyBaseSurfaceData(NewLandscapeData);

	NewLandscapeData->Landscapes = Landscapes;
	NewLandscapeData->Bounds = Bounds;
	NewLandscapeData->DataProps = DataProps;
	NewLandscapeData->LandscapeInfos = LandscapeInfos;
	NewLandscapeData->BoundsToLandscapeInfos = BoundsToLandscapeInfos;
	NewLandscapeData->LandscapeCache = LandscapeCache;

	return NewLandscapeData;
}

TArray<FPCGTaskId> UPCGLandscapeData::PrepareForSpatialQuery(FPCGContext* InContext, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGLandscapeData::PrepareForSpatialQuery);
	if (!PCGSpatialData::CVarEnablePrepareForSpatialQuery.GetValueOnAnyThread())
	{
		return {};
	}

	if (!LandscapeCache)
	{
		return {};
	}

	if (LandscapeCache->SerializationMode == EPCGLandscapeCacheSerializationMode::NeverSerialize && PCGHelpers::IsRuntimeOrPIE())
	{
		// Hotfix note: commented this error has it could do false positives, esp. in the GPU use case.
		// @todo_pcg: use a warning here gated with a cvar instead.
		//PCGLog::Landscape::LogLandscapeCacheNotAvailableError(InContext);
		return {};
	}

	const ULandscapeInfo* LandscapeInfo = GetLandscapeInfo(InBounds.GetCenter());
	ALandscapeProxy* LandscapeProxy = LandscapeInfo ? LandscapeInfo->GetLandscapeProxy() : nullptr;
	if (!LandscapeProxy)
	{
		return {};
	}

	const FTransform LandscapeTransform = LandscapeProxy->LandscapeActorToWorld();

	FBox InverseTransformedBounds = InBounds.InverseTransformBy(LandscapeTransform);

	const FVector MinPoint = InverseTransformedBounds.Min;
	const FVector MaxPoint = InverseTransformedBounds.Max;

	const FIntPoint MinComponentMapKey(FMath::FloorToInt(MinPoint.X / LandscapeInfo->ComponentSizeQuads), FMath::FloorToInt(MinPoint.Y / LandscapeInfo->ComponentSizeQuads));
	const FIntPoint MaxComponentMapKey(FMath::FloorToInt(MaxPoint.X / LandscapeInfo->ComponentSizeQuads), FMath::FloorToInt(MaxPoint.Y / LandscapeInfo->ComponentSizeQuads));

	// Already cached
	if (LandscapeCache->AreCacheEntriesReady(LandscapeInfo, MinComponentMapKey, MaxComponentMapKey))
	{
		return {};
	}

	FPCGScheduleGenericParams Params([WeakThis = TWeakObjectPtr<const UPCGLandscapeData>(this), WeakLandscapeInfo = TWeakObjectPtr<const ULandscapeInfo>(LandscapeInfo), MinComponentMapKey, MaxComponentMapKey](FPCGContext* InContext)
	{
		const ULandscapeInfo* LandscapeInfo = WeakLandscapeInfo.Get();
		const UPCGLandscapeData* LandscapeData = WeakThis.Get();
		if (LandscapeInfo && LandscapeData)
		{
			LandscapeData->LandscapeCache->PrimeCache(LandscapeInfo, MinComponentMapKey, MaxComponentMapKey);
		}
				
		return true;
	},
	InContext->ExecutionSource.Get());

#if WITH_EDITOR
	// Creation of new cache entries is not thread safe because of FLandscapeComponentDataInterface 
	Params.bCanExecuteOnlyOnMainThread = true;
#else
	// Outside of editor we can only load existing entries which is thread safe
	Params.bCanExecuteOnlyOnMainThread = false;
#endif

	return { InContext->ScheduleGeneric(Params) };
}

void UPCGLandscapeData::InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const
{
	check(MetadataToInitialize);

	// Initialize the new metadata normally
	Super::InitializeTargetMetadata(InParams, MetadataToInitialize);

	if (!UseMetadata())
	{
		// If we have no extra metadata, early out.
		return;
	}

	// Only add those special attributes if the metadata to initialize support elements domain
	FPCGMetadataDomain* MetadataDomain = MetadataToInitialize->GetMetadataDomain(PCGMetadataDomainID::Elements);
	if (!MetadataDomain)
	{
		return;
	}

	// TODO: find a better way to do this - maybe there should be a prototype metadata in the landscape cache
	if (LandscapeCache)
	{
		if (DataProps.bGetLayerWeights)
		{
			FGCScopeGuard Guard;
			for (TSoftObjectPtr<ALandscapeProxy> Landscape : Landscapes)
			{
				const TArray<FName> Layers = LandscapeCache->GetLayerNames(Landscape.Get());

				for (const FName& Layer : Layers)
				{
					MetadataDomain->FindOrCreateAttribute<float>(Layer, 0.0f, /*bAllowInterpolation=*/true);
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogPCG, Error, "Landscape data is unable to access the landscape cache (Note: Will happen if there is no PCG world actor.");
	}

	// Create secondary attributes as we need them
	if (DataProps.bGetActorReference)
	{
		MetadataDomain->FindOrCreateAttribute<FSoftObjectPath>(PCGPointDataConstants::ActorReferenceAttribute, FSoftObjectPath(), /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	}

	if (DataProps.bGetPhysicalMaterial)
	{
		MetadataDomain->FindOrCreateAttribute<FSoftObjectPath>(PCGWorldQueryConstants::PhysicalMaterialReferenceAttribute, FSoftObjectPath(), /*bAllowInterpolation=*/false, /*bOverrideParent*/false);
	}

	if (DataProps.bGetComponentCoordinates)
	{
		MetadataDomain->FindOrCreateAttribute<int32>(PCGLandscapeDataConstants::ComponentXAttribute, 0, /*bAllowsInterpolation=*/false);
		MetadataDomain->FindOrCreateAttribute<int32>(PCGLandscapeDataConstants::ComponentYAttribute, 0, /*bAllowsInterpolation=*/false);
	}
}
