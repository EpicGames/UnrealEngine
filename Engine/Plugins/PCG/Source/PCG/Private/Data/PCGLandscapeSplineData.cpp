// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGLandscapeSplineData.h"

#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPolyLineData.h"
#include "Elements/PCGSplineSampler.h"
#include "Helpers/PCGHelpers.h"

#include "LandscapeSplineControlPoint.h"
#include "LandscapeSplineSegment.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGLandscapeSplineData)

namespace PCGLandscapeDataHelpers
{
	// This function assumes that the A-B segment has a "1" density, while the C-D segment has a "0" density
	FVector::FReal GetDensityInQuad(const FVector& A, const FVector& B, const FVector& C, const FVector& D, const FVector& P)
	{
		// Since the landscape has a strict Z behavior and our points might not be directly on the plane, we should only consider
		// the 2D plane. When we support other axes, we could just remove the normal component off the position to correct.
		const FVector::FReal Tolerance = UE_SMALL_NUMBER;
		FVector BaryABC = FMath::GetBaryCentric2D(P, A, B, C);

		if (BaryABC.X >= -Tolerance && BaryABC.Y >= -Tolerance && BaryABC.Z >= -Tolerance)
		{
			return 1.0f - FMath::Max(BaryABC.Z, 0);
		}

		FVector BaryACD = FMath::GetBaryCentric2D(P, A, C, D);

		if (BaryACD.X >= -Tolerance && BaryACD.Y >= -Tolerance && BaryACD.Z >= -Tolerance)
		{
			return FMath::Max(BaryACD.X, 0);
		}

		return FVector::FReal(-1);
	}
}

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoLandscapeSpline, UPCGLandscapeSplineData)

bool UPCGLandscapeSplineData::CheckSpline() const
{
	if (!Spline.IsValid())
	{
		if (!bLoggedInvalidSpline)
		{
			UE_LOGF(LogPCG, Warning, "Invalid landscape spline component");
			bLoggedInvalidSpline = true;
		}
		
		return false;
	}
	else
	{
		return true;
	}
}

void UPCGLandscapeSplineData::Initialize(ULandscapeSplinesComponent* InSplineComponent)
{
	check(InSplineComponent);
	Spline = InSplineComponent;

	UpdateReparamTable();
}

void UPCGLandscapeSplineData::PostLoad()
{
	Super::PostLoad();
	UpdateReparamTable();
}

void UPCGLandscapeSplineData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(Ar);
}

FTransform UPCGLandscapeSplineData::GetTransform() const
{
	return CheckSpline() ? Spline->GetComponentTransform() : FTransform::Identity;
}

int UPCGLandscapeSplineData::GetNumSegments() const
{
	return CheckSpline() ? Spline->GetSegments().Num() : 0;
}

FVector::FReal UPCGLandscapeSplineData::GetSegmentLength(int SegmentIndex) const
{
	return GetDistanceAtSegmentStart(SegmentIndex + 1) - GetDistanceAtSegmentStart(SegmentIndex);
}

FTransform UPCGLandscapeSplineData::GetTransformAtDistance(int SegmentIndex, FVector::FReal Distance, bool bWorldSpace, FBox* OutBounds) const
{
	if (!CheckSpline() || GetNumSegments() == 0)
	{
		return GetTransform();
	}

	const int ClampedSegmentIndex = FMath::Clamp(SegmentIndex, 0, GetNumSegments()-1);

	const ULandscapeSplineSegment* Segment = Spline->GetSegments()[ClampedSegmentIndex];
	check(Segment);

	const TArray<FLandscapeSplineInterpPoint>& InterpPoints = Segment->GetPoints();
	const int32 SecondToLastPointIndex = InterpPoints.Num() - 2;

	if (SecondToLastPointIndex < 0)
	{
		// Not enough points in the segment to interpolate a transform.
		return GetTransform();
	}

	int32 PointIndex = 0;
	FVector::FReal Alpha = 0.0f;

	if (ClampedSegmentIndex == SegmentIndex)
	{
		GetInterpPointAtDistance(ClampedSegmentIndex, Distance, PointIndex, /*bComputeAlpha=*/true, Alpha);

		// If our RightPoint ends up being on the next segment, clamp it back to the current segment.
		PointIndex = FMath::Clamp(PointIndex, 0, SecondToLastPointIndex);
	}
	else if(ClampedSegmentIndex < SegmentIndex)
	{
		// If we've landed beyond the last segment, just clamp to the end of the last segment.
		Alpha = 1.0f;
		PointIndex = SecondToLastPointIndex;
	}

	const FLandscapeSplineInterpPoint& PreviousPoint = InterpPoints[PointIndex];
	const FLandscapeSplineInterpPoint& CurrentPoint = InterpPoints[PointIndex + 1];

	const FVector XAxis = CurrentPoint.Center - PreviousPoint.Center;
	const FVector PreviousYAxis = PreviousPoint.Right - PreviousPoint.Center;
	const FVector CurrentYAxis = CurrentPoint.Right - CurrentPoint.Center;
	const FVector PreviousZAxis = (XAxis ^ PreviousYAxis).GetSafeNormal(UE_SMALL_NUMBER, FVector::ZAxisVector);
	const FVector CurrentZAxis = (XAxis ^ CurrentYAxis).GetSafeNormal(UE_SMALL_NUMBER, FVector::ZAxisVector);

	FTransform PreviousTransform = FTransform(XAxis, PreviousYAxis, PreviousZAxis, PreviousPoint.Center);
	FTransform CurrentTransform = FTransform(XAxis, CurrentYAxis, CurrentZAxis, CurrentPoint.Center);

	PreviousTransform.BlendWith(CurrentTransform, Alpha);

	if (OutBounds)
	{
		// Important note: the box here is going to be useful to be able to specify the relative sizes of the falloffs
		*OutBounds = FBox::BuildAABB(FVector::ZeroVector, FVector::OneVector);
		OutBounds->Min.Y *= (CurrentPoint.FalloffLeft - CurrentPoint.Center).Length() / (CurrentPoint.Left - CurrentPoint.Center).Length();
		OutBounds->Max.Y *= (CurrentPoint.FalloffRight - CurrentPoint.Center).Length() / (CurrentPoint.Right - CurrentPoint.Center).Length();
	}

	if (bWorldSpace)
	{
		PreviousTransform *= Spline->GetComponentTransform();
	}

	return PreviousTransform;
}

FVector::FReal UPCGLandscapeSplineData::GetCurvatureAtDistance(int SegmentIndex, FVector::FReal Distance) const
{
	if (!CheckSpline() || GetNumSegments() == 0)
	{
		return 0.0;
	}

	const int ClampedSegmentIndex = FMath::Clamp(SegmentIndex, 0, GetNumSegments()-1);

	const ULandscapeSplineSegment* Segment = Spline->GetSegments()[SegmentIndex];
	check(Segment);

	const TArray<FLandscapeSplineInterpPoint>& InterpPoints = Segment->GetPoints();

	// Need at least three points to compute the curvature.
	if (InterpPoints.Num() < 3)
	{
		return 0;
	}

	int32 PointIndex = 0;

	if (ClampedSegmentIndex == SegmentIndex)
	{
		FVector::FReal Alpha;
		GetInterpPointAtDistance(ClampedSegmentIndex, Distance, PointIndex, /*bComputeAlpha=*/false, Alpha);
	}
	else if (ClampedSegmentIndex < SegmentIndex)
	{
		PointIndex = FMath::Max(0, InterpPoints.Num() - 2);
	}

	// If our sample overshoots the segment, clamp it back to the last point.
	if (PointIndex == InterpPoints.Num() - 1)
	{
		--PointIndex;
	}

	// We don't need to clamp the current point index like we do in GetTransformAtDistance(), because we've already
	// decremented the PointIndex so that we can perform the backward 2nd derivative.
	const FLandscapeSplineInterpPoint& PreviousPoint = InterpPoints[PointIndex];
	const FLandscapeSplineInterpPoint& CurrentPoint = InterpPoints[PointIndex + 1];

	FVector FirstDerivative = FVector::ZeroVector;
	FVector SecondDerivative = FVector::ZeroVector;

	// Compute curvature using finite differences - here h is 1 because that's the only base unit we have.
	// Warning: Precision will be poor.
	// If last point -> use backward 2nd derivative.
	if (PointIndex >= InterpPoints.Num() - 2)
	{
		const FLandscapeSplineInterpPoint& PreviousPreviousPoint = InterpPoints[PointIndex - 1];

		// f'(x) = (f(x) - f(x-h)) / h
		FirstDerivative = (CurrentPoint.Center - PreviousPoint.Center);
		// f''(x) = (f(x) - 2f(x - h) + f(x - 2h)) / h2
		SecondDerivative = (CurrentPoint.Center - 2 * PreviousPoint.Center + PreviousPreviousPoint.Center);
	}
	// Otherwise, use central 2nd derivative.
	else
	{
		const FLandscapeSplineInterpPoint& NextPoint = InterpPoints[PointIndex + 2];

		// f'(x) ~= (f(x+h) - f(x-h)) / 2h
		FirstDerivative = (NextPoint.Center - PreviousPoint.Center) / 2.0;
		// f''(x) = (f(x+h) - 2f(x) + f(x-h)) / h2
		SecondDerivative = NextPoint.Center - 2 * CurrentPoint.Center + PreviousPoint.Center;
	}

	const FVector::FReal FirstDerivativeLength = FMath::Max(FirstDerivative.Length(), UE_DOUBLE_SMALL_NUMBER);
	const FVector ForwardVector = FirstDerivative / FirstDerivativeLength;
	const FVector CurvatureVector = SecondDerivative - (SecondDerivative | ForwardVector) * ForwardVector;
	const FVector::FReal Curvature = CurvatureVector.Length() / FirstDerivativeLength;

	return FMath::Sign(CurvatureVector | (CurrentPoint.Right - CurrentPoint.Center)) * Curvature;
}

float UPCGLandscapeSplineData::GetInputKeyAtDistance(int SegmentIndex, FVector::FReal Distance) const
{
	const float FullDistance = GetDistanceAtSegmentStart(SegmentIndex) + Distance;
	return ReparamTable.Eval(FullDistance, 0.0f);
}

void UPCGLandscapeSplineData::GetTangentsAtSegmentStart(int SegmentIndex, FVector& OutArriveTangent, FVector& OutLeaveTangent) const
{
	if (!CheckSpline())
	{
		return;
	}

	const TArray<TObjectPtr<ULandscapeSplineSegment>>& Segments = Spline->GetSegments();

	const TObjectPtr<ULandscapeSplineSegment> PreviousSegment = SegmentIndex > 0 && Segments.IsValidIndex(SegmentIndex - 1) ? Segments[SegmentIndex - 1] : nullptr;
	const TObjectPtr<ULandscapeSplineSegment> CurrentSegment = Segments.IsValidIndex(SegmentIndex) ? Segments[SegmentIndex] : nullptr;

	// Arrive tangent lives in the end-point of the previous segment
	const TObjectPtr<ULandscapeSplineControlPoint> ArrivePoint = PreviousSegment ? PreviousSegment->Connections[1].ControlPoint : nullptr;

	// Leave tangent lives in the start-point of the current segment
	const TObjectPtr<ULandscapeSplineControlPoint> LeavePoint = CurrentSegment ? CurrentSegment->Connections[0].ControlPoint : nullptr;

	OutArriveTangent = ArrivePoint ? ArrivePoint->Rotation.Vector() * -PreviousSegment->Connections[1].TangentLen : FVector::Zero();
	OutLeaveTangent = LeavePoint ? LeavePoint->Rotation.Vector() * CurrentSegment->Connections[0].TangentLen : FVector::Zero();
}

FVector::FReal UPCGLandscapeSplineData::GetDistanceAtSegmentStart(int SegmentIndex) const
{
	FVector::FReal Distance = 0.0;

	// Implementation note: It would be cheaper to loop over segments instead of ReparamPoints, but that is not robust since
	// InterpPoints on the segments may not be 1:1 with the ReparamPoints. This can happen for poorly formed Landscape Splines where
	// the adjacent control points are identical points.
	for (int32 ReparamIndex = 0; ReparamIndex < ReparamTable.Points.Num(); ++ReparamIndex)
	{
		const FVector::FReal CurrentDistance = ReparamTable.Points[ReparamIndex].InVal;

		if (ReparamTable.Points[ReparamIndex].OutVal > SegmentIndex)
		{
			break;
		}

		Distance = CurrentDistance;
	}

	return Distance;
}

FVector UPCGLandscapeSplineData::GetLocationAtAlpha(float Alpha) const
{
	const float InputKey = GetInputKeyAtAlpha(Alpha);
	const int SegmentIndex = FMath::TruncToInt(InputKey);
	return GetLocationAtDistance(SegmentIndex, FMath::Frac(InputKey) * GetSegmentLength(SegmentIndex));
}

FTransform UPCGLandscapeSplineData::GetTransformAtAlpha(float Alpha) const
{
	const float InputKey = GetInputKeyAtAlpha(Alpha);
	const int SegmentIndex = FMath::TruncToInt(InputKey);
	return GetTransformAtDistance(SegmentIndex, FMath::Frac(InputKey) * GetSegmentLength(SegmentIndex));
}

const UPCGPointData* UPCGLandscapeSplineData::CreatePointData(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGLandscapeSplineData::CreatePointData);

	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGLandscapeSplineData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGLandscapeSplineData::CreatePointArrayData);

	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGLandscapeSplineData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);

	FPCGInitializeFromDataParams InitializeFromDataParams(this);
	InitializeFromDataParams.bInheritSpatialData = false;
	Data->InitializeFromDataWithParams(InitializeFromDataParams);
	
	FPCGSplineSamplerParams SamplerParams;
	SamplerParams.Mode = EPCGSplineSamplingMode::Distance;
	SamplerParams.Dimension = EPCGSplineSamplingDimension::OnHorizontal;

	PCGSplineSamplerHelpers::SampleLineData(Context, /*LineData=*/this, /*InBoundingShapeData=*/nullptr, /*InProjectionTarget=*/nullptr, /*InProjectionParams=*/{}, SamplerParams, Data);

	if (CheckSpline())
	{
		UE_LOGF(LogPCG, Verbose, "Landscape spline %ls generated %d points", *Spline->GetFName().ToString(), Data->GetNumPoints());
	}

	return Data;
}

FBox UPCGLandscapeSplineData::GetBounds() const
{
	FBox Bounds(EForceInit::ForceInit);

	if (!CheckSpline())
	{		
		return Bounds;
	}

	for (const TObjectPtr<ULandscapeSplineSegment>& Segment : Spline->GetSegments())
	{
		Bounds += Segment->GetBounds();
	}

	if (Bounds.IsValid)
	{
		Bounds = Bounds.TransformBy(Spline->GetComponentToWorld());
	}

	return Bounds;
}

bool UPCGLandscapeSplineData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// TODO : add metadata support on poly lines
	// TODO : add support for bounds

	// This does not move the query point, but does not take into account the Z axis at all - so this is inherently a projection. There's some
	// things that need double checking but we should look at incorporating the Z axis into the calculation.
	if (!CheckSpline())
	{
		return false;
	}

	const ULandscapeSplinesComponent* SplinePtr = Spline.Get();

	OutPoint.Transform = InTransform;
	OutPoint.SetLocalBounds(InBounds); // TODO: should maybe do Min.Z = Max.Z = 0 ?

	const FVector Position = SplinePtr->GetComponentTransform().InverseTransformPosition(OutPoint.Transform.GetLocation());

	float PointDensity = 0.0f;

	for(const TObjectPtr<ULandscapeSplineSegment>& Segment : SplinePtr->GetSegments())
	{
		// Considering the landscape spline always exists on the landscape,
		// we'll ignore the Z component of the input here for the bounds check.
		if (!Segment->GetBounds().IsInsideOrOnXY(Position))
		{
			continue;
		}

		const TArray<FLandscapeSplineInterpPoint>& InterpPoints = Segment->GetPoints();

		float SegmentDensity = 0.0f;

		for (int PointIndex = 1; PointIndex < InterpPoints.Num(); ++PointIndex)
		{
			const FLandscapeSplineInterpPoint& Start = InterpPoints[PointIndex - 1];
			const FLandscapeSplineInterpPoint& End = InterpPoints[PointIndex];

			float Density = 0.0f;

			// Note: these checks have no prior information on the structure of the data, except that they form quads.
			// Considering that the points on a given control point are probably aligned, we could do an early check
			// in the original quad (start left falloff -> start right falloff -> end right falloff -> end left falloff)
			// TODO: this sequence here can be optimized knowing that some checks will be redundant.
			// Important note: the order and selection of points is important to the density computation. This assumes the first 2 points are the "1" density edge
			Density = FMath::Max(Density, PCGLandscapeDataHelpers::GetDensityInQuad(Start.Center, End.Center, End.Left, Start.Left, Position) >= 0 ? 1.0f : 0.0f);
			Density = FMath::Max(Density, PCGLandscapeDataHelpers::GetDensityInQuad(Start.Left, End.Left, End.FalloffLeft, Start.FalloffLeft, Position));
			Density = FMath::Max(Density, PCGLandscapeDataHelpers::GetDensityInQuad(End.Center, Start.Center, Start.Right, End.Right, Position) >= 0 ? 1.0f : 0.0f);
			Density = FMath::Max(Density, PCGLandscapeDataHelpers::GetDensityInQuad(End.Right, Start.Right, Start.FalloffRight, End.FalloffRight, Position));

			if (Density > SegmentDensity)
			{
				SegmentDensity = Density;
			}
		}

		if (SegmentDensity > PointDensity)
		{
			PointDensity = SegmentDensity;
		}
	}

	OutPoint.Density = PointDensity;
	return OutPoint.Density > 0;
}

UPCGSpatialData* UPCGLandscapeSplineData::CopyInternal(FPCGContext* Context) const
{
	UPCGLandscapeSplineData* NewLandscapeSplineData = FPCGContext::NewObject_AnyThread<UPCGLandscapeSplineData>(Context);

	NewLandscapeSplineData->Spline = Spline;
	NewLandscapeSplineData->ReparamTable = ReparamTable;

	return NewLandscapeSplineData;
}

void UPCGLandscapeSplineData::UpdateReparamTable()
{
	if (!CheckSpline())
	{
		return;
	}

	const TArray<TObjectPtr<ULandscapeSplineSegment>>& Segments = Spline->GetSegments();
	const int32 NumSegments = Segments.Num();

	FVector::FReal AccumulatedDistance = 0.0f;

	// Add a point for the first control point of the spline.
	ReparamTable.Points.Emplace(AccumulatedDistance, /*InputKey=*/0, /*ArriveTangent=*/0.0f, /*LeaveTangent=*/0.0f, CIM_Linear);

	// Create a curve mapping DistanceAlongSpline -> InputKey at that point. Accumulate the distance over each segment as we insert points into the ReparamTable.
	for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
	{
		const TObjectPtr<ULandscapeSplineSegment>& Segment = Segments[SegmentIndex];
		check(Segment);

		const TArray<FLandscapeSplineInterpPoint>& InterpPoints = Segment->GetPoints();
		const int32 NumPoints = InterpPoints.Num();

		// Add a point for each InterpPoint on the segment.
		for (int PointIndex = 1; PointIndex < NumPoints; ++PointIndex)
		{
			const FLandscapeSplineInterpPoint& Start = InterpPoints[PointIndex - 1];
			const FLandscapeSplineInterpPoint& End = InterpPoints[PointIndex];
			const FVector::FReal Distance = FVector::Distance(Start.Center, End.Center);

			// Skip points that overlap the previous point. We should not have duplicate distance entries in the ReparamTable.
			// Note: This means InterpPoints are not 1:1 with ReparamTable entries.
			if (FMath::IsNearlyZero(Distance, /*ErrorTolerance=*/UE_KINDA_SMALL_NUMBER))
			{
				continue;
			}

			AccumulatedDistance += Distance;

			const float Param = PointIndex / (NumPoints - 1.0f);
			ReparamTable.Points.Emplace(AccumulatedDistance, SegmentIndex + Param, /*ArriveTangent=*/0.0f, /*LeaveTangent=*/0.0f, CIM_Linear);
		}
	}
}

void UPCGLandscapeSplineData::GetInterpPointAtDistance(int SegmentIndex, FVector::FReal Distance, int32& OutPointIndex, bool bComputeAlpha, FVector::FReal& OutAlpha) const
{
	const FVector::FReal DistanceToSegment = GetDistanceAtSegmentStart(SegmentIndex);
	const FVector::FReal DistanceToSample = DistanceToSegment + Distance; // Total distance along the spline to the point we are sampling at.

	const int32 ReparamIndex = ReparamTable.GetPointIndexForInputValue(DistanceToSegment + Distance);
	check(ReparamTable.Points.IsValidIndex(ReparamIndex));

	// Find the index of the InterpPoint which begins the segment our desired transform lies on.
	// We can get this from the ReparamTable because it should contain one sample for each InterpPoint.
	const int32 SegmentReparamIndex = ReparamTable.GetPointIndexForInputValue(DistanceToSegment);
	OutPointIndex = ReparamIndex - SegmentReparamIndex;

	if (bComputeAlpha)
	{
		// Blend between LeftPoint and RightPoint by finding the ratio of distance to our sample vs length of the segment.
		// This ratio should be relative to the interp segment (which is formed by the line from LeftPoint to RightPoint),
		// since that is what we're blending on, not the entire spline segment.
		const FVector::FReal DistanceToPrevPoint = ReparamTable.Points[ReparamIndex].InVal;
		const FVector::FReal DistanceToNextPoint = ReparamTable.Points[FMath::Min(ReparamIndex + 1, ReparamTable.Points.Num() - 1)].InVal;
		const FVector::FReal InterpSegmentLength = DistanceToNextPoint - DistanceToPrevPoint;
		const FVector::FReal InterpSegmentDistanceToSample = DistanceToSample - DistanceToPrevPoint;

		OutAlpha = FMath::IsNearlyZero(InterpSegmentDistanceToSample) ? 0.0f : FMath::Clamp(InterpSegmentDistanceToSample / InterpSegmentLength, 0.0f, 1.0f);
	}
}
