// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGPointArrayData.h"

#include "PCGContext.h"

#include "Data/PCGPointData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPointArrayData)

TAutoConsoleVariable<bool> CVarPCGEnablePointArrayDataParenting(
	TEXT("pcg.EnablePointArrayDataParenting"),
	true,
	TEXT("Whether to enable inheritance of data on PointArrayData (memory savings)"));

void UPCGPointArrayData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(PointArray.GetSizeBytes());
}

void UPCGPointArrayData::VisitDataNetwork(TFunctionRef<void(const UPCGData*)> Action) const
{
	Super::VisitDataNetwork(Action);
	if (ParentData)
	{
		ParentData->VisitDataNetwork(Action);
	}
}

const UPCGPointData* UPCGPointArrayData::ToPointData(FPCGContext* Context, const FBox& InBounds) const
{
	UPCGPointData* PointData = FPCGContext::NewObject_AnyThread<UPCGPointData>(Context);
	PointData->InitializeFromData(this);

	SetPoints(this, PointData, {}, /*bCopyAll=*/true);

	return PointData;
}

UPCGSpatialData* UPCGPointArrayData::CopyInternal(FPCGContext* Context) const
{
	UPCGPointArrayData* NewPointData = FPCGContext::NewObject_AnyThread<UPCGPointArrayData>(Context);
	
	// If inheritance is supported we are going to inherit from this data in InitializeSpatialDataInternal
	if (!SupportsSpatialDataInheritance())
	{
		NewPointData->PointArray = PointArray;
	}

	return NewPointData;
}

void UPCGPointArrayData::CopyPropertiesTo(UPCGBasePointData* To, int32 ReadStartIndex, int32 WriteStartIndex, int32 Count, EPCGPointNativeProperties Properties) const
{
	if (Count <= 0)
	{
		return;
	}

	if (UPCGPointArrayData* PointArrayData = Cast<UPCGPointArrayData>(To))
	{
		PointArrayData->AllocateProperties(GetAllocatedProperties());

		if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::Transform))
		{
			GetProperty<EPCGPointNativeProperties::Transform>()->CopyTo(*PointArrayData->GetProperty<EPCGPointNativeProperties::Transform>(/*bWithInheritance=*/false), ReadStartIndex, WriteStartIndex, Count);
		}

		if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::Density))
		{
			GetProperty<EPCGPointNativeProperties::Density>()->CopyTo(*PointArrayData->GetProperty<EPCGPointNativeProperties::Density>(/*bWithInheritance=*/false), ReadStartIndex, WriteStartIndex, Count);
		}

		if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::BoundsMin))
		{
			GetProperty<EPCGPointNativeProperties::BoundsMin>()->CopyTo(*PointArrayData->GetProperty<EPCGPointNativeProperties::BoundsMin>(/*bWithInheritance=*/false), ReadStartIndex, WriteStartIndex, Count);
		}

		if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::BoundsMax))
		{
			GetProperty<EPCGPointNativeProperties::BoundsMax>()->CopyTo(*PointArrayData->GetProperty<EPCGPointNativeProperties::BoundsMax>(/*bWithInheritance=*/false), ReadStartIndex, WriteStartIndex, Count);
		}

		if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::Color))
		{
			GetProperty<EPCGPointNativeProperties::Color>()->CopyTo(*PointArrayData->GetProperty<EPCGPointNativeProperties::Color>(/*bWithInheritance=*/false), ReadStartIndex, WriteStartIndex, Count);
		}

		if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::Steepness))
		{
			GetProperty<EPCGPointNativeProperties::Steepness>()->CopyTo(*PointArrayData->GetProperty<EPCGPointNativeProperties::Steepness>(/*bWithInheritance=*/false), ReadStartIndex, WriteStartIndex, Count);
		}

		if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::Seed))
		{
			GetProperty<EPCGPointNativeProperties::Seed>()->CopyTo(*PointArrayData->GetProperty<EPCGPointNativeProperties::Seed>(/*bWithInheritance=*/false), ReadStartIndex, WriteStartIndex, Count);
		}

		if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::MetadataEntry))
		{
			GetProperty<EPCGPointNativeProperties::MetadataEntry>()->CopyTo(*PointArrayData->GetProperty<EPCGPointNativeProperties::MetadataEntry>(/*bWithInheritance=*/false), ReadStartIndex, WriteStartIndex, Count);
		}
	}
	else
	{
		Super::CopyPropertiesTo(To, ReadStartIndex, WriteStartIndex, Count, Properties);
	}
}

void UPCGPointArrayData::InitializeSpatialDataInternal(const FPCGInitializeFromDataParams& InParams)
{
	Super::InitializeSpatialDataInternal(InParams);
		
	if (const UPCGPointArrayData* SourceParentData = Cast<UPCGPointArrayData>(InParams.Source); InParams.bInheritSpatialData && SourceParentData && SupportsSpatialDataInheritance())
	{
		// Some nodes will call DuplicateData and then call InitializeFromData so it is possible we already have set the parent
		check(ParentData == nullptr || ParentData == InParams.Source);

		if (ParentData != InParams.Source)
		{
			SetNumPoints(SourceParentData->GetNumPoints());
			InheritedProperties = static_cast<uint32>(EPCGPointNativeProperties::All);
			ParentData = const_cast<UPCGPointArrayData*>(SourceParentData);
		}
	}
}

EPCGPointNativeProperties UPCGPointArrayData::GetAllocatedProperties(bool bWithInheritance) const
{
	EPCGPointNativeProperties AllocatedProperties = PointArray.GetAllocatedProperties();
	if (bWithInheritance && ParentData)
	{
		AllocatedProperties |= ParentData->GetAllocatedProperties(bWithInheritance);
	}
	return AllocatedProperties;
}

bool UPCGPointArrayData::SupportsSpatialDataInheritance() const
{
	return CVarPCGEnablePointArrayDataParenting.GetValueOnAnyThread();
}

void UPCGPointArrayData::Flatten()
{
	Super::Flatten();

	FlattenPropertiesIfNeeded();
	
	check(!ParentData);
}

void UPCGPointArrayData::FlattenPropertiesIfNeeded(EPCGPointNativeProperties Properties)
{	
	if (EnumHasAnyFlags(Properties, EPCGPointNativeProperties::Transform))
	{
		FlattenPropertyIfNeeded<EPCGPointNativeProperties::Transform>();
	}
	
	if (EnumHasAnyFlags(Properties, EPCGPointNativeProperties::Steepness))
	{
		FlattenPropertyIfNeeded<EPCGPointNativeProperties::Steepness>();
	}

	if (EnumHasAnyFlags(Properties, EPCGPointNativeProperties::BoundsMin))
	{
		FlattenPropertyIfNeeded<EPCGPointNativeProperties::BoundsMin>();
	}

	if (EnumHasAnyFlags(Properties, EPCGPointNativeProperties::BoundsMax))
	{
		FlattenPropertyIfNeeded<EPCGPointNativeProperties::BoundsMax>();
	}

	if (EnumHasAnyFlags(Properties, EPCGPointNativeProperties::Color))
	{
		FlattenPropertyIfNeeded<EPCGPointNativeProperties::Color>();
	}

	if (EnumHasAnyFlags(Properties, EPCGPointNativeProperties::Density))
	{
		FlattenPropertyIfNeeded<EPCGPointNativeProperties::Density>();
	}

	if (EnumHasAnyFlags(Properties, EPCGPointNativeProperties::Seed))
	{
		FlattenPropertyIfNeeded<EPCGPointNativeProperties::Seed>();
	}

	if (EnumHasAnyFlags(Properties, EPCGPointNativeProperties::MetadataEntry))
	{
		FlattenPropertyIfNeeded<EPCGPointNativeProperties::MetadataEntry>();
	}
}

void UPCGPointArrayData::SetNumPoints(int32 InNumPoints, bool bInitializeValues)
{
	if (ParentData && ParentData->GetNumPoints() != InNumPoints)
	{
		FlattenPropertiesIfNeeded();
	}

	if (InNumPoints != PointArray.GetNumPoints())
	{
		PointArray.SetNumPoints(InNumPoints, bInitializeValues);
		DirtyCache();
	}
}

void UPCGPointArrayData::RemoveAt(int32 Index)
{
	if(Index >= 0 && Index < PointArray.GetNumPoints())
	{
		FlattenPropertiesIfNeeded();
		PointArray.RemoveAt(Index);
		DirtyCache();
	}
}

void UPCGPointArrayData::AllocateProperties(EPCGPointNativeProperties Properties)
{
	FlattenPropertiesIfNeeded(Properties);
	PointArray.Allocate(Properties);
}

void UPCGPointArrayData::FreeProperties(EPCGPointNativeProperties Properties)
{
	FlattenPropertiesIfNeeded(Properties);
	PointArray.Free(Properties);
}

void UPCGPointArrayData::MoveRange(int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
{
	FlattenPropertiesIfNeeded();
	PointArray.MoveRange(RangeStartIndex, MoveToIndex, NumElements);
}

void UPCGPointArrayData::CopyUnallocatedPropertiesFrom(const UPCGBasePointData* InPointData)
{
	if (HasSpatialDataParent())
	{
		return;
	}

	if (const UPCGPointArrayData* InPointArrayData = Cast<UPCGPointArrayData>(InPointData))
	{
		InPointArrayData->GetProperty<EPCGPointNativeProperties::Transform>()->CopyUnallocatedProperty(*GetProperty<EPCGPointNativeProperties::Transform>(/*bWithInheritance=*/false));
		InPointArrayData->GetProperty<EPCGPointNativeProperties::Density>()->CopyUnallocatedProperty(*GetProperty<EPCGPointNativeProperties::Density>(/*bWithInheritance=*/false));
		InPointArrayData->GetProperty<EPCGPointNativeProperties::BoundsMin>()->CopyUnallocatedProperty(*GetProperty<EPCGPointNativeProperties::BoundsMin>(/*bWithInheritance=*/false));
		InPointArrayData->GetProperty<EPCGPointNativeProperties::BoundsMax>()->CopyUnallocatedProperty(*GetProperty<EPCGPointNativeProperties::BoundsMax>(/*bWithInheritance=*/false));
		InPointArrayData->GetProperty<EPCGPointNativeProperties::Color>()->CopyUnallocatedProperty(*GetProperty<EPCGPointNativeProperties::Color>(/*bWithInheritance=*/false));
		InPointArrayData->GetProperty<EPCGPointNativeProperties::Steepness>()->CopyUnallocatedProperty(*GetProperty<EPCGPointNativeProperties::Steepness>(/*bWithInheritance=*/false));
		InPointArrayData->GetProperty<EPCGPointNativeProperties::Seed>()->CopyUnallocatedProperty(*GetProperty<EPCGPointNativeProperties::Seed>(/*bWithInheritance=*/false));
		InPointArrayData->GetProperty<EPCGPointNativeProperties::MetadataEntry>()->CopyUnallocatedProperty(*GetProperty<EPCGPointNativeProperties::MetadataEntry>(/*bWithInheritance=*/false));
	}
}

TArray<FTransform> UPCGPointArrayData::GetTransformsCopy() const
{
	TArray<FTransform> TransformsCopy;
	const TConstPCGValueRange<FTransform> ReadValueRange = GetConstTransformValueRange();
	
	TransformsCopy.Reserve(ReadValueRange.Num());
	Algo::Transform(ReadValueRange, TransformsCopy, [](const FTransform& Transform) { return Transform; });

	return TransformsCopy;
}

FPCGPointTransform::ValueRange UPCGPointArrayData::GetTransformValueRange(bool bAllocate)
{ 
	FlattenPropertyIfNeeded<EPCGPointNativeProperties::Transform>();
	return PointArray.GetTransformValueRange(bAllocate); 
}

FPCGPointDensity::ValueRange UPCGPointArrayData::GetDensityValueRange(bool bAllocate)
{ 
	FlattenPropertyIfNeeded<EPCGPointNativeProperties::Density>();
	return PointArray.GetDensityValueRange(bAllocate); 
}

FPCGPointBoundsMin::ValueRange UPCGPointArrayData::GetBoundsMinValueRange(bool bAllocate)
{ 
	FlattenPropertyIfNeeded<EPCGPointNativeProperties::BoundsMin>();
	return PointArray.GetBoundsMinValueRange(bAllocate); 
}

FPCGPointBoundsMax::ValueRange UPCGPointArrayData::GetBoundsMaxValueRange(bool bAllocate)
{
	FlattenPropertyIfNeeded<EPCGPointNativeProperties::BoundsMax>();
	return PointArray.GetBoundsMaxValueRange(bAllocate); 
}

FPCGPointColor::ValueRange UPCGPointArrayData::GetColorValueRange(bool bAllocate)
{ 
	FlattenPropertyIfNeeded<EPCGPointNativeProperties::Color>();
	return PointArray.GetColorValueRange(bAllocate); 
}

FPCGPointSteepness::ValueRange UPCGPointArrayData::GetSteepnessValueRange(bool bAllocate)
{ 
	FlattenPropertyIfNeeded<EPCGPointNativeProperties::Steepness>();
	return PointArray.GetSteepnessValueRange(bAllocate); 
}

FPCGPointSeed::ValueRange UPCGPointArrayData::GetSeedValueRange(bool bAllocate)
{ 
	FlattenPropertyIfNeeded<EPCGPointNativeProperties::Seed>();
	return PointArray.GetSeedValueRange(bAllocate); 
}

FPCGPointMetadataEntry::ValueRange UPCGPointArrayData::GetMetadataEntryValueRange(bool bAllocate)
{ 
	FlattenPropertyIfNeeded<EPCGPointNativeProperties::MetadataEntry>();
	return PointArray.GetMetadataEntryValueRange(bAllocate); 
}

FPCGPointTransform::ConstValueRange UPCGPointArrayData::GetConstTransformValueRange() const
{ 
	return GetProperty<EPCGPointNativeProperties::Transform>()->GetConstValueRange();
}

FPCGPointDensity::ConstValueRange UPCGPointArrayData::GetConstDensityValueRange() const
{ 
	return GetProperty<EPCGPointNativeProperties::Density>()->GetConstValueRange();
}

FPCGPointBoundsMin::ConstValueRange UPCGPointArrayData::GetConstBoundsMinValueRange() const
{ 
	return GetProperty<EPCGPointNativeProperties::BoundsMin>()->GetConstValueRange();
}

FPCGPointBoundsMax::ConstValueRange UPCGPointArrayData::GetConstBoundsMaxValueRange() const
{ 
	return GetProperty<EPCGPointNativeProperties::BoundsMax>()->GetConstValueRange();
}

FPCGPointColor::ConstValueRange UPCGPointArrayData::GetConstColorValueRange() const
{ 
	return GetProperty<EPCGPointNativeProperties::Color>()->GetConstValueRange();
}

FPCGPointSteepness::ConstValueRange UPCGPointArrayData::GetConstSteepnessValueRange() const
{ 
	return GetProperty<EPCGPointNativeProperties::Steepness>()->GetConstValueRange();
}

FPCGPointSeed::ConstValueRange UPCGPointArrayData::GetConstSeedValueRange() const
{ 
	return GetProperty<EPCGPointNativeProperties::Seed>()->GetConstValueRange();
}

FPCGPointMetadataEntry::ConstValueRange UPCGPointArrayData::GetConstMetadataEntryValueRange() const
{ 
	return GetProperty<EPCGPointNativeProperties::MetadataEntry>()->GetConstValueRange();
}
