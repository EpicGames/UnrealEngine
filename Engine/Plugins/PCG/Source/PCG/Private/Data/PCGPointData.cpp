// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGPointData.h"

#include "PCGContext.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGTagHelpers.h"
#include "Metadata/PCGMetadataAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Algo/Transform.h"
#include "Containers/StridedView.h"
#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPointData)

#define PCG_POINTDATA_MAKERANGE_BASE(RangeType, MakeViewType, Property, Field) RangeType<typename TPCGPointNativeProperty<Property>::Type>(MakeViewType(sizeof(FPCGPoint), Points.IsEmpty() ? nullptr : &Points[0].Field, Points.Num()));

#define PCG_POINTDATA_MAKERANGE(Property, Field) PCG_POINTDATA_MAKERANGE_BASE(TPCGValueRange, MakeStridedView, Property, Field)
#define PCG_POINTDATA_MAKECONSTRANGE(Property, Field) PCG_POINTDATA_MAKERANGE_BASE(TConstPCGValueRange, MakeConstStridedView, Property, Field)

namespace PCGPointHelpers
{
	void Lerp(const FPCGPoint& A, const FPCGPoint& B, float Ratio, const UPCGMetadata* SourceMetadata, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata)
	{
		check(Ratio >= 0 && Ratio <= 1.0f);
		// TODO: this might be incorrect. See UKismetMathLibrary::TLerp instead
		OutPoint.Transform = FTransform(
			FMath::Lerp(A.Transform.GetRotation(), B.Transform.GetRotation(), Ratio),
			FMath::Lerp(A.Transform.GetLocation(), B.Transform.GetLocation(), Ratio),
			FMath::Lerp(A.Transform.GetScale3D(), B.Transform.GetScale3D(), Ratio));
		OutPoint.Density = FMath::Lerp(A.Density, B.Density, Ratio);
		OutPoint.BoundsMin = FMath::Lerp(A.BoundsMin, B.BoundsMin, Ratio);
		OutPoint.BoundsMax = FMath::Lerp(A.BoundsMax, B.BoundsMax, Ratio);
		OutPoint.Color = FMath::Lerp(A.Color, B.Color, Ratio);
		OutPoint.Steepness = FMath::Lerp(A.Steepness, B.Steepness, Ratio);

		if (OutMetadata && SourceMetadata && SourceMetadata->GetAttributeCount() > 0)
		{
			UPCGMetadataAccessorHelpers::InitializeMetadataWithParent(OutPoint, OutMetadata, ((Ratio <= 0.5f) ? A : B), SourceMetadata);

			TArray<TPair<const FPCGPoint*, float>, TInlineAllocator<2>> WeightedPoints;
			WeightedPoints.Emplace(&A, Ratio);
			WeightedPoints.Emplace(&B, 1.0f - Ratio);

			OutMetadata->ComputePointWeightedAttribute(OutPoint, MakeArrayView(WeightedPoints), SourceMetadata);
		}
	}

	void BilerpWithSnapping(const FPCGPoint& X0Y0, const FPCGPoint& X1Y0, const FPCGPoint& X0Y1, const FPCGPoint& X1Y1, const UPCGMetadata* SourceMetadata, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata, float XFactor, float YFactor)
	{
		const bool bIsOnLeftEdge = (XFactor < KINDA_SMALL_NUMBER);
		const bool bIsOnRightEdge = (XFactor > 1.0f - KINDA_SMALL_NUMBER);
		const bool bIsOnTopEdge = (YFactor < KINDA_SMALL_NUMBER);
		const bool bIsOnBottomEdge = (YFactor > 1.0f - KINDA_SMALL_NUMBER);

		auto CopyPoint = [&OutPoint, &OutMetadata, &SourceMetadata](const FPCGPoint& PointToCopy)
		{
			PCGMetadataEntryKey OutPointEntryKey = OutPoint.MetadataEntry;
			OutPoint = PointToCopy;
			OutPoint.MetadataEntry = OutPointEntryKey;

			if (OutMetadata)
			{
				OutMetadata->SetPointAttributes(PointToCopy, SourceMetadata, OutPoint);
			}
		};

		if (bIsOnLeftEdge || bIsOnRightEdge || bIsOnTopEdge || bIsOnBottomEdge)
		{
			if (bIsOnLeftEdge)
			{
				if (bIsOnTopEdge)
				{
					CopyPoint(X0Y0);
				}
				else if (bIsOnBottomEdge)
				{
					CopyPoint(X0Y1);
				}
				else
				{
					Lerp(X0Y0, X0Y1, YFactor, SourceMetadata, OutPoint, OutMetadata);
				}
			}
			else if (bIsOnRightEdge)
			{
				if (bIsOnTopEdge)
				{
					CopyPoint(X1Y0);
				}
				else if (bIsOnBottomEdge)
				{
					CopyPoint(X1Y1);
				}
				else
				{
					Lerp(X1Y0, X1Y1, YFactor, SourceMetadata, OutPoint, OutMetadata);
				}
			}
			else if (bIsOnTopEdge)
			{
				Lerp(X0Y0, X1Y0, XFactor, SourceMetadata, OutPoint, OutMetadata);
			}
			else // bIsOnBottomEdge
			{
				Lerp(X0Y1, X1Y1, XFactor, SourceMetadata, OutPoint, OutMetadata);
			}
		}
		else
		{
			Bilerp(X0Y0, X1Y0, X0Y1, X1Y1, SourceMetadata, OutPoint, OutMetadata, XFactor, YFactor);
		}
	}

	void Bilerp(const FPCGPoint& X0Y0, const FPCGPoint& X1Y0, const FPCGPoint& X0Y1, const FPCGPoint& X1Y1, const UPCGMetadata* SourceMetadata, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata, float XFactor, float YFactor)
	{
		// Interpolate X0Y0-X1Y0 and X0Y1-X1Y1 using XFactor
		FPCGPoint Y0Lerp;
		FPCGPoint Y1Lerp;

		Lerp(X0Y0, X1Y0, XFactor, SourceMetadata, Y0Lerp, OutMetadata);
		Lerp(X0Y1, X1Y1, XFactor, SourceMetadata, Y1Lerp, OutMetadata);
		// Interpolate between the two points using YFactor
		Lerp(Y0Lerp, Y1Lerp, YFactor, SourceMetadata, OutPoint, OutMetadata);
	}
}

FPCGPointRef::FPCGPointRef(const FPCGPoint& InPoint)
{
	Point = &InPoint;
	Bounds = InPoint.GetDensityBounds();
}

FPCGPointRef::FPCGPointRef(const FPCGPoint& InPoint, const FBox& InOverrideBounds)
{
	Point = &InPoint;
	Bounds = FBoxSphereBounds(InOverrideBounds.TransformBy(InPoint.Transform));
}

void UPCGPointData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Points.GetAllocatedSize() + Octree.GetSizeBytes());
}

TArray<FPCGPoint>& UPCGPointData::GetMutablePoints()
{
	DirtyCache();
	return Points;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const UPCGPointData::PointOctree& UPCGPointData::GetOctree() const
{
	if (bOctreeOldIsDirty)
	{
		RebuildOctreeOld();
	}

	return Octree;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TUniquePtr<IPCGAttributeAccessor> UPCGPointData::CreateStaticAccessor(const FPCGAttributePropertySelector& InSelector, bool bQuiet)
{
	TUniquePtr<IPCGAttributeAccessor> Accessor;
	
	if (InSelector.GetSelection() == EPCGAttributePropertySelection::Property)
	{
		const FName PropertyName = InSelector.GetName();
		if (const FProperty* Property = FPCGPoint::StaticStruct()->FindPropertyByName(PropertyName))
		{
			Accessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(Property);
		}
		else if (FPCGPoint::HasCustomPropertyGetterSetter(PropertyName))
		{
			Accessor = FPCGPoint::CreateCustomPropertyAccessor(PropertyName);
		}
	}
	
	return Accessor;
}

FPCGAttributeAccessorMethods UPCGPointData::GetPointAccessorMethods()
{
	auto CreateAccessorFunc = [](UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet)
	{
		return CreateStaticAccessor(InSelector, bQuiet);
	};

	auto CreateConstAccessorFunc = [](const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<const IPCGAttributeAccessor>
	{
		return CreateStaticAccessor(InSelector, bQuiet);
	};

	auto CreateAccessorKeysFunc = [](UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<IPCGAttributeAccessorKeys>
	{
		const FPCGMetadataDomainID DomainID = InData->GetMetadataDomainIDFromSelector(InSelector);
		if (DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements)
		{
			UPCGPointData* PointData = CastChecked<UPCGPointData>(InData);
			TArrayView<FPCGPoint> View(PointData->GetMutablePoints());
			return MakeUnique<FPCGAttributeAccessorKeysPoints>(View);
		}
		else
		{
			return {};
		}
	};
	
	auto CreateConstAccessorKeysFunc = [](const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<const IPCGAttributeAccessorKeys>
	{
		const FPCGMetadataDomainID DomainID = InData->GetMetadataDomainIDFromSelector(InSelector);
		if (DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements)
		{
			const UPCGPointData* PointData = CastChecked<UPCGPointData>(InData);
			return MakeUnique<const FPCGAttributeAccessorKeysPoints>(PointData->GetPoints());
		}
		else
		{
			return {};
		}
	};
	
	FPCGAttributeAccessorMethods Methods
	{
		.CreateAccessorFunc = CreateAccessorFunc,
		.CreateConstAccessorFunc = CreateConstAccessorFunc,
		.CreateAccessorKeysFunc = CreateAccessorKeysFunc,
		.CreateConstAccessorKeysFunc = CreateConstAccessorKeysFunc
	};

	return Methods;
}

TArray<FTransform> UPCGPointData::GetTransformsCopy() const
{
	TArray<FTransform> Transforms;
	Transforms.Reserve(Points.Num());
	Algo::Transform(Points, Transforms, [](const FPCGPoint& Point) { return Point.Transform; });
	return Transforms;
}

FPCGPointTransform::ValueRange UPCGPointData::GetTransformValueRange(bool bAllocate)
{
	return PCG_POINTDATA_MAKERANGE(EPCGPointNativeProperties::Transform, Transform);
}

FPCGPointDensity::ValueRange UPCGPointData::GetDensityValueRange(bool bAllocate)
{
	return PCG_POINTDATA_MAKERANGE(EPCGPointNativeProperties::Density, Density);
}

FPCGPointBoundsMin::ValueRange UPCGPointData::GetBoundsMinValueRange(bool bAllocate)
{
	return PCG_POINTDATA_MAKERANGE(EPCGPointNativeProperties::BoundsMin, BoundsMin);
}

FPCGPointBoundsMax::ValueRange UPCGPointData::GetBoundsMaxValueRange(bool bAllocate)
{
	return PCG_POINTDATA_MAKERANGE(EPCGPointNativeProperties::BoundsMax, BoundsMax);
}

FPCGPointColor::ValueRange UPCGPointData::GetColorValueRange(bool bAllocate)
{
	return PCG_POINTDATA_MAKERANGE(EPCGPointNativeProperties::Color, Color);
}

FPCGPointSteepness::ValueRange UPCGPointData::GetSteepnessValueRange(bool bAllocate)
{
	return PCG_POINTDATA_MAKERANGE(EPCGPointNativeProperties::Steepness, Steepness);
}

FPCGPointSeed::ValueRange UPCGPointData::GetSeedValueRange(bool bAllocate)
{
	return PCG_POINTDATA_MAKERANGE(EPCGPointNativeProperties::Seed, Seed);
}

FPCGPointMetadataEntry::ValueRange UPCGPointData::GetMetadataEntryValueRange(bool bAllocate)
{
	return PCG_POINTDATA_MAKERANGE(EPCGPointNativeProperties::MetadataEntry, MetadataEntry);
}

FPCGPointTransform::ConstValueRange UPCGPointData::GetConstTransformValueRange() const
{
	return PCG_POINTDATA_MAKECONSTRANGE(EPCGPointNativeProperties::Transform, Transform);
}

FPCGPointDensity::ConstValueRange UPCGPointData::GetConstDensityValueRange() const
{
	return PCG_POINTDATA_MAKECONSTRANGE(EPCGPointNativeProperties::Density, Density);
}

FPCGPointBoundsMin::ConstValueRange UPCGPointData::GetConstBoundsMinValueRange() const
{
	return PCG_POINTDATA_MAKECONSTRANGE(EPCGPointNativeProperties::BoundsMin, BoundsMin);
}

FPCGPointBoundsMax::ConstValueRange UPCGPointData::GetConstBoundsMaxValueRange() const
{
	return PCG_POINTDATA_MAKECONSTRANGE(EPCGPointNativeProperties::BoundsMax, BoundsMax);
}

FPCGPointColor::ConstValueRange UPCGPointData::GetConstColorValueRange() const
{
	return PCG_POINTDATA_MAKECONSTRANGE(EPCGPointNativeProperties::Color, Color);
}

FPCGPointSteepness::ConstValueRange UPCGPointData::GetConstSteepnessValueRange() const
{
	return PCG_POINTDATA_MAKECONSTRANGE(EPCGPointNativeProperties::Steepness, Steepness);
}

FPCGPointSeed::ConstValueRange UPCGPointData::GetConstSeedValueRange() const
{
	return PCG_POINTDATA_MAKECONSTRANGE(EPCGPointNativeProperties::Seed, Seed);
}

FPCGPointMetadataEntry::ConstValueRange UPCGPointData::GetConstMetadataEntryValueRange() const
{
	return PCG_POINTDATA_MAKECONSTRANGE(EPCGPointNativeProperties::MetadataEntry, MetadataEntry);
}

void UPCGPointData::SetPoints(const TArray<FPCGPoint>& InPoints)
{
	GetMutablePoints() = InPoints;
}

void UPCGPointData::SetNumPoints(int32 InNumPoints, bool bInitializeValues)
{ 
	if (Points.Num() != InNumPoints)
	{
		if (bInitializeValues)
		{
			Points.SetNum(InNumPoints);
		}
		else
		{
			Points.SetNumUninitialized(InNumPoints);
		}
		DirtyCache();
	}
}

FPCGPoint UPCGPointData::GetPoint(int32 Index) const
{
	// This check has an overhead but this method is blueprint callable and we want to avoid crashing
	if (Points.IsValidIndex(Index))
	{
		return Points[Index];
	}
	else
	{
		UE_LOGF(LogPCG, Error, "Invalid index in GetPoint call");
		return FPCGPoint();
	}
}

void UPCGPointData::RebuildOctreeOld() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PCG::TScopeLock Lock(CachedDataLock);

	if (!bOctreeOldIsDirty)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGPointData::RebuildOctreeOld)
	check(bOctreeOldIsDirty);

	FBox PointBounds = GetBounds();
	const bool bHasInvalidBounds = PointBounds.ContainsNaN();

	if (bHasInvalidBounds)
	{
		UE_LOGF(LogPCG, Error, "Invalid bounds in the point data makes it impossible to create a valid Octree.");
		PointBounds = FBox(EForceInit::ForceInit);
	}

	TOctree2<FPCGPointRef, FPCGPointRefSemantics> NewOctree(PointBounds.GetCenter(), PointBounds.GetExtent().Length());

	if (!bHasInvalidBounds)
	{
		for (const FPCGPoint& Point : Points)
		{
			NewOctree.AddElement(FPCGPointRef(Point));
		}
	}

	Octree = MoveTemp(NewOctree);
	bOctreeOldIsDirty = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UPCGSpatialData* UPCGPointData::CopyInternal(FPCGContext* Context) const
{
	UPCGPointData* NewPointData = FPCGContext::NewObject_AnyThread<UPCGPointData>(Context);
	NewPointData->GetMutablePoints() = GetPoints();

	return NewPointData;
}

void UPCGPointData::CopyPointsTo(UPCGBasePointData* OutData, int32 ReadStartIndex, int32 WriteStartIndex, int32 Count) const
{
	if (Count <= 0)
	{
		return;
	}

	if (UPCGPointData* OutPointData = Cast<UPCGPointData>(OutData))
	{
		std::memcpy(&OutPointData->GetMutablePoints()[WriteStartIndex], &Points[ReadStartIndex], sizeof(FPCGPoint) * Count);
	}
	else
	{
		Super::CopyPointsTo(OutData, ReadStartIndex, WriteStartIndex, Count);
	}
}

void UPCGPointData::MoveRange(int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
{
	if (RangeStartIndex != MoveToIndex && NumElements > 0)
	{
		check(Points.IsValidIndex(MoveToIndex) && Points.IsValidIndex(MoveToIndex + NumElements - 1) && Points.IsValidIndex(RangeStartIndex) && Points.IsValidIndex(RangeStartIndex + NumElements - 1));
		std::memmove(&Points[MoveToIndex], &Points[RangeStartIndex], sizeof(FPCGPoint) * NumElements);
	}
}

#undef PCG_POINTDATA_MAKERANGE_BASE
#undef PCG_POINTDATA_MAKERANGE
#undef PCG_POINTDATA_MAKECONSTRANGE