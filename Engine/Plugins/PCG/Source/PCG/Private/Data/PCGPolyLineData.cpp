// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGPolyLineData.h"
#include "Data/PCGSpatialData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPolyLineData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoPolyline, UPCGPolyLineData)

FBox UPCGPolyLineData::GetBounds() const
{
	FBox Bounds(EForceInit::ForceInit);

	const int NumSegments = GetNumSegments();

	for (int SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
	{
		Bounds += GetLocationAtDistance(SegmentIndex, 0);
		Bounds += GetLocationAtDistance(SegmentIndex, GetSegmentLength(SegmentIndex));
	}
	
	return Bounds;
}

FVector::FReal UPCGPolyLineData::GetLength() const
{
	FVector::FReal Length = 0.0;

	for (int SegmentIndex = 0; SegmentIndex < GetNumSegments(); ++SegmentIndex)
	{
		Length += GetSegmentLength(SegmentIndex);
	}

	return Length;
}

int UPCGPolyLineData::GetNumVertices() const
{
	int NumPoints = GetNumSegments();
	if (NumPoints > 0 && !IsClosed())
	{
		++NumPoints;
	}
	
	return NumPoints;
}

FTransform UPCGPolyLineData::K2_GetTransformAtDistance(int SegmentIndex, double Distance, FBox& OutBounds, bool bWorldSpace) const
{
	return GetTransformAtDistance(SegmentIndex, Distance, bWorldSpace, &OutBounds);
}

float UPCGPolyLineData::GetInputKeyAtAlpha(float Alpha) const
{
	Alpha = FMath::Clamp(Alpha, 0, 1);
	return static_cast<float>(GetNumSegments()) * Alpha;
}

void UPCGPolyLineData::GetTangentsAtSegmentStart(int SegmentIndex, FVector& OutArriveTangent, FVector& OutLeaveTangent) const
{
	OutArriveTangent = FVector::Zero();
	OutLeaveTangent = FVector::Zero();
}

float UPCGPolyLineData::GetAlphaAtDistance(int SegmentIndex, FVector::FReal Distance) const
{
	const int32 NumSegments = GetNumSegments();
	if (NumSegments < 1 || SegmentIndex < 0 || (SegmentIndex == 0 && Distance <= UE_DOUBLE_SMALL_NUMBER))
	{
		return 0.f;
	}
	else if (SegmentIndex >= NumSegments) // By definition, if the index is >= NumSegments, the alpha should be 1
	{
		return 1.f;
	}

	// Find the starting alpha of segments up to the current one.
	const double SegmentStartAlpha = static_cast<double>(SegmentIndex) / NumSegments;

	// Note: SegmentLength can be 0 for a start index, invalid index, or if control points are co-located.
	const double SegmentLength = GetSegmentLength(SegmentIndex);

	// Clamp the distance, as a distance longer than the current segment could artificially inflate the alpha.
	const double LocalDistance = FMath::Clamp(Distance, 0.0, SegmentLength);

	// Start the local alpha with the local distance past the segment length.
	const double SegmentLocalAlpha = (SegmentLength > UE_DOUBLE_SMALL_NUMBER ? (LocalDistance / SegmentLength) : 0.f) / NumSegments;

	return SegmentStartAlpha + SegmentLocalAlpha;
}
