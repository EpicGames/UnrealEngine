// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGPolygon2DData.h"

#include "Containers/StaticArray.h"
#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPointArrayData.h"
#include "Elements/Polygon/PCGSurfaceFromPolygon2D.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGPropertyAccessor.h"
#include "Metadata/Accessors/PCGPolygon2DAccessor.h"


#include "SegmentTypes.h"
#include "Serialization/ArchiveCrc32.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPolygon2DData)

#define LOCTEXT_NAMESPACE "PCGPolygon2DData"

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoPolygon2D, UPCGPolygon2DData)

namespace PCGPolygon2DData
{
	namespace Constants
	{
		const float KeyEpsilon = 1.0e-6f;
	}

	// @todo_pcg : could suggest to move this to the polygon implementation
	void SerializePolygon(UE::Geometry::FGeneralPolygon2d& Polygon, FArchive& Archive)
	{
		if (Archive.IsLoading())
		{
			Polygon.ClearHoles();

			TArray<FVector2d> OuterVertices;
			Archive << OuterVertices;
			Polygon.SetOuter(UE::Geometry::FPolygon2d(MoveTemp(OuterVertices)));

			int32 NumHoles = 0;
			Archive << NumHoles;

			for (int HoleIndex = 0; HoleIndex < NumHoles; ++HoleIndex)
			{
				TArray<FVector2d> HoleVertices;
				Archive << HoleVertices;
				Polygon.AddHole(UE::Geometry::FPolygon2d(MoveTemp(HoleVertices)), /*bCheckContainment=*/false, /*bCheckOrientation=*/false);
			}
		}
		else
		{
			const UE::Geometry::FPolygon2d& Outer = Polygon.GetOuter();
			const TArray<FVector2d>& OuterVertices = Outer.GetVertices();
			Archive << const_cast<TArray<FVector2d>&>(OuterVertices);

			const TArray<UE::Geometry::FPolygon2d>& Holes = Polygon.GetHoles();
			int32 NumHoles = Holes.Num();
			Archive << NumHoles;

			for (const UE::Geometry::FPolygon2d& Hole : Holes)
			{
				const TArray<FVector2d>& HoleVertices = Hole.GetVertices();
				Archive << const_cast<TArray<FVector2d>&>(HoleVertices);
			}
		}
	}

	void ReversePolygon(UE::Geometry::FGeneralPolygon2d& Polygon, TArray<int64>& PolygonEntryKeys)
	{
		Polygon.Reverse();
		if (!PolygonEntryKeys.IsEmpty())
		{
			int VertexOffset = 0;
			int VertexCount = Polygon.GetOuter().VertexCount();
			Algo::Reverse(&PolygonEntryKeys[VertexOffset], VertexCount);
			VertexOffset += VertexCount;

			for (const UE::Geometry::FPolygon2d& Hole : Polygon.GetHoles())
			{
				VertexCount = Hole.VertexCount();
				Algo::Reverse(&PolygonEntryKeys[VertexOffset], VertexCount);
				VertexOffset += VertexCount;
			}
		}
	}

	int PolygonVertexCount(const UE::Geometry::FGeneralPolygon2d& InPoly)
	{
		int Count = InPoly.GetOuter().VertexCount();
		for (const UE::Geometry::FPolygon2d& Hole : InPoly.GetHoles())
		{
			Count += Hole.VertexCount();
		}

		return Count;
	}

	int GetVertexIndex(const UE::Geometry::FGeneralPolygon2d& InPoly, int InSegmentIndex, int InHoleIndex)
	{
		int StartVertexIndex = -1;
		int EndVertexIndex = -1;
		GetStartEndVertexIndices(InPoly, InSegmentIndex, InHoleIndex, StartVertexIndex, EndVertexIndex);

		return StartVertexIndex;
	}

	void GetStartEndVertexIndices(const UE::Geometry::FGeneralPolygon2d& InPoly, int InSegmentIndex, int InHoleIndex, int& OutStartVertexIndex, int& OutEndVertexIndex)
	{
		int Offset = 0;
		int CurrentPolyVertexCount = InPoly.GetOuter().VertexCount();

		if (InHoleIndex >= 0)
		{
			Offset = CurrentPolyVertexCount;
		}

		for (int HoleIndex = 0; HoleIndex <= InHoleIndex; ++HoleIndex)
		{
			CurrentPolyVertexCount = InPoly.GetHoles()[HoleIndex].VertexCount();
			if (HoleIndex < InHoleIndex)
			{
				Offset += CurrentPolyVertexCount;
			}
		}

		OutStartVertexIndex = Offset + InSegmentIndex;
		OutEndVertexIndex = Offset + ((InSegmentIndex + 1) % CurrentPolyVertexCount);
	}

	TUniquePtr<IPCGAttributeAccessor> CreatePropertyAccessor(EPCGPolygon2DProperties Property, bool bQuiet, bool bConst)
	{
		if (bConst)
		{
			switch (Property)
			{
			case EPCGPolygon2DProperties::Position:
				return MakeUnique<FPCGPolygon2DVerticesAccessor<FVector, EPCGPolygon2DProperties::Position>>(bConst);
			case EPCGPolygon2DProperties::Rotation:
				return MakeUnique<FPCGPolygon2DVerticesAccessor<FQuat, EPCGPolygon2DProperties::Rotation>>(bConst);
			case EPCGPolygon2DProperties::SegmentIndex:
				return MakeUnique<FPCGPolygon2DVerticesAccessor<int32, EPCGPolygon2DProperties::SegmentIndex>>(bConst);
			case EPCGPolygon2DProperties::HoleIndex:
				return MakeUnique<FPCGPolygon2DVerticesAccessor<int32, EPCGPolygon2DProperties::HoleIndex>>(bConst);
			case EPCGPolygon2DProperties::SegmentLength:
				return MakeUnique<FPCGPolygon2DVerticesAccessor<double, EPCGPolygon2DProperties::SegmentLength>>(bConst);
			case EPCGPolygon2DProperties::LocalPosition:
				return MakeUnique<FPCGPolygon2DVerticesAccessor<FVector2d, EPCGPolygon2DProperties::LocalPosition>>(bConst);
			case EPCGPolygon2DProperties::LocalRotation:
				return MakeUnique<FPCGPolygon2DVerticesAccessor<FQuat, EPCGPolygon2DProperties::LocalRotation>>(bConst);
			default:
				if (!bQuiet)
				{
					UE_LOGF(LogPCG, Error, "EPCGPolygon2DProperties value '%d' does not exist.", EnumToUnderlyingType(Property));
				}
				return TUniquePtr<IPCGAttributeAccessor>();
			}
		}
		else
		{
			switch (Property)
			{
			case EPCGPolygon2DProperties::Position:
				return MakeUnique<FPCGPolygon2DVerticesAccessor<FVector, EPCGPolygon2DProperties::Position>>(bConst);
			case EPCGPolygon2DProperties::LocalPosition:
				return MakeUnique<FPCGPolygon2DVerticesAccessor<FVector2d, EPCGPolygon2DProperties::LocalPosition>>(bConst);
			case EPCGPolygon2DProperties::Rotation: //fall-through
			case EPCGPolygon2DProperties::SegmentIndex: //fall-through
			case EPCGPolygon2DProperties::HoleIndex: //fall-through
			case EPCGPolygon2DProperties::SegmentLength:
			case EPCGPolygon2DProperties::LocalRotation:
				if (!bQuiet)
				{
					UE_LOGF(LogPCG, Error, "EPCGPolygon2DProperties value '%d' is read-only.", EnumToUnderlyingType(Property));
				}
				return TUniquePtr<IPCGAttributeAccessor>();
			default:
				if (!bQuiet)
				{
					UE_LOGF(LogPCG, Error, "EPCGPolygon2DProperties value '%d' does not exist.", EnumToUnderlyingType(Property));
				}
				return TUniquePtr<IPCGAttributeAccessor>();
			}
		}
	}

	TUniquePtr<IPCGAttributeAccessor> CreateDataPropertyAccessor(EPCGPolygon2DDataProperties DataProperty, bool bQuiet, bool bConst)
	{
		if (bConst)
		{
			switch(DataProperty)
			{
			case EPCGPolygon2DDataProperties::Transform:
				return MakeUnique<FPCGPolygon2DDataAccessor<FTransform, EPCGPolygon2DDataProperties::Transform>>(bConst);
			case EPCGPolygon2DDataProperties::Area:
				return MakeUnique<FPCGPolygon2DDataAccessor<double, EPCGPolygon2DDataProperties::Area>>(bConst);
			case EPCGPolygon2DDataProperties::Perimeter:
				return MakeUnique<FPCGPolygon2DDataAccessor<double, EPCGPolygon2DDataProperties::Perimeter>>(bConst);
			case EPCGPolygon2DDataProperties::BoundsMin:
				return MakeUnique<FPCGPolygon2DDataAccessor<FVector2d, EPCGPolygon2DDataProperties::BoundsMin>>(bConst);
			case EPCGPolygon2DDataProperties::BoundsMax:
				return MakeUnique<FPCGPolygon2DDataAccessor<FVector2d, EPCGPolygon2DDataProperties::BoundsMax>>(bConst);
			case EPCGPolygon2DDataProperties::SegmentCount:
				return MakeUnique<FPCGPolygon2DDataAccessor<int32, EPCGPolygon2DDataProperties::SegmentCount>>(bConst);
			case EPCGPolygon2DDataProperties::OuterSegmentCount:
				return MakeUnique<FPCGPolygon2DDataAccessor<int32, EPCGPolygon2DDataProperties::OuterSegmentCount>>(bConst);
			case EPCGPolygon2DDataProperties::HoleCount:
				return MakeUnique<FPCGPolygon2DDataAccessor<int32, EPCGPolygon2DDataProperties::HoleCount>>(bConst);
			case EPCGPolygon2DDataProperties::LongestOuterSegmentIndex:
				return MakeUnique<FPCGPolygon2DDataAccessor<int32, EPCGPolygon2DDataProperties::LongestOuterSegmentIndex>>(bConst);
			case EPCGPolygon2DDataProperties::IsClockwise:
				return MakeUnique<FPCGPolygon2DDataAccessor<bool, EPCGPolygon2DDataProperties::IsClockwise>>(bConst);
			default:
				if (!bQuiet)
				{
					UE_LOGF(LogPCG, Error, "EPCGPolygon2DDataProperties value '%d' does not exist.", EnumToUnderlyingType(DataProperty));
				}
				return TUniquePtr<IPCGAttributeAccessor>();
			}
		}
		else
		{
			// Only the transform can be set at the data level.
			if (DataProperty == EPCGPolygon2DDataProperties::Transform)
			{
				return MakeUnique<FPCGPolygon2DDataAccessor<FTransform, EPCGPolygon2DDataProperties::Transform>>(bConst);
			}
			else
			{
				return TUniquePtr<IPCGAttributeAccessor>();
			}
		}
	}
} // namespace PCGPolygon2DData

UPCGPolygon2DData::UPCGPolygon2DData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	check(Metadata);
	Metadata->SetupDomain(PCGMetadataDomainID::Elements, /*bIsDefault=*/true);
}

bool FPCGDataTypeInfoPolygon2D::SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const
{
	// Polygon can convert to surface
	if (OutputType.IsSameType(EPCGDataType::Surface))
	{
		if (OptionalOutConversionSettings)
		{
			*OptionalOutConversionSettings = UPCGCreateSurfaceFromPolygon2DSettings::StaticClass();
		}

		return true;
	}
	else
	{
		return FPCGDataTypeInfoPolyline::SupportsConversionTo(ThisType, OutputType, OptionalOutConversionSettings, OptionalOutCompatibilityMessage);
	}
}

void UPCGPolygon2DData::Serialize(FArchive& InArchive)
{
	LLM_SCOPE_BYTAG(PCG);
	Super::Serialize(InArchive);

	InArchive.UsingCustomVersion(FPCGCustomVersion::GUID);

	PCGPolygon2DData::SerializePolygon(Polygon, InArchive);
	InArchive << Transform;
	InArchive << VertexEntryKeys;

	if (InArchive.IsLoading())
	{
		UpdateMappings();
	}
}

void UPCGPolygon2DData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(
		sizeof(UE::Geometry::FGeneralPolygon2d) +
		Polygon.GetAllocatedSize() +
		sizeof(Transform) +
		VertexEntryKeys.GetAllocatedSize() +
		SegmentIndexToSegmentAndHoleIndices.GetAllocatedSize() +
		SegmentAndHoleIndicesToSegmentIndex.GetAllocatedSize() +
		sizeof(int));
}

void UPCGPolygon2DData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// The code below has non-trivial cost...
	if (!bFullDataCrc)
	{
		// Fallback to UID
		AddUIDToCrc(Ar);
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGPolygon2DData::AddToCrc);

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;
	PCGPolygon2DData::SerializePolygon(const_cast<UE::Geometry::FGeneralPolygon2d&>(Polygon), Ar);
	Ar << const_cast<FTransform&>(Transform);

	if (Metadata)
	{
		Metadata->AddToCrc(Ar, bFullDataCrc);
	}
}

FPCGMetadataDomainID UPCGPolygon2DData::GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const
{
	const FName DomainName = InSelector.GetDomainName();

	if (DomainName == PCGPolygon2DData::VertexDomainName)
	{
		return PCGMetadataDomainID::Elements;
	}
	else
	{
		return Super::GetMetadataDomainIDFromSelector(InSelector);
	}
}

bool UPCGPolygon2DData::SetDomainFromDomainID(const FPCGMetadataDomainID& InDomainID, FPCGAttributePropertySelector& InOutSelector) const
{
	if (InDomainID == PCGMetadataDomainID::Elements)
	{
		InOutSelector.SetDomainName(PCGPolygon2DData::VertexDomainName, /*bResetExtraNames=*/false);
		return true;
	}
	else
	{
		return Super::SetDomainFromDomainID(InDomainID, InOutSelector);
	}
}

bool UPCGPolygon2DData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// Project point down to the 2d plane.
	const FTransform LocalSpaceTransform = InTransform.GetRelativeTransform(Transform);
	const FBox BoundsInLocalSpace = InBounds.TransformBy(LocalSpaceTransform);

	if(BoundsInLocalSpace.Min.Z > 0 || BoundsInLocalSpace.Max.Z < 0)
	{
		return false;
	}

	const FVector2D Center = FVector2D(BoundsInLocalSpace.GetCenter());

	// @todo_pcg: Check if the bounds (as a polygon) overlap with the polygon.
	// Note that the polygon class has an overlap test, but not at the general polygon level
	// We'll test just the center until then.
	if (!Polygon.Contains(Center))
	{
		return false;
	}

	new(&OutPoint) FPCGPoint(InTransform, /*Density=*/1.0f, /*Seed=*/0);
	OutPoint.SetLocalBounds(InBounds);

	// @todo_pcg: this could be done a bit more cleaner in conjunction with the contains test.
	// The issue here is that the key should be driven by the closest segment where the point is "inside" not outside.
	// In the mean time, this might be a reasonable approximation.
	if (OutMetadata)
	{
		int HoleIndex = INDEX_NONE;
		int SegmentIndex = INDEX_NONE;
		double DistanceParameter = 0.0;

		double NearestSegmentSquaredDistance = Polygon.DistanceSquared(Center, HoleIndex, SegmentIndex, DistanceParameter);
		const double SegmentRatio = Polygon.Segment(SegmentIndex, HoleIndex).ConvertToUnitRange(DistanceParameter);

		const int GlobalSegmentIndex = SegmentAndHoleIndicesToSegmentIndex[TPair<int, int>(SegmentIndex, HoleIndex)];
		const float NearestKey = GlobalSegmentIndex + FMath::Min(SegmentRatio, 1.0f - PCGPolygon2DData::Constants::KeyEpsilon);

		WriteMetadataToPoint(NearestKey, OutPoint, OutMetadata);
	}

	return true;
}

bool UPCGPolygon2DData::ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// Project point down to the 2d plane.
	const FTransform LocalSpaceTransform = InTransform.GetRelativeTransform(Transform);
	const FBox BoundsInLocalSpace = InBounds.TransformBy(LocalSpaceTransform);

	// @todo_pcg: Check if the bounds (as a polygon) overlap with the polygon.
	// Note that the polygon class has an overlap test, but not at the general polygon level
	// We'll test just the center until then.
	const FVector2D Center = FVector2D(BoundsInLocalSpace.GetCenter());
	if (!Polygon.Contains(Center))
	{
		return false;
	}

	new(&OutPoint) FPCGPoint(InTransform, /*Density=*/1.0f, /*Seed=*/0);

	if (InParams.bProjectPositions)
	{
		OutPoint.Transform.SetLocation(Transform.TransformPosition(FVector(Center.X, Center.Y, 0.0)));
	}

	if (InParams.bProjectRotations)
	{
		// Want to project the rotation so that the up vector aligns to the surface's up vector
		FQuat UpVectorRotation = FQuat::FindBetweenNormals(InTransform.GetRotation().GetUpVector(), Transform.GetRotation().GetUpVector());
		OutPoint.Transform.SetRotation(UpVectorRotation * InTransform.GetRotation());
	}

	// @todo_pcg: this could be done a bit more cleaner in conjunction with the contains test.
	// The issue here is that the key should be driven by the closest segment where the point is "inside" not outside.
	// In the mean time, this might be a reasonable approximation.
	if (OutMetadata)
	{
		int HoleIndex = INDEX_NONE;
		int SegmentIndex = INDEX_NONE;
		double DistanceParameter = 0.0;

		double NearestSegmentSquaredDistance = Polygon.DistanceSquared(Center, HoleIndex, SegmentIndex, DistanceParameter);
		const double SegmentRatio = Polygon.Segment(SegmentIndex, HoleIndex).ConvertToUnitRange(DistanceParameter);

		const int GlobalSegmentIndex = SegmentAndHoleIndicesToSegmentIndex[TPair<int, int>(SegmentIndex, HoleIndex)];
		const float NearestKey = GlobalSegmentIndex + FMath::Min(SegmentRatio, 1.0f - PCGPolygon2DData::Constants::KeyEpsilon);

		WriteMetadataToPoint(NearestKey, OutPoint, OutMetadata);
	}

	// Implementation note - the polygon transform has always a unit scale, so there's nothing to project against.
	return true;
}

void UPCGPolygon2DData::WriteMetadataToPoint(float InputKey, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	return WriteMetadataToEntry(InputKey, OutPoint.MetadataEntry, OutMetadata);
}

void UPCGPolygon2DData::WriteMetadataToEntry(float InputKey, PCGMetadataEntryKey& OutEntryKey, UPCGMetadata* OutMetadata) const
{
	if (!OutMetadata || !ensure(Metadata) || VertexEntryKeys.IsEmpty())
	{
		return;
	}

	const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomain(PCGMetadataDomainID::Elements);
	FPCGMetadataDomain* OutMetadataDomain = OutMetadata->GetMetadataDomain(PCGMetadataDomainID::Elements);

	if (!MetadataDomain || !OutMetadataDomain)
	{
		return;
	}

	const float InterpolationRatio = FMath::Frac(InputKey);
	const int CurrentVertexIndex = FMath::FloorToInt32(InputKey);

	int NextVertexIndex = INDEX_NONE;
	int CurrentSegmentIndex = INDEX_NONE;
	int CurrentHoleIndex = INDEX_NONE;

	if (const TPair<int, int>* CurrentPair = SegmentIndexToSegmentAndHoleIndices.Find(CurrentVertexIndex))
	{
		CurrentSegmentIndex = CurrentPair->Key;
		CurrentHoleIndex = CurrentPair->Value;
	}
	else
	{
		// invalid index
		OutEntryKey = PCGInvalidEntryKey;
		return;
	}

	// If this fails, it should have failed the pair search, hence the check here.
	check(VertexEntryKeys.IsValidIndex(CurrentVertexIndex));
	PCGMetadataEntryKey CurrentVertexEntryKey = VertexEntryKeys[CurrentVertexIndex];

	// Implementation note: there's no need to compute property interpolation if we're directly on the first vertex of the segment.
	if (InterpolationRatio <= PCGPolygon2DData::Constants::KeyEpsilon)
	{
		OutMetadataDomain->InitializeOnSet(OutEntryKey, CurrentVertexEntryKey, MetadataDomain);
		return;
	}

	if (const int* TentativeNextVertex = SegmentAndHoleIndicesToSegmentIndex.Find(TPair<int, int>(CurrentSegmentIndex + 1, CurrentHoleIndex)))
	{
		NextVertexIndex = *TentativeNextVertex;
	}
	else if (const int* TentativeFirstVertex = SegmentAndHoleIndicesToSegmentIndex.Find(TPair<int, int>(0, CurrentHoleIndex)))
	{
		NextVertexIndex = *TentativeFirstVertex;
	}
	else
	{
		// Invalid index
		OutMetadataDomain->InitializeOnSet(OutEntryKey, CurrentVertexEntryKey, MetadataDomain);
		return;
	}

	// If this fails, it should have failed the pair search, hence the check here.
	check(VertexEntryKeys.IsValidIndex(NextVertexIndex));
	PCGMetadataEntryKey NextVertexEntryKey = VertexEntryKeys[NextVertexIndex];

	// Implementation note: there's no need to compute property interpolation if we're directly on the second vertex of the segment.
	if (InterpolationRatio >= 1.0f - PCGPolygon2DData::Constants::KeyEpsilon || CurrentVertexEntryKey == PCGInvalidEntryKey)
	{
		OutMetadataDomain->InitializeOnSet(OutEntryKey, NextVertexEntryKey, MetadataDomain);
		return;
	}

	// Finally, perform full interpolation if it's needed and valid
	OutMetadataDomain->InitializeOnSet(OutEntryKey, CurrentVertexEntryKey, MetadataDomain);

	if (CurrentVertexEntryKey != PCGInvalidEntryKey && NextVertexEntryKey != PCGInvalidEntryKey)
	{
		TStaticArray<TPair<PCGMetadataEntryKey, float>, 2> Coefficients;
		Coefficients[0] = { CurrentVertexEntryKey, 1.0f - InterpolationRatio };
		Coefficients[1] = { NextVertexEntryKey, InterpolationRatio };
		OutMetadataDomain->ComputeWeightedAttribute(OutEntryKey, Coefficients, MetadataDomain);
	}
}

UPCGSpatialData* UPCGPolygon2DData::CopyInternal(FPCGContext* Context) const
{
	UPCGPolygon2DData* NewPolygonData = FPCGContext::NewObject_AnyThread<UPCGPolygon2DData>(Context);
	NewPolygonData->Polygon = Polygon;
	NewPolygonData->VertexEntryKeys = VertexEntryKeys;
	NewPolygonData->Transform = Transform;
	NewPolygonData->SegmentIndexToSegmentAndHoleIndices = SegmentIndexToSegmentAndHoleIndices;
	NewPolygonData->SegmentAndHoleIndicesToSegmentIndex = SegmentAndHoleIndicesToSegmentIndex;
	NewPolygonData->SegmentCount = SegmentCount;
	return NewPolygonData;
}

FBox UPCGPolygon2DData::GetBounds() const
{
	const UE::Geometry::TAxisAlignedBox2<double> Bounds2D = Polygon.Bounds();

	constexpr double FixedHalfHeight = 50.0;
	const FBox Bounds(FVector(Bounds2D.Min, -FixedHalfHeight), FVector(Bounds2D.Max, +FixedHalfHeight));
	return Bounds.TransformBy(Transform);
}

const UPCGPointData* UPCGPolygon2DData::CreatePointData(FPCGContext* Context) const
{
	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGPolygon2DData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGPolygon2DData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	UPCGBasePointData* PointData = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);

	FPCGInitializeFromDataParams InitializeFromDataParams(this);
	InitializeFromDataParams.bInheritSpatialData = false;

	PointData->InitializeFromDataWithParams(InitializeFromDataParams);

	// Implementation note: since polygons can have holes, we will always export the hole index property as an attribute.
	FName HoleIndexAttributeName = FName(StaticEnum<EPCGPolygon2DProperties>()->GetNameStringByValue(static_cast<int64>(EPCGPolygon2DProperties::HoleIndex)));
	PointData->Metadata->FindOrCreateAttribute<int>(HoleIndexAttributeName, -1, /*bAllowsInterpolation=*/false);

	// Allocate properties & points.
	const EPCGPointNativeProperties PropertiesToAllocate = EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::MetadataEntry;

	const int NumPoints = GetNumVertices();
	PointData->SetNumPoints(NumPoints);
	PointData->AllocateProperties(PropertiesToAllocate);

	const bool bHasVertexMetadata = !VertexEntryKeys.IsEmpty();
	check(VertexEntryKeys.IsEmpty() || VertexEntryKeys.Num() == NumPoints);
	
	FPCGPointValueRanges OutRanges(PointData, /*bAllocate=*/false);

	for (int PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
	{
		OutRanges.TransformRange[PointIndex] = GetTransformAtDistance(PointIndex, 0.0, /*bWorldSpace=*/true);

		if (bHasVertexMetadata)
		{
			OutRanges.MetadataEntryRange[PointIndex] = VertexEntryKeys[PointIndex];
		}

		PointData->Metadata->InitializeOnSet(OutRanges.MetadataEntryRange[PointIndex]);
	}

	// Write $HoleIndex to HoleIndex attribute
	if(ensure(!SegmentIndexToSegmentAndHoleIndices.IsEmpty()))
	{
		TArray<int> HoleIndices;
		HoleIndices.Reserve(SegmentIndexToSegmentAndHoleIndices.Num());
		for (int VertexIndex = 0; VertexIndex < NumPoints; ++VertexIndex)
		{
			HoleIndices.Add(SegmentIndexToSegmentAndHoleIndices[VertexIndex].Value);
		}

		FPCGAttributePropertyOutputSelector HoleIndexSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyOutputSelector>(HoleIndexAttributeName);
		PCGAttributeAccessorHelpers::WriteAllValues<int>(PointData, HoleIndexSelector, HoleIndices, /*SourceSelector=*/nullptr, Context);
	}

	return PointData;
}

int UPCGPolygon2DData::GetNumSegments() const
{
	return SegmentCount;
}

double UPCGPolygon2DData::GetSegmentLength(int SegmentIndex) const
{
	if (SegmentIndex < 0 || SegmentIndex >= SegmentCount)
	{
		return 0.0;
	}

	// Implementation note - when we parse the polygon we build the transform such that the scale is always 1.
	const TPair<int, int> SegmentMapping = SegmentIndexToSegmentAndHoleIndices[SegmentIndex];
	return Polygon.Segment(SegmentMapping.Key, SegmentMapping.Value).Length();
}

FVector UPCGPolygon2DData::GetLocationAtAlpha(float Alpha) const
{
	return GetTransformAtAlpha(Alpha).GetLocation();
}

FTransform UPCGPolygon2DData::GetTransformAtAlpha(float InAlpha) const
{
	if (SegmentCount == 0)
	{
		return FTransform::Identity;
	}

	const double Alpha = FMath::Clamp(InAlpha, 0.0, 1.0 - UE_SMALL_NUMBER);

	// Note: alpha in splines is a value from 0 to 1, with each segment having the same relative value.
	const double SegmentKey = Alpha * SegmentCount;
	const int SegmentIndex = FMath::FloorToInt(SegmentKey);
	const double Ratio = FMath::Frac(SegmentKey);
	const TPair<int, int> SegmentMapping = SegmentIndexToSegmentAndHoleIndices[SegmentIndex];

	const UE::Geometry::FSegment2d Segment = Polygon.Segment(SegmentMapping.Key, SegmentMapping.Value);
	const FVector2D PointOnSegment = Segment.PointBetween(Ratio);

	FTransform OutTransform;
	OutTransform.SetLocation(FVector(PointOnSegment, 0.0));
	
	// @todo_pcg: check if Z sign has to be consequent with the polygon winding (likely)
	const FQuat Rotation = Segment.Direction.IsNearlyZero() ? FQuat::Identity : FRotationMatrix::MakeFromXZ(FVector(Segment.Direction, 0.0), FVector::UpVector).ToQuat();
	OutTransform.SetRotation(Rotation);
	
	OutTransform.SetScale3D(FVector::OneVector);

	return OutTransform * Transform;
}

FTransform UPCGPolygon2DData::GetTransformAtDistance(int SegmentIndex, double Distance, bool bWorldSpace, FBox* OutBounds) const
{
	if (OutBounds)
	{
		*OutBounds = FBox::BuildAABB(FVector::ZeroVector, FVector::OneVector);
	}

	FTransform OutTransform;

	if (SegmentIndex < 0 || SegmentIndex >= SegmentCount)
	{
		return FTransform::Identity;
	}

	const TPair<int, int> SegmentMapping = SegmentIndexToSegmentAndHoleIndices[SegmentIndex];
	const UE::Geometry::FSegment2d Segment = Polygon.Segment(SegmentMapping.Key, SegmentMapping.Value);
	
	// @todo_pcg: should we normalize the distance by scale if the user is asking for world space coordinates?
	const FVector2D PointOnSegment = Segment.PointAt(Distance - Segment.Extent);

	OutTransform.SetLocation(FVector(PointOnSegment, 0.0));

	// @todo_pcg: check if Z sign has to be consequent with the polygon winding (likely)
	const FQuat Rotation = Segment.Direction.IsNearlyZero() ? FQuat::Identity : FRotationMatrix::MakeFromXZ(FVector(Segment.Direction, 0.0), FVector::UpVector).ToQuat();
	OutTransform.SetRotation(Rotation);

	OutTransform.SetScale3D(FVector::OneVector);

	if (bWorldSpace)
	{
		OutTransform = OutTransform * Transform;
	}

	return OutTransform;
}

float UPCGPolygon2DData::GetInputKeyAtDistance(int SegmentIndex, double Distance) const
{
	if (SegmentIndex < 0)
	{
		return 0;
	}
	else if (SegmentIndex >= SegmentCount)
	{
		return SegmentCount;
	}
	else
	{
		const TPair<int, int> SegmentMapping = SegmentIndexToSegmentAndHoleIndices[SegmentIndex];
		const UE::Geometry::FSegment2d Segment = Polygon.Segment(SegmentMapping.Key, SegmentMapping.Value);
		// @todo_pcg : clamp to 1 the distance/segment.length?
		return SegmentIndex + (Distance / Segment.Length());
	}
}

double UPCGPolygon2DData::GetDistanceAtSegmentStart(int SegmentIndex) const
{
	SegmentIndex = FMath::Clamp(SegmentIndex, 0, SegmentCount);

	double Distance = 0;
	int CurrentSegmentIndex = 0;
	while (CurrentSegmentIndex < SegmentIndex && CurrentSegmentIndex < SegmentCount)
	{
		TPair<int, int> SegmentMapping = SegmentIndexToSegmentAndHoleIndices[CurrentSegmentIndex];
		UE::Geometry::FSegment2d Segment = Polygon.Segment(SegmentMapping.Key, SegmentMapping.Value);
		Distance += Segment.Length();
		++CurrentSegmentIndex;
	}

	return Distance;
}

void UPCGPolygon2DData::SetPolygon(const UE::Geometry::FGeneralPolygon2d& InPolygon, TConstArrayView<PCGMetadataEntryKey>* InOptionalEntryKeys)
{
	Polygon = InPolygon;
	UpdateMappings();

	if (InOptionalEntryKeys && InOptionalEntryKeys->Num() == SegmentIndexToSegmentAndHoleIndices.Num())
	{
		VertexEntryKeys = *InOptionalEntryKeys;
	}
}

void UPCGPolygon2DData::SetPolygon(UE::Geometry::FGeneralPolygon2d&& InPolygon, TConstArrayView<PCGMetadataEntryKey>* InOptionalEntryKeys)
{
	Polygon = MoveTemp(InPolygon);
	UpdateMappings();

	if (InOptionalEntryKeys && InOptionalEntryKeys->Num() == SegmentIndexToSegmentAndHoleIndices.Num())
	{
		VertexEntryKeys = *InOptionalEntryKeys;
	}
}

void UPCGPolygon2DData::SetTransform(const FTransform& InTransform, bool bCheckWinding)
{
	bool bNeedsToReversePolygon = false;

	if (bCheckWinding)
	{
		bNeedsToReversePolygon = (FMath::Sign(InTransform.GetRotation().GetUpVector() | Transform.GetRotation().GetUpVector()) < 0);
	}

	Transform = InTransform;

	if (bNeedsToReversePolygon)
	{
		PCGPolygon2DData::ReversePolygon(Polygon, VertexEntryKeys);
	}
}

void UPCGPolygon2DData::UpdateMappings()
{
	SegmentIndexToSegmentAndHoleIndices.Reset();
	SegmentAndHoleIndicesToSegmentIndex.Reset();
	SegmentCount = 0;

	const UE::Geometry::FPolygon2d& Outer = Polygon.GetOuter();
	for (int VertexIndex = 0; VertexIndex < Outer.VertexCount(); ++VertexIndex)
	{
		SegmentIndexToSegmentAndHoleIndices.Add(SegmentCount, TPair<int, int>(VertexIndex, -1));
		SegmentAndHoleIndicesToSegmentIndex.Add(TPair<int, int>(VertexIndex, -1), SegmentCount);
		++SegmentCount;
	}

	const TArray<UE::Geometry::FPolygon2d>& Holes = Polygon.GetHoles();
	for (int HoleIndex = 0; HoleIndex < Holes.Num(); ++HoleIndex)
	{
		const UE::Geometry::FPolygon2d& Hole = Holes[HoleIndex];
		for (int VertexIndex = 0; VertexIndex < Hole.VertexCount(); ++VertexIndex)
		{
			SegmentIndexToSegmentAndHoleIndices.Add(SegmentCount, TPair<int, int>(VertexIndex, HoleIndex));
			SegmentAndHoleIndicesToSegmentIndex.Add(TPair<int, int>(VertexIndex, HoleIndex), SegmentCount);
			++SegmentCount;
		}
	}
}

void UPCGPolygon2DData::AllocateMetadataEntries()
{
	const int32 NumVertices = GetNumVertices();

	if (VertexEntryKeys.Num() != NumVertices)
	{
		VertexEntryKeys.Empty(NumVertices);
	}

	if (VertexEntryKeys.IsEmpty())
	{
		VertexEntryKeys.Reserve(NumVertices);
		for (int32 i = 0; i < NumVertices; ++i)
		{
			VertexEntryKeys.Add(PCGInvalidEntryKey);
		}
	}
}

TUniquePtr<IPCGAttributeAccessor> UPCGPolygon2DData::CreateStaticAccessor(const UPCGPolygon2DData* Data, const FPCGAttributePropertySelector& InSelector, bool bQuiet, bool bConst)
{
	const FName DomainName = InSelector.GetDomainName();
	if (InSelector.GetSelection() == EPCGAttributePropertySelection::Property && (DomainName.IsNone() || DomainName == PCGPolygon2DData::VertexDomainName))
	{
		const FName PropertyName = InSelector.GetName();

		if (const int64 EnumValue = StaticEnum<EPCGPolygon2DProperties>()->GetValueByName(PropertyName); EnumValue != INDEX_NONE)
		{
			const EPCGPolygon2DProperties NativeProperty = static_cast<EPCGPolygon2DProperties>(EnumValue);
			return PCGPolygon2DData::CreatePropertyAccessor(NativeProperty, bQuiet, bConst);
		}

		if (!bQuiet)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FailCreateAccessor", "Property {0} is not valid for a polygon vertex."), FText::FromName(PropertyName)));
		}
	}
	else if (InSelector.GetSelection() == EPCGAttributePropertySelection::Property && DomainName == PCGDataConstants::DataDomainName)
	{
		const FName PropertyName = InSelector.GetName();

		if (const int64 EnumValue = StaticEnum<EPCGPolygon2DDataProperties>()->GetValueByName(PropertyName); EnumValue != INDEX_NONE)
		{
			const EPCGPolygon2DDataProperties NativeDataProperty = static_cast<EPCGPolygon2DDataProperties>(EnumValue);
			return PCGPolygon2DData::CreateDataPropertyAccessor(NativeDataProperty, bQuiet, bConst);
		}
	}

	return {};
}

TUniquePtr<IPCGAttributeAccessorKeys> UPCGPolygon2DData::CreateAccessorKeys(const FPCGAttributePropertySelector& InSelector, bool bQuiet)
{
	const EPCGAttributePropertySelection Selection = InSelector.GetSelection();
	const FPCGMetadataDomainID DomainID = GetMetadataDomainIDFromSelector(InSelector);

	// Global data
	if (DomainID == PCGMetadataDomainID::Data && (Selection == EPCGAttributePropertySelection::Property || Selection == EPCGAttributePropertySelection::ExtraProperty))
	{
		return MakeUnique<FPCGAttributeAccessorKeysPolygon2DData>(this, /*bInGlobalData=*/true);
	}

	// Vertices data
	if ((DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements) && (Selection == EPCGAttributePropertySelection::Property || Selection == EPCGAttributePropertySelection::ExtraProperty))
	{
		return MakeUnique<FPCGAttributeAccessorKeysPolygon2DData>(this, /*bInGlobalData=*/false);
	}

	// Metadata in the vertices domain
	if ((DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements) && Selection == EPCGAttributePropertySelection::Attribute)
	{
		return MakeUnique<FPCGAttributeAccessorKeysPolygon2DDataEntries>(this);
	}

	return nullptr;
}

TUniquePtr<const IPCGAttributeAccessorKeys> UPCGPolygon2DData::CreateConstAccessorKeys(const FPCGAttributePropertySelector& InSelector, bool bQuiet) const
{
	const EPCGAttributePropertySelection Selection = InSelector.GetSelection();
	const FPCGMetadataDomainID DomainID = GetMetadataDomainIDFromSelector(InSelector);

	// Global data
	if (DomainID == PCGMetadataDomainID::Data && (Selection == EPCGAttributePropertySelection::Property || Selection == EPCGAttributePropertySelection::ExtraProperty))
	{
		return MakeUnique<FPCGAttributeAccessorKeysPolygon2DData>(this, /*bInGlobalData=*/true);
	}

	// Vertices data
	if ((DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements) && (Selection == EPCGAttributePropertySelection::Property || Selection == EPCGAttributePropertySelection::ExtraProperty))
	{
		return MakeUnique<FPCGAttributeAccessorKeysPolygon2DData>(this, /*bInGlobalData=*/false);
	}

	// Metadata in the vertices domain
	if ((DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements) && Selection == EPCGAttributePropertySelection::Attribute)
	{
		return MakeUnique<FPCGAttributeAccessorKeysPolygon2DDataEntries>(this);
	}

	return nullptr;
}

FPCGAttributeAccessorMethods UPCGPolygon2DData::GetPolygon2DAccessorMethods()
{
	auto CreateAccessorFunc = [](UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<IPCGAttributeAccessor>
		{
			return CreateStaticAccessor(CastChecked<UPCGPolygon2DData>(InData), InSelector, bQuiet, /*bConst=*/false);
		};

	auto CreateConstAccessorFunc = [](const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<const IPCGAttributeAccessor>
		{
			return CreateStaticAccessor(CastChecked<UPCGPolygon2DData>(InData), InSelector, bQuiet, /*bConst=*/true);
		};

	auto CreateAccessorKeysFunc = [](UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<IPCGAttributeAccessorKeys>
		{
			return CastChecked<UPCGPolygon2DData>(InData)->CreateAccessorKeys(InSelector, bQuiet);
		};

	auto CreateConstAccessorKeysFunc = [](const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<const IPCGAttributeAccessorKeys>
		{
			return CastChecked<UPCGPolygon2DData>(InData)->CreateConstAccessorKeys(InSelector, bQuiet);
		};

	FPCGAttributeAccessorMethods Methods
	{
		.CreateAccessorFunc = CreateAccessorFunc,
		.CreateConstAccessorFunc = CreateConstAccessorFunc,
		.CreateAccessorKeysFunc = CreateAccessorKeysFunc,
		.CreateConstAccessorKeysFunc = CreateConstAccessorKeysFunc
	};

#if WITH_EDITOR
	TArray<FText, TInlineAllocator<2>> Menus = { LOCTEXT("Polygon2VerticesSelectorMenuEntry", "Polygon 2d") };
	FText& SubMenu = Menus.Emplace_GetRef();

	SubMenu = LOCTEXT("PolygonVerticesSelectorMenuEntryPoints", "Vertices");
	Methods.FillSelectorMenuEntryFromEnum<EPCGPolygon2DProperties>(Menus);

	SubMenu = LOCTEXT("PolygonSelectorMenuEntryGlobal", "Global");
	Methods.FillSelectorMenuEntryFromEnum<EPCGPolygon2DDataProperties>(Menus);
#endif // WITH_EDITOR

	return Methods;
}

#undef LOCTEXT_NAMESPACE
