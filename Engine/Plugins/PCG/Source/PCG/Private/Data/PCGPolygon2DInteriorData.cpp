// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGPolygon2DInteriorData.h"

#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGSurfaceSampler.h"

#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPolygon2DInteriorData)

void UPCGPolygon2DInteriorSurfaceData::Initialize(FPCGContext* Context, const UPCGPolygon2DData* InPolygonData)
{
	check(InPolygonData);
	Polygon = InPolygonData;
}

void UPCGPolygon2DInteriorSurfaceData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	if (Metadata)
	{
		Metadata->AddToCrc(Ar, bFullDataCrc);
	}

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	// @todo_pcg either move the contents of the polygon in this class or use ComputeCrc to defer to the polygon for crc.
	if (Polygon)
	{
		Polygon->AddToCrc(Ar, bFullDataCrc);
	}
}

FBox UPCGPolygon2DInteriorSurfaceData::GetBounds() const
{
	check(Polygon);
	return Polygon->GetBounds();
}

bool UPCGPolygon2DInteriorSurfaceData::SamplePoint(const FTransform& InTransform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	check(Polygon);
	return Polygon->SamplePoint(InTransform, Bounds, OutPoint, OutMetadata);
}

bool UPCGPolygon2DInteriorSurfaceData::ProjectPoint(const FTransform& InTransform, const FBox& Bounds, const FPCGProjectionParams& Params, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	check(Polygon);
	return Polygon->ProjectPoint(InTransform, Bounds, Params, OutPoint, OutMetadata);
}

UPCGSpatialData* UPCGPolygon2DInteriorSurfaceData::CopyInternal(FPCGContext* Context) const
{
	UPCGPolygon2DInteriorSurfaceData* NewData = FPCGContext::NewObject_AnyThread<UPCGPolygon2DInteriorSurfaceData>(Context);

	CopyBaseSurfaceData(NewData);
	NewData->Polygon = Polygon;

	return NewData;
}

const UPCGPointData* UPCGPolygon2DInteriorSurfaceData::CreatePointData(FPCGContext* Context) const
{
	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()));
}

const UPCGPointArrayData* UPCGPolygon2DInteriorSurfaceData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()));
}

const UPCGBasePointData* UPCGPolygon2DInteriorSurfaceData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGPolygon2DInteriorSurfaceData::CreatePointData);

	// Default surface sampling
	const PCGSurfaceSampler::FSurfaceSamplerParams SamplerParams;
	UPCGBasePointData* Data = PCGSurfaceSampler::SampleSurface(Context, SamplerParams, this, /*InBoundingShape=*/nullptr, GetBounds(), PointDataClass);

	return Data;
}
