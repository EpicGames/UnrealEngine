// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGPrimitiveData.h"

#include "CollisionShape.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGVolumeSampler.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPrimitiveData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoPrimitive, UPCGPrimitiveData)

void UPCGPrimitiveData::Initialize(UPrimitiveComponent* InPrimitive)
{
	check(InPrimitive);
	Primitive = InPrimitive;
	CachedBounds = Primitive->Bounds.GetBox();
	// Not obvious to find strict bounds, leave at the default value
}

void UPCGPrimitiveData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(Ar);
}

bool UPCGPrimitiveData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// Pure implementation

	FCollisionShape CollisionShape;
	CollisionShape.SetBox(FVector3f(InBounds.GetExtent() * InTransform.GetScale3D()));

	const FVector BoxCenter = InTransform.TransformPosition(InBounds.GetCenter());

	if (ensure(IsValid(Primitive.Get())) && Primitive->OverlapComponent(BoxCenter, InTransform.GetRotation(), CollisionShape))
	{
		OutPoint.Transform = InTransform;
		OutPoint.SetLocalBounds(InBounds);
		OutPoint.Density = 1.0f;
		return true;
	}
	else
	{
		return false;
	}
}

const UPCGPointData* UPCGPrimitiveData::CreatePointData(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGPrimitiveData::CreatePointData);

	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGPrimitiveData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGPrimitiveData::CreatePointArrayData);

	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGPrimitiveData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	PCGVolumeSampler::FVolumeSamplerParams SamplerParams;
	SamplerParams.VoxelSize = VoxelSize;
	SamplerParams.Bounds = GetBounds();

	const UPCGBasePointData* Data = PCGVolumeSampler::SampleVolume(Context, PointDataClass, SamplerParams, this);

	if (Data && ensure(Primitive.Get()))
	{
		UE_LOGF(LogPCG, Verbose, "Primitive %ls extracted %d points", *Primitive->GetFName().ToString(), Data->GetNumPoints());
	}

	return Data;
}

UPCGSpatialData* UPCGPrimitiveData::CopyInternal(FPCGContext* Context) const
{
	UPCGPrimitiveData* NewPrimitiveData = FPCGContext::NewObject_AnyThread<UPCGPrimitiveData>(Context);

	NewPrimitiveData->VoxelSize = VoxelSize;
	NewPrimitiveData->Primitive = Primitive;
	NewPrimitiveData->CachedBounds = CachedBounds;
	NewPrimitiveData->CachedStrictBounds = CachedStrictBounds;

	return NewPrimitiveData;
}
