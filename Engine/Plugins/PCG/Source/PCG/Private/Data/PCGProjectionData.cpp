// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGProjectionData.h"

#include "PCGContext.h"
#include "Data/PCGLandscapeSplineData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGProjectionParams.h"
#include "Helpers/PCGAsync.h"
#include "Metadata/PCGMetadataAccessor.h"

#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGProjectionData)

namespace PCGProjectionPrivate
{
	FVector4 ApplyProjectionColorBlend(const FVector4& SourceColor, const FVector4& TargetColor, const EPCGProjectionColorBlendMode BlendMode)
	{
		auto ClampVector = [](FVector4 Vector) -> FVector4
		{
			Vector[0] = FMath::Clamp(Vector[0], 0.0, 1.0);
			Vector[1] = FMath::Clamp(Vector[1], 0.0, 1.0);
			Vector[2] = FMath::Clamp(Vector[2], 0.0, 1.0);
			Vector[3] = FMath::Clamp(Vector[3], 0.0, 1.0);
			return Vector;
		};

		switch (BlendMode)
		{
			case EPCGProjectionColorBlendMode::SourceValue:
				return SourceColor;
			case EPCGProjectionColorBlendMode::TargetValue:
				return TargetColor;
			case EPCGProjectionColorBlendMode::Add:
				return ClampVector(SourceColor + TargetColor);
			case EPCGProjectionColorBlendMode::Subtract:
				return ClampVector(SourceColor - TargetColor);
			case EPCGProjectionColorBlendMode::Multiply:
				return SourceColor * TargetColor;
			default:
				checkNoEntry();
				return FVector4::Zero();
		}
	}
}

void UPCGProjectionData::Initialize(const UPCGSpatialData* InSource, const UPCGSpatialData* InTarget, const FPCGProjectionParams& InProjectionParams)
{
	check(InSource && InTarget);
	// TODO: improve support for higher-dimension projection.
	// The problem is that there isn't a valid 1:1 mapping otherwise
	check(InSource->GetDimension() <= InTarget->GetDimension());
	Source = InSource;
	Target = InTarget;
	TargetActor = InSource->TargetActor;

	ProjectionParams = InProjectionParams;

	CachedBounds = ProjectBounds(Source->GetBounds());
	CachedStrictBounds = ProjectBounds(Source->GetStrictBounds());
}

void UPCGProjectionData::PostLoad()
{
	Super::PostLoad();

	ProjectionParams.ApplyDeprecation();
}

FPCGCrc UPCGProjectionData::ComputeCrc(bool bFullDataCrc) const
{
	FArchiveCrc32 Ar;

	AddToCrc(Ar, bFullDataCrc);

	// Chain together CRCs of operands
	check(Source && Target);
	uint32 CrcSource = Source->GetOrComputeCrc(bFullDataCrc).GetValue();
	uint32 CrcTarget = Target->GetOrComputeCrc(bFullDataCrc).GetValue();

	Ar << CrcSource;
	Ar << CrcTarget;
	
	return FPCGCrc(Ar.GetCrc());
}

void UPCGProjectionData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// Implementation note: no metadata in composite data at this point.
	// @todo_pcg: need metadata serialization now

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	Ar << ProjectionParams;
}

int UPCGProjectionData::GetDimension() const
{
	check(Source && Target);
	return FMath::Min(Source->GetDimension(), Target->GetDimension());
}

FBox UPCGProjectionData::GetBounds() const
{
	check(Source && Target);
	return CachedBounds;
}

FBox UPCGProjectionData::GetStrictBounds() const
{
	check(Source && Target);
	return CachedStrictBounds;
}

FVector UPCGProjectionData::GetNormal() const
{
	check(Source && Target);
	if (Source->GetDimension() > Target->GetDimension())
	{
		return Source->GetNormal();
	}
	else
	{
		return Target->GetNormal();
	}
}

FBox UPCGProjectionData::ProjectBounds(const FBox& InBounds) const
{
	FBox Bounds(EForceInit::ForceInit);

	const FBox PointAABB = FBox::BuildAABB(FVector::ZeroVector, FVector::ZeroVector);

	for (int Corner = 0; Corner < 8; ++Corner)
	{
		const FVector CornerPoint = FVector(
			(Corner / 4) ? InBounds.Max.X : InBounds.Min.X,
			((Corner / 2) % 2) ? InBounds.Max.Y : InBounds.Min.Y,
			(Corner % 2) ? InBounds.Max.Z : InBounds.Min.Z);

		FPCGPoint ProjectedPoint;
		if (Target->ProjectPoint(FTransform(CornerPoint), PointAABB, ProjectionParams, ProjectedPoint, nullptr))
		{
			Bounds += ProjectedPoint.Transform.GetLocation();
		}
		else
		{
			Bounds += CornerPoint;
		}
	}

	// Fixup the Z direction, as transforming the corners is not sufficient
	const FVector::FReal HalfHeight = 0.5 * (InBounds.Max.Z - InBounds.Min.Z);
	FVector BoundsCenter = InBounds.GetCenter();
	Bounds += BoundsCenter + Target->GetNormal() * HalfHeight;
	Bounds += BoundsCenter - Target->GetNormal() * HalfHeight;

	return Bounds;
}

bool UPCGProjectionData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(UPCGProjectionData::SamplePoint);
	
	// Detecting if a point is in a projection is often non-trivial. Projection is not in general a bijection and we cannot simply unproject
	// the point from the Target and check if it is in the Source. In this case we approximate the image of the projection and check
	// if the query point is in the image.
	if (RequiresCollapseToSample())
	{
		// Passing nullptr for the context means the operation will execute single threaded which is not ideal. To mitigate this we
		// prewarm the point cache when this projection data is constructed in the projection element.
		return ToBasePointData(nullptr)->SamplePoint(InTransform, InBounds, OutPoint, OutMetadata);
	}

	FPCGPoint PointFromSource;
	if (!Source->SamplePoint(InTransform, InBounds, PointFromSource, OutMetadata))
	{
		return false;
	}

	// This relies on fact that SamplePoint moves the point. This will be replaced with a ProjectPoint() call.
	FPCGPoint PointFromTarget;
	if (!Target->SamplePoint(PointFromSource.Transform, PointFromSource.GetLocalBounds(), PointFromTarget, OutMetadata))
	{
		return false;
	}

	// Merge points into a single point
	OutPoint = PointFromSource;

	ApplyProjectionResult(PointFromTarget, OutPoint);

	if (OutMetadata && PointFromTarget.MetadataEntry != PCGInvalidEntryKey)
	{
		OutMetadata->MergePointAttributesSubset(PointFromSource, OutMetadata, Source->Metadata, PointFromTarget, OutMetadata, Target->Metadata, OutPoint, ProjectionParams.AttributeMergeOperation);
	}

	return true;
}

bool UPCGProjectionData::HasNonTrivialTransform() const
{
	return Target->HasNonTrivialTransform();
}

bool UPCGProjectionData::RequiresCollapseToSample() const
{
	// Detecting if a point is in a projection is often non-trivial. Projection is not in general a bijection and we cannot simply unproject
	// the point from the Target and check if it is in the Source. 
	
	// There are cases where projection is a bijection. Like projecting volumes onto volumes (which is sampling). A non-PCG example is projection in
	// graphics using homogeneous coordinates - points can be unprojected back to original positions in 3D space.
	
	// There are cases where a projection is not technically a bijection, however we can still sample it. To illustrate, projection of a spline straight
	// down onto a terrain is such an example and is already covered via UPCGSplineProjectionData which overrides methods from this class. On the other
	// hand projecting a spline onto a terrain in a non-straight-down direction already complicates things because the spline projection will get
	// shadowed by the terrain (akin to terrain shadows cast by sunlight). We could raycast/raymarch from each query point towards the spline
	// to check for occlusion by the terrain, and also do a similar trick to what's in UPCGSplineProjectionData to get closest point. The spline
	// could intersect the terrain multiple times, so this will likely be expensive and take time to implement robustly. The alternative
	// of collapsing might seems favorable.

	// If we are losing precision from a collapse and we think we can sample without collapse, such cases could be detected and added here. Cases
	// involving projecting points should not be added here because a collapse calls ToPointData() which just returns the point data.
	
	// Keep in mind that detecting these cases robustly would ideally walk the upstream graph if it is a 'composite' network - i.e. if we want
	// to allow projection onto landscapes without collapse, we'd ideally check if the composite network is equivalent to a landscape (e.g. a landscape
	// intersected with a volume) rather than only checking if the immediate projection source is a particular type. A concrete example of this
	// failing would be the In and Actor graph input pins which can be backed by composite networks.
	
	// Sampling is trivial if we are not actually moving anything around..
	bool bRequiresCollapse = ProjectionParams.bProjectPositions;

	// Projection of a spline onto a surface is currently easy to sample - don't need a collapse. If projection direction exposed, will need to check that.
	if (Cast<UPCGSplineData>(Source) != nullptr && Target->GetDimension() == 2)
	{
		bRequiresCollapse = false;
	}

	// Projection of a landscape spline onto a surface is currently easy to sample - don't need a collapse. If projection direction exposed, will need to check that.
	if (Cast<UPCGLandscapeSplineData>(Source) != nullptr && Target->GetDimension() == 2)
	{
		bRequiresCollapse = false;
	}
	
	return bRequiresCollapse;
}

const UPCGPointData* UPCGProjectionData::CreatePointData(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGProjectionData::CreatePointData);

	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGProjectionData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGProjectionData::CreatePointArrayData);

	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGProjectionData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	// TODO: add mechanism in the ToPointData so we can pass in a transform
	// so we can forego creating the points twice if they're not used.
	const UPCGBasePointData* SourcePointData = Source->ToBasePointData(Context);
	const UPCGMetadata* SourceMetadata = SourcePointData->Metadata;

	UPCGBasePointData* PointData = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
	
	// Copy metadata attributes from source point including values
	FPCGInitializeFromDataParams InitializeFromDataParams(this);
	InitializeFromDataParams.bInheritSpatialData = false;
	// Since we have collapsed the source data, we need to inherit from this one.
	InitializeFromDataParams.SourceOverride = SourcePointData;
	PointData->InitializeFromDataWithParams(InitializeFromDataParams);
	UPCGMetadata* OutMetadata = PointData->Metadata;
	check(OutMetadata);

	// The projection operation will write into this temporary metadata
	UPCGMetadata* TempTargetMetadata = nullptr;
	if (Target->Metadata)
	{
		TempTargetMetadata = FPCGContext::NewObject_AnyThread<UPCGMetadata>(Context);
		TempTargetMetadata->SetupDomainsFromPCGDataType<UPCGBasePointData>();

		// We achieve filtering of metadata attributes by manipulating this temporary metadata, which works because
		// the projection operation operates on the attributes in this metadata.

		// Behavior modes:
		// * An excluded attribute that exists on source data will be kept and unchanged
		// * An excluded attribute that does not exist on source data will not be kept in result
		// * Included attributes are the only attributes that can be changed during projection
		// * Included attributes are the only attributes that will be added from target data
		SetupTargetMetadata(TempTargetMetadata);
	}

	auto InitializeFunc = [PointData, SourcePointData]()
	{
		PointData->SetNumPoints(SourcePointData->GetNumPoints(), /*bInitializeValues=*/false);

		// @todo_pcg: try to optimize allocations here
		PointData->AllocateProperties(SourcePointData->GetAllocatedProperties() | EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Color | EPCGPointNativeProperties::Density | EPCGPointNativeProperties::MetadataEntry);
		PointData->CopyUnallocatedPropertiesFrom(SourcePointData);
	};

	auto MoveDataRangeFunc = [PointData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
	{
		PointData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
	};

	auto FinishedFunc = [PointData](int32 NumPoints)
	{
		PointData->SetNumPoints(NumPoints);
	};

	auto ProcessRangeFunc = [this, PointData, SourcePointData, SourceMetadata, OutMetadata, TempTargetMetadata](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
	{
		int32 NumWritten = 0;

		const FConstPCGPointValueRanges InRanges(SourcePointData);
		FPCGPointValueRanges OutRanges(PointData, /*bAllocate=*/false);

		for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
		{
			const int32 WriteIndex = StartWriteIndex + NumWritten;
			FPCGPoint PointFromTarget;
			bool bValidProjection = true;

			const FBox LocalBounds = PCGPointHelpers::GetLocalBounds(InRanges.BoundsMinRange[ReadIndex], InRanges.BoundsMaxRange[ReadIndex]);
			if (!Target->ProjectPoint(InRanges.TransformRange[ReadIndex], LocalBounds, ProjectionParams, PointFromTarget, TempTargetMetadata))
			{
				if (!bKeepZeroDensityPoints)
				{
					continue;
				}
				else
				{
					// Point is rejected, mark its density to zero, put it in a state where we won't affect the output point
					bValidProjection = false;
					PointFromTarget.Transform = InRanges.TransformRange[ReadIndex];
					PointFromTarget.Color = InRanges.ColorRange[ReadIndex];
					PointFromTarget.Density = 0;
					PointFromTarget.MetadataEntry = PCGInvalidEntryKey;
				}
			}

			// Merge points into a single point
			OutRanges.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);

			// Apply projection result. Some of the params are already used inside ProjectPoint, so we are just applying the remaining bits that ProjectPoint() did not have access to.
			// TODO this would be cleaner if there was a ProjectPoint that took an FPCGPoint
			OutRanges.TransformRange[WriteIndex] = PointFromTarget.Transform;
			
			if (bValidProjection)
			{
				OutRanges.ColorRange[WriteIndex] = PCGProjectionPrivate::ApplyProjectionColorBlend(InRanges.ColorRange[ReadIndex], PointFromTarget.Color, ProjectionParams.ColorBlendMode);
			}
			
			OutRanges.DensityRange[WriteIndex] *= PointFromTarget.Density;

			if (OutMetadata && TempTargetMetadata && PointFromTarget.MetadataEntry != PCGInvalidEntryKey)
			{
				// Merge metadata to produce final attribute values
				OutMetadata->MergeAttributesSubset(InRanges.MetadataEntryRange[ReadIndex], SourceMetadata, SourceMetadata, PointFromTarget.MetadataEntry, TempTargetMetadata, TempTargetMetadata, OutRanges.MetadataEntryRange[WriteIndex], ProjectionParams.AttributeMergeOperation);
			}

			++NumWritten;
		}

		return NumWritten;
	};

	FPCGAsync::AsyncProcessingRangeEx(
		Context ? &Context->AsyncState : nullptr,
		SourcePointData->GetNumPoints(),
		InitializeFunc,
		ProcessRangeFunc,
		MoveDataRangeFunc,
		FinishedFunc,
		/*bEnableTimeSlicing=*/false);


	UE_LOGF(LogPCG, Verbose, "Projection generated %d points from %d source points", PointData->GetNumPoints(), SourcePointData->GetNumPoints());

	return PointData;
}

void UPCGProjectionData::ApplyProjectionResult(const FPCGPoint& InTargetPoint, FPCGPoint& InOutProjected) const
{
	if (ProjectionParams.bProjectPositions)
	{
		InOutProjected.Transform.SetLocation(InTargetPoint.Transform.GetLocation());
	}

	if (ProjectionParams.bProjectRotations)
	{
		InOutProjected.Transform.SetRotation(InTargetPoint.Transform.GetRotation());
	}

	if (ProjectionParams.bProjectScales)
	{
		InOutProjected.Transform.SetScale3D(InTargetPoint.Transform.GetScale3D());
	}

	InOutProjected.Color = PCGProjectionPrivate::ApplyProjectionColorBlend(InOutProjected.Color, InTargetPoint.Color, ProjectionParams.ColorBlendMode);

	InOutProjected.Density *= InTargetPoint.Density;
}

void UPCGProjectionData::GetIncludeExcludeAttributeNames(TSet<FName>& OutAttributeNames) const
{
	if (ProjectionParams.AttributeList.IsEmpty())
	{
		return;
	}

	TArray<FString> AttributeNameStrings;
	ProjectionParams.AttributeList.ParseIntoArray(AttributeNameStrings, TEXT(","), true);

	for (const FString& Attribute : AttributeNameStrings)
	{
		OutAttributeNames.Add(FName(*Attribute));
	}
}

void UPCGProjectionData::CopyBaseProjectionClass(UPCGProjectionData* NewProjectionData) const
{
	NewProjectionData->Source = Source;
	NewProjectionData->Target = Target;
	NewProjectionData->CachedBounds = CachedBounds;
	NewProjectionData->CachedStrictBounds = CachedStrictBounds;
	NewProjectionData->ProjectionParams = ProjectionParams;
}

UPCGSpatialData* UPCGProjectionData::CopyInternal(FPCGContext* Context) const
{
	UPCGProjectionData* NewProjectionData = FPCGContext::NewObject_AnyThread<UPCGProjectionData>(Context);

	CopyBaseProjectionClass(NewProjectionData);

	return NewProjectionData;
}

void UPCGProjectionData::SetupTargetMetadata(UPCGMetadata* MetadataToInitialize) const
{
	if (!Target || !Target->ConstMetadata())
	{
		return;
	}
	
	FPCGInitializeFromDataParams TargetInitializeFromDataParams(Target);
	
	TSet<FName> AttributesFilter;
	GetIncludeExcludeAttributeNames(AttributesFilter);

	FPCGMetadataDomainInitializeParams MetadataParams(nullptr, &AttributesFilter);
	MetadataParams.FilterMode = ProjectionParams.AttributeMode;
	TargetInitializeFromDataParams.MetadataInitializeParams.DomainInitializeParams.Emplace(PCGMetadataDomainID::Elements, std::move(MetadataParams));

	Target->InitializeTargetMetadata(TargetInitializeFromDataParams, MetadataToInitialize);
}

void UPCGProjectionData::InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const
{
	check(InParams.bInheritMetadata);
	check(MetadataToInitialize);

	// Duplicate data case, call the spatial base method
	if (InParams.bIsDuplicatingData)
	{
		UPCGSpatialData::InitializeTargetMetadata(InParams, MetadataToInitialize);
		return;
	}

	if (Source)
	{
		FPCGInitializeFromDataParams ParamsCopy = InParams;
		ParamsCopy.Source = InParams.SourceOverride ? InParams.SourceOverride : Source;
		ParamsCopy.Source->InitializeTargetMetadata(ParamsCopy, MetadataToInitialize);
	}

	SetupTargetMetadata(MetadataToInitialize);

	MetadataToInitialize->AddAttributes(Metadata);
}