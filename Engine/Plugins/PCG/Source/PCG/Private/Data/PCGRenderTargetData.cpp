// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGRenderTargetData.h"

#include "PCGContext.h"
#include "Engine/TextureRenderTarget2D.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGRenderTargetData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoRenderTarget2D, UPCGRenderTargetData)

namespace PCGRenderTargetDataHelpers
{
	static bool IsReadbackSupported(ETextureRenderTargetFormat InFormat)
	{
		return InFormat == RTF_RGBA16f || InFormat == RTF_RGBA32f || InFormat == RTF_RGBA8 || InFormat == RTF_RGB10A2;
	}
}

void UPCGRenderTargetData::Initialize(UTextureRenderTarget2D* InRenderTarget, const FTransform& InTransform, bool bInTakeOwnershipOfRenderTarget)
{
	InitializeInternal(InRenderTarget, InTransform, /*bInReadbackToCPU=*/false, bInTakeOwnershipOfRenderTarget);
}

void UPCGRenderTargetData::K2_Initialize(UTextureRenderTarget2D* InRenderTarget, const FTransform& InTransform, bool bInReadbackToCPU, bool bInTakeOwnershipOfRenderTarget)
{
	InitializeInternal(InRenderTarget, InTransform, bInReadbackToCPU, bInTakeOwnershipOfRenderTarget);
}

void UPCGRenderTargetData::InitializeInternal(UTextureRenderTarget2D* InRenderTarget, const FTransform& InTransform, bool bInReadbackToCPU, bool bInTakeOwnershipOfRenderTarget)
{
	RenderTarget = InRenderTarget;
	Transform = InTransform;
	bOwnsRenderTarget = bInTakeOwnershipOfRenderTarget;

	ColorData.Reset();

	if (RenderTarget)
	{
		Width = RenderTarget->SizeX;
		Height = RenderTarget->SizeY;
		Format = RenderTarget->GetFormat();

		const FTextureRHIRef InitRHI = GetTextureRHI();
		NumMips = InitRHI ? InitRHI->GetDesc().NumMips : 0;

		if (!PCGRenderTargetDataHelpers::IsReadbackSupported(RenderTarget->RenderTargetFormat))
		{
			UE_LOGF(LogPCG, Verbose, "UPCGRenderTargetData initialized with render target format '%ls' which does not support CPU readback. Sampling from CPU will return empty data.", *UEnum::GetValueAsString(RenderTarget->RenderTargetFormat));

			// Mark CPU data as "done" even though we have none. Consumers should stop waiting for readback and treat this as unsamplable (IsValid will return false because ColorData is empty).
			bIsReadbackComplete = true;
		}

		// Never take resource ownership on assets.
		if (RenderTarget->IsAsset())
		{
			bOwnsRenderTarget = false;
		}
	}

	Bounds = FBox(EForceInit::ForceInit);
	Bounds += FVector(-1.0f, -1.0f, 0.0f);
	Bounds += FVector(1.0f, 1.0f, 0.0f);
	Bounds = Bounds.TransformBy(Transform);

	if (bInReadbackToCPU)
	{
		RequestCPUReadback();
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UPCGRenderTargetData::Initialize(UTextureRenderTarget2D* InRenderTarget, const FTransform& InTransform, bool bInSkipReadbackToCPU, bool bInTakeOwnershipOfRenderTarget)
{
	InitializeInternal(InRenderTarget, InTransform, /*bInReadbackToCPU=*/!bInSkipReadbackToCPU, bInTakeOwnershipOfRenderTarget);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UPCGRenderTargetData::RequestCPUReadback()
{
	check(IsInGameThread());

	if (IsReadbackComplete())
	{
		return true;
	}

	if (!RenderTarget)
	{
		bIsReadbackComplete = true;
		return true;
	}

	if (!PCGRenderTargetDataHelpers::IsReadbackSupported(RenderTarget->RenderTargetFormat))
	{
		UE_LOGF(LogPCG, Error, "UPCGRenderTargetData::RequestCPUReadback called on a render target with unsupported format '%ls'.", *UEnum::GetValueAsString(RenderTarget->RenderTargetFormat));
		bIsReadbackComplete = true;
		return true;
	}

	if (FTextureRenderTargetResource* RTResource = RenderTarget->GameThread_GetRenderTargetResource())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGRenderTargetData::RequestCPUReadback);
		const FIntRect Rect = FIntRect(0, 0, RenderTarget->SizeX, RenderTarget->SizeY);
		const FReadSurfaceDataFlags ReadPixelFlags(RCM_MinMax);
		RTResource->ReadLinearColorPixels(ColorData, ReadPixelFlags, Rect);
	}

	bIsReadbackComplete = true;
	return true;
}

void UPCGRenderTargetData::ReleaseTransientResources(const TCHAR* InReason)
{
	if (bOwnsRenderTarget && RenderTarget)
	{
		RenderTarget->ReleaseResource();
		RenderTarget = nullptr;
		bOwnsRenderTarget = false;
	}
}

void UPCGRenderTargetData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(Ar);
}

UTexture* UPCGRenderTargetData::GetTexture() const
{
	return RenderTarget;
}

FTextureRHIRef UPCGRenderTargetData::GetTextureRHI() const
{
	// TODO: This makes no attempt to actually acquire the resource after it has been written or, nor does it ensure
	// resource transitions/barriers. Only works if RT is already populated and does not work if RT is rendered every frame.
	FTextureResource* Resource = RenderTarget ? RenderTarget->GetResource() : nullptr;
	return Resource ? Resource->GetTextureRHI() : nullptr;
}

UPCGSpatialData* UPCGRenderTargetData::CopyInternal(FPCGContext* Context) const
{
	UPCGRenderTargetData* NewRenderTargetData = FPCGContext::NewObject_AnyThread<UPCGRenderTargetData>(Context);

	CopyBaseTextureData(NewRenderTargetData);

	// TODO: We can't really support copying owned things at this point, so we will relinquish assumed ownership.
	if (!ensure(!bOwnsRenderTarget))
	{
		const_cast<UPCGRenderTargetData*>(this)->bOwnsRenderTarget = false;
	}

	NewRenderTargetData->RenderTarget = RenderTarget;
	NewRenderTargetData->bOwnsRenderTarget = false;

	return NewRenderTargetData;
}
