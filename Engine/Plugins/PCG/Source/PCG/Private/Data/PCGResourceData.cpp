// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGResourceData.h"

#include "Engine/AssetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGResourceData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoResource, UPCGResourceData)

// @todo_pcg: A preferable API might be to hold the load handle as a member and provide functions to query the state.
TSharedPtr<FStreamableHandle> UPCGResourceData::RequestResourceLoad(bool bAsynchronous) const
{
	if (!bAsynchronous)
	{
		return UAssetManager::GetStreamableManager().RequestSyncLoad(GetResourcePath());
	}
	else
	{
		return UAssetManager::GetStreamableManager().RequestAsyncLoad(GetResourcePath(), nullptr);
	}
}
