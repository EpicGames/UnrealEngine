// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGSpatialData.h"

#include "PCGContext.h"
#include "Data/PCGDifferenceData.h"
#include "Data/PCGIntersectionData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGProjectionData.h"
#include "Data/PCGUnionData.h"
#include "Elements/PCGCollapseElement.h"
#include "Elements/PCGExecuteBlueprint.h"
#include "Elements/PCGMakeConcreteElement.h"

#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSpatialData)

namespace PCGSpatialData
{
	TAutoConsoleVariable<bool> CVarEnablePrepareForSpatialQuery(
		TEXT("pcg.SpatialData.EnablePrepareForSpatialQuery"),
		true,
		TEXT("Enable UPCGSpatialData subclass PrepareForSpatialQuery task scheduling"));
}

FPCGInitializeFromDataParams::FPCGInitializeFromDataParams()
	: FPCGInitializeFromDataParams(nullptr)
{
}

FPCGInitializeFromDataParams::FPCGInitializeFromDataParams(const UPCGSpatialData* InSource)
	: Source(InSource)
	, MetadataInitializeParams(InSource ? InSource->ConstMetadata() : nullptr)
{
}

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoSpatial, UPCGSpatialData)

bool FPCGDataTypeInfoSpatial::SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const
{
	// Spatial can convert to points.
	if (OutputType.IsSameType(EPCGDataType::Point))
	{
		if (OptionalOutConversionSettings)
		{
			*OptionalOutConversionSettings = UPCGCollapseSettings::StaticClass();
		}

		return true;
	}
	else
	{
		return FPCGDataTypeInfo::SupportsConversionTo(ThisType, OutputType, OptionalOutConversionSettings, OptionalOutCompatibilityMessage);
	}
}

PCG_DEFINE_TYPE_INFO_WITHOUT_CLASS(FPCGDataTypeInfoConcrete)

bool FPCGDataTypeInfoConcrete::SupportsConversionFrom(const FPCGDataTypeIdentifier& InputType, const FPCGDataTypeIdentifier& ThisType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const
{
	if (ThisType.IsSameType(EPCGDataType::Concrete) && (InputType.IsSameType(FPCGDataTypeInfo::AsId()) || InputType.IsSameType(FPCGDataTypeInfoSpatial::AsId()) || InputType.IsSameType(FPCGDataTypeInfoComposite::AsId())))
	{
		if (OptionalOutConversionSettings)
		{
			*OptionalOutConversionSettings = UPCGMakeConcreteSettings::StaticClass();
		}

		return true;
	}
	else
	{
		return FPCGDataTypeInfoSpatial::SupportsConversionFrom(InputType, ThisType, OptionalOutConversionSettings, OptionalOutCompatibilityMessage);
	}
}


PCG_DEFINE_TYPE_INFO_WITHOUT_CLASS(FPCGDataTypeInfoComposite)

UPCGSpatialData::UPCGSpatialData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Metadata = ObjectInitializer.CreateDefaultSubobject<UPCGMetadata>(this, TEXT("Metadata"));
	Metadata->SetupDomain(PCGMetadataDomainID::Data, /*bIsDefault=*/true);
}

const UPCGBasePointData* UPCGSpatialData::ToBasePointData(FPCGContext* Context, const FBox& InBounds) const
{
	if (CVarPCGEnablePointArrayData.GetValueOnAnyThread())
	{
		return ToPointArrayData(Context, InBounds);
	}
	else
	{
		return ToPointData(Context, InBounds);
	}
}

const UPCGBasePointData* UPCGSpatialData::ToBasePointDataWithContext(const FPCGBlueprintContextHandle& ContextHandle) const
{
	if (TSharedPtr<FPCGContextHandle> NativeContextHandle = ContextHandle.Handle.Pin())
	{
		if (FPCGContext* Context = NativeContextHandle->GetContext())
		{
			return ToBasePointData(Context);
		}
	}

	UE_LOGF(LogPCG, Error, "UPCGSpatialData::ToBasePointDataWithContext:: Invalid context handle");
	return nullptr;
}

const UPCGPointArrayData* UPCGSpatialData::ToPointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	// @todo_pcg: this is sub-optimal but until we implement this in other spatial types (including the cached types) it works.
	if (const UPCGPointData* PointData = ToPointData(Context))
	{
		UPCGPointArrayData* PointArrayData = FPCGContext::NewObject_AnyThread<UPCGPointArrayData>(Context);

		FPCGInitializeFromDataParams InitializeFromDataParams(PointData);
		InitializeFromDataParams.bInheritSpatialData = false;
		PointArrayData->InitializeFromDataWithParams(InitializeFromDataParams);

		UPCGBasePointData::SetPoints(PointData, PointArrayData, {}, /*bCopyAll=*/true);

		return PointArrayData;
	}
	else
	{
		return nullptr;
	}
}

void FPCGPointDataCache::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	if (CachedPointData)
	{
		const_cast<UPCGBasePointData*>(CachedPointData.Get())->GetResourceSizeEx(CumulativeResourceSize);
	}

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(CachedBoundedPointDataBoxes.GetAllocatedSize() + CachedBoundedPointData.GetAllocatedSize());

	for (const UPCGBasePointData* Data : CachedBoundedPointData)
	{
		if (Data)
		{
			const_cast<UPCGBasePointData*>(Data)->GetResourceSizeEx(CumulativeResourceSize);
		}
	}
}

void UPCGSpatialDataWithPointCache::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	PointDataCache.GetResourceSizeEx(CumulativeResourceSize);
	PointArrayDataCache.GetResourceSizeEx(CumulativeResourceSize);
}

const UPCGBasePointData* FPCGPointDataCache::ToBasePointDataInternal(FPCGContext* Context, const FBox& InBounds, bool bSupportsBoundedPointData, PCG::FLock& InCacheLock, TFunctionRef<const UPCGBasePointData*(FPCGContext*, const FBox&)> CreatePointDataFunc)
{
	if (InBounds.IsValid && bSupportsBoundedPointData)
	{
		const UPCGBasePointData* BoundedPointData = nullptr;
		{
			PCG::TScopeLock ScopeLock(InCacheLock);
			check(CachedBoundedPointDataBoxes.Num() == CachedBoundedPointData.Num());
			for (int CachedDataIndex = 0; CachedDataIndex < CachedBoundedPointDataBoxes.Num(); ++CachedDataIndex)
			{
				if (InBounds.Equals(CachedBoundedPointDataBoxes[CachedDataIndex]))
				{
					BoundedPointData = CachedBoundedPointData[CachedDataIndex];
					break;
				}
			}

			if (!BoundedPointData)
			{
				BoundedPointData = CreatePointDataFunc(Context, InBounds);
				CachedBoundedPointDataBoxes.Add(InBounds);
				CachedBoundedPointData.Add(BoundedPointData);
			}
		}

		return BoundedPointData;
	}
	else
	{
		if (!CachedPointData)
		{
			PCG::TScopeLock ScopeLock(InCacheLock);
			if (!CachedPointData)
			{
				CachedPointData = CreatePointDataFunc(Context, InBounds);
			}
		}

		return CachedPointData;
	}
}

const UPCGPointData* UPCGSpatialDataWithPointCache::ToPointData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointData>(PointDataCache.ToBasePointDataInternal(Context, InBounds, SupportsBoundedPointData(), CacheLock, [this](FPCGContext* InContext, const FBox& InBounds) { return CreatePointData(InContext, InBounds); }), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGSpatialDataWithPointCache::ToPointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointArrayData>(PointArrayDataCache.ToBasePointDataInternal(Context, InBounds, SupportsBoundedPointData(), CacheLock, [this](FPCGContext* InContext, const FBox& InBounds) { return CreatePointArrayData(InContext, InBounds); }), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGSpatialDataWithPointCache::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	// Default implementation (not optimal, should be overloaded in sub classes)
	return CreatePointData(Context, InBounds)->ToPointArrayData(Context);
}

float UPCGSpatialData::GetDensityAtPosition(const FVector& InPosition) const
{
	FPCGPoint TemporaryPoint;
	if (SamplePoint(FTransform(InPosition), FBox::BuildAABB(FVector::ZeroVector, FVector::ZeroVector), TemporaryPoint, nullptr))
	{
		return TemporaryPoint.Density;
	}
	else
	{
		return 0;
	}
}

bool UPCGSpatialData::K2_SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	return SamplePoint(InTransform, InBounds, OutPoint, OutMetadata);
}

bool UPCGSpatialData::K2_ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	return ProjectPoint(InTransform, InBounds, InParams, OutPoint, OutMetadata);
}

void UPCGSpatialData::SamplePoints(const TArrayView<const TPair<FTransform, FBox>>& InSamples, const TArrayView<FPCGPoint>& OutPoints, UPCGMetadata* OutMetadata) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSpatialData::SamplePoints);
	check(InSamples.Num() == OutPoints.Num());
	for(int Index = 0; Index < InSamples.Num(); ++Index)
	{
		const TPair<FTransform, FBox>& Sample = InSamples[Index];
		FPCGPoint& OutPoint = OutPoints[Index];

		if (!SamplePoint(Sample.Key, Sample.Value, OutPoint, OutMetadata))
		{
			OutPoint.Density = 0;
		}
	}
}

bool UPCGSpatialData::ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// Fallback implementation - calls SamplePoint because SamplePoint was being used for projection previously.
	
	// TODO This is a crutch until we implement ProjectPoint everywhere

	const bool bResult = SamplePoint(InTransform, InBounds, OutPoint, OutMetadata);

	// Respect the projection params that we can at this point given our available data (InTransform)

	if (!InParams.bProjectPositions)
	{
		OutPoint.Transform.SetLocation(InTransform.GetLocation());
	}

	if (!InParams.bProjectRotations)
	{
		OutPoint.Transform.SetRotation(InTransform.GetRotation());
	}

	if (!InParams.bProjectScales)
	{
		OutPoint.Transform.SetScale3D(InTransform.GetScale3D());
	}

	return bResult;
}

void UPCGSpatialData::ProjectPoints(const TArrayView<const TPair<FTransform, FBox>>& InSamples, const FPCGProjectionParams& InParams, const TArrayView<FPCGPoint>& OutPoints, UPCGMetadata* OutMetadata) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSpatialData::ProjectPoints);
	check(InSamples.Num() == OutPoints.Num());
	for (int Index = 0; Index < InSamples.Num(); ++Index)
	{
		const TPair<FTransform, FBox>& Sample = InSamples[Index];
		FPCGPoint& OutPoint = OutPoints[Index];

		if (!ProjectPoint(Sample.Key, Sample.Value, InParams, OutPoint, OutMetadata))
		{
			OutPoint.Density = 0;
		}
	}
}

UPCGIntersectionData* UPCGSpatialData::K2_IntersectWith(const UPCGSpatialData* InOther) const
{
	return IntersectWith(UPCGBlueprintBaseElement::ResolveContext(), InOther);
}

UPCGIntersectionData* UPCGSpatialData::IntersectWith(FPCGContext* InContext, const UPCGSpatialData* InOther) const
{
	UPCGIntersectionData* IntersectionData = FPCGContext::NewObject_AnyThread<UPCGIntersectionData>(InContext);
	IntersectionData->Initialize(this, InOther);

	return IntersectionData;
}

UPCGSpatialData* UPCGSpatialData::K2_ProjectOn(const UPCGSpatialData* InOther, const FPCGProjectionParams& InParams) const
{
	return ProjectOn(UPCGBlueprintBaseElement::ResolveContext(), InOther, InParams);
}

UPCGSpatialData* UPCGSpatialData::ProjectOn(FPCGContext* InContext, const UPCGSpatialData* InOther, const FPCGProjectionParams& InParams) const
{
	// Check necessary conditions. Fail to project -> return copy of projection source, i.e. projection not performed.
	if (!InOther)
	{
		UE_LOGF(LogPCG, Warning, "No projection target specified, no projection will occur");
		return DuplicateData(InContext);
	}

	if (GetDimension() > InOther->GetDimension())
	{
		UE_LOGF(LogPCG, Error, "Dimension of projection source (%d) must be less than or equal to that of the projection target (%d)", GetDimension(), InOther->GetDimension());
		return DuplicateData(InContext);
	}

	const UPCGSpatialData* ConcreteTarget = InOther->FindFirstConcreteShapeFromNetwork();
	if (!ConcreteTarget)
	{
		UE_LOGF(LogPCG, Error, "Could not find a concrete shape in the target data to project onto.");
		return DuplicateData(InContext);
	}

	UPCGProjectionData* ProjectionData = FPCGContext::NewObject_AnyThread<UPCGProjectionData>(InContext);
	ProjectionData->Initialize(this, ConcreteTarget, InParams);

	return ProjectionData;
}

UPCGUnionData* UPCGSpatialData::K2_UnionWith(const UPCGSpatialData* InOther) const
{
	return UnionWith(UPCGBlueprintBaseElement::ResolveContext(), InOther);
}

UPCGUnionData* UPCGSpatialData::UnionWith(FPCGContext* InContext, const UPCGSpatialData* InOther) const
{
	UPCGUnionData* UnionData = FPCGContext::NewObject_AnyThread<UPCGUnionData>(InContext);
	UnionData->Initialize(this, InOther);

	return UnionData;
}

UPCGDifferenceData* UPCGSpatialData::K2_Subtract(const UPCGSpatialData* InOther) const
{
	return Subtract(UPCGBlueprintBaseElement::ResolveContext(), InOther);
}

UPCGDifferenceData* UPCGSpatialData::Subtract(FPCGContext* InContext, const UPCGSpatialData* InOther) const
{
	UPCGDifferenceData* DifferenceData = FPCGContext::NewObject_AnyThread<UPCGDifferenceData>(InContext);
	DifferenceData->Initialize(this);
	DifferenceData->AddDifference(InContext, InOther);

	return DifferenceData;
}

UPCGMetadata* UPCGSpatialData::CreateEmptyMetadata()
{
	if (Metadata)
	{
		UE_LOGF(LogPCG, Warning, "Spatial data already had metadata");
	}

	Metadata = NewObject<UPCGMetadata>(this);
	return Metadata;
}

void UPCGSpatialData::InitializeMetadata(const FPCGInitializeFromDataParams& InParams)
{
	if (InParams.bInheritMetadata)
	{
		InitializeMetadataInternal(InParams);
	}
}

void UPCGSpatialData::InitializeMetadataInternal(const FPCGInitializeFromDataParams& InParams)
{
	check(InParams.Source);
	InParams.Source->InitializeTargetMetadata(InParams, Metadata);
}

void UPCGSpatialData::InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const
{
	check(InParams.bInheritMetadata);
	check(MetadataToInitialize);

	// Making sure the metadata initialized params are setup correctly.
	const FPCGMetadataInitializeParams* ParamsToUse = &InParams.MetadataInitializeParams;
	FPCGMetadataInitializeParams TemporaryParams{nullptr};
	if (InParams.MetadataInitializeParams.Parent != InParams.Source->ConstMetadata())
	{
		TemporaryParams = InParams.MetadataInitializeParams;
		TemporaryParams.Parent = InParams.Source->ConstMetadata();
		ParamsToUse = &TemporaryParams;
	}

	// If the metadata to initialize already have a parent, we can't initialize it again, so just add the attributes.
	if (MetadataToInitialize->GetParent())
	{
		MetadataToInitialize->AddAttributes(*ParamsToUse);
	}
	else
	{
		MetadataToInitialize->Initialize(*ParamsToUse);
	}
}

void UPCGSpatialData::InitializeSpatialDataInternal(const FPCGInitializeFromDataParams& InParams)
{
	if (InParams.Source && TargetActor.IsExplicitlyNull())
	{
		TargetActor = InParams.Source->TargetActor;
	}
}

void UPCGSpatialData::InitializeFromData(const UPCGSpatialData* InSource, const UPCGMetadata* InMetadataParentOverride, bool bInheritMetadata, bool bInheritAttributes)
{
	FPCGInitializeFromDataParams Params(InSource);
	Params.bInheritMetadata = bInheritMetadata;
	Params.bInheritAttributes = bInheritAttributes;
	Params.bInheritSpatialData = true;
	
	InitializeSpatialDataInternal(Params);
	if (InSource)
	{
		if (InMetadataParentOverride)
		{
			// Log a warning and ask people to change it
			// UE_DEPRECATED(5.6, "To be removed")
			UE_LOGF(LogPCG, Warning, "MetadataParentOverride is deprecated and should not be used anymore. The source data is now responsible to initialize it.");
			Metadata->Initialize(InMetadataParentOverride, bInheritAttributes);
		}
		else
		{
			InitializeMetadata(Params);
		}
	}
}

void UPCGSpatialData::InitializeFromDataWithParams(const FPCGInitializeFromDataParams& InParams)
{
	// Make sure to keep InitializeFromData on par, anything implemented here should be replicated in InitializeFromData.
	// We have to have a deprecation path for InMetadataParentOverride, but we don't want to pollute FPCGInitializeFromDataParams with deprecation
	// as there was no major release in between this change and when it was added.
	InitializeSpatialDataInternal(InParams);
	if (InParams.Source)
	{
		InitializeMetadata(InParams);
	}
}

UPCGSpatialData* UPCGSpatialData::DuplicateData(FPCGContext* Context, bool bInitializeMetadata) const
{
	UPCGSpatialData* NewSpatialData = CopyInternal(Context);
	check(NewSpatialData);
	check(NewSpatialData->Metadata);
	
	FPCGInitializeFromDataParams Params(this);
	Params.bInheritMetadata = bInitializeMetadata;
	Params.bIsDuplicatingData = true;
	NewSpatialData->InitializeFromDataWithParams(Params);

	if (bHasCachedLastSelector)
	{
		NewSpatialData->SetLastSelector(CachedLastSelector);
	}

	return NewSpatialData;
}

void UPCGSpatialData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	if (Metadata)
	{
		// Can impact results of downstream node execution.
		FName LatestAttribute = Metadata->GetLatestAttributeNameOrNone();
		Ar << LatestAttribute;
	}
}

bool UPCGSpatialData::HasCachedLastSelector() const
{
	return bHasCachedLastSelector || (Metadata && Metadata->GetAttributeCount() > 0);
}

FPCGAttributePropertyInputSelector UPCGSpatialData::GetCachedLastSelector() const
{
	if (bHasCachedLastSelector)
	{
		return CachedLastSelector;
	}

	FPCGAttributePropertyInputSelector TempSelector{};

	// If we have attribute and no last selector, create a cached last selector on the latest attribute, to catch "CreateAttribute" calls that didn't use accessors.
	if (Metadata && Metadata->GetAttributeCount() > 0)
	{
		TempSelector.SetAttributeName(Metadata->GetLatestAttributeNameOrNone());
	}

	return TempSelector;
}

void UPCGSpatialData::SetLastSelector(const FPCGAttributePropertySelector& InSelector)
{
	// Check that it is not a Last or Source selector
	if (InSelector.GetSelection() == EPCGAttributePropertySelection::Attribute &&
		(InSelector.GetAttributeName() == PCGMetadataAttributeConstants::LastAttributeName
			|| InSelector.GetAttributeName() == PCGMetadataAttributeConstants::LastCreatedAttributeName
			|| InSelector.GetAttributeName() == PCGMetadataAttributeConstants::SourceAttributeName
			|| InSelector.GetAttributeName() == PCGMetadataAttributeConstants::SourceNameAttributeName))
	{
		return;
	}

	bHasCachedLastSelector = true;
	CachedLastSelector.ImportFromOtherSelector(InSelector);
}
