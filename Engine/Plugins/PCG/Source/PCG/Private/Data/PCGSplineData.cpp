// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGSplineData.h"

#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPolyLineData.h"
#include "Data/PCGProjectionData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGCreateSurfaceFromSpline.h"
#include "Elements/PCGSplineSampler.h"
#include "Graph/PCGGraphCache.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGSplineAccessor.h"

#include "Components/SplineComponent.h"
#include "Serialization/ArchiveCrc32.h"
#include "Serialization/ArchiveCountMem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplineData)

#define LOCTEXT_NAMESPACE "PCGSplineData"

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoSpline, UPCGSplineData)

bool FPCGDataTypeInfoSpline::SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const
{
	// Spline can convert to surface
	if (OutputType.IsSameType(EPCGDataType::Surface))
	{
		if (OptionalOutConversionSettings)
		{
			*OptionalOutConversionSettings = UPCGCreateSurfaceFromSplineSettings::StaticClass();
		}

		return true;
	}
	else
	{
		return FPCGDataTypeInfoPolyline::SupportsConversionTo(ThisType, OutputType, OptionalOutConversionSettings, OptionalOutCompatibilityMessage);
	}
}

UPCGSplineData::UPCGSplineData()
{
	check(Metadata);
	Metadata->SetupDomain(PCGMetadataDomainID::Elements, /*bIsDefault=*/true);
}

void UPCGSplineData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Lightweight - skip
	if (FPCGGraphCache::CacheSizeDetailLevel() == 2)
	{
		return;
	}

	FArchiveCountMem Archive(this);
	FPCGSplineStruct::StaticStruct()->SerializeItem(FStructuredArchiveFromArchive(Archive).GetSlot(), &SplineStruct, /*Defaults*/nullptr);
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Archive.GetNum() + sizeof(FBox));
}

void UPCGSplineData::Initialize(const USplineComponent* InSpline)
{
	check(InSpline);

	SplineStruct.Initialize(InSpline);

	CachedBounds = PCGHelpers::GetActorBounds(InSpline->GetOwner());

	// Expand bounds by the radius of points, otherwise sections of the curve that are close
	// to the bounds will report an invalid density.
	FVector SplinePointsRadius = FVector::ZeroVector;
	const FInterpCurveVector& SplineScales = SplineStruct.GetSplinePointsScale();
	for (const FInterpCurvePoint<FVector>& SplineScale : SplineScales.Points)
	{
		SplinePointsRadius = FVector::Max(SplinePointsRadius, SplineScale.OutVal.GetAbs());
	}

	CachedBounds = CachedBounds.ExpandBy(SplinePointsRadius, SplinePointsRadius);
}

void UPCGSplineData::Initialize(const TArray<FSplinePoint>& InSplinePoints, bool bIsClosedLoop, const FTransform& InTransform, TArray<PCGMetadataEntryKey> InOptionalEntryKeys)
{
	SplineStruct.Initialize(InSplinePoints, bIsClosedLoop, InTransform, std::move(InOptionalEntryKeys));

	CachedBounds = SplineStruct.GetBounds();

	// Expand bounds by the radius of points, otherwise sections of the curve that are close
	// to the bounds will report an invalid density.
	FVector SplinePointsRadius = FVector::ZeroVector;
	const FInterpCurveVector& SplineScales = SplineStruct.GetSplinePointsScale();
	for (const FInterpCurvePoint<FVector>& SplineScale : SplineScales.Points)
	{
		SplinePointsRadius = FVector::Max(SplinePointsRadius, SplineScale.OutVal.GetAbs());
	}

	CachedBounds = CachedBounds.ExpandBy(SplinePointsRadius, SplinePointsRadius);
	CachedBounds = CachedBounds.TransformBy(InTransform);
}

void UPCGSplineData::K2_Initialize(const TArray<FSplinePoint>& InSplinePoints, bool bInClosedLoop, const FTransform& InTransform,
	TArray<int64> InOptionalEntryKeys)
{
	Initialize(InSplinePoints, bInClosedLoop, InTransform, std::move(InOptionalEntryKeys));
}

void UPCGSplineData::Initialize(const FPCGSplineStruct& InSplineStruct)
{
	SplineStruct = InSplineStruct;
	CachedBounds = SplineStruct.GetBounds();

	// Expand bounds by the radius of points, otherwise sections of the curve that are close
	// to the bounds will report an invalid density.
	FVector SplinePointsRadius = FVector::ZeroVector;
	const FInterpCurveVector& SplineScales = SplineStruct.GetSplinePointsScale();
	for (const FInterpCurvePoint<FVector>& SplineScale : SplineScales.Points)
	{
		SplinePointsRadius = FVector::Max(SplinePointsRadius, SplineScale.OutVal.GetAbs());
	}

	CachedBounds = CachedBounds.ExpandBy(SplinePointsRadius, SplinePointsRadius);
	CachedBounds = CachedBounds.TransformBy(SplineStruct.Transform);
}

void UPCGSplineData::ApplyTo(USplineComponent* InSplineComponent) const
{
	SplineStruct.ApplyTo(InSplineComponent);
}

void UPCGSplineData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	if (!bFullDataCrc)
	{
		// Fallback to UID
		AddUIDToCrc(Ar);
		return;
	}

	if (Metadata)
	{
		Metadata->AddToCrc(Ar, bFullDataCrc);
	}

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	Ar << SplineStruct;
}

FPCGMetadataDomainID UPCGSplineData::GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const
{
	const FName DomainName = InSelector.GetDomainName();

	if (DomainName == PCGSplineData::ControlPointDomainName)
	{
		return PCGMetadataDomainID::Elements;
	}
	else
	{
		return Super::GetMetadataDomainIDFromSelector(InSelector);
	}
}

bool UPCGSplineData::SetDomainFromDomainID(const FPCGMetadataDomainID& InDomainID, FPCGAttributePropertySelector& InOutSelector) const
{
	if (InDomainID == PCGMetadataDomainID::Elements)
	{
		InOutSelector.SetDomainName(PCGSplineData::ControlPointDomainName, /*bResetExtraNames=*/false);
		return true;
	}
	else
	{
		return Super::SetDomainFromDomainID(InDomainID, InOutSelector);
	}
}

void UPCGSplineData::InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const
{
	check(MetadataToInitialize);

	Super::InitializeTargetMetadata(InParams, MetadataToInitialize);

	FPCGMetadataDomain* MetadataDomain = MetadataToInitialize->GetMetadataDomain(PCGMetadataDomainID::Elements);
	if (!MetadataDomain)
	{
		return;
	}

	for (const FName& ChannelName : SplineStruct.GetAttributeChannelNames())
	{
		MetadataDomain->FindOrCreateAttribute<float>(ChannelName, /*DefaultValue=*/0.f);
	}
}

FTransform UPCGSplineData::GetTransform() const
{
	return SplineStruct.GetTransform();
}

int UPCGSplineData::GetNumSegments() const
{
	return SplineStruct.GetNumberOfSplineSegments();
}

FVector::FReal UPCGSplineData::GetSegmentLength(int SegmentIndex) const
{
	if (SegmentIndex >= SplineStruct.GetNumberOfSplineSegments() || SegmentIndex < 0)
	{
		return 0.0;
	}

	return SplineStruct.GetDistanceAlongSplineAtSplinePoint(SegmentIndex + 1) - SplineStruct.GetDistanceAlongSplineAtSplinePoint(SegmentIndex);
}

FVector UPCGSplineData::GetLocationAtDistance(int SegmentIndex, FVector::FReal Distance, bool bWorldSpace) const
{
	return SplineStruct.GetLocationAtDistanceAlongSpline(SplineStruct.GetDistanceAlongSplineAtSplinePoint(SegmentIndex) + Distance, bWorldSpace ? ESplineCoordinateSpace::World : ESplineCoordinateSpace::Local);
}

FTransform UPCGSplineData::GetTransformAtDistance(int SegmentIndex, FVector::FReal Distance, bool bWorldSpace, FBox* OutBounds) const
{
	if (OutBounds)
	{
		*OutBounds = FBox::BuildAABB(FVector::ZeroVector, FVector::OneVector);
	}

	return SplineStruct.GetTransformAtDistanceAlongSpline(SplineStruct.GetDistanceAlongSplineAtSplinePoint(SegmentIndex) + Distance, bWorldSpace ? ESplineCoordinateSpace::World : ESplineCoordinateSpace::Local, /*bUseScale=*/true);
}

FVector::FReal UPCGSplineData::GetCurvatureAtDistance(int SegmentIndex, FVector::FReal Distance) const
{
	const float FullDistance = SplineStruct.GetDistanceAlongSplineAtSplinePoint(SegmentIndex) + Distance;
	const float Param = SplineStruct.GetInputKeyAtDistanceAlongSpline(FullDistance);

	// Since we need the first derivative (e.g. very similar to direction) to have its norm, we'll get the value directly
	const FVector FirstDerivative = SplineStruct.GetSplinePointsPosition().EvalDerivative(Param, FVector::ZeroVector);
	const FVector::FReal FirstDerivativeLength = FMath::Max(FirstDerivative.Length(), UE_DOUBLE_SMALL_NUMBER);
	const FVector ForwardVector = FirstDerivative / FirstDerivativeLength;
	const FVector SecondDerivative = SplineStruct.GetSplinePointsPosition().EvalSecondDerivative(Param, FVector::ZeroVector);
	// Orthogonalize the second derivative and obtain the curvature vector
	const FVector CurvatureVector = SecondDerivative - (SecondDerivative | ForwardVector) * ForwardVector;
	
	// Finally, the curvature is the ratio of the norms of the curvature vector over the first derivative norm
	const FVector::FReal Curvature = CurvatureVector.Length() / FirstDerivativeLength;

	// Compute sign based on sign of curvature vs. right axis
	const FVector RightVector = SplineStruct.GetRightVectorAtSplineInputKey(Param, ESplineCoordinateSpace::Local);
	return FMath::Sign(RightVector | CurvatureVector) * Curvature;
}

float UPCGSplineData::GetInputKeyAtDistance(int SegmentIndex, FVector::FReal Distance) const
{
	const float FullDistance = GetDistanceAtSegmentStart(SegmentIndex) + Distance;
	return SplineStruct.GetInputKeyAtDistanceAlongSpline(FullDistance);
}

float UPCGSplineData::GetInputKeyAtAlpha(float InAlpha) const
{
	// Alpha is a measure of progression along the spline, where segments have an equal portion.
	const float Alpha = FMath::Clamp(InAlpha, 0.0f, 1.0f);
	const int NumSegments = GetNumSegments();

	const float UnnormalizedAlpha = Alpha * NumSegments;
	const int SegmentIndex = static_cast<int>(UnnormalizedAlpha);
	const float Remainder = UnnormalizedAlpha - static_cast<float>(SegmentIndex);

	return GetInputKeyAtDistance(SegmentIndex, Remainder * GetSegmentLength(SegmentIndex));
}

void UPCGSplineData::GetTangentsAtSegmentStart(int SegmentIndex, FVector& OutArriveTangent, FVector& OutLeaveTangent) const
{
	check(SplineStruct.GetSplinePointsPosition().Points.IsValidIndex(SegmentIndex));
	OutArriveTangent = SplineStruct.GetSplinePointsPosition().Points[SegmentIndex].ArriveTangent;
	OutLeaveTangent = SplineStruct.GetSplinePointsPosition().Points[SegmentIndex].LeaveTangent;
}

FVector::FReal UPCGSplineData::GetDistanceAtSegmentStart(int SegmentIndex) const
{
	return SplineStruct.GetDistanceAlongSplineAtSplinePoint(SegmentIndex);
}

FVector UPCGSplineData::GetLocationAtAlpha(float Alpha) const
{
	return SplineStruct.GetLocationAtSplineInputKey(GetInputKeyAtAlpha(Alpha), ESplineCoordinateSpace::World);
}

FTransform UPCGSplineData::GetTransformAtAlpha(float Alpha) const
{
	return SplineStruct.GetTransformAtSplineInputKey(GetInputKeyAtAlpha(Alpha), ESplineCoordinateSpace::World);
}

float UPCGSplineData::GetLoopKeyOffset() const
{
	return SplineStruct.GetLoopKeyOffset();
}

void UPCGSplineData::WriteMetadataToPoint(float InputKey, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	return WriteMetadataToEntry(InputKey, OutPoint.MetadataEntry, OutMetadata);
}

void UPCGSplineData::WriteMetadataToEntry(float InputKey, PCGMetadataEntryKey& OutEntryKey, UPCGMetadata* OutMetadata) const
{
	if (!OutMetadata || !ensure(Metadata))
	{
		return;
	}

	const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomain(PCGMetadataDomainID::Elements);
	FPCGMetadataDomain* OutMetadataDomain = OutMetadata->GetMetadataDomain(PCGMetadataDomainID::Elements);

	if (!MetadataDomain || !OutMetadataDomain)
	{
		return;
	}
	
	// TODO: Better interpolation, for now just interpolate values that can be interpolated, otherwise treat the attribute as constant along the segment.
	TConstArrayView<PCGMetadataEntryKey> EntryKeys = SplineStruct.GetConstControlPointsEntryKeys();
	if (!EntryKeys.IsEmpty())
	{
		const TTuple<int, float> PreviousIndexAndKey = SplineStruct.GetSegmentStartIndexAndKeyAtInputKey(InputKey);
		const int PreviousIndex = PreviousIndexAndKey.Get<0>();
		const float PreviousInputKey = PreviousIndexAndKey.Get<1>();

		if (PreviousIndex == INDEX_NONE)
		{
			return;
		}
		
		int32 NextIndex = PreviousIndex + 1;
		float NextInputKey = 0;
		// In case the spline is not closed, we cannot go further than the last point.
		// In case the spline is closed, we have to get the input key first (to get the input key associated with the last point, not the first point)
		// then reset it to zero if it is the last point.
		if (!IsClosed())
		{
			NextIndex = FMath::Min(NextIndex, SplineStruct.GetNumberOfPoints() - 1);
			NextInputKey = SplineStruct.GetInputKeyAtSegmentStart(NextIndex);
		}
		else
		{
			NextInputKey = SplineStruct.GetInputKeyAtSegmentStart(NextIndex);
			NextIndex %= SplineStruct.GetNumberOfPoints();
		}

		check(EntryKeys.IsValidIndex(PreviousIndex) && EntryKeys.IsValidIndex(NextIndex));
		
		OutMetadataDomain->InitializeOnSet(OutEntryKey, EntryKeys[PreviousIndex], MetadataDomain);
		
		if (PreviousIndex != NextIndex && !FMath::IsNearlyEqual(PreviousInputKey, NextInputKey))
		{
			const float Alpha = (InputKey - PreviousInputKey) / (NextInputKey - PreviousInputKey);
			TStaticArray<TPair<PCGMetadataEntryKey, float>, 2> Coefficients;
			Coefficients[0] = {EntryKeys[PreviousIndex], 1.0f - Alpha};
			Coefficients[1] = {EntryKeys[NextIndex], Alpha};
			OutMetadataDomain->ComputeWeightedAttribute(OutEntryKey, Coefficients, MetadataDomain);
		}
	}

	if (const TArray<FName>& ChannelNames = SplineStruct.GetAttributeChannelNames(); ChannelNames.Num() > 0)
	{
		OutMetadataDomain->InitializeOnSet(OutEntryKey);

		// Sample attribute channels,
		for (const FName& ChannelName : ChannelNames)
		{
			if (FPCGMetadataAttribute<float>* UserAttribute = OutMetadataDomain->FindOrCreateAttribute<float>(ChannelName, /*DefaultValue=*/0.f))
			{
				UserAttribute->SetValue(OutEntryKey, SplineStruct.GetAttributeValue(InputKey, ChannelName));
			}
		}
	}
}

TUniquePtr<IPCGAttributeAccessor> UPCGSplineData::CreateStaticAccessor(const FPCGAttributePropertySelector& InSelector, bool bQuiet)
{
	static const FStructProperty* PCGSplineStructProperty = CastFieldChecked<FStructProperty>(UPCGSplineData::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPCGSplineData, SplineStruct)));

	const FName DomainName = InSelector.GetDomainName();

	if (InSelector.GetSelection() == EPCGAttributePropertySelection::Property && (DomainName.IsNone() || DomainName == PCGSplineData::ControlPointDomainName))
	{
		const FName PropertyName = InSelector.GetName();
		
		if (PropertyName == TEXT("Position"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FVector, EPCGControlPointsAccessorTarget::Location, /*bWorldCoordinates=*/true>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("LocalPosition"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FVector, EPCGControlPointsAccessorTarget::Location, /*bWorldCoordinates=*/false>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("Rotation"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FQuat, EPCGControlPointsAccessorTarget::Rotation, /*bWorldCoordinates=*/true>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("LocalRotation"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FQuat, EPCGControlPointsAccessorTarget::Rotation, /*bWorldCoordinates=*/false>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("Scale"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FVector, EPCGControlPointsAccessorTarget::Scale, /*bWorldCoordinates=*/true>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("LocalScale"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FVector, EPCGControlPointsAccessorTarget::Scale, /*bWorldCoordinates=*/false>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("Transform"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FTransform, EPCGControlPointsAccessorTarget::Transform, /*bWorldCoordinates=*/true>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("LocalTransform"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FTransform, EPCGControlPointsAccessorTarget::Transform, /*bWorldCoordinates=*/false>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("ArriveTangent"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FVector, EPCGControlPointsAccessorTarget::ArriveTangent, /*bWorldCoordinates=*/false>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("LeaveTangent"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FVector, EPCGControlPointsAccessorTarget::LeaveTangent, /*bWorldCoordinates=*/false>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("InterpType"))
		{
			return MakeUnique<FPCGControlPointsAccessor<FPCGEnumPropertyAccessor::Type, EPCGControlPointsAccessorTarget::InterpMode, /*bWorldCoordinates=*/false>>(PCGSplineStructProperty);
		}

		if (!bQuiet)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FailCreateAccessor", "Property {0} is not valid for a control point."), FText::FromName(PropertyName)));
		}
	}
	else if (InSelector.GetSelection() == EPCGAttributePropertySelection::Property && DomainName == PCGDataConstants::DataDomainName)
	{
		const FName PropertyName = InSelector.GetName();
		
		if (PropertyName == TEXT("SplineTransform"))
		{
			return MakeUnique<FPCGSplineAccessorV2<FTransform, EPCGSplineDataProperties::SplineTransform>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("IsClosed"))
		{
			return MakeUnique<FPCGSplineAccessorV2<bool, EPCGSplineDataProperties::IsClosed>>(PCGSplineStructProperty);
		}
		else if (PropertyName == TEXT("Length"))
		{
			return MakeUnique<FPCGSplineAccessorV2<double, EPCGSplineDataProperties::Length>>(PCGSplineStructProperty);
		}
	}
	
	return {};
}

TUniquePtr<IPCGAttributeAccessorKeys> UPCGSplineData::CreateAccessorKeys(const FPCGAttributePropertySelector& InSelector, bool bQuiet)
{
    const EPCGAttributePropertySelection Selection = InSelector.GetSelection();
	const FPCGMetadataDomainID DomainID = GetMetadataDomainIDFromSelector(InSelector);

	// Global data
	if (DomainID == PCGMetadataDomainID::Data && (Selection == EPCGAttributePropertySelection::Property || Selection == EPCGAttributePropertySelection::ExtraProperty))
	{
		return MakeUnique<FPCGAttributeAccessorKeysSplineData>(this, /*bInGlobalData=*/true);
	}

	// Control point data
	if ((DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements) && (Selection == EPCGAttributePropertySelection::Property || Selection == EPCGAttributePropertySelection::ExtraProperty))
	{
		return MakeUnique<FPCGAttributeAccessorKeysSplineData>(this, /*bInGlobalData=*/false);
	}

	// Metadata in the control point domain
	if ((DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements) && Selection == EPCGAttributePropertySelection::Attribute)
	{
		return MakeUnique<FPCGAttributeAccessorKeysSplineDataEntries>(this);
	}
	
	return {};
}

TUniquePtr<const IPCGAttributeAccessorKeys> UPCGSplineData::CreateConstAccessorKeys(const FPCGAttributePropertySelector& InSelector, bool bQuiet) const
{
	const EPCGAttributePropertySelection Selection = InSelector.GetSelection();
	const FPCGMetadataDomainID DomainID = GetMetadataDomainIDFromSelector(InSelector);

	// Global data
	if (DomainID == PCGMetadataDomainID::Data && (Selection == EPCGAttributePropertySelection::Property || Selection == EPCGAttributePropertySelection::ExtraProperty))
	{
		return MakeUnique<const FPCGAttributeAccessorKeysSplineData>(this, /*bInGlobalData=*/true);
	}

	// Control point data
	if ((DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements) && (Selection == EPCGAttributePropertySelection::Property || Selection == EPCGAttributePropertySelection::ExtraProperty))
	{
		return MakeUnique<const FPCGAttributeAccessorKeysSplineData>(this, /*bInGlobalData=*/false);
	}

	// Metadata in the control point domain
	if ((DomainID.IsDefault() || DomainID == PCGMetadataDomainID::Elements) && Selection == EPCGAttributePropertySelection::Attribute)
	{
		return MakeUnique<const FPCGAttributeAccessorKeysSplineDataEntries>(this);
	}
	
	return {};
}

FPCGAttributeAccessorMethods UPCGSplineData::GetSplineAccessorMethods()
{
	auto CreateAccessorFunc = [](UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<IPCGAttributeAccessor>
	{
		return CreateStaticAccessor(InSelector, bQuiet);
	};

	auto CreateConstAccessorFunc = [](const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<const IPCGAttributeAccessor>
	{
		return CreateStaticAccessor(InSelector, bQuiet);
	};

	auto CreateAccessorKeysFunc = [](UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<IPCGAttributeAccessorKeys>
	{
		return CastChecked<UPCGSplineData>(InData)->CreateAccessorKeys(InSelector, bQuiet);
	};
	
	auto CreateConstAccessorKeysFunc = [](const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet) -> TUniquePtr<const IPCGAttributeAccessorKeys>
	{
		return CastChecked<UPCGSplineData>(InData)->CreateConstAccessorKeys(InSelector, bQuiet);
	};
	
	FPCGAttributeAccessorMethods Methods
	{
		.CreateAccessorFunc = CreateAccessorFunc,
		.CreateConstAccessorFunc = CreateConstAccessorFunc,
		.CreateAccessorKeysFunc = CreateAccessorKeysFunc,
		.CreateConstAccessorKeysFunc = CreateConstAccessorKeysFunc
	};

#if WITH_EDITOR
	TArray<FText, TInlineAllocator<2>> Menus = {LOCTEXT("ControlPointSelectorMenuEntry", "Spline")};
	FText& SubMenu = Menus.Emplace_GetRef();
	
	SubMenu = LOCTEXT("ControlPointSelectorMenuEntryPoints", "Control Points");
	Methods.FillSelectorMenuEntryFromEnum<EPCGSplineStructProperties>(Menus);
	
	SubMenu = LOCTEXT("ControlPointSelectorMenuEntryGlobal", "Global");
	Methods.FillSelectorMenuEntryFromEnum<EPCGSplineDataProperties>(Menus);
#endif // WITH_EDITOR

	return Methods;
}

const UPCGPointData* UPCGSplineData::CreatePointData(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSplineData::CreatePointData);

	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGSplineData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSplineData::CreatePointArrayData);

	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGSplineData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);

	FPCGInitializeFromDataParams InitializeFromDataParams(this);
	InitializeFromDataParams.bInheritSpatialData = false;
	Data->InitializeFromDataWithParams(InitializeFromDataParams);

	FPCGSplineSamplerParams SamplerParams;
	SamplerParams.Mode = EPCGSplineSamplingMode::Distance;

	PCGSplineSamplerHelpers::SampleLineData(Context, this, /*InBoundingShape=*/nullptr, /*InProjectionTarget=*/nullptr, /*InProjectionParams=*/{}, SamplerParams, Data);
	UE_LOGF(LogPCG, Verbose, "Spline generated %d points", Data->GetNumPoints());

	return Data;
}

TArray<FSplinePoint> UPCGSplineData::GetSplinePoints() const
{
	const FInterpCurveVector& ControlPointsPosition = SplineStruct.GetSplinePointsPosition();
	const FInterpCurveQuat& ControlPointsRotation = SplineStruct.GetSplinePointsRotation();
	const FInterpCurveVector& ControlPointsScale = SplineStruct.GetSplinePointsScale();

	TArray<FSplinePoint> ControlPoints;
	ControlPoints.Reserve(ControlPointsPosition.Points.Num());

	for (int i = 0; i < ControlPointsPosition.Points.Num(); ++i)
	{
		ControlPoints.Emplace(ControlPointsPosition.Points[i].InVal,
			ControlPointsPosition.Points[i].OutVal,
			ControlPointsPosition.Points[i].ArriveTangent,
			ControlPointsPosition.Points[i].LeaveTangent,
			ControlPointsRotation.Points[i].OutVal.Rotator(),
			ControlPointsScale.Points[i].OutVal,
			ConvertInterpCurveModeToSplinePointType(ControlPointsPosition.Points[i].InterpMode));
	}

	return ControlPoints;
}

TArray<int64> UPCGSplineData::GetMetadataEntryKeysForSplinePoints() const
{
	return TArray<int64>(SplineStruct.GetConstControlPointsEntryKeys());
}

FBox UPCGSplineData::GetBounds() const
{
	return CachedBounds;
}

bool UPCGSplineData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// TODO: support proper bounds

	// This is a pure SamplePoint implementation.
	
	// Find nearest point on spline
	const FVector InPosition = InTransform.GetLocation();
	float NearestPointKey = SplineStruct.FindInputKeyClosestToWorldLocation(InPosition);
	FTransform NearestTransform = SplineStruct.GetTransformAtSplineInputKey(NearestPointKey, ESplineCoordinateSpace::World, true);
	FVector LocalPoint = NearestTransform.InverseTransformPosition(InPosition);
	
	// Linear fall off based on the distance to the nearest point
	// TODO: should be based on explicit settings
	float Distance = LocalPoint.Length();
	if (Distance > 1.0f)
	{
		return false;
	}
	else
	{
		OutPoint.Transform = InTransform;
		OutPoint.SetLocalBounds(InBounds);
		OutPoint.Density = 1.0f - Distance;

		WriteMetadataToPoint(NearestPointKey, OutPoint, OutMetadata);

		return true;
	}
}

UPCGSpatialData* UPCGSplineData::ProjectOn(FPCGContext* InContext, const UPCGSpatialData* InOther, const FPCGProjectionParams& InParams) const
{
	if (InOther->GetDimension() == 2)
	{
		UPCGSplineProjectionData* SplineProjectionData = FPCGContext::NewObject_AnyThread<UPCGSplineProjectionData>(InContext);
		SplineProjectionData->Initialize(this, InOther, InParams);
		return SplineProjectionData;
	}
	else
	{
		return Super::ProjectOn(InContext, InOther, InParams);
	}
}

UPCGSpatialData* UPCGSplineData::CopyInternal(FPCGContext* Context) const
{
	UPCGSplineData* NewSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);

	CopySplineData(NewSplineData);

	return NewSplineData;
}

void UPCGSplineData::CopySplineData(UPCGSplineData* InCopy) const
{
	InCopy->SplineStruct = SplineStruct;
	InCopy->CachedBounds = CachedBounds;
}

bool UPCGSplineProjectionData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// TODO: support metadata - we don't currently have a good representation of what metadata entries mean for non-point data
	// TODO: use InBounds when sampling spline (sample in area rather than at closest point)

	if (!ProjectionParams.bProjectPositions)
	{
		// If we're not moving anything around, then just defer to super which will sample 3D spline, to make SamplePoint() consistent with behaviour
		// on 'concrete' data (points).
		return Super::SamplePoint(InTransform, InBounds, OutPoint, OutMetadata);
	}

	// Find nearest point on projected spline by lifting point along projection direction to closest position on spline. This way
	// when we sample the spline we get a similar result to if the spline had been projected onto the surface.

	const FVector InPosition = InTransform.GetLocation();
	check(GetSpline());
	const FPCGSplineStruct& Spline = GetSpline()->SplineStruct;
	check(GetSurface());
	const FVector& SurfaceNormal = GetSurface()->GetNormal();

	// Project to 2D space
	const FTransform LocalTransform = InTransform * Spline.GetTransform().Inverse();
	FVector2D LocalPosition2D = Project(LocalTransform.GetLocation());
	float Dummy;
	// Find nearest key on 2D spline
	float NearestInputKey = ProjectedPosition.InaccurateFindNearest(LocalPosition2D, Dummy);
	// TODO: if we didn't want to hand off density computation to the spline and do it here instead, we could do it in 2D space.
	// Find point on original spline using the previously found key. Note this is an approximation that might not hold true since
	// we are changing the curve length. Also, to support surface orientations that are not axis aligned, the project function
	// probably needs to construct into a coordinate space and project onto it rather than discarding an axis, otherwise project
	// coordinates may be non-uniformly scaled.
	const FVector NearestPointOnSpline = Spline.GetLocationAtSplineInputKey(NearestInputKey, ESplineCoordinateSpace::World);
	const FVector PointOnLine = FMath::ClosestPointOnInfiniteLine(InPosition, InPosition + SurfaceNormal, NearestPointOnSpline);

	// In the following statements we check if point lies in projection of spline onto landscape, which is true if:
	//  * When we hoist the point up to the nearest point on the unprojected spline, it overlaps the spline
	//  * The point is on the landscape
	
	// TODO: this is super inefficient, could be done in 2D if we duplicate the sampling code
	FPCGPoint SplinePoint;
	if (GetSpline()->SamplePoint(FTransform(PointOnLine), InBounds, SplinePoint, OutMetadata))
	{
		FPCGPoint SurfacePoint;
		if (GetSurface()->SamplePoint(InTransform, InBounds, SurfacePoint, OutMetadata))
		{
			OutPoint = SplinePoint;

			ApplyProjectionResult(SurfacePoint, OutPoint);

			if (OutMetadata)
			{
				if (SplinePoint.MetadataEntry != PCGInvalidEntryKey && SurfacePoint.MetadataEntry != PCGInvalidEntryKey)
				{
					OutMetadata->MergePointAttributesSubset(SplinePoint, OutMetadata, GetSpline()->Metadata, SurfacePoint, OutMetadata, GetSurface()->Metadata, OutPoint, ProjectionParams.AttributeMergeOperation);
				}
				else if (SurfacePoint.MetadataEntry != PCGInvalidEntryKey)
				{
					OutPoint.MetadataEntry = SurfacePoint.MetadataEntry;
				}
			}

			return true;
		}
	}

	return false;
}

FVector2D UPCGSplineProjectionData::Project(const FVector& InVector) const
{
	check(GetSurface());
	const FVector& SurfaceNormal = GetSurface()->GetNormal();
	FVector Projection = InVector - InVector.ProjectOnToNormal(SurfaceNormal);

	// Find the largest coordinate of the normal and use as the projection axis
	int BiggestCoordinateAxis = 0;
	FVector::FReal BiggestCoordinate = FMath::Abs(SurfaceNormal[BiggestCoordinateAxis]);
	for (int Axis = 1; Axis < 3; ++Axis)
	{
		FVector::FReal AbsoluteCoordinateValue = FMath::Abs(SurfaceNormal[Axis]);
		if (AbsoluteCoordinateValue > BiggestCoordinate)
		{
			BiggestCoordinate = AbsoluteCoordinateValue;
			BiggestCoordinateAxis = Axis;
		}
	}

	// Discard the projection axis coordinate
	FVector2D Projection2D;
	int AxisIndex = 0;
	for (int Axis = 0; Axis < 3; ++Axis)
	{
		if (Axis != BiggestCoordinateAxis)
		{
			Projection2D[AxisIndex++] = Projection[Axis];
		}
	}

	return Projection2D;
}

void UPCGSplineProjectionData::Initialize(const UPCGSplineData* InSourceSpline, const UPCGSpatialData* InTargetSurface, const FPCGProjectionParams& InParams)
{
	Super::Initialize(InSourceSpline, InTargetSurface, InParams);

	check(GetSurface());
	const FVector& SurfaceNormal = GetSurface()->GetNormal();

	if (GetSpline())
	{
		const FInterpCurveVector& SplinePosition = GetSpline()->SplineStruct.GetSplinePointsPosition();

		// Build projected spline data
		ProjectedPosition.bIsLooped = SplinePosition.bIsLooped;
		ProjectedPosition.LoopKeyOffset = SplinePosition.LoopKeyOffset;

		ProjectedPosition.Points.Reserve(SplinePosition.Points.Num());

		for (const FInterpCurvePoint<FVector>& SplinePoint : SplinePosition.Points)
		{
			FInterpCurvePoint<FVector2D>& ProjectedPoint = ProjectedPosition.Points.Emplace_GetRef();

			ProjectedPoint.InVal = SplinePoint.InVal;
			ProjectedPoint.OutVal = Project(SplinePoint.OutVal);
			// TODO: correct tangent if it becomes null
			ProjectedPoint.ArriveTangent = Project(SplinePoint.ArriveTangent).GetSafeNormal();
			ProjectedPoint.LeaveTangent = Project(SplinePoint.LeaveTangent).GetSafeNormal();
			ProjectedPoint.InterpMode = SplinePoint.InterpMode;
		}
	}
}

void UPCGSplineProjectionData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(Ar);
}

const UPCGSplineData* UPCGSplineProjectionData::GetSpline() const
{
	return Cast<const UPCGSplineData>(Source);
}

const UPCGSpatialData* UPCGSplineProjectionData::GetSurface() const
{
	return Target;
}

UPCGSpatialData* UPCGSplineProjectionData::CopyInternal(FPCGContext* Context) const
{
	UPCGSplineProjectionData* NewProjectionData = FPCGContext::NewObject_AnyThread<UPCGSplineProjectionData>(Context);

	CopyBaseProjectionClass(NewProjectionData);

	NewProjectionData->ProjectedPosition = ProjectedPosition;

	return NewProjectionData;
}

#undef LOCTEXT_NAMESPACE