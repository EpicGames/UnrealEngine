// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGSplineInteriorSurfaceData.h"

#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSplineData.h"
#include "Elements/PCGSplineSampler.h"

#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplineInteriorSurfaceData)

void UPCGSplineInteriorSurfaceData::Initialize(FPCGContext* Context, const UPCGSplineData* InSplineData)
{
	check(InSplineData);

	InitializeFromData(InSplineData);
	SplineStruct = InSplineData->SplineStruct;

	CacheData(Context);
}

void UPCGSplineInteriorSurfaceData::PostLoad()
{
	Super::PostLoad();
	CacheData(nullptr);
}

void UPCGSplineInteriorSurfaceData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	if (Metadata)
	{
		Metadata->AddToCrc(Ar, bFullDataCrc);
	}

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	Ar << SplineStruct;
}

bool UPCGSplineInteriorSurfaceData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	if (!PointInsidePolygon(InTransform, InBounds))
	{
		return false;
	}

	const FVector InLocation = InTransform.GetLocation();

	// Project the sampled point onto the approximate spline surface.
	const FVector::FReal ProjectionHeight = PCGSplineSamplerHelpers::ProjectOntoSplineInteriorSurface(CachedSplinePoints, InLocation);
	const FVector::FReal SampleMinHeight = InLocation.Z + InBounds.Min.Z;
	const FVector::FReal SampleMaxHeight = InLocation.Z + InBounds.Max.Z;

	// Discard if the sample point lies above/below the spline surface.
	if (SampleMinHeight > ProjectionHeight || SampleMaxHeight < ProjectionHeight)
	{
		return false;
	}

	new(&OutPoint) FPCGPoint(InTransform, /*Density=*/1.0f, /*Seed=*/0);
	OutPoint.SetLocalBounds(InBounds);

	return true;
}

bool UPCGSplineInteriorSurfaceData::ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	if (!PointInsidePolygon(InTransform, InBounds))
	{
		return false;
	}

	FTransform ProjectedTransform = InTransform;
	const FVector InLocation = InTransform.GetLocation();
	const FVector::FReal ProjectionHeight = (InParams.bProjectPositions || InParams.bProjectRotations) ? PCGSplineSamplerHelpers::ProjectOntoSplineInteriorSurface(CachedSplinePoints, InLocation) : 0.0f;

	if (InParams.bProjectPositions)
	{
		// Project the sampled point onto the approximate spline surface.
		ProjectedTransform.SetLocation(FVector(InLocation.X, InLocation.Y, ProjectionHeight));
	}

	if (InParams.bProjectRotations)
	{
		// Project rotation by sampling the rotation of the nearest point on the spline.
		float OutDistanceSquared, OutSegment;
		const float NearestSplineKey = SplineStruct.GetSplinePointsPosition().FindNearest(FVector(InLocation.X, InLocation.Y, ProjectionHeight), OutDistanceSquared, OutSegment);
		ProjectedTransform.SetRotation(SplineStruct.GetQuaternionAtSplineInputKey(NearestSplineKey, ESplineCoordinateSpace::Local));
	}

	new(&OutPoint) FPCGPoint(ProjectedTransform, /*Density=*/1.0f, /*Seed=*/0);
	OutPoint.SetLocalBounds(InBounds);

	return true;
}

const UPCGPointData* UPCGSplineInteriorSurfaceData::CreatePointData(FPCGContext* Context) const
{
	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()));
}

const UPCGPointArrayData* UPCGSplineInteriorSurfaceData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()));
}

const UPCGBasePointData* UPCGSplineInteriorSurfaceData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSplineData::CreatePointData);

	UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
	Data->InitializeFromData(this);

	FPCGSplineSamplerParams SamplerParams;
	SamplerParams.Dimension = EPCGSplineSamplingDimension::OnInterior;
	SamplerParams.bProjectOntoSurface = true;

	UPCGSplineData* SplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
	SplineData->Initialize(SplineStruct);

	// TODO: Ideally SampleInteriorData could just consume a UPCGSplineInteriorSurfaceData or FPCGSplineStruct, to avoid the extra copy on the spline struct.
	PCGSplineSamplerHelpers::SampleInteriorData(Context, SplineData, /*InBoundingShape=*/nullptr, /*InProjectionTarget=*/nullptr, /*InProjectionParams=*/{}, SamplerParams, Data);
	UE_LOGF(LogPCG, Verbose, "SplineInteriorSurface generated %d points.", Data->GetNumPoints());

	return Data;
}

UPCGSpatialData* UPCGSplineInteriorSurfaceData::CopyInternal(FPCGContext* Context) const
{
	UPCGSplineInteriorSurfaceData* NewData = FPCGContext::NewObject_AnyThread<UPCGSplineInteriorSurfaceData>(Context);

	CopyBaseSurfaceData(NewData);
	NewData->SplineStruct = SplineStruct;
	NewData->CachedBounds = CachedBounds;
	NewData->CachedSplinePoints = CachedSplinePoints;
	NewData->CachedPolygon2D = CachedPolygon2D;
#if WITH_EDITOR
	NewData->bNeedsToCache = false;
#endif

	return NewData;
}

void UPCGSplineInteriorSurfaceData::CacheData(FPCGContext* Context)
{
#if WITH_EDITOR
	bNeedsToCache = false;
#endif

	CachedBounds = SplineStruct.GetBounds();

	// Expand bounds by the radius of points, otherwise sections of the curve that are close
	// to the bounds will report an invalid density.
	FVector SplinePointsRadius = FVector::ZeroVector;
	const FInterpCurveVector& SplineScales = SplineStruct.GetSplinePointsScale();
	for (const FInterpCurvePoint<FVector>& SplineScale : SplineScales.Points)
	{
		SplinePointsRadius = FVector::Max(SplinePointsRadius, SplineScale.OutVal.GetAbs());
	}

	CachedBounds = CachedBounds.ExpandBy(SplinePointsRadius, SplinePointsRadius);
	CachedBounds = CachedBounds.TransformBy(SplineStruct.Transform);

	FPCGSplineSamplerParams SamplerParams;
	SamplerParams.Mode = EPCGSplineSamplingMode::Subdivision;
	SamplerParams.SubdivisionsPerSegment = 5;

	UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(Context);
	UPCGSplineData* SplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
	SplineData->Initialize(SplineStruct);

	// TODO: Ideally SampleLineData could just consume a UPCGSplineInteriorSurfaceData or FPCGSplineStruct, to avoid the extra copy on the spline struct.
	// TODO: It might be preferable to directly sample the spline than use the spline sampler, since it does a lot of work we don't care about.
	PCGSplineSamplerHelpers::SampleLineData(Context, SplineData, /*InBoundingShape=*/nullptr, /*InProjectionTarget=*/nullptr, /*InProjectionParams=*/{}, SamplerParams, PointData);

	// Cache the points which describe the polygon of our spline.
	const TConstPCGValueRange<FTransform> TransformRange = PointData->GetConstTransformValueRange();
	
	TArray<FVector2D> SplinePoints2D;
	SplinePoints2D.Reserve(TransformRange.Num());
	CachedSplinePoints.Reserve(TransformRange.Num());
	
	for (const FTransform& PointTransform : TransformRange)
	{
		CachedSplinePoints.Add(PointTransform.GetLocation());
		SplinePoints2D.Add(FVector2D(PointTransform.GetLocation()));
	}

	CachedPolygon2D = UE::Geometry::FPolygon2d(MoveTemp(SplinePoints2D));
	if (CachedPolygon2D.IsClockwise())
	{
		CachedPolygon2D.Reverse();
	}
}

bool UPCGSplineInteriorSurfaceData::PointInsidePolygon(const FTransform& InTransform, const FBox& InBounds) const
{
#if WITH_EDITOR
	check(!bNeedsToCache);
#endif

	FBox TransformedBounds;
	if (InTransform.IsRotationNormalized())
	{
		TransformedBounds = InBounds.TransformBy(InTransform);
	}
	else
	{
		FTransform TranslationAndScale = InTransform;
		TranslationAndScale.SetRotation(FQuat::Identity);
		TransformedBounds = InBounds.TransformBy(TranslationAndScale);
	}

	// Test point bounds against the spline bounds.
	if (CachedBounds.ComputeSquaredDistanceToBox(TransformedBounds) > 0.0)
	{
		return false;
	}

	const FVector PointLocation = InTransform.GetLocation();

	// Test sample location against spline interior.
	// TODO: This could also consume a transform + bounds to accept rotated bounds that overlap the polygon.
	return PCGSplineSamplerHelpers::PointInsidePolygon2D(CachedPolygon2D, FVector2D(PointLocation));
}
