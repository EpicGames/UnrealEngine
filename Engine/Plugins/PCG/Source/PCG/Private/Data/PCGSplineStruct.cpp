// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGSplineStruct.h"

#include "Curves/Splines/TangentSpline.h"
#include "Metadata/PCGMetadataCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplineStruct)

namespace PCGSplineStruct
{
	bool GUseSplineCurves = true;
	FAutoConsoleVariableRef CVarUseSplineCurves(
		TEXT("PCGSplineStruct.UseSplineCurves"),
		GUseSplineCurves,
		TEXT("When true, SplineCurves is the 'source of truth'.")
	);
	
	static int32 UpperBound(const TArray<FInterpCurvePoint<FVector>>& SplinePoints, float Value)
	{
		int32 Count = SplinePoints.Num();
		int32 First = 0;

		while (Count > 0)
		{
			const int32 Middle = Count / 2;
			if (Value >= SplinePoints[First + Middle].InVal)
			{
				First += Middle + 1;
				Count -= Middle + 1;
			}
			else
			{
				Count = Middle;
			}
		}

		return First;
	}

	// Note: copied verbatim from USplineComponent::CalcBounds
    static FBoxSphereBounds CalcBounds(const FSplineCurves& SplineCurves, bool bClosedLoop, const FTransform& LocalToWorld)
    {
//#if SPLINE_FAST_BOUNDS_CALCULATION
//		FBox BoundingBox(0);
//		for (const auto& InterpPoint : SplineCurves.Position.Points)
//		{
//			BoundingBox += InterpPoint.OutVal;
//		}
//
//		return FBoxSphereBounds(BoundingBox.TransformBy(LocalToWorld));
//#else
    	const int32 NumPoints = SplineCurves.Position.Points.Num();
    	const int32 NumSegments = bClosedLoop ? NumPoints : NumPoints - 1;

    	FVector Min(WORLD_MAX);
    	FVector Max(-WORLD_MAX);
    	if (NumSegments > 0)
    	{
    		for (int32 Index = 0; Index < NumSegments; Index++)
    		{
    			const bool bLoopSegment = (Index == NumPoints - 1);
    			const int32 NextIndex = bLoopSegment ? 0 : Index + 1;
    			const FInterpCurvePoint<FVector>& ThisInterpPoint = SplineCurves.Position.Points[Index];
    			FInterpCurvePoint<FVector> NextInterpPoint = SplineCurves.Position.Points[NextIndex];
    			if (bLoopSegment)
    			{
    				NextInterpPoint.InVal = ThisInterpPoint.InVal + SplineCurves.Position.LoopKeyOffset;
    			}

    			CurveVectorFindIntervalBounds(ThisInterpPoint, NextInterpPoint, Min, Max);
    		}
    	}
    	else if (NumPoints == 1)
    	{
    		Min = Max = SplineCurves.Position.Points[0].OutVal;
    	}
    	else
    	{
    		Min = FVector::ZeroVector;
    		Max = FVector::ZeroVector;
    	}

    	return FBoxSphereBounds(FBox(Min, Max).TransformBy(LocalToWorld));
//#endif
    }
	
	// Note: copied verbatim from USplineComponent::CalcBounds
	static FBoxSphereBounds CalcBounds(const FSpline& Spline, bool bClosedLoop, const FTransform& LocalToWorld)
	{
//#if SPLINE_FAST_BOUNDS_CALCULATION
//		FBox BoundingBox(0);
//		for (const auto& InterpPoint : Spline.Position.Points)
//		{
//			BoundingBox += InterpPoint.OutVal;
//		}
//
//		return FBoxSphereBounds(BoundingBox.TransformBy(LocalToWorld));
//#else
		const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();

		if (!TangentSpline)
		{
			ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
			return FBoxSphereBounds();
		}

		const int32 NumPoints = TangentSpline->GetSpline().GetNumPoints();
		const int32 NumSegments = bClosedLoop ? NumPoints : NumPoints - 1;
		const FInterpCurveVector& SplinePoints = TangentSpline->GetSplinePointsPosition();

		FVector Min(WORLD_MAX);
		FVector Max(-WORLD_MAX);
		if (NumSegments > 0)
		{
			for (int32 Index = 0; Index < NumSegments; Index++)
			{
				const bool bLoopSegment = (Index == NumPoints - 1);
				const int32 NextIndex = bLoopSegment ? 0 : Index + 1;
				const FInterpCurvePoint<FVector>& ThisInterpPoint = SplinePoints.Points[Index];
				FInterpCurvePoint<FVector> NextInterpPoint = SplinePoints.Points[NextIndex];
				if (bLoopSegment)
				{
					NextInterpPoint.InVal = ThisInterpPoint.InVal + SplinePoints.LoopKeyOffset;
				}

				CurveVectorFindIntervalBounds(ThisInterpPoint, NextInterpPoint, Min, Max);
			}
		}
		else if (NumPoints == 1)
		{
			Min = Max = SplinePoints.Points[0].OutVal;
		}
		else
		{
			Min = FVector::ZeroVector;
			Max = FVector::ZeroVector;
		}

		return FBoxSphereBounds(FBox(Min, Max).TransformBy(LocalToWorld));
//#endif
	}
}

void FPCGSplineStruct::Initialize(const USplineComponent* InSplineComponent)
{
	check(InSplineComponent);

	WarninglessSplineCurves() = InSplineComponent->GetSplineCurves();
	Spline = InSplineComponent->GetSpline();
	
	Transform = InSplineComponent->GetComponentTransform();
	DefaultUpVector = InSplineComponent->DefaultUpVector;
	ReparamStepsPerSegment = InSplineComponent->ReparamStepsPerSegment;
	bClosedLoop = InSplineComponent->IsClosedLoop();

	Bounds = InSplineComponent->Bounds;
	LocalBounds = InSplineComponent->CalcLocalBounds();

	ControlPointsEntryKeys.Empty();
}

void FPCGSplineStruct::Initialize(const TArray<FSplinePoint>& InSplinePoints, bool bIsClosedLoop, const FTransform& InTransform, TArray<PCGMetadataEntryKey> InOptionalEntryKeys)
{
	if (PCGSplineStruct::GUseSplineCurves)
	{
		Spline = FSpline(UE::Spline::ESplineType::Unimplemented);
	}
	else
	{
		Spline = FSpline(UE::Spline::ESplineType::Tangent);
	}

	Transform = InTransform;
	DefaultUpVector = FVector::ZAxisVector;
	ReparamStepsPerSegment = 10; // default value in USplineComponent

	bClosedLoop = bIsClosedLoop;
	const bool bShouldUseOptionalEntryKeys = !InOptionalEntryKeys.IsEmpty() && InSplinePoints.Num() == InOptionalEntryKeys.Num();
	AddPoints(InSplinePoints, true, bShouldUseOptionalEntryKeys ? &InOptionalEntryKeys : nullptr);

	if (ShouldUseSplineCurves())
	{
		Bounds = PCGSplineStruct::CalcBounds(WarninglessSplineCurves(), bClosedLoop, InTransform);
		LocalBounds = PCGSplineStruct::CalcBounds(WarninglessSplineCurves(), bClosedLoop, FTransform::Identity);
	}
	else
	{
		Bounds = PCGSplineStruct::CalcBounds(Spline, bClosedLoop, InTransform);
		LocalBounds = PCGSplineStruct::CalcBounds(Spline, bClosedLoop, FTransform::Identity);
	}

	if (!bShouldUseOptionalEntryKeys)
	{
		// If we have a mismatch, we can't set the entry keys, so reset it
		ControlPointsEntryKeys.Empty();
	}
}

void FPCGSplineStruct::ApplyTo(USplineComponent* InSplineComponent) const
{
	check(InSplineComponent);

	InSplineComponent->ClearSplinePoints(false);
	InSplineComponent->SetWorldTransform(Transform);
	InSplineComponent->DefaultUpVector = DefaultUpVector;
	InSplineComponent->ReparamStepsPerSegment = ReparamStepsPerSegment;

	if (ShouldUseSplineCurves())
	{
		InSplineComponent->SetSpline(WarninglessSplineCurves());
	}
	else
	{
		InSplineComponent->SetSpline(Spline);
	}
	
	InSplineComponent->bStationaryEndpoints = false;
	// TODO: metadata? might not be needed
	InSplineComponent->SetClosedLoop(bClosedLoop);
	InSplineComponent->UpdateSpline();
	InSplineComponent->UpdateBounds();
}

FPCGSplineStruct::FPCGSplineStruct(const FPCGSplineStruct& Other)
{
	*this = Other;
}

FPCGSplineStruct::FPCGSplineStruct(FPCGSplineStruct&& Other)
{
	*this = MoveTemp(Other);
}

FPCGSplineStruct& FPCGSplineStruct::operator=(const FPCGSplineStruct& Other)
{
	WarninglessSplineCurves() = Other.WarninglessSplineCurves();
	Spline = Other.Spline;
	
	Transform = Other.Transform;
	DefaultUpVector = Other.DefaultUpVector;
	ReparamStepsPerSegment = Other.ReparamStepsPerSegment;
	bClosedLoop = Other.bClosedLoop;

	Bounds = Other.Bounds;
	LocalBounds = Other.LocalBounds;

	ControlPointsEntryKeys = Other.ControlPointsEntryKeys;
	
	return *this;
}

FPCGSplineStruct& FPCGSplineStruct::operator=(FPCGSplineStruct&& Other)
{
	WarninglessSplineCurves() = MoveTemp(Other.WarninglessSplineCurves());
	Spline = MoveTemp(Other.Spline);
	
	Transform = MoveTemp(Other.Transform);
	DefaultUpVector = MoveTemp(Other.DefaultUpVector);
	ReparamStepsPerSegment = MoveTemp(Other.ReparamStepsPerSegment);
	bClosedLoop = MoveTemp(Other.bClosedLoop);

	Bounds = MoveTemp(Other.Bounds);
	LocalBounds = MoveTemp(Other.LocalBounds);

	ControlPointsEntryKeys = MoveTemp(Other.ControlPointsEntryKeys);
	
	return *this;
}

void FPCGSplineStruct::AddPoint(const FSplinePoint& InSplinePoint, bool bUpdateSpline, const int64* InOptionalControlPointEntryKey)
{
	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->AddPoint(InSplinePoint);
	}
 
	const int32 Index = PCGSplineStruct::UpperBound(WarninglessSplineCurves().Position.Points, InSplinePoint.InputKey);
	
	WarninglessSplineCurves().Position.Points.Insert(FInterpCurvePoint<FVector>(
		InSplinePoint.InputKey,
		InSplinePoint.Position,
		InSplinePoint.ArriveTangent,
		InSplinePoint.LeaveTangent,
		ConvertSplinePointTypeToInterpCurveMode(InSplinePoint.Type)
	), Index);

	WarninglessSplineCurves().Rotation.Points.Insert(FInterpCurvePoint<FQuat>(
		InSplinePoint.InputKey,
		InSplinePoint.Rotation.Quaternion(),
		FQuat::Identity,
		FQuat::Identity,
		CIM_CurveAuto
	), Index);

	WarninglessSplineCurves().Scale.Points.Insert(FInterpCurvePoint<FVector>(
		InSplinePoint.InputKey,
		InSplinePoint.Scale,
		FVector::ZeroVector,
		FVector::ZeroVector,
		CIM_CurveAuto
	), Index);

	if (!ControlPointsEntryKeys.IsEmpty())
	{
		ControlPointsEntryKeys.Insert(InOptionalControlPointEntryKey ? *InOptionalControlPointEntryKey : PCGInvalidEntryKey, Index);
	}
	else if (InOptionalControlPointEntryKey)
	{
		// We have an entry key but the array of control points entry keys is empty. Fill it with invalid entry keys values and set this point entry
		// key.
		ControlPointsEntryKeys.SetNumUninitialized(GetNumberOfPoints());
		for (int32 i = 0; i < ControlPointsEntryKeys.Num(); ++i)
		{
			ControlPointsEntryKeys[i] = i == Index ? *InOptionalControlPointEntryKey : PCGInvalidEntryKey;
		}
	}

	if (bUpdateSpline)
	{
		UpdateSpline();
	}
}

void FPCGSplineStruct::AddPoints(const TArray<FSplinePoint>& InSplinePoints, bool bUpdateSpline, const TArray<int64>* InOptionalControlPointsEntryKeys)
{
	if (!ControlPointsEntryKeys.IsEmpty())
	{
		ControlPointsEntryKeys.Reserve(ControlPointsEntryKeys.Num() + InSplinePoints.Num());
	}

	check(!InOptionalControlPointsEntryKeys || InSplinePoints.Num() == InOptionalControlPointsEntryKeys->Num());

	for (int SplinePointIndex = 0; SplinePointIndex < InSplinePoints.Num(); ++SplinePointIndex)
	{
		AddPoint(InSplinePoints[SplinePointIndex], false, InOptionalControlPointsEntryKeys ? &((*InOptionalControlPointsEntryKeys)[SplinePointIndex]) : nullptr);
	}

	if (bUpdateSpline)
	{
		UpdateSpline();
	}
}

void FPCGSplineStruct::SetLocation(int32 Index, const FVector& InLocation)
{
	WarninglessSplineCurves().Position.Points[Index].OutVal = InLocation;

	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->SetLocation(Index, InLocation);
	}
}

FVector FPCGSplineStruct::GetLocation(const int32 Index) const
{
	FVector Location = FVector::ZeroVector;

	if (ShouldUseSplineCurves())
	{
		Location = WarninglessSplineCurves().Position.Points[Index].OutVal;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Location = TangentSpline->GetLocation(Index);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return Location;
}
	
void FPCGSplineStruct::SetArriveTangent(const int32 Index, const FVector& InArriveTangent)
{
	WarninglessSplineCurves().Position.Points[Index].ArriveTangent = InArriveTangent;

	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->SetInTangent(Index, InArriveTangent);
	}
}

FVector FPCGSplineStruct::GetArriveTangent(const int32 Index) const
{
	FVector Tangent = FVector::ZeroVector;

	if (ShouldUseSplineCurves())
	{
		Tangent = WarninglessSplineCurves().Position.Points[Index].ArriveTangent;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Tangent = TangentSpline->GetInTangent(Index);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return Tangent;
}
	
void FPCGSplineStruct::SetLeaveTangent(const int32 Index, const FVector& InLeaveTangent)
{
	WarninglessSplineCurves().Position.Points[Index].LeaveTangent = InLeaveTangent;

	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->SetOutTangent(Index, InLeaveTangent);
	}
}

FVector FPCGSplineStruct::GetLeaveTangent(const int32 Index) const
{
	FVector Tangent = FVector::ZeroVector;

	if (ShouldUseSplineCurves())
	{
		Tangent = WarninglessSplineCurves().Position.Points[Index].LeaveTangent;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Tangent = TangentSpline->GetOutTangent(Index);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return Tangent;
}

void FPCGSplineStruct::SetRotation(int32 Index, const FQuat& InRotation)
{
	WarninglessSplineCurves().Rotation.Points[Index].OutVal = InRotation;

	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->SetRotation(Index, InRotation);
	}
}

FQuat FPCGSplineStruct::GetRotation(const int32 Index) const
{
	FQuat Quat = FQuat::Identity;

	if (ShouldUseSplineCurves())
	{
		Quat = WarninglessSplineCurves().Rotation.Points[Index].OutVal;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Quat = TangentSpline->GetRotation(Index);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return Quat;
}
	
void FPCGSplineStruct::SetScale(int32 Index, const FVector& InScale)
{
	WarninglessSplineCurves().Scale.Points[Index].OutVal = InScale;

	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->SetScale(Index, InScale);
	}
}

FVector FPCGSplineStruct::GetScale(const int32 Index) const
{
	FVector Scale = FVector::OneVector;

	if (ShouldUseSplineCurves())
	{
		Scale = WarninglessSplineCurves().Scale.Points[Index].OutVal;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Scale = TangentSpline->GetScale(Index);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return Scale;
}
	
void FPCGSplineStruct::SetSplinePointType(int32 Index, EInterpCurveMode Type)
{
	WarninglessSplineCurves().Position.Points[Index].InterpMode = Type;

	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->SetSplinePointType(Index, Type);
	}
}

EInterpCurveMode FPCGSplineStruct::GetSplinePointType(int32 Index) const
{
	EInterpCurveMode Mode = CIM_Unknown;

	if (ShouldUseSplineCurves())
	{
		Mode = WarninglessSplineCurves().Position.Points[Index].InterpMode.GetValue();
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Mode = TangentSpline->GetSplinePointType(Index);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return Mode;
}

const TArray<FName>& FPCGSplineStruct::GetAttributeChannelNames() const
{
	if (ShouldUseSplineCurves())
	{
		if (!CachedSortedAttributeChannelNames.IsEmpty())
		{
			CachedSortedAttributeChannelNames.Empty();
		}
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		if (TArray<FName> Channels = TangentSpline->GetAttributeChannelNamesByValueType<float>(); Channels.Num() != CachedSortedAttributeChannelNames.Num())
		{
			Channels.Sort([](const FName& A, const FName& B)
				{ return A.ToString() < B.ToString(); });

			if (PCG::TScopeLock ScopeLock(Lock); Channels.Num() != CachedSortedAttributeChannelNames.Num())
			{
				CachedSortedAttributeChannelNames = MoveTemp(Channels);
			}
		}
	}

	return CachedSortedAttributeChannelNames;
}

float FPCGSplineStruct::GetAttributeValue(float InputKey, FName ChannelName) const
{
	if (ShouldUseSplineCurves())
	{
		return 0.f;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->EvaluateAttribute<float>(ChannelName, InputKey);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
}

void FPCGSplineStruct::UpdateSpline()
{
	// If SplineComponent.UseSplineCurves is false and FSpline is not implemented, initialize it from SplineCurves.
	if (PCGSplineStruct::GUseSplineCurves == false && Spline.GetCurrentImplementation() != UE::Spline::ESplineType::Tangent)
	{
		Spline = WarninglessSplineCurves();
	}

	const bool bLoopPositionOverride = false;
	const bool bStationaryEndpoints = false;
	const float LoopPosition = 0.0f;

	FTangentSpline::FUpdateSplineParams Params{bClosedLoop, bStationaryEndpoints, ReparamStepsPerSegment, bLoopPositionOverride, LoopPosition};

	WarninglessSplineCurves().UpdateSpline(Params.bClosedLoop, Params.bStationaryEndpoints, Params.ReparamStepsPerSegment, Params.bLoopPositionOverride, Params.LoopPosition, Params.Scale3D);

	if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		TangentSpline->UpdateSpline(Params);
	}
}

int FPCGSplineStruct::GetNumberOfSplineSegments() const
{
	const int32 NumPoints = GetNumberOfPoints();
	return (bClosedLoop ? NumPoints : FMath::Max(0, NumPoints - 1));
}

int FPCGSplineStruct::GetNumberOfPoints() const
{
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Position.Points.Num();
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetSpline().GetNumPoints();
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0;
	}
}

float FPCGSplineStruct::GetLoopKeyOffset() const
{
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Position.LoopKeyOffset;
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetSplinePointsPosition().LoopKeyOffset;
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
}

void FPCGSplineStruct::SetLoopKeyOffset(float LoopKeyOffset)
{
	//@todo_pcg - have a proper way to set this during initialization.
	if (bClosedLoop)
	{
		const bool bStationaryEndpoints = false;
		const bool bLoopPositionOverride = true;
		const float LoopPosition = LoopKeyOffset + (WarninglessSplineCurves().Position.Points.IsEmpty() ? 0.0f : WarninglessSplineCurves().Position.Points.Last().InVal);

		FTangentSpline::FUpdateSplineParams Params{ bClosedLoop, bStationaryEndpoints, ReparamStepsPerSegment, bLoopPositionOverride, LoopPosition };

		WarninglessSplineCurves().UpdateSpline(Params.bClosedLoop, Params.bStationaryEndpoints, Params.ReparamStepsPerSegment, Params.bLoopPositionOverride, Params.LoopPosition, Params.Scale3D);

		if (FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
		{
			TangentSpline->UpdateSpline(Params);

		}
	}
}

FVector::FReal FPCGSplineStruct::GetSplineLength() const
{
	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().GetSplineLength();
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		const float MaxParameter = TangentSpline->GetParameterSpace().Max;
		return TangentSpline->GetDistanceAtParameter(MaxParameter);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
}

FBox FPCGSplineStruct::GetBounds() const
{
	FBoxSphereBounds BoundsFromStruct;

	if (ShouldUseSplineCurves())
	{
		BoundsFromStruct = PCGSplineStruct::CalcBounds(WarninglessSplineCurves(), bClosedLoop, FTransform::Identity);
	}
	else
	{
		BoundsFromStruct = PCGSplineStruct::CalcBounds(Spline, bClosedLoop, FTransform::Identity);
	}

	return BoundsFromStruct.GetBox();
}

FBox FPCGSplineStruct::GetSegmentBounds(int32 SegmentIndex) const
{
	if (SegmentIndex < 0 || SegmentIndex >= GetNumberOfSplineSegments())
	{
		return FBox(EForceInit::ForceInit);
	}

	const FInterpCurveVector& SplinePointsPosition = GetSplinePointsPosition();

	const int32 NumPoints = SplinePointsPosition.Points.Num();
	const bool bLoopSegment = (SegmentIndex == NumPoints - 1);
	const int32 NextIndex = bLoopSegment ? 0 : SegmentIndex + 1;
	const FInterpCurvePoint<FVector>& ThisInterpPoint = SplinePointsPosition.Points[SegmentIndex];
	FInterpCurvePoint<FVector> NextInterpPoint = SplinePointsPosition.Points[NextIndex];
	if (bLoopSegment)
	{
		NextInterpPoint.InVal = ThisInterpPoint.InVal + SplinePointsPosition.LoopKeyOffset;
	}

	FVector CurrentMin(WORLD_MAX);
	FVector CurrentMax(-WORLD_MAX);

	CurveVectorFindIntervalBounds(ThisInterpPoint, NextInterpPoint, CurrentMin, CurrentMax);

	return FBox(CurrentMin, CurrentMax);
}

const FInterpCurveVector& FPCGSplineStruct::GetSplinePointsScale() const
{
	const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();

	// We normally branch directly on ShouldUseSplineCurves(), but because this function returns a const-ref
	// we need a safe fallback if ShouldUseSplineCurves() is false and TangentSpline is unexpectedly nullptr.
	// Cache the value and assert the contract up front, then fall back to the spline curves data if needed.
	const bool bShouldUseSplineCurves = ShouldUseSplineCurves();
	if (!bShouldUseSplineCurves)
	{
		ensureAlwaysMsgf(TangentSpline, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	if (bShouldUseSplineCurves || !TangentSpline)
	{
		return WarninglessSplineCurves().Scale;
	}
	else
	{
		return TangentSpline->GetSplinePointsScale();
	}
}

const FInterpCurveVector& FPCGSplineStruct::GetSplinePointsPosition() const
{
	const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();

	// We normally branch directly on ShouldUseSplineCurves(), but because this function returns a const-ref
	// we need a safe fallback if ShouldUseSplineCurves() is false and TangentSpline is unexpectedly nullptr.
	// Cache the value and assert the contract up front, then fall back to the spline curves data if needed.
	const bool bShouldUseSplineCurves = ShouldUseSplineCurves();
	if (!bShouldUseSplineCurves)
	{
		ensureAlwaysMsgf(TangentSpline, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	if (bShouldUseSplineCurves || !TangentSpline)
	{
		return WarninglessSplineCurves().Position;
	}
	else
	{
		return TangentSpline->GetSplinePointsPosition();
	}
}

const FInterpCurveQuat& FPCGSplineStruct::GetSplinePointsRotation() const
{
	const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>();

	// We normally branch directly on ShouldUseSplineCurves(), but because this function returns a const-ref
	// we need a safe fallback if ShouldUseSplineCurves() is false and TangentSpline is unexpectedly nullptr.
	// Cache the value and assert the contract up front, then fall back to the spline curves data if needed.
	const bool bShouldUseSplineCurves = ShouldUseSplineCurves();
	if (!bShouldUseSplineCurves)
	{
		ensureAlwaysMsgf(TangentSpline, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	if (bShouldUseSplineCurves || !TangentSpline)
	{
		return WarninglessSplineCurves().Rotation;
	}
	else
	{
		return TangentSpline->GetSplinePointsRotation();
	}
}

FVector::FReal FPCGSplineStruct::GetDistanceAlongSplineAtSplinePoint(int32 PointIndex) const
{
	if (IsClosedLoop() && PointIndex == GetNumberOfPoints())
	{
		// Special case, if we are closed and the index here is 1 past the last valid point, the length is the full spline.
		return GetSplineLength();
	}
	
	if (ShouldUseSplineCurves())
	{
		const int32 NumPoints = GetNumberOfPoints();
		const int32 NumSegments = bClosedLoop ? NumPoints : NumPoints - 1;
		const int32 NumReparamPoints = WarninglessSplineCurves().ReparamTable.Points.Num();
        
		// Ensure that if the reparam table is not prepared yet we don't attempt to access it. This can happen
		// early in the construction of the spline component object.
		if ((PointIndex >= 0) && (PointIndex < NumSegments + 1) && ((PointIndex * ReparamStepsPerSegment) < NumReparamPoints))
		{
			return WarninglessSplineCurves().ReparamTable.Points[PointIndex * ReparamStepsPerSegment].InVal;
		}
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		const float ParameterAtIndex = TangentSpline->GetParameterAtIndex(PointIndex);
		return TangentSpline->GetDistanceAtParameter(ParameterAtIndex);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return 0.0f;
}

FVector FPCGSplineStruct::GetLocationAtDistanceAlongSpline(FVector::FReal Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Param = GetInputKeyAtDistanceAlongSpline(Distance);
	return GetLocationAtSplineInputKey(Param, CoordinateSpace);
}

FTransform FPCGSplineStruct::GetTransformAtDistanceAlongSpline(FVector::FReal Distance, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale) const
{
	const float Param = GetInputKeyAtDistanceAlongSpline(Distance);
	return GetTransformAtSplineInputKey(Param, CoordinateSpace, bUseScale);
}

float FPCGSplineStruct::GetInputKeyAtDistanceAlongSpline(FVector::FReal Distance) const
{
	const int32 NumPoints = GetNumberOfPoints();

	if (NumPoints < 2)
	{
		return 0.0f;
	}

	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().ReparamTable.Eval(Distance, 0.0f);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->GetParameterAtDistance(Distance);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
}

FVector FPCGSplineStruct::GetRightVectorAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const FQuat Quat = GetQuaternionAtSplineInputKey(InKey, ESplineCoordinateSpace::Local);
	FVector RightVector = Quat.RotateVector(FVector::RightVector);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		RightVector = Transform.TransformVectorNoScale(RightVector);
	}

	return RightVector;
}

FTransform FPCGSplineStruct::GetTransformAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale) const
{
	const FVector Location(GetLocationAtSplineInputKey(InKey, ESplineCoordinateSpace::Local));
	const FQuat Rotation(GetQuaternionAtSplineInputKey(InKey, ESplineCoordinateSpace::Local));
	const FVector Scale = bUseScale ? GetScaleAtSplineInputKey(InKey) : FVector(1.0f);

	FTransform KeyTransform(Rotation, Location, Scale);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		KeyTransform = KeyTransform * Transform;
	}

	return KeyTransform;
}

float FPCGSplineStruct::FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const
{
	const FVector LocalLocation = Transform.InverseTransformPosition(WorldLocation);
	float Dummy;

	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Position.FindNearest(LocalLocation, Dummy);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->FindNearest(LocalLocation, Dummy);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
}

float FPCGSplineStruct::FindInputKeyOnSegmentClosestToWorldLocation(const FVector& WorldLocation, int32 Index) const
{
	const FVector LocalLocation = Transform.InverseTransformPosition(WorldLocation);
	float Dummy;

	if (ShouldUseSplineCurves())
	{
		return WarninglessSplineCurves().Position.FindNearestOnSegment(LocalLocation, Index, Dummy);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		return TangentSpline->FindNearestOnSegment(LocalLocation, Index, Dummy);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return 0.f;
	}
}

TTuple<int, float> FPCGSplineStruct::GetSegmentStartIndexAndKeyAtInputKey(float InKey) const
{
	const int32 Index = GetSplinePointsPosition().GetPointIndexForInputValue(InKey);
	return {Index, GetInputKeyAtSegmentStart(Index)};
}

float FPCGSplineStruct::GetInputKeyAtSegmentStart(int InSegmentIndex) const
{
	const auto& SplinePointsPosition = GetSplinePointsPosition();

	if (IsClosedLoop() && !SplinePointsPosition.Points.IsEmpty() && InSegmentIndex >= SplinePointsPosition.Points.Num())
	{
		// In case of a closed loop, the last point is the first point, and the input key is the last point + LoopKeyOffset
		return SplinePointsPosition.Points.Last().InVal + SplinePointsPosition.LoopKeyOffset;
	}
	else
	{
		return SplinePointsPosition.Points.IsValidIndex(InSegmentIndex) ? SplinePointsPosition.Points[InSegmentIndex].InVal : 0.0f;
	}
}

void FPCGSplineStruct::AllocateMetadataEntries()
{
	// Add robustness to cleanup everything if we have a mismatch between the number of points and the number of entry keys
	const int32 NumPoints = GetNumberOfPoints();
	
	if (ControlPointsEntryKeys.Num() != NumPoints)
	{
		ControlPointsEntryKeys.Empty(NumPoints);
	}
	
	if (ControlPointsEntryKeys.IsEmpty())
	{
		ControlPointsEntryKeys.Reserve(NumPoints);
		for (int32 i = 0; i < NumPoints; ++i)
		{
			ControlPointsEntryKeys.Add(PCGInvalidEntryKey);
		}
	}
}

FVector FPCGSplineStruct::GetLocationAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	FVector Location = ShouldUseSplineCurves()
		? WarninglessSplineCurves().Position.Eval(InKey, FVector::ZeroVector)
		: Spline.Evaluate(InKey);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Location = Transform.TransformPosition(Location);
	}

	return Location;
}

FQuat FPCGSplineStruct::GetQuaternionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	FQuat Quat = FQuat::Identity;
	if (ShouldUseSplineCurves())
	{
		Quat = WarninglessSplineCurves().Rotation.Eval(InKey, FQuat::Identity);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Quat = TangentSpline->EvaluateRotation(InKey);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
		return Quat;
	}
	Quat.Normalize();

	FVector Direction = ShouldUseSplineCurves()
		? WarninglessSplineCurves().Position.EvalDerivative(InKey, FVector::ZeroVector).GetSafeNormal()
		: Spline.EvaluateDerivative(InKey).GetSafeNormal();
	
	const FVector UpVector = Quat.RotateVector(DefaultUpVector);

	FQuat Rot = (FRotationMatrix::MakeFromXZ(Direction, UpVector)).ToQuat();

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Rot = Transform.GetRotation() * Rot;
	}

	return Rot;
}

FVector FPCGSplineStruct::GetScaleAtSplineInputKey(float InKey) const
{
	FVector Scale = FVector::OneVector;

	if (ShouldUseSplineCurves())
	{
		Scale = WarninglessSplineCurves().Scale.Eval(InKey, FVector::OneVector);
	}
	else if (const FTangentSpline* TangentSpline = Spline.Get<UE::Spline::ESplineType::Tangent>())
	{
		Scale = TangentSpline->EvaluateScale(InKey);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("ShouldUseSplineCurves() returned false but Spline failed to yield an FTangentSpline!"));
	}

	return Scale;
}

FVector FPCGSplineStruct::GetTangentAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	FVector Tangent = ShouldUseSplineCurves()
		? WarninglessSplineCurves().Position.EvalDerivative(InKey, FVector::ZeroVector)
		: Spline.EvaluateDerivative(InKey);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Tangent = Transform.TransformVector(Tangent);
	}

	return Tangent;
}

/** Taken from USplineComponent::ConvertSplineSegmentToPolyLine. */
bool FPCGSplineStruct::ConvertSplineSegmentToPolyLine(int32 SplinePointStartIndex, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const
{
	OutPoints.Empty();
	
	TArray<double> DummyDistances;

	const double StartDist = GetDistanceAlongSplineAtSplinePoint(SplinePointStartIndex);
	const double StopDist = GetDistanceAlongSplineAtSplinePoint(SplinePointStartIndex + 1);

	const int32 NumLines = 2; // Dichotomic subdivision of the spline segment
	double Dist = StopDist - StartDist;
	double SubstepSize = Dist / NumLines;
	if (SubstepSize == 0.0)
	{
		// There is no distance to cover, so handle the segment with a single point
		OutPoints.Add(GetLocationAtDistanceAlongSpline(StopDist, CoordinateSpace));
		return true;
	}

	// Early out if the segment is linear, because we don't need to do subdivision.
	if (GetSplinePointType(SplinePointStartIndex) == EInterpCurveMode::CIM_Linear)
	{
		OutPoints.Add(GetLocationAtDistanceAlongSpline(StartDist, CoordinateSpace));
		OutPoints.Add(GetLocationAtDistanceAlongSpline(StopDist, CoordinateSpace));
		return true;
	}

	double SubstepStartDist = StartDist;
	for (int32 i = 0; i < NumLines; ++i)
	{
		double SubstepEndDist = SubstepStartDist + SubstepSize;
		TArray<FVector> NewPoints;
		DummyDistances.Reset();
		// Recursively sub-divide each segment until the requested precision is reached :
		if (DivideSplineIntoPolylineRecursiveWithDistancesHelper(SubstepStartDist, SubstepEndDist, CoordinateSpace, MaxSquareDistanceFromSpline, NewPoints, DummyDistances))
		{
			if (OutPoints.Num() > 0)
			{
				check(OutPoints.Last() == NewPoints[0]); // our last point must be the same as the new segment's first
				OutPoints.RemoveAt(OutPoints.Num() - 1);
			}
			OutPoints.Append(NewPoints);
		}

		SubstepStartDist = SubstepEndDist;
	}

	return (OutPoints.Num() > 0);
}

/** Taken from USplineComponent::ConvertSplineToPolyLine. */
bool FPCGSplineStruct::ConvertSplineToPolyLine(ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const
{
	int32 NumSegments = GetNumberOfSplineSegments();
	OutPoints.Empty();
	OutPoints.Reserve(NumSegments * 2); // We sub-divide each segment in at least 2 sub-segments, so let's start with this amount of points

	TArray<FVector> SegmentPoints;
	for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
	{
		if (ConvertSplineSegmentToPolyLine(SegmentIndex, CoordinateSpace, MaxSquareDistanceFromSpline, SegmentPoints))
		{
			if (OutPoints.Num() > 0)
			{
				check(OutPoints.Last() == SegmentPoints[0]); // our last point must be the same as the new segment's first
				OutPoints.RemoveAt(OutPoints.Num() - 1);
			}
			OutPoints.Append(SegmentPoints);
		}
	}

	return (OutPoints.Num() > 0);
}

/** Taken from USplineComponent::DivideSplineIntoPolylineRecursiveWithDistancesHelper. */
bool FPCGSplineStruct::DivideSplineIntoPolylineRecursiveWithDistancesHelper(float StartDistanceAlongSpline, float EndDistanceAlongSpline, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline) const
{
	double Dist = EndDistanceAlongSpline - StartDistanceAlongSpline;
	if (Dist <= 0.0f)
	{
		return false;
	}
	double MiddlePointDistancAlongSpline = StartDistanceAlongSpline + Dist / 2.0f;
	FVector Samples[3];
	Samples[0] = GetLocationAtDistanceAlongSpline(StartDistanceAlongSpline, CoordinateSpace);
	Samples[1] = GetLocationAtDistanceAlongSpline(MiddlePointDistancAlongSpline, CoordinateSpace);
	Samples[2] = GetLocationAtDistanceAlongSpline(EndDistanceAlongSpline, CoordinateSpace);

	if (FMath::PointDistToSegmentSquared(Samples[1], Samples[0], Samples[2]) > MaxSquareDistanceFromSpline)
	{
		TArray<FVector> NewPoints[2];
		TArray<double> NewDistancesAlongSpline[2];
		DivideSplineIntoPolylineRecursiveWithDistancesHelper(StartDistanceAlongSpline, MiddlePointDistancAlongSpline, CoordinateSpace, MaxSquareDistanceFromSpline, NewPoints[0], NewDistancesAlongSpline[0]);
		DivideSplineIntoPolylineRecursiveWithDistancesHelper(MiddlePointDistancAlongSpline, EndDistanceAlongSpline, CoordinateSpace, MaxSquareDistanceFromSpline, NewPoints[1], NewDistancesAlongSpline[1]);
		if ((NewPoints[0].Num() > 0) && (NewPoints[1].Num() > 0))
		{
			check(NewPoints[0].Last() == NewPoints[1][0]);
			check(NewDistancesAlongSpline[0].Last() == NewDistancesAlongSpline[1][0]);
			NewPoints[0].RemoveAt(NewPoints[0].Num() - 1);
			NewDistancesAlongSpline[0].RemoveAt(NewDistancesAlongSpline[0].Num() - 1);
		}
		NewPoints[0].Append(NewPoints[1]);
		NewDistancesAlongSpline[0].Append(NewDistancesAlongSpline[1]);
		OutPoints.Append(NewPoints[0]);
		OutDistancesAlongSpline.Append(NewDistancesAlongSpline[0]);
	}
	else
	{
		// The middle point is close enough to the other 2 points, let's keep those and stop the recursion :
		OutPoints.Add(Samples[0]);
		OutDistancesAlongSpline.Add(StartDistanceAlongSpline);
		// For a constant spline, the end can be the exact same as the start; in this case, just add the point once
		if (Samples[0] != Samples[2])
		{
			OutPoints.Add(Samples[2]);
			OutDistancesAlongSpline.Add(EndDistanceAlongSpline);
		}
		
	}

	check(OutPoints.Num() == OutDistancesAlongSpline.Num())
	return (OutPoints.Num() > 0);
}

void FPCGSplineStruct::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.IsLoading() && Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
	{
		// If PCGSplineStruct.UseSplineCurves is false and Spline was not previously implemented
		// (implying it is not consistent with SplineCurves), initialize Spline from SplineCurves.
		if (Spline.GetPreviousImplementation() == UE::Spline::ESplineType::Unimplemented && !PCGSplineStruct::GUseSplineCurves)
		{
			Spline = WarninglessSplineCurves();
		}
	}
#endif // WITH_EDITOR
}

bool FPCGSplineStruct::ShouldUseSplineCurves() const
{
	return PCGSplineStruct::GUseSplineCurves || Spline.GetCurrentImplementation() != UE::Spline::ESplineType::Tangent;
}