// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGStaticMeshResourceData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGStaticMeshResourceData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoStaticMeshResource, UPCGStaticMeshResourceData)

void UPCGStaticMeshResourceData::Initialize(TSoftObjectPtr<UStaticMesh> InStaticMesh)
{
	StaticMesh = InStaticMesh;
}

FSoftObjectPath UPCGStaticMeshResourceData::GetResourcePath() const
{
	return StaticMesh.ToSoftObjectPath();
}
