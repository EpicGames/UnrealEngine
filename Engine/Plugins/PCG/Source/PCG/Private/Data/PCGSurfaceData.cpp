// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGSurfaceData.h"
#include "Data/PCGSpatialData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSurfaceData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoSurface, UPCGSurfaceData)

void UPCGSurfaceData::CopyBaseSurfaceData(UPCGSurfaceData* NewSurfaceData) const
{
	NewSurfaceData->Transform = Transform;
	NewSurfaceData->bKeepZeroDensityPoints = bKeepZeroDensityPoints;
}
