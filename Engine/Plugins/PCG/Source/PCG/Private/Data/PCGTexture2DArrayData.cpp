// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGTexture2DArrayData.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"

#include "TextureResource.h"
#include "Engine/Texture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGTexture2DArrayData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoTexture2DArray, UPCGTexture2DArrayData)

void UPCGTexture2DArrayData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(Ar);
}

UPCGSpatialData* UPCGTexture2DArrayData::CopyInternal(FPCGContext* Context) const
{
	UPCGTexture2DArrayData* NewData = FPCGContext::NewObject_AnyThread<UPCGTexture2DArrayData>(Context);

	CopyTexture2DArrayData(NewData);

	return NewData;
}

const UPCGPointData* UPCGTexture2DArrayData::CreatePointData(FPCGContext* InContext) const
{
	return CastChecked<UPCGPointData>(CreateBasePointData(InContext, UPCGPointData::StaticClass()));
}

const UPCGPointArrayData* UPCGTexture2DArrayData::CreatePointArrayData(FPCGContext* InContext, const FBox& InBounds) const
{
	return CastChecked<UPCGPointArrayData>(CreateBasePointData(InContext, UPCGPointArrayData::StaticClass()));
}

bool UPCGTexture2DArrayData::Initialize(UTexture* InTexture, const FPCGTexture2DArrayDataInitParams& InParams)
{
	Texture = InTexture;
	ResourceType = EPCGTextureResourceType::TextureObject;

	return InitializeInternal(InParams);
}

bool UPCGTexture2DArrayData::Initialize(TRefCountPtr<IPooledRenderTarget> InExportedTextureHandle, const FPCGTexture2DArrayDataInitParams& InParams)
{
	TextureHandle = InExportedTextureHandle;
	ResourceType = EPCGTextureResourceType::ExportedTexture;

	return InitializeInternal(InParams);
}

FTextureRHIRef UPCGTexture2DArrayData::GetRHI() const
{
	if (ResourceType == EPCGTextureResourceType::TextureObject)
	{
		FTextureResource* Resource = Texture.IsValid() ? Texture->GetResource() : nullptr;
		return Resource ? Resource->GetTextureRHI() : nullptr;
	}
	else
	{
		return TextureHandle ? TextureHandle->GetRHI() : nullptr;
	}
}

bool UPCGTexture2DArrayData::InitializeInternal(const FPCGTexture2DArrayDataInitParams& InParams)
{
	if (ensure(GetRHI()))
	{
		const FRHITextureDesc& Desc = GetRHI()->GetDesc();
		Resolution = Desc.Extent;
		ArraySize = Desc.ArraySize;
		NumMips = Desc.NumMips;
		Format = Desc.Format;
	}

	Transform = InParams.Transform;
	Filter = InParams.Filter;

	Bounds = FBox(EForceInit::ForceInit);
	Bounds += FVector(-1.0f, -1.0f, 0.0f);
	Bounds += FVector(1.0f, 1.0f, 0.0f);
	Bounds = Bounds.TransformBy(Transform);

	return true;
}

const UPCGBasePointData* UPCGTexture2DArrayData::CreateBasePointData(FPCGContext* InContext, TSubclassOf<UPCGBasePointData> InPointDataClass) const
{
	UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(InContext, GetTransientPackage(), InPointDataClass);
	Data->InitializeFromData(this);

	return Data;
}

void UPCGTexture2DArrayData::CopyTexture2DArrayData(UPCGTexture2DArrayData* NewTextureData) const
{
	CopyBaseSurfaceData(NewTextureData);

	NewTextureData->Texture = Texture;
	NewTextureData->TextureHandle = TextureHandle;
	NewTextureData->ResourceType = ResourceType;
	NewTextureData->Resolution = Resolution;
	NewTextureData->ArraySize = ArraySize;
	NewTextureData->NumMips = NumMips;
	NewTextureData->Format = Format;
	NewTextureData->Filter = Filter;
	NewTextureData->Bounds = Bounds;
}
