// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGTexture2DBaseData.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"

#include "TextureResource.h"
#include "Engine/Texture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGTexture2DBaseData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoTexture2DBase, UPCGTexture2DBaseData)

void UPCGTexture2DBaseData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	if (TextureHandle.IsValid())
	{
		CumulativeResourceSize.AddDedicatedVideoMemoryBytes(TextureHandle->ComputeMemorySize());
	}
}

void UPCGTexture2DBaseData::CopyTexture2DBaseData(UPCGTexture2DBaseData* NewTextureData) const
{
	CopyBaseSurfaceData(NewTextureData);

	NewTextureData->Format = Format;
	NewTextureData->NumMips = NumMips;
	NewTextureData->Filter = Filter;
	NewTextureData->TextureHandle = TextureHandle;
	NewTextureData->Bounds = Bounds;
}

FTransform UPCGTexture2DBaseData::ComputeTransform(const IPCGGraphExecutionSource* InExecutionSource)
{
	FTransform Transform = FTransform::Identity;

	if (InExecutionSource)
	{
		const FBox Bounds = InExecutionSource->GetExecutionState().GetBounds();
		Transform = InExecutionSource->GetExecutionState().GetTransform();
		Transform.SetScale3D(Transform.GetScale3D() * Bounds.GetExtent());
	}

	return Transform;
}
