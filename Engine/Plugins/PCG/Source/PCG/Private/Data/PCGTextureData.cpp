// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGTextureData.h"

#include "PCGContext.h"
#include "PCGTextureReadback.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Graph/PCGGraphCache.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"

#include "RHIStaticStates.h"
#include "RenderCaptureInterface.h"
#include "TextureResource.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGTextureData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoTexture2DSingleBase, UPCGTexture2DSingleBaseData)
PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoTexture2D, UPCGTextureData)

namespace PCGTextureSamplingHelpers
{
	static int32 GTriggerReadbackCaptureDispatches = 0;
	static FAutoConsoleVariableRef CVarTriggerReadbackCapture(
		TEXT("pcg.GPU.TriggerRenderCaptures.TextureReadback"),
		GTriggerReadbackCaptureDispatches,
		TEXT("Trigger GPU readback captures for this many of the subsequent texture data initializations."));

	TOptional<bool> IsTextureCPUAccessible(UTexture2D* Texture)
	{
		if (!Texture)
		{
			return false;
		}

#if WITH_EDITOR
		if (!Texture->IsAsyncCacheComplete())
		{
			return {};
		}
#endif

		FTexturePlatformData* PlatformData = Texture ? Texture->GetPlatformData() : nullptr;

		return PlatformData && PlatformData->GetHasCpuCopy();
	}

	TOptional<bool> CanGPUTextureBeCPUAccessed(UTexture2D* Texture)
	{
		// SRGB textures need to be GPU sampled.
		if (!Texture || Texture->SRGB)
		{
			return false;
		}

#if WITH_EDITOR
		if (!Texture->IsAsyncCacheComplete())
		{
			return {};
		}
#endif

		FTexturePlatformData* PlatformData = Texture->GetPlatformData();

		// If a CPU copy is available, this is a CPU texture and not a GPU texture, so we return false.
		if (!PlatformData || PlatformData->GetHasCpuCopy())
		{
			return false;
		}

		return PlatformData->Mips.Num() == 1 && PlatformData->PixelFormat == PF_B8G8R8A8;
	}

	template<typename ValueType>
	ValueType SampleInternal(FVector2D PositionLocalSpace,
		int32 Width,
		int32 Height,
		EPCGTextureFilter Filter,
		TFunctionRef<ValueType(int32 Index)> SamplingFunction)
	{
		const double TexelX = PositionLocalSpace.X * Width;
		const double TexelY = PositionLocalSpace.Y * Height;

		ValueType Result{};

		if (Filter == EPCGTextureFilter::Point)
		{
			const int32 X = FMath::Clamp(FMath::FloorToInt(TexelX), 0, Width - 1);
			const int32 Y = FMath::Clamp(FMath::FloorToInt(TexelY), 0, Height - 1);

			Result = SamplingFunction(X + Y * Width);
		}
		else if (Filter == EPCGTextureFilter::Bilinear)
		{
			// Accounts for texel values being at texel centers
			const double TexelXOffset = TexelX - 0.5;
			const double TexelYOffset = TexelY - 0.5;

			const int32 X0 = FMath::Clamp(FMath::FloorToInt(TexelXOffset), 0, Width - 1);
			const int32 X1 = FMath::Min(X0 + 1, Width - 1);
			const int32 Y0 = FMath::Clamp(FMath::FloorToInt(TexelYOffset), 0, Height - 1);
			const int32 Y1 = FMath::Min(Y0 + 1, Height - 1);

			const ValueType SampleX0Y0 = SamplingFunction(X0 + Y0 * Width);
			const ValueType SampleX1Y0 = SamplingFunction(X1 + Y0 * Width);
			const ValueType SampleX0Y1 = SamplingFunction(X0 + Y1 * Width);
			const ValueType SampleX1Y1 = SamplingFunction(X1 + Y1 * Width);

			Result = FMath::BiLerp(SampleX0Y0, SampleX1Y0, SampleX0Y1, SampleX1Y1, TexelXOffset - X0, TexelYOffset - Y0);
		}
		else
		{
			ensureMsgf(false, TEXT("Unrecognized PCG texture filtering mode."));
		}

		return Result;
	}

	template<typename ValueType>
	bool Sample(const FVector2D& InPosition,
		const FBox2D& InSurface,
		const UPCGTexture2DSingleBaseData* InTextureData,
		int32 Width,
		int32 Height,
		ValueType& SampledValue,
		TFunctionRef<ValueType(int32 Index)> SamplingFunction)
	{
		check(Width > 0 && Height > 0);
		if (Width <= 0 || Height <= 0 || InSurface.GetSize().SquaredLength() <= 0)
		{
			return false;
		}

		check(InTextureData);

		const FVector2D LocalSpacePos = (InPosition - InSurface.Min) / InSurface.GetSize();
		FVector2D Pos = FVector2D::ZeroVector;
		if (!InTextureData->bUseAdvancedTiling)
		{
			Pos.X = FMath::Clamp(LocalSpacePos.X, 0.0, 1.0);
			Pos.Y = FMath::Clamp(LocalSpacePos.Y, 0.0, 1.0);
		}
		else
		{
			// Conceptually, we are building "tiles" in texture space with the origin being in the middle of the [0, 0] tile.
			// The offset is given in a ratio of [0, 1], applied "before" scaling & rotation.
			// Rotation is done around the center given, where the center is (0.5, 0.5) + offset
			// Scaling controls the horizon of tiles, and the tile selection is done through min-max bounds, in tile space,
			// with the origin tile being from -0.5 to 0.5.
			const FRotator Rotation = FRotator(0.0, -InTextureData->Rotation, 0.0);
			FVector Scale = FVector(InTextureData->Tiling, 1.0);
			Scale.X = ((FMath::Abs(Scale.X) > SMALL_NUMBER) ? (1.0 / Scale.X) : 0.0);
			Scale.Y = ((FMath::Abs(Scale.Y) > SMALL_NUMBER) ? (1.0 / Scale.Y) : 0.0);
			const FVector Translation = FVector(0.5 + InTextureData->CenterOffset.X, 0.5 + InTextureData->CenterOffset.Y, 0);

			FTransform Transform = FTransform(Rotation, Translation, Scale);

			// Transform to tile-space
			const FVector2D SamplePosition = FVector2D(Transform.InverseTransformPosition(FVector(LocalSpacePos, 0.f)));

			if (InTextureData->bUseTileBounds && !InTextureData->TileBounds.IsInsideOrOn(SamplePosition))
			{
				return false;
			}

			FVector::FReal X = FMath::Frac(SamplePosition.X + 0.5);
			FVector::FReal Y = FMath::Frac(SamplePosition.Y + 0.5);

			Pos = FVector2D(X, Y);
		}

		SampledValue = SampleInternal(Pos, Width, Height, InTextureData->Filter, SamplingFunction);
		return true;
	}

	float SampleFloatChannel(const FLinearColor& InColor, EPCGTextureColorChannel ColorChannel)
	{
		switch (ColorChannel)
		{
		case EPCGTextureColorChannel::Red:
			return InColor.R;
		case EPCGTextureColorChannel::Green:
			return InColor.G;
		case EPCGTextureColorChannel::Blue:
			return InColor.B;
		case EPCGTextureColorChannel::Alpha:
		default:
			return InColor.A;
		}
	}
}

void UPCGTexture2DSingleBaseData::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DensityFunction != EPCGTextureDensityFunction::Multiply)
	{
		bUseDensitySourceChannel = false;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	// If ColorData is already serialized then the readback is complete.
	if (!ColorData.IsEmpty())
	{
		bIsReadbackComplete = true;
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EPCGTextureDensityFunction UPCGTexture2DSingleBaseData::GetDensityFunctionEquivalent() const
{
	return bUseDensitySourceChannel ? EPCGTextureDensityFunction::Multiply : EPCGTextureDensityFunction::Ignore;
}

void UPCGTexture2DSingleBaseData::SetDensityFunctionEquivalent(EPCGTextureDensityFunction InDensityFunction)
{
	bUseDensitySourceChannel = (InDensityFunction != EPCGTextureDensityFunction::Ignore);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UPCGTexture2DSingleBaseData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// TODO: add metadata support
	// TODO: add sampling along the bounds

	// TODO: needs unpicking of sample vs projection. I believe the below is a projection.. But semantics are slightly different.
	// 1 - We have some information telling us the 'z' size of the surface allowing us to reject points that would be too far from the surface, maybe including some density falloff by distance
	// 2 - We suppose that the surface has an infinite 'z' size, in which case the sampling is basically the same as the sampling, except that it does not change the position
	// 3 - The surface is infinitesimal - we'll return something if and only if the point overlaps with the projected position

	if (!IsValid())
	{
		return false;
	}

	if (ColorData.IsEmpty())
	{
		if (!bEmittedNoReadbackDataError)
		{
			UE_LOGF(LogPCG, Error, "Texture data was not read back to CPU - point cannot be sampled.");
			bEmittedNoReadbackDataError = true;
		}

		return false;
	}

	// Compute transform
	// TODO: embed local bounds center offset at this time?
	OutPoint.Transform = InTransform;
	FVector PointPositionInLocalSpace = Transform.InverseTransformPosition(InTransform.GetLocation());
	OutPoint.Transform.SetLocation(Transform.TransformPosition(PointPositionInLocalSpace));
	OutPoint.SetLocalBounds(InBounds); // TODO: should set Min.Z = Max.Z = 0;

	// Compute density & color (& metadata)
	// TODO: sample in the bounds given, not only on a single pixel
	FVector2D Position2D(PointPositionInLocalSpace.X, PointPositionInLocalSpace.Y);
	FBox2D Surface(FVector2D(-1.0f, -1.0f), FVector2D(1.0f, 1.0f));

	FLinearColor Color = FLinearColor(EForceInit::ForceInit);
	if (PCGTextureSamplingHelpers::Sample<FLinearColor>(Position2D, Surface, this, Width, Height, Color, [this](int32 Index) { return ColorData[Index]; }))
	{
		OutPoint.Color = Color;
		OutPoint.Density = bUseDensitySourceChannel ? PCGTextureSamplingHelpers::SampleFloatChannel(Color, ColorChannel) : 1.0f;
		return OutPoint.Density > 0 || bKeepZeroDensityPoints;
	}
	else
	{
		return false;
	}
}

const UPCGPointData* UPCGTexture2DSingleBaseData::CreatePointData(FPCGContext* Context) const
{
	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()));
}

const UPCGPointArrayData* UPCGTexture2DSingleBaseData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()));
}

const UPCGBasePointData* UPCGTexture2DSingleBaseData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGTexture2DSingleBaseData::CreatePointData);
	FBox2D LocalSurfaceBounds(FVector2D(-1.0f, -1.0f), FVector2D(1.0f, 1.0f));

	UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
	Data->InitializeFromData(this);

	// Early out for invalid data
	if (!IsValid())
	{
		UE_LOGF(LogPCG, Error, "Texture data does not have valid sizes - will return empty data.");
		return Data;
	}

	if (ColorData.IsEmpty())
	{
		UE_LOGF(LogPCG, Error, "Texture data was not read back to CPU - will return empty data.");

		return Data;
	}

	// Map target texel size to the current physical size of the texture data.
	const FVector::FReal XSize = 2.0 * Transform.GetScale3D().X;
	const FVector::FReal YSize = 2.0 * Transform.GetScale3D().Y;

	const int32 XCount = FMath::Floor(XSize / TexelSize);
	const int32 YCount = FMath::Floor(YSize / TexelSize);
	const int32 PointCount = XCount * YCount;

	if (PointCount <= 0)
	{
		UE_LOGF(LogPCG, Warning, "Texture data has a texel size larger than its data - will return empty data");
		return Data;
	}

	const FBox2D Surface(FVector2D(-1.0f, -1.0f), FVector2D(1.0f, 1.0f));

	Data->SetNumPoints(PointCount, /*bInitializeValues=*/false);
	Data->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Density | EPCGPointNativeProperties::Color);

	// Check if we are dealing with always allocated properties or not (if not set the constant extents
	if (Data->GetAllocatedProperties() != EPCGPointNativeProperties::All)
	{
		Data->SetExtents(FVector(TexelSize / 2.0));
	}

	auto ProcessRangeFunc = [this, XCount, YCount, &Surface, Data](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
	{
		int32 NumWritten = 0;
		FPCGPointValueRanges OutRanges(Data, /*bAllocate=*/false);

		for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
		{
			const int X = (ReadIndex % XCount);
			const int Y = (ReadIndex / XCount);

			const int32 WriteIndex = StartWriteIndex + NumWritten;

			// Full expression: -1.0 + 2.0 * (X + /*TexelCenter*/0.5) / XCount
			FVector2D LocalCoordinate((2.0 * X + 1.0) / XCount - 1.0, (2.0 * Y + 1.0) / YCount - 1.0);
			FLinearColor Color = FLinearColor(EForceInit::ForceInit);

			if (PCGTextureSamplingHelpers::Sample<FLinearColor>(LocalCoordinate, Surface, this, Width, Height, Color, [this](int32 Index) { return ColorData[Index]; }))
			{
				const float Density = bUseDensitySourceChannel ? PCGTextureSamplingHelpers::SampleFloatChannel(Color, ColorChannel) : 1.0f;
				if (Density > 0 || bKeepZeroDensityPoints)
				{
					FVector LocalPosition(LocalCoordinate, 0);
					FPCGPoint OutPoint(FTransform(Transform.TransformPosition(LocalPosition)),
						Density,
						PCGHelpers::ComputeSeed(X, Y));

					// Always the same extents so property wasn't allocated
					OutPoint.SetExtents(FVector(TexelSize / 2.0));
					OutPoint.Color = Color;

					OutRanges.SetFromPoint(WriteIndex, OutPoint);
					++NumWritten;
				}
			}
		}
		
		return NumWritten;
	};

	auto MoveDataRangeFunc = [Data](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
	{
		Data->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
	};

	auto FinishedFunc = [Data](int32 NumWritten)
	{
		Data->SetNumPoints(NumWritten);
	};

	FPCGAsync::AsyncProcessingRangeEx(
		Context ? &Context->AsyncState : nullptr,
		PointCount,
		[] {},
		ProcessRangeFunc,
		MoveDataRangeFunc,
		FinishedFunc,
		/*bEnableTimeSlicing=*/false);
		
	return Data;
}

bool UPCGTexture2DSingleBaseData::IsValid() const
{
	// Valid if dimensions are set AND we either already have CPU data or readback is still pending.
	return Height > 0 && Width > 0 && (!bIsReadbackComplete || !ColorData.IsEmpty());
}

bool UPCGTexture2DSingleBaseData::SamplePointLocal(const FVector2D& LocalPosition, FVector4& OutColor, float& OutDensity) const
{
	if (!IsValid())
	{
		return false;
	}

	if (ColorData.IsEmpty())
	{
		if (!bEmittedNoReadbackDataError)
		{
			UE_LOGF(LogPCG, Error, "Texture data was not read back to CPU - point cannot be sampled.");
			bEmittedNoReadbackDataError = true;
		}

		return false;
	}

	FVector2D Pos;
	Pos.X = FMath::Frac(LocalPosition.X);
	Pos.Y = FMath::Frac(LocalPosition.Y);

	const FLinearColor OutSample = PCGTextureSamplingHelpers::SampleInternal<FLinearColor>(Pos, Width, Height, Filter, [this](int32 Index) { return ColorData[Index]; });

	OutColor = OutSample;
	OutDensity = bUseDensitySourceChannel ? PCGTextureSamplingHelpers::SampleFloatChannel(OutSample, ColorChannel) : 1.0f;
	
	return OutDensity > 0.0 || bKeepZeroDensityPoints;
}

void UPCGTexture2DSingleBaseData::CopyBaseTextureData(UPCGTexture2DSingleBaseData* NewTextureData) const
{
	CopyTexture2DBaseData(NewTextureData);

	NewTextureData->bUseDensitySourceChannel = bUseDensitySourceChannel;
	NewTextureData->ColorChannel = ColorChannel;
	NewTextureData->TexelSize = TexelSize;
	NewTextureData->bUseAdvancedTiling = bUseAdvancedTiling;
	NewTextureData->Tiling = Tiling;
	NewTextureData->CenterOffset = CenterOffset;
	NewTextureData->Rotation = Rotation;
	NewTextureData->bUseTileBounds = bUseTileBounds;
	NewTextureData->TileBounds = TileBounds;
	NewTextureData->ColorData = ColorData;
	NewTextureData->Height = Height;
	NewTextureData->Width = Width;
	// Reset readback state so the copy can independently trigger readback if ColorData is empty.
	NewTextureData->bReadbackInitiated = false;
	// If ColorData was populated on the source (direct-init or already-read-back), the copy has it too.
	NewTextureData->bIsReadbackComplete = !ColorData.IsEmpty();
}

bool UPCGTexture2DSingleBaseData::RequestCPUReadback() const
{
	// Workaround for data that should be mutable but cannot be mutable because they are UProperties.
	return const_cast<UPCGTexture2DSingleBaseData*>(this)->RequestCPUReadback();
}

void UPCGTextureData::InitializeInternal(UTexture* InTexture, uint32 InTextureIndex, const FTransform& InTransform, bool* bOutInitializeCompleted, bool bCreateCPUDuplicateEditorOnly)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGTextureData::InitializeInternal);

	auto SetInitCompleted = [bOutInitializeCompleted](bool bIsDone)
	{
		if (bOutInitializeCompleted)
		{
			*bOutInitializeCompleted = bIsDone;
		}
	};

	if (bSuccessfullyInitialized)
	{
		SetInitCompleted(true);
		return;
	}

	if (!InTexture)
	{
		SetInitCompleted(true);
		return;
	}

	ResourceType = EPCGTextureResourceType::TextureObject;
	Texture = InTexture;
	TextureIndex = InTextureIndex;
	Transform = InTransform;
	const FTextureRHIRef InitRHI = GetTextureRHI();
	Format = InitRHI ? InitRHI->GetDesc().Format : PF_Unknown;
	NumMips = InitRHI ? InitRHI->GetDesc().NumMips : 0;

	Bounds = FBox(EForceInit::ForceInit);
	Bounds += FVector(-1.0f, -1.0f, 0.0f);
	Bounds += FVector(1.0f, 1.0f, 0.0f);
	Bounds = Bounds.TransformBy(Transform);

	// Always defer CPU readback. It will be triggered lazily by ConvertInputsIfNeeded when a downstream element needs it.
	FTextureRHIRef ResourceRHI = GetTextureRHI();
	const FIntPoint Extent = ResourceRHI ? ResourceRHI->GetDesc().Extent : FIntPoint::ZeroValue;
	Width = Extent.X;
	Height = Extent.Y;

#if WITH_EDITOR
	if (bCreateCPUDuplicateEditorOnly && !PrepareCPUDuplicateTexture())
	{
		SetInitCompleted(false);
		return;
	}
#endif

	bSuccessfullyInitialized = true;
	SetInitCompleted(true);
}

bool UPCGTextureData::Initialize(UTexture* InTexture, uint32 InTextureIndex, const FTransform& InTransform, bool bCreateCPUDuplicateEditorOnly)
{
	ResourceType = EPCGTextureResourceType::TextureObject;

	bool bInitializeDone = false;
	InitializeInternal(InTexture, InTextureIndex, InTransform, &bInitializeDone, bCreateCPUDuplicateEditorOnly);
	return bInitializeDone;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UPCGTextureData::Initialize(UTexture* InTexture, uint32 InTextureIndex, const FTransform& InTransform, bool bCreateCPUDuplicateEditorOnly, bool bInSkipReadbackToCPU)
{
	return Initialize(InTexture, InTextureIndex, InTransform, bCreateCPUDuplicateEditorOnly);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UPCGTextureData::Initialize(TRefCountPtr<IPooledRenderTarget> InTextureHandle, uint32 InTextureIndex, const FTransform& InTransform)
{
	ResourceType = EPCGTextureResourceType::ExportedTexture;
	TextureHandle = InTextureHandle;
	TextureIndex = InTextureIndex;
	Transform = InTransform;
	Format = InTextureHandle.IsValid() ? InTextureHandle->GetDesc().Format : PF_Unknown;

	Bounds = FBox(EForceInit::ForceInit);
	Bounds += FVector(-1.0f, -1.0f, 0.0f);
	Bounds += FVector(1.0f, 1.0f, 0.0f);
	Bounds = Bounds.TransformBy(Transform);

	// Always defer CPU readback. It will be triggered lazily when needed.
	const FIntPoint Extent = InTextureHandle.IsValid() ? InTextureHandle->GetDesc().Extent : FIntPoint::ZeroValue;
	Width = Extent.X;
	Height = Extent.Y;
	bSuccessfullyInitialized = true;

	return bSuccessfullyInitialized;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UPCGTextureData::Initialize(TRefCountPtr<IPooledRenderTarget> InTextureHandle, uint32 InTextureIndex, const FTransform& InTransform, bool bInSkipReadbackToCPU)
{
	return Initialize(MoveTemp(InTextureHandle), InTextureIndex, InTransform);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UPCGTexture2DSingleBaseData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(ColorData.GetAllocatedSize());
}

void UPCGTextureData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

#if WITH_EDITORONLY_DATA
	if (FPCGGraphCache::IsGatheringCacheSize() && 
		FPCGGraphCache::CacheSizeDetailLevel() == 0 &&
		::IsValid(DuplicateTexture) && 
		DuplicateTexture->Source.IsValid() && 
		DuplicateTexture->Source.GetNumMips() > 0)
	{
		DuplicateTexture->Source.LockMipReadOnly(0);
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(DuplicateTexture->Source.CalcMipSize(0));
		DuplicateTexture->Source.UnlockMip(0);
	}
#endif
}

void UPCGTextureData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(Ar);
}

FTextureRHIRef UPCGTextureData::GetTextureRHI() const
{
	if (ResourceType == EPCGTextureResourceType::TextureObject)
	{
		FTextureResource* Resource = Texture.IsValid() ? Texture->GetResource() : nullptr;
		return Resource ? Resource->GetTextureRHI() : nullptr;
	}
	else
	{
		return TextureHandle ? TextureHandle->GetRHI() : nullptr;
	}
}

UPCGSpatialData* UPCGTextureData::CopyInternal(FPCGContext* Context) const
{
	UPCGTextureData* NewTextureData = FPCGContext::NewObject_AnyThread<UPCGTextureData>(Context);

	CopyBaseTextureData(NewTextureData);

	NewTextureData->ResourceType = ResourceType;
	NewTextureData->Texture = Texture;

	return NewTextureData;
}

TOptional<bool> UPCGTextureData::TryReadbackFromCPUTexture()
{
	if (!Texture.IsValid())
	{
		return false;
	}

	// CPU Textures currently only support UTexture2D.
	UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
	if (!Texture2D)
	{
		return false;
	}

#if WITH_EDITOR
	if (!Texture2D->IsAsyncCacheComplete())
	{
		// Wait until texture ready before interrogating it for access options.
		return {};
	}
#endif

	FSharedImageConstRef CPUTextureRef = Texture2D->GetCPUCopy();
	if (!CPUTextureRef.IsValid())
	{
		return false;
	}

	Width = CPUTextureRef->SizeX;
	Height = CPUTextureRef->SizeY;

	const int32 PixelCount = Width * Height;
	ColorData.SetNum(PixelCount);

	if (CPUTextureRef->Format == ERawImageFormat::G8)
	{
		const TArrayView64<const uint8> DataView = CPUTextureRef->AsG8();

		for (int32 D = 0; D < PixelCount; ++D)
		{
			ColorData[D] = FColor(DataView[D], DataView[D], DataView[D]).ReinterpretAsLinear();
		}
	}
	else if (CPUTextureRef->Format == ERawImageFormat::BGRA8)
	{
		const TArrayView64<const FColor> DataView = CPUTextureRef->AsBGRA8();

		// Memory representation of FColor is BGRA, so we reinterpret as FLinearColor to get RGBA.
		for (int32 D = 0; D < PixelCount; ++D)
		{
			ColorData[D] = DataView[D].ReinterpretAsLinear();
		}
	}
	else if (CPUTextureRef->Format == ERawImageFormat::BGRE8)
	{
		const TArrayView64<const FColor> DataView = CPUTextureRef->AsBGRE8();

		// Memory representation of FColor is BGRA, so we reinterpret as FLinearColor to get RGBA.
		for (int32 D = 0; D < PixelCount; ++D)
		{
			ColorData[D] = DataView[D].ReinterpretAsLinear();
		}
	}
	else if (CPUTextureRef->Format == ERawImageFormat::RGBA16)
	{
		const TArrayView64<const uint16> DataView = CPUTextureRef->AsRGBA16();
		check(PixelCount * 4 == DataView.Num());

		for (int32 D = 0; D < PixelCount; ++D)
		{
			const uint32 Index = D * 4;
			ColorData[D] = FLinearColor(DataView[Index + 0], DataView[Index + 1], DataView[Index + 2], DataView[Index + 3]);
		}
	}
	else if (CPUTextureRef->Format == ERawImageFormat::RGBA16F)
	{
		const TArrayView64<const FFloat16Color> DataView = CPUTextureRef->AsRGBA16F();

		for (int32 D = 0; D < PixelCount; ++D)
		{
			ColorData[D] = FLinearColor(DataView[D]);
		}
	}
	else if (CPUTextureRef->Format == ERawImageFormat::RGBA32F)
	{
		const TArrayView64<const FLinearColor> DataView = CPUTextureRef->AsRGBA32F();

		for (int32 D = 0; D < PixelCount; ++D)
		{
			ColorData[D] = DataView[D];
		}
	}
	else if (CPUTextureRef->Format == ERawImageFormat::G16)
	{
		const TArrayView64<const uint16> DataView = CPUTextureRef->AsG16();

		for (int32 D = 0; D < PixelCount; ++D)
		{
			ColorData[D] = FColor(DataView[D], DataView[D], DataView[D]).ReinterpretAsLinear();
		}
	}
	else if (CPUTextureRef->Format == ERawImageFormat::R16F)
	{
		const TArrayView64<const FFloat16> DataView = CPUTextureRef->AsR16F();

		for (int32 D = 0; D < PixelCount; ++D)
		{
			ColorData[D] = FLinearColor(DataView[D], DataView[D], DataView[D]);
		}
	}
	else if (CPUTextureRef->Format == ERawImageFormat::R32F)
	{
		const TArrayView64<const float> DataView = CPUTextureRef->AsR32F();

		for (int32 D = 0; D < PixelCount; ++D)
		{
			ColorData[D] = FLinearColor(DataView[D], DataView[D], DataView[D]);
		}
	}
	else
	{
		UE_LOGF(LogPCG, Error, "PCGTextureReadback has an invalid format (%d) for CPU texture '%ls'.", CPUTextureRef->Format, *Texture2D->GetFName().ToString());

		Width = 0;
		Height = 0;
		ColorData.SetNum(0);

		return false;
	}

	return true;
}

bool UPCGTextureData::DispatchReadbackFromGPU()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGTextureData::DispatchReadbackFromGPU);

	if (bReadbackInitiated)
	{
		return bIsReadbackComplete;
	}

	if (ResourceType == EPCGTextureResourceType::TextureObject)
	{
		if (!Texture.IsValid())
		{
			return true;
		}

		if (!bUpdatedReadbackTextureResource)
		{
			// Only initialize the GPU resource if one does not already exist. Calling UpdateResource() on a texture with an existing GPU resource recreates
			// it from platform data, destroying any GPU-rendered content that has not been downloaded back to CPU.
			if (!Texture->GetResource())
			{
				Texture->UpdateResource();
			}

			bUpdatedReadbackTextureResource = true;
		}

		if (Texture->HasPendingInitOrStreaming())
		{
			return false;
		}
	}

	if (FTextureRHIRef TextureRHI = GetTextureRHI())
	{
		FPCGTextureReadbackDispatchParams Params;
		Params.SourceTexture = TextureRHI;
		// We should always use a point filter sampler since we are trying to get a 1 to 1 copy of the texture. We will do our own filtering later.
		Params.SourceSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Params.SourceTextureIndex = TextureIndex;

		const FIntVector TextureSize = TextureRHI->GetDesc().GetSize();
		Params.SourceDimensions = FIntPoint(TextureSize.X, TextureSize.Y);

		TWeakObjectPtr<UPCGTextureData> ThisWeakPtr(this);
		const FString ResourceName = *TextureRHI->GetName().ToString();

		RenderCaptureInterface::FScopedCapture RenderCapture(PCGTextureSamplingHelpers::GTriggerReadbackCaptureDispatches > 0, TEXT("PCGTextureReadbackCapture"));
		PCGTextureSamplingHelpers::GTriggerReadbackCaptureDispatches = FMath::Max(PCGTextureSamplingHelpers::GTriggerReadbackCaptureDispatches - 1, 0);

		FPCGTextureReadbackInterface::Dispatch(Params, [ThisWeakPtr, TextureSize, ResourceName](void* OutBuffer, int32 ReadbackWidth, int32 ReadbackHeight)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UPCGTextureData::DispatchReadbackFromGPU::DispatchCallback);

			UPCGTextureData* This = ThisWeakPtr.IsValid() ? ThisWeakPtr.Get() : nullptr;
			if (!This)
			{
				return;
			}

			// Texture readbacks can require memory alignment, e.g. a 127x127 texture can be readback as 128x128.
			// So when initializing the CPU data, we should ignore the additional pixels.
			const int32 ReadbackPaddingWidth = ReadbackWidth - TextureSize.X;
			const int32 ReadbackPaddingHeight = ReadbackHeight - TextureSize.Y;

			if (ReadbackPaddingWidth < 0 || ReadbackPaddingHeight < 0)
			{
				UE_LOGF(LogPCG, Error,
					"PCGTextureData readback has smaller dimensions than the source texture '%ls'. Expected greater than or equal to (%d, %d), received (%d, %d).",
					*ResourceName,
					TextureSize.X, TextureSize.Y,
					ReadbackWidth, ReadbackHeight);
				This->bIsReadbackComplete = true;
				return;
			}

			if (const FColor* FormattedImageData = reinterpret_cast<const FColor*>(OutBuffer))
			{
				This->Width = TextureSize.X;
				This->Height = TextureSize.Y;
				This->ColorData.SetNum(TextureSize.X * TextureSize.Y);

				int32 ActualTexelIndex = 0;

				for (int32 TexelY = 0; TexelY < TextureSize.Y; ++TexelY)
				{
					for (int32 TexelX = 0; TexelX < TextureSize.X; ++TexelX)
					{
						const int32 ReadbackTexelIndex = TexelY * ReadbackWidth + TexelX;
						This->ColorData[ActualTexelIndex++] = FormattedImageData[ReadbackTexelIndex].ReinterpretAsLinear();
					}
				}
			}
			else
			{
				UE_LOGF(LogPCG, Error, "PCGTextureData unable to get readback results from '%ls'.", *ResourceName);
			}

			This->bIsReadbackComplete = true;
		});

		bReadbackInitiated = true;
	}
	else
	{
		if (Texture.IsValid())
		{
			UE_LOGF(LogPCG, Error, "PCGTextureData failed to acquire texture resource for '%ls'.", *Texture->GetFName().ToString());
		}
		else
		{
			UE_LOGF(LogPCG, Error, "PCGTextureData failed to acquire texture resource.");
		}

		// Mark as complete (failed) so we don't retry forever.
		bIsReadbackComplete = true;
		return true;
	}

	// Not complete - wait for readback result.
	return false;
}

bool UPCGTextureData::RequestCPUReadback()
{
	check(IsInGameThread());

	if (IsReadbackComplete())
	{
		return true;
	}

	// Try CPU texture path first (synchronous, cheap).
	if (ResourceType == EPCGTextureResourceType::TextureObject)
	{
		TOptional<bool> ReadbackFromCPU = TryReadbackFromCPUTexture();
		if (!ReadbackFromCPU.IsSet())
		{
			// Still waiting on async texture compilation.
			return false;
		}

		if (*ReadbackFromCPU)
		{
			bIsReadbackComplete = true;
			return true;
		}
	}

#if WITH_EDITOR
	// Try the BulkData path, which reads exact full-resolution pixels from the texture's (or its duplicate's) uncompressed bulk data, avoiding GPU compression/streaming clamping.
	TOptional<bool> ReadbackFromBulkData = TryReadbackFromTextureBulkData();
	if (!ReadbackFromBulkData.IsSet())
	{
		// Still waiting on async texture compilation.
		return false;
	}

	if (*ReadbackFromBulkData)
	{
		bIsReadbackComplete = true;
		return true;
	}
#endif

	// Fall back to async GPU readback.
	return DispatchReadbackFromGPU();
}

#if WITH_EDITOR
TOptional<bool> UPCGTextureData::TryReadbackFromTextureBulkData()
{
	UTexture2D* TextureAs2D = Cast<UTexture2D>(Texture.Get());
	if (!TextureAs2D)
	{
		if (UTexture2DArray* Texture2DArray = Cast<UTexture2DArray>(Texture))
		{
			TextureAs2D = Texture2DArray->SourceTextures.IsValidIndex(TextureIndex) ? Texture2DArray->SourceTextures[TextureIndex] : nullptr;
		}
	}

	UTexture2D* TextureForReadback = nullptr;
	TOptional<bool> bCanGPUTextureBeCPUAccessed = PCGTextureSamplingHelpers::CanGPUTextureBeCPUAccessed(TextureAs2D);
	if (!bCanGPUTextureBeCPUAccessed.IsSet())
	{
		return {};
	}

	if (TextureAs2D && *bCanGPUTextureBeCPUAccessed)
	{
		TextureForReadback = TextureAs2D;
	}
	else
	{
		TOptional<bool> bCanGPUTextureBeCPUAccessedDupe = PCGTextureSamplingHelpers::CanGPUTextureBeCPUAccessed(DuplicateTexture);
		if (!bCanGPUTextureBeCPUAccessedDupe.IsSet())
		{
			return {};
		}

		if (*bCanGPUTextureBeCPUAccessedDupe)
		{
			TextureForReadback = DuplicateTexture;
		}
	}

	if (!TextureForReadback)
	{
		return false;
	}

	if (!TextureForReadback->IsAsyncCacheComplete())
	{
		return {};
	}

	FTexturePlatformData* PlatformData = TextureForReadback->GetPlatformData();
	if (!PlatformData)
	{
		UE_LOGF(LogPCG, Warning, "GetPlatformData failed");
		return false;
	}

	bool bBulkDataAccessed = false;

	if (const uint8_t* BulkData = reinterpret_cast<const uint8_t*>(PlatformData->Mips.IsEmpty() ? nullptr : PlatformData->Mips[0].BulkData.LockReadOnly()))
	{
		bBulkDataAccessed = true;

		Width = PlatformData->SizeX;
		Height = PlatformData->SizeY;
		const int32 PixelCount = Width * Height;
		ColorData.SetNum(PixelCount);

		const FColor* FormattedImageData = reinterpret_cast<const FColor*>(BulkData);
		for (int32 D = 0; D < PixelCount; ++D)
		{
			ColorData[D] = FormattedImageData[D].ReinterpretAsLinear();
		}

		PlatformData->Mips[0].BulkData.Unlock();
	}
	else if (Texture.IsValid())
	{
		UE_LOGF(LogPCG, Error, "PCGTextureData unable to get bulk data from '%ls'.", *Texture->GetFName().ToString());
	}

	return bBulkDataAccessed;
}

bool UPCGTextureData::PrepareCPUDuplicateTexture()
{
	// Force-CPU-sampling path: create a duplicate texture with settings that keep it CPU-accessible at full resolution.
	// Needed for textures that would otherwise be compressed/resolution-clamped by the streaming system on GPU readback.
	UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
	if (!Texture2D)
	{
		if (UTexture2DArray* Texture2DArray = Cast<UTexture2DArray>(Texture))
		{
			Texture2D = Texture2DArray->SourceTextures.IsValidIndex(TextureIndex) ? Texture2DArray->SourceTextures[TextureIndex] : nullptr;
		}
	}

	if (!Texture2D)
	{
		return true;
	}

	TOptional<bool> CanGPUTextureBeCPUAccessed = PCGTextureSamplingHelpers::CanGPUTextureBeCPUAccessed(Texture2D);
	if (!CanGPUTextureBeCPUAccessed.IsSet())
	{
		// Wait until we can ascertain this.
		return false;
	}

	TOptional<bool> IsCPUAccessible = PCGTextureSamplingHelpers::IsTextureCPUAccessible(Texture2D);
	if (!IsCPUAccessible.IsSet())
	{
		// Wait until we can ascertain this.
		return false;
	}

	if (!DuplicateTexture && !*CanGPUTextureBeCPUAccessed && !*IsCPUAccessible)
	{
		// Duplicate the texture and change access flags (editor only). RequestCPUReadback will read from the duplicate's BulkData.
		FObjectDuplicationParameters DuplicationParams(Texture2D, /*Outer=*/this);
		DuplicationParams.FlagMask = GetFlags() & ~(RF_Standalone | RF_Public);
		DuplicateTexture = CastChecked<UTexture2D>(StaticDuplicateObjectEx(DuplicationParams));
	}

	if (DuplicateTexture && !bDuplicateTextureInitialized)
	{
		// Wait until texture compilation is complete on the duplicated texture, otherwise we can crash in PreEditChange().
		if (!DuplicateTexture->IsAsyncCacheComplete())
		{
			return false;
		}

		DuplicateTexture->PreEditChange(nullptr);
		DuplicateTexture->MipGenSettings = TextureMipGenSettings::TMGS_NoMipmaps;
		DuplicateTexture->CompressionSettings = TextureCompressionSettings::TC_VectorDisplacementmap; // Non-compressed format (B8G8R8A8) needed to keep the texture CPU-accessible.
		DuplicateTexture->SRGB = false;
		DuplicateTexture->PostEditChange();

		bDuplicateTextureInitialized = true;
	}

	return true;
}
#endif
