// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGToolData.h"

#include "GameFramework/Actor.h"

FName FPCGInteractiveToolWorkingBaseData::GetWorkingDataIdentifier() const
{
	return WorkingDataIdentifier;
}

FPCGGraphToolData::FPCGGraphToolData()
	: InitialActorClassToSpawn(AActor::StaticClass())
{

}

const TInstancedStruct<FPCGInteractiveToolWorkingBaseData>* FPCGInteractiveToolDataContainer::FindToolDataStruct(FName WorkingDataIdentifier) const
{
	for (const TInstancedStruct<FPCGInteractiveToolWorkingBaseData>& Data : ToolData)
	{
		if (const FPCGInteractiveToolWorkingBaseData* DataPtr = Data.GetPtr<FPCGInteractiveToolWorkingBaseData>(); DataPtr && DataPtr->GetWorkingDataIdentifier().IsEqual(WorkingDataIdentifier))
		{
			return &Data;
		}
	}

	return nullptr;
}

TInstancedStruct<FPCGInteractiveToolWorkingBaseData>* FPCGInteractiveToolDataContainer::FindMutableToolDataStruct(FName WorkingDataIdentifier)
{
	for (TInstancedStruct<FPCGInteractiveToolWorkingBaseData>& Data : ToolData)
	{
		if (const FPCGInteractiveToolWorkingBaseData* DataPtr = Data.GetPtr<FPCGInteractiveToolWorkingBaseData>(); DataPtr && DataPtr->GetWorkingDataIdentifier().IsEqual(WorkingDataIdentifier))
		{
			return &Data;
		}
	}

	return nullptr;
}

int32 FPCGInteractiveToolDataContainer::RemoveToolData(FName WorkingDataIdentifier)
{
	int32 Result = 0;
	for (auto It = ToolData.CreateIterator(); It; ++It)
	{
		if (const FPCGInteractiveToolWorkingBaseData* DataPtr = It->GetPtr<FPCGInteractiveToolWorkingBaseData>(); DataPtr && DataPtr->GetWorkingDataIdentifier().IsEqual(WorkingDataIdentifier))
		{
			It.RemoveCurrent();
			Result++;
		}
	}

	return Result;
}
