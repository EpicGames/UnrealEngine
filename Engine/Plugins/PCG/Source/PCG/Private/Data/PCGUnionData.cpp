// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGUnionData.h"

#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadataAccessor.h"

#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGUnionData)

namespace PCGUnionDataMaths
{
	float ComputeDensity(float InDensityToUpdate, float InOtherDensity, EPCGUnionDensityFunction DensityFunction)
	{
		if (DensityFunction == EPCGUnionDensityFunction::ClampedAddition)
		{
			return FMath::Min(InDensityToUpdate + InOtherDensity, 1.0f);
		}
		else if (DensityFunction == EPCGUnionDensityFunction::Binary)
		{
			return (InOtherDensity > 0) ? 1.0f : InDensityToUpdate;
		}
		else // Maximum
		{
			return FMath::Max(InDensityToUpdate, InOtherDensity);
		}
	}

	float UpdateDensity(float& InDensityToUpdate, float InOtherDensity, EPCGUnionDensityFunction DensityFunction)
	{
		InDensityToUpdate = ComputeDensity(InDensityToUpdate, InOtherDensity, DensityFunction);
		return InDensityToUpdate;
	}
}

void UPCGUnionData::Initialize(const UPCGSpatialData* InA, const UPCGSpatialData* InB)
{
	check(InA && InB);
	AddData(InA);
	AddData(InB);
}

void UPCGUnionData::AddData(const UPCGSpatialData* InData)
{
	check(InData);
	check(Metadata);

	Data.Add(InData);

	if (Data.Num() == 1)
	{
		TargetActor = InData->TargetActor;
		CachedBounds = InData->GetBounds();
		CachedStrictBounds = InData->GetStrictBounds();
		CachedDimension = InData->GetDimension();
	}
	else
	{
		CachedBounds += InData->GetBounds();
		CachedStrictBounds = PCGHelpers::OverlapBounds(CachedStrictBounds, InData->GetStrictBounds());
		CachedDimension = FMath::Max(CachedDimension, InData->GetDimension());
	}

	if (!FirstNonTrivialTransformData && InData->HasNonTrivialTransform())
	{
		FirstNonTrivialTransformData = InData;
	}
}

void UPCGUnionData::VisitDataNetwork(TFunctionRef<void(const UPCGData*)> Action) const
{
	for (const TObjectPtr<const UPCGSpatialData>& Datum : Data)
	{
		if (Datum)
		{
			Datum->VisitDataNetwork(Action);
		}
	}
}

FPCGCrc UPCGUnionData::ComputeCrc(bool bFullDataCrc) const
{
	FArchiveCrc32 Ar;

	AddToCrc(Ar, bFullDataCrc);

	// Chain together CRCs of operands
	int32 NumOperands = Data.Num();
	Ar << NumOperands;

	for (const TObjectPtr<const UPCGSpatialData>& Datum : Data)
	{
		if (Datum)
		{
			uint32 DatumCrc = Datum->GetOrComputeCrc(bFullDataCrc).GetValue();
			Ar << DatumCrc;
		}
	}
	
	return FPCGCrc(Ar.GetCrc());
}

void UPCGUnionData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// Implementation note: no metadata in composite data at this point.
	// @todo_pcg: need metadata serialization now

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	uint32 UnionTypeValue = static_cast<uint32>(UnionType);
	Ar << UnionTypeValue;

	uint32 DensityFunctionValue = static_cast<uint32>(DensityFunction);
	Ar << DensityFunctionValue;
}

int UPCGUnionData::GetDimension() const
{
	return CachedDimension;
}

FBox UPCGUnionData::GetBounds() const
{
	return CachedBounds;
}

FBox UPCGUnionData::GetStrictBounds() const
{
	return CachedStrictBounds;
}

bool UPCGUnionData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(UPCGUnionData::SamplePoint);
	FTransform PointTransform = InTransform;
	bool bHasSetPoint = false;

	if (FirstNonTrivialTransformData)
	{
		if (FirstNonTrivialTransformData->SamplePoint(InTransform, InBounds, OutPoint, OutMetadata))
		{
			PointTransform = OutPoint.Transform;
			bHasSetPoint = true;

			if (DensityFunction == EPCGUnionDensityFunction::Binary && OutPoint.Density > 0)
			{
				OutPoint.Density = 1.0f;
			}
		}
	}

	TArray<const UPCGSpatialData*> DataRawPtr = Data;

	const bool bSkipLoop = (bHasSetPoint && !OutMetadata && OutPoint.Density >= 1.0f);
	const int32 DataCount = DataRawPtr.Num();
	for (int32 DataIndex = 0; DataIndex < DataCount && !bSkipLoop; ++DataIndex)
	{
		if (DataRawPtr[DataIndex] == FirstNonTrivialTransformData)
		{
			continue;
		}

		FPCGPoint PointInData;
		if(DataRawPtr[DataIndex]->SamplePoint(PointTransform, InBounds, PointInData, OutMetadata))
		{
			if (!bHasSetPoint)
			{
				OutPoint = PointInData;
				bHasSetPoint = true;
			}
			else
			{
				// Update density
				PCGUnionDataMaths::UpdateDensity(OutPoint.Density, PointInData.Density, DensityFunction);

				OutPoint.Color = FVector4(
					FMath::Max(OutPoint.Color.X, PointInData.Color.X),
					FMath::Max(OutPoint.Color.Y, PointInData.Color.Y),
					FMath::Max(OutPoint.Color.Z, PointInData.Color.Z),
					FMath::Max(OutPoint.Color.W, PointInData.Color.W));

				// Merge properties into OutPoint
				if (OutMetadata)
				{
					if (OutPoint.MetadataEntry != PCGInvalidEntryKey && PointInData.MetadataEntry != PCGInvalidEntryKey)
					{
						OutMetadata->MergePointAttributesSubset(OutPoint, OutMetadata, OutMetadata, PointInData, OutMetadata, DataRawPtr[DataIndex]->Metadata, OutPoint, EPCGMetadataOp::Max);
					}
					else if (PointInData.MetadataEntry != PCGInvalidEntryKey)
					{
						OutPoint.MetadataEntry = PointInData.MetadataEntry;
					}
				}
			}
			
			if (bHasSetPoint && !OutMetadata && OutPoint.Density >= 1.0f)
			{
				break;
			}
		}
	}

	return (bHasSetPoint && OutPoint.Density > 0);
}

bool UPCGUnionData::HasNonTrivialTransform() const
{
	return (FirstNonTrivialTransformData != nullptr || Super::HasNonTrivialTransform());
}

const UPCGSpatialData* UPCGUnionData::FindFirstConcreteShapeFromNetwork() const
{
	// Return first concrete candidate data.
	for (const TObjectPtr<const UPCGSpatialData>& Datum : Data)
	{
		const UPCGSpatialData* Candidate = Datum ? Datum->FindFirstConcreteShapeFromNetwork() : nullptr;
		if (Candidate)
		{
			return Candidate;
		}
	}

	return nullptr;
}

const UPCGPointData* UPCGUnionData::CreatePointData(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGUnionData::CreatePointData);

	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass(), [](FPCGContext* InContext, const UPCGSpatialData* InSpatialData) { return InSpatialData->ToPointData(InContext); }), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGUnionData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGUnionData::CreatePointArrayData);

	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass(), [](FPCGContext* InContext, const UPCGSpatialData* InSpatialData) { return InSpatialData->ToPointArrayData(InContext); }), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGUnionData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass, TFunctionRef<const UPCGBasePointData*(FPCGContext*, const UPCGSpatialData*)> ToPointDataFunc) const
{
	const bool bBinaryDensity = (DensityFunction == EPCGUnionDensityFunction::Binary);

	// Trivial results
	if (Data.Num() == 0)
	{
		UE_LOGF(LogPCG, Error, "Invalid union");
		return nullptr;
	}
	else if (Data.Num() == 1 && !bBinaryDensity)
	{
		UE_LOGF(LogPCG, Verbose, "Union is trivial");
		return ToPointDataFunc(Context, Data[0]);
	}

	// Cache raw pointers for metadata as these are much faster to use
	TArray<const UPCGSpatialData*> DataRawPtr = Data;
	TArray<const UPCGMetadata*> InputMetadatas;
	InputMetadatas.SetNumUninitialized(Data.Num());
	
	const UPCGBasePointData* FirstSource = ToPointDataFunc(Context, DataRawPtr[0]);
	InputMetadatas[0] = FirstSource->Metadata;

	for (int32 i = 1; i < DataRawPtr.Num(); i++)
	{
		InputMetadatas[i] = ToPointDataFunc(Context, DataRawPtr[i])->Metadata;
	}

	UPCGBasePointData* PointData = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
	UPCGMetadata* OutMetadata = PointData->MutableMetadata();

	FPCGInitializeFromDataParams InitializeFromDataParams(this);
	InitializeFromDataParams.bInheritSpatialData = false;
	// Since we have collapsed the source data, we need to inherit from this one.
	InitializeFromDataParams.SourceOverride = FirstSource;
	PointData->InitializeFromDataWithParams(InitializeFromDataParams);

	switch (UnionType)
	{
	case EPCGUnionType::LeftToRightPriority:
	default:
		CreateSequentialPointData(Context, DataRawPtr, InputMetadatas, PointData, OutMetadata, /*bLeftToRight=*/true, ToPointDataFunc);
		break;

	case EPCGUnionType::RightToLeftPriority:
		CreateSequentialPointData(Context, DataRawPtr, InputMetadatas, PointData, OutMetadata, /*bLeftToRight=*/false, ToPointDataFunc);
		break;

	case EPCGUnionType::KeepAll:
		{
			int32 PointInputCount = 0;
			TArray<const UPCGBasePointData*> PointInputDatas;
			PointInputDatas.SetNumUninitialized(DataRawPtr.Num());
			for (int32 Index = 0; Index < PointInputDatas.Num(); ++Index)
			{
				PointInputDatas[Index] = ToPointDataFunc(Context, DataRawPtr[Index]);
				PointInputCount += PointInputDatas[Index]->GetNumPoints();
			}

			const EPCGPointNativeProperties PropertiesToAllocate = UPCGBasePointData::GetPropertiesToAllocateFromPointData(PointInputDatas);

			if (PointInputCount > 0)
			{
				PointData->SetNumPoints(PointInputCount);
				PointData->AllocateProperties(PropertiesToAllocate);

				int32 NumWritten = 0;

				for (int32 DataIndex = 0; DataIndex < DataRawPtr.Num(); ++DataIndex)
				{
					const UPCGSpatialData* Datum = DataRawPtr[DataIndex];
					const UPCGBasePointData* DatumPointData = PointInputDatas[DataIndex];
					const UPCGMetadata* DatumPointMetadata = DatumPointData->Metadata;

					const int32 DatumNumPoints = DatumPointData->GetNumPoints();
					const int32 TargetPointOffset = NumWritten;

					if (DatumNumPoints > 0)
					{
						DatumPointData->CopyPointsTo(PointData, 0, TargetPointOffset, DatumNumPoints);
						NumWritten += DatumNumPoints;

						if (DataIndex > 0)
						{
							TPCGValueRange<int64> TargetMetadataEntryRange = PointData->GetMetadataEntryValueRange();

							// TODO: could optimize case where there is a common parent between Data 0 and current data, for points that still point to common parent metadata.
							for (int32 TargetIndex = TargetPointOffset; TargetIndex < TargetPointOffset + DatumNumPoints; ++TargetIndex)
							{
								TargetMetadataEntryRange[TargetIndex] = PCGInvalidEntryKey;
							}

							if (OutMetadata && DatumPointMetadata && DatumPointMetadata->GetAttributeCount() > 0)
							{
								TArray<PCGMetadataEntryKey, TInlineAllocator<256>> DatumKeys;
								DatumKeys.SetNumUninitialized(DatumNumPoints);

								const TConstPCGValueRange<int64> DatumMetadataEntryRange = DatumPointData->GetConstMetadataEntryValueRange();
								for (int32 DatumIndex = 0; DatumIndex < DatumKeys.Num(); ++DatumIndex)
								{
									DatumKeys[DatumIndex] = DatumMetadataEntryRange[DatumIndex];
								}

								TArray<PCGMetadataEntryKey, TInlineAllocator<256>> TargetKeys;
								TargetKeys.SetNumUninitialized(DatumNumPoints);

								for (int32 TargetIndex = 0; TargetIndex < TargetKeys.Num(); ++TargetIndex)
								{
									TargetKeys[TargetIndex] = PCGInvalidEntryKey;
								}

								PointData->Metadata->SetAttributes(DatumKeys, DatumPointMetadata, TargetKeys, Context);

								// Write back
								for (int32 TargetIndex = 0; TargetIndex < TargetKeys.Num(); ++TargetIndex)
								{
									TargetMetadataEntryRange[TargetPointOffset + TargetIndex] = TargetKeys[TargetIndex];
								}
							}
						}
					}
				}

				// Correct density for binary-style union
				if (bBinaryDensity)
				{
					if (EnumHasAllFlags(PointData->GetAllocatedProperties(), EPCGPointNativeProperties::Density))
					{
						TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange(/*bAllocate=*/false);
						for (float& Density : DensityRange)
						{
							Density = ((Density > 0) ? 1.0f : 0);
						}
					}
					else
					{
						PointData->SetDensity(PointData->GetDensity(0) > 0 ? 1.0f : 0);
					}
				}
			}
		}
		break;
	}

	UE_LOGF(LogPCG, Verbose, "Union generated %d points out of %d data sources", PointData->GetNumPoints(), Data.Num());

	return PointData;
}

void UPCGUnionData::CreateSequentialPointData(FPCGContext* Context, TArray<const UPCGSpatialData*>& InputDatas, TArray<const UPCGMetadata*>& InputMetadatas, UPCGBasePointData* OutPointData, UPCGMetadata* OutMetadata, bool bLeftToRight, TFunctionRef<const UPCGBasePointData* (FPCGContext*, const UPCGSpatialData*)> ToPointDataFunc) const
{
	check(OutPointData);
		
	int32 PointInputCount = 0;
	
	TArray<const UPCGBasePointData*> PointInputDatas;
	PointInputDatas.SetNumUninitialized(InputDatas.Num());
	for (int32 Index = 0; Index < PointInputDatas.Num(); ++Index)
	{
		PointInputDatas[Index] = ToPointDataFunc(Context, InputDatas[Index]);
		PointInputCount += PointInputDatas[Index]->GetNumPoints();
	}
	
	OutPointData->SetNumPoints(PointInputCount, /*bInitializeValues=*/false);

	const EPCGPointNativeProperties PropertiesToAllocate = UPCGBasePointData::GetPropertiesToAllocateFromPointData(PointInputDatas);
	OutPointData->AllocateProperties(PropertiesToAllocate | EPCGPointNativeProperties::MetadataEntry | EPCGPointNativeProperties::Density);
	const bool bSetColor = EnumHasAnyFlags(PropertiesToAllocate, EPCGPointNativeProperties::Color);

	struct FIndexParams
	{
		FIndexParams(const int32 InFirstDataIndex, const int32 InLastDataIndex, const int32 InDataIndexIncrement)
			: FirstDataIndex(InFirstDataIndex), LastDataIndex(InLastDataIndex), DataIndexIncrement(InDataIndexIncrement), CurrentIndex(0) {}

		const int32 FirstDataIndex;
		const int32 LastDataIndex;
		const int32 DataIndexIncrement;
		int32 CurrentIndex;
	};

	FIndexParams IndexParams(
		/*InFirstDataIndex=*/bLeftToRight ? 0 : InputDatas.Num() - 1,
		/*InLastDataIndex=*/bLeftToRight ? InputDatas.Num() : -1,
		/*InDataIndexIncrement=*/bLeftToRight ? 1 : -1);

	int32 PointOffset = 0;

	auto MoveDataRange = [OutPointData, &PointOffset](int32 ReadIndex, int32 WriteIndex, int32 Count)
	{
		OutPointData->MoveRange(ReadIndex + PointOffset, WriteIndex + PointOffset, Count);
	};

	auto Finished = [&PointOffset](int32 Count)
	{
		PointOffset += Count;
	};
		
	auto ProcessRange = [this, &PointOffset, &PointInputDatas, &InputDatas, &InputMetadatas, OutPointData, OutMetadata, bSetColor](int32 StartReadIndex, int32 StartWriteIndex, int32 Count, FIndexParams& IndexParams)
	{
		const UPCGBasePointData* CurrentPointData = PointInputDatas[IndexParams.CurrentIndex];
				
		const FConstPCGPointValueRanges InRanges(CurrentPointData);
		FPCGPointValueRanges OutRanges(OutPointData, /*bAllocate=*/false);

		// Use const range for values we are not explicitely writing to
		const FConstPCGPointValueRanges ConstOutRanges(OutPointData);

		int32 NumWritten = 0;

		for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
		{
			const int32 WriteIndex = StartWriteIndex + NumWritten;

			// Discard point if it is already covered by a previous data
			bool bPointToExclude = false;
			for (int32 PreviousDataIndex = IndexParams.FirstDataIndex; PreviousDataIndex != IndexParams.CurrentIndex; PreviousDataIndex += IndexParams.DataIndexIncrement)
			{
				if (InputDatas[PreviousDataIndex]->GetDensityAtPosition(InRanges.TransformRange[ReadIndex].GetLocation()) != 0)
				{
					bPointToExclude = true;
					break;
				}
			}

			if (bPointToExclude)
			{
				continue;
			}
						
			check(OutMetadata);
						
			OutRanges.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);
			++NumWritten;

			if (OutMetadata->GetParent() != InputMetadatas[IndexParams.CurrentIndex])
			{
				OutRanges.MetadataEntryRange[WriteIndex] = OutMetadata->AddEntry();
								
				// Since we can't inherit from the parent point, we'll set the values directly here
				OutMetadata->SetAttributes(InRanges.MetadataEntryRange[ReadIndex], InputMetadatas[IndexParams.CurrentIndex], OutRanges.MetadataEntryRange[WriteIndex]);
			}

			if (DensityFunction == EPCGUnionDensityFunction::Binary && OutRanges.DensityRange[WriteIndex] > 0)
			{
				OutRanges.DensityRange[WriteIndex] = 1.0f;
			}

			// Update density & metadata based on current & following data
			for (int32 FollowingDataIndex = IndexParams.CurrentIndex + IndexParams.DataIndexIncrement; FollowingDataIndex != IndexParams.LastDataIndex; FollowingDataIndex += IndexParams.DataIndexIncrement)
			{
				const UPCGMetadata* FollowingMetadata = InputMetadatas[FollowingDataIndex];

				// If density is saturated and there are no metadata attributes then we can skip this data as it will not contribute.
				if (OutRanges.DensityRange[WriteIndex] >= 1.0f && (!FollowingMetadata || FollowingMetadata->GetAttributeCount() == 0))
				{
					continue;
				}

				FPCGPoint PointInData;

				const FBox LocalBounds = PCGPointHelpers::GetLocalBounds(ConstOutRanges.BoundsMinRange[WriteIndex], ConstOutRanges.BoundsMaxRange[WriteIndex]);
				if (InputDatas[FollowingDataIndex]->SamplePoint(ConstOutRanges.TransformRange[WriteIndex], LocalBounds, PointInData, OutMetadata))
				{
					// Update density
					PCGUnionDataMaths::UpdateDensity(OutRanges.DensityRange[WriteIndex], PointInData.Density, DensityFunction);

					if (bSetColor)
					{
						const FVector4& InColor = InRanges.ColorRange[ReadIndex];
						FVector4& OutColor = OutRanges.ColorRange[WriteIndex];

						// Update color
						OutColor = FVector4(
							FMath::Max(OutColor.X, InColor.X),
							FMath::Max(OutColor.Y, InColor.Y),
							FMath::Max(OutColor.Z, InColor.Z),
							FMath::Max(OutColor.W, InColor.W));
					}

					if (OutRanges.MetadataEntryRange[WriteIndex] != PCGInvalidEntryKey && PointInData.MetadataEntry != PCGInvalidEntryKey)
					{
						OutMetadata->MergeAttributesSubset(OutRanges.MetadataEntryRange[WriteIndex], OutMetadata, OutMetadata, PointInData.MetadataEntry, OutMetadata, FollowingMetadata, OutRanges.MetadataEntryRange[WriteIndex], EPCGMetadataOp::Max);
					}
					else if (PointInData.MetadataEntry != PCGInvalidEntryKey)
					{
						OutRanges.MetadataEntryRange[WriteIndex] = PointInData.MetadataEntry;
					}
				}
			}
		}

		return NumWritten;
	};

	// Note: this is a O(N^2) implementation. 
	// TODO: It is easy to implement a kind of divide & conquer algorithm here, but it will require some temporary storage.
	for (int32 DataIndex = IndexParams.FirstDataIndex; DataIndex != IndexParams.LastDataIndex; DataIndex += IndexParams.DataIndexIncrement)
	{
		IndexParams.CurrentIndex = DataIndex;

		auto ProcessRangeCurrentIndex = [&IndexParams, &ProcessRange, &PointOffset](int32 StartReadIndex, int32 StartWriteIndex, int32 Count) -> int32
		{
			return ProcessRange(StartReadIndex, StartWriteIndex + PointOffset, Count, IndexParams);
		};

		const UPCGBasePointData* CurrentPointData = PointInputDatas[DataIndex];
		OutPointData->CopyUnallocatedPropertiesFrom(CurrentPointData);

		FPCGAsync::AsyncProcessingRangeEx(
			Context ? &Context->AsyncState : nullptr,
			CurrentPointData->GetNumPoints(),
			[]() {},
			ProcessRangeCurrentIndex,
			MoveDataRange,
			Finished,
			/*bEnableTimeSlicing=*/false);
	}

	// Set final point count
	OutPointData->SetNumPoints(PointOffset);
}

UPCGSpatialData* UPCGUnionData::CopyInternal(FPCGContext* Context) const
{
	UPCGUnionData* NewUnionData = FPCGContext::NewObject_AnyThread<UPCGUnionData>(Context);

	NewUnionData->Data = Data;
	NewUnionData->FirstNonTrivialTransformData = FirstNonTrivialTransformData;
	NewUnionData->UnionType = UnionType;
	NewUnionData->DensityFunction = DensityFunction;
	NewUnionData->CachedBounds = CachedBounds;
	NewUnionData->CachedStrictBounds = CachedStrictBounds;
	NewUnionData->CachedDimension = CachedDimension;

	return NewUnionData;
}

void UPCGUnionData::InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const
{
	check(InParams.bInheritMetadata);
	check(MetadataToInitialize);

	// Duplicate data case, call the spatial base method
	if (InParams.bIsDuplicatingData)
	{
		UPCGSpatialData::InitializeTargetMetadata(InParams, MetadataToInitialize);
		return;
	}

	if (!Data.IsEmpty())
	{
		// In the case of the Union, we initialize the data from all the sources, special case for the first element, use the source override if specified.
		// The union will add its own attributes afterward.
		FPCGInitializeFromDataParams CopyParams = InParams;
		CopyParams.SourceOverride = nullptr;
		
		for (int32 i = 0; i < Data.Num(); ++i)
		{
			if (const UPCGSpatialData* UnionedData = Data[i])
			{
				CopyParams.Source = (InParams.SourceOverride && i == 0) ? InParams.SourceOverride.Get() : UnionedData;
				CopyParams.Source->InitializeTargetMetadata(CopyParams, MetadataToInitialize);
			}
		}
	}

	MetadataToInitialize->AddAttributes(Metadata);
}

