// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGVirtualTextureData.h"

#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"

#include "Components/RuntimeVirtualTextureComponent.h"
#include "VT/RuntimeVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGVirtualTextureData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoVirtualTexture, UPCGVirtualTextureData)

void UPCGVirtualTextureData::Initialize(const URuntimeVirtualTextureComponent* InVirtualTextureComponent)
{
	check(InVirtualTextureComponent);
	RuntimeVirtualTexture = InVirtualTextureComponent->GetVirtualTexture();
}

UPCGSpatialData* UPCGVirtualTextureData::CopyInternal(FPCGContext* Context) const
{
	UPCGVirtualTextureData* NewTextureData = FPCGContext::NewObject_AnyThread<UPCGVirtualTextureData>(Context);

	CopyBaseSurfaceData(NewTextureData);

	NewTextureData->RuntimeVirtualTexture = RuntimeVirtualTexture;

	return NewTextureData;
}

const UPCGPointData* UPCGVirtualTextureData::CreatePointData(FPCGContext* Context) const
{
	UPCGPointData* Data = FPCGContext::NewObject_AnyThread<UPCGPointData>(Context);
	Data->InitializeFromData(this);

	return Data;
}

const UPCGPointArrayData* UPCGVirtualTextureData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	UPCGPointArrayData* Data = FPCGContext::NewObject_AnyThread<UPCGPointArrayData>(Context);
	Data->InitializeFromData(this);

	return Data;
}
