// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGVolumeData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGVolumeSampler.h"
#include "Helpers/PCGHelpers.h"

#include "Components/BrushComponent.h"
#include "Engine/CollisionProfile.h"
#include "GameFramework/Volume.h"

#include "Chaos/ChaosEngineInterface.h"
#include "Physics/PhysicsInterfaceDeclares.h"
#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGVolumeData)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoVolume, UPCGVolumeData)

UPCGVolumeData::~UPCGVolumeData()
{
	ReleaseInternalBodyInstance();
}

void UPCGVolumeData::ReleaseInternalBodyInstance()
{
	if (VolumeBodyInstance)
	{
		if (VolumeBodyInstance->IsValidBodyInstance())
		{
			VolumeBodyInstance->TermBody();
		}

		delete VolumeBodyInstance;
		VolumeBodyInstance = nullptr;
	}
}

void UPCGVolumeData::Initialize(AVolume* InVolume)
{
	check(InVolume);
	Volume = InVolume;
	
	if (PCGHelpers::IsRuntimeOrPIE())
	{
		const UBrushComponent* Brush = Volume->GetBrushComponent();
		if (Brush && Brush->BodyInstance.GetCollisionProfileName() == UCollisionProfile::NoCollision_ProfileName)
		{
			UE_LOGF(LogPCG, Warning, "Volume Data points to a Brush Component which is set to NoCollision and may not function outside of editor.");
		}
	}

	FBoxSphereBounds BoxSphereBounds = Volume->GetBounds();
	// TODO: Compute the strict bounds, we must find a FBox inscribed into the oriented box.
	// Currently, we'll leave the strict bounds empty and fall back to checking against the local box
	Bounds = FBox::BuildAABB(BoxSphereBounds.Origin, BoxSphereBounds.BoxExtent);

	SetupVolumeBodyInstance();
}

void UPCGVolumeData::SetupVolumeBodyInstance()
{
	if (AVolume* CurrentVolume = Volume.Get())
	{
		// Keep a "sceneless" equivalent body so we can do queries against it without locking constraints
		if (UBrushComponent* BrushComponent = CurrentVolume->GetBrushComponent())
		{
			FBodyInstance* BodyInstance = BrushComponent->GetBodyInstance();
			UBodySetup* BodySetup = BrushComponent->GetBodySetup();

			// In some instances, non-collidable bodies will not be initialized, but it's not an issue for PCG so we can continue regardless.
			// Otherwise, require that the body is not dynamic.
			if (BodyInstance && BodySetup && (!FPhysicsInterface::IsValid(BodyInstance->GetPhysicsActor()) || !BodyInstance->IsDynamic()))
			{
				ReleaseInternalBodyInstance();

				VolumeBodyInstance = new FBodyInstance();
				VolumeBodyInstance->bAutoWeld = false;
				VolumeBodyInstance->bSimulatePhysics = false;
				VolumeBodyInstance->InitBody(BodySetup, BrushComponent->GetComponentTransform(), nullptr, nullptr);
			}
		}
	}
}

void UPCGVolumeData::Initialize(const FBox& InBounds)
{
	Bounds = InBounds;
	StrictBounds = InBounds;
}

void UPCGVolumeData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// Implementation note: no metadata in this data at this point.

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	if (VolumeBodyInstance)
	{
		// Weird implementation to fix ambiguous call between the FBodyInstance friend function (which isn't exposed) and the one in the FArchiveCrc32 code.
		FBodyInstance::StaticStruct()->SerializeItem(Ar, const_cast<FBodyInstance*>(VolumeBodyInstance), nullptr);
	}

	// Implementation note: we will not consider the volume pointer in this instance
	Ar.Serialize((void*)&VoxelSize, sizeof(FVector));

	// TODO: move this to helper function
	auto SerializeBounds = [&Ar](FBox* Box)
	{
		check(Box);
		Ar << Box->IsValid;

		if (Box->IsValid)
		{
			Ar << Box->Min;
			Ar << Box->Max;
		}
	};

	SerializeBounds(const_cast<FBox*>(&Bounds));
	SerializeBounds(const_cast<FBox*>(&StrictBounds));
}

FBox UPCGVolumeData::GetBounds() const
{
	return Bounds;
}

FBox UPCGVolumeData::GetStrictBounds() const
{
	return StrictBounds;
}

const UPCGPointData* UPCGVolumeData::CreatePointData(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGVolumeData::CreatePointData);

	return CastChecked<UPCGPointData>(CreateBasePointData(Context, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGVolumeData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGVolumeData::CreatePointArrayData);

	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGVolumeData::CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	PCGVolumeSampler::FVolumeSamplerParams SamplerParams;
	SamplerParams.VoxelSize = VoxelSize;
	SamplerParams.Bounds = GetBounds();

	const UPCGBasePointData* Data = PCGVolumeSampler::SampleVolume(Context, PointDataClass, SamplerParams, this);

	if (Data)
	{
		UE_LOGF(LogPCG, Verbose, "Volume extracted %d points", Data->GetNumPoints());
	}

	return Data;
}

bool UPCGVolumeData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(UPCGVolumeData::SamplePoint);
	// TODO: add metadata
	// TODO: consider bounds

	// This is a pure implementation

	const FVector InPosition = InTransform.GetLocation();
	if (PCGHelpers::IsInsideBounds(GetBounds(), InPosition))
	{
		float PointDensity = 0.0f;

		if (!Volume.IsValid() || PCGHelpers::IsInsideBounds(GetStrictBounds(), InPosition))
		{
			PointDensity = 1.0f;
		}
		else if (VolumeBodyInstance)
		{
			float OutDistanceSquared = -1.0f;
			if (FPhysicsInterface::GetSquaredDistanceToBody(VolumeBodyInstance, InPosition, OutDistanceSquared))
			{
				PointDensity = (OutDistanceSquared == 0.0f ? 1.0f : 0.0f);
			}
		}
		else
		{
			PointDensity = Volume->EncompassesPoint(InPosition) ? 1.0f : 0.0f;
		}

		OutPoint.Transform = InTransform;
		OutPoint.SetLocalBounds(InBounds);
		OutPoint.Density = PointDensity;

		return OutPoint.Density > 0;
	}
	else
	{
		return false;
	}
}

void UPCGVolumeData::CopyBaseVolumeData(UPCGVolumeData* NewVolumeData) const
{
	NewVolumeData->VoxelSize = VoxelSize;
	NewVolumeData->Volume = Volume;
	NewVolumeData->Bounds = Bounds;
	NewVolumeData->StrictBounds = StrictBounds;
}

UPCGSpatialData* UPCGVolumeData::CopyInternal(FPCGContext* Context) const
{
	UPCGVolumeData* NewVolumeData = FPCGContext::NewObject_AnyThread<UPCGVolumeData>(Context);

	CopyBaseVolumeData(NewVolumeData);

	// Initializing physics currently requires being on the GT.
	// In this case, we'll queue a GT task to do the initialization, which will be done by the end of the caller task here.
	// @todo_pcg: in future releases (5.7+) we might want to share the body instance instead, which would be simpler & more efficient
	// assuming we can't change the body instance in question.
	if (VolumeBodyInstance)
	{
		auto FinishVolumeSetup = [WeakVolumeData = TWeakObjectPtr<UPCGVolumeData>(NewVolumeData)](FPCGContext* InContext)
		{
			if (UPCGVolumeData* VolumeData = WeakVolumeData.Get())
			{
				VolumeData->SetupVolumeBodyInstance();
			}

			return true;
		};

		if (IsInGameThread())
		{
			FinishVolumeSetup(Context);
		}
		else
		{
			FPCGScheduleGenericParams Params(FinishVolumeSetup, Context->ExecutionSource.Get());

			// Physics setup must be run on the GT.
			Params.bCanExecuteOnlyOnMainThread = true;

			FPCGTaskId MainThreadTaskId = Context->ScheduleGeneric(Params);

			if (MainThreadTaskId != InvalidPCGTaskId)
			{
				Context->DynamicDependencies.Add(MainThreadTaskId);
			}
		}
	}

	return NewVolumeData;
}
