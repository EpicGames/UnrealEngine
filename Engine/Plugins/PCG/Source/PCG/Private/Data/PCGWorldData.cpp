// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGWorldData.h"

#include "PCGComponent.h"
#include "PCGSubsystem.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSurfaceData.h"
#include "Data/PCGVolumeData.h"
#include "Elements/PCGSurfaceSampler.h"
#include "Elements/PCGVolumeSampler.h"
#include "Grid/PCGLandscapeCache.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGWorldQueryHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "LandscapeProxy.h"
#include "Components/BrushComponent.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGWorldData)

#define LOCTEXT_NAMESPACE "PCGWorldData"

namespace PCGWorldCommonQueryParams
{
	template<class T, class Container, typename Func>
	bool ExtractObjectFiltersIfNeeded(const FPCGWorldCommonQueryParams& InParams, const UPCGData* InData, Container& OutContainer, FPCGContext* InContext, Func TransformLambda)
	{
		if (InParams.ActorFilterFromInput != EPCGWorldQueryFilter::None)
		{
			TArray<FSoftObjectPath> FilterActors;

			if (!PCGAttributeAccessorHelpers::ExtractAllValues(InData, InParams.ActorFilterInputSource, FilterActors, InContext))
			{
				return false;
			}

			OutContainer.Empty(FilterActors.Num());
			Algo::Transform(FilterActors, OutContainer, TransformLambda);
		}

		return true;
	}
}

void FPCGWorldCommonQueryParams::AddFilterPinIfNeeded(TArray<FPCGPinProperties>& PinProperties) const
{
	if (ActorFilterFromInput != EPCGWorldQueryFilter::None)
	{
		FPCGPinProperties& FilterActors = PinProperties.Emplace_GetRef(PCGWorldRayHitConstants::FilterActorPinLabel, EPCGDataType::PointOrParam);
#if WITH_EDITOR
		FilterActors.Tooltip = LOCTEXT("ActorFilterFromInputTooltip", "All hit actors will be filtered against this list. Can be 1 list or N lists, N being the number of data in Origins pin.");
#endif // WITH_EDITOR
	}
}

// deprecated
bool FPCGWorldCommonQueryParams::ExtractActorFiltersIfNeeded(const UPCGData* InData, TArray<TSoftObjectPtr<AActor>>& OutArray, FPCGContext* InContext) const
{
	return PCGWorldCommonQueryParams::ExtractObjectFiltersIfNeeded<TSoftObjectPtr<AActor>>(*this, InData, OutArray, InContext,
		[](const FSoftObjectPath& ActorPath) -> TSoftObjectPtr<AActor> { return TSoftObjectPtr<AActor>(ActorPath); });
}

// deprecated
bool FPCGWorldCommonQueryParams::ExtractLoadedActorFiltersIfNeeded(const UPCGData* InData, TSet<TObjectKey<AActor>>& OutSet, FPCGContext* InContext) const
{
	return PCGWorldCommonQueryParams::ExtractObjectFiltersIfNeeded<TObjectKey<AActor>>(*this, InData, OutSet, InContext,
	[](const FSoftObjectPath& ActorPath) -> TObjectKey<AActor> { return TObjectKey<AActor>(Cast<AActor>(ActorPath.ResolveObject())); });
}

bool FPCGWorldCommonQueryParams::ExtractObjectFiltersIfNeeded(const UPCGData* InData, TArray<FSoftObjectPath>& OutArray, FPCGContext* InContext) const
{
	if (ActorFilterFromInput != EPCGWorldQueryFilter::None)
	{
		if (!PCGAttributeAccessorHelpers::ExtractAllValues(InData, ActorFilterInputSource, OutArray, InContext))
		{
			return false;
		}
	}

	return true;
}

bool FPCGWorldCommonQueryParams::ExtractLoadedObjectFiltersIfNeeded(const UPCGData* InData, TSet<FObjectKey>& OutSet, FPCGContext* InContext) const
{
	return PCGWorldCommonQueryParams::ExtractObjectFiltersIfNeeded<FObjectKey>(*this, InData, OutSet, InContext,
		[](const FSoftObjectPath& ObjectPath) -> FObjectKey { return FObjectKey(ObjectPath.ResolveObject()); });
}

void FPCGWorldCommonQueryParams::Initialize()
{
	if (ActorTagFilter == EPCGWorldQueryFilter::None)
	{
		ParsedActorTagsList.Reset();
	}
	else
	{
		TArray<FString> ParsedList = PCGHelpers::GetStringArrayFromCommaSeparatedList(ActorTagsList);
		ParsedActorTagsList.Reset();
		for (const FString& Tag : ParsedList)
		{
			ParsedActorTagsList.Add(FName(Tag));
		}
	}
}

void FPCGWorldVolumetricQueryParams::Initialize()
{
	FPCGWorldCommonQueryParams::Initialize();
}

void FPCGWorldRaycastQueryParams::Initialize()
{
	FPCGWorldCommonQueryParams::Initialize();
}

void FPCGWorldRayHitQueryParams::Initialize()
{
	FPCGWorldCommonQueryParams::Initialize();
}

#if WITH_EDITOR
void FPCGWorldCommonQueryParams::CommonPostLoad()
{
	if (bIgnoreLandscapeHits_DEPRECATED != false)
	{
		SelectLandscapeHits = EPCGWorldQuerySelectLandscapeHits::Exclude;
		bIgnoreLandscapeHits_DEPRECATED = false;
	}
}
#endif // WITH_EDITOR

void FPCGWorldRaycastQueryParams::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.IsLoading() && Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
	{
		CommonPostLoad();
	}
#endif
}

void FPCGWorldVolumetricQueryParams::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.IsLoading() && Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
	{
		CommonPostLoad();
	}
#endif
}

void FPCGWorldRayHitQueryParams::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.IsLoading() && Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
	{
		CommonPostLoad();
	}
#endif
}

void UPCGWorldVolumetricData::Initialize(IPCGGraphExecutionSource* InExecutionSource, const FBox& InBounds)
{
	Super::Initialize(InBounds);
	SetOriginatingSource(InExecutionSource);
	check(InExecutionSource && InExecutionSource->GetExecutionState().GetWorld());
}

IPCGGraphExecutionSource* UPCGWorldVolumetricData::GetOriginatingSource() const
{
	// Need to guard the resolve object against GC
	FGCScopeGuard Guard{};
	return Cast<IPCGGraphExecutionSource>(OriginatingObject.ResolveObject()); 
}

UWorld* UPCGWorldVolumetricData::GetWorld() const
{
	if (IPCGGraphExecutionSource* ExecutionSource = GetOriginatingSource())
	{
		return ExecutionSource->GetExecutionState().GetWorld();
	}
	else
	{
		return nullptr;
	}
}

void UPCGWorldVolumetricData::SetOriginatingSource(IPCGGraphExecutionSource* InExecutionSource) 
{ 
	OriginatingObject = Cast<UObject>(InExecutionSource); 
}

const TSet<FObjectKey>& UPCGWorldVolumetricData::GetCachedFilteredObjects() const
{
	if (!FilteredObjects.IsEmpty() && CachedFilteredObjectsDirty)
	{
		PCG::TScopeLock Lock(CachedFilteredObjectsLock);
		if (CachedFilteredObjectsDirty)
		{
			CachedFilteredObjects.Empty(FilteredObjects.Num());
			Algo::Transform(FilteredObjects, CachedFilteredObjects, [](const FSoftObjectPath& SoftObjectPath) { return FObjectKey(SoftObjectPath.ResolveObject()); });
			CachedFilteredObjectsDirty = false;
		}
	}

	return CachedFilteredObjects;
}

void UPCGWorldVolumetricData::PostLoad()
{
	Super::PostLoad();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
#if WITH_EDITOR
	// Move data into new member
	if (!ActorFilter.FilterActors_DEPRECATED.IsEmpty())
	{
		Algo::Transform(ActorFilter.FilterActors_DEPRECATED, GetMutableFilteredObjects(), [](const TSoftObjectPtr<AActor> InActor) { return InActor.ToSoftObjectPath(); });
	}

	if(OriginatingComponent.IsValid())
	{
		SetOriginatingSource(OriginatingComponent.Get());
		OriginatingComponent = nullptr;
	}
#endif // WITH_EDITOR

PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Cache is dirty if filtered objects is not empty on load
	CachedFilteredObjectsDirty = GetConstFilteredObjects().Num() > 0;
}

bool UPCGWorldVolumetricData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// This is a pure implementation.
	UWorld* ResolvedWorld = GetWorld();
	if (!ensure(ResolvedWorld))
	{
		return false;
	}
	
	FPCGMetadataAttribute<FSoftObjectPath>* ActorOverlappedAttribute = ((OutMetadata && QueryParams.bGetReferenceToActorHit) ? OutMetadata->FindOrCreateAttribute<FSoftObjectPath>(PCGPointDataConstants::ActorReferenceAttribute) : nullptr);

	FCollisionObjectQueryParams ObjectQueryParams(QueryParams.CollisionChannel);
	FCollisionShape CollisionShape = FCollisionShape::MakeBox(InBounds.GetExtent() * InTransform.GetScale3D());
	FCollisionQueryParams Params; // TODO: apply properties from the settings when/if they exist
	Params.bTraceComplex = QueryParams.bTraceComplex;

	TArray<FOverlapResult> Overlaps;
	/*bool bOverlaps =*/ ResolvedWorld->OverlapMultiByObjectType(Overlaps, InTransform.TransformPosition(InBounds.GetCenter()), InTransform.GetRotation(), ObjectQueryParams, CollisionShape, Params);

	TOptional<FOverlapResult> Overlap = PCGWorldQueryHelpers::FilterOverlapResults(QueryParams, GetOriginatingSource(), Overlaps, GetCachedFilteredObjects());

	// If searched for overlap and found one, or didn't search and didn't find one, set the point and return true. Otherwise, return false.
	if (Overlap.IsSet() == QueryParams.bSearchForOverlap)
	{
		OutPoint = FPCGPoint(InTransform, 1.0f, UPCGBlueprintHelpers::ComputeSeedFromPosition(InTransform.GetLocation()));
		OutPoint.SetLocalBounds(InBounds);

		if (Overlap.IsSet() && ActorOverlappedAttribute && Overlap.GetValue().GetActor())
		{
			OutMetadata->InitializeOnSet(OutPoint.MetadataEntry);
			ActorOverlappedAttribute->SetValue(OutPoint.MetadataEntry, FSoftObjectPath(Overlap.GetValue().GetActor()));
		}

		return true;
	}
	else
	{
		return false;
	}
}

const UPCGPointData* UPCGWorldVolumetricData::CreatePointData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGWorldVolumetricData::CreatePointData);
	return CastChecked<UPCGPointData>(CreateBasePointData(Context, InBounds, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGWorldVolumetricData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGWorldVolumetricData::CreatePointArrayData);
	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, InBounds, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGWorldVolumetricData::CreateBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const
{		
	FBox EffectiveBounds = Bounds;
	if (InBounds.IsValid)
	{
		if (Bounds.IsValid)
		{
			EffectiveBounds = Bounds.Overlap(InBounds);
		}
		else
		{
			EffectiveBounds = InBounds;
		}
	}
	
	// Early out
	if (!EffectiveBounds.IsValid)
	{
		if (!Bounds.IsValid && !InBounds.IsValid)
		{
			UE_LOGF(LogPCG, Error, "PCG World Volumetric Data cannot generate without sampling bounds. Consider using a Volume Sampler with the Unbounded option disabled.");
		}

		UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
		
		FPCGInitializeFromDataParams InitializeFromDataParams(this);
		InitializeFromDataParams.bInheritSpatialData = false;
		Data->InitializeFromDataWithParams(InitializeFromDataParams);
		
		return Data;
	}

	PCGVolumeSampler::FVolumeSamplerParams SamplerParams;
	SamplerParams.VoxelSize = VoxelSize;
	SamplerParams.Bounds = EffectiveBounds;

	UPCGBasePointData* Data = PCGVolumeSampler::SampleVolume(Context, PointDataClass, SamplerParams, this);
	UE_LOGF(LogPCG, Verbose, "Volumetric world extracted %d points", Data->GetNumPoints());

	return Data;
}

UPCGSpatialData* UPCGWorldVolumetricData::CopyInternal(FPCGContext* Context) const
{
	UPCGWorldVolumetricData* NewVolumetricData = FPCGContext::NewObject_AnyThread<UPCGWorldVolumetricData>(Context);

	CopyBaseVolumeData(NewVolumetricData);

	NewVolumetricData->SetOriginatingSource(GetOriginatingSource());
	NewVolumetricData->QueryParams = QueryParams;
	NewVolumetricData->QueryParams.Initialize();
	NewVolumetricData->GetMutableFilteredObjects() = GetConstFilteredObjects();

	return NewVolumetricData;
}

/** World Ray Hit data implementation */
void UPCGWorldRayHitData::Initialize(IPCGGraphExecutionSource* InExecutionSource, const FTransform& InTransform, const FBox& InBounds, const FBox& InLocalBounds)
{
	// To save object path
	SetOriginatingSource(InExecutionSource);
	Transform = InTransform;
	Bounds = InBounds;
	LocalBounds = InLocalBounds;
}

IPCGGraphExecutionSource* UPCGWorldRayHitData::GetOriginatingSource() const
{
	// Need to guard the resolve object against GC
	FGCScopeGuard Guard{};
	return Cast<IPCGGraphExecutionSource>(OriginatingObject.ResolveObject());
}

UWorld* UPCGWorldRayHitData::GetWorld() const
{
	if (IPCGGraphExecutionSource* ExecutionSource = GetOriginatingSource())
	{
		return ExecutionSource->GetExecutionState().GetWorld();
	}
	else
	{
		return nullptr;
	}
}

void UPCGWorldRayHitData::SetOriginatingSource(IPCGGraphExecutionSource* InExecutionSource)
{
	OriginatingObject = Cast<UObject>(InExecutionSource);
}

const TSet<FObjectKey>& UPCGWorldRayHitData::GetCachedFilteredObjects() const
{
	if (!FilteredObjects.IsEmpty() && CachedFilteredObjectsDirty)
	{
		PCG::TScopeLock Lock(CachedFilteredObjectsLock);
		if (CachedFilteredObjectsDirty)
		{
			CachedFilteredObjects.Empty(FilteredObjects.Num());
			Algo::Transform(FilteredObjects, CachedFilteredObjects, [](const FSoftObjectPath& SoftObjectPath) { return FObjectKey(SoftObjectPath.ResolveObject()); });
			CachedFilteredObjectsDirty = false;
		}
	}

	return CachedFilteredObjects;
}

void UPCGWorldRayHitData::PostLoad()
{
	Super::PostLoad();

PRAGMA_DISABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
	// Move data into new member
	if (!ActorFilter.FilterActors_DEPRECATED.IsEmpty())
	{
		Algo::Transform(ActorFilter.FilterActors_DEPRECATED, GetMutableFilteredObjects(), [](const TSoftObjectPtr<AActor> InActor) { return InActor.ToSoftObjectPath(); });
	}

	if(OriginatingComponent.IsValid())
	{
		SetOriginatingSource(OriginatingComponent.Get());
		OriginatingComponent = nullptr;
	}
#endif // WITH_EDITOR	

PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Cache is dirty if filtered objects is not empty on load
	CachedFilteredObjectsDirty = GetConstFilteredObjects().Num() > 0;
}

void UPCGWorldRayHitData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(Ar);
}

bool UPCGWorldRayHitData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// TODO: This seems to be a projection - along a direction. I suspect that UPCGWorldVolumetricData is the SamplePoint(), and this is the ProjectPoint() (in a direction)?
	UWorld* ResolvedWorld = GetWorld();
	if (!ensure(ResolvedWorld))
	{
		return false;
	}

	// Todo: consider prebuilding this
	const FCollisionObjectQueryParams ObjectQueryParams(QueryParams.CollisionChannel);
	const FCollisionQueryParams CollisionQueryParams = QueryParams.ToCollisionQuery(); // TODO: apply properties from the settings when/if they exist

	FVector RayOrigin = InTransform.GetLocation() - ((InTransform.GetLocation() - QueryParams.RayOrigin) | QueryParams.RayDirection) * QueryParams.RayDirection;
	FVector RayEnd = RayOrigin + QueryParams.RayDirection * QueryParams.RayLength;

	TArray<FHitResult> Hits;
	if (ResolvedWorld->SweepMultiByObjectType(
				Hits,
				RayOrigin,
				RayEnd,
				FQuat(CollisionShape.ShapeRotation),
				ObjectQueryParams,
				CollisionShape.ToCollisionShape(),
				CollisionQueryParams))
	{
		TOptional<FHitResult> HitResult = PCGWorldQueryHelpers::FilterRayHitResults(QueryParams, GetOriginatingSource(), Hits, GetCachedFilteredObjects());

		if (HitResult.IsSet())
		{
			const FHitResult& Hit = HitResult.GetValue();
			OutPoint = FPCGPoint(PCGWorldQueryHelpers::GetOrthonormalImpactTransform(Hit), 1.0f, UPCGBlueprintHelpers::ComputeSeedFromPosition(Hit.Location));
			PCGWorldQueryHelpers::ApplyRayHitMetadata(Hit, QueryParams, QueryParams.RayDirection, OutPoint.Transform, OutPoint.MetadataEntry, OutMetadata, ResolvedWorld);
			return true;
		}
	}

	return false;
}

const UPCGPointData* UPCGWorldRayHitData::CreatePointData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointData>(CreateBasePointData(Context, InBounds, UPCGPointData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGPointArrayData* UPCGWorldRayHitData::CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointArrayData>(CreateBasePointData(Context, InBounds, UPCGPointArrayData::StaticClass()), ECastCheckedType::NullAllowed);
}

const UPCGBasePointData* UPCGWorldRayHitData::CreateBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	FBox EffectiveBounds = Bounds;
	if (InBounds.IsValid)
	{
		if (Bounds.IsValid)
		{
			EffectiveBounds = Bounds.Overlap(InBounds);
		}
		else
		{
			EffectiveBounds = InBounds;
		}
	}

	// Early out
	if (!EffectiveBounds.IsValid)
	{
		if (!Bounds.IsValid && !InBounds.IsValid)
		{
			UE_LOGF(LogPCG, Error, "PCG World Ray Hit Data cannot generate without sampling bounds. Consider using a Surface Sampler with the Unbounded option disabled.");
		}

		UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
		Data->InitializeFromData(this);
		return Data;
	}

	// The default params will be fine in this case
	const PCGSurfaceSampler::FSurfaceSamplerParams Params;
	return PCGSurfaceSampler::SampleSurface(Context, Params, /*InSurface=*/this, /*InBoundingShape=*/nullptr, EffectiveBounds, PointDataClass);
}

void UPCGWorldRayHitData::InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const
{
	check(MetadataToInitialize);

	// Initialize the new metadata normally
	Super::InitializeTargetMetadata(InParams, MetadataToInitialize);
	
	// Only add those special attributes if the metadata to initialize support elements domain
	FPCGMetadataDomain* MetadataDomain = MetadataToInitialize->GetMetadataDomain(PCGMetadataDomainID::Elements);
	if (!MetadataDomain)
	{
		return;
	}
	
	// Add all landscape layers
	for (const FName LayerName : CachedLandscapeLayerNames)
	{
		MetadataDomain->FindOrCreateAttribute<float>(LayerName, 0.0f, /*bAllowInterpolation=*/true);
	}

	// Then all the other attributes
	PCGWorldQueryHelpers::CreateRayHitAttributes(QueryParams, MetadataDomain);
}

UPCGSpatialData* UPCGWorldRayHitData::CopyInternal(FPCGContext* Context) const
{
	UPCGWorldRayHitData* NewData = FPCGContext::NewObject_AnyThread<UPCGWorldRayHitData>(Context);

	CopyBaseSurfaceData(NewData);

	NewData->SetOriginatingSource(GetOriginatingSource());
	NewData->Bounds = Bounds;
	NewData->QueryParams = QueryParams;
	NewData->QueryParams.Initialize();
	NewData->GetMutableFilteredObjects() = GetConstFilteredObjects();
	NewData->CachedLandscapeLayerNames = CachedLandscapeLayerNames;

	return NewData;
}

#undef LOCTEXT_NAMESPACE