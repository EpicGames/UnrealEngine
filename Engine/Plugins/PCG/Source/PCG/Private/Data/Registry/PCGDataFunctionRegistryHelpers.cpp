// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Registry/PCGDataFunctionRegistryHelpers.h"

#include "PCGModule.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGTagHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttribute.h"

#include "Components/BillboardComponent.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/Actor.h"
#include "Materials/MaterialInstance.h"

void FPCGActorMeshParsingContext::CreateTagAttributes(AActor* InActor)
{
	check(InActor);

	if (!TagToAttributeMap || !SanitizedAttributeNames)
	{
		return;
	}

	UPCGMetadata* Metadata = PointMetadata ? PointMetadata : (PointData ? PointData->MutableMetadata() : nullptr);
	if (!Metadata)
	{
		return;
	}

	// Derive reserved attribute names from the actual names of the pre-created attribute pointers,
	// so we don't create tag attributes that would collide with them.
	TSet<FName> ReservedAttributeNames;	
	{
		TArray<FName> DefaultDomainAttributeNames;
		TArray<EPCGMetadataTypes> DefaultDomainAttributeTypes;
		Metadata->GetAttributes(DefaultDomainAttributeNames, DefaultDomainAttributeTypes);

		ReservedAttributeNames.Append(DefaultDomainAttributeNames);
	}

	const TArray<FName>& Tags = ActorTags ? *ActorTags : InActor->Tags;
	for (const FName& Tag : Tags)
	{
		PCG::Private::FParseTagResult TagData(Tag);

		if (!TagData.IsValid())
		{
			continue;
		}

		const FName OriginalAttributeName(TagData.GetOriginalAttribute());
		const FName SanitizedAttributeName(TagData.Attribute);

		if (ReservedAttributeNames.Contains(SanitizedAttributeName) || TagToAttributeMap->Contains(OriginalAttributeName) || SanitizedAttributeNames->Contains(SanitizedAttributeName))
		{
			continue;
		}

		if (!PCG::Private::CreateAttributeFromTag(TagData, Metadata))
		{
			continue;
		}

		if (TagData.HasBeenSanitized())
		{
			UE_LOGF(LogPCG, Warning, "Sanitized tag string on actor '%ls' to remove invalid characters: '%ls' -> '%ls'", *InActor->GetName(), *TagData.GetOriginalAttribute(), *TagData.Attribute);
		}

		TagToAttributeMap->Add(OriginalAttributeName, Metadata->GetMutableAttribute(SanitizedAttributeName));
		SanitizedAttributeNames->Add(SanitizedAttributeName);
	}
}

int FPCGActorMeshParsingContext::ParseActorComponents(AActor* InActor)
{
	check(InActor && PointData);

	UPCGMetadata* Metadata = PointMetadata ? PointMetadata : PointData->MutableMetadata();
	const TArray<FName>& Tags = ActorTags ? *ActorTags : InActor->Tags;

	CreateTagAttributes(InActor);

	// Get relevant visual actor components
	TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents;
	InActor->GetComponents(PrimitiveComponents);

	// Count number of points we'll add so we do a single reallocation
	const int NumPointsBefore = PointData->GetNumPoints();
	int NumPointsToAdd = 0;

	{
		// Implementation note - code keeps the same layout as parsing code below for readability
		for (UPrimitiveComponent* PrimComp : PrimitiveComponents)
		{
			if (bIgnorePCGCreatedComponents && PrimComp->ComponentTags.Contains(PCGHelpers::DefaultPCGTag))
			{
				continue;
			}

			// Skip billboard components - they have no meaningful geometry
			if (Cast<UBillboardComponent>(PrimComp))
			{
				continue;
			}

			if (UStaticMeshComponent* SMC = Cast<UStaticMeshComponent>(PrimComp))
			{
				UStaticMesh* StaticMesh = SMC->GetStaticMesh();
				if (!StaticMesh)
				{
					continue;
				}

				if (UInstancedStaticMeshComponent* ISMC = Cast<UInstancedStaticMeshComponent>(SMC))
				{
					NumPointsToAdd += ISMC->GetNumInstances();
				}
				else
				{
					++NumPointsToAdd;
				}
			}
			else if (USkinnedMeshComponent* SKMC = Cast<USkinnedMeshComponent>(PrimComp))
			{
				USkinnedAsset* SkinnedAsset = SKMC->GetSkinnedAsset();
				if (!SkinnedAsset)
				{
					continue;
				}

				if (UInstancedSkinnedMeshComponent* ISKMC = Cast<UInstancedSkinnedMeshComponent>(SKMC))
				{
					NumPointsToAdd += ISKMC->GetInstanceCount();
				}
				else
				{
					++NumPointsToAdd;
				}
			}
			else if (!bOnlyMeshComponents)
			{
				++NumPointsToAdd;
			}
		}
	}

	// If we're not going to add any points we can early out.
	if (NumPointsToAdd == 0)
	{
		return NumPointsToAdd;
	}

	// Allocate new points and gather ranges
	PointData->SetNumPoints(NumPointsBefore + NumPointsToAdd);

	TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
	TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange();
	TPCGValueRange<FVector> BoundsMinRange = PointData->GetBoundsMinValueRange();
	TPCGValueRange<FVector> BoundsMaxRange = PointData->GetBoundsMaxValueRange();
	TPCGValueRange<int64> MetadataRange = PointData->GetMetadataEntryValueRange();

	// Write actor reference value
	const FSoftObjectPath ActorPath(InActor);
	PCGMetadataValueKey ActorReferenceValueKey = ActorReferenceAttribute ? ActorReferenceAttribute->AddValue(ActorPath) : PCGNotFoundValueKey;

	// Lambda to write all attributes on a newly-allocated point slot.
	auto WritePointAttributes = [this, Metadata, &Tags, &TransformRange, &SeedRange, &BoundsMinRange, &BoundsMaxRange, &MetadataRange, ActorReferenceValueKey](int32 PointIndex, const FTransform& Transform, const PCGMetadataValueKey ComponentReferenceValueKey, const PCGMetadataValueKey MeshPathValueKey, const PCGMetadataValueKey SkeletalMeshPathValueKey, const FBox& Bounds, const PCGMetadataValueKey MaterialPathValueKey)
	{
		TransformRange[PointIndex] = Transform;
		SeedRange[PointIndex] = PCGHelpers::ComputeSeedFromPosition(Transform.GetLocation());
		BoundsMinRange[PointIndex] = Bounds.Min;
		BoundsMaxRange[PointIndex] = Bounds.Max;

		int64& MetadataEntry = MetadataRange[PointIndex];
		Metadata->InitializeOnSet(MetadataEntry);

		if (MeshAttribute && MeshPathValueKey != PCGNotFoundValueKey)
		{
			MeshAttribute->SetValueFromValueKey(MetadataEntry, MeshPathValueKey);
		}

		if (SkeletalMeshAttribute && SkeletalMeshPathValueKey != PCGNotFoundValueKey)
		{
			SkeletalMeshAttribute->SetValueFromValueKey(MetadataEntry, SkeletalMeshPathValueKey);
		}

		if (MaterialAttribute && MaterialPathValueKey != PCGNotFoundValueKey)
		{
			MaterialAttribute->SetValueFromValueKey(MetadataEntry, MaterialPathValueKey);
		}

		if (ActorIndexAttribute)
		{
			ActorIndexAttribute->SetValue(MetadataEntry, ActorIndex);
		}

		if (ParentIndexAttribute)
		{
			ParentIndexAttribute->SetValue(MetadataEntry, ParentIndex);
		}

		if (RelativeTransformAttribute)
		{
			RelativeTransformAttribute->SetValue(MetadataEntry, RelativeTransform);
		}

		if (HierarchyDepthAttribute)
		{
			HierarchyDepthAttribute->SetValue(MetadataEntry, HierarchyDepth);
		}

		if (ActorReferenceAttribute)
		{
			ActorReferenceAttribute->SetValueFromValueKey(MetadataEntry, ActorReferenceValueKey);
		}

		if (ComponentReferenceAttribute)
		{
			ComponentReferenceAttribute->SetValueFromValueKey(MetadataEntry, ComponentReferenceValueKey);
		}

		// Set tag attribute values from the pre-built map
		if (TagToAttributeMap)
		{
			for (const FName& Tag : Tags)
			{
				PCG::Private::FParseTagResult TagData(Tag);
				if (TagData.IsValid() && TagToAttributeMap->Contains(FName(TagData.GetOriginalAttribute())))
				{
					PCG::Private::SetAttributeFromTag(TagData, Metadata, MetadataEntry);
				}
			}
		}
	};

	int PointWriteIndex = NumPointsBefore;

	for (UPrimitiveComponent* PrimComp : PrimitiveComponents)
	{
		if (bIgnorePCGCreatedComponents && PrimComp->ComponentTags.Contains(PCGHelpers::DefaultPCGTag))
		{
			continue;
		}

		// Skip billboard components - they have no meaningful geometry
		if (Cast<UBillboardComponent>(PrimComp))
		{
			continue;
		}

		const FSoftObjectPath ComponentPath(PrimComp);

		if (UStaticMeshComponent* SMC = Cast<UStaticMeshComponent>(PrimComp))
		{
			UStaticMesh* StaticMesh = SMC->GetStaticMesh();
			if (!StaticMesh)
			{
				continue;
			}

			// Prepare value keys.
			const PCGMetadataValueKey ComponentReferenceValueKey = ComponentReferenceAttribute ? ComponentReferenceAttribute->AddValue(ComponentPath) : PCGNotFoundValueKey;
			const PCGMetadataValueKey MeshPathValueKey = MeshAttribute ? MeshAttribute->AddValue(FSoftObjectPath(StaticMesh)) : PCGNotFoundValueKey;
			const PCGMetadataValueKey SkeletalMeshPathValueKey = PCGNotFoundValueKey;
			PCGMetadataValueKey MaterialPathValueKey = PCGNotFoundValueKey;

			if(MaterialAttribute)
			{
				TArray<UMaterialInterface*> Materials = SMC->GetMaterials();
				if (!Materials.IsEmpty())
				{
					// Walk up the material instance chain to find the nearest public asset
					UMaterialInterface* Material = Materials[0];
					UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
					while (MaterialInstance && !MaterialInstance->IsAsset())
					{
						Material = MaterialInstance->Parent;
						MaterialInstance = Cast<UMaterialInstance>(Material);
					}

					MaterialPathValueKey = MaterialAttribute->AddValue(FSoftObjectPath(Material));
				}
			}

			const FBox MeshBounds = StaticMesh->GetBoundingBox();

			if (UInstancedStaticMeshComponent* ISMC = Cast<UInstancedStaticMeshComponent>(SMC))
			{
				const int32 InstanceCount = ISMC->GetNumInstances();
				for (int32 I = 0; I < InstanceCount; ++I)
				{
					FTransform InstanceTransform;
					ISMC->GetInstanceTransform(I, InstanceTransform, /*bWorldSpace=*/true);
					WritePointAttributes(PointWriteIndex + I, InstanceTransform, ComponentReferenceValueKey, MeshPathValueKey, SkeletalMeshPathValueKey, MeshBounds, MaterialPathValueKey);
				}

				PointWriteIndex += InstanceCount;
			}
			else
			{
				WritePointAttributes(PointWriteIndex, SMC->GetComponentTransform(), ComponentReferenceValueKey, MeshPathValueKey, SkeletalMeshPathValueKey, MeshBounds, MaterialPathValueKey);
				++PointWriteIndex;
			}
		}
		else if (USkinnedMeshComponent* SKMC = Cast<USkinnedMeshComponent>(PrimComp))
		{
			USkinnedAsset* SkinnedAsset = SKMC->GetSkinnedAsset();
			if (!SkinnedAsset)
			{
				continue;
			}

			// Prepare value keys.
			const PCGMetadataValueKey ComponentReferenceValueKey = ComponentReferenceAttribute ? ComponentReferenceAttribute->AddValue(ComponentPath) : PCGNotFoundValueKey;
			const PCGMetadataValueKey MeshPathValueKey = PCGNotFoundValueKey;
			const PCGMetadataValueKey SkeletalMeshPathValueKey = SkeletalMeshAttribute ? SkeletalMeshAttribute->AddValue(FSoftObjectPath(SkinnedAsset)) : PCGNotFoundValueKey;
			PCGMetadataValueKey MaterialPathValueKey = PCGNotFoundValueKey;
			
			if(MaterialAttribute)
			{
				TArray<UMaterialInterface*> Materials = SKMC->GetMaterials();
				if (!Materials.IsEmpty())
				{
					// Walk up the material instance chain to find the nearest public asset
					UMaterialInterface* Material = Materials[0];
					UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
					while (MaterialInstance && !MaterialInstance->IsAsset())
					{
						Material = MaterialInstance->Parent;
						MaterialInstance = Cast<UMaterialInstance>(Material);
					}

					MaterialPathValueKey = MaterialAttribute->AddValue(FSoftObjectPath(Material));
				}
			}
			
			// @todo_pcg: SkinnedAsset doesn't have GetBoundsExtended (as StaticMesh) but SkeletalMesh does have Positive/Negative. Should we use one of those?
			const FBox MeshBounds = SkinnedAsset->GetBounds().GetBox();

			if (UInstancedSkinnedMeshComponent* ISKMC = Cast<UInstancedSkinnedMeshComponent>(SKMC))
			{
				const int32 InstanceCount = ISKMC->GetInstanceCount();
				for (int32 I = 0; I < InstanceCount; ++I)
				{
					FTransform InstanceTransform;
					ISKMC->GetInstanceTransform(ISKMC->GetInstanceId(I), InstanceTransform, /*bWorldSpace=*/true);
					WritePointAttributes(PointWriteIndex + I, InstanceTransform, ComponentReferenceValueKey, MeshPathValueKey, SkeletalMeshPathValueKey, MeshBounds, MaterialPathValueKey);
				}

				PointWriteIndex += InstanceCount;
			}
			else
			{
				WritePointAttributes(PointWriteIndex, SKMC->GetComponentTransform(), ComponentReferenceValueKey, MeshPathValueKey, SkeletalMeshPathValueKey, MeshBounds, MaterialPathValueKey);
				++PointWriteIndex;
			}
		}
		else if (!bOnlyMeshComponents)
		{
			// Non-SM primitive: single point at the component transform with component-local bounds
			const FBoxSphereBounds LocalBounds = PrimComp->CalcLocalBounds();
			PCGMetadataValueKey ComponentReferenceValueKey = ComponentReferenceAttribute ? ComponentReferenceAttribute->AddValue(ComponentPath) : PCGNotFoundValueKey;
			WritePointAttributes(PointWriteIndex, PrimComp->GetComponentTransform(), ComponentReferenceValueKey, PCGNotFoundValueKey, PCGNotFoundValueKey, LocalBounds.GetBox(), PCGNotFoundValueKey);
			++PointWriteIndex;
		}
	}

	check(PointWriteIndex == NumPointsBefore + NumPointsToAdd);

	return NumPointsToAdd;
}
