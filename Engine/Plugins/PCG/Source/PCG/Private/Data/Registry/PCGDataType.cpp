// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Registry/PCGDataType.h"

#include "PCGData.h"
#include "PCGParamData.h"
#include "PCGSettings.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGDynamicMeshData.h"
#include "Data/PCGLandscapeData.h"
#include "Data/PCGLandscapeSplineData.h"
#include "Data/PCGPolygon2DData.h"
#include "Data/PCGRenderTargetData.h"
#include "Data/PCGPrimitiveData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGStaticMeshResourceData.h"
#include "Data/PCGTextureData.h"
#include "Data/PCGVirtualTextureData.h"
#include "Data/PCGVolumeData.h"
#include "Helpers/PCGSubgraphHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataType)

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfo, UPCGData)
PCG_DEFINE_TYPE_INFO_WITHOUT_CLASS(FPCGDataTypeInfoOther)

FPCGDataTypeBaseId::FPCGDataTypeBaseId(const UScriptStruct* InStruct)
	: Struct(InStruct && InStruct->IsChildOf(FPCGDataTypeInfo::StaticStruct()) ? InStruct : nullptr)
{
}

bool FPCGDataTypeBaseId::IsValid() const
{
	return Struct != nullptr;
}

bool FPCGDataTypeBaseId::IsChildOf(const FPCGDataTypeBaseId& Other) const
{
	return Struct && Struct->IsChildOf(Other.Struct);
}

FString FPCGDataTypeBaseId::ToString() const
{
	if (!IsValid())
	{
		return TEXT("None");
	}
	else if (Struct == FPCGDataTypeInfo::StaticStruct())
	{
		return TEXT("Any");
	}
#if WITH_EDITOR
	else if (Struct->HasMetaData(PCGObjectMetadata::DataTypeDisplayName))
	{
		return Struct->GetMetaData(PCGObjectMetadata::DataTypeDisplayName);
	}
#endif // WITH_EDITOR
	else
	{
		// Strip `PCGDataTypeInfo` of the name
		static const FString Prefix = TEXT("DataTypeInfo");
		static const int32 PrefixSize = Prefix.Len();

		FString Result = Struct->GetName();
		const int32 PrefixIndex = Result.Find(Prefix);
		if (PrefixIndex != INDEX_NONE)
		{
			return FString(FStringView(Result).RightChop(PrefixIndex + PrefixSize));
		}
		else
		{
			return Result;
		}
	}
}


bool FPCGDataTypeInfo::SupportsConversionFrom(const FPCGDataTypeIdentifier& InputType, const FPCGDataTypeIdentifier& ThisType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const
{
	return false;
}

bool FPCGDataTypeInfo::SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const
{
	return false;
}

TOptional<TArray<UPCGNode*>> FPCGDataTypeInfo::AddConversionNodesFrom(const FPCGDataTypeIdentifier& InputType, const FPCGDataTypeIdentifier& ThisType, UPCGGraph* InGraph, UPCGPin* InUpstreamPin, UPCGPin* InDownstreamPin) const
{
	TSubclassOf<UPCGSettings> SettingsClass;
	if (SupportsConversionFrom(InputType, ThisType, &SettingsClass) && SettingsClass)
	{
		return TArray<UPCGNode*>{FPCGSubgraphHelpers::SpawnNodeAndConnect(InGraph, InUpstreamPin, InDownstreamPin, SettingsClass)};
	}
	else
	{
		return {};
	}
}

TOptional<TArray<UPCGNode*>> FPCGDataTypeInfo::AddConversionNodesTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, UPCGGraph* InGraph, UPCGPin* InUpstreamPin, UPCGPin* InDownstreamPin) const
{
	TSubclassOf<UPCGSettings> SettingsClass;
	if (SupportsConversionTo(ThisType, OutputType, &SettingsClass) && SettingsClass)
	{
		return TArray<UPCGNode*>{FPCGSubgraphHelpers::SpawnNodeAndConnect(InGraph, InUpstreamPin, InDownstreamPin, SettingsClass)};
	}
	else
	{
		return {};
	}
}

#if WITH_EDITOR

TOptional<FText> FPCGDataTypeInfo::GetSubtypeTooltip(const FPCGDataTypeIdentifier& ThisType) const
{
	return {};
}

TOptional<FText> FPCGDataTypeInfo::GetExtraTooltip(const FPCGDataTypeIdentifier& ThisType) const
{
	return {};
}

bool FPCGDataTypeInfo::Hidden() const
{
	return false;
}
#endif //WITH_EDITOR

FPCGDataTypeBaseId FPCGDataTypeBaseId::MakeFromLegacyType(EPCGDataType DataType)
{
	switch (DataType)
	{
	case EPCGDataType::Any:
		return Construct<FPCGDataTypeInfo>();
	case EPCGDataType::Point:
		return Construct<FPCGDataTypeInfoPoint>();
	case EPCGDataType::Spline:
		return Construct<FPCGDataTypeInfoSpline>();
	case EPCGDataType::LandscapeSpline:
		return Construct<FPCGDataTypeInfoLandscapeSpline>();
	case EPCGDataType::PolyLine:
		return Construct<FPCGDataTypeInfoPolyline>();
	case EPCGDataType::Polygon2D:
		return Construct<FPCGDataTypeInfoPolygon2D>();
	case EPCGDataType::Landscape:
		return Construct<FPCGDataTypeInfoLandscape>();
	case EPCGDataType::Texture:
		return Construct<FPCGDataTypeInfoTexture2D>();
	case EPCGDataType::RenderTarget:
		return Construct<FPCGDataTypeInfoRenderTarget2D>();
	case EPCGDataType::VirtualTexture:
		return Construct<FPCGDataTypeInfoVirtualTexture>();
	case EPCGDataType::BaseTexture:
		return Construct<FPCGDataTypeInfoTexture2DSingleBase>();
	case EPCGDataType::Surface:
		return Construct<FPCGDataTypeInfoSurface>();
	case EPCGDataType::Volume:
		return Construct<FPCGDataTypeInfoVolume>();
	case EPCGDataType::Primitive:
		return Construct<FPCGDataTypeInfoPrimitive>();
	case EPCGDataType::DynamicMesh:
		return Construct<FPCGDataTypeInfoDynamicMesh>();
	case EPCGDataType::StaticMeshResource:
		return Construct<FPCGDataTypeInfoStaticMeshResource>();
	case EPCGDataType::Concrete:
		return Construct<FPCGDataTypeInfoConcrete>();
	case EPCGDataType::Composite:
		return Construct<FPCGDataTypeInfoComposite>();
	case EPCGDataType::Spatial:
		return Construct<FPCGDataTypeInfoSpatial>();
	case EPCGDataType::ProxyForGPU:
		return Construct<FPCGDataTypeInfoProxyForGPU>();
	case EPCGDataType::Param:
		return Construct<FPCGDataTypeInfoParam>();
	case EPCGDataType::Settings:
		return Construct<FPCGDataTypeInfoSettings>();
	case EPCGDataType::Other:
		return Construct<FPCGDataTypeInfoOther>();
	case EPCGDataType::None: // fall-through
	default:
		return FPCGDataTypeBaseId{};
	}
}

EPCGDataType FPCGDataTypeBaseId::AsLegacyType() const
{
#define PCG_DATA_TYPE_TEST(TestStruct) if (Struct == TestStruct::StaticStruct()) { return TestStruct::GetStaticAssociatedLegacyType(); }
		
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfo)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoSpatial)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoConcrete)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoPoint)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoComposite)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoParam)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoSpline)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoLandscapeSpline)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoPolyline)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoPolygon2D)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoLandscape)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoTexture2D)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoRenderTarget2D)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoVirtualTexture)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoTexture2DSingleBase)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoSurface)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoVolume)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoPrimitive)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoDynamicMesh)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoStaticMeshResource)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoConcrete)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoComposite)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoSpatial)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoProxyForGPU)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoParam)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoSettings)
	PCG_DATA_TYPE_TEST(FPCGDataTypeInfoOther)
		
#undef PCG_DATA_TYPE_TEST

	return EPCGDataType::None;
}
