// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Registry/PCGDataTypeIdentifier.h"

#include "Containers/StaticArray.h"
#include "PCGContext.h"
#include "PCGData.h"
#include "PCGModule.h"

#include "Algo/AnyOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataTypeIdentifier)

namespace PCGDataTypeIdentifier
{
	/**
	* Sorted by hand, with the highest number of bits set first
	* When we initialize an identifier from a EPCGDataType, we need to go in hierarchical order, by checking the highest in the hierarchy
	* (aka. the ones with the most bits sets), to be sure to always have the most compact representation.
	* (i.e. if there are EPCGDataType::Texture | EPCGDataType::RenderTarget, we have to make sure to check for EPCGDataType::BaseTexture first, as it is the parent)
	*/
	static constexpr const TStaticArray<EPCGDataType, 23>& HierarchyDataType{
			EPCGDataType::Any,         // 30 bits
			EPCGDataType::Spatial,     // 12 bits
			EPCGDataType::Concrete,    // 11 bits
			EPCGDataType::Surface,     // 4 bits
			EPCGDataType::PolyLine,    // 3 bits
			EPCGDataType::BaseTexture, // 2 bits
		
			EPCGDataType::Point,
			EPCGDataType::Spline,
			EPCGDataType::LandscapeSpline,
			EPCGDataType::Polygon2D,
			EPCGDataType::Landscape,
			EPCGDataType::RenderTarget,
			EPCGDataType::Texture,
			EPCGDataType::VirtualTexture,
			EPCGDataType::Volume,
			EPCGDataType::Primitive,
			EPCGDataType::DynamicMesh,
			EPCGDataType::StaticMeshResource,
			EPCGDataType::Composite,
			EPCGDataType::ProxyForGPU,
			EPCGDataType::Param,
			EPCGDataType::Settings,
			EPCGDataType::Other,
		};
}

void FPCGDataTypeIdentifier::ComposeFromLegacyType(EPCGDataType LegacyType)
{
	FPCGDataTypeBaseId BaseId = FPCGDataTypeBaseId::MakeFromLegacyType(LegacyType);
	if (BaseId.IsValid())
	{
		ComposeFromBaseId(BaseId);
		return;
	}

	// Go through all the enum, from the root types to the child types.
	for (EPCGDataType Type : PCGDataTypeIdentifier::HierarchyDataType)
	{
		if ((Type & LegacyType) == Type)
		{
			BaseId = FPCGDataTypeBaseId::MakeFromLegacyType(Type);
			if (BaseId.IsValid())
			{
				ComposeFromBaseId(BaseId);
				LegacyType &= ~Type;
			}
		}

		if (!LegacyType)
		{
			break;
		}
	}
}

void FPCGDataTypeIdentifier::ComposeFromPCGDataSubclass(TSubclassOf<UPCGData> Subclass)
{
	if (Subclass)
	{
		ComposeFromBaseId(Subclass->GetDefaultObject<UPCGData>()->GetDataTypeId());
	}
}

bool FPCGDataTypeIdentifier::PrepareForCompose(const FPCGDataTypeBaseId& BaseId)
{
	if (!BaseId.IsValid())
	{
		return false;
	}

	// By definition, adding Any will remove everything
	if (BaseId == FPCGDataTypeInfo::AsId())
	{
		Ids.Empty(1);
		return true;
	}

	// Make sure to never add Ids that are already covered by a wider type (or the same one), and replace any type that is the child of the added type.
	// Order is not important, so we remove at swap.
	// By construction, we can never have a type that would be a child and a parent of a type in Ids (either it is a child or a parent)
	// i.e. if we have the hierarchy A -> B -> C, an identifier C | A is not constructible, so we could never try to add B to it.
	for (int32 i = Ids.Num() - 1; i >= 0; --i)
	{
		const FPCGDataTypeBaseId& ThisId = Ids[i];

		// This also catch BaseId == ThisId, which mean we do not remove it if it is the same (but won't add it either).
		if (BaseId.IsChildOf(ThisId))
		{
			return false;
		}
		else if (ThisId.IsChildOf(BaseId))
		{
			Ids.RemoveAtSwap(i);
		}
	}

	return true;
}

void FPCGDataTypeIdentifier::ComposeFromBaseId(const FPCGDataTypeBaseId& BaseId)
{
	if (PrepareForCompose(BaseId))
	{
		// No need for add unique, as the unicity was checked in PrepareForCompose
		Ids.Add(BaseId);
	}
}

void FPCGDataTypeIdentifier::ComposeFromBaseId(FPCGDataTypeBaseId&& BaseId)
{
	if (PrepareForCompose(BaseId))
	{
		// No need for add unique, as the unicity was checked in PrepareForCompose
		Ids.Add(MoveTemp(BaseId));
	}
}

FPCGDataTypeIdentifier::FPCGDataTypeIdentifier(EPCGDataType LegacyType)
{
	ComposeFromLegacyType(LegacyType);
}

FPCGDataTypeIdentifier::FPCGDataTypeIdentifier(const FPCGDataTypeBaseId& BaseId)
{
	ComposeFromBaseId(BaseId);
}

FPCGDataTypeIdentifier::FPCGDataTypeIdentifier(const TSubclassOf<UPCGData>& DataClass)
{
	ComposeFromPCGDataSubclass(DataClass);
}

FPCGDataTypeIdentifier FPCGDataTypeIdentifier::Construct(TConstArrayView<FPCGDataTypeIdentifier> InIds)
{
	FPCGDataTypeIdentifier Result;
	
	for (const FPCGDataTypeIdentifier& Id : InIds)
	{
		for (const FPCGDataTypeBaseId& BaseId : Id.GetIds())
		{
			Result.ComposeFromBaseId(BaseId);
		}
	}

	return Result;
}

FPCGDataTypeIdentifier FPCGDataTypeIdentifier::Construct(TConstArrayView<TSubclassOf<UPCGData>> Classes)
{
	FPCGDataTypeIdentifier Result;
	
	for (TSubclassOf<UPCGData> Class : Classes)
	{
		Result.ComposeFromPCGDataSubclass(Class);
	}

	return Result;
}

FPCGDataTypeIdentifier::operator EPCGDataType() const
{
	EPCGDataType Result = EPCGDataType::None;
	for (const FPCGDataTypeBaseId& Id : Ids)
	{
		Result |= Id.AsLegacyType();
	}

	return Result;
}

FPCGDataTypeIdentifier& FPCGDataTypeIdentifier::operator|=(EPCGDataType OtherLegacyType)
{
	*this = operator|(OtherLegacyType);
	return *this;
}

FPCGDataTypeIdentifier& FPCGDataTypeIdentifier::operator|=(const FPCGDataTypeIdentifier& Other)
{
	// If we are invalid, just copy Other
	if (!IsValid())
	{
		*this = Other;
	}
	else
	{
		for (const FPCGDataTypeBaseId& Id : Other.GetIds())
		{
			ComposeFromBaseId(Id);
		}
	}

	return *this;
}

FPCGDataTypeIdentifier& FPCGDataTypeIdentifier::operator|=(FPCGDataTypeIdentifier&& Other)
{
	// If we are invalid, just swap with Other
	if (!IsValid())
	{
		Swap(*this, Other);
	}
	else
	{
		for (FPCGDataTypeBaseId& Id : Other.Ids)
		{
			ComposeFromBaseId(MoveTemp(Id));
		}
	}
	
	return *this;
}

FPCGDataTypeIdentifier FPCGDataTypeIdentifier::operator|(EPCGDataType OtherLegacyType) const
{
	return *this | FPCGDataTypeIdentifier{OtherLegacyType};
}

FPCGDataTypeIdentifier FPCGDataTypeIdentifier::operator|(const FPCGDataTypeIdentifier& Other) const
{
	FPCGDataTypeIdentifier Result = *this;
	Result |= Other;
	return Result;
}

FPCGDataTypeIdentifier FPCGDataTypeIdentifier::operator|(FPCGDataTypeIdentifier&& Other) const
{
	FPCGDataTypeIdentifier Result = *this;
	Result |= MoveTemp(Other);
	return Result;
}

FPCGDataTypeIdentifier& FPCGDataTypeIdentifier::operator&=(EPCGDataType OtherLegacyType)
{
	*this = operator&(OtherLegacyType);
	return *this;
}

FPCGDataTypeIdentifier& FPCGDataTypeIdentifier::operator&=(const FPCGDataTypeIdentifier& Other)
{
	TArray<FPCGDataTypeBaseId> CommonTypes;
	
	for (const FPCGDataTypeBaseId& ThisID : Ids)
	{
		if (!ensure(ThisID.IsValid()))
		{
			continue;
		}
		
		for (const FPCGDataTypeBaseId& OtherID : Other.GetIds())
		{
			if (!ensure(OtherID.IsValid()))
			{
				continue;
			}
			
			if (ThisID.IsChildOf(OtherID))
			{
				CommonTypes.AddUnique(ThisID);
			}
			else if (OtherID.IsChildOf(ThisID))
			{
				CommonTypes.AddUnique(OtherID);
			}
		}
	}

	Ids = MoveTemp(CommonTypes);
	
	return *this;
}

FPCGDataTypeIdentifier FPCGDataTypeIdentifier::operator&(EPCGDataType OtherLegacyType) const
{
	return *this & FPCGDataTypeIdentifier{OtherLegacyType};
}

FPCGDataTypeIdentifier FPCGDataTypeIdentifier::operator&(const FPCGDataTypeIdentifier& Other) const
{
	TArray<FPCGDataTypeBaseId> CommonTypes;
	
	for (const FPCGDataTypeBaseId& ThisID : Ids)
	{
		for (const FPCGDataTypeBaseId& OtherID : Other.GetIds())
		{
			if (ThisID.IsChildOf(OtherID))
			{
				CommonTypes.AddUnique(ThisID);
			}
			else if (OtherID.IsChildOf(ThisID))
			{
				CommonTypes.AddUnique(OtherID);
			}
		}
	}

	FPCGDataTypeIdentifier Result;
	Result.Ids = MoveTemp(CommonTypes);

	return Result;
}

bool FPCGDataTypeIdentifier::Intersects(const FPCGDataTypeIdentifier& Other) const
{
	for (const FPCGDataTypeBaseId& ThisID : Ids)
	{
		for (const FPCGDataTypeBaseId& OtherID : Other.GetIds())
		{
			if (ThisID.IsChildOf(OtherID) || OtherID.IsChildOf(ThisID))
			{
				return true;
			}
		}
	}

	return false;
}

FPCGDataTypeIdentifier FPCGDataTypeIdentifier::Compose(const FPCGDataTypeIdentifier& Other)
{
	return Compose(TConstArrayView<FPCGDataTypeIdentifier>{{*this, Other}});
}

FPCGDataTypeIdentifier FPCGDataTypeIdentifier::Compose(TConstArrayView<FPCGDataTypeIdentifier> IDs)
{
	return FPCGModule::GetConstDataTypeRegistry().GetIdentifiersComposition(IDs);
}

bool FPCGDataTypeIdentifier::operator!() const
{
	return !IsValid();
}

bool FPCGDataTypeIdentifier::SupportsType(EPCGDataType OtherLegacyType) const
{
	const FPCGDataTypeIdentifier OtherID(OtherLegacyType);
	if (OtherID.IsComposition())
	{
		return Algo::AllOf(OtherID.GetIds(), [this](const FPCGDataTypeBaseId& Id) { return Ids.Contains(Id); });
	}
	else
	{
		return SupportsType(FPCGDataTypeIdentifier(OtherLegacyType));
	}
}

FPCGDataTypeBaseId FPCGDataTypeIdentifier::GetId() const
{
	if (ensure(!IsComposition() && IsValid()))
	{
		return Ids[0];
	}
	else
	{
		return FPCGDataTypeBaseId{};
	}
}

TConstArrayView<FPCGDataTypeBaseId> FPCGDataTypeIdentifier::GetIds() const
{
	return Ids;
};

bool FPCGDataTypeIdentifier::IsSameType(const FPCGDataTypeIdentifier& Other) const
{
	return (!IsValid() && !Other.IsValid()) || (Ids.Num() == Other.Ids.Num() && SupportsType(Other));
}

bool FPCGDataTypeIdentifier::SupportsType(const FPCGDataTypeIdentifier& Other) const
{
	return IsValid() && Algo::AllOf(Other.GetIds(), [this](const FPCGDataTypeBaseId& Id) { return Ids.Contains(Id); });
}

bool FPCGDataTypeIdentifier::IsWider(const FPCGDataTypeIdentifier& Other) const
{
	if (IsSameType(Other))
	{
		return false;
	}
	
	const FPCGDataTypeIdentifier Intersection = *this & Other;
	return !Intersection.IsSameType(*this);
}

bool FPCGDataTypeIdentifier::IsIdentical(const FPCGDataTypeIdentifier& Other) const
{
	return IsSameType(Other) && CustomSubtype == Other.CustomSubtype;
}

bool FPCGDataTypeIdentifier::IsChildOf(const FPCGDataTypeIdentifier& Other) const
{
	if (IsComposition() || Other.IsComposition())
	{
		return (*this & Other).IsSameType(*this);
	}
	else
	{
		// Optimization if none are compositions
		return GetId().IsChildOf(Other.GetId());
	}
}

bool FPCGDataTypeIdentifier::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	const UEnum* Enum = StaticEnum<EPCGDataType>();
	check(Enum);
	
	if (Tag.GetType().IsEnum(Enum->GetFName()))
	{
		// Enum are serialized as FName (cf. TryLoadEnumValueByName)
		FName EnumValueName;
		Slot << EnumValueName;
		
		if (EnumValueName != NAME_None)
        {
        	int64 EnumValue = Enum->GetValueOrBitfieldFromString(*EnumValueName.ToString());
        	if (EnumValue != INDEX_NONE)
        	{
        		FPCGDataTypeIdentifier Temp = FPCGDataTypeIdentifier(static_cast<EPCGDataType>(EnumValue));
        		Ids = Temp.GetIds();
        	}
        }

		return true;
	}
	else
	{
		return false;
	}
}

FString FPCGDataTypeIdentifier::ToString() const
{
	return IsValid() ? FString::JoinBy(Ids, TEXT(" | "), [](const FPCGDataTypeBaseId& Id) { return Id.ToString(); }) : TEXT("None");
}

FText FPCGDataTypeIdentifier::ToDisplayText() const
{
	return FText::FromString(ToString());
}

#if WITH_EDITOR
FText FPCGDataTypeIdentifier::GetSubtypeTooltip() const
{
	if (IsValid() && !IsComposition())
	{
		if (const FPCGDataTypeInfo* Info = FPCGModule::GetConstDataTypeRegistry().GetTypeInfo(GetId()))
		{
			if (TOptional<FText> Tooltip = Info->GetSubtypeTooltip(*this); Tooltip.IsSet())
			{
				return MoveTemp(Tooltip.GetValue());
			}
		}
	}

	return ToDisplayText();
}

FText FPCGDataTypeIdentifier::GetExtraTooltip() const
{
	if (IsValid() && !IsComposition())
	{
		if (const FPCGDataTypeInfo* Info = FPCGModule::GetConstDataTypeRegistry().GetTypeInfo(GetId()))
		{
			if (TOptional<FText> Tooltip = Info->GetExtraTooltip(*this); Tooltip.IsSet())
			{
				return MoveTemp(Tooltip.GetValue());
			}
		}
	}

	return {};
}
#endif // WITH_EDITOR

FPCGDataTypeIdentifier UPCGDataTypeIdentifierHelpers::GetIdentifierFromLegacyType(EPCGExclusiveDataType LegacyDataType)
{
	const UEnum* DataTypeEnum = StaticEnum<EPCGDataType>();
	const UEnum* ExclusiveDataTypeEnum = StaticEnum<EPCGExclusiveDataType>();

	EPCGDataType InternalDataType = EPCGDataType::Other;

	if (DataTypeEnum && ExclusiveDataTypeEnum)
	{
		FName ExclusiveDataTypeName = ExclusiveDataTypeEnum->GetNameByValue(static_cast<__underlying_type(EPCGExclusiveDataType)>(LegacyDataType));
		if (ensure(ExclusiveDataTypeName != NAME_None))
		{
			const int64 MatchingType = DataTypeEnum->GetValueByName(ExclusiveDataTypeName);
			if (ensure(MatchingType != INDEX_NONE))
			{
				InternalDataType = static_cast<EPCGDataType>(MatchingType);
			}
		}
	}

	return FPCGDataTypeIdentifier(InternalDataType);
}

uint32 GetTypeHash(const FPCGDataTypeIdentifier& Id)
{
	// Hash needs to be stable to Ids order.
	TArray<int32, TInlineAllocator<16>> IdsHash;
	IdsHash.Reserve(Id.Ids.Num());
	Algo::Transform(Id.Ids, IdsHash, [](const FPCGDataTypeBaseId& BaseId) { return GetTypeHash(BaseId); });
	IdsHash.Sort();
	return HashCombine(GetTypeHash(IdsHash), Id.CustomSubtype);
}
