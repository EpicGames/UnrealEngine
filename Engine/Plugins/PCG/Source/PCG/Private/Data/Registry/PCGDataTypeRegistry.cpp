// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Registry/PCGDataTypeRegistry.h"

#include "PCGCommon.h"
#include "PCGData.h"

#if WITH_EDITOR
#include "Tests/PCGTestsCommon.h"
#endif // WITH_EDITOR

#include "UObject/UObjectIterator.h"

namespace PCGDataTypeHierarchy
{
/**
 * Node for the hierarchy tree that can go up (to the parent) and down (to the children)
 */
struct FNode
{
	~FNode();

	void AddChild(FNode* Child);

	FPCGDataTypeBaseId BaseID;

	FNode* Parent = nullptr;

	// Use pointers for parent stability. Otherwise, we can have an array allocation in the grandparent that invalidate the parent pointer.
	TArray<FNode*> Children;
};

/**
* Hierarchy tree used to query the hierarchy for the types, to find common ancestors for example.
* Expected to be filled by the registry after all the types are loaded.
*/
struct FTree
{
	FTree();

	void AddEntry(const FPCGDataTypeBaseId& BaseID);
	void RemoveEntry(const FPCGDataTypeBaseId& BaseID);

	/**
	 * Verify if TypeA and TypeB has a hierarchy relationship.
	 * OutCommonAncestor will contain the type if of the type that is higher in the hierarchy than TypeA and TypeB.
	 * If TypeA is a child of TypeB, OutCommonAncestor will be TypeB, and vice versa.
	 * returns false if identifiers are invalid.
	 */
	bool QueryHierarchy(const FPCGDataTypeIdentifier& TypeA, const FPCGDataTypeIdentifier& TypeB, FPCGDataTypeIdentifier& OutCommonAncestor) const;
	
	template <typename Func> requires std::is_invocable_r_v<FPCGDataTypeRegistry::ESearchCommand, Func, FNode*, int32>
	void BreadthSearch(Func InFunc) const;

	template <typename Func> requires std::is_invocable_r_v<FPCGDataTypeRegistry::ESearchCommand, Func, FNode*, int32>
	void DepthSearch(Func InFunc) const;

	FNode Root;
};
	
template <typename Func> requires std::is_invocable_r_v<FPCGDataTypeRegistry::ESearchCommand, Func, FNode*, int32>
void FTree::BreadthSearch(Func InFunc) const
{
	// Const-cast to not have 2 versions, const and non-const, since those are private.
	// Will have the node and its depth.
	TArray<TTuple<FNode*, int32>, TInlineAllocator<64>> List{{const_cast<FNode*>(&Root), 0}};
	int32 i = 0;

	while (i < List.Num())
	{
		FNode* Curr = List[i].Get<0>();
		const int32 Depth = List[i++].Get<1>();

		const FPCGDataTypeRegistry::ESearchCommand Command = InFunc(Curr, Depth);
	
		if (Command == FPCGDataTypeRegistry::ESearchCommand::Stop)
		{
			return;
		}

		if (Command == FPCGDataTypeRegistry::ESearchCommand::ExpandAndContinue)
		{
			for (FNode* Child : Curr->Children)
			{
				List.Emplace(Child, Depth + 1);
			}
		}
	}
}

template <typename Func> requires std::is_invocable_r_v<FPCGDataTypeRegistry::ESearchCommand, Func, FNode*, int32>
void FTree::DepthSearch(Func InFunc) const
{
	// Const-cast to not have 2 versions, const and non-const, since those are private.
	// Will have the node and its depth.
	TArray<TTuple<FNode*, int32>, TInlineAllocator<64>> Stack{{const_cast<FNode*>(&Root), 0}};

	while (!Stack.IsEmpty())
	{
		TTuple<FNode*, int32> It = Stack.Pop(EAllowShrinking::No);
		FNode* Curr = It.Get<0>();
		const int32 Depth = It.Get<1>();
	
		const FPCGDataTypeRegistry::ESearchCommand Command = InFunc(Curr, Depth);
	
		if (Command == FPCGDataTypeRegistry::ESearchCommand::Stop)
		{
			return;
		}

		if (Command == FPCGDataTypeRegistry::ESearchCommand::ExpandAndContinue)
		{
			for (FNode* Child : Curr->Children)
			{
				Stack.Emplace(Child, Depth + 1);
			}
		}
	}
}

FNode::~FNode()
{
	for (FNode* Child : Children)
	{
		delete Child;
	}
}

void FNode::AddChild(FNode* Child)
{
	Child->Parent = this;
	Children.Add(Child);
}

FTree::FTree()
{
	Root.BaseID = FPCGDataTypeInfo::StaticStruct();
}

void FTree::AddEntry(const FPCGDataTypeBaseId& BaseID)
{
	FNode* Parent = &Root;

	FNode* NewNode = new FNode();
	NewNode->BaseID = BaseID;

	const UScriptStruct* EntryStruct = BaseID.GetStruct();

	BreadthSearch([&Parent, EntryStruct](FNode* Curr, int32 Depth) -> FPCGDataTypeRegistry::ESearchCommand
	{
		if (EntryStruct->IsChildOf(Curr->BaseID.GetStruct()))
		{
			Parent = Curr;
		}

		// Don't stop until the end.
		return FPCGDataTypeRegistry::ESearchCommand::ExpandAndContinue;
	});

	// If the parent have children, we need to check for all of them if they are a child of the new entry
	int32 i = 0 ;
	while (i < Parent->Children.Num())
	{
		FNode* Curr = Parent->Children[i];
		if (Curr->BaseID.GetStruct()->IsChildOf(EntryStruct))
		{
			NewNode->AddChild(Curr);
			Parent->Children.RemoveAt(i);
		}
		else
		{
			++i;
		}
	}

	// Finally setup the new node
	Parent->AddChild(NewNode);
}

void FTree::RemoveEntry(const FPCGDataTypeBaseId& BaseID)
{
	DepthSearch([&BaseID](FNode* Curr, int32 Depth) -> FPCGDataTypeRegistry::ESearchCommand
	{
		if (Curr->BaseID == BaseID)
		{
			for (FNode* Child : Curr->Children)
			{
				Child->Parent = Curr->Parent;
			}

			Curr->Parent->Children.Remove(Curr);
			// Transfer ownership
			Curr->Parent->Children.Append(Curr->Children);
			// Make sure to empty to avoid the delete to destroy the children.
			Curr->Children.Empty();

			delete Curr;
		
			return FPCGDataTypeRegistry::ESearchCommand::Stop;
		}
		else
		{
			return FPCGDataTypeRegistry::ESearchCommand::ExpandAndContinue;
		}
	});
}

bool FTree::QueryHierarchy(const FPCGDataTypeIdentifier& TypeA, const FPCGDataTypeIdentifier& TypeB,
	FPCGDataTypeIdentifier& OutCommonAncestor) const
{
	if (!TypeA.IsValid() || !TypeB.IsValid())
	{
		return false;
	}

	if (TypeA.IsSameType(TypeB))
	{
		OutCommonAncestor = TypeA;
		return true;
	}

	const bool TypeAIsComposition = TypeA.IsComposition();
	const bool TypeBIsComposition = TypeB.IsComposition();

	// In case one is a composition and not the other, check if they support the non-composition.
	if (TypeAIsComposition && !TypeBIsComposition && TypeA.SupportsType(TypeB))
	{
		OutCommonAncestor = TypeA;
		return true;
	}

	if (TypeBIsComposition && !TypeAIsComposition && TypeB.SupportsType(TypeA))
	{
		OutCommonAncestor = TypeB;
		return true;
	}

	// Breadth search to find which comes first in the hierarchy
	const FNode* FirstFound = nullptr;
	const FNode* SecondFound = nullptr;
	BreadthSearch([&FirstFound, &SecondFound, &TypeA, &TypeB](const FNode* Curr, int32 Depth) -> FPCGDataTypeRegistry::ESearchCommand
	{
		bool bFound = false;
		const FPCGDataTypeIdentifier CurrID{Curr->BaseID};

		if (TypeA.SupportsType(CurrID))
		{
			if (!FirstFound)
			{
				FirstFound = Curr;
				return FPCGDataTypeRegistry::ESearchCommand::ExpandAndContinue;
			}

			bFound = true;
		}

		if (TypeB.SupportsType(CurrID))
		{
			if (!FirstFound)
			{
				FirstFound = Curr;
				return FPCGDataTypeRegistry::ESearchCommand::ExpandAndContinue;
			}

			bFound = true;
		}

		if (!bFound)
		{
			return FPCGDataTypeRegistry::ESearchCommand::ExpandAndContinue;
		}
		else
		{
			SecondFound = Curr;
			return FPCGDataTypeRegistry::ESearchCommand::Stop;
		}
	});

	if (!FirstFound || !SecondFound)
	{
		return false;
	}

	using FHierarchyArray = TArray<const FNode*, TInlineAllocator<16>>;

	auto BuildHierarchy = [](const FNode* Start)-> FHierarchyArray
	{
		FHierarchyArray Hierarchy;
		const FNode* UpstreamCurr = Start;
			
		while (UpstreamCurr)
		{
			Hierarchy.Add(UpstreamCurr);
			UpstreamCurr = UpstreamCurr->Parent;
		}

		return Hierarchy;
	};

	const FHierarchyArray FirstHierarchy = BuildHierarchy(FirstFound);
	const FHierarchyArray SecondHierarchy = BuildHierarchy(SecondFound);

	int32 i = FirstHierarchy.Num() - 1;
	int32 j = SecondHierarchy.Num() - 1;

	while (i >= 0 && j >= 0 && FirstHierarchy[i] == SecondHierarchy[j])
	{
		--i;
		--j;
	}

	// All types have a common root type (FPCGDataTypeInfo) so FirstHierarchy and SecondHierarchy will always have the same "Last" member (same ancestor).
	// By construction, FirstHierarchy[i+1] will never be out of bounds.
	OutCommonAncestor = FPCGDataTypeIdentifier{FirstHierarchy[i+1]->BaseID};

	return OutCommonAncestor.IsValid();
}
} // namespace PCGDataTypeHierarchy

FPCGDataTypeRegistry::FPCGDataTypeRegistry()
{
	HierarchyTree = MakeUnique<PCGDataTypeHierarchy::FTree>();
	
	// Pre-register the base type
	Mapping.Emplace(FPCGDataTypeInfo::StaticStruct(), FPCGDataTypeInfo::StaticStruct());
}

void FPCGDataTypeRegistry::RegisterEntry(const FPCGDataTypeBaseId& BaseID)
{
	check(IsInGameThread());
	
	if (!ensureMsgf(!Mapping.Contains(BaseID), TEXT("Tried to register a PCG data type that has the same identifier as another already registered.")))
	{
		return;
	}

	Mapping.Emplace(BaseID, BaseID.GetStruct());
	HierarchyTree->AddEntry(BaseID);
}

void FPCGDataTypeRegistry::Shutdown()
{
	Mapping.Empty();
	HierarchyTree.Reset();

#if WITH_EDITOR
	CombinationPinColors.Empty();
	SingleTypePinColors.Empty();
	CombinationPinIcons.Empty();
	SingleTypePinIcons.Empty();
#endif // WITH_EDITOR
}

FPCGDataTypeRegistry::~FPCGDataTypeRegistry() = default;

EPCGDataTypeCompatibilityResult FPCGDataTypeRegistry::IsCompatible(const FPCGDataTypeIdentifier& InType, const FPCGDataTypeIdentifier& OutType, FText* OptionalOutCompatibilityMessage) const
{
	if (!InType.IsValid() || !OutType.IsValid())
	{
		return EPCGDataTypeCompatibilityResult::UnknownType;
	}

	auto IsCompatibleForSubtype = [this, &InType, &OutType, &OptionalOutCompatibilityMessage]()
	{
		if (!InType.IsComposition())
		{
			const TInstancedStruct<FPCGDataTypeInfo>* InDefIt = Mapping.Find(InType.GetId());
			return InDefIt ? InDefIt->Get().IsCompatibleForSubtype(InType, OutType, OptionalOutCompatibilityMessage) : EPCGDataTypeCompatibilityResult::UnknownType;
		}
		else
		{
			return EPCGDataTypeCompatibilityResult::Compatible;
		}
	};

	if (InType.IsSameType(OutType))
	{
		return IsCompatibleForSubtype();
	}

	// Run the intersection between the two.
	const FPCGDataTypeIdentifier Intersection = InType & OutType;

	// If the intersection gives InType, it means OutType accept all InType, so it is compatible
	if (Intersection.IsSameType(InType))
	{
		return IsCompatibleForSubtype();
	}
	
	// If the intersection gives OutType, or something else that is valid, it means that InType is wider than OutType, so we might need a filter or a conversion.
	// Conversion can only happen if InType and OutType are not compositions
	if (Intersection.IsSameType(OutType) || Intersection.IsValid())
	{
		if (!InType.IsComposition() && !OutType.IsComposition())
		{
			const TInstancedStruct<FPCGDataTypeInfo>* InDefIt = Mapping.Find(InType.GetId());
			const TInstancedStruct<FPCGDataTypeInfo>* OutDefIt = Mapping.Find(OutType.GetId());

			if (!InDefIt || !OutDefIt)
			{
				return EPCGDataTypeCompatibilityResult::UnknownType;
			}

			const bool SupportConversion = InDefIt->Get().SupportsConversionTo(InType, OutType, /*OptionalOutConversionSettings=*/nullptr, OptionalOutCompatibilityMessage)
				|| OutDefIt->Get().SupportsConversionFrom(InType, OutType, /*OptionalOutConversionSettings=*/nullptr, OptionalOutCompatibilityMessage);

			return SupportConversion ? EPCGDataTypeCompatibilityResult::RequireConversion : EPCGDataTypeCompatibilityResult::RequireFilter;
		}
		else
		{
			return EPCGDataTypeCompatibilityResult::RequireFilter;
		}
	}

	// If we have no intersection and either of them is a composition, it's ambiguous, as we might require multiple conversions,
	// so mark it as not compatible and let the user filter and convert as they see fit.
	if (InType.IsComposition() || OutType.IsComposition())
	{
		return EPCGDataTypeCompatibilityResult::NotCompatible;
	}

	// Finally we check if we have a conversion
	const TInstancedStruct<FPCGDataTypeInfo>* InDefIt = Mapping.Find(InType.GetId());
	const TInstancedStruct<FPCGDataTypeInfo>* OutDefIt = Mapping.Find(OutType.GetId());

	if (!InDefIt || !OutDefIt)
	{
		return EPCGDataTypeCompatibilityResult::UnknownType;
	}

	const bool SupportConversion = InDefIt->Get().SupportsConversionTo(InType, OutType, /*OptionalOutConversionSettings=*/nullptr, OptionalOutCompatibilityMessage)
		|| OutDefIt->Get().SupportsConversionFrom(InType, OutType, /*OptionalOutConversionSettings=*/nullptr, OptionalOutCompatibilityMessage);
	
	return SupportConversion ? EPCGDataTypeCompatibilityResult::RequireConversion : EPCGDataTypeCompatibilityResult::NotCompatible;
}

const FPCGDataTypeInfo* FPCGDataTypeRegistry::GetTypeInfo(const FPCGDataTypeBaseId& ID) const
{
	const TInstancedStruct<FPCGDataTypeInfo>* TypeInfo = Mapping.Find(ID);
	return TypeInfo ? TypeInfo->GetPtr() : nullptr;
}

/**
 * Goal: From a list of identifiers, aggregate all the types, and find the common ancestor of them all
 * Algorithm overview:
 * - Gather all the unique types
 * - Initialize result to the first unique type and for the rest find the common ancestor with result. The common ancestor becomes the result and we continue.
 * - If we ever find Any, we can stop and return it.
 */
FPCGDataTypeIdentifier FPCGDataTypeRegistry::GetIdentifiersUnion(TConstArrayView<FPCGDataTypeIdentifier> Identifiers) const
{
	if (Identifiers.IsEmpty())
	{
		return FPCGDataTypeIdentifier{};
	}

	static const FPCGDataTypeBaseId AnyType = FPCGDataTypeBaseId::Construct<FPCGDataTypeInfo>();
	
	// Extract all unique IDs
	TArray<FPCGDataTypeBaseId, TInlineAllocator<32>> UniqueIDs;

	for (const FPCGDataTypeIdentifier& Identifier : Identifiers)
	{
		for (const FPCGDataTypeBaseId& ID : Identifier.GetIds())
		{
			// If we have the "Any" type in our composition, it is the parent of all the types, so the composition is "Any"
			if (ID == AnyType)
			{
				return FPCGDataTypeIdentifier{AnyType};
			}
			
			UniqueIDs.AddUnique(ID);
		}
	}

	if (UniqueIDs.IsEmpty())
	{
		return FPCGDataTypeIdentifier{};
	}
	
	FPCGDataTypeIdentifier Result{UniqueIDs[0]};

	for (int32 i = 1; i < UniqueIDs.Num(); ++i)
	{
		const FPCGDataTypeBaseId& ID = UniqueIDs[i];
		
		if (Result.IsSameType(ID))
		{
			continue;
		}

		FPCGDataTypeIdentifier CommonAncestor{};
		if (!HierarchyTree->QueryHierarchy(Result, ID, CommonAncestor))
		{
			return FPCGDataTypeIdentifier{};
		}

		if (CommonAncestor.IsSameType(AnyType))
		{
			return FPCGDataTypeIdentifier{AnyType};
		}

		Result = CommonAncestor;
	}

	return Result;
}

/**
 * Goal: From a list of identifiers, aggregate all the types, and reduce types that are fully represent the parent type
 *       (i.e. if we compose DataB and DataC that are the only children of DataA, the composition should return DataA)
 * Algorithm overview:
 * - Gather all the unique types
 * - Do a depth search in the hierarchy tree to look for our unique types
 *    + When we find a type, we mark at which depth it is and add it to our list of nodes.
 *    + Since it's Depth first, any type that we found that is at a greater depth that the one marked is a child, and can be discarded for the composition.
 *    + At the moment we have a Depth greater or equal than the one marked, we are on another branch, and we reset our marked depth.
 * - When all of our types are found, we sort them by depth in descending order (deeper first), we need the descending order because we might merge 2 types
 *   that can be merged with something at a higher level of the hierarchy. (i.e. D and E are child of B, and B and C are child of A, combining C, D and E should give A.
 * - We iterate through our sorted list, and we look for all the other nodes that are on the same depth (potential siblings). We count all of them that share the same parent.
 * - If the number of siblings is equal to the number of children of the parent type, we can merge them all into the parent type.
 * - When the merge is done, we return the aggregation of the all the remaining types.
 */
FPCGDataTypeIdentifier FPCGDataTypeRegistry::GetIdentifiersComposition(TConstArrayView<FPCGDataTypeIdentifier> Identifiers) const
{
	if (Identifiers.IsEmpty())
	{
		return FPCGDataTypeIdentifier{};
	}

	static const FPCGDataTypeBaseId AnyType = FPCGDataTypeBaseId::Construct<FPCGDataTypeInfo>();
	
	// Extract all unique IDs
	TArray<FPCGDataTypeBaseId, TInlineAllocator<32>> UniqueIDs;

	for (const FPCGDataTypeIdentifier& Identifier : Identifiers)
	{
		for (const FPCGDataTypeBaseId& ID : Identifier.GetIds())
		{
			// If we have the "Any" type in our composition, it is the parent of all the types, so the composition is "Any"
			if (ID == AnyType)
			{
				return FPCGDataTypeIdentifier{AnyType};
			}
			
			UniqueIDs.AddUnique(ID);
		}
	}

	// If we just have a single one, return this one
	if (UniqueIDs.Num() == 1)
	{
		return FPCGDataTypeIdentifier{UniqueIDs[0]};
	}

	// Find all the nodes, and sort them by depth.
	TArray<TTuple<const PCGDataTypeHierarchy::FNode*, int32>, TInlineAllocator<32>> Nodes;
	Nodes.Reserve(UniqueIDs.Num());

	HierarchyTree->DepthSearch([&UniqueIDs, &Nodes](const PCGDataTypeHierarchy::FNode* Node, int32 Depth)
	{
		if (UniqueIDs.Contains(Node->BaseID))
		{
			Nodes.Emplace(Node, Depth);
			// We found one, no need to check the children, as they won't contribute to the composition.
			return ESearchCommand::Continue;
		}

		// Continue until we find them all.
		return ESearchCommand::ExpandAndContinue;
	});

	// Sort by depth, starting at the deepest.
	Nodes.Sort([](const auto& LH, const auto& RH) { return LH.template Get<1>() > RH.template Get<1>(); });

	// At that point, we have only types that are on different branches.
	// If they are on the same depth, we try to reduce them
	int32 i = 0;
	TArray<int32, TInlineAllocator<32>> Indices;
	
	while (i < Nodes.Num() - 1)
	{
		int32 SameParentNum = 1;
		Indices.SetNum(0, EAllowShrinking::No);
		
		for (int32 j = i + 1; j < Nodes.Num(); ++j)
		{
			if (Nodes[i].Get<1>() == Nodes[j].Get<1>() && Nodes[i].Get<0>()->Parent == Nodes[j].Get<0>()->Parent)
			{
				SameParentNum++;
				Indices.Add(j);
			}
		}

		if (SameParentNum > 1 && SameParentNum == Nodes[i].Get<0>()->Parent->Children.Num())
		{
			// All the types at a given level are combined. Remove all of them and replace it with the parent type.
			// The parent type can't be in the composition, because if it were, those children won't have been added while doing the depth search
			Nodes[i].Get<0>() = Nodes[i].Get<0>()->Parent;
			Nodes[i].Get<1>()--;
			for (int32 j = Indices.Num() - 1; j >= 0; --j)
			{
				Nodes.RemoveAt(Indices[j]);
			}

			// We have a merge, so try again at index i.
		}
		else
		{
			// Otherwise, check the next
			++i;
		}
	}

	// Returns the aggregation of the types after the reduce
	FPCGDataTypeIdentifier Result{};
	Result.Ids.Reserve(Nodes.Num());
	for (const auto& Node : Nodes)
	{
		Result.Ids.Emplace(Node.Get<0>()->BaseID);
	}

	return Result;
}

/**
 * Goal: From a source and list of difference, return the source type without any type that are in the difference.
 * Algorithm overview:
 * - Get all the leaves for the source type
 * - Remove any leaf that is a child of a type in the difference
 * - Re-compose everything.
 */
FPCGDataTypeIdentifier FPCGDataTypeRegistry::GetIdentifiersDifference(const FGetIdentifiersDifferenceParams& Params) const
{
	if (Params.DifferenceIdentifiers.IsEmpty() || !Params.SourceIdentifier || !Params.SourceIdentifier->IsValid())
	{
		return Params.SourceIdentifier ? *Params.SourceIdentifier : FPCGDataTypeIdentifier{};
	}

	const FPCGDataTypeIdentifier& SourceIdentifier = *Params.SourceIdentifier; 
	
	// Extract all leaves
	TArray<FPCGDataTypeBaseId, TInlineAllocator<32>> SourceLeaves;
	int32 CurrentFoundDepth = -1;
	int32 Found = 0;

	HierarchyTree->DepthSearch([&CurrentFoundDepth, &SourceIdentifier, &Params, &Found, &SourceLeaves](PCGDataTypeHierarchy::FNode* Curr, int32 Depth) -> ESearchCommand
	{
		if (CurrentFoundDepth == Depth)
		{
			CurrentFoundDepth = -1;
		}
			
		if (CurrentFoundDepth == -1 && SourceIdentifier.GetIds().Contains(Curr->BaseID))
		{
			CurrentFoundDepth = Depth;
			Found++;
		}
			
		if (CurrentFoundDepth != -1 && CurrentFoundDepth <= Depth && Curr->Children.IsEmpty())
		{
			const bool bShouldInclude = (Params.Filter == FGetIdentifiersDifferenceParams::IncludeFilteredTypes) == (Params.FilteredTypes.Contains(Curr->BaseID));

			if (bShouldInclude)
			{
				SourceLeaves.Add(Curr->BaseID);
			}
		}

		return CurrentFoundDepth == -1 && Found == SourceIdentifier.GetIds().Num() ? ESearchCommand::Stop : ESearchCommand::ExpandAndContinue;
	});

	bool bHasRemoved = false;
	for (const FPCGDataTypeIdentifier& Difference : Params.DifferenceIdentifiers)
	{
		for (const FPCGDataTypeBaseId& DifferenceId : Difference.GetIds())
		{
			while (!SourceLeaves.IsEmpty())
			{
				int32 Index = SourceLeaves.IndexOfByPredicate([&DifferenceId](const FPCGDataTypeBaseId& SourceId) { return SourceId.IsChildOf(DifferenceId);});
				if (Index == INDEX_NONE)
				{
					break;
				}

				bHasRemoved = true;
				SourceLeaves.RemoveAtSwap(Index);
			}

			if (SourceLeaves.IsEmpty())
			{
				break;
			}
		}

		if (SourceLeaves.IsEmpty())
		{
			break;
		}
	}

	if (!bHasRemoved)
	{
		return SourceIdentifier;
	}

	TArray<FPCGDataTypeIdentifier, TInlineAllocator<32>> FinalIdentifiers;
	FinalIdentifiers.Reserve(SourceLeaves.Num());
	Algo::Transform(SourceLeaves, FinalIdentifiers, [](const FPCGDataTypeBaseId& Id) { return Id; });
	return GetIdentifiersComposition(FinalIdentifiers);
}

void FPCGDataTypeRegistry::HierarchyDepthSearch(TFunctionRef<ESearchCommand(const FPCGDataTypeBaseId& Id, int32 Depth)> Callback) const
{
	HierarchyTree->DepthSearch([&Callback](const PCGDataTypeHierarchy::FNode* Node, int32 Depth)
	{
		check(Node);
		return Callback(Node->BaseID, Depth);
	});
}

#if WITH_EDITOR
namespace PCGDataTypeRegistry
{
	template <typename Function>
	void AddUniqueOrdered(const FPCGDataTypeIdentifier& InId, TArray<TTuple<FPCGDataTypeIdentifier, Function>>& InContainer, Function&& InFunction)
	{
		bool bInserted = false;
		for (int32 i = 0; i < InContainer.Num(); ++i)
		{
			const FPCGDataTypeIdentifier& CurrId = InContainer[i].template Get<0>();
			if (!ensure(!InId.IsSameType(CurrId)))
			{
				UE_LOGF(LogPCG, Error, "Try to register a type '%ls' that is already registered", *InId.ToString());
				return;
			}

			// Children will come always first in the array
			if (InId.IsChildOf(CurrId))
			{
				InContainer.EmplaceAt(i, InId, std::forward<Function>(InFunction));
				bInserted = true;
				break;
			}
		}

		if (!bInserted)
		{
			InContainer.Emplace(InId, std::forward<Function>(InFunction));
		}
	}
	
	template <typename Function>
	void RemoveFromContainer(const FPCGDataTypeIdentifier& InId, TArray<TTuple<FPCGDataTypeIdentifier, Function>>& InContainer)
	{
		if (const int32 Index = InContainer.IndexOfByPredicate([&InId](const auto& It) { return It.template Get<0>() == InId; }) ; Index != INDEX_NONE)
		{
			InContainer.RemoveAt(Index);
		}
	}

	template <typename Function, typename ...Args>
	decltype(auto) CallFunction(const FPCGDataTypeIdentifier& InId, const TArray<TTuple<FPCGDataTypeIdentifier, Function>>& InContainer, bool& bSuccess, Args&& ...InArgs)
	{
		if (const auto* It = InContainer.FindByPredicate([&InId](const auto& It) { return InId.IsChildOf(It.template Get<0>()); }))
		{
			bSuccess = true;
			return It->template Get<1>()(InId, InArgs...);
		}
		else
		{
			bSuccess = false;
			using ReturnType = std::invoke_result_t<Function, const FPCGDataTypeIdentifier&, Args...>;
			return ReturnType{};
		}
	}

	template <typename Function, typename ...Args>
	decltype(auto) CallFunctionWithComposition(
		const FPCGDataTypeRegistry& InRegistry,
		const FPCGDataTypeIdentifier& InId,
		const TArray<TTuple<FPCGDataTypeIdentifier, Function>>& InContainerComposition,
		const TArray<TTuple<FPCGDataTypeIdentifier, Function>>& InContainerSingleType,
		Args&& ...InArgs)
	{
		using ReturnType = std::invoke_result_t<Function, const FPCGDataTypeIdentifier&, Args...>;
		bool bSuccess = false;
		
		if (InId.IsComposition())
		{
			ReturnType Result = PCGDataTypeRegistry::CallFunction(InId, InContainerComposition, bSuccess, InArgs...);
			if (!bSuccess)
			{
				// Retry with the Union of the types
				FPCGDataTypeIdentifier TypeUnion = InRegistry.GetIdentifiersUnion({InId});
				if (!TypeUnion.IsValid() || !ensure(!TypeUnion.IsComposition()))
				{
					TypeUnion = FPCGDataTypeBaseId::Construct<FPCGDataTypeInfo>();
				}

				return PCGDataTypeRegistry::CallFunction(TypeUnion, InContainerSingleType, bSuccess, InArgs...);
			}
			else
			{
				return Result;
			}
		}
		else
		{
			return PCGDataTypeRegistry::CallFunction(InId, InContainerSingleType, bSuccess, InArgs...);
		}
	}
}

void FPCGDataTypeRegistry::RegisterPinColorFunction(const FPCGDataTypeIdentifier& InId, FPCGPinColorQueryFunction InFunction)
{
	PCGDataTypeRegistry::AddUniqueOrdered(InId, InId.IsComposition() ? CombinationPinColors : SingleTypePinColors, MoveTemp(InFunction));
}

void FPCGDataTypeRegistry::UnregisterPinColorFunction(const FPCGDataTypeIdentifier& InId)
{
	PCGDataTypeRegistry::RemoveFromContainer(InId, InId.IsComposition() ? CombinationPinColors : SingleTypePinColors);
}

FLinearColor FPCGDataTypeRegistry::GetPinColor(const FPCGDataTypeIdentifier& InId) const
{
	return PCGDataTypeRegistry::CallFunctionWithComposition(*this, InId, CombinationPinColors, SingleTypePinColors);
}

void FPCGDataTypeRegistry::RegisterPinIconsFunction(const FPCGDataTypeIdentifier& InId, FPCGPinIconsQueryFunction InFunction)
{
	PCGDataTypeRegistry::AddUniqueOrdered(InId, InId.IsComposition() ? CombinationPinIcons : SingleTypePinIcons, MoveTemp(InFunction));
}

void FPCGDataTypeRegistry::UnregisterPinIconsFunction(const FPCGDataTypeIdentifier& InId)
{
	PCGDataTypeRegistry::RemoveFromContainer(InId, InId.IsComposition() ? CombinationPinIcons : SingleTypePinIcons);
}

TTuple<const FSlateBrush*, const FSlateBrush*> FPCGDataTypeRegistry::GetPinIcons(const FPCGDataTypeIdentifier& InId, const FPCGPinProperties& InProperties, bool bIsInput) const
{
	return PCGDataTypeRegistry::CallFunctionWithComposition(*this, InId, CombinationPinIcons, SingleTypePinIcons, InProperties, bIsInput);
}
#endif // WITH_EDITOR

void FPCGDataTypeRegistry::RegisterKnownTypes()
{
	const UScriptStruct* BaseStruct = FPCGDataTypeInfo::StaticStruct();
	
	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		// Skip FPCGDataTypeInfo as it is already register by default
		if (*It != BaseStruct && It->IsChildOf(BaseStruct))
		{
			RegisterEntry(FPCGDataTypeBaseId{*It});
		}
	}
}

// Tests for hierarchy
#if WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGHierarchyTest, FPCGTestBaseClass, "Plugins.PCG.DataTypeRegistry.Hierarchy", PCGTestsCommon::TestFlags)

bool FPCGHierarchyTest::RunTest(const FString& Parameters)
{
	TTuple<FPCGDataTypeIdentifier, FPCGDataTypeIdentifier, FPCGDataTypeIdentifier> TestValues[] =
	{
		{EPCGDataType::Any, EPCGDataType::Spatial, EPCGDataType::Any},
		{EPCGDataType::Any, EPCGDataType::Point, EPCGDataType::Any},
		{EPCGDataType::Point, EPCGDataType::Point, EPCGDataType::Point},
		{EPCGDataType::Spatial, EPCGDataType::Point, EPCGDataType::Spatial},
		{EPCGDataType::Spatial, EPCGDataType::Volume, EPCGDataType::Spatial},
		{EPCGDataType::Spatial, EPCGDataType::Surface, EPCGDataType::Spatial},
		{EPCGDataType::Spatial, EPCGDataType::Spline, EPCGDataType::Spatial},
		{EPCGDataType::Spline, EPCGDataType::Point, EPCGDataType::Concrete},
		{EPCGDataType::Volume, EPCGDataType::Point, EPCGDataType::Concrete},
		{EPCGDataType::Spline, EPCGDataType::LandscapeSpline, EPCGDataType::PolyLine},
		{EPCGDataType::Landscape, EPCGDataType::Texture, EPCGDataType::Surface},
		{EPCGDataType::RenderTarget, EPCGDataType::Texture, EPCGDataType::BaseTexture},
		{EPCGDataType::VirtualTexture, EPCGDataType::Texture, EPCGDataType::Surface},
		{EPCGDataType::Point, EPCGDataType::Param, EPCGDataType::Any},
		{EPCGDataType::PointOrParam, EPCGDataType::Param, EPCGDataType::PointOrParam},
		{EPCGDataType::PointOrParam, EPCGDataType::Point, EPCGDataType::PointOrParam},
		{EPCGDataType::PointOrParam, EPCGDataType::Volume, EPCGDataType::Any},
	};

	const FPCGDataTypeRegistry& Registry = FPCGModule::GetConstDataTypeRegistry();

	const UEnum* Enum = StaticEnum<EPCGDataType>();
	check(Enum);

	for (const auto& [TypeAIdentifier, TypeBIdentifier, ExpectedCommonAncestorIdentifier] : TestValues)
	{
		FPCGDataTypeIdentifier CommonAncestorAToB{};
		FPCGDataTypeIdentifier CommonAncestorBToA{};
		
		UTEST_TRUE(
			FString::Printf(TEXT("QueryHierarchy between %s and %s is valid"), *TypeAIdentifier.ToString(), *TypeBIdentifier.ToString()),
			Registry.HierarchyTree->QueryHierarchy(TypeAIdentifier, TypeBIdentifier, CommonAncestorAToB)
		);
		
		UTEST_TRUE(
			FString::Printf(TEXT("QueryHierarchy between %s and %s is valid"), *TypeBIdentifier.ToString(), *TypeAIdentifier.ToString()),
			Registry.HierarchyTree->QueryHierarchy(TypeBIdentifier, TypeAIdentifier, CommonAncestorBToA)
		);

		UTEST_TRUE(
			FString::Printf(TEXT("Expected ancestor between %s and %s is %s"), *TypeAIdentifier.ToString(), *TypeBIdentifier.ToString(), *ExpectedCommonAncestorIdentifier.ToString()),
			ExpectedCommonAncestorIdentifier.IsSameType(CommonAncestorAToB)
		);
		
		UTEST_TRUE(
			FString::Printf(TEXT("Expected ancestor between %s and %s is %s"), *TypeBIdentifier.ToString(), *TypeAIdentifier.ToString(), *ExpectedCommonAncestorIdentifier.ToString()),
			ExpectedCommonAncestorIdentifier.IsSameType(CommonAncestorBToA)
		);
	}

	return true;
}
#endif // WITH_EDITOR