// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Registry/PCGDataViewRegistry.h"

#include "Data/DataView/PCGDataView.h"
#include "Data/DataView/PCGDataViewInterface.h"

FPCGDataViewRegistry::FPCGDataViewRegistry() = default;
FPCGDataViewRegistry::FPCGDataViewRegistry(FPCGDataViewRegistry&&) = default;
FPCGDataViewRegistry& FPCGDataViewRegistry::operator=(FPCGDataViewRegistry&&) = default;
FPCGDataViewRegistry::~FPCGDataViewRegistry() = default;

void FPCGDataViewRegistry::RegisterPropertySelector(const TSubclassOf<UPCGData> InDataClass, TUniquePtr<const IPCGDataViewPropertySelector> PropertySelector)
{
	if (ensureMsgf(!PropertySelectorRegistry.Find(InDataClass), TEXT("Cannot register multiple property selectors of the same type.")))
	{
		PropertySelectorRegistry.Add(InDataClass, MoveTemp(PropertySelector));
	}
}

void FPCGDataViewRegistry::UnregisterPropertySelector(const TSubclassOf<UPCGData> InDataClass)
{
	if (ensureMsgf(PropertySelectorRegistry.Find(InDataClass), TEXT("Attempted to unregister a data class that was not registered.")))
	{
		PropertySelectorRegistry.Remove(InDataClass);
	}
}

void FPCGDataViewRegistry::RegisterConverter(const TSubclassOf<UPCGData> InDataClass, const TSubclassOf<UPCGDataViewConverterBase> InConverterClass)
{
	if (TArray<TSubclassOf<UPCGDataViewConverterBase>>* ClassRegistry = ConverterRegistry.Find(InDataClass))
	{
		const int32 ConverterIndex = ClassRegistry->Find(InConverterClass);
		if (ensureMsgf(ConverterIndex == INDEX_NONE, TEXT("Cannot register multiple data view converters of the same type.")))
		{
			ConverterRegistry[InDataClass].Emplace(InConverterClass);
		}
	}
	else
	{
		ConverterRegistry.Add(InDataClass, {InConverterClass});
	}
}

void FPCGDataViewRegistry::UnregisterConverter(const TSubclassOf<UPCGData> InDataClass, const TSubclassOf<UPCGDataViewConverterBase> InConverterClass)
{
	if (ensureMsgf(ConverterRegistry.Find(InDataClass), TEXT("Attempted to unregister a data class that was not registered.")))
	{
		if (ensureMsgf(INDEX_NONE != ConverterRegistry[InDataClass].Find(InConverterClass), TEXT("Attempted to unregister a converter that was not registered.")))
		{
			ConverterRegistry[InDataClass].Remove(InConverterClass);
			if (ConverterRegistry[InDataClass].IsEmpty())
			{
				ConverterRegistry.Remove(InDataClass);
			}
		}
	}
}

const IPCGDataViewPropertySelector* FPCGDataViewRegistry::GetPropertySelector(const FPCGDataView& InDataView) const
{
	if (InDataView.IsValid())
	{
		// Climb the class hierarchy.
		UClass* DataClass = InDataView.ViewedData->GetClass();
		while (DataClass)
		{
			if (const TUniquePtr<const IPCGDataViewPropertySelector>* PropertySelectorPtr = PropertySelectorRegistry.Find(DataClass); PropertySelectorPtr && PropertySelectorPtr->IsValid())
			{
				return PropertySelectorPtr->Get();
			}
			else
			{
				DataClass = DataClass->GetSuperClass();
			}
		}
	}

	return nullptr;
}

const UPCGDataViewConverterBase* FPCGDataViewRegistry::GetConverter(const FPCGDataView& InDataView, TSubclassOf<UPCGDataViewConverterBase> InConverterClass) const
{
	if (InDataView.IsValid())
	{
		// Climb the class hierarchy.
		UClass* DataClass = InDataView.ViewedData->GetClass();
		while (DataClass && DataClass != UPCGData::StaticClass()) // @todo_pcg: The base converter should work for UPCGData
		{
			if (const TArray<TSubclassOf<UPCGDataViewConverterBase>>* ConverterClassArray = ConverterRegistry.Find(DataClass))
			{
				for (TSubclassOf<UPCGDataViewConverterBase> ConverterClass : *ConverterClassArray)
				{
					if (ConverterClass == InConverterClass)
					{
						return GetDefault<UPCGDataViewConverterBase>(ConverterClass);
					}
				}
			}

			DataClass = DataClass->GetSuperClass();
		}
	}

	return nullptr;
}
