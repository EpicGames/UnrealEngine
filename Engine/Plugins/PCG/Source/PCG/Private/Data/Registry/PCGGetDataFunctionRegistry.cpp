// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Registry/PCGGetDataFunctionRegistry.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGData.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionStateInterface.h"
#include "Compute/PCGComputeCommon.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGCollisionShapeData.h"
#include "Data/PCGIntersectionData.h"
#include "Data/PCGLandscapeData.h"
#include "Data/PCGLandscapeSplineData.h"
#include "Data/PCGPrimitiveData.h"
#include "Data/PCGProjectionData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGUnionData.h"
#include "Data/PCGVirtualTextureData.h"
#include "Data/PCGVolumeData.h"
#include "Data/Registry/PCGDataFunctionRegistryHelpers.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGHelpers.h"

#include "LandscapeComponent.h"
#include "LandscapeProxy.h"
#include "LandscapeSplinesComponent.h"
#include "Components/ActorComponent.h"
#include "Components/BillboardComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "Components/ShapeComponent.h"
#include "Components/SplineComponent.h"
#include "Components/SplineMeshComponent.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Volume.h"
#include "VT/RuntimeVirtualTexture.h"

#define LOCTEXT_NAMESPACE "PCGGetDataFunctionRegistry"

FPCGGetDataFunctionRegistry::FFunctionHandle FPCGGetDataFunctionRegistry::RegisterDataFromActorFunction(const FPCGGetDataFunctionRegistry::FDataFromActorFunction& InFunction)
{
	check(IsInGameThread());
	ActorParsingFunctions.Emplace(InFunction, NextFunctionHandle);
	return NextFunctionHandle++;
}

void FPCGGetDataFunctionRegistry::UnregisterDataFromActorFunction(FPCGGetDataFunctionRegistry::FFunctionHandle InFunctionHandle)
{
	check(IsInGameThread());
	int FunctionIndex = ActorParsingFunctions.IndexOfByPredicate([InFunctionHandle](const TPair<FDataFromActorFunction, FFunctionHandle>& InPair) { return InPair.Value == InFunctionHandle; });
	if (FunctionIndex != INDEX_NONE)
	{
		ActorParsingFunctions.RemoveAt(FunctionIndex);
	}
}

FPCGGetDataFunctionRegistry::FFunctionHandle FPCGGetDataFunctionRegistry::RegisterDataFromComponentFunction(const FPCGGetDataFunctionRegistry::FDataFromComponentFunction& InFunction)
{
	check(IsInGameThread());
	ComponentParsingFunctions.Emplace(InFunction, NextFunctionHandle);
	return NextFunctionHandle++;
}

void FPCGGetDataFunctionRegistry::UnregisterDataFromComponentFunction(FPCGGetDataFunctionRegistry::FFunctionHandle InFunctionHandle)
{
	check(IsInGameThread());
	int FunctionIndex = ComponentParsingFunctions.IndexOfByPredicate([InFunctionHandle](const TPair<FDataFromComponentFunction, FFunctionHandle>& InPair) { return InPair.Value == InFunctionHandle; });
	if (FunctionIndex != INDEX_NONE)
	{
		ComponentParsingFunctions.RemoveAt(FunctionIndex);
	}
}

int FPCGGetDataFunctionRegistry::GetDataFromActor(FPCGContext* InContext, const FPCGGetDataFunctionRegistryParams& InParams, AActor* InActor, FPCGGetDataFunctionRegistryOutput& Output) const
{
	int DataCountBefore = Output.Collection.TaggedData.Num();

	// Try custom getters first
	for (const TPair<FDataFromActorFunction, FFunctionHandle>& ActorParsingPair : ActorParsingFunctions)
	{
		if (ActorParsingPair.Key(InContext, InParams, InActor, Output))
		{
			break;
		}
	}

	if (Output.Collection.TaggedData.Num() != DataCountBefore)
	{
		return Output.Collection.TaggedData.Num() - DataCountBefore;
	}

	// Default actor parsing, if any
	return DefaultDataFromActor(InContext, InParams, InActor, Output);
}

int FPCGGetDataFunctionRegistry::DefaultDataFromActor(FPCGContext* InContext, const FPCGGetDataFunctionRegistryParams& InParams, AActor* Actor, FPCGGetDataFunctionRegistryOutput& Output) const
{
	check(Actor);

	// Early out if the actor gets rejected by the component selector
	if (InParams.ComponentSelector && !InParams.ComponentSelector->FilterActor(Actor))
	{
		return 0;
	}

	const EPCGDataType InDataFilter = InParams.DataTypeFilter;
	const bool bParseActor = InParams.bParseActor;
	const IPCGGraphExecutionSource* Source = InParams.Source;

	auto NameTagsToStringTags = [](const FName& InName) { return InName.ToString(); };
	TSet<FString> ActorTags;
	Algo::Transform(Actor->Tags, ActorTags, NameTagsToStringTags);

	// Fill in collection based on the data on the given actor.
	// Some actor types we will forego full parsing to build strictly on the actor existence, such as partition actors, volumes and landscape
	// TODO: review the !bParseActor cases - it might make sense to have just a point for a partition actor, even if we preintersect it.
	APCGPartitionActor* PartitionActor = (!!(InDataFilter & EPCGDataType::Spatial) || !!(InDataFilter & EPCGDataType::Volume)) ? Cast<APCGPartitionActor>(Actor) : nullptr;
	ALandscapeProxy* LandscapeActor = !!(InDataFilter & EPCGDataType::Landscape) ? Cast<ALandscapeProxy>(Actor) : nullptr;
	AVolume* VolumeActor = !!(InDataFilter & EPCGDataType::Volume) ? Cast<AVolume>(Actor) : nullptr;
	if (!bParseActor && !!(InDataFilter & EPCGDataType::Point))
	{
		UPCGBasePointData* Data = FPCGContext::NewPointData_AnyThread(InContext);
		Data->InitializeFromActor(Actor, &Output.bSanitizedTagAttributeNames);

		FPCGTaggedData& TaggedData = Output.Collection.TaggedData.Emplace_GetRef();
		TaggedData.Data = Data;
		TaggedData.Tags = ActorTags;
	}
	else if (InDataFilter == EPCGDataType::Point)
	{
		// Implementation note: when we're asking explicitly for point data, this behaves like the 'level to asset' workflow
		// but operates on all primitive components (not just SMCs).
		UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(InContext);
		UPCGMetadata* PointMetadata = PointData->MutableMetadata();
		PointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::MetadataEntry | EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax | EPCGPointNativeProperties::Seed);

		FPCGMetadataAttribute<FSoftObjectPath>* MeshAttribute = PointMetadata->CreateAttribute<FSoftObjectPath>(PCGLevelToAssetConstants::MeshAttributeName, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
		FPCGMetadataAttribute<FSoftObjectPath>* SkeletalMeshAttribute = PointMetadata->CreateAttribute<FSoftObjectPath>(PCGLevelToAssetConstants::SkeletalMeshAttributeName, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
		FPCGMetadataAttribute<FSoftObjectPath>* MaterialAttribute = PointMetadata->CreateAttribute<FSoftObjectPath>(PCGLevelToAssetConstants::MaterialAttributeName, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
		FPCGMetadataAttribute<FSoftObjectPath>* ActorReferenceAttribute = PointMetadata->CreateAttribute<FSoftObjectPath>(PCGAttributeNameConstants::ActorReferenceAttribute, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);
		FPCGMetadataAttribute<FSoftObjectPath>* ComponentReferenceAttribute = PointMetadata->CreateAttribute<FSoftObjectPath>(PCGAttributeNameConstants::ComponentReferenceAttribute, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/true);

		TMap<FName, FPCGMetadataAttributeBase*> TagToAttributeMap;
		TSet<FName> SanitizedAttributeNames;

		FPCGActorMeshParsingContext Context;
		Context.PointData = PointData;
		Context.PointMetadata = PointMetadata;
		Context.MeshAttribute = MeshAttribute;
		Context.SkeletalMeshAttribute = SkeletalMeshAttribute;
		Context.MaterialAttribute = MaterialAttribute;
		Context.ActorReferenceAttribute = ActorReferenceAttribute;
		Context.ComponentReferenceAttribute = ComponentReferenceAttribute;
		Context.TagToAttributeMap = &TagToAttributeMap;
		Context.SanitizedAttributeNames = &SanitizedAttributeNames;
		Context.bOnlyMeshComponents = false;
		Context.bIgnorePCGCreatedComponents = InParams.bIgnorePCGGeneratedComponents;

		if (Context.ParseActorComponents(Actor) == 0)
		{
			// Fallback on standard single point per actor path
			PointData->AddSinglePointFromActor(Actor, &Output.bSanitizedTagAttributeNames);
		}

		FPCGTaggedData& TaggedData = Output.Collection.TaggedData.Emplace_GetRef();
		TaggedData.Data = PointData;
		TaggedData.Tags = ActorTags;
	}
	else if (PartitionActor)
	{
		// If we have a Partition Actor, the execution source must be a PCG Component.
		const UPCGComponent* Component = Cast<UPCGComponent>(Source);

		// Invalid processing otherwise because of the this usage
		if (!Component || Component->GetOwner() != Actor)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidComponent", "Could not get data from actor {0}, execution source is invalid."), FText::FromString(PartitionActor->GetActorNameOrLabel())), InContext);
			return 0;
		}

		UPCGVolumeData* VolumeData = FPCGContext::NewObject_AnyThread<UPCGVolumeData>(InContext);
		UPCGSpatialData* Result = VolumeData;
		if (InDataFilter == EPCGDataType::Volume)
		{
			VolumeData->Initialize(PCGHelpers::GetGridBounds(Actor, Component));
		}
		else
		{
			VolumeData->Initialize(PartitionActor->GetFixedBounds());

			if (Component && PartitionActor->GetOriginalComponent(Component))
			{
				PCG_EXECUTION_CACHE_VALIDATION_CREATE_ORIGINAL_SCOPE(const_cast<UPCGComponent*>(Component));
				// Important note: we do NOT call the collection version here, as we want to have a union if that's the case
				const UPCGSpatialData* OriginalComponentSpatialData = Cast<const UPCGSpatialData>(Component->GetOriginalActorPCGData());

				if (OriginalComponentSpatialData)
				{
					Result = Result->IntersectWith(nullptr, OriginalComponentSpatialData);
				}
			}
		}

		FPCGTaggedData& TaggedData = Output.Collection.TaggedData.Emplace_GetRef();
		TaggedData.Data = Result;
		// No need to keep partition actor tags, though we might want to push PCG grid GUID at some point
	}
	else if (LandscapeActor)
	{
		UPCGLandscapeData* Data = FPCGContext::NewObject_AnyThread<UPCGLandscapeData>(InContext);
		const UPCGGraph* PCGGraph = Source ? Source->GetExecutionState().GetGraph() : nullptr;

		FPCGLandscapeDataProps LandscapeDataProps;
		LandscapeDataProps.bGetHeightOnly = false;
		LandscapeDataProps.bGetLayerWeights = (!PCGGraph || PCGGraph->bLandscapeUsesMetadata);
		LandscapeDataProps.bSampleVirtualTextures = false;

		Data->Initialize({ LandscapeActor }, PCGHelpers::GetGridBounds(Actor, Source), LandscapeDataProps);

		FPCGTaggedData& TaggedData = Output.Collection.TaggedData.Emplace_GetRef();
		TaggedData.Data = Data;
		TaggedData.Tags = ActorTags;
	}
	else if (VolumeActor)
	{
		UPCGVolumeData* Data = FPCGContext::NewObject_AnyThread<UPCGVolumeData>(InContext);
		Data->Initialize(VolumeActor);

		FPCGTaggedData& TaggedData = Output.Collection.TaggedData.Emplace_GetRef();
		TaggedData.Data = Data;
		TaggedData.Tags = ActorTags;
	}
	else
	{
		// Use component-based approach
		int AddedData = 0;
		TInlineComponentArray<UActorComponent*> ActorComponents;
		Actor->GetComponents(ActorComponents);

		for (UActorComponent* ActorComponent : ActorComponents)
		{
			AddedData += GetDataFromComponent(InContext, InParams, ActorComponent, Output);
		}
		
		// IMPLEMENTATION NOTE: API BREAK HERE, we normally would have returned a single point here if there were no valid components
		return AddedData;
	}

	return 1;
}

int FPCGGetDataFunctionRegistry::GetDataFromComponent(FPCGContext* InContext, const FPCGGetDataFunctionRegistryParams& InParams, UActorComponent* InComponent, FPCGGetDataFunctionRegistryOutput& Output) const
{
	check(InComponent);

	if (InParams.ComponentSelector && !InParams.ComponentSelector->FilterComponent(InComponent))
	{
		return 0;
	}

	int DataCountBefore = Output.Collection.TaggedData.Num();

	for (const TPair<FDataFromComponentFunction, FFunctionHandle>& ComponentParsingPair : ComponentParsingFunctions)
	{
		if (ComponentParsingPair.Key(InContext, InParams, InComponent, Output))
		{
			break;
		}
	}

	if (Output.Collection.TaggedData.Num() != DataCountBefore)
	{
		return Output.Collection.TaggedData.Num() - DataCountBefore;
	}

	// Default component parsing, if any
	return DefaultDataFromComponent(InContext, InParams, InComponent, Output);
}

int FPCGGetDataFunctionRegistry::DefaultDataFromComponent(FPCGContext* InContext, const FPCGGetDataFunctionRegistryParams& InParams, UActorComponent* InComponent, FPCGGetDataFunctionRegistryOutput& Output) const
{
	check(InComponent);

	if (InParams.bIgnorePCGGeneratedComponents && InComponent->ComponentTags.Contains(PCGHelpers::DefaultPCGTag))
	{
		return 0;
	}

	auto NameTagsToStringTags = [](const FName& InName) { return InName.ToString(); };
	const EPCGDataType InDataFilter = InParams.DataTypeFilter;

	// Landscape spline component
	// Splines component
	// Shape component
	// Primitive component, but ignore billboard components
	// RuntimeVirtualTexture component
	FPCGTaggedData TaggedData;

	if (!TaggedData.Data && !!(InDataFilter & EPCGDataType::Spline))
	{
		if (ULandscapeSplinesComponent* LandscapeSplineComponent = Cast<ULandscapeSplinesComponent>(InComponent))
		{
			UPCGLandscapeSplineData* SplineData = FPCGContext::NewObject_AnyThread<UPCGLandscapeSplineData>(InContext);
			SplineData->Initialize(LandscapeSplineComponent);
			TaggedData.Data = SplineData;
		}
		else if (USplineComponent* SplineComponent = Cast<USplineComponent>(InComponent); SplineComponent && SplineComponent->GetNumberOfSplinePoints() > 0)
		{
			UPCGSplineData* SplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(InContext);
			SplineData->Initialize(SplineComponent);
			TaggedData.Data = SplineData;
		}
	}

	if (!TaggedData.Data && !!(InDataFilter & EPCGDataType::Primitive))
	{
		if (UShapeComponent* ShapeComponent = Cast<UShapeComponent>(InComponent); ShapeComponent && UPCGCollisionShapeData::IsSupported(ShapeComponent))
		{
			UPCGCollisionShapeData* ShapeData = FPCGContext::NewObject_AnyThread<UPCGCollisionShapeData>(InContext);
			ShapeData->Initialize(ShapeComponent);
			TaggedData.Data = ShapeData;
		}
		else if (UBillboardComponent* BillboardComponent = Cast<UBillboardComponent>(InComponent))
		{
			// Exception: skip the billboard component
		}
		else if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InComponent))
		{
			UPCGPrimitiveData* PrimitiveData = FPCGContext::NewObject_AnyThread<UPCGPrimitiveData>(InContext);
			PrimitiveData->Initialize(PrimitiveComponent);
			TaggedData.Data = PrimitiveData;
		}
	}

	if (!TaggedData.Data && !!(InDataFilter & EPCGDataType::VirtualTexture))
	{
		if (URuntimeVirtualTextureComponent* VirtualTextureComponent = Cast<URuntimeVirtualTextureComponent>(InComponent))
		{
			UPCGVirtualTextureData* VirtualTextureData = FPCGContext::NewObject_AnyThread<UPCGVirtualTextureData>(InContext);
			VirtualTextureData->Initialize(VirtualTextureComponent);
			TaggedData.Data = VirtualTextureData;

			if (const URuntimeVirtualTexture* VirtualTexture = VirtualTextureData->GetRuntimeVirtualTexture())
			{
				FString VirtualTextureName;
				VirtualTexture->GetName(VirtualTextureName);

				const UEnum* MaterialTypeEnum = StaticEnum<ERuntimeVirtualTextureMaterialType>();
				check(MaterialTypeEnum);

				const FString MaterialTypeString = MaterialTypeEnum->GetNameStringByValue(static_cast<int64>(VirtualTexture->GetMaterialType()));

				TaggedData.Tags.Add(PCGComputeHelpers::GetPrefixedDataLabel(VirtualTextureName));
				TaggedData.Tags.Add(PCGComputeHelpers::GetPrefixedDataLabel(MaterialTypeString));
			}

		}
	}

	if (TaggedData.Data)
	{
		Algo::Transform(InComponent->ComponentTags, TaggedData.Tags, NameTagsToStringTags);

		if (InParams.bAddActorTags && InComponent->GetOwner())
		{
			TSet<FString> ActorTags;
			Algo::Transform(InComponent->GetOwner()->Tags, ActorTags, NameTagsToStringTags);
			TaggedData.Tags.Append(ActorTags);
		}

		Output.Collection.TaggedData.Add(MoveTemp(TaggedData));
		return 1;
	}	
	else
	{
		return 0;
	}
}

#undef LOCTEXT_NAMESPACE