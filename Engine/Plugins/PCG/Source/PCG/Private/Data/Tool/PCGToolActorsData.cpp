// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Tool/PCGToolActorsData.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"

#if WITH_EDITOR
#include "InteractiveTool.h"
#include "ActorFactories/ActorFactory.h"
#endif


void FPCGInteractiveToolWorkingData_Actors::InitializeRuntimeElementData(FPCGContext* InContext) const
{
	FPCGInteractiveToolWorkingData::InitializeRuntimeElementData(InContext);

	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	FPCGTaggedData& Output = Outputs.Emplace_GetRef();

#if WITH_EDITOR
	// Implementation note - we duplicate here to make sure we are not preserving the parent, because it might still change, which would cause issues.
	check(IsInGameThread());
	Output.Data = Cast<UPCGData>(StaticDuplicateObject(SelectedActorsData, GetTransientPackage()));
#else
	Output.Data = SelectedActorsData;
#endif

}

bool FPCGInteractiveToolWorkingData_Actors::IsValid() const
{
	if (!SelectedActorsData)
	{
		return false;
	}

	const UPCGMetadata* Metadata = SelectedActorsData->ConstMetadata();
	check(Metadata);
	const FPCGMetadataAttribute<FSoftObjectPath>* ActorReference = Metadata->GetConstTypedAttribute<FSoftObjectPath>(PCGPointDataConstants::ActorReferenceAttribute);
	
	// Should have at least one actor reference to be valid.
	return ActorReference && ActorReference->GetNumberOfEntries() > 0;
}

UPCGParamData* FPCGInteractiveToolWorkingData_Actors::GetParamData() const
{
	return SelectedActorsData;
}

#if WITH_EDITOR
void FPCGInteractiveToolWorkingData_Actors::InitializeInternal(const FPCGInteractiveToolWorkingDataContext& Context)
{
	SelectedActorsData = NewObject<UPCGParamData>();
}
#endif  //WITH_EDITOR
