// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Tool/PCGToolBaseData.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"

#include "GameFramework/Actor.h"

#if WITH_EDITOR
#include "InteractiveTool.h"
#endif

bool FPCGInteractiveToolWorkingData::IsInitialized() const
{
	return WorkingDataIdentifier.IsValid() && !WorkingDataIdentifier.IsNone();
}

bool FPCGInteractiveToolWorkingData::IsValid() const
{
	return true;
}

void FPCGInteractiveToolWorkingData::InitializeRuntimeElementData(FPCGContext* InContext) const
{
}

#if WITH_EDITOR
void FPCGInteractiveToolWorkingData::Initialize(const FPCGInteractiveToolWorkingDataContext& Context)
{
	ensureMsgf(Context.OwningPCGComponent.IsValid() && Context.OwningActor.IsValid(), TEXT("Owning PCG Component or Owning Actor invalid! This should never happen so an assumption was hurt."));
	ensureMsgf(Context.WorkingDataIdentifier.IsNone() == false && Context.WorkingDataIdentifier.IsValid(), TEXT("WorkingDataIdentifier needs to be set when initializing tool data."));
	WorkingDataIdentifier = Context.WorkingDataIdentifier;
	
	InitializeInternal(Context);
}

void FPCGInteractiveToolWorkingData::InitializeInternal(const FPCGInteractiveToolWorkingDataContext& Context)
{
}

void FPCGInteractiveToolWorkingData::OnToolStart(const FPCGInteractiveToolWorkingDataContext& Context)
{
}

void FPCGInteractiveToolWorkingData::OnToolApply(const FPCGInteractiveToolWorkingDataContext& Context)
{
	GeneratedResources.Finalize();
	OnToolShutdown(Context);
}

void FPCGInteractiveToolWorkingData::OnToolCancel(const FPCGInteractiveToolWorkingDataContext& Context)
{
	GeneratedResources.Cleanup();
	OnToolShutdown(Context);
}

void FPCGInteractiveToolWorkingDataGeneratedResources::Finalize()
{
	for (TWeakObjectPtr<UActorComponent> GeneratedComponent : GeneratedComponents)
	{
		if (GeneratedComponent.IsValid())
		{
			GeneratedComponent->ClearFlags(RF_Transient);
		}
	}

	GeneratedComponents.Empty();
}

void FPCGInteractiveToolWorkingDataGeneratedResources::Cleanup()
{
	for (TWeakObjectPtr<UActorComponent> GeneratedComponent : GeneratedComponents)
	{
		if (GeneratedComponent.IsValid())
		{
			if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(GeneratedComponent))
			{
				PCGComponent->Cleanup(true);
			}

			GeneratedComponent->DestroyComponent();
		}
	}

	GeneratedComponents.Empty();
}

#endif // WITH_EDITOR
