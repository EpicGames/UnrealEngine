// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Tool/PCGToolPointData.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Helpers/PCGHelpers.h"
#include "Data/PCGPointArrayData.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"

#include "Logging/StructuredLog.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

#if WITH_EDITOR
void FPCGInteractiveToolWorkingData_PointArrayData::InitializeInternal(const FPCGInteractiveToolWorkingDataContext& Context)
{
	GeneratedPointData = NewObject<UPCGPointArrayData>(Context.OwningPCGComponent.Get());

	if (Context.OwningActor.IsValid() && !PCGHelpers::GetActorBounds(Context.OwningActor.Get()).IsValid)
	{
		// Create a dummy box component so we can have proper bounds on the actor (otherwise PCG will fail it).
		BoxComponent = NewObject<UBoxComponent>(Context.OwningActor.Get());

		// Transient until Applied, but always transactional
		BoxComponent->SetFlags(RF_Transient | RF_Transactional);

		if (!Context.DataInstanceIdentifier.IsNone())
		{
			BoxComponent->ComponentTags.Add(Context.DataInstanceIdentifier);
		}

		BoxComponent->SetCollisionObjectType(ECC_WorldStatic);
		BoxComponent->SetCollisionResponseToAllChannels(ECR_Ignore);
		BoxComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		BoxComponent->SetGenerateOverlapEvents(false);
		BoxComponent->SetBoxExtent(FVector::OneVector);
		BoxComponent->SetSimulatePhysics(false);

		// If there wasn't a root component or it's the default one, we can certainly replace it.
		if (!Context.OwningActor->GetRootComponent() || Context.OwningActor->GetRootComponent()->GetFName() == USceneComponent::GetDefaultSceneRootVariableName())
		{
			USceneComponent* DefaultSceneRootComponent = Context.OwningActor->GetRootComponent();

			BoxComponent->SetMobility(EComponentMobility::Static);
			Context.OwningActor->SetRootComponent(BoxComponent.Get());

			if (DefaultSceneRootComponent)
			{
				DefaultSceneRootComponent->DestroyComponent();
			}
		}
		else
		{
			BoxComponent->SetMobility(Context.OwningActor->GetRootComponent()->GetMobility());
			BoxComponent->AttachToComponent(Context.OwningActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, false));
		}

		BoxComponent->RegisterComponent();
		Context.OwningActor->AddInstanceComponent(BoxComponent.Get());

		GeneratedResources.GeneratedComponents.Add(BoxComponent.Get());
	}
}

void FPCGInteractiveToolWorkingData_PointArrayData::OnToolStart(const FPCGInteractiveToolWorkingDataContext& Context)
{
	if (!OnToolStartPointArray)
	{
		OnToolStartPointArray = Cast<UPCGPointArrayData>(StaticDuplicateObject(GeneratedPointData, GetTransientPackage()));
	}

	Super::OnToolStart(Context);
}

void FPCGInteractiveToolWorkingData_PointArrayData::OnToolApply(const FPCGInteractiveToolWorkingDataContext& Context)
{
	// Update bounds if needed (e.g. they were originally created from this tool.)
	if (BoxComponent.IsValid() && GeneratedPointData)
	{
		FBox PointBounds = GeneratedPointData->GetBounds();
		if (PointBounds.IsValid)
		{
			BoxComponent->SetRelativeLocation(PointBounds.GetCenter());

			FVector NewExtent = PointBounds.GetExtent().ComponentMax(FVector::OneVector);
			BoxComponent->SetBoxExtent(NewExtent);
		}
	}

	Super::OnToolApply(Context);
}

void FPCGInteractiveToolWorkingData_PointArrayData::OnToolCancel(const FPCGInteractiveToolWorkingDataContext& Context)
{
	if (OnToolStartPointArray != nullptr)
	{
		// We'd expect no relationship between the generated point data on tool cancel, but we'll make sure this is correct here.
		GeneratedPointData = Cast<UPCGPointArrayData>(StaticDuplicateObject(OnToolStartPointArray, Context.OwningPCGComponent.Get()));

		FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(GeneratedPointData, EmptyPropertyChangedEvent);
	}

	Super::OnToolCancel(Context);
}

void FPCGInteractiveToolWorkingData_PointArrayData::OnToolShutdown(const FPCGInteractiveToolWorkingDataContext& Context)
{
	if (OnToolStartPointArray)
	{
		OnToolStartPointArray->MarkAsGarbage();
		OnToolStartPointArray = nullptr;
	}

	Super::OnToolShutdown(Context);
}

void FPCGInteractiveToolWorkingData_PointArrayData::OnResetToolDataRequested(const FPCGInteractiveToolWorkingDataContext& Context)
{
	if (GeneratedPointData)
	{
		GeneratedPointData->SetNumPoints(0);

		FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(GeneratedPointData, EmptyPropertyChangedEvent);
	}
}
#endif // WITH_EDITOR

void FPCGInteractiveToolWorkingData_PointArrayData::InitializeRuntimeElementData(FPCGContext* Context) const
{
	Super::InitializeRuntimeElementData(Context);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	FPCGTaggedData& Output = Outputs.Emplace_GetRef();

	if (GeneratedPointData)
	{
#if WITH_EDITOR
		// Implementation note - we duplicate here to make sure we are not preserving the parent, because it might still change, which would cause issues.
		check(IsInGameThread());
		Output.Data = Cast<UPCGData>(StaticDuplicateObject(GeneratedPointData, GetTransientPackage()));
#else
		Output.Data = GeneratedPointData->DuplicateData(Context);
#endif

#if WITH_EDITOR
		FPCGSelectionKey Key = FPCGSelectionKey::CreateFromObjectPtr(GeneratedPointData);
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, MoveTemp(Key), false);
#endif
	}
}

bool FPCGInteractiveToolWorkingData_PointArrayData::IsValid() const
{
	return GeneratedPointData != nullptr;
}

UPCGPointArrayData* FPCGInteractiveToolWorkingData_PointArrayData::GetPointArrayData() const
{
	return GeneratedPointData;
}
