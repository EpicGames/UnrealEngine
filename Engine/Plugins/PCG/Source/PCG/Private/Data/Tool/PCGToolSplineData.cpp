// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Tool/PCGToolSplineData.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"

#include "Components/SplineComponent.h"
#include "Engine/Engine.h"
#include "Logging/StructuredLog.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

USplineComponent* FPCGInteractiveToolWorkingData_Spline::GetSplineComponent() const
{
	return SplineComponent.Get();
}

#if WITH_EDITOR
void FPCGInteractiveToolWorkingData_Spline::InitializeInternal(const FPCGInteractiveToolWorkingDataContext& Context)
{
	SplineComponent = FindOrGenerateSplineComponent(Context);
}

void FPCGInteractiveToolWorkingData_Spline::OnToolStart(const FPCGInteractiveToolWorkingDataContext& Context)
{
	Super::OnToolStart(Context);

	if (SplineComponent.IsValid() && !OnToolStartSplineComponent)
	{
		OnToolStartSplineComponent = Cast<USplineComponent>(StaticDuplicateObject(SplineComponent.Get(), GetTransientPackage()));
		OnToolStartSplineTransform = SplineComponent->GetComponentTransform();
	}
}

void FPCGInteractiveToolWorkingData_Spline::OnToolCancel(const FPCGInteractiveToolWorkingDataContext& Context)
{
	// If the Spline Component is valid and isn't a generated component, we revert its state.
	if (SplineComponent.IsValid() && GeneratedResources.GeneratedComponents.Contains(SplineComponent.Get()) == false)
	{
		if (OnToolStartSplineComponent)
		{
			USplineComponent* SplineComponentRaw = SplineComponent.Get();
			SplineComponent.Reset();

			// Restore the old spline
			AActor* Owner = SplineComponentRaw->GetOwner();
			USceneComponent* RootComponent = SplineComponentRaw->GetAttachmentRoot();
			const bool bIsRoot = SplineComponentRaw == RootComponent;

			const FString ComponentName = SplineComponentRaw->GetName();

			// Rename component to allow the old spline to retrieve its name
			// It will also remove/add them to InstanceComponent (done in UActorComponent::PostRename)
			SplineComponentRaw->Rename(nullptr, GetTransientPackage());
			OnToolStartSplineComponent->Rename(*ComponentName, Owner);

			if (ensure(Owner) && bIsRoot)
			{
				Owner->SetRootComponent(OnToolStartSplineComponent);
			}
			else if (ensure(RootComponent) && !bIsRoot)
			{
				OnToolStartSplineComponent->SetupAttachment(RootComponent);
			}
			else
			{
				// Owner and/or RootComponent is/are dead, we can no longer do anything
				SplineComponentRaw->DestroyComponent();
				OnToolStartSplineComponent = nullptr;
				return;
			}

			OnToolStartSplineComponent->RegisterComponent();

			// @todo_pcg: It seems we can end up in a very bad state where the splines are invalid (the number
			// of points in position, rotation and scale are not the same). Since we do not have a clear repro,
			// try to fail gracefully.
			const FInterpCurveVector& Positions = OnToolStartSplineComponent->GetSplinePointsPosition();
			const FInterpCurveQuat& Rotations = OnToolStartSplineComponent->GetSplinePointsRotation();
			const FInterpCurveVector& Scales = OnToolStartSplineComponent->GetSplinePointsScale();
			if (!ensure(Positions.Points.Num() == Rotations.Points.Num() && Positions.Points.Num() == Scales.Points.Num()))
			{
				OnToolStartSplineComponent->ResetToDefault();
			}

			OnToolStartSplineComponent->UpdateSpline();

			OnToolStartSplineComponent->SetWorldTransform(OnToolStartSplineTransform);

			FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(OnToolStartSplineComponent.Get(), EmptyPropertyChangedEvent);

			// Then reset the hold on the component
			OnToolStartSplineComponent = nullptr;
			SplineComponentRaw->DestroyComponent();
		}
	}

	// This will delete the spline component if it was generated
	Super::OnToolCancel(Context);
}

void FPCGInteractiveToolWorkingData_Spline::OnToolShutdown(const FPCGInteractiveToolWorkingDataContext& Context)
{
	if (OnToolStartSplineComponent)
	{
		OnToolStartSplineComponent->MarkAsGarbage();
		OnToolStartSplineComponent = nullptr;
	}

	Super::OnToolShutdown(Context);
}

void FPCGInteractiveToolWorkingData_Spline::OnResetToolDataRequested(const FPCGInteractiveToolWorkingDataContext& Context)
{
	if (SplineComponent.IsValid())
	{
		SplineComponent->ClearSplinePoints(/*bUpdateSpline=*/true);

		FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(SplineComponent.Get(), EmptyPropertyChangedEvent);
	}
}

USplineComponent* FPCGInteractiveToolWorkingData_Spline::FindOrGenerateSplineComponent(const FPCGInteractiveToolWorkingDataContext& Context)
{
	USplineComponent* MatchingSplineComponent = FindMatchingSplineComponent(Context);

	// If we didn't find any spline component to match with, we create it now
	if (MatchingSplineComponent == nullptr)
	{
		MatchingSplineComponent = GenerateMatchingSplineComponent(Context);
		GeneratedResources.GeneratedComponents.Add(MatchingSplineComponent);
	}

	ensureMsgf(MatchingSplineComponent != nullptr, TEXT("Matching Spline Component should always be found after initialization as it was either found or created."));
	return MatchingSplineComponent;
}

USplineComponent* FPCGInteractiveToolWorkingData_Spline::FindMatchingSplineComponent(const FPCGInteractiveToolWorkingDataContext& Context)
{
	TArray<USplineComponent*> AllSplineComponents;
	Context.OwningActor->GetComponents<USplineComponent>(AllSplineComponents, false);

	if (!Context.DataInstanceIdentifier.IsNone())
	{
		// When the DataInstanceIdentifier is set, we only care about the splines with the given tag
		AllSplineComponents.RemoveAll([DataInstanceIdentifier = Context.DataInstanceIdentifier](USplineComponent* Candidate)
			{
				return Candidate->ComponentHasTag(DataInstanceIdentifier) == false;
			});
	}

	USplineComponent* MatchingSplineComponent = AllSplineComponents.IsEmpty() ? nullptr : AllSplineComponents[0];

	if (AllSplineComponents.Num() > 1 && MatchingSplineComponent != nullptr)
	{
		UE_LOGFMT(LogPCG, Warning, "More than one spline component found. Choosing the first available: {0}", MatchingSplineComponent->GetName());
	}

	return MatchingSplineComponent;
}

USplineComponent* FPCGInteractiveToolWorkingData_Spline::GenerateMatchingSplineComponent(const FPCGInteractiveToolWorkingDataContext& Context)
{
	// Can't add a spline component to an actor that have no root.
	if (!Context.OwningActor.IsValid() || !Context.OwningActor->GetRootComponent())
	{
		return nullptr;
	}

	USplineComponent* MatchingSplineComponent = NewObject<USplineComponent>(Context.OwningActor.Get());
	MatchingSplineComponent->SetMobility(Context.OwningActor->GetRootComponent()->GetMobility());
	// Make sure we do not inherit from the actor rotation and scale
	MatchingSplineComponent->AttachToComponent(Context.OwningActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, false));

	// Transient until Applied, but always transactional
	MatchingSplineComponent->SetFlags(RF_Transient | RF_Transactional);

	if (Context.DataInstanceIdentifier.IsNone() == false)
	{
		MatchingSplineComponent->ComponentTags.Add(Context.DataInstanceIdentifier);
	}

	MatchingSplineComponent->ClearSplinePoints();
	MatchingSplineComponent->bSplineHasBeenEdited = true;

	// We set it to open or closed using OnInitialized based on the user property
	MatchingSplineComponent->SetClosedLoop(false);

	MatchingSplineComponent->RegisterComponent();
	Context.OwningActor->AddInstanceComponent(MatchingSplineComponent);

	return MatchingSplineComponent;
}
#endif // WITH_EDITOR

void FPCGInteractiveToolWorkingData_Spline::InitializeRuntimeElementData(FPCGContext* Context) const
{
	Super::InitializeRuntimeElementData(Context);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	if (SplineComponent.IsValid())
	{
		FPCGTaggedData& Output = Outputs.Emplace_GetRef();
		// We duplicate it so that PCG can take 'ownership'.
		// If not, it can't process references and caches properly
		// Might be relying on the fact it shouldn't access the same object over multiple generations
		// And if we recreate it every time it gets properly managed
		// If it gets dropped from cache, it's expected to be invalid
		UPCGSplineData* SplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
		SplineData->Initialize(SplineComponent.Get());
		Output.Data = SplineData;
#if WITH_EDITOR
		FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(SplineComponent.ToSoftObjectPath());
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, MoveTemp(Key), false);
#endif
	}
}

bool FPCGInteractiveToolWorkingData_Spline::IsValid() const
{
	return SplineComponent.IsValid();
}

#if WITH_EDITOR
USplineComponent* FPCGInteractiveToolWorkingData_SplineSurface::FindMatchingSplineComponent(const FPCGInteractiveToolWorkingDataContext& Context)
{
	TArray<USplineComponent*> AllClosedSplineComponents;
	Context.OwningActor->GetComponents<USplineComponent>(AllClosedSplineComponents, false);

	// We remove all spline components that are open loops since this working data struct is only for spline surfaces
	AllClosedSplineComponents.RemoveAll([](USplineComponent* Candidate)
		{
			return Candidate->IsClosedLoop() == false;
		});

	USplineComponent* MatchingSplineComponent = nullptr;
	if (Context.DataInstanceIdentifier.IsNone())
	{
		if (AllClosedSplineComponents.Num() > 0)
		{
			MatchingSplineComponent = AllClosedSplineComponents[0];
		}
	}
	else
	{
		// When the DataInstanceIdentifier is set, we only care about the closed splines with the given tag
		AllClosedSplineComponents.RemoveAll([DataInstanceIdentifier = Context.DataInstanceIdentifier](USplineComponent* Candidate)
			{
				return Candidate->ComponentHasTag(DataInstanceIdentifier) == false;
			});

		// If we find a matching spline component with the specific tag, use i
		if (AllClosedSplineComponents.Num() > 0)
		{
			MatchingSplineComponent = Cast<USplineComponent>(AllClosedSplineComponents[0]);
		}
	}

	if (AllClosedSplineComponents.Num() > 1 && MatchingSplineComponent != nullptr)
	{
		UE_LOGFMT(LogPCG, Warning, "More than one spline component found. Choosing the first available: {0}", MatchingSplineComponent->GetName());
	}

	return MatchingSplineComponent;
}

USplineComponent* FPCGInteractiveToolWorkingData_SplineSurface::GenerateMatchingSplineComponent(const FPCGInteractiveToolWorkingDataContext& Context)
{
	USplineComponent* MatchingSplineComponent = NewObject<USplineComponent>(Context.OwningActor.Get());
	MatchingSplineComponent->SetMobility(Context.OwningActor->GetRootComponent()->GetMobility());
	MatchingSplineComponent->SetupAttachment(Context.OwningActor->GetRootComponent());
	// Transient until Applied, but always transactional
	MatchingSplineComponent->SetFlags(RF_Transient | RF_Transactional);

	if (Context.DataInstanceIdentifier.IsNone() == false)
	{
		MatchingSplineComponent->ComponentTags.Add(Context.DataInstanceIdentifier);
	}

	MatchingSplineComponent->ClearSplinePoints();
	MatchingSplineComponent->bSplineHasBeenEdited = true;
	MatchingSplineComponent->SetClosedLoop(true);

	MatchingSplineComponent->RegisterComponent();
	Context.OwningActor->AddInstanceComponent(MatchingSplineComponent);

	return MatchingSplineComponent;
}
#endif // WITH_EDITOR