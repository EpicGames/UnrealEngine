// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/Tool/PCGToolVolumeData.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGCollisionShapeData.h"
#include "Data/PCGVolumeData.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"

#include "Components/BoxComponent.h"
#include "Components/BrushComponent.h"
#include "Engine/Engine.h"
#include "Engine/BrushBuilder.h"
#include "Logging/StructuredLog.h"
#include "GameFramework/Volume.h"

#if WITH_EDITOR
#include "ActorFactories/ActorFactory.h"
#endif

bool FPCGInteractiveToolWorkingData_Volume::IsValid() const
{
	return BrushComponent.IsValid() || BoxComponent.IsValid();
}

void FPCGInteractiveToolWorkingData_Volume::InitializeRuntimeElementData(FPCGContext* Context) const
{
	Super::InitializeRuntimeElementData(Context);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	if (!Volume && !BoxComponent.IsValid())
	{
		return;
	}

	FPCGTaggedData& Output = Outputs.Emplace_GetRef();

	if (Volume)
	{
		UPCGVolumeData* VolumeData = FPCGContext::NewObject_AnyThread<UPCGVolumeData>(Context);
		VolumeData->Initialize(Volume);
		Output.Data = VolumeData;

#if WITH_EDITOR
		FPCGSelectionKey Key = FPCGSelectionKey::CreateFromObjectPtr(Volume);
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, MoveTemp(Key), false);
#endif
	}
	else if (BoxComponent.IsValid())
	{
		UPCGCollisionShapeData* BoxData = FPCGContext::NewObject_AnyThread<UPCGCollisionShapeData>(Context);
		BoxData->Initialize(BoxComponent.Get());
		Output.Data = BoxData;

#if WITH_EDITOR
		FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(BoxComponent.ToSoftObjectPath());
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, MoveTemp(Key), false);
#endif
	}
}

#if WITH_EDITOR
void FPCGInteractiveToolWorkingData_Volume::InitializeInternal(const FPCGInteractiveToolWorkingDataContext& Context)
{
	if (AVolume* ActorAsVolume = Cast<AVolume>(Context.OwningActor.Get()))
	{
		Volume = ActorAsVolume;
		BrushComponent = FindOrCreateBrushComponent(Context);
	}
	else
	{
		BoxComponent = FindBoxComponent(Context);
	}
}

void FPCGInteractiveToolWorkingData_Volume::OnToolStart(const FPCGInteractiveToolWorkingDataContext& Context)
{
	Super::OnToolStart(Context);

	bool bUpdateTransform = false;

	if (BoxComponent.IsValid() && !OnToolStartBoxComponent)
	{
		OnToolStartBoxComponent = Cast<UBoxComponent>(StaticDuplicateObject(BoxComponent.Get(), GetTransientPackage()));
		bUpdateTransform = true;
	}

	if (BrushComponent.IsValid() && !OnToolStartBrushComponent)
	{
		OnToolStartBrushComponent = Cast<UBrushComponent>(StaticDuplicateObject(BrushComponent.Get(), GetTransientPackage()));
		bUpdateTransform = true;
	}

	if (Volume && Volume->BrushBuilder && !OnToolStartBrushBuilder)
	{
		OnToolStartBrushBuilder = DuplicateObject<UBrushBuilder>(Volume->BrushBuilder, GetTransientPackage());
		bUpdateTransform = true;
	}

	if (bUpdateTransform)
	{
		if (Volume)
		{
			OnToolStartActorTransform = Volume->GetActorTransform();
		}
		else if(BoxComponent.IsValid())
		{
			OnToolStartActorTransform = BoxComponent->GetOwner() ? BoxComponent->GetOwner()->GetActorTransform() : FTransform::Identity;
			OnToolStartBoxTransform = BoxComponent->GetComponentTransform();
		}
	}
}

void FPCGInteractiveToolWorkingData_Volume::OnToolCancel(const FPCGInteractiveToolWorkingDataContext& Context)
{
	if (Volume && BrushComponent.IsValid() && !GeneratedResources.GeneratedComponents.Contains(BrushComponent.Get()) && OnToolStartBrushComponent)
	{
		// Restore the brush component to its previous state
		UEngine::CopyPropertiesForUnrelatedObjects(OnToolStartBrushComponent, BrushComponent.Get());

		Volume->SetActorTransform(OnToolStartActorTransform);

		if (OnToolStartBrushBuilder && Volume->GetBrushComponent())
		{
			// Recreate the brush from the original actor
			UActorFactory::CreateBrushForVolumeActor(Volume.Get(), OnToolStartBrushBuilder);
		}

		FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(BrushComponent.Get(), EmptyPropertyChangedEvent);
	}
	else if (BoxComponent.IsValid() && !GeneratedResources.GeneratedComponents.Contains(BoxComponent.Get()) && OnToolStartBoxComponent)
	{
		// Restore the box component to its previous state
		UEngine::CopyPropertiesForUnrelatedObjects(OnToolStartBoxComponent, BoxComponent.Get());

		BoxComponent->SetWorldTransform(OnToolStartBoxTransform);

		FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(BoxComponent.Get(), EmptyPropertyChangedEvent);

		// Since we've changed the transform and the box might have been the root, we need to trigger a change on the actor.
		if (AActor* Owner = BoxComponent->GetOwner())
		{
			Owner->PostEditChange();
		}
	}

	Super::OnToolCancel(Context);
}

void FPCGInteractiveToolWorkingData_Volume::OnToolShutdown(const FPCGInteractiveToolWorkingDataContext& Context)
{
	if (OnToolStartBrushBuilder)
	{
		OnToolStartBrushBuilder->MarkAsGarbage();
		OnToolStartBrushBuilder = nullptr;
	}

	if (OnToolStartBrushComponent)
	{
		OnToolStartBrushComponent->MarkAsGarbage();
		OnToolStartBrushComponent = nullptr;
	}

	if (OnToolStartBoxComponent)
	{
		OnToolStartBoxComponent->MarkAsGarbage();
		OnToolStartBoxComponent = nullptr;
	}

	OnToolStartActorTransform = FTransform::Identity;
	OnToolStartBoxTransform = FTransform::Identity;

	Super::OnToolShutdown(Context);
}

UBrushComponent* FPCGInteractiveToolWorkingData_Volume::FindOrCreateBrushComponent(const FPCGInteractiveToolWorkingDataContext& Context)
{
	UBrushComponent* MatchingComponent = FindMatchingBrushComponent(Context);

	if (!MatchingComponent)
	{
		MatchingComponent = GenerateMatchingBrushComponent(Context);
		GeneratedResources.GeneratedComponents.Add(MatchingComponent);
	}

	ensureMsgf(MatchingComponent != nullptr, TEXT("Matching Brush Component should always be found after initialization as it was either found or created."));
	return MatchingComponent;
}

UBrushComponent* FPCGInteractiveToolWorkingData_Volume::FindMatchingBrushComponent(const FPCGInteractiveToolWorkingDataContext& Context)
{
	TArray<UBrushComponent*> AllBrushComponents;
	if (Context.OwningActor.IsValid())
	{
		Context.OwningActor->GetComponents<UBrushComponent>(AllBrushComponents, false);
	}

	UBrushComponent* MatchingComponent = nullptr;

	if (!Context.DataInstanceIdentifier.IsNone())
	{
		AllBrushComponents.RemoveAll([DataInstanceIdentifier = Context.DataInstanceIdentifier](UBrushComponent* Candidate)
		{
			return !Candidate || !Candidate->ComponentHasTag(DataInstanceIdentifier);
		});
	}

	MatchingComponent = AllBrushComponents.IsEmpty() ? nullptr : AllBrushComponents[0];

	if (AllBrushComponents.Num() > 1 && MatchingComponent != nullptr)
	{
		UE_LOGFMT(LogPCG, Warning, "More than one brush component found. Choosing the first available: {0}", MatchingComponent->GetName());
	}

	return MatchingComponent;
}

UBrushComponent* FPCGInteractiveToolWorkingData_Volume::GenerateMatchingBrushComponent(const FPCGInteractiveToolWorkingDataContext& Context)
{
	// Can't add a brush component to an actor that have no root.
	if (!Context.OwningActor.IsValid() || !Context.OwningActor->GetRootComponent())
	{
		return nullptr;
	}

	UBrushComponent* MatchingBrushComponent = NewObject<UBrushComponent>(Context.OwningActor.Get());
	// ABrush setup
	MatchingBrushComponent->Mobility = EComponentMobility::Static;
	MatchingBrushComponent->SetCanEverAffectNavigation(false);
	// AVolume setup
	MatchingBrushComponent->AlwaysLoadOnClient = true;
	MatchingBrushComponent->AlwaysLoadOnServer = true;
	// APCGVolume setup
	MatchingBrushComponent->SetCollisionObjectType(ECC_WorldStatic);
	MatchingBrushComponent->SetCollisionResponseToAllChannels(ECR_Ignore);
	MatchingBrushComponent->SetGenerateOverlapEvents(false);
	// Transient until applied, but always transactional.
	MatchingBrushComponent->SetFlags(RF_Transient | RF_Transactional);

	if (!Context.DataInstanceIdentifier.IsNone())
	{
		MatchingBrushComponent->ComponentTags.Add(Context.DataInstanceIdentifier);
	}

	MatchingBrushComponent->RegisterComponent();
	Context.OwningActor->AddInstanceComponent(MatchingBrushComponent);

	return MatchingBrushComponent;
}

UBoxComponent* FPCGInteractiveToolWorkingData_Volume::FindBoxComponent(const FPCGInteractiveToolWorkingDataContext& Context)
{
	TArray<UBoxComponent*> AllBoxComponents;
	if (Context.OwningActor.IsValid())
	{
		Context.OwningActor->GetComponents<UBoxComponent>(AllBoxComponents, false);
	}

	UBoxComponent* MatchingComponent = nullptr;

	if (!Context.DataInstanceIdentifier.IsNone())
	{
		AllBoxComponents.RemoveAll([DataInstanceIdentifier = Context.DataInstanceIdentifier](UBoxComponent* Candidate)
		{
			return !Candidate || !Candidate->ComponentHasTag(DataInstanceIdentifier);
		});
	}

	MatchingComponent = AllBoxComponents.IsEmpty() ? nullptr : AllBoxComponents[0];

	if (AllBoxComponents.Num() > 1 && MatchingComponent != nullptr)
	{
		UE_LOGFMT(LogPCG, Warning, "More than one box component found. Choosing the first available: {0}", MatchingComponent->GetName());
	}

	return MatchingComponent;
}

#endif // WITH_EDITOR