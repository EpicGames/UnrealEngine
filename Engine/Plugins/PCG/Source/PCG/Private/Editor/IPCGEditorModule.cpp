// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/IPCGEditorModule.h"

static IPCGEditorModule* PCGEditorModulePtr = nullptr;

IPCGEditorModule* IPCGEditorModule::Get()
{
	return PCGEditorModulePtr;
}

void IPCGEditorModule::SetEditorModule(IPCGEditorModule* InModule)
{
	check(PCGEditorModulePtr == nullptr || InModule == nullptr);
	PCGEditorModulePtr = InModule;
}

#if WITH_EDITOR
bool IPCGEditorModule::ShouldTreatViewportAsGenerationSource(const APCGWorldActor* InPCGWorldActor)
{
	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		return PCGEditorModule->ShouldTreatViewportAsGenerationSourceInternal(InPCGWorldActor);
	}

	return false;
}
#endif