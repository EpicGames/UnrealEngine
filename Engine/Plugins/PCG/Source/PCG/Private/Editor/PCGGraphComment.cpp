// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/PCGGraphComment.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGraphComment)

#if WITH_EDITOR
void FPCGGraphCommentNodeData::InitializeFromCommentNode(const UEdGraphNode_Comment& CommentNode)
{
	NodePosX = CommentNode.NodePosX;
	NodePosY = CommentNode.NodePosY;
	NodeWidth = CommentNode.NodeWidth;
	NodeHeight = CommentNode.NodeHeight;
	NodeComment = CommentNode.NodeComment;
	CommentColor = CommentNode.CommentColor;
	FontSize = CommentNode.FontSize;
	bCommentBubbleVisible_InDetailsPanel = CommentNode.bCommentBubbleVisible_InDetailsPanel;
	bColorCommentBubble = CommentNode.bColorCommentBubble;
	MoveMode = static_cast<uint8>(CommentNode.MoveMode);
	NodeDetails = CommentNode.NodeDetails;
	CommentDepth = CommentNode.CommentDepth;
	GUID = CommentNode.NodeGuid;
	bCommentBubblePinned = CommentNode.bCommentBubblePinned;
	bCommentBubbleVisible = CommentNode.bCommentBubbleVisible;
}
#endif // WITH_EDITOR
