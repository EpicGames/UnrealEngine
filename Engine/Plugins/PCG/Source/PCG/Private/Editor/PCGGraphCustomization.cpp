// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/PCGGraphCustomization.h"
#include "PCGGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGraphCustomization)

const FPCGGraphEditorCustomization* FPCGGraphEditorCustomization::GetParent() const
{
	return nullptr;
}

bool FPCGGraphEditorCustomization::Accepts(const FText& InCategory) const
{
	if (bFilterNodesByCategory)
	{
		FString Category = InCategory.ToString();

		for (const FString& Filter : FilteredCategories)
		{
			if (Category.StartsWith(Filter))
			{
				return NodeFilterType == EPCGGraphEditorFiltering::Allow;
			}
		}
	}

	// "Default" behavior -> delegate to parent
	if (const FPCGGraphEditorCustomization* Parent = GetParent())
	{
		return Parent->Accepts(InCategory);
	}
	else
	{
		return !bFilterNodesByCategory || NodeFilterType != EPCGGraphEditorFiltering::Allow;
	}
}

bool FPCGGraphEditorCustomization::FiltersSubgraphs() const
{
	if (bFilterSubgraphs)
	{
		return true;
	}
	else if (const FPCGGraphEditorCustomization* Parent = GetParent())
	{
		return Parent->FiltersSubgraphs();
	}
	else
	{
		return false;
	}
}

bool FPCGGraphEditorCustomization::FilterSubgraph(const FSoftObjectPath& InSubgraphPath) const
{
	if (bFilterSubgraphs)
	{
		const TSoftObjectPtr<UPCGGraph> SubgraphSoftPtr = TSoftObjectPtr<UPCGGraph>(InSubgraphPath);

		if (FilteredSubgraphTypes.Contains(SubgraphSoftPtr))
		{
			return SubgraphFilterType != EPCGGraphEditorFiltering::Allow;
		}
	}

	// Otherwise, delegate filtering to parent if any
	if (const FPCGGraphEditorCustomization* Parent = GetParent())
	{
		return Parent->FilterSubgraph(InSubgraphPath);
	}
	else
	{
		return bFilterSubgraphs && SubgraphFilterType == EPCGGraphEditorFiltering::Allow;
	} 
}

bool FPCGGraphEditorCustomization::FiltersSettings() const
{
	if (bFilterSettings)
	{
		return true;
	}
	else if (const FPCGGraphEditorCustomization* Parent = GetParent())
	{
		return Parent->FiltersSettings();
	}
	else
	{
		return false;
	}
}

bool FPCGGraphEditorCustomization::FilterSettings(const TSubclassOf<UPCGSettings>& InSettingsClass) const
{
	if (bFilterSettings)
	{
		if (FilteredSettingsTypes.ContainsByPredicate([InSettingsClass](const TSubclassOf<UPCGSettings>& Other) { return InSettingsClass->IsChildOf(Other);}))
		{
			return SubgraphFilterType != EPCGGraphEditorFiltering::Allow;
		}
	}

	// Otherwise, delegate filtering to parent if any
	if (const FPCGGraphEditorCustomization* Parent = GetParent())
	{
		return Parent->FilterSettings(InSettingsClass);
	}
	else
	{
		return bFilterSettings && SubgraphFilterType == EPCGGraphEditorFiltering::Allow;
	} 
}
