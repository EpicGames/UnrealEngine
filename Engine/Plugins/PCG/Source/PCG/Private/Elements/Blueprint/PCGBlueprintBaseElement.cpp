// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Blueprint/PCGBlueprintBaseElement.h"

#include "PCGModule.h"
#include "PCGPin.h"
#include "Elements/PCGExecuteBlueprint.h"

#include "Algo/Transform.h"
#include "Engine/World.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Math/RandomStream.h"

#if WITH_EDITOR
#include "UObject/ObjectSaveContext.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBlueprintBaseElement)

#define LOCTEXT_NAMESPACE "PCGBlueprintBaseElement"

#if WITH_EDITOR
namespace PCGBlueprintHelper
{
	TSet<TWeakObjectPtr<UObject>> GetDataDependencies(UPCGBlueprintBaseElement* InElement, int32 MaxDepth)
	{
		check(InElement && InElement->GetClass());
		UClass* BPClass = InElement->GetClass();

		TSet<TObjectPtr<UObject>> Dependencies;
		PCGHelpers::GatherDependencies(InElement, Dependencies, MaxDepth);
		TSet<TWeakObjectPtr<UObject>> WeakDependencies;

		Algo::Transform(Dependencies, WeakDependencies, [](const TObjectPtr<UObject>& InObjectPtr) { return TWeakObjectPtr<UObject>(InObjectPtr); });

		return WeakDependencies;
	}
}
#endif // WITH_EDITOR

UWorld* UPCGBlueprintBaseElement::GetWorld() const
{
#if WITH_EDITOR
	return GWorld;
#else
	return InstanceWorld ? InstanceWorld : Super::GetWorld();
#endif
}

void UPCGBlueprintBaseElement::PostLoad()
{
	Super::PostLoad();
	Initialize();
}

void UPCGBlueprintBaseElement::BeginDestroy()
{
#if WITH_EDITOR
	if (!DataDependencies.IsEmpty())
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
		DataDependencies.Empty();
	}
#endif

	Super::BeginDestroy();
}

#if WITH_EDITOR
FPCGAssetCachedPins UPCGBlueprintBaseElement::BuildCachedPins() const
{
	FPCGAssetCachedPins Pins;
	Pins.bHasCachedPins = true;
	Pins.InputPins = GetInputPins();
	Pins.OutputPins = GetOutputPins();

	return Pins;
}

void UPCGBlueprintBaseElement::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Cache pins for asset retrieval
	CachedPins = BuildCachedPins();
}
#endif // WITH_EDITOR

void UPCGBlueprintBaseElement::Execute_Implementation(const FPCGDataCollection& Input, FPCGDataCollection& Output)
{
}

void UPCGBlueprintBaseElement::Initialize()
{
#if WITH_EDITOR
	UpdateDependencies();
#endif
}

FPCGBlueprintContextHandle UPCGBlueprintBaseElement::GetContextHandle() const
{
	FPCGBlueprintContextHandle BlueprintHandle;
	BlueprintHandle.Handle = CurrentContext ? CurrentContext->GetOrCreateHandle() : nullptr;
	return BlueprintHandle;
}

int32 UPCGBlueprintBaseElement::GetSeedWithContext(const FPCGBlueprintContextHandle& InContextHandle) const
{
	if (TSharedPtr<FPCGContextHandle> NativeContextHandle = InContextHandle.Handle.Pin())
	{
		if (const FPCGContext* Context = NativeContextHandle->GetContext())
		{
			return Context->GetSeed();
		}
	}

	return 42;
}

FRandomStream UPCGBlueprintBaseElement::GetRandomStreamWithContext(const FPCGBlueprintContextHandle& InContextHandle) const
{
	return FRandomStream(GetSeedWithContext(InContextHandle));
}

void UPCGBlueprintBaseElement::SetCurrentContext(FPCGContext* InCurrentContext)
{
	ensure(CurrentContext == nullptr || InCurrentContext == nullptr || CurrentContext == InCurrentContext);
	CurrentContext = InCurrentContext;

	// Make sure to create handle so that multi-threaded blueprint calls don't have to worry about concurrent creation
	if (CurrentContext)
	{
		CurrentContext->GetOrCreateHandle();
	}
}

FPCGContext* UPCGBlueprintBaseElement::ResolveContext()
{
	if (FFrame::GetThreadLocalTopStackFrame() && FFrame::GetThreadLocalTopStackFrame()->Object)
	{
		if (UPCGBlueprintBaseElement* Caller = Cast<UPCGBlueprintBaseElement>(FFrame::GetThreadLocalTopStackFrame()->Object))
		{
			return Caller->CurrentContext;
		}
	}

	return nullptr;
}

#if WITH_EDITOR
void UPCGBlueprintBaseElement::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UpdateDependencies();

	OnBlueprintElementChangedDelegate.Broadcast(this);
}

void UPCGBlueprintBaseElement::UpdateDependencies()
{
	// Avoid calculating dependencies for graph execution element
	if (!GetOuter()->IsA<UPCGBlueprintSettings>())
	{
		return;
	}

	// Backup to know if we need to unregister from the Delegate or not
	const bool bHadDependencies = DataDependencies.Num() > 0;

	// Since we don't really know what changed, let's just rebuild our data dependencies
	DataDependencies = PCGBlueprintHelper::GetDataDependencies(this, DependencyParsingDepth);

	// Only Bind to event if we do have dependencies
	if (!DataDependencies.IsEmpty())
	{
		if (!bHadDependencies)
		{
			FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UPCGBlueprintBaseElement::OnDependencyChanged);
		}
	}
	else if (bHadDependencies)
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	}
}

void UPCGBlueprintBaseElement::OnDependencyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		return;
	}

	// There are many engine notifications that aren't needed for us, esp. wrt to compilation
	if (PropertyChangedEvent.Property == nullptr && PropertyChangedEvent.ChangeType == EPropertyChangeType::Unspecified)
	{
		return;
	}

	if (!DataDependencies.Contains(Object))
	{
		return;
	}

	OnBlueprintElementChangedDelegate.Broadcast(this);
}

FString UPCGBlueprintBaseElement::GetParentClassName()
{
	return FObjectPropertyBase::GetExportPath(UPCGBlueprintBaseElement::StaticClass());
}
#endif // WITH_EDITOR

FName UPCGBlueprintBaseElement::NodeTitleOverride_Implementation() const
{
	return NAME_None;
}

FLinearColor UPCGBlueprintBaseElement::NodeColorOverride_Implementation() const
{
	return FLinearColor::White;
}

EPCGSettingsType UPCGBlueprintBaseElement::NodeTypeOverride_Implementation() const
{
	return EPCGSettingsType::Blueprint;
}

bool UPCGBlueprintBaseElement::IsCacheableOverride_Implementation() const
{
	return bIsCacheable;
}

int32 UPCGBlueprintBaseElement::DynamicPinTypesOverride_Implementation(const UPCGSettings* InSettings, const UPCGPin* InPin) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return static_cast<int32>(InSettings->UPCGSettings::GetCurrentPinTypesID(InPin));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TSet<FName> UPCGBlueprintBaseElement::CustomInputLabels() const
{
	TSet<FName> Labels;
	for (const FPCGPinProperties& PinProperty : CustomInputPins)
	{
		Labels.Emplace(PinProperty.Label);
	}

	return Labels;
}

TSet<FName> UPCGBlueprintBaseElement::CustomOutputLabels() const
{
	TSet<FName> Labels;
	for (const FPCGPinProperties& PinProperty : CustomOutputPins)
	{
		Labels.Emplace(PinProperty.Label);
	}

	return Labels;
}

TArray<FPCGPinProperties> UPCGBlueprintBaseElement::GetInputPins() const
{
	if (CurrentContext)
	{
		if (const UPCGBlueprintSettings* Settings = CurrentContext->GetInputSettings<UPCGBlueprintSettings>())
		{
			return Settings->InputPinProperties();
		}
	}
	else if (const UPCGBlueprintSettings* OriginalSettings = Cast<UPCGBlueprintSettings>(GetOuter()))
	{
		return OriginalSettings->InputPinProperties();
	}

	// Can't retrieve settings - return only custom pins then
	return CustomInputPins;
}

TArray<FPCGPinProperties> UPCGBlueprintBaseElement::GetOutputPins() const
{
	if (CurrentContext)
	{
		if (const UPCGBlueprintSettings* Settings = CurrentContext->GetInputSettings<UPCGBlueprintSettings>())
		{
			return Settings->OutputPinProperties();
		}
	}
	else if (const UPCGBlueprintSettings* OriginalSettings = Cast<UPCGBlueprintSettings>(GetOuter()))
	{
		return OriginalSettings->OutputPinProperties();
	}

	// Can't retrieve settings - return only custom pins then
	return CustomOutputPins;
}

bool UPCGBlueprintBaseElement::GetInputPinByLabel(FName InPinLabel, FPCGPinProperties& OutFoundPin) const
{
	TArray<FPCGPinProperties> InputPinProperties = GetInputPins();
	for (const FPCGPinProperties& InputPin : InputPinProperties)
	{
		if (InputPin.Label == InPinLabel)
		{
			OutFoundPin = InputPin;
			return true;
		}
	}

	OutFoundPin = FPCGPinProperties();
	return false;;
}

bool UPCGBlueprintBaseElement::GetOutputPinByLabel(FName InPinLabel, FPCGPinProperties& OutFoundPin) const
{
	TArray<FPCGPinProperties> OutputPinProperties = GetOutputPins();
	for (const FPCGPinProperties& OutputPin : OutputPinProperties)
	{
		if (OutputPin.Label == InPinLabel)
		{
			OutFoundPin = OutputPin;
			return true;
		}
	}

	OutFoundPin = FPCGPinProperties();
	return false;
}

void UPCGBlueprintBaseElement::ThrowRequiresGameThreadException(FPCGContext* InContext, const FString& InFunctionName) const
{
	const FText FormatText = FText::Format(LOCTEXT("FunctionRequiresGameThread", "Function '{0}' requires game thread and is being called outside of game thread. Make sure to enable 'bRequiresGameThread' on your Blueprint."), FText::FromString(InFunctionName));
	PCGE_LOG_C(Error, GraphAndLog, InContext, FormatText);
	UPCGBlueprintHelpers::ThrowBlueprintException(FormatText);
}

#undef LOCTEXT_NAMESPACE