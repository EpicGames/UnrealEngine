// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Blueprint/PCGBlueprintPointProcessorElement.h"

#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"

void UPCGBlueprintPointProcessorElement::PointLoopInitialize_Implementation(const UPCGBasePointData* InputPointData, UPCGBasePointData* OutputPointData)
{
	FPCGInitializeFromDataParams InitializeFromDataParams(InputPointData);
	InitializeFromDataParams.bInheritMetadata = true;
	InitializeFromDataParams.bInheritSpatialData = true;
	OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);
}

void UPCGBlueprintPointProcessorElement::PointLoop(const UPCGBasePointData* InputPointData, UPCGBasePointData*& OutputPointData, bool bGameThread)
{
	if (bGameThread && !IsInGameThread())
	{
		ThrowRequiresGameThreadException(CurrentContext, TEXT("PointLoop"));
		return;
	}

	if (!InputPointData)
	{
		return;
	}

	OutputPointData = FPCGContext::NewPointData_AnyThread(CurrentContext);
	PointLoopInitialize(InputPointData, OutputPointData);

	auto ProcessFunc = [this, InputPointData, OutputPointData](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
	{
		FPCGPointInputRange InputRange{ InputPointData, StartReadIndex, Count };
		FPCGPointOutputRange OutputRange{ OutputPointData, StartWriteIndex, Count };

		return PointLoopBody(InputRange, OutputRange);
	};
		
	auto MoveDataRangeFunc = [OutputPointData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
	{
		OutputPointData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
	};

	auto FinishedFunc = [OutputPointData](int32 NumWritten)
	{
		OutputPointData->SetNumPoints(NumWritten);
	};

	{
		TGuardValue<bool> IsCallingBlueprint(CurrentContext->AsyncState.bIsCallingBlueprint, true);
		TGuardValue<int32> NumTaskOverride(CurrentContext->AsyncState.NumAvailableTasks, bGameThread ? 1 : CurrentContext->AsyncState.NumAvailableTasks);

		FPCGAsync::AsyncProcessingRangeEx(
			&CurrentContext->AsyncState,
			InputPointData->GetNumPoints(),
			/*InitializeFunc=*/[]() {},
			ProcessFunc,
			MoveDataRangeFunc,
			FinishedFunc,
			/*bEnableTimeSlicing=*/false
		);
	}
}

void UPCGBlueprintPointProcessorElement::IterationLoopInitialize_Implementation(int32 NumIterations, const UPCGBasePointData* InputPointDataA, const UPCGBasePointData* InputPointDataB, UPCGBasePointData* OutputPointData)
{
	const UPCGBasePointData* InheritFromData = InputPointDataA ? InputPointDataA : InputPointDataB;
	if (InheritFromData)
	{
		FPCGInitializeFromDataParams InitializeFromDataParams(InheritFromData);
		InitializeFromDataParams.bInheritMetadata = true;
		InitializeFromDataParams.bInheritSpatialData = false;
		OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);
	}
	OutputPointData->SetNumPoints(NumIterations);
}

void UPCGBlueprintPointProcessorElement::IterationLoop(int32 NumIterations, const UPCGBasePointData* InputPointDataA, const UPCGBasePointData* InputPointDataB, UPCGBasePointData*& OutputPointData, bool bGameThread)
{
	if (bGameThread && !IsInGameThread())
	{
		ThrowRequiresGameThreadException(CurrentContext, TEXT("IterationLoop"));
		return;
	}

	OutputPointData = FPCGContext::NewPointData_AnyThread(CurrentContext);
	IterationLoopInitialize(NumIterations, InputPointDataA, InputPointDataB, OutputPointData);

	auto ProcessFunc = [this, InputPointDataA, InputPointDataB, OutputPointData](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
	{
		// Give full input ranges as we don't know how those will be accessed in the nested loop body
		const FPCGPointInputRange InputRangeA{ InputPointDataA, 0, InputPointDataA ? InputPointDataA->GetNumPoints() : 0 };
		const FPCGPointInputRange InputRangeB{ InputPointDataB, 0, InputPointDataB ? InputPointDataB->GetNumPoints() : 0 };
		FPCGPointOutputRange OutputRange{ OutputPointData, StartWriteIndex, Count };

		return IterationLoopBody(StartReadIndex, InputRangeA, InputRangeB, OutputRange);
	};

	auto MoveDataRangeFunc = [OutputPointData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
	{
		OutputPointData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
	};

	auto FinishedFunc = [OutputPointData](int32 NumWritten)
	{
		OutputPointData->SetNumPoints(NumWritten);
	};

	{
		TGuardValue<bool> IsCallingBlueprint(CurrentContext->AsyncState.bIsCallingBlueprint, true);
		TGuardValue<int32> NumTaskOverride(CurrentContext->AsyncState.NumAvailableTasks, bGameThread ? 1 : CurrentContext->AsyncState.NumAvailableTasks);

		FPCGAsync::AsyncProcessingRangeEx(
			&CurrentContext->AsyncState,
			NumIterations,
			/*InitializeFunc=*/[]() {},
			ProcessFunc,
			MoveDataRangeFunc,
			FinishedFunc,
			/*bEnableTimeSlicing=*/false
		);
	}
}

