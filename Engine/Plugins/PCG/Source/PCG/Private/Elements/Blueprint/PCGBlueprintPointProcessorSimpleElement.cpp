// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Blueprint/PCGBlueprintPointProcessorSimpleElement.h"
#include "Helpers/PCGAsync.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBlueprintPointProcessorSimpleElement)

namespace PCGBlueprintPointProcessorSimpleElement
{
	constexpr int32 RunawayResetFrequency = 1024;
}

UPCGBlueprintPointProcessorSimpleElement::UPCGBlueprintPointProcessorSimpleElement()
	: UPCGBlueprintBaseElement()
{
	bHasDefaultInPin = false;
	bHasDefaultOutPin = false;
	CustomInputPins.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point).SetRequiredPin();
	CustomOutputPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
}

void UPCGBlueprintPointProcessorSimpleElement::Execute_Implementation(const FPCGDataCollection& Input, FPCGDataCollection& Output)
{
	for (const FPCGTaggedData& InputData : Input.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		const UPCGBasePointData* InputPointData = Cast<const UPCGBasePointData>(InputData.Data);
		if (!InputPointData)
		{
			continue;
		}

		UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(CurrentContext);
		
		FPCGInitializeFromDataParams InitializeFromDataParams(InputPointData);
		InitializeFromDataParams.bInheritMetadata = bInheritMetadata;
		InitializeFromDataParams.bInheritSpatialData = bInheritSpatialData;
		OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);
		OutputPointData->SetNumPoints(InputPointData->GetNumPoints());
		
		// If we are not inheriting the metadata but we are inheriting the spatial data, make sure to reset the entry keys
		if (!bInheritMetadata && bInheritSpatialData)
		{
			OutputPointData->SetMetadataEntry(PCGInvalidEntryKey);
		}
		
		OutputPointData->AllocateProperties((EPCGPointNativeProperties)PropertiesToAllocate);

		Initialize(InputPointData, OutputPointData);

		TFunction<int32(int32, int32, int32)> RangeProcessFunc = [this, InputPointData, OutputPointData](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			FPCGPointInputRange InputRange{ InputPointData, StartReadIndex, Count };
			FPCGPointOutputRange OutputRange{ OutputPointData, StartWriteIndex, Count };

			return RangePointLoopBody(InputRange, OutputRange);
		};

		TFunction<int32(int32, int32, int32)> PerPointProcessFunc = [this, InputPointData, OutputPointData](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			const FConstPCGPointValueRanges InputValueRanges(InputPointData);
			FPCGPointValueRanges OutputValueRanges(OutputPointData, /*bAllocate=*/false);

			int32 NumWritten = 0;

			for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
			{
				if (ReadIndex % PCGBlueprintPointProcessorSimpleElement::RunawayResetFrequency == 0)
				{
					GInitRunaway(); // Reset periodically the iteration count, because we know we're in a fixed size loop.
				}

				FPCGPoint InPoint = InputValueRanges.GetPoint(ReadIndex);
				
				FPCGPoint OutPoint;

				if (bInheritSpatialData)
				{
					OutPoint = InPoint;
				}

				if (PointLoopBody(InputPointData, InPoint, OutputPointData, OutPoint, ReadIndex))
				{
					OutputValueRanges.SetFromPoint(StartWriteIndex + NumWritten, OutPoint);
					++NumWritten;
				}
			}

			return NumWritten;
		};

		TFunction<int32(int32, int32, int32)> ProcessFunc = GetClass()->IsFunctionImplementedInScript(GET_FUNCTION_NAME_CHECKED(UPCGBlueprintPointProcessorSimpleElement, RangePointLoopBody)) ? RangeProcessFunc : PerPointProcessFunc;

		auto MoveDataRangeFunc = [OutputPointData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
		{
			OutputPointData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		};

		auto FinishedFunc = [OutputPointData](int32 NumWritten)
		{
			OutputPointData->SetNumPoints(NumWritten);
		};

		{
			TGuardValue<bool> IsCallingBlueprint(CurrentContext->AsyncState.bIsCallingBlueprint, true);

			FPCGAsync::AsyncProcessingRangeEx(
				&CurrentContext->AsyncState,
				InputPointData->GetNumPoints(),
				/*InitializeFunc=*/[]() {},
				ProcessFunc,
				MoveDataRangeFunc,
				FinishedFunc,
				/*bEnableTimeSlicing=*/false
			);
		}
		
		FPCGTaggedData& OutputData = Output.TaggedData.Emplace_GetRef(InputData);
		OutputData.Data = OutputPointData;
		OutputData.Pin = PCGPinConstants::DefaultOutputLabel;
	}
}
