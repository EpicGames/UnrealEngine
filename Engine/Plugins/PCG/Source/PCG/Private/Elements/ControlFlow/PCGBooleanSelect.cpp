// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGBooleanSelect.h"

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGPin.h"
#include "Elements/PCGGather.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBooleanSelect)

#define LOCTEXT_NAMESPACE "FPCGBooleanSelectElement"

namespace PCGBooleanSelectConstants
{
	const FName InputLabelA = TEXT("Input A");
	const FName InputLabelB = TEXT("Input B");
	const FText NodeTitleBase = LOCTEXT("NodeTitle", "Select");
}

#if WITH_EDITOR
FText UPCGBooleanSelectSettings::GetDefaultNodeTitle() const
{
	// TODO: This should statically update or dynamically update, if overridden, for which input was selected, ie. Select (Input A)
	return PCGBooleanSelectConstants::NodeTitleBase;
}

FText UPCGBooleanSelectSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Control flow node that will select all input data on either Pin A or Pin B only, based on the 'Use Input B' property - which can also be overridden.");
}
#endif // WITH_EDITOR

FPCGDataTypeIdentifier UPCGBooleanSelectSettings::GetCurrentPinTypesID(const UPCGPin* Pin) const
{
	// Output pin depends on the union of all non-advanced input pins. Advanced pins are dynamic user overrides and
	// should not affect the output pin type.
	if (Pin->IsOutputPin())
	{
		const FPCGDataTypeIdentifier InputTypeUnion = GetTypeUnionIDOfAllIncidentEdges({PCGBooleanSelectConstants::InputLabelA, PCGBooleanSelectConstants::InputLabelB});
		return InputTypeUnion.IsValid() ? InputTypeUnion : FPCGDataTypeIdentifier{EPCGDataType::Any};
	}
	else
	{
		return Super::GetCurrentPinTypesID(Pin);
	}
}

TArray<FPCGPinProperties> UPCGBooleanSelectSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGBooleanSelectConstants::InputLabelA,
		EPCGDataType::Any,
		/*bInAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("FirstInputPinTooltip", "Will only be used if 'Use Input B' (overridable) is false"));
	PinProperties.Emplace(PCGBooleanSelectConstants::InputLabelB,
		EPCGDataType::Any,
		/*bInAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("SecondInputPinTooltip", "Will only be used if 'Use Input B' (overridable) is true"));

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGBooleanSelectSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel,
		EPCGDataType::Any,
		/*bInAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("OutputPinTooltipV2", "All inputs will be gathered into a single data collection"));

	return PinProperties;
}

FPCGElementPtr UPCGBooleanSelectSettings::CreateElement() const
{
	return MakeShared<FPCGBooleanSelectElement>();
}

bool FPCGBooleanSelectElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBooleanSelectElement::ExecuteInternal);

	const UPCGBooleanSelectSettings* Settings = Context->GetInputSettings<UPCGBooleanSelectSettings>();
	check(Settings);

	const FName SelectedPinLabel = Settings->bUseInputB ? PCGBooleanSelectConstants::InputLabelB : PCGBooleanSelectConstants::InputLabelA;

	// Reuse the functionality of the Gather Node
	Context->OutputData = PCGGather::GatherDataForPin(Context->InputData, SelectedPinLabel);

	return true;
}

#undef LOCTEXT_NAMESPACE
