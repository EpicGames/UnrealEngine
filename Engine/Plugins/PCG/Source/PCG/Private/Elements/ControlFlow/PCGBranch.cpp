// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGBranch.h"

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGPin.h"
#include "Elements/PCGGather.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBranch)

#define LOCTEXT_NAMESPACE "FPCGBranchElement"

namespace PCGBranchConstants
{
	const FName OutputLabelA = TEXT("Output A");
	const FName OutputLabelB = TEXT("Output B");
	const FText NodeTitleBase = LOCTEXT("NodeTitle", "Branch");
}

#if WITH_EDITOR
FText UPCGBranchSettings::GetDefaultNodeTitle() const
{
	// TODO: This should statically update or dynamically update, if overridden, for which branch was taken, ie. Branch (Output A)
	return PCGBranchConstants::NodeTitleBase;
}

FText UPCGBranchSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Control flow node that will route the input to either Output A or Output B, based on the 'Output To B' property - which can also be overridden.");
}

EPCGChangeType UPCGBranchSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;

	// Static branches are processed during graph compilation and are part of the graph structure.
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bEnabled)
		|| InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGBranchSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGBranchConstants::OutputLabelA,
		EPCGDataType::Any,
		/*bInAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("OutputPinTooltipA", "Will only route input if 'Output To B' (overridable) is false"));
	PinProperties.Emplace(PCGBranchConstants::OutputLabelB,
		EPCGDataType::Any,
		/*bInAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("OutputPinTooltipB", "Will only route input if 'Output To B' (overridable) is true"));

	return PinProperties;
}

FPCGElementPtr UPCGBranchSettings::CreateElement() const
{
	return MakeShared<FPCGBranchElement>();
}

bool UPCGBranchSettings::IsPinStaticallyActive(const FName& PinLabel) const
{
	if (!bEnabled)
	{
		// Disabled - everything passed through first pin.
		return PinLabel == PCGBranchConstants::OutputLabelA;
	}

	// Dynamic branches are never known in advance - assume all branches are active prior to execution.
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB)))
	{
		return true;
	}

	return PinLabel == (bOutputToB ? PCGBranchConstants::OutputLabelB : PCGBranchConstants::OutputLabelA);
}

bool UPCGBranchSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	if (!InPin->IsOutputPin())
	{
		return Super::IsPinUsedByNodeExecution(InPin);
	}

	// Dynamic branches are never known in advance - assume all branches are active prior to execution.
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB)))
	{
		return true;
	}

	// Branch must be both enabled and set to B in order for output pin B to be active.
	const FName ActiveOutputPinLabel = (bEnabled && bOutputToB) ? PCGBranchConstants::OutputLabelB : PCGBranchConstants::OutputLabelA;

	return InPin->Properties.Label == ActiveOutputPinLabel;
}

bool FPCGBranchElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBranchElement::ExecuteInternal);

	const UPCGBranchSettings* Settings = Context->GetInputSettings<UPCGBranchSettings>();
	check(Settings);

	const FName SelectedPinLabel = Settings->bOutputToB ? PCGBranchConstants::OutputLabelB : PCGBranchConstants::OutputLabelA;

	// Reuse the functionality of the Gather Node
	Context->OutputData = PCGGather::GatherDataForPin(Context->InputData, PCGPinConstants::DefaultInputLabel, SelectedPinLabel);

	// Output bitmask of deactivated pins.
	Context->OutputData.InactiveOutputPinBitmask = Settings->bOutputToB ? 1 : 2;

	return true;
}

#undef LOCTEXT_NAMESPACE
