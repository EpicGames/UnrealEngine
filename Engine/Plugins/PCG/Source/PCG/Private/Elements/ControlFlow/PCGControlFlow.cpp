// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGControlFlow.h"

#include "Internationalization/Text.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGControlFlow)

FText FEnumSelector::GetDisplayName() const 
{
	return Class ? Class->GetDisplayNameTextByValue(Value) : FText{};
}

FString FEnumSelector::GetCultureInvariantDisplayName() const 
{
	return GetDisplayName().BuildSourceString();
}
