// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGMultiSelect.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Elements/PCGGather.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMultiSelect)

#define LOCTEXT_NAMESPACE "FPCGMultiSelectElement"

namespace PCGMultiSelectConstants
{
	const FText NodeTitleBase = LOCTEXT("NodeTitleBase", "Select (Multi)");
}

void UPCGMultiSelectSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// @todo_pcg To be behind a version bump in 5.5. Cannot do that in an hotfix
	// Make sure we rename the pin properties that were serialized with a localized text. Since we can't exactly match the text
	// with the enum value, we'll go with index.
	if (SelectionMode == EPCGControlFlowSelectionMode::Enum)
	{
		UPCGNode* OuterNode = Cast<UPCGNode>(GetOuter());
		if (OuterNode)
		{
			TArray<UPCGPin*> SerializedInputPins = OuterNode->GetInputPins();

			// We need to remove the override pins, find the overrides pin index and remove all following
			const int32 OverridesPinIndex = SerializedInputPins.IndexOfByPredicate([](const UPCGPin* Pin) { return Pin && Pin->Properties.Label == PCGPinConstants::DefaultParamsLabel; });
			if (OverridesPinIndex != INDEX_NONE)
			{
				SerializedInputPins.SetNum(OverridesPinIndex);
			}

			// It we have a num mismatch, we can't recover
			if (SerializedInputPins.Num() == CachedPinLabels.Num())
			{
				// -1 since we don't need to check "Default"
				for (int32 i = 0; i < CachedPinLabels.Num() - 1; ++i)
				{
					if (SerializedInputPins[i] && SerializedInputPins[i]->Properties.Label != CachedPinLabels[i])
					{
						OuterNode->RenameInputPin(SerializedInputPins[i]->Properties.Label, CachedPinLabels[i], /*bBroadcastUpdate=*/false);
					}
				}
			}
		}

		if (EnumValueToPinLabelMapping.IsEmpty())
		{
			CacheEnumValueToPinLabelsMapping();
		}
	}
#endif // WITH_EDITOR

	CachePinLabels();
}

void UPCGMultiSelectSettings::UpdateSettings()
{
	CacheEnumValueToPinLabelsMapping();
	CachePinLabels();
}

void UPCGMultiSelectSettings::OnOverrideSettingsDuplicatedInternal(bool bSkippedPostLoad)
{
	Super::OnOverrideSettingsDuplicatedInternal(bSkippedPostLoad);
	if (bSkippedPostLoad)
	{
		CachePinLabels();
		// We don't want to update settings here, because we don't want to erase the enum value to pin label mapping.
	}
}

#if WITH_EDITOR
void UPCGMultiSelectSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Only need to change the pin labels if the options have changed
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGMultiSelectSettings, SelectionMode) ||
		PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGMultiSelectSettings, IntOptions) ||
		PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGMultiSelectSettings, StringOptions) ||
		PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGMultiSelectSettings, EnumSelection))
	{
		UpdateSettings();
	}
}

FText UPCGMultiSelectSettings::GetDefaultNodeTitle() const
{
	return PCGMultiSelectConstants::NodeTitleBase;
}

FText UPCGMultiSelectSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Control flow node that will select all input data on a single input pin that matches a given selection mode and corresponding 'selection' property - which can also be overridden.");
}
#endif // WITH_EDITOR

FPCGDataTypeIdentifier UPCGMultiSelectSettings::GetCurrentPinTypesID(const UPCGPin* Pin) const
{
	// Output pin depends on the union of all non-advanced input pins. Advanced pins are dynamic user overrides and
	// should not affect the output pin type.
	if (Pin->IsOutputPin())
	{
		TArray<FName, TInlineAllocator<32>> PinLabels;
		Algo::TransformIf(Pin->Node->GetInputPins(), PinLabels, [](const UPCGPin* InPin) { return InPin && !InPin->Properties.IsAdvancedPin();}, [](const UPCGPin* InPin) { return InPin->Properties.Label; });
		
		const FPCGDataTypeIdentifier InputTypeUnion = GetTypeUnionIDOfAllIncidentEdges(PinLabels);
		return InputTypeUnion.IsValid() ? InputTypeUnion : FPCGDataTypeIdentifier{EPCGDataType::Any};
	}
	else
	{
		return Super::GetCurrentPinTypesID(Pin);
	}
}

FString UPCGMultiSelectSettings::GetAdditionalTitleInformation() const
{
	switch (SelectionMode)
	{
		case EPCGControlFlowSelectionMode::Integer:
		{
			FString Subtitle = PCGControlFlowConstants::SubtitleInt.ToString();
			if (!IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGMultiSelectSettings, IntegerSelection)))
			{
				Subtitle += FString::Format(TEXT(": {0}"), {IntegerSelection});
			}

			return Subtitle;
		}

		case EPCGControlFlowSelectionMode::Enum:
			if (EnumSelection.Class)
			{
				FString Subtitle = EnumSelection.Class->GetName();
				if (!IsPropertyOverriddenByPin({GET_MEMBER_NAME_CHECKED(UPCGMultiSelectSettings, EnumSelection), GET_MEMBER_NAME_CHECKED(FEnumSelector, Value)}))
				{
					if (FName* EnumValueName = EnumValueToPinLabelMapping.Find(EnumSelection.Value))
					{
						Subtitle += FString::Format(TEXT(": {0}"), { EnumValueName->ToString() });
					}
				}

				return Subtitle;
			}
			else
			{
				return PCGControlFlowConstants::SubtitleEnum.ToString();
			}

		case EPCGControlFlowSelectionMode::String:
			{
				FString Subtitle = PCGControlFlowConstants::SubtitleString.ToString();
				if (!IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGMultiSelectSettings, StringSelection)))
				{
					Subtitle += FString::Format(TEXT(": {0}"), {StringSelection});
				}

				return Subtitle;
			}

		default:
			checkNoEntry();
			break;
	}

	return PCGMultiSelectConstants::NodeTitleBase.ToString();
}

TArray<FPCGPinProperties> UPCGMultiSelectSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	switch (SelectionMode)
	{
		case EPCGControlFlowSelectionMode::Integer:
			for (const int32 Value : IntOptions)
			{
				PinProperties.Emplace(FName(FString::FromInt(Value)));
			}
			break;
		case EPCGControlFlowSelectionMode::String:
			for (const FString& Value : StringOptions)
			{
				PinProperties.Emplace(FName(Value));
			}
			break;
		case EPCGControlFlowSelectionMode::Enum:
			if (EnumValueToPinLabelMapping.IsEmpty())
			{
				CacheEnumValueToPinLabelsMapping();
			}

			for (const auto& [Value, PinName] : EnumValueToPinLabelMapping)
			{
				PinProperties.Emplace(PinName);
			}
			break;
		default:
			break;
	}

	PinProperties.Emplace(PCGControlFlowConstants::DefaultPathPinLabel);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGMultiSelectSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel,
		EPCGDataType::Any,
		/*bInAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("OutputPinTooltip", "All input on the selected input pin will be forwarded directly to the output."));

	return PinProperties;
}

FPCGElementPtr UPCGMultiSelectSettings::CreateElement() const
{
	return MakeShared<FPCGMultiSelectElement>();
}

bool UPCGMultiSelectSettings::IsValuePresent(const int32 Value) const
{
	return IntOptions.Contains(Value);
}

bool UPCGMultiSelectSettings::IsValuePresent(const FString& Value) const
{
	return StringOptions.Contains(Value);
}

bool UPCGMultiSelectSettings::IsValuePresent(const int64 Value) const
{
	if (!EnumSelection.Class)
	{
		return false;
	}

	const int64 Index = EnumSelection.Class->GetIndexByValue(Value);
	return Index != INDEX_NONE && Index < EnumSelection.Class->NumEnums() - 1;
}

bool UPCGMultiSelectSettings::GetSelectedPinLabel(FName& OutSelectedPinLabel) const
{
	if (CachedPinLabels.IsEmpty())
	{
		return false;
	}

	int32 Index = INDEX_NONE;
	if (SelectionMode == EPCGControlFlowSelectionMode::Integer && IsValuePresent(IntegerSelection))
	{
		Index = IntOptions.IndexOfByKey(IntegerSelection);
	}
	else if (SelectionMode == EPCGControlFlowSelectionMode::String && IsValuePresent(StringSelection))
	{
		Index = StringOptions.IndexOfByKey(StringSelection);
	}
	else if (SelectionMode == EPCGControlFlowSelectionMode::Enum && IsValuePresent(EnumSelection.Value))
	{
		const FName* PinLabel = EnumValueToPinLabelMapping.Find(EnumSelection.Value);

		// Return index if found, otherwise fallback to the index after the options which will be "Default" pin.
		Index = PinLabel ? CachedPinLabels.IndexOfByKey(*PinLabel) : INDEX_NONE;
	}

	// If value is not found, use Default pin.
	OutSelectedPinLabel = CachedPinLabels.IsValidIndex(Index) ? CachedPinLabels[Index] : PCGControlFlowConstants::DefaultPathPinLabel;

	return true;
}

void UPCGMultiSelectSettings::CachePinLabels()
{
	CachedPinLabels.Empty();
	Algo::Transform(InputPinProperties(), CachedPinLabels, [](const FPCGPinProperties& Property)
	{
		return Property.Label;
	});
}

void UPCGMultiSelectSettings::CacheEnumValueToPinLabelsMapping() const
{
	EnumValueToPinLabelMapping.Empty();

	if (SelectionMode != EPCGControlFlowSelectionMode::Enum)
	{
		return;
	}

	// -1 to bypass the MAX value
	for (int32 Index = 0; EnumSelection.Class && Index < EnumSelection.Class->NumEnums() - 1; ++Index)
	{
		bool bHidden = false;
#if WITH_EDITOR
		// HasMetaData is editor only, so there will be extra pins at runtime, but that should be okay
		bHidden = EnumSelection.Class->HasMetaData(TEXT("Hidden"), Index) || EnumSelection.Class->HasMetaData(TEXT("Spacer"), Index);
#endif // WITH_EDITOR

		if (!bHidden)
		{
			const FString EnumDisplayName = EnumSelection.Class->GetDisplayNameTextByIndex(Index).BuildSourceString();
			EnumValueToPinLabelMapping.Emplace( EnumSelection.Class->GetValueByIndex(Index), FName(EnumDisplayName));
		}
	}
}

bool FPCGMultiSelectElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMultiSelectElement::ExecuteInternal);

	const UPCGMultiSelectSettings* Settings = Context->GetInputSettings<UPCGMultiSelectSettings>();
	check(Settings);

	FName SelectedPinLabel;
	if (!Settings->GetSelectedPinLabel(SelectedPinLabel))
	{
		PCGE_LOG_C(Error, GraphAndLog, Context, LOCTEXT("ValueDoesNotExist", "Selected value is not a valid option."));
		return true;
	}

	// Reuse the functionality of the Gather node
	Context->OutputData = PCGGather::GatherDataForPin(Context->InputData, SelectedPinLabel);

	return true;
}

#undef LOCTEXT_NAMESPACE
