// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGPlatformSwitch.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Elements/PCGGather.h"
#include "Helpers/PCGHelpers.h"

#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "UObject/ObjectSaveContext.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPlatformSwitch)

#define LOCTEXT_NAMESPACE "FPCGPlatformSwitchElement"

namespace PCGPlatformSwitch
{
	static const FName DefaultLabel = TEXT("Default");

#if WITH_EDITOR
	static bool GetCurrentPlatformInfoFromEditor(FName& OutPlatform, FName& OutPlatformGroup)
	{
		// Ask the editor for the active preview platform
		FName IniPlatform;
		if (GEditor && GEditor->GetPreviewPlatformName(IniPlatform))
		{
			OutPlatform = IniPlatform;
			OutPlatformGroup = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(IniPlatform).PlatformGroupName;
			return true;
		}

		// Single active target platform selected by the user (Project Launcher / Platforms UI)
		const TArray<ITargetPlatform*>& ActiveTPs = GetTargetPlatformManagerRef().GetActiveTargetPlatforms();
		if (ActiveTPs.Num() == 1 && ActiveTPs[0])
		{
			OutPlatform = ActiveTPs[0]->GetPlatformInfo().IniPlatformName;
			OutPlatformGroup = ActiveTPs[0]->GetPlatformInfo().PlatformGroupName;
			return true;
		}

		// Device Profile (covers Mobile Preview / device-specific previews in-editor)
		if (UDeviceProfile* ActiveProfile = UDeviceProfileManager::Get().GetActiveProfile())
		{
			// DeviceType typically matches the Ini platform name
			OutPlatform = *ActiveProfile->DeviceType;
			OutPlatformGroup = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(OutPlatform).PlatformGroupName;
			return true;
		}

		// Fallback: the platform the editor is running on
		if (const ITargetPlatform* Running = GetTargetPlatformManagerRef().GetRunningTargetPlatform())
		{
			OutPlatform = Running->GetPlatformInfo().IniPlatformName;
			OutPlatformGroup = Running->GetPlatformInfo().PlatformGroupName;
			return true;
		}

		return ensure(false);
	}
#endif // WITH_EDITOR
}

bool UPCGPlatformSwitchSettings::IsPinStaticallyActive(const FName& PinLabel) const
{
	FName CurrentPlatform, CurrentPlatformGroup;

	if (GetCurrentPlatformInfo(CurrentPlatform, CurrentPlatformGroup))
	{
		if (PinLabel != PCGPlatformSwitch::DefaultLabel)
		{
			// Active if the platform for this pin is selected.
			return IsPlatformSelected(PinLabel, CurrentPlatform, CurrentPlatformGroup);
		}
		else
		{
			// Default only active if no other pin is active.
			for (FName Platform : GetSanitizedPlatforms())
			{
				if (IsPlatformSelected(Platform, CurrentPlatform, CurrentPlatformGroup))
				{
					return false;
				}
			}

			return true;
		}
	}

	return false;
}

FString UPCGPlatformSwitchSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	FString Selection;
	if (PlatformOutputs.IsEmpty())
	{
		Selection = TEXT("None");
	}
	else if (PlatformOutputs.Num() == 1)
	{
		Selection = PlatformOutputs[0].ToString();
	}
	else
	{
		Selection = TEXT("Multiple");
	}

	return FString::Format(TEXT("Platform: {0}"), { Selection });
#else
	return {};
#endif
}

#if WITH_EDITOR
void UPCGPlatformSwitchSettings::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
	
	if (ObjectSaveContext.IsCooking())
	{
		ensure(PCGPlatformSwitch::GetCurrentPlatformInfoFromEditor(CookedPlatform, CookedPlatformGroup));
	}
}

FText UPCGPlatformSwitchSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Statically activates/deactivates output pins based on the current platform or platform group (desktop/console/mobile), useful for culling downstream nodes.");
}

EPCGChangeType UPCGPlatformSwitchSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGPlatformSwitchSettings, bEnabled)
		|| InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGPlatformSwitchSettings, PlatformOutputs))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}

TArray<FName> UPCGPlatformSwitchSettings::GetPlatformOptions() const
{
	const TArray<const FDataDrivenPlatformInfo*>& SortedPlatforms = FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly);

	// Platform names and platform group names.
	TArray<FName> BasePlatformNameArray;
	TArray<FName> PlatformGroupNameArray;

	// Create mapping from platform to platform groups and remove postfixes and invalid platform names
	for (const FDataDrivenPlatformInfo* DDPI : SortedPlatforms)
	{
		// Add platform name if it isn't already set, and also add to group mapping
		if (!BasePlatformNameArray.Contains(DDPI->IniPlatformName))
		{
			BasePlatformNameArray.Add(DDPI->IniPlatformName);
			PlatformGroupNameArray.AddUnique(DDPI->PlatformGroupName);
		}
	}

	TArray<FName> PlatformOptions = MoveTemp(PlatformGroupNameArray);
	PlatformOptions.Append(BasePlatformNameArray);

	return PlatformOptions;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGPlatformSwitchSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true,
		LOCTEXT("OutputPinTooltip", "All input will be forwarded directly to the active output pin."));

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGPlatformSwitchSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	PinProperties.Emplace(PCGPlatformSwitch::DefaultLabel, EPCGDataType::Any);

	for (FName Platform : GetSanitizedPlatforms())
	{
		PinProperties.Emplace(Platform, EPCGDataType::Any);
	}

	return PinProperties;
}

FPCGElementPtr UPCGPlatformSwitchSettings::CreateElement() const
{
	return MakeShared<FPCGPlatformSwitchElement>();
}

bool UPCGPlatformSwitchSettings::IsPlatformSelected(FName InPlatform) const
{
	FName CurrentPlatform, CurrentPlatformGroup;

	if (GetCurrentPlatformInfo(CurrentPlatform, CurrentPlatformGroup))
	{
		return IsPlatformSelected(InPlatform, CurrentPlatform, CurrentPlatformGroup);
	}

	return false;
}

bool UPCGPlatformSwitchSettings::IsPlatformSelected(FName InPlatform, FName InCurrentPlatform, FName InCurrentPlatformGroup) const
{
	ensure(!InCurrentPlatform.IsNone());

	return InPlatform == InCurrentPlatform || (InPlatform == InCurrentPlatformGroup && !InCurrentPlatformGroup.IsNone());
}

bool UPCGPlatformSwitchSettings::GetCurrentPlatformInfo(FName& OutPlatform, FName& OutPlatformGroup) const
{
#if WITH_EDITOR
	return ensure(PCGPlatformSwitch::GetCurrentPlatformInfoFromEditor(OutPlatform, OutPlatformGroup));
#else
	OutPlatform = CookedPlatform;
	OutPlatformGroup = CookedPlatformGroup;
	return true;
#endif
}

TArray<FName> UPCGPlatformSwitchSettings::GetSanitizedPlatforms() const
{
	TArray<FName> Platforms;
	Platforms.Reserve(PlatformOutputs.Num());

	for (FName Platform : PlatformOutputs)
	{
		if (!Platform.IsNone() && !Platforms.Contains(Platform))
		{
			Platforms.Add(Platform);
		}
	}

	return Platforms;
}

bool FPCGPlatformSwitchElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPlatformSwitchElement::ExecuteInternal);

	const UPCGPlatformSwitchSettings* Settings = InContext->GetInputSettings<UPCGPlatformSwitchSettings>();
	check(Settings);

	const FPCGDataCollection InputData = PCGGather::GatherDataForPin(InContext->InputData, PCGPinConstants::DefaultInputLabel, PCGPlatformSwitch::DefaultLabel);

	const TArray<FName> Platforms = Settings->GetSanitizedPlatforms();
	InContext->OutputData.TaggedData.Reserve(InputData.TaggedData.Num() * (1 + Platforms.Num()));

	FName CurrentPlatform, CurrentPlatformGroup;
	if (!Settings->GetCurrentPlatformInfo(CurrentPlatform, CurrentPlatformGroup))
	{
		return true;
	}

	bool bAnyOutputSelected = false;

	// Copy of data for each platform output.
	for (FName Platform : Platforms)
	{
		if (Settings->IsPlatformSelected(Platform, CurrentPlatform, CurrentPlatformGroup))
		{
			for (FPCGTaggedData TaggedData : InputData.TaggedData)
			{
				TaggedData.Pin = Platform;
				InContext->OutputData.TaggedData.Add(TaggedData);
			}

			bAnyOutputSelected = true;
		}
	}

	if (!bAnyOutputSelected)
	{
		// Fall back to Default pin.
		InContext->OutputData.TaggedData.Append(InputData.TaggedData);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
