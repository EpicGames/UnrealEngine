// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGPlatformSwitchBase.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPlatformSwitchBase)

#if WITH_EDITOR
void UPCGPlatformSwitchSettingsBase::PostInitProperties()
{
	Super::PostInitProperties();

	SetupPreviewPlatformChangeEvents();
}

void UPCGPlatformSwitchSettingsBase::PostLoad()
{
	Super::PostLoad();

	SetupPreviewPlatformChangeEvents();
}

void UPCGPlatformSwitchSettingsBase::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	SetupPreviewPlatformChangeEvents();
}

void UPCGPlatformSwitchSettingsBase::PostEditImport()
{
	Super::PostEditImport();

	SetupPreviewPlatformChangeEvents();
}

void UPCGPlatformSwitchSettingsBase::BeginDestroy()
{
	TearDownPreviewPlatformChangeEvents();

	Super::BeginDestroy();
}

void UPCGPlatformSwitchSettingsBase::SetupPreviewPlatformChangeEvents()
{
	if (GEditor)
	{
		if (!PreviewPlatformChangedHandle.IsValid())
		{
			PreviewPlatformChangedHandle = GEditor->OnPreviewShaderPlatformChanged().AddUObject(this, &UPCGPlatformSwitchSettingsBase::OnPreviewPlatformChanged);
		}
	}
}

void UPCGPlatformSwitchSettingsBase::TearDownPreviewPlatformChangeEvents()
{
	if (GEditor)
	{
		if (PreviewPlatformChangedHandle.IsValid())
		{
			GEditor->OnPreviewShaderPlatformChanged().Remove(PreviewPlatformChangedHandle);
		}
	}
}

void UPCGPlatformSwitchSettingsBase::OnPreviewPlatformChanged(EShaderPlatform /*NewShaderPlatform*/)
{
	// We don't broadcast structural from here. The PCGSubsystem will refresh all runtime gen components when preview platform changes.
	OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Cosmetic);
}
#endif // WITH_EDITOR

bool UPCGPlatformSwitchSettingsBase::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	if (!InPin->IsOutputPin())
	{
		return Super::IsPinUsedByNodeExecution(InPin);
	}

	return IsPinStaticallyActive(InPin->Properties.Label);
}
