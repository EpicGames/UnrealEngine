// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGQualityBranch.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGPin.h"
#include "PCGSubsystem.h"
#include "Elements/PCGGather.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGQualityBranch)

#define LOCTEXT_NAMESPACE "FPCGQualityBranchElement"

#if WITH_EDITOR
FText UPCGQualityBranchSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Control flow node that dynamically routes input data based on 'pcg.Quality' setting.");
}
#endif // WITH_EDITOR

FString UPCGQualityBranchSettings::GetAdditionalTitleInformation() const
{
	return PCGQualityHelpers::GetQualityPinLabel().ToString();
}

bool UPCGQualityBranchSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	if (!InPin || !InPin->IsOutputPin())
	{
		return Super::IsPinUsedByNodeExecution(InPin);
	}

	const FName PinLabel = InPin->Properties.Label;

	// If disabled, passthrough to Default pin.
	if (!bEnabled)
	{
		return PinLabel == PCGQualityHelpers::PinLabelDefault;
	}

	const int32 QualityLevel = UPCGSubsystem::GetPCGQualityLevel();
	bool bIsQualityPinActive = false;

	// If the Pin matches the QualityLevel, check if the pin is enabled.
	switch (QualityLevel)
	{
		case 0:
			bIsQualityPinActive = bUseLowPin;
			break;
		case 1:
			bIsQualityPinActive = bUseMediumPin;
			break;
		case 2:
			bIsQualityPinActive = bUseHighPin;
			break;
		case 3:
			bIsQualityPinActive = bUseEpicPin;
			break;
		case 4:
			bIsQualityPinActive = bUseCinematicPin;
			break;
		default:
			break;
	}

	if (bIsQualityPinActive)
	{
		return PinLabel == PCGQualityHelpers::GetQualityPinLabel();
	}
	else
	{
		return PinLabel == PCGQualityHelpers::PinLabelDefault;
	}
}

TArray<FPCGPinProperties> UPCGQualityBranchSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	PinProperties.Emplace(PCGQualityHelpers::PinLabelDefault,
		EPCGDataType::Any,
		/*bInAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("OutputPinTooltipDefault", "All unusued quality pins funnel through this default pin."));

	if (bUseLowPin)
	{
		PinProperties.Emplace(PCGQualityHelpers::PinLabelLow,
			EPCGDataType::Any,
			/*bInAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("OutputPinTooltipLow", "Active if pcg.Quality setting is set to 0."));
	}

	if (bUseMediumPin)
	{
		PinProperties.Emplace(PCGQualityHelpers::PinLabelMedium,
			EPCGDataType::Any,
			/*bInAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("OutputPinTooltipMedium", "Active if pcg.Quality setting is set to 1."));
	}

	if (bUseHighPin)
	{
		PinProperties.Emplace(PCGQualityHelpers::PinLabelHigh,
			EPCGDataType::Any,
			/*bInAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("OutputPinTooltipHigh", "Active if pcg.Quality setting is set to 2."));
	}

	if (bUseEpicPin)
	{
		PinProperties.Emplace(PCGQualityHelpers::PinLabelEpic,
			EPCGDataType::Any,
			/*bInAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("OutputPinTooltipEpic", "Active if pcg.Quality setting is set to 3."));
	}

	if (bUseCinematicPin)
	{
		PinProperties.Emplace(PCGQualityHelpers::PinLabelCinematic,
			EPCGDataType::Any,
			/*bInAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("OutputPinTooltipCinematic", "Active if pcg.Quality setting is set to 4."));
	}

	return PinProperties;
}

FPCGElementPtr UPCGQualityBranchSettings::CreateElement() const
{
	return MakeShared<FPCGQualityBranchElement>();
}

void FPCGQualityBranchElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	// Add the quality level to CRC so that we can fully refresh when quality level changes.
	Crc.Combine(UPCGSubsystem::GetPCGQualityLevel());

	OutCrc = Crc;
}

bool FPCGQualityBranchElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGQualityBranchElement::ExecuteInternal);

	const UPCGQualityBranchSettings* Settings = Context->GetInputSettings<UPCGQualityBranchSettings>();
	check(Settings);

	if (!PCGHelpers::IsRuntimeGeneration(Context->ExecutionSource.Get()))
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("UsedOutsideOfRuntimeGen", "Using Runtime Quality Branch node in a non-RuntimeGen execution domain. Set your PCG component's 'Generation Trigger' to 'Generate At Runtime'."), Context);
	}

	const int32 QualityLevel = UPCGSubsystem::GetPCGQualityLevel();

	FName ActivePinNames[PCGQualityHelpers::NumPins];
	int32 NumActivePins = 0;
	int32 ActivePinIndex = 0;

	auto ProcessQualityPin = [QualityLevel, &ActivePinIndex, &ActivePinNames, &NumActivePins](int32 PinQualityLevel, bool bIsPinActive, FName PinLabel)
	{
		if (bIsPinActive)
		{
			if (QualityLevel == PinQualityLevel)
			{
				ActivePinIndex = NumActivePins;
			}

			ActivePinNames[NumActivePins++] = PinLabel;
		}
	};

	ActivePinNames[NumActivePins++] = PCGQualityHelpers::PinLabelDefault;
	ProcessQualityPin(/*PinQualityLevel=*/0, Settings->bUseLowPin, PCGQualityHelpers::PinLabelLow);
	ProcessQualityPin(/*PinQualityLevel=*/1, Settings->bUseMediumPin, PCGQualityHelpers::PinLabelMedium);
	ProcessQualityPin(/*PinQualityLevel=*/2, Settings->bUseHighPin, PCGQualityHelpers::PinLabelHigh);
	ProcessQualityPin(/*PinQualityLevel=*/3, Settings->bUseEpicPin, PCGQualityHelpers::PinLabelEpic);
	ProcessQualityPin(/*PinQualityLevel=*/4, Settings->bUseCinematicPin, PCGQualityHelpers::PinLabelCinematic);

	// Reuse the functionality of the Gather Node
	Context->OutputData = PCGGather::GatherDataForPin(Context->InputData, PCGPinConstants::DefaultInputLabel, ActivePinNames[ActivePinIndex]);

	// Output bitmask of deactivated pins.
	const uint64 AllPinMask = (1ULL << NumActivePins) - 1;
	Context->OutputData.InactiveOutputPinBitmask = (~(1ULL << ActivePinIndex)) & AllPinMask;

	return true;
}

#undef LOCTEXT_NAMESPACE
