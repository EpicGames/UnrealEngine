// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGQualitySelect.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGPin.h"
#include "PCGSubsystem.h"
#include "Elements/PCGGather.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGQualitySelect)

#define LOCTEXT_NAMESPACE "FPCGQualitySelectElement"

#if WITH_EDITOR
FText UPCGQualitySelectSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Selects from input pins based on 'pcg.Quality' setting.");
}
#endif // WITH_EDITOR

FString UPCGQualitySelectSettings::GetAdditionalTitleInformation() const
{
	return PCGQualityHelpers::GetQualityPinLabel().ToString();
}

FPCGDataTypeIdentifier UPCGQualitySelectSettings::GetCurrentPinTypesID(const UPCGPin* Pin) const
{
	// Output pin depends on the union of all non-advanced input pins. Advanced pins are dynamic user overrides and
	// should not affect the output pin type.
	if (Pin->IsOutputPin())
	{
		TArray<FName, TInlineAllocator<32>> PinLabels;
		Algo::TransformIf(Pin->Node->GetInputPins(), PinLabels, [](const UPCGPin* InPin) { return InPin && !InPin->Properties.IsAdvancedPin();}, [](const UPCGPin* InPin) { return InPin->Properties.Label; });
		
		const FPCGDataTypeIdentifier InputTypeUnion = GetTypeUnionIDOfAllIncidentEdges(PinLabels);
		return InputTypeUnion.IsValid() ? InputTypeUnion : FPCGDataTypeIdentifier{EPCGDataType::Any};
	}
	else
	{
		return Super::GetCurrentPinTypesID(Pin);
	}
}

TArray<FPCGPinProperties> UPCGQualitySelectSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	PinProperties.Emplace(PCGQualityHelpers::PinLabelDefault,
		EPCGDataType::Any,
		/*bInAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("InputPinTooltipDefault", "This default pin will be used for any quality levels that do not have enabled pins."));

	if (bUseLowPin)
	{
		PinProperties.Emplace(PCGQualityHelpers::PinLabelLow,
			EPCGDataType::Any,
			/*bInAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("InputPinTooltipLow", "Consumed if 'pcg.Quality' setting is set to 0."));
	}

	if (bUseMediumPin)
	{
		PinProperties.Emplace(PCGQualityHelpers::PinLabelMedium,
			EPCGDataType::Any,
			/*bInAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("InputPinTooltipMedium", "Consumed if 'pcg.Quality' setting is set to 1."));
	}

	if (bUseHighPin)
	{
		PinProperties.Emplace(PCGQualityHelpers::PinLabelHigh,
			EPCGDataType::Any,
			/*bInAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("InputPinTooltipHigh", "Consumed if 'pcg.Quality' setting is set to 2."));
	}

	if (bUseEpicPin)
	{
		PinProperties.Emplace(PCGQualityHelpers::PinLabelEpic,
			EPCGDataType::Any,
			/*bInAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("InputPinTooltipEpic", "Consumed if 'pcg.Quality' setting is set to 3."));
	}

	if (bUseCinematicPin)
	{
		PinProperties.Emplace(PCGQualityHelpers::PinLabelCinematic,
			EPCGDataType::Any,
			/*bInAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("InputPinTooltipCinematic", "Consumed if 'pcg.Quality' setting is set to 4."));
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGQualitySelectSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGQualitySelectSettings::CreateElement() const
{
	return MakeShared<FPCGQualitySelectElement>();
}

void FPCGQualitySelectElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	// Add the quality level to CRC so that we can fully refresh when quality level changes.
	Crc.Combine(UPCGSubsystem::GetPCGQualityLevel());

	OutCrc = Crc;
}

bool FPCGQualitySelectElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGQualitySelectElement::ExecuteInternal);

	const UPCGQualitySelectSettings* Settings = Context->GetInputSettings<UPCGQualitySelectSettings>();
	check(Settings);

	if (!PCGHelpers::IsRuntimeGeneration(Context->ExecutionSource.Get()))
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("UsedOutsideOfRuntimeGen", "Using Runtime Quality Select node in a non-RuntimeGen execution domain. Set your PCG component's 'Generation Trigger' to 'Generate At Runtime'."), Context);
	}

	const int32 QualityLevel = UPCGSubsystem::GetPCGQualityLevel();

	FName ActivePinNames[PCGQualityHelpers::NumPins];
	int32 NumActivePins = 0;
	int32 ActivePinIndex = 0;

	auto ProcessQualityPin = [QualityLevel, &ActivePinIndex, &ActivePinNames, &NumActivePins](int32 PinQualityLevel, bool bIsPinActive, FName PinLabel)
	{
		if (bIsPinActive)
		{
			if (QualityLevel == PinQualityLevel)
			{
				ActivePinIndex = NumActivePins;
			}

			ActivePinNames[NumActivePins++] = PinLabel;
		}
	};

	ActivePinNames[NumActivePins++] = PCGQualityHelpers::PinLabelDefault;
	ProcessQualityPin(/*PinQualityLevel=*/0, Settings->bUseLowPin, PCGQualityHelpers::PinLabelLow);
	ProcessQualityPin(/*PinQualityLevel=*/1, Settings->bUseMediumPin, PCGQualityHelpers::PinLabelMedium);
	ProcessQualityPin(/*PinQualityLevel=*/2, Settings->bUseHighPin, PCGQualityHelpers::PinLabelHigh);
	ProcessQualityPin(/*PinQualityLevel=*/3, Settings->bUseEpicPin, PCGQualityHelpers::PinLabelEpic);
	ProcessQualityPin(/*PinQualityLevel=*/4, Settings->bUseCinematicPin, PCGQualityHelpers::PinLabelCinematic);

	Context->OutputData = PCGGather::GatherDataForPin(Context->InputData, ActivePinNames[ActivePinIndex], PCGPinConstants::DefaultOutputLabel);

	return true;
}

#undef LOCTEXT_NAMESPACE
