// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGShaderFeatureLevelSwitch.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Elements/PCGGather.h"
#include "Helpers/PCGHelpers.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "RHIStrings.h"
#include "Interfaces/ITargetPlatform.h"
#include "UObject/ObjectSaveContext.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGShaderFeatureLevelSwitch)

#define LOCTEXT_NAMESPACE "PCGShaderFeatureLevelSwitchElement"

namespace PCGShaderFeatureLevelSwitch
{
	static const FName DefaultLabel = TEXT("Default");

	static bool GetFeatureLevelFromName(FName InFeatureLevelName, ERHIFeatureLevel::Type& OutFeatureLevel)
	{
		if (InFeatureLevelName == FName(TEXT("ES3_1")))
		{
			OutFeatureLevel = ERHIFeatureLevel::ES3_1;
			return true;
		}
		else if (InFeatureLevelName == FName(TEXT("SM5")))
		{
			OutFeatureLevel = ERHIFeatureLevel::SM5;
			return true;
		}
		else if (InFeatureLevelName == FName(TEXT("SM6")))
		{
			OutFeatureLevel = ERHIFeatureLevel::SM6;
			return true;
		}
		else
		{
			OutFeatureLevel = ERHIFeatureLevel::Num;
			return false;
		}
	}

#if WITH_EDITOR
	static ERHIFeatureLevel::Type GetCurrentFeatureLevel()
	{
		if (GEditor && GEditor->IsPreviewPlatformActive())
		{
			// PreviewFeatureLevel is the host render level (e.g. Mac Metal SM5 renders via SM6), so derive the target level from the preview shader format to match the cook.
			if (!GEditor->PreviewPlatform.PreviewShaderFormatName.IsNone())
			{
				const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(GEditor->PreviewPlatform.PreviewShaderFormatName);
				if (ShaderPlatform != SP_NumPlatforms)
				{
					return GetMaxSupportedFeatureLevel(ShaderPlatform);
				}
			}

			return GEditor->GetActiveFeatureLevelPreviewType();
		}
		else
		{
			ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::ES3_1;

			TArray<FName> ShaderFormats;

			for (const ITargetPlatform* Platform : GetTargetPlatformManagerRef().GetActiveTargetPlatforms())
			{
				if (!Platform)
				{
					continue;
				}

				ShaderFormats.Reset();
				Platform->GetAllTargetedShaderFormats(ShaderFormats);

				for (FName Format : ShaderFormats)
				{
					const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(Format);
					const ERHIFeatureLevel::Type PlatformFeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);
					FeatureLevel = FMath::Max(FeatureLevel, PlatformFeatureLevel);
				}
			}

			return FeatureLevel;
		}
	}
#endif // WITH_EDITOR
}

bool UPCGShaderFeatureLevelSwitchSettings::IsPinStaticallyActive(const FName& PinLabel) const
{
	ERHIFeatureLevel::Type CurrentFeatureLevel = GetCurrentShaderFeatureLevel();

	if (PinLabel != PCGShaderFeatureLevelSwitch::DefaultLabel)
	{
		// Active if the platform for this pin is selected.
		return IsRHIFeatureLevelSelected(PinLabel, CurrentFeatureLevel);
	}
	else
	{
		// Default only active if no other pin is active.
		for (FName FeatureLevel : GetSanitizedRHIFeatureLevelNames())
		{
			if (IsRHIFeatureLevelSelected(FeatureLevel, CurrentFeatureLevel))
			{
				return false;
			}
		}

		return true;
	}
}

FString UPCGShaderFeatureLevelSwitchSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	FString Selection;
	if (RHIFeatureLevelOutputs.IsEmpty())
	{
		Selection = TEXT("None");
	}
	else if (RHIFeatureLevelOutputs.Num() == 1)
	{
		Selection = RHIFeatureLevelOutputs[0].ToString();
	}
	else
	{
		Selection = TEXT("Multiple");
	}

	return FString::Format(TEXT("Max Feature Level: {0}"), { Selection });
#else
	return {};
#endif
}

#if WITH_EDITOR
void UPCGShaderFeatureLevelSwitchSettings::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (ObjectSaveContext.IsCooking())
	{
		CookedFeatureLevel = PCGShaderFeatureLevelSwitch::GetCurrentFeatureLevel();
	}
}

FText UPCGShaderFeatureLevelSwitchSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Statically activates/deactivates output pins based on the maximum supported shader feature level, useful for culling downstream nodes.");
}

EPCGChangeType UPCGShaderFeatureLevelSwitchSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGShaderFeatureLevelSwitchSettings, bEnabled)
		|| InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGShaderFeatureLevelSwitchSettings, RHIFeatureLevelOutputs))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}

TArray<FName> UPCGShaderFeatureLevelSwitchSettings::GetRHIFeatureLevelOptions() const
{
	return { TEXT("ES3_1"), TEXT("SM5"), TEXT("SM6") };
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGShaderFeatureLevelSwitchSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true,
		LOCTEXT("OutputPinTooltip", "All input will be forwarded directly to the active output pin."));

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGShaderFeatureLevelSwitchSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	PinProperties.Emplace(PCGShaderFeatureLevelSwitch::DefaultLabel, EPCGDataType::Any);

	for (FName FeatureLevelName : GetSanitizedRHIFeatureLevelNames())
	{
		PinProperties.Emplace(FeatureLevelName, EPCGDataType::Any);
	}

	return PinProperties;
}

FPCGElementPtr UPCGShaderFeatureLevelSwitchSettings::CreateElement() const
{
	return MakeShared<FPCGShaderFeatureLevelSwitchElement>();
}

bool UPCGShaderFeatureLevelSwitchSettings::IsRHIFeatureLevelSelected(ERHIFeatureLevel::Type InFeatureLevel, ERHIFeatureLevel::Type InCurrentFeatureLevel) const
{
	return InFeatureLevel == InCurrentFeatureLevel;
}

bool UPCGShaderFeatureLevelSwitchSettings::IsRHIFeatureLevelSelected(FName InFeatureLevelName, ERHIFeatureLevel::Type InCurrentFeatureLevel) const
{
	ERHIFeatureLevel::Type FeatureLevel;
	return PCGShaderFeatureLevelSwitch::GetFeatureLevelFromName(InFeatureLevelName, FeatureLevel) && IsRHIFeatureLevelSelected(FeatureLevel, InCurrentFeatureLevel);
}

ERHIFeatureLevel::Type UPCGShaderFeatureLevelSwitchSettings::GetCurrentShaderFeatureLevel() const
{
#if WITH_EDITOR
	return PCGShaderFeatureLevelSwitch::GetCurrentFeatureLevel();
#else
	return static_cast<ERHIFeatureLevel::Type>(CookedFeatureLevel);
#endif
}

TArray<ERHIFeatureLevel::Type> UPCGShaderFeatureLevelSwitchSettings::GetSanitizedRHIFeatureLevels() const
{
	TArray<FName> FeatureLevelNames = GetSanitizedRHIFeatureLevelNames();

	TArray<ERHIFeatureLevel::Type> FeatureLevels;
	FeatureLevels.Reserve(FeatureLevelNames.Num());

	for (FName FeatureLevelName : FeatureLevelNames)
	{
		ERHIFeatureLevel::Type FeatureLevel;
		if (GetFeatureLevelFromName(FeatureLevelName, FeatureLevel))
		{
			FeatureLevels.Add(FeatureLevel);
		}
	}

	return FeatureLevels;
}

TArray<FName> UPCGShaderFeatureLevelSwitchSettings::GetSanitizedRHIFeatureLevelNames() const
{
	TArray<FName> FeatureLevels;
	FeatureLevels.Reserve(RHIFeatureLevelOutputs.Num());

	for (FName FeatureLevel : RHIFeatureLevelOutputs)
	{
		if (!FeatureLevel.IsNone() && !FeatureLevels.Contains(FeatureLevel))
		{
			FeatureLevels.Add(FeatureLevel);
		}
	}

	return FeatureLevels;
}

bool FPCGShaderFeatureLevelSwitchElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGShaderFeatureLevelSwitchElement::ExecuteInternal);

	const UPCGShaderFeatureLevelSwitchSettings* Settings = InContext->GetInputSettings<UPCGShaderFeatureLevelSwitchSettings>();
	check(Settings);

	const FPCGDataCollection InputData = PCGGather::GatherDataForPin(InContext->InputData, PCGPinConstants::DefaultInputLabel, PCGShaderFeatureLevelSwitch::DefaultLabel);

	const TArray<FName> FeatureLevelNames = Settings->GetSanitizedRHIFeatureLevelNames();
	InContext->OutputData.TaggedData.Reserve(InputData.TaggedData.Num() * (1 + FeatureLevelNames.Num()));

	ERHIFeatureLevel::Type CurrentFeatureLevel = Settings->GetCurrentShaderFeatureLevel();

	bool bAnyOutputSelected = false;

	// Copy of data for each supported feature level.
	for (FName FeatureLevelName : FeatureLevelNames)
	{
		if (Settings->IsRHIFeatureLevelSelected(FeatureLevelName, CurrentFeatureLevel))
		{
			for (FPCGTaggedData TaggedData : InputData.TaggedData)
			{
				TaggedData.Pin = FeatureLevelName;
				InContext->OutputData.TaggedData.Add(TaggedData);
			}

			bAnyOutputSelected = true;
		}
	}

	if (!bAnyOutputSelected)
	{
		// Fall back to Default pin.
		InContext->OutputData.TaggedData.Append(InputData.TaggedData);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
