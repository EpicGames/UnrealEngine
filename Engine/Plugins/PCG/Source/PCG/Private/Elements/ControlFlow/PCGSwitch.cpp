// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGSwitch.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Elements/PCGGather.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSwitch)

#define LOCTEXT_NAMESPACE "FPCGSwitchElement"

namespace PCGSwitchConstants
{
	const FText NodeTitleBase = LOCTEXT("NodeTitleBase", "Switch");
}

void UPCGSwitchSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// @todo_pcg To be behind a version bump in 5.5. Cannot do that in an hotfix
	// Make sure we rename the pin properties that were serialized with a localized text. Since we can't exactly match the text
	// with the enum value, we'll go with index.
	if (SelectionMode == EPCGControlFlowSelectionMode::Enum)
	{
		UPCGNode* OuterNode = Cast<UPCGNode>(GetOuter());
		if (OuterNode)
		{
			TArray<UPCGPin*> SerializedOutputPins = OuterNode->GetOutputPins();
			// It we have a num mismatch, we can't recover
			if (SerializedOutputPins.Num() == CachedPinLabels.Num())
			{
				// -1 since we don't need to check "Default"
				for (int32 i = 0; i < CachedPinLabels.Num() - 1; ++i)
				{
					if (SerializedOutputPins[i] && SerializedOutputPins[i]->Properties.Label != CachedPinLabels[i])
					{
						OuterNode->RenameOutputPin(SerializedOutputPins[i]->Properties.Label, CachedPinLabels[i], /*bBroadcastUpdate=*/false);
					}
				}
			}
		}

		if (EnumValueToPinLabelMapping.IsEmpty())
		{
			CacheEnumValueToPinLabelsMapping();
		}
	}
#endif // WITH_EDITOR

	CachePinLabels();
}

void UPCGSwitchSettings::UpdateSettings()
{
	CacheEnumValueToPinLabelsMapping();
	CachePinLabels();
}

void UPCGSwitchSettings::OnOverrideSettingsDuplicatedInternal(bool bSkippedPostLoad)
{
	Super::OnOverrideSettingsDuplicatedInternal(bSkippedPostLoad);
	if (bSkippedPostLoad)
	{
		CachePinLabels();
		// We don't want to update settings here, because we don't want to erase the enum value to pin label mapping.
	}
}

#if WITH_EDITOR
void UPCGSwitchSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Only need to change the pin labels if the options have changed
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, SelectionMode) ||
		PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, IntOptions) ||
		PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, StringOptions) ||
		PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, EnumSelection))
	{
		UpdateSettings();
	}
}

FText UPCGSwitchSettings::GetDefaultNodeTitle() const
{
	return PCGSwitchConstants::NodeTitleBase;
}

FText UPCGSwitchSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Control flow node that passes through input data to a specific output pin that matches a given selection mode and corresponding 'selection' property - which can also be overridden.");
}
#endif // WITH_EDITOR

bool UPCGSwitchSettings::IsPinStaticallyActive(const FName& PinLabel) const
{
	if (!bEnabled)
	{
		// If node disabled, active pin is first pin - first option or otherwise default.
		return PinLabel == (CachedPinLabels.IsEmpty() ? PCGPinConstants::DefaultInputLabel : CachedPinLabels[0]);
	}

	// Dynamic branches are never known in advance - assume all branches are active prior to execution.
	if (IsSwitchDynamic())
	{
		return true;
	}

	FName ActiveOutputPinLabel;
	if (!GetSelectedPinLabel(ActiveOutputPinLabel))
	{
		return false;
	}

	return PinLabel == ActiveOutputPinLabel;
}

FString UPCGSwitchSettings::GetAdditionalTitleInformation() const
{
	switch (SelectionMode)
	{
		case EPCGControlFlowSelectionMode::Integer:
		{
			FString Subtitle = PCGControlFlowConstants::SubtitleInt.ToString();
			if (!IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, IntegerSelection)))
			{
				Subtitle += FString::Format(TEXT(": {0}"), {IntegerSelection});
			}

			return Subtitle;
		}

		case EPCGControlFlowSelectionMode::Enum:
		{
			if (EnumSelection.Class)
			{
				FString Subtitle = EnumSelection.Class->GetName();
				if (!IsPropertyOverriddenByPin({GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, EnumSelection), GET_MEMBER_NAME_CHECKED(FEnumSelector, Value)}))
				{
					if (FName* EnumValueName = EnumValueToPinLabelMapping.Find(EnumSelection.Value))
					{
						Subtitle += FString::Format(TEXT(": {0}"), { EnumValueName->ToString() });
					}
				}

				return Subtitle;
			}
			else
			{
				return PCGControlFlowConstants::SubtitleEnum.ToString();
			}
		}

		case EPCGControlFlowSelectionMode::String:
		{
			FString Subtitle = PCGControlFlowConstants::SubtitleString.ToString();
			if (!IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, StringSelection)))
			{
				Subtitle += FString::Format(TEXT(": {0}"), {StringSelection});
			}

			return Subtitle;
		}

		default:
			checkNoEntry();
			break;
	}

	return PCGSwitchConstants::NodeTitleBase.ToString();
}

#if WITH_EDITOR
EPCGChangeType UPCGSwitchSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, bEnabled) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, IntegerSelection) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, StringSelection) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, EnumSelection) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, SelectionMode))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGSwitchSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel,
		EPCGDataType::Any,
		/*bInAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("OutputPinTooltip", "All input will be forwarded directly to the selected output pin."));

	InputPinProperty.SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSwitchSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	switch (SelectionMode)
	{
		case EPCGControlFlowSelectionMode::Integer:
			for (const int32 Value : IntOptions)
			{
				PinProperties.Emplace(FName(FString::FromInt(Value)));
			}
			break;

		case EPCGControlFlowSelectionMode::String:
			for (const FString& Value : StringOptions)
			{
				PinProperties.Emplace(FName(Value));
			}
			break;

		case EPCGControlFlowSelectionMode::Enum:
			if (EnumValueToPinLabelMapping.IsEmpty())
			{
				CacheEnumValueToPinLabelsMapping();
			}
			
			for (const auto& [Value, PinName] : EnumValueToPinLabelMapping)
			{
				PinProperties.Emplace(PinName);
			}
			break;

		default:
			break;
	}

	PinProperties.Emplace(PCGControlFlowConstants::DefaultPathPinLabel);

	return PinProperties;
}

FPCGElementPtr UPCGSwitchSettings::CreateElement() const
{
	return MakeShared<FPCGSwitchElement>();
}

bool UPCGSwitchSettings::IsSwitchDynamic() const
{
	bool bIsDynamic = false;

	bIsDynamic |= (SelectionMode == EPCGControlFlowSelectionMode::Integer && IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, IntegerSelection)));
	bIsDynamic |= (SelectionMode == EPCGControlFlowSelectionMode::String && IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, StringSelection)));
	bIsDynamic |= (SelectionMode == EPCGControlFlowSelectionMode::Enum && IsPropertyOverriddenByPin({GET_MEMBER_NAME_CHECKED(UPCGSwitchSettings, EnumSelection), GET_MEMBER_NAME_CHECKED(FEnumSelector, Value)}));

	return bIsDynamic;
}

bool UPCGSwitchSettings::IsValuePresent(const int32 Value) const
{
	return IntOptions.Contains(Value);
}

bool UPCGSwitchSettings::IsValuePresent(const FString& Value) const
{
	return StringOptions.Contains(Value);
}

bool UPCGSwitchSettings::IsValuePresent(const int64 Value) const
{
	if (!EnumSelection.Class)
	{
		return false;
	}

	const int64 Index = EnumSelection.Class->GetIndexByValue(Value);
	return Index != INDEX_NONE && Index < EnumSelection.Class->NumEnums() - 1;
}

int UPCGSwitchSettings::GetSelectedOutputPinIndex() const
{
	if (SelectionMode == EPCGControlFlowSelectionMode::Integer)
	{
		// Return selected pin index or if no selection matched return the index after the options which will be "Default" pin.
		const int Index = IntOptions.IndexOfByKey(IntegerSelection);
		return Index != INDEX_NONE ? Index : IntOptions.Num();
	}
	else if (SelectionMode == EPCGControlFlowSelectionMode::String)
	{
		// Return selected pin index or if no selection matched return the index after the options which will be "Default" pin.
		const int Index = StringOptions.IndexOfByKey(StringSelection);
		return Index != INDEX_NONE ? Index : StringOptions.Num();
	}
	else if (SelectionMode == EPCGControlFlowSelectionMode::Enum)
	{
		const FName* PinLabel = EnumValueToPinLabelMapping.Find(EnumSelection.Value);

		// Return index if found, otherwise fallback to the index after the options which will be "Default" pin.
		const int FoundIndex = PinLabel ? CachedPinLabels.IndexOfByKey(*PinLabel) : INDEX_NONE;
		return FoundIndex != INDEX_NONE ? FoundIndex : EnumValueToPinLabelMapping.Num();
	}

	return INDEX_NONE;
}

bool UPCGSwitchSettings::GetSelectedPinLabel(FName& OutSelectedPinLabel) const
{
	if (CachedPinLabels.IsEmpty())
	{
		return false;
	}
	
	if ((SelectionMode == EPCGControlFlowSelectionMode::Integer && IsValuePresent(IntegerSelection))
		|| (SelectionMode == EPCGControlFlowSelectionMode::String && IsValuePresent(StringSelection))
		|| (SelectionMode == EPCGControlFlowSelectionMode::Enum && IsValuePresent(EnumSelection.Value)))
	{
		const int32 Index = GetSelectedOutputPinIndex();

		if (!CachedPinLabels.IsValidIndex(Index))
		{
			return false;
		}

		OutSelectedPinLabel = CachedPinLabels[Index];
	}
	else
	{
		OutSelectedPinLabel = PCGControlFlowConstants::DefaultPathPinLabel;
	}

	return true;
}

void UPCGSwitchSettings::CachePinLabels()
{
	CachedPinLabels.Empty();
	Algo::Transform(OutputPinProperties(), CachedPinLabels, [](const FPCGPinProperties& Property)
	{
		return Property.Label;
	});
}

void UPCGSwitchSettings::CacheEnumValueToPinLabelsMapping() const
{
	EnumValueToPinLabelMapping.Empty();

	if (SelectionMode != EPCGControlFlowSelectionMode::Enum)
	{
		return;
	}
	
	// -1 to bypass the MAX value
	for (int32 Index = 0; EnumSelection.Class && Index < EnumSelection.Class->NumEnums() - 1; ++Index)
	{
		bool bHidden = false;
#if WITH_EDITOR
		// HasMetaData is editor only, so there will be extra pins at runtime, but that should be okay
		bHidden = EnumSelection.Class->HasMetaData(TEXT("Hidden"), Index) || EnumSelection.Class->HasMetaData(TEXT("Spacer"), Index);
#endif // WITH_EDITOR

		if (!bHidden)
		{
			const FString EnumDisplayName = EnumSelection.Class->GetDisplayNameTextByIndex(Index).BuildSourceString();
			EnumValueToPinLabelMapping.Emplace( EnumSelection.Class->GetValueByIndex(Index), FName(EnumDisplayName));
		}
	}
}

bool UPCGSwitchSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	if (!InPin->IsOutputPin())
	{
		return Super::IsPinUsedByNodeExecution(InPin);
	}

	// Dynamic control flow never known statically - assume all branches are active prior to execution.
	if (IsSwitchDynamic())
	{
		return true;
	}

	if (CachedPinLabels.IsEmpty())
	{
		// Labels not ready yet. Assumed used.
		return true;
	}

	const int SelectedPinIndex = GetSelectedOutputPinIndex();
	if (!CachedPinLabels.IsValidIndex(SelectedPinIndex))
	{
		ensure(false);
		return true;
	}

	// TODO disabled state? Discussing with Ryan.
	//const FName ActiveOutputPinLabel = (bEnabled && bOutputToB) ? PCGBranchConstants::OutputLabelB : PCGBranchConstants::OutputLabelA;

	return InPin->Properties.Label == CachedPinLabels[SelectedPinIndex];
}

bool FPCGSwitchElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSwitchElement::ExecuteInternal);

	const UPCGSwitchSettings* Settings = Context->GetInputSettings<UPCGSwitchSettings>();
	check(Settings);

	FName SelectedPinLabel;
	if (!Settings->GetSelectedPinLabel(SelectedPinLabel))
	{
		PCGE_LOG_C(Error, GraphAndLog, Context, LOCTEXT("ValueDoesNotExist", "Selected value is not a valid option."));
		return true;
	}

	// Reuse the functionality of the Gather node
	Context->OutputData = PCGGather::GatherDataForPin(Context->InputData, PCGPinConstants::DefaultInputLabel, SelectedPinLabel);

	// Output bitmask of deactivated pins.
	if (Context->Node)
	{
		const int NumOutputPins = Context->Node->GetOutputPins().Num();
		if (ensure(NumOutputPins > 0))
		{
			const int SelectedPinIndex = Settings->GetSelectedOutputPinIndex();
			if (ensure(SelectedPinIndex != INDEX_NONE))
			{
				const int AllPinsMask = (1 << NumOutputPins) - 1;
				Context->OutputData.InactiveOutputPinBitmask = ~(1ULL << SelectedPinIndex) & AllPinsMask;
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
