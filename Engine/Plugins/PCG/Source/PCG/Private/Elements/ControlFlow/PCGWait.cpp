// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/ControlFlow/PCGWait.h"

#include "PCGComponent.h"
#include "PCGModule.h"
#include "PCGSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGWait)

FPCGElementPtr UPCGWaitSettings::CreateElement() const
{
	return MakeShared<FPCGWaitElement>();
}

TArray<FPCGPinProperties> UPCGWaitSettings::InputPinProperties() const
{
	return {};
}

TArray<FPCGPinProperties> UPCGWaitSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DependencyPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);
	DependencyPin.Usage = EPCGPinUsage::DependencyOnly;

	return PinProperties;
}

bool FPCGWaitElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWaitElement::Execute);
	check(InContext);
	FPCGWaitContext* Context = static_cast<FPCGWaitContext*>(InContext);
	const UPCGWaitSettings* Settings = Context->GetInputSettings<UPCGWaitSettings>();
	check(Settings);

	const double CurrentTime = FPlatformTime::Seconds();
	const int64 CurrentEngineFrame = GFrameCounter;
	const int64 CurrentRenderFrame = GFrameCounterRenderThread;

	if (!Context->bQueriedTimers)
	{
		Context->StartTime = CurrentTime;
		Context->StartEngineFrame = CurrentEngineFrame;
		Context->StartRenderFrame = CurrentRenderFrame;
		Context->bQueriedTimers = true;
	}

	const bool bTimeDone = (CurrentTime - Context->StartTime) >= Settings->WaitTimeInSeconds;
	const bool bEngineFramesDone = (CurrentEngineFrame - Context->StartEngineFrame) >= (uint64)Settings->WaitTimeInEngineFrames;
	const bool bRenderFramesDone = (CurrentRenderFrame - Context->StartRenderFrame) >= (uint64)Settings->WaitTimeInRenderFrames;

	if((Settings->bEndWaitWhenAllConditionsAreMet && bTimeDone && bEngineFramesDone && bRenderFramesDone) ||
		(!Settings->bEndWaitWhenAllConditionsAreMet && (bTimeDone || bEngineFramesDone || bRenderFramesDone)))
	{
		Context->OutputData = Context->InputData;

		return true;
	}
	else
	{
		Context->bIsPaused = true;
		FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
		{
			FPCGContext::FSharedContext<FPCGWaitContext> SharedContext(ContextHandle);
			if (FPCGWaitContext* ContextPtr = SharedContext.Get())
			{
				ContextPtr->bIsPaused = false;
			}
		});

		return false;
	}
}
