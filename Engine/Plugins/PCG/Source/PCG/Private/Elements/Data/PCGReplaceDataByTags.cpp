// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Data/PCGReplaceDataByTags.h"

#include "PCGContext.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGReplaceDataByTags)

namespace PCGReplaceDataByTags
{
	namespace Constants
	{
		const FLazyName ReplacementLabel = TEXT("Replacement");
	}
}

TArray<FPCGPinProperties> UPCGReplaceDataByTagSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any).SetRequiredPin();
	PinProperties.Emplace_GetRef(PCGReplaceDataByTags::Constants::ReplacementLabel).SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGReplaceDataByTagSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGDataTypeIdentifier UPCGReplaceDataByTagSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	// In this specific instance we'll do the union of both input pins.
	if (InPin->IsOutputPin())
	{
		const FPCGDataTypeIdentifier InputTypeUnion = GetTypeUnionIDOfIncidentEdges(PCGPinConstants::DefaultInputLabel);
		const FPCGDataTypeIdentifier ReplacementTypeUnion = GetTypeUnionIDOfIncidentEdges(PCGReplaceDataByTags::Constants::ReplacementLabel);

		const FPCGDataTypeIdentifier FinalInputsUnion = (InputTypeUnion | ReplacementTypeUnion);
		return FinalInputsUnion.IsValid() ? FinalInputsUnion : InPin->Properties.AllowedTypes;
	}

	return InPin->Properties.AllowedTypes;
}

bool FPCGReplaceDataByTagElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGReplaceDataByTagElement::Execute);

	check(Context);

	const UPCGReplaceDataByTagSettings* Settings = Context->GetInputSettings<UPCGReplaceDataByTagSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData> Replacements = Context->InputData.GetInputsByPin(PCGReplaceDataByTags::Constants::ReplacementLabel);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	auto FindTag = [](const TSet<FString>& InTags, const FString& InTagToFind) -> FString
	{
		for(const FString& InTag : InTags)
		{
			// check for InTagToFind existence (either exactly the tag) or a Tag:Value.
			if (InTag == InTagToFind || InTag.StartsWith(InTagToFind + ":"))
			{
				return InTag;
			}
		}

		return FString();
	};

	TArray<FString> TagsFromSettings;
	for (const FString& Tag : Settings->Tags)
	{
		TagsFromSettings.Append(PCGHelpers::GetStringArrayFromCommaSeparatedList(Tag));
	}

	// For each input, if there is a data in the replacement set that has the same tags, then use this one instead.
	for (const FPCGTaggedData& Input : Inputs)
	{
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		// If there are no specified tags, then replace data only if there is a match in the replacements that has the same tags (and potentially more).
		for (const FPCGTaggedData& Replacement : Replacements)
		{
			bool bReplaceByThisData = false;

			// If the settings don't specify what tags to match against, then do a full inclusion test (e.g. all tags from input are in replacement data)
			if (TagsFromSettings.IsEmpty())
			{
				bReplaceByThisData = !Input.Tags.IsEmpty() && Replacement.Tags.Includes(Input.Tags);
			}
			// Otherwise, select the tags and compare them (to support Tag:Value entries)
			else
			{
				bReplaceByThisData = true;

				for (const FString& Tag : TagsFromSettings)
				{
					FString TagFromInput = FindTag(Input.Tags, Tag);
					FString TagFromReplacement = FindTag(Replacement.Tags, Tag);

					// If the tag hasn't be found (first condition) or if it has a different full tag (incl. value) then we can't use it as a replacement
					if (TagFromInput.IsEmpty() || TagFromInput != TagFromReplacement)
					{
						bReplaceByThisData = false;
						break;
					}
				}
			}

			if (bReplaceByThisData)
			{
				Output = Replacement;
				break;
			}
		}
	}

	return true;
}