// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Editor/PCGApplySplineToComponent.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGSplineData.h"
#include "Elements/PCGAddComponent.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Components/SplineComponent.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGApplySplineToComponent)

#define LOCTEXT_NAMESPACE "PCGApplySplineToComponent"

UPCGApplySplineToComponentSettings::UPCGApplySplineToComponentSettings()
{
	ComponentReferenceAttribute.SetAttributeName(PCGAddComponentConstants::ComponentReferenceAttribute);
}

TArray<FPCGPinProperties> UPCGApplySplineToComponentSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGApplySplineToComponentConstants::ComponentPinLabel, EPCGDataType::Param).SetRequiredPin();
	PinProperties.Emplace_GetRef(PCGApplySplineToComponentConstants::SplinePinLabel, EPCGDataType::Spline).SetRequiredPin();
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGApplySplineToComponentSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DepPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any);
#if WITH_EDITOR
	DepPin.Tooltip = PCGPinConstants::Tooltips::ExecutionDependencyTooltip;
#endif // WITH_EDITOR
	DepPin.Usage = EPCGPinUsage::DependencyOnly;

	return PinProperties;
}

FPCGElementPtr UPCGApplySplineToComponentSettings::CreateElement() const
{
	return MakeShared<FPCGApplySplineToComponentElement>();
}

bool FPCGApplySplineToComponentElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGApplySplineToComponentElement::Execute);
	check(Context);

	const UPCGApplySplineToComponentSettings* Settings = Context->GetInputSettings<UPCGApplySplineToComponentSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> ComponentReferenceInputs = Context->InputData.GetInputsByPin(PCGApplySplineToComponentConstants::ComponentPinLabel);
	const TArray<FPCGTaggedData> SplineInputs = Context->InputData.GetInputsByPin(PCGApplySplineToComponentConstants::SplinePinLabel);

	// Flatten all component soft-object-paths from the param pin into one array so that
	// a single param data with N entries matches N spline data objects on the spline pin.
	TArray<FSoftObjectPath> AllComponentPaths;

	for (int32 DataIndex = 0; DataIndex < ComponentReferenceInputs.Num(); ++DataIndex)
	{
		const UPCGData* Data = ComponentReferenceInputs[DataIndex].Data;
		if (!Data)
		{
			continue;
		}

		TArray<FSoftObjectPath> Paths;
		if (!PCGAttributeAccessorHelpers::ExtractAllValues(Data, Settings->ComponentReferenceAttribute, Paths, Context))
		{
			continue;
		}

		if (Paths.Num() != 1 && ComponentReferenceInputs.Num() != 1)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidComponentReferenceCardinality", "Number of entries in component reference data not expected - all attributes set should have a single entry or there should be a single attribute set."), Context);
			return true;
		}

		AllComponentPaths.Append(MoveTemp(Paths));
	}

	if (AllComponentPaths.IsEmpty())
	{
		// Nothing to do
		return true;
	}

	// Collect all spline data objects from the spline pin.
	TArray<const UPCGSplineData*> AllSplineData;
	AllSplineData.Reserve(SplineInputs.Num());

	for (const FPCGTaggedData& TaggedData : SplineInputs)
	{
		if (const UPCGSplineData* SplineData = Cast<const UPCGSplineData>(TaggedData.Data))
		{
			AllSplineData.Add(SplineData);
		}
	}

	if (AllComponentPaths.Num() != AllSplineData.Num())
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("CardinalityMismatch", "Mismatch between component reference count ({0}) and spline data count ({1})."), AllComponentPaths.Num(), AllSplineData.Num()), Context);
		return true;
	}

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("ApplyToSplineComponentTransaction", "Applying PCG Data to Spline Components"), Context->ExecutionSource.Get() && Context->ExecutionSource->GetExecutionState().UseTransactions());
#endif

	int32 UnresolvedCount = 0;
	for (int32 Index = 0; Index < AllComponentPaths.Num(); ++Index)
	{
		const FSoftObjectPath& ComponentPath = AllComponentPaths[Index];

		UObject* ResolvedObject = ComponentPath.ResolveObject();
		if (!ResolvedObject)
		{
			++UnresolvedCount;
			continue;
		}

		USplineComponent* SplineComponent = Cast<USplineComponent>(ResolvedObject);
		if (!SplineComponent)
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("NotASplineComponent", "Object at path '{0}' is not a USplineComponent and will be skipped."), FText::FromString(ComponentPath.ToString())), Context);
			continue;
		}

		SplineComponent->Modify();
		AllSplineData[Index]->ApplyTo(SplineComponent);
	}

	if (!Settings->bSilenceWarningOnUnresolvedPath && UnresolvedCount > 0)
	{
		PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("UnresolvedPaths", "Encountered {0} empty or unloaded component path(s) that were skipped."), UnresolvedCount), Context);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
