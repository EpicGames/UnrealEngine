// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Editor/PCGGetEditorCameras.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGHelpers.h"

#if WITH_EDITOR
#include "Editor.h"
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#include "Editor/UnrealEdTypes.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetEditorCameras)

#define LOCTEXT_NAMESPACE "PCGGetEditorCamerasElement"

namespace PCGGetEditorCameras
{
	namespace Constants
	{
		const FLazyName IsPerspectiveName = TEXT("IsPerspective");
		const FLazyName ViewportTypeName = TEXT("ViewportType");
		const FLazyName IsActiveName = TEXT("IsActive");
		const FLazyName FOVName = TEXT("FOV");
		const FLazyName OrthoZoomName = TEXT("OrthoZoom");
		const FLazyName NearClipPlaneName = TEXT("NearClipPlane");
		const FLazyName FarClipPlaneOverrideName = TEXT("FarClipPlaneOverride");
	}

	namespace Helpers
	{
#if WITH_EDITOR
		static FString ViewportTypeToString(ELevelViewportType InType)
		{
			switch (InType)
			{
			case LVT_Perspective: return TEXT("Perspective");
			case LVT_OrthoXY: return TEXT("Top");
			case LVT_OrthoNegativeXY: return TEXT("Bottom");
			case LVT_OrthoNegativeYZ: return TEXT("Front");
			case LVT_OrthoYZ: return TEXT("Back");
			case LVT_OrthoNegativeXZ: return TEXT("Left");
			case LVT_OrthoXZ: return TEXT("Right");
			case LVT_OrthoFreelook: return TEXT("OrthoFreelook");
			default: return TEXT("Unknown");
			}
		}
#endif
	}
}

#if WITH_EDITOR
FText UPCGGetEditorCamerasSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Editor Cameras");
}

FText UPCGGetEditorCamerasSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", 
		"Produces one point per editor viewport camera.\n"
		"Perspective viewports appear first.\n"
		"Each point carries camera attributes:\n"
		"IsPerspective, ViewportType, FOV, OrthoZoom, NearClipPlane, FarClipPlaneOverride.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGetEditorCamerasSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
	return PinProperties;
}

FPCGElementPtr UPCGGetEditorCamerasSettings::CreateElement() const
{
	return MakeShared<FPCGGetEditorCamerasElement>();
}

bool FPCGGetEditorCamerasElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetEditorCamerasElement::Execute);
	check(Context);

#if WITH_EDITOR
	if (!GEditor)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("NoEditor", "GEditor is not available; no data will be produced."));
		return true;
	}

	// Separate perspective and orthographic clients; perspective comes first.
	TArray<FEditorViewportClient*> PerspectiveClients;
	TArray<FEditorViewportClient*> OrthoClients;

	for (FEditorViewportClient* Client : GEditor->GetAllViewportClients())
	{
		if (!Client || !Client->IsVisible() || !Client->IsLevelEditorClient())
		{
			continue;
		}

		if (Client->IsPerspective())
		{
			PerspectiveClients.Add(Client);
		}
		else
		{
			OrthoClients.Add(Client);
		}
	}

	if (PerspectiveClients.IsEmpty() && OrthoClients.IsEmpty())
	{
		PCGE_LOG(Verbose, LogOnly, LOCTEXT("NoViewports", "No visible viewport clients found."));
		return true;
	}

	const int32 TotalCount = PerspectiveClients.Num() + OrthoClients.Num();

	UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(Context);
	PointData->SetNumPoints(TotalCount);
	PointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::MetadataEntry);

	TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange(/*bAllocate=*/false);
	TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange(/*bAllocate=*/false);
	TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange(/*bAllocate=*/false);

	UPCGMetadata* Metadata = PointData->MutableMetadata();
	check(Metadata);

	using namespace PCGGetEditorCameras::Constants;

	FPCGMetadataAttribute<bool>* IsPerspectiveAttribute = Metadata->FindOrCreateAttribute<bool>(IsPerspectiveName, false, /*bAllowsInterpolation=*/false);
	FPCGMetadataAttribute<FString>* ViewportTypeAttribute = Metadata->FindOrCreateAttribute<FString>(ViewportTypeName, FString(), /*bAllowsInterpolation=*/false);
	FPCGMetadataAttribute<bool>* IsActiveAttribute = Metadata->FindOrCreateAttribute<bool>(IsActiveName, false, /*bAllowsInterpolation=*/false);
	FPCGMetadataAttribute<float>* FOVAttribute = Metadata->FindOrCreateAttribute<float>  (FOVName, 0.f, /*bAllowsInterpolation=*/true);
	FPCGMetadataAttribute<float>* OrthoZoomAttribute = Metadata->FindOrCreateAttribute<float>  (OrthoZoomName, 0.f, /*bAllowsInterpolation=*/true);
	FPCGMetadataAttribute<float>* NearClipAttribute = Metadata->FindOrCreateAttribute<float>  (NearClipPlaneName, 0.f, /*bAllowsInterpolation=*/true);
	FPCGMetadataAttribute<float>* FarClipAttribute = Metadata->FindOrCreateAttribute<float>  (FarClipPlaneOverrideName, -1.f, /*bAllowsInterpolation=*/true);

	auto AddViewportPoint = [&TransformRange, &SeedRange, &MetadataEntryRange, Metadata, IsPerspectiveAttribute, ViewportTypeAttribute, IsActiveAttribute, FOVAttribute, OrthoZoomAttribute, NearClipAttribute, FarClipAttribute](FEditorViewportClient* Client, int32 PointIndex)
	{
		const FVector  Location = Client->GetViewLocation();
		const FRotator Rotation = Client->GetViewRotation();

		TransformRange[PointIndex] = FTransform(Rotation, Location);
		SeedRange[PointIndex] = PCGHelpers::ComputeSeedFromPosition(Location);
		const PCGMetadataEntryKey Entry = Metadata->AddEntry();
		MetadataEntryRange[PointIndex] = Entry;

		const bool bIsPerspective = Client->IsPerspective();
		IsPerspectiveAttribute->SetValue(Entry, bIsPerspective);
		ViewportTypeAttribute->SetValue(Entry, PCGGetEditorCameras::Helpers::ViewportTypeToString(Client->GetViewportType()));

		const bool bIsLastActive = (Client == GLastKeyLevelEditingViewportClient);
		IsActiveAttribute->SetValue(Entry, bIsLastActive);

		if (bIsPerspective)
		{
			FOVAttribute->SetValue(Entry, Client->ViewFOV);
		}
		else
		{
			OrthoZoomAttribute->SetValue(Entry, Client->GetOrthoZoom());
		}

		NearClipAttribute->SetValue(Entry, Client->GetNearClipPlane());
		FarClipAttribute->SetValue(Entry, Client->GetFarClipPlaneOverride());
	};

	int32 PointIndex = 0;
	for (FEditorViewportClient* Client : PerspectiveClients)
	{
		AddViewportPoint(Client, PointIndex++);
	}

	for (FEditorViewportClient* Client : OrthoClients)
	{
		AddViewportPoint(Client, PointIndex++);
	}

	Context->OutputData.TaggedData.Emplace_GetRef().Data = PointData;

#else // WITH_EDITOR
	PCGE_LOG(Warning, GraphAndLog, LOCTEXT("NotAvailableAtRuntime", "Get Editor Cameras is only available in editor builds."));
#endif // WITH_EDITOR

	return true;
}

#undef LOCTEXT_NAMESPACE
