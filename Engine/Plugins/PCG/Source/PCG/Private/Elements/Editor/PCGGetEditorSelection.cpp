// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Editor/PCGGetEditorSelection.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Engine/Selection.h"
#include "GameFramework/Actor.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetEditorSelection)

#define LOCTEXT_NAMESPACE "PCGGetEditorSelectionElement"

FPCGElementPtr UPCGGetEditorSelectionSettings::CreateElement() const
{
	return MakeShared<FPCGGetEditorSelectionElement>();
}

bool FPCGGetEditorSelectionElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetEditorSelectionElement::Execute);
	check(Context);

	const UPCGGetEditorSelectionSettings* Settings = Context->GetInputSettings<UPCGGetEditorSelectionSettings>();
	check(Settings);

#if WITH_EDITOR
	if (!GEditor)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("NoEditor", "GEditor is not available; no data will be produced."));
		return true;
	}

	TArray<AActor*> SelectedActors;
	if (USelection* Selection = GEditor->GetSelectedActors())
	{
		Selection->GetSelectedObjects<AActor>(SelectedActors);
	}

	if (SelectedActors.IsEmpty())
	{
		PCGE_LOG(Verbose, LogOnly, LOCTEXT("NoActorsSelected", "No actors are currently selected."));
		return true;
	}

	// All selected actors merged into a single point data element.
	UPCGBasePointData* PointData = nullptr;
	bool bAnyAttributeNameWasSanitized = false;

	for (AActor* Actor : SelectedActors)
	{
		if (!Actor)
		{
			continue;
		}

		if (!PointData)
		{
			PointData = FPCGContext::NewPointData_AnyThread(Context);
		}

		bool bAttributeNameWasSanitized = false;
		PointData->AddSinglePointFromActor(Actor, &bAttributeNameWasSanitized);
		bAnyAttributeNameWasSanitized |= bAttributeNameWasSanitized;
	}

	if (!PointData)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("OnlyInvalidActors", "All selected actors were invalid, will not produce any data."));
		return true;
	}

	if (bAnyAttributeNameWasSanitized && !Settings->bSilenceSanitizedAttributeNameWarnings)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("TagAttributeNamesSanitized", "One or more tag names contained invalid characters and were sanitized when creating the corresponding attributes."));
	}

	FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
	TaggedData.Data = PointData;

#else // WITH_EDITOR
	PCGE_LOG(Warning, GraphAndLog, LOCTEXT("NotAvailableAtRuntime", "Get Data From Selection is only available in editor builds."));
#endif // WITH_EDITOR

	return true;
}

#undef LOCTEXT_NAMESPACE
