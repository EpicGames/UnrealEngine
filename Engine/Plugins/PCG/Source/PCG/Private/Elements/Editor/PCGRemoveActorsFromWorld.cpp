// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Editor/PCGRemoveActorsFromWorld.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGActorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"

#include "GameFramework/Actor.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGRemoveActorsFromWorld)

#define LOCTEXT_NAMESPACE "PCGRemoveActorsFromWorldElement"

UPCGRemoveActorsFromWorldSettings::UPCGRemoveActorsFromWorldSettings()
{
	ActorReferenceAttribute.SetAttributeName(PCGPointDataConstants::ActorReferenceAttribute);
}

#if WITH_EDITOR
FText UPCGRemoveActorsFromWorldSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Remove Actors From World");
}

FText UPCGRemoveActorsFromWorldSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Removes actors from the world based on actor references stored in the input point data or param data.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGRemoveActorsFromWorldSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::PointOrParam);
	InputPin.SetRequiredPin();
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGRemoveActorsFromWorldSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DepPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any);
#if WITH_EDITOR
	DepPin.Tooltip = PCGPinConstants::Tooltips::ExecutionDependencyTooltip;
#endif // WITH_EDITOR
	DepPin.Usage = EPCGPinUsage::DependencyOnly;

	return PinProperties;
}

FPCGElementPtr UPCGRemoveActorsFromWorldSettings::CreateElement() const
{
	return MakeShared<FPCGRemoveActorsFromWorldElement>();
}

bool FPCGRemoveActorsFromWorldElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRemoveActorsFromWorldElement::ExecuteInternal);

	const UPCGRemoveActorsFromWorldSettings* Settings = Context->GetInputSettings<UPCGRemoveActorsFromWorldSettings>();
	check(Settings);

	// Retrieve the world through the execution source
	UWorld* World = nullptr;
	if (const IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get())
	{
		World = ExecutionSource->GetExecutionState().GetWorld();
	}

	if (!IsValid(World))
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NoWorldContext", "No valid world context available."), Context);
		return true;
	}

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	TArray<TSoftObjectPtr<AActor>> ActorsToRemove;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGData* InputData = Input.Data;
		if (!InputData)
		{
			continue;
		}

		const FPCGAttributePropertyInputSelector ActorRefSelector = Settings->ActorReferenceAttribute.CopyAndFixLast(InputData);
		TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, ActorRefSelector);
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, ActorRefSelector);

		if (!Accessor.IsValid() || !Keys.IsValid())
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(ActorRefSelector, Context);
			continue;
		}

		const int32 NumEntries = Keys->GetNum();
		if (NumEntries == 0)
		{
			continue;
		}

		auto GatherActors = [&ActorsToRemove, Context, Settings](const TArrayView<FSoftObjectPath>& ActorPaths, int Start, int Range)
		{
			for (const FSoftObjectPath& ActorPath : ActorPaths)
			{
				if (ActorPath.IsNull())
				{
					continue;
				}

				AActor* Actor = TSoftObjectPtr<AActor>(ActorPath).Get();
				if (!IsValid(Actor))
				{
					if (!Settings->bSilenceActorNotFoundWarnings)
					{
						PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("ActorNotFound", "Could not find actor '{0}' in the world. It may not be loaded or may have already been removed."), FText::FromString(ActorPath.ToString())), Context);
					}
					continue;
				}

				ActorsToRemove.Emplace(Actor);
			}
		};

		if (!PCGMetadataElementCommon::ApplyOnAccessorRange<FSoftObjectPath>(*Keys, *Accessor, GatherActors, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
		{
			PCGLog::Metadata::LogFailToGetAttributeError<FString>(ActorRefSelector, Accessor.Get(), Context);
		}
	}

	if (!ActorsToRemove.IsEmpty())
	{
#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("RemoveActorsFromPCG", "Removing actors from PCG"), Context->ExecutionSource->GetExecutionState().UseTransactions());
#endif
		UPCGActorHelpers::DeleteActors(World, ActorsToRemove);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
