// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Grammar/PCGDuplicateCrossSections.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGSplineData.h"
#include "Grammar/PCGGrammarParser.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "UObject/UE5ReleaseStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDuplicateCrossSections)

#define LOCTEXT_NAMESPACE "PCGDuplicateCrossSectionsElement"

void UPCGDuplicateCrossSectionsSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::ExtraOutputAttributesOnDataDomainPCG)
	{
		// For all previous nodes, we'll force this option to false. for retro-compatibility
		bExtraOutputAttributesOnDataDomain = false;
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
FName UPCGDuplicateCrossSectionsSettings::GetDefaultNodeName() const
{
	return FName(TEXT("DuplicateCrossSections"));
}

FText UPCGDuplicateCrossSectionsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Duplicate Cross-Sections");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGDuplicateCrossSectionsSettings::CreateElement() const
{
	return MakeShared<FPCGDuplicateCrossSectionsElement>();
}

TArray<FPCGPinProperties> UPCGDuplicateCrossSectionsSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Result;
	FPCGPinProperties& InputPin = Result.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spline);
	InputPin.SetRequiredPin();

	if (bModuleInfoAsInput)
	{
		FPCGPinProperties& ModuleInfoPin = Result.Emplace_GetRef(PCGSubdivisionBase::Constants::ModulesInfoPinLabel, EPCGDataType::Param);
		ModuleInfoPin.SetRequiredPin();
	}

	return Result;
}

TArray<FPCGPinProperties> UPCGDuplicateCrossSectionsSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Result;
	Result.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spline);

	return Result;
}

bool FPCGDuplicateCrossSectionsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDuplicateCrossSectionsElement::Execute);

	check(InContext);

	const UPCGDuplicateCrossSectionsSettings* Settings = InContext->GetInputSettings<UPCGDuplicateCrossSectionsSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	const UPCGParamData* ModuleInfoParamData = nullptr;
	PCGSubdivisionBase::FModuleInfoMap ModulesInfo = GetModulesInfoMap(InContext, Settings, ModuleInfoParamData);

	const bool bMatchAndSetAttributes = Settings->bForwardAttributesFromModulesInfo && ModuleInfoParamData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGSplineData* InputSplineData = Cast<const UPCGSplineData>(Input.Data);

		if (!InputSplineData)
		{
			continue;
		}

		const TArray<FInterpCurvePointVector>& ControlPoints = InputSplineData->SplineStruct.GetSplinePointsPosition().Points;
		if (ControlPoints.Num() < 2)
		{
			continue;
		}

		FVector ExtrudeVector = Settings->ExtrudeVector;

		auto GetValueFromAttribute = [InputSplineData, InContext]<typename T>(const FPCGAttributePropertyInputSelector& InSelector, T& OutValue) -> bool
		{
			const FPCGAttributePropertyInputSelector Selector = InSelector.CopyAndFixLast(InputSplineData);
			const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputSplineData, Selector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InputSplineData, Selector);
			if (!Accessor || !Keys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, InContext);
				return false;
			}

			if (!Accessor->Get(OutValue, *Keys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::Metadata::LogFailToGetAttributeError<T>(Selector, Accessor.Get(), InContext);
				return false;
			}

			return true;
		};

		if (Settings->bExtrudeVectorAsAttribute && !GetValueFromAttribute(Settings->ExtrudeVectorAttribute, ExtrudeVector))
		{
			continue;
		}

		const double ExtrudeLength = ExtrudeVector.Length();
		if (FMath::IsNearlyZero(ExtrudeLength))
		{
			continue;
		}

		// Set seed if required
		int32 AdditionalSeed = 0;
		if (Settings->bUseSeedAttribute)
		{
			const FPCGAttributePropertyInputSelector Selector = Settings->SeedAttribute.CopyAndFixLast(InputSplineData);
			TUniquePtr<const IPCGAttributeAccessor> SeedAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputSplineData, Selector);
			TUniquePtr<const IPCGAttributeAccessorKeys> SeedAccessorKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputSplineData, Selector);

			if (!SeedAccessor || !SeedAccessorKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, InContext);
			}
			// Otherwise, get the value, if it fails, the attribute wasn't compatible
			else if (!SeedAccessor->Get(AdditionalSeed, *SeedAccessorKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::Metadata::LogFailToGetAttributeError<int32>(Selector, SeedAccessor.Get(), InContext);
			}
		}

		const FVector ExtrudeDirection = ExtrudeVector / ExtrudeLength;

		double MinSize = 0.0;
		PCGGrammar::FTokenizedGrammar TokenizedGrammar = GetTokenizedGrammar(InContext, InputSplineData, Settings, ModulesInfo, MinSize);

		if (!TokenizedGrammar.IsValid())
		{
			continue;
		}

		TArray<PCGSubdivisionBase::TModuleInstance<PCGGrammar::FTokenizedModule>> Instances;
		double RemainingLength = 0.0;
		const bool bHeightSubdivideSuccess = PCGSubdivisionBase::Subdivide(*TokenizedGrammar.ModuleGrammar, ExtrudeLength, Instances, RemainingLength, InContext, AdditionalSeed);

		if (!bHeightSubdivideSuccess)
		{
			if (!Settings->bSilenceGrammarDoesNotFitErrors)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("GrammarDoesNotFit", "Grammar does not fit for this segment."), InContext);
			}

			continue;
		}

		FVector CurrentDisplacement = FVector::ZeroVector;
		int32 SplineIndex = 0;

		for (const PCGSubdivisionBase::TModuleInstance<PCGGrammar::FTokenizedModule>& Instance : Instances)
		{
			const FName Symbol = Instance.Module->Descriptor->Symbol;
			const FPCGSubdivisionSubmodule& CurrentBlock = ModulesInfo[Symbol];
			const FVector Size = ExtrudeDirection * CurrentBlock.Size * (FVector::OneVector + Instance.ExtraScale);
			
			FPCGSplineStruct NewSpline = InputSplineData->SplineStruct;
			NewSpline.Transform.AddToTranslation(CurrentDisplacement);

			UPCGSplineData* NewSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(InContext);
			NewSplineData->Initialize(NewSpline);
			NewSplineData->InitializeFromData(InputSplineData);

			check(NewSplineData->MutableMetadata());

			FPCGMetadataDomain* MetadataDomain = Settings->bExtraOutputAttributesOnDataDomain
				? NewSplineData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)
				: NewSplineData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements);

			check(MetadataDomain);

			MetadataDomain->FindOrCreateAttribute<FName>(Settings->SymbolAttributeName, Symbol, false, false);

			if (!bMatchAndSetAttributes && Settings->bOutputDebugColorAttribute)
			{
				MetadataDomain->FindOrCreateAttribute<FVector4>(Settings->DebugColorAttributeName, FVector4(CurrentBlock.DebugColor, 1.0), false, false);
			}

			if (Settings->bOutputSizeAttribute)
			{
				MetadataDomain->FindOrCreateAttribute<FVector>(Settings->SizeAttributeName, Size, false, false);
			}

			if (!bMatchAndSetAttributes && Settings->bOutputScalableAttribute)
			{
				MetadataDomain->FindOrCreateAttribute<bool>(Settings->ScalableAttributeName, CurrentBlock.bScalable, false, false);
			}

			if (Settings->bOutputSplineIndexAttribute)
			{
				MetadataDomain->FindOrCreateAttribute<int32>(Settings->SplineIndexAttributeName, SplineIndex++, false, false);
			}

			Outputs.Add_GetRef(Input).Data = NewSplineData;
			CurrentDisplacement += Size;
		}
	}

	if (bMatchAndSetAttributes)
	{
		MatchAndSetAttributes(Inputs, Outputs, ModuleInfoParamData, Settings, Settings->bExtraOutputAttributesOnDataDomain ? PCGMetadataDomainID::Data : PCGMetadataDomainID::Elements);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
