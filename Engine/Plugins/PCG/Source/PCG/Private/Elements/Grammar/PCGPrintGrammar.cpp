// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Grammar/PCGPrintGrammar.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Grammar/PCGGrammarParser.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPrintGrammar)

#define LOCTEXT_NAMESPACE "PCGPrintGrammarElement"

namespace PCGPrintGrammarConstants
{
	const FName OutputLabel(TEXT("Description"));
}

TArray<FPCGPinProperties> UPCGPrintGrammarSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPrintGrammarConstants::OutputLabel, EPCGDataType::Param);
	return PinProperties;
}

FPCGElementPtr UPCGPrintGrammarSettings::CreateElement() const
{
	return MakeShared<FPCGPrintGrammarElement>();
}

bool FPCGPrintGrammarElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPrintGrammarElement::Execute);

	const UPCGPrintGrammarSettings* Settings = Context->GetInputSettings<UPCGPrintGrammarSettings>();

	FPCGGrammarResult Result = PCGGrammar::Parse(Settings->Grammar);

	TStringBuilder<0> Description;

	for (const FPCGGrammarResult::FLog& Log : Result.GetLogs())
	{
		switch (Log.Verbosity)
		{
			case FPCGGrammarResult::ELogType::Log:
				UE_LOGF(LogPCG, Log, "%ls", *Log.Message.ToString());
				break;
			case FPCGGrammarResult::ELogType::Warning:
				PCGLog::LogWarningOnGraph(Log.Message, Context);
				break;
			case FPCGGrammarResult::ELogType::Error:
				PCGLog::LogErrorOnGraph(Log.Message, Context);
				break;
			default:
				checkNoEntry();
				break;
		}
	}

	if (!Result.bSuccess)
	{
		return true;
	}

	auto PrintDescriptor = [&Description](const PCGGrammar::FModuleDescriptor& Descriptor, auto&& PrintRecurse) -> void
		{
			Description.Append("Module[");
			switch (Descriptor.Type)
			{
			case PCGGrammar::EModuleType::Root:
				Description.Append("Root");
				break;
			case PCGGrammar::EModuleType::Literal:
				Description.Append("Literal: ");
				Description.Append(Descriptor.Symbol.ToString());
				Description.Append(" Weight: ");
				Description.Append(FString::FromInt(Descriptor.Weight));
				break;
			case PCGGrammar::EModuleType::Sequence:
				Description.Append("Sequence");
				break;
			case PCGGrammar::EModuleType::Stochastic:
				Description.Append("Stochastic");
				break;
			case PCGGrammar::EModuleType::Priority:
				Description.Append("Priority");
				break;
			}

			Description.Append("] - Repetitions: ");
			if (Descriptor.Repetitions == PCGGrammar::InfiniteRepetition)
			{
				Description.Append("0 or more");
			}
			else if(Descriptor.Repetitions == PCGGrammar::AtLeastOneRepetition)
			{
				Description.Append("1 or more");
			}
			else
			{
				Description.Append(FString::FromInt(Descriptor.Repetitions));
			}
			Description.Append("\n");

			for (const PCGGrammar::FModuleDescriptor& Submodule : Descriptor.Submodules)
			{
				PrintRecurse(Submodule, PrintRecurse);
			}
		};

	PrintDescriptor(Result.Root, PrintDescriptor);

	FPCGTaggedData& OutData = Context->OutputData.TaggedData.Emplace_GetRef();

	const UPCGParamData* OutParamData = NewObject<UPCGParamData>();
	UPCGMetadata* Metadata = OutParamData->Metadata;
	check(OutParamData && Metadata);

	OutData.Data = OutParamData;

	if (FPCGMetadataAttribute<FString>* Attribute = Metadata->CreateAttribute(PCGPrintGrammarConstants::OutputLabel, FString(), false, true))
	{
		Attribute->SetValue(0, Description);
	}
	else
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("CreateAttributeFailed", "Couldn't create new attribute for the description."), Context);
		return true;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
