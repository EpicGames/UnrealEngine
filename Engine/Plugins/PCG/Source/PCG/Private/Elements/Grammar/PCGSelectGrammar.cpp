// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Grammar/PCGSelectGrammar.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include "Algo/Count.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSelectGrammar)

#define LOCTEXT_NAMESPACE "PCGSelectGrammarElement"

namespace PCGSelectGrammar
{
namespace Constants
{
	const FName CriteriaDataPinLabel = TEXT("Selection Criteria Data");
	const FName OutputGrammarAttributeName = TEXT("Grammar");
	// Most likely used value for comparison with grammars
	const FString ComparedValueAttributeString = TEXT("$ScaledLocalSize.X");

	constexpr uint16 BinaryEnumIndex = static_cast<uint16>(EPCGSelectGrammarComparator::BinaryOps) + 1;
	constexpr uint16 TernaryEnumIndex = static_cast<uint16>(EPCGSelectGrammarComparator::TernaryOps) + 1;
}

struct FGrammarSizePair
{
	FGrammarSizePair(const FString& InGrammar, double InSize)
		: Grammar(InGrammar)
		, Size(InSize)
	{}

	FString Grammar;
	double Size;
};

// Function indices must match EPCGSelectGrammarComparator indices directly
template <typename T>
using TBinarySignature = bool(*)(const T&, const T&);

template <typename T>
static constexpr TBinarySignature<T> BinaryCompare[] =
{
	[](const T&, const T&) { return true; }, // Select
	PCG::Private::MetadataTraits<T>::Less,
	PCG::Private::MetadataTraits<T>::LessOrEqual,
	[](const T& LHS, const T& RHS) { return PCG::Private::MetadataTraits<T>::Equal(LHS, RHS); }, // Adapter for 'Equal', as it changed from binary to ternary with tolerance
	PCG::Private::MetadataTraits<T>::GreaterOrEqual,
	PCG::Private::MetadataTraits<T>::Greater,
};

template <typename T>
using TTernarySignature = bool(*)(const T&, const T&, const T&);

template <typename T>
static constexpr TTernarySignature<T> TernaryCompare[] =
{
	// Exclusive range
	[](const T& Value, const T& Start, const T& End) { return PCG::Private::MetadataTraits<T>::Greater(Value, Start) && PCG::Private::MetadataTraits<T>::Less(Value, End); },
	// Inclusive range
	[](const T& Value, const T& Start, const T& End) { return PCG::Private::MetadataTraits<T>::GreaterOrEqual(Value, Start) && PCG::Private::MetadataTraits<T>::LessOrEqual(Value, End); }
};
}

UPCGSelectGrammarSettings::UPCGSelectGrammarSettings()
{
	ComparedValueAttribute.Update(PCGSelectGrammar::Constants::ComparedValueAttributeString);
	OutputGrammarAttribute.SetAttributeName(PCGSelectGrammar::Constants::OutputGrammarAttributeName);
}

#if WITH_EDITOR
FName UPCGSelectGrammarSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SelectGrammar"));
}

FText UPCGSelectGrammarSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Select Grammar");
}

FText UPCGSelectGrammarSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Select a grammar by comparing an input attribute against a provided criteria.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGSelectGrammarSettings::CreateElement() const
{
	return MakeShared<FPCGSelectGrammarElement>();
}

void UPCGSelectGrammarSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (!Criteria.IsEmpty())
	{
		for (FPCGSelectGrammarCriterion& Criterion : Criteria)
		{
			if (Criterion.Comparator == EPCGSelectGrammarComparator::BinaryOps)
			{
				Criterion.Comparator = EPCGSelectGrammarComparator::Select;
			}
			else if (Criterion.Comparator == EPCGSelectGrammarComparator::TernaryOps)
			{
				Criterion.Comparator = EPCGSelectGrammarComparator::RangeExclusive;
			}
		}
	}
#endif // WITH_EDITOR
}

TArray<FPCGPinProperties> UPCGSelectGrammarSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, FPCGDataTypeIdentifier::Construct(FPCGDataTypeInfoPoint::AsId(), FPCGDataTypeInfoSpline::AsId()));
	InputPin.SetRequiredPin();
#if WITH_EDITOR
	InputPin.Tooltip = LOCTEXT("InputPinTooltip", "A grammar will be selected based on criteria for all points.");
#endif // WITH_EDITOR

	if (bCriteriaAsInput)
	{
		FPCGPinProperties& CriteriaPin = PinProperties.Emplace_GetRef(PCGSelectGrammar::Constants::CriteriaDataPinLabel, EPCGDataType::Param, false, false);
		CriteriaPin.SetRequiredPin();
#if WITH_EDITOR
		CriteriaPin.Tooltip = LOCTEXT("CriteriaPinTooltip", "Criteria to compare against an input value for conditionally selecting a grammar. Will be evaluated in sequential order.");
#endif // WITH_EDITOR
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSelectGrammarSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeIdentifier::Construct(FPCGDataTypeInfoPoint::AsId(), FPCGDataTypeInfoSpline::AsId()));

	return PinProperties;
}

bool FPCGSelectGrammarElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelectGrammarElement::Execute);

	const UPCGSelectGrammarSettings* Settings = InContext->GetInputSettings<UPCGSelectGrammarSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	const TArray<FPCGTaggedData> CriteriaDataInputs = InContext->InputData.GetInputsByPin(PCGSelectGrammar::Constants::CriteriaDataPinLabel);

	if (Inputs.IsEmpty())
	{
		return true;
	}

	TArray<FPCGSelectGrammarCriterion> Criteria = Settings->bCriteriaAsInput ? TArray<FPCGSelectGrammarCriterion>{} : Settings->Criteria;
	if (Settings->bCriteriaAsInput && !CriteriaDataInputs.IsEmpty())
	{
		if (CriteriaDataInputs.Num() != 1)
		{
			PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGSelectGrammar::Constants::CriteriaDataPinLabel, InContext);
		}

		// Should only be one, so take the first
		if (const UPCGParamData* CriteriaParamData = Cast<UPCGParamData>(CriteriaDataInputs[0].Data))
		{
			check(CriteriaParamData->Metadata);

			const FPCGSelectGrammarCriteriaAttributeNames& CriteriaAttributeNames =
					Settings->bRemapCriteriaAttributeNames
						? Settings->CriteriaAttributeNames
						: FPCGSelectGrammarCriteriaAttributeNames();

			// Attempt an extraction of attribute set matching 'FPCGSelectGrammarCriterion'
			TMap<FName, TTuple<FName, bool>> PropertyNameMapping;
			PropertyNameMapping.Emplace(GET_MEMBER_NAME_CHECKED(FPCGSelectGrammarCriterion, Key), {CriteriaAttributeNames.KeyAttributeName, /*bCanBeDefaulted=*/true});
			PropertyNameMapping.Emplace(GET_MEMBER_NAME_CHECKED(FPCGSelectGrammarCriterion, FirstValue), {CriteriaAttributeNames.FirstValueAttributeName, /*bCanBeDefaulted=*/true});
			PropertyNameMapping.Emplace(GET_MEMBER_NAME_CHECKED(FPCGSelectGrammarCriterion, Comparator), {CriteriaAttributeNames.ComparatorAttributeName, /*bCanBeDefaulted=*/true});
			PropertyNameMapping.Emplace(GET_MEMBER_NAME_CHECKED(FPCGSelectGrammarCriterion, SecondValue), {CriteriaAttributeNames.SecondValueAttributeName, /*bCanBeDefaulted=*/true});
			PropertyNameMapping.Emplace(GET_MEMBER_NAME_CHECKED(FPCGSelectGrammarCriterion, Grammar), {CriteriaAttributeNames.GrammarAttributeName, /*bCanBeDefaulted=*/true});

			Criteria = PCGPropertyHelpers::ExtractAttributeSetAsArrayOfStructs<FPCGSelectGrammarCriterion>(CriteriaParamData, &PropertyNameMapping, InContext);
		}
		else
		{
			PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::Param, PCGSelectGrammar::Constants::CriteriaDataPinLabel, InContext);
			return true;
		}
	}

	// Early out with nothing to compare to
	if (Criteria.IsEmpty())
	{
		InContext->OutputData = InContext->InputData;
		return true;
	}

	// Sum the number of select criteria to check if they are all select.
	int32 NumSelectCriteria = 0;
	// Validate the criteria, to ensure the ranges are in order, etc
	for (const FPCGSelectGrammarCriterion& Criterion : Criteria)
	{
		switch (Criterion.Comparator)
		{
			case EPCGSelectGrammarComparator::Select:
			{
				++NumSelectCriteria;
			}
			break;

			case EPCGSelectGrammarComparator::RangeExclusive: // fall-through
			case EPCGSelectGrammarComparator::RangeInclusive:
			{
				if (Criterion.FirstValue > Criterion.SecondValue)
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidCriteriaRange", "Criteria range is inverted for criterion with Key->Grammar: '{0}'->'{1}'"), FText::FromName(Criterion.Key), FText::FromString(Criterion.Grammar)), InContext);
					return true;
				}
			}
			break;

			default:
				break;
		}
	}

	// In this case, all the comparators are 'Select', so bypass the criteria checks.
	const bool bBypassCriteria = NumSelectCriteria == Criteria.Num();

	// Pre-process the criteria into a map for O(1) access later
	TMap<FName, TArray<FPCGSelectGrammarCriterion, TInlineAllocator<16>>> CriteriaMap;
	for (FPCGSelectGrammarCriterion& Criterion : Criteria)
	{
		if (!CriteriaMap.Contains(Criterion.Key))
		{
			CriteriaMap.Emplace(Criterion.Key, {});
		}

		CriteriaMap[Criterion.Key].Emplace(std::move(Criterion));
	}

	for (const TTuple<FName, TArray<FPCGSelectGrammarCriterion, TInlineAllocator<16>>>& Tuple : CriteriaMap)
	{
		const int32 SelectionCount = Algo::CountIf(Tuple.Get<1>(), [](const FPCGSelectGrammarCriterion& InCriterion) { return InCriterion.Comparator == EPCGSelectGrammarComparator::Select; });
		if (SelectionCount > 1)
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("MultipleSelectSameCriteria", "Multiple 'Select' comparators found on the criteria with same key: {0}"), FText::FromName(Tuple.Get<0>())), InContext);
		}
	}

	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;
	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGData* InputData = Input.Data;
		if (!InputData || (!InputData->IsA<UPCGBasePointData>() && !InputData->IsA<UPCGSplineData>()))
		{
			continue;
		}

		FPCGAttributePropertyInputSelector KeySelector = Settings->KeyAttribute.CopyAndFixLast(InputData);
		TUniquePtr<const IPCGAttributeAccessor> InputKeyAccessor;
		TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys;

		if (Settings->bKeyAsAttribute)
		{
			InputKeyAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, KeySelector);
			InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, KeySelector);
		}
		else
		{
			InputKeyAccessor = MakeUnique<const FPCGConstantValueAccessor<FName>>(Settings->Key);
			// Dummy single key
			InputKeys = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();
		}
		
		if (!InputKeyAccessor || !InputKeys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(KeySelector, InContext);
			continue;
		}

		UPCGData* OutputData = InputData->DuplicateData(InContext);
		Outputs.Emplace_GetRef(Input).Data = OutputData;

		FPCGAttributePropertyOutputSelector OutputGrammarSelector = Settings->OutputGrammarAttribute.CopyAndFixSource(&KeySelector);
		FPCGMetadataDomain* MetadataDomain = OutputData->MutableMetadata()->GetMetadataDomainFromSelector(OutputGrammarSelector);
		FPCGMetadataAttribute<FString>* OutputAttribute = MetadataDomain->FindOrCreateAttribute(OutputGrammarSelector.GetName(), FString{}, /*bAllowsInterpolation=*/false);
		TUniquePtr<IPCGAttributeAccessor> WriteGrammarAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputAttribute, MetadataDomain);
		TUniquePtr<IPCGAttributeAccessorKeys> OutputKeys = PCGAttributeAccessorHelpers::CreateKeys(OutputData, OutputGrammarSelector);
		if (!WriteGrammarAccessor || !OutputKeys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(OutputGrammarSelector, InContext);
			continue;
		}

		auto ProcessWithoutCriteria = [&WriteGrammarAccessor, &OutputKeys, &CriteriaMap, bPassThroughKey = Settings->bCopyKeyForUnselectedGrammar](const TArrayView<FName>& KeyValues, int32 Start, int32 Range)
		{
			TArray<FString, TInlineAllocator<PCGMetadataElementCommon::DefaultChunkSize>> Grammars;
			Grammars.SetNum(Range);

			// Iterate through this chunk of points and evaluate the selection criteria for each point
			for (int32 i = 0; i < Range; ++i)
			{
				const FName& Key = KeyValues[i];
				if (!CriteriaMap.Contains(Key) || CriteriaMap[Key].IsEmpty())
				{
					continue;
				}

				Grammars[i] = CriteriaMap[Key][0].Grammar;
			}

			WriteGrammarAccessor->SetRange<FString>(Grammars, Start, *OutputKeys);
		};

		auto ProcessWithCriteria = [&WriteGrammarAccessor, &OutputKeys, &CriteriaMap, bPassThroughKey = Settings->bCopyKeyForUnselectedGrammar](const TArrayView<FName>& KeyValues, const TArrayView<double>& CompareValues, int32 Start, int32 Range)
		{
			TArray<FString, TInlineAllocator<PCGMetadataElementCommon::DefaultChunkSize>> Grammars;
			Grammars.SetNum(Range);

			// Iterate through this chunk of points and evaluate the selection criteria for each point
			for (int32 i = 0; i < Range; ++i)
			{
				bool bFoundGrammar = false;

				const FName& Key = KeyValues[i];
				if (!CriteriaMap.Contains(Key) || CriteriaMap[Key].IsEmpty())
				{
					continue;
				}

				const TArray<FPCGSelectGrammarCriterion, TInlineAllocator<16>>& CriteriaForThisKey = CriteriaMap[Key];
				for (const FPCGSelectGrammarCriterion& Criterion : CriteriaForThisKey)
				{
					check(Key == Criterion.Key);

					// Find the comparison function based on the enum selection
					const uint16 EnumIndex = static_cast<uint16>(Criterion.Comparator);
					if (EnumIndex < PCGSelectGrammar::Constants::TernaryEnumIndex) // Binary comparison
					{
						// Convert enum index to static function array index
						const uint16 CompIndex = EnumIndex - PCGSelectGrammar::Constants::BinaryEnumIndex;
						check(CompIndex < std::size(PCGSelectGrammar::BinaryCompare<double>))
						if (PCGSelectGrammar::BinaryCompare<double>[CompIndex](CompareValues[i], Criterion.FirstValue))
						{
							bFoundGrammar = true;
							Grammars[i] = Criterion.Grammar;
							break;
						}
					}
					else // Ternary comparison
					{
						const uint16 CompIndex = EnumIndex - PCGSelectGrammar::Constants::TernaryEnumIndex;
						check(CompIndex < std::size(PCGSelectGrammar::TernaryCompare<double>))
						if (PCGSelectGrammar::TernaryCompare<double>[CompIndex](CompareValues[i], Criterion.FirstValue, Criterion.SecondValue))
						{
							bFoundGrammar = true;
							Grammars[i] = Criterion.Grammar;
							break;
						}
					}
				}

				if (bPassThroughKey && !bFoundGrammar)
				{
					Grammars[i] = KeyValues[i].ToString();
				}
			}

			WriteGrammarAccessor->SetRange<FString>(Grammars, Start, *OutputKeys);
		};

		if (bBypassCriteria)
		{
			PCGMetadataElementCommon::ApplyOnAccessorRange<FName>(*InputKeys, *InputKeyAccessor, ProcessWithoutCriteria);
		}
		else
		{
			FPCGAttributePropertyInputSelector ComparedValueSelector = Settings->ComparedValueAttribute.CopyAndFixLast(InputData);
			TUniquePtr<const IPCGAttributeAccessor> ComparedValuesAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, ComparedValueSelector);
			TUniquePtr<const IPCGAttributeAccessorKeys> ComparedValuesKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, ComparedValueSelector);

			if (!ComparedValuesAccessor || !ComparedValuesKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(ComparedValueSelector, InContext);
				continue;
			}

			// TODO: For now, cast everything to double for numerical comparison, but this could easily expand in the future
			TStaticArray<const IPCGAttributeAccessor*, 2> Accessors = {InputKeyAccessor.Get(), ComparedValuesAccessor.Get()};
			TStaticArray<const IPCGAttributeAccessorKeys*, 2> Keys = {InputKeys.Get(), ComparedValuesKeys.Get()};

			// Technically we can mix Data and Elements domains, but if the output grammar goes into the data domain, we don't need to process more than one. (Effectively evaluating only the first element).
			const int32 Count = OutputKeys->GetNum();
			PCGMetadataElementCommon::ApplyOnMultiAccessorsRange<FName, double>(Keys, Accessors, ProcessWithCriteria, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible, PCGMetadataElementCommon::DefaultChunkSize, Count);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
