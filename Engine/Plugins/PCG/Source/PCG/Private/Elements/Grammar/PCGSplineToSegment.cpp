// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Grammar/PCGSplineToSegment.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplineToSegment)

#define LOCTEXT_NAMESPACE "PCGSegmentSplineElement"

namespace PCGSplineToSegment
{
	static const FName PreviousTangentAttributeName = TEXT("PreviousTangent");
	static const FName NextTangentAttributeName = TEXT("NextTangent");
	static const FName SegmentIndexAttributeName = TEXT("SegmentIndex");
	static const FName PreviousIndexAttributeName = TEXT("SegmentPreviousIndex");
	static const FName NextIndexAttributeName = TEXT("SegmentNextIndex");
	static const FName PreviousAngleAttributeName = TEXT("PreviousAngle");
	static const FName NextAngleAttributeName = TEXT("NextAngle");
	static const FName ClockwiseAttributeName = TEXT("Clockwise");

	template <typename T>
	FPCGMetadataAttribute<T>* CreateAttributeWithLog(const FName AttributeName, const bool bAllowInterpolation, UPCGMetadata* Metadata, FPCGContext* Context, T DefaultValue = PCG::Private::MetadataTraits<T>::ZeroValue())
	{
		FPCGMetadataAttribute<T>* Attribute = Metadata->FindOrCreateAttribute<T>(AttributeName, DefaultValue, bAllowInterpolation, false);
		if (!Attribute)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FailCreateAttribute", "Failed to create attribute {0}."), FText::FromName(AttributeName)), Context);
		}

		return Attribute;
	}
}

#if WITH_EDITOR
FName UPCGSplineToSegmentSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SplineToSegment"));
}

FText UPCGSplineToSegmentSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Spline to Segment");
}

FText UPCGSplineToSegmentSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Take a spline as input and create a point data, with each point being a segment defined by 2 connected control points.\n"
		"The point position will be in the middle of those 2 control points, and the extents of the point will be half the difference between those 2 control points.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGSplineToSegmentSettings::CreateElement() const
{
	return MakeShared<FPCGSplineToSegmentElement>();
}

TArray<FPCGPinProperties> UPCGSplineToSegmentSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Result;
	FPCGPinProperties& InputPin = Result.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spline);
	InputPin.SetRequiredPin();

	return Result;
}

TArray<FPCGPinProperties> UPCGSplineToSegmentSettings::OutputPinProperties() const
{ 
	return Super::DefaultPointOutputPinProperties();
}

bool FPCGSplineToSegmentElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSegmentSplineElement::Execute);

	check(InContext);

	const UPCGSplineToSegmentSettings* Settings = InContext->GetInputSettings<UPCGSplineToSegmentSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGSplineData* InputSplineData = Cast<const UPCGSplineData>(Input.Data);

		if (!InputSplineData)
		{
			continue;
		}

		UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(InContext);
		OutputPointData->InitializeFromData(InputSplineData);

		const int32 NumSegments = InputSplineData->GetNumSegments();
		if (NumSegments < 1)
		{
			// If we have a malformed spline, just make an empty point data
			Outputs.Emplace_GetRef(Input).Data = OutputPointData;
			continue;
		}

		FPCGMetadataAttribute<FVector>* PreviousTangentAttribute = nullptr;
		FPCGMetadataAttribute<FVector>* NextTangentAttribute = nullptr;
		FPCGMetadataAttribute<int32>* SegmentIndexAttribute = nullptr;
		FPCGMetadataAttribute<int32>* PreviousIndexAttribute = nullptr;
		FPCGMetadataAttribute<int32>* NextIndexAttribute = nullptr;
		FPCGMetadataAttribute<double>* PreviousAngleAttribute = nullptr;
		FPCGMetadataAttribute<double>* NextAngleAttribute = nullptr;

		bool bHasMetadata = false;

		if (Settings->bExtractTangents)
		{
			PreviousTangentAttribute = PCGSplineToSegment::CreateAttributeWithLog<FVector>(PCGSplineToSegment::PreviousTangentAttributeName, false, OutputPointData->Metadata, InContext);
			NextTangentAttribute = PCGSplineToSegment::CreateAttributeWithLog<FVector>(PCGSplineToSegment::NextTangentAttributeName, false, OutputPointData->Metadata, InContext);

			if (!PreviousTangentAttribute || !NextTangentAttribute)
			{
				continue;
			}

			bHasMetadata = true;
		}

		if (Settings->bExtractAngles)
		{
			PreviousAngleAttribute = PCGSplineToSegment::CreateAttributeWithLog<double>(PCGSplineToSegment::PreviousAngleAttributeName, true, OutputPointData->Metadata, InContext);
			NextAngleAttribute = PCGSplineToSegment::CreateAttributeWithLog<double>(PCGSplineToSegment::NextAngleAttributeName, true, OutputPointData->Metadata, InContext);

			if (!PreviousAngleAttribute || !NextAngleAttribute)
			{
				continue;
			}

			bHasMetadata = true;
		}

		if (Settings->bExtractConnectivityInfo)
		{
			PreviousIndexAttribute = PCGSplineToSegment::CreateAttributeWithLog<int32>(PCGSplineToSegment::PreviousIndexAttributeName, false, OutputPointData->Metadata, InContext, INDEX_NONE);
			NextIndexAttribute = PCGSplineToSegment::CreateAttributeWithLog<int32>(PCGSplineToSegment::NextIndexAttributeName, false, OutputPointData->Metadata, InContext);
			SegmentIndexAttribute = PCGSplineToSegment::CreateAttributeWithLog<int32>(PCGSplineToSegment::SegmentIndexAttributeName, false, OutputPointData->Metadata, InContext);

			if (!PreviousIndexAttribute || !NextIndexAttribute || !SegmentIndexAttribute)
			{
				continue;
			}

			bHasMetadata = true;
		}

		OutputPointData->SetNumPoints(NumSegments);
		OutputPointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::MetadataEntry | EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax);

		FPCGPointValueRanges OutRanges(OutputPointData, /*bAllocate*/false);

		TArray<FBox> Segments;
		Segments.Reserve(NumSegments);
		double CumulativeAngle = 0.0;
		double FirstAngle = 0.0;
		const bool bIsClosed = InputSplineData->SplineStruct.IsClosedLoop();

		for (int32 Index = 0; Index < NumSegments; ++Index)
		{
			const FVector ControlPoint = InputSplineData->SplineStruct.GetLocationAtSplineInputKey(Index, ESplineCoordinateSpace::World);
			// Note: We're iterating on segments and it is safe to access the final control point (ie. Index + 1)
			const FVector NextControlPoint = InputSplineData->SplineStruct.GetLocationAtSplineInputKey(Index + 1, ESplineCoordinateSpace::World);
			const FVector ControlPointZUp = InputSplineData->SplineStruct.GetQuaternionAtSplineInputKey(Index, ESplineCoordinateSpace::World).GetAxisZ();

			FVector ArriveTangent, LeaveTangent;

			InputSplineData->GetTangentsAtSegmentStart(Index, ArriveTangent, LeaveTangent);
			const double CrossProduct = (ArriveTangent.X * LeaveTangent.Y - ArriveTangent.Y * LeaveTangent.X);
			const double AbsAngle = FMath::Acos(ArriveTangent.CosineAngle2D(LeaveTangent));
			const double Angle = FMath::RadiansToDegrees(FMath::Sign(CrossProduct) * AbsAngle);

			if (bHasMetadata)
			{
				OutputPointData->Metadata->InitializeOnSet(OutRanges.MetadataEntryRange[Index]);
			}

			// Points will be placed at the middle of the segment with bounds min and max matching control points on X (Y and Z will be set to -1 and 1 for min and max respectively)
			// We'll also reset the point scale
			const FVector Extents = (NextControlPoint - ControlPoint) * 0.5;
			const double ExtentsLength = Extents.Length();
			OutRanges.TransformRange[Index].SetLocation(ControlPoint + Extents);
			OutRanges.TransformRange[Index].SetRotation(FRotationMatrix::MakeFromXZ(Extents, ControlPointZUp).ToQuat());
			OutRanges.BoundsMinRange[Index] = FVector(-ExtentsLength, 0.0, 0.0);
			OutRanges.BoundsMaxRange[Index] = FVector(ExtentsLength, 1.0, 1.0);

			OutRanges.SeedRange[Index] = PCGHelpers::ComputeSeedFromPosition(OutRanges.TransformRange[Index].GetLocation());

			auto SetMetadata = [&](const int32 Index, const FVector& Tangent, const double Angle)
			{
				const int32 PreviousIndex = (Index + NumSegments - 1) % NumSegments;
				const PCGMetadataEntryKey PreviousPointKey = OutRanges.MetadataEntryRange[PreviousIndex];
				const PCGMetadataEntryKey CurrentPointKey = OutRanges.MetadataEntryRange[Index];

				if (PreviousTangentAttribute && NextTangentAttribute)
				{
					NextTangentAttribute->SetValue(PreviousPointKey, Tangent);
					PreviousTangentAttribute->SetValue(CurrentPointKey, Tangent);
				}

				if (PreviousAngleAttribute && NextAngleAttribute)
				{
					NextAngleAttribute->SetValue(PreviousPointKey, Angle);
					PreviousAngleAttribute->SetValue(CurrentPointKey, Angle);
				}

				if (PreviousIndexAttribute && NextIndexAttribute)
				{
					NextIndexAttribute->SetValue(PreviousPointKey, Index);
					PreviousIndexAttribute->SetValue(CurrentPointKey, PreviousIndex);
					SegmentIndexAttribute->SetValue(CurrentPointKey, Index);
				}
			};

			if (Index > 0)
			{
				CumulativeAngle += Angle;

				if (bHasMetadata)
				{
					SetMetadata(Index, ArriveTangent, Angle);
				}
			}
			else
			{
				// Need to save the first angle to set it later.
				FirstAngle = Angle;
			}


			// For the last point, if it's a closed spline, set the arrive tangent and angle for the first point
			if (bIsClosed && bHasMetadata && Index == (NumSegments - 1))
			{
				SetMetadata(0, LeaveTangent, FirstAngle);
			}
		}

		if (Settings->bExtractClockwiseInfo)
		{
			PCGSplineToSegment::CreateAttributeWithLog<bool>(PCGSplineToSegment::ClockwiseAttributeName, false, OutputPointData->Metadata, InContext, CumulativeAngle <= 0.0);
		}

		FPCGTaggedData& Output = Outputs.Emplace_GetRef(Input);
		Output.Data = OutputPointData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
