// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Grammar/PCGSubdivideSegment.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Grammar/PCGGrammarParser.h"
#include "Helpers/PCGPointHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSubdivideSegment)

#define LOCTEXT_NAMESPACE "PCGSubdivideSegmentElement"

class PCGSubdivideSegmentHelpers
{
public:
	struct FParameters
	{
		FPCGMetadataAttribute<FName>* SymbolAttribute = nullptr;
		FPCGMetadataAttribute<FVector4>* DebugColorAttribute = nullptr;
		FPCGMetadataAttribute<int32>* ModuleIndexAttribute = nullptr;
		FPCGMetadataAttribute<bool>* IsFirstPointAttribute = nullptr;
		FPCGMetadataAttribute<bool>* IsFinalPointAttribute = nullptr;
		FPCGMetadataAttribute<int32>* ExtremityNeighborIndexAttribute = nullptr;

		PCGSubdivisionBase::FModuleInfoMap ModulesInfo;
		TMap<FString, PCGGrammar::FTokenizedGrammar> CachedModules;
		TArray<int32> CornerIndexes;

		FVector SubdivisionDirection;
		FVector PerpendicularSubdivisionDirection;

		const UPCGSubdivideSegmentSettings* Settings = nullptr;
		FPCGContext* Context = nullptr;
		const UPCGBasePointData* InPointData = nullptr;
		UPCGBasePointData* OutPointData = nullptr;
	};

	static void Process(FParameters& InOutParameters, const FString& InGrammar, const bool bFlipAxis, int32 AdditionalSeed, int32 Index)
	{
		if (!InOutParameters.CachedModules.Contains(InGrammar))
		{
			double MinSize;
			InOutParameters.CachedModules.Emplace(InGrammar, PCGSubdivisionBase::GetTokenizedGrammar(InOutParameters.Context, InGrammar, InOutParameters.ModulesInfo, MinSize));
		}

		const PCGGrammar::FTokenizedGrammar& CurrentTokenizedGrammar = InOutParameters.CachedModules[InGrammar];

		if(!CurrentTokenizedGrammar.IsValid())
		{
			return;
		}

		FConstPCGPointValueRanges InRanges(InOutParameters.InPointData);
		const FPCGPoint Point = InRanges.GetPoint(Index);

		FTransform TransformNoTranslation = Point.Transform;
		TransformNoTranslation.SetLocation(FVector::ZeroVector);

		FBox Segment = Point.GetLocalBounds();
		FVector PointScaledSize = Point.GetScaledLocalSize();
		if (bFlipAxis)
		{
			// Swap coordinates on the subdivision direction
			const FVector PreviousMin = Segment.Min;
			Segment.Min = Segment.Min * InOutParameters.PerpendicularSubdivisionDirection + Segment.Max * InOutParameters.SubdivisionDirection;
			Segment.Max = Segment.Max * InOutParameters.PerpendicularSubdivisionDirection + PreviousMin * InOutParameters.SubdivisionDirection;
		}

		const FVector Direction = TransformNoTranslation.TransformVectorNoScale(bFlipAxis ? -InOutParameters.SubdivisionDirection : InOutParameters.SubdivisionDirection).GetSafeNormal();
		const FVector OtherDirection = TransformNoTranslation.TransformVectorNoScale(PointScaledSize * InOutParameters.PerpendicularSubdivisionDirection) * 0.5;
		const FVector HalfExtents2D = PointScaledSize * InOutParameters.PerpendicularSubdivisionDirection * 0.5;
		const double Size = FMath::Abs(PointScaledSize.Dot(InOutParameters.SubdivisionDirection));

		TArray<PCGSubdivisionBase::TModuleInstance<PCGGrammar::FTokenizedModule>> ModulesInstances;
		double RemainingSubdivide;
		const bool bSubdivideSuccess = PCGSubdivisionBase::Subdivide(*CurrentTokenizedGrammar.ModuleGrammar, Size, ModulesInstances, RemainingSubdivide, InOutParameters.Context, AdditionalSeed);

		if (!bSubdivideSuccess)
		{
			if (!InOutParameters.Settings->bSilenceGrammarDoesNotFitErrors)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("GrammarDoesNotFit", "Grammar does not fit for this segment."), InOutParameters.Context);
			}

			return;
		}

		if (!InOutParameters.Settings->bAcceptIncompleteSubdivision && !FMath::IsNearlyZero(RemainingSubdivide))
		{
			PCGLog::LogWarningOnGraph(LOCTEXT("FailSubdivisionFullLength", "One segment has an incomplete subdivision (grammar does not fit the whole segment)."), InOutParameters.Context);
			return;
		}

		// Now we have our segment subdivided, create the final points
		const FVector InitialPos = Point.Transform.TransformPosition(Segment.Min);
		FVector CurrentPos = InitialPos;
		int32 ModuleIndex = 0;

		const int32 FirstModuleIndex = InOutParameters.OutPointData->GetNumPoints();

		InOutParameters.OutPointData->SetNumPoints(InOutParameters.OutPointData->GetNumPoints() + ModulesInstances.Num(), /*bInitializeValues=*/false);
		InOutParameters.OutPointData->AllocateProperties(InOutParameters.InPointData->GetAllocatedProperties());
		InOutParameters.OutPointData->CopyUnallocatedPropertiesFrom(InOutParameters.InPointData);
		
		FPCGPointValueRanges OutRanges(InOutParameters.OutPointData, /*bAllocateProperties=*/false);

		for (int32 ModuleInstanceIndex = 0; ModuleInstanceIndex < ModulesInstances.Num(); ModuleInstanceIndex++)
		{
			const PCGSubdivisionBase::TModuleInstance<PCGGrammar::FTokenizedModule>& ModuleInstance = ModulesInstances[ModuleInstanceIndex];

			const FName Symbol = ModuleInstance.Module->Descriptor->Symbol;
			const FVector Scale = FVector::OneVector + (InOutParameters.SubdivisionDirection * ModuleInstance.ExtraScale);
			const FPCGSubdivisionSubmodule& SubdivisionSubmodule = InOutParameters.ModulesInfo[Symbol];
			const bool bIsFirstModule = (ModuleInstanceIndex == 0);
			const bool bIsFinalModule = (ModuleInstanceIndex == ModulesInstances.Num() - 1);

			const double HalfDisplacement = SubdivisionSubmodule.Size * 0.5;
			const double HalfScaledDisplacement = Scale.Dot(InOutParameters.SubdivisionDirection) * HalfDisplacement;

			const FVector LocalBoundsExtents = InOutParameters.SubdivisionDirection * HalfDisplacement + HalfExtents2D;
			const FVector HalfStep = HalfScaledDisplacement * Direction;
			const FVector Position = CurrentPos + HalfStep;
			CurrentPos = Position + HalfStep;

			FPCGPoint OutPoint(Point);
			OutPoint.Transform = FTransform(Point.Transform.GetRotation(), Position + OtherDirection, Scale);
			OutPoint.SetLocalBounds(FBox(-LocalBoundsExtents, LocalBoundsExtents));
			InOutParameters.OutPointData->Metadata->InitializeOnSet(OutPoint.MetadataEntry);
			if (InOutParameters.SymbolAttribute)
			{
				InOutParameters.SymbolAttribute->SetValue(OutPoint.MetadataEntry, Symbol);
			}

			if (InOutParameters.DebugColorAttribute)
			{
				InOutParameters.DebugColorAttribute->SetValue(OutPoint.MetadataEntry, FVector4(SubdivisionSubmodule.DebugColor, 1.0));
			}

			if (InOutParameters.ModuleIndexAttribute)
			{
				InOutParameters.ModuleIndexAttribute->SetValue(OutPoint.MetadataEntry, ModuleIndex++);
			}

			if (bIsFirstModule && InOutParameters.IsFirstPointAttribute)
			{
				InOutParameters.IsFirstPointAttribute->SetValue(OutPoint.MetadataEntry, true);
			}

			if (bIsFinalModule && InOutParameters.IsFinalPointAttribute)
			{
				InOutParameters.IsFinalPointAttribute->SetValue(OutPoint.MetadataEntry, true);
			}

			OutRanges.SetFromPoint(FirstModuleIndex + ModuleInstanceIndex, OutPoint);
		}

		if (bFlipAxis)
		{
			InOutParameters.CornerIndexes.Add(InOutParameters.OutPointData->GetNumPoints() - 1);
			InOutParameters.CornerIndexes.Add(FirstModuleIndex);
		}
		else
		{
			InOutParameters.CornerIndexes.Add(FirstModuleIndex);
			InOutParameters.CornerIndexes.Add(InOutParameters.OutPointData->GetNumPoints() - 1);
		}
	}
};

#if WITH_EDITOR
FName UPCGSubdivideSegmentSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SubdivideSegment"));
}

FText UPCGSubdivideSegmentSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Subdivide Segment");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGSubdivideSegmentSettings::CreateElement() const
{
	return MakeShared<FPCGSegmentSubdivisionElement>();
}

TArray<FPCGPinProperties> UPCGSubdivideSegmentSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Result;
	FPCGPinProperties& InputPin = Result.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	InputPin.SetRequiredPin();

	if (bModuleInfoAsInput)
	{
		FPCGPinProperties& ModuleInfoPin = Result.Emplace_GetRef(PCGSubdivisionBase::Constants::ModulesInfoPinLabel, EPCGDataType::Param);
		ModuleInfoPin.SetRequiredPin();
	}

	return Result;
}

TArray<FPCGPinProperties> UPCGSubdivideSegmentSettings::OutputPinProperties() const
{
	return Super::DefaultPointOutputPinProperties();
}

bool FPCGSegmentSubdivisionElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSubdivideSegmentElement::Execute);

	check(InContext);

	const UPCGSubdivideSegmentSettings* Settings = InContext->GetInputSettings<UPCGSubdivideSegmentSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	PCGSubdivideSegmentHelpers::FParameters Parameters{};

	Parameters.SubdivisionDirection = FVector::ZeroVector;

	switch (Settings->SubdivisionAxis)
	{
	case EPCGSplitAxis::X:
		Parameters.SubdivisionDirection = FVector::XAxisVector;
		break;
	case EPCGSplitAxis::Y:
		Parameters.SubdivisionDirection = FVector::YAxisVector;
		break;
	case EPCGSplitAxis::Z:
		Parameters.SubdivisionDirection = FVector::ZAxisVector;
		break;
	default:
		PCGLog::LogErrorOnGraph(LOCTEXT("InvalidAxis", "Invalid Subdivision Axis enum value."), InContext);
		return true;
	}

	Parameters.PerpendicularSubdivisionDirection = FVector::OneVector - Parameters.SubdivisionDirection;

	const UPCGParamData* ModuleInfoParamData = nullptr;
	Parameters.ModulesInfo = GetModulesInfoMap(InContext, Settings, ModuleInfoParamData);

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGBasePointData* InputPointData = Cast<const UPCGBasePointData>(Input.Data);

		if (!InputPointData)
		{
			continue;
		}

		TUniquePtr<const IPCGAttributeAccessor> GrammarAccessor;
		TUniquePtr<const IPCGAttributeAccessor> FlipAxisAccessor;
		TUniquePtr<const IPCGAttributeAccessor> SeedAccessor;
		TUniquePtr<const IPCGAttributeAccessorKeys> GrammarKeys;
		TUniquePtr<const IPCGAttributeAccessorKeys> FlipAxisKeys;
		TUniquePtr<const IPCGAttributeAccessorKeys> SeedKeys;


		const bool bShouldUseAccessors = Settings->GrammarSelection.bGrammarAsAttribute || Settings->bFlipAxisAsAttribute || Settings->bUseSeedAttribute;
		constexpr int32 DefaultAdditionalSeed = 0;

		if (bShouldUseAccessors)
		{
			// Create accessors for grammar, flip axis and seed as soon as either is needed. For the ones that aren't needed, we will create a constant accessor to uniformize the code below, even though there is a small performance cost.
			auto CreateAccessorAndKeys = [InputPointData, InContext]<typename T>(TUniquePtr<const IPCGAttributeAccessor>&OutAccessor, TUniquePtr<const IPCGAttributeAccessorKeys>& OutKeys, bool bValueAsAttribute, const FPCGAttributePropertyInputSelector & InputSelector, const T& ConstantValue) -> bool
			{
				if (bValueAsAttribute)
				{
					const FPCGAttributePropertyInputSelector Selector = InputSelector.CopyAndFixLast(InputPointData);
					OutAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputPointData, Selector);
					OutKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputPointData, Selector);

					if (!OutAccessor || !OutKeys)
					{
						PCGLog::Metadata::LogFailToCreateAccessorError(Selector, InContext);
						return false;
					}
				}
				else
				{
					OutAccessor = MakeUnique<FPCGConstantValueAccessor<T>>(ConstantValue);
					// Dummy single key
					OutKeys = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();
				}

				return true;
			};

			if (!CreateAccessorAndKeys(GrammarAccessor, GrammarKeys, Settings->GrammarSelection.bGrammarAsAttribute, Settings->GrammarSelection.GrammarAttribute, Settings->GrammarSelection.GrammarString)
				|| !CreateAccessorAndKeys(FlipAxisAccessor, FlipAxisKeys, Settings->bFlipAxisAsAttribute, Settings->FlipAxisAttribute, Settings->bShouldFlipAxis)
				|| !CreateAccessorAndKeys(SeedAccessor, SeedKeys, Settings->bUseSeedAttribute, Settings->SeedAttribute, DefaultAdditionalSeed))
			{
				continue;
			}
		}

		UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(InContext);

		FPCGInitializeFromDataParams InitializeFromDataParams(InputPointData);
		InitializeFromDataParams.bInheritSpatialData = false;
		OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);

		Parameters.Settings = Settings;
		Parameters.Context = InContext;
		Parameters.InPointData = InputPointData;
		Parameters.OutPointData = OutputPointData;

		auto CreateAndValidateAttribute = [InContext, OutputMetadata = OutputPointData->Metadata]<typename T>(const FName AttributeName, const T DefaultValue, const bool bShouldCreate, FPCGMetadataAttribute<T>*& OutAttribute) -> bool
		{
			if (bShouldCreate)
			{
				OutAttribute = OutputMetadata->FindOrCreateAttribute<T>(AttributeName, DefaultValue, false, true);
				if (!OutAttribute)
				{
					PCGLog::Metadata::LogFailToCreateAttributeError<T>(AttributeName, InContext);
					return false;
				}
			}

			return true;
		};

		if (!CreateAndValidateAttribute(Settings->SymbolAttributeName, FName(NAME_None), true, Parameters.SymbolAttribute)
			|| !CreateAndValidateAttribute(Settings->DebugColorAttributeName, FVector4::Zero(), Settings->bOutputDebugColorAttribute, Parameters.DebugColorAttribute)
			|| !CreateAndValidateAttribute(Settings->ModuleIndexAttributeName, -1, Settings->bOutputModuleIndexAttribute, Parameters.ModuleIndexAttribute)
			|| !CreateAndValidateAttribute(Settings->IsFirstAttributeName, false, Settings->bOutputExtremityAttributes, Parameters.IsFirstPointAttribute)
			|| !CreateAndValidateAttribute(Settings->IsFinalAttributeName, false, Settings->bOutputExtremityAttributes, Parameters.IsFinalPointAttribute)
			|| !CreateAndValidateAttribute(Settings->ExtremityNeighborIndexAttributeName, -1, Settings->bOutputExtremityNeighborIndexAttribute, Parameters.ExtremityNeighborIndexAttribute))
		{
			continue;
		}

		if (bShouldUseAccessors)
		{
			check(GrammarAccessor && FlipAxisAccessor && SeedAccessor);
			auto Process = [&Parameters](const FString& InGrammar, const bool bFlipAxis, int32 AdditionalSeed, int32 Index) -> void
			{
				PCGSubdivideSegmentHelpers::Process(Parameters, InGrammar, bFlipAxis, AdditionalSeed, Index);
			};

			PCGMetadataElementCommon::ApplyOnMultiAccessors<FString, bool, int32>({GrammarKeys.Get(), FlipAxisKeys.Get(), SeedKeys.Get()}, { GrammarAccessor.Get(), FlipAxisAccessor.Get(), SeedAccessor.Get()}, Process);
		}
		else
		{
			for (int32 SegmentIndex = 0; SegmentIndex < Parameters.InPointData->GetNumPoints(); ++SegmentIndex)
			{
				PCGSubdivideSegmentHelpers::Process(Parameters, Settings->GrammarSelection.GrammarString, Settings->bShouldFlipAxis, DefaultAdditionalSeed, SegmentIndex);
			}
		}

		if (!Parameters.OutPointData->IsEmpty())
		{
			// Set the extremity neighbor indexes
			if (Parameters.ExtremityNeighborIndexAttribute)
			{
				TPCGValueRange<int64> MetadataEntryRange = Parameters.OutPointData->GetMetadataEntryValueRange(/*bAllocate=*/true);

				// Go 2 by 2
				check(Parameters.CornerIndexes.Num() % 2 == 0);

				for (int32 j = 0; j < Parameters.CornerIndexes.Num(); j += 2)
				{
					const int32 PreviousModuleIndex = Parameters.CornerIndexes[j == 0 ? Parameters.CornerIndexes.Num() - 1 : j - 1];
					const int32 NextModuleIndex = Parameters.CornerIndexes[j == Parameters.CornerIndexes.Num() - 2 ? 0 : j + 2];

					Parameters.ExtremityNeighborIndexAttribute->SetValue(MetadataEntryRange[Parameters.CornerIndexes[j]], PreviousModuleIndex);
					Parameters.ExtremityNeighborIndexAttribute->SetValue(MetadataEntryRange[Parameters.CornerIndexes[j+1]], NextModuleIndex);
				}
			}

			FPCGTaggedData& Output = Outputs.Emplace_GetRef(Input);
			Output.Data = OutputPointData;
		}
	}

	if (Settings->bForwardAttributesFromModulesInfo && ModuleInfoParamData)
	{
		MatchAndSetAttributes(Inputs, Outputs, ModuleInfoParamData, Settings);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
