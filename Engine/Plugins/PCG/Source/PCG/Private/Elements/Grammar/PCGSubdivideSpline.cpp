// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Grammar/PCGSubdivideSpline.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGHelpers.h"
#include "Kismet/KismetMathLibrary.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSubdivideSpline)

#define LOCTEXT_NAMESPACE "PCGSubdivideSplineElement"

namespace PCGSubdivideSplineHelpers
{
	struct FParameters
	{
		FPCGContext* Context = nullptr;
		const UPCGSubdivideSplineSettings* Settings = nullptr;

		PCGSubdivisionBase::FModuleInfoMap ModulesInfo;
		TMap<FString, PCGGrammar::FTokenizedGrammar> CachedModules;
		bool bAcceptIncompleteSubdivision = false;
		double ModuleHeight = 0.0;
		int32 AdditionalSeed = 0;
		double BisectionTolerance = 0.01;

		const UPCGPolyLineData* PolyLineData = nullptr;
		UPCGBasePointData* OutputPointData = nullptr;

		FPCGMetadataAttribute<FName>* SymbolAttribute = nullptr;
		FPCGMetadataAttribute<FVector4>* DebugColorAttribute = nullptr;
		FPCGMetadataAttribute<int32>* ModuleIndexAttribute = nullptr;
		FPCGMetadataAttribute<bool>* IsFirstPointAttribute = nullptr;
		FPCGMetadataAttribute<bool>* IsFinalPointAttribute = nullptr;
	};

	double GetLinearDistanceBetweenPolyLineAlphas(const UPCGPolyLineData* PolyLineData, double FirstAlpha, double SecondAlpha)
	{
		check(PolyLineData);

		FirstAlpha = FMath::Clamp(FirstAlpha, 0, 1);
		SecondAlpha = FMath::Clamp(SecondAlpha, 0, 1);
		const FVector FirstPoint = PolyLineData->GetLocationAtAlpha(FirstAlpha);
		const FVector SecondPoint = PolyLineData->GetLocationAtAlpha(SecondAlpha);
		return FVector::Distance(FirstPoint, SecondPoint);
	}

	// Using a numerical method (s.a. bisection), determine the spline alpha from a segment length. Bisection can be slower, but guaranteed to converge.
	static double FindRootAtLinearDistance_Bisection(const UPCGPolyLineData* PolyLineData, const double SegmentLength, const double StartingAlpha, const double Tolerance)
	{
		check(PolyLineData);

		// Bisect a number of times, before taking an estimate
		static constexpr uint16 BisectionCountLimit = 64u;

		double Low = StartingAlpha;
		double High = 1.0;
		double Estimate = 0.0;

		for (uint16 BisectionCount = 0u; BisectionCount < BisectionCountLimit; ++BisectionCount)
		{
			Estimate = (Low + High) * 0.5;
			// TODO: If performance is an issue, this can deal with Dist^2 instead to save the sqrt. But, it will skew the tolerance.
			const double LinearDistance = GetLinearDistanceBetweenPolyLineAlphas(PolyLineData, StartingAlpha, Estimate);
			check(LinearDistance >= 0);

			if (FMath::IsNearlyEqual(LinearDistance, SegmentLength, Tolerance))
			{
				return Estimate;
			}

			if (LinearDistance < SegmentLength)
			{
				Low = Estimate;
			}
			else
			{
				High = Estimate;
			}
		}

		// Couldn't find the root yet, so estimate
		return (Low + High) * 0.5;
	}

	static void Process(FParameters& InOutParameters, const FString& InGrammar)
	{
		if (!InOutParameters.CachedModules.Contains(InGrammar))
		{
			double MinSize;
			InOutParameters.CachedModules.Emplace(InGrammar, PCGSubdivisionBase::GetTokenizedGrammar(InOutParameters.Context, InGrammar, InOutParameters.ModulesInfo, MinSize));
		}

		const PCGGrammar::FTokenizedGrammar& CurrentTokenizedGrammar = InOutParameters.CachedModules[InGrammar];

		if (!CurrentTokenizedGrammar.IsValid())
		{
			return;
		}

		const double SplineLength = InOutParameters.PolyLineData->GetLength();
		const double HalfModuleHeight = InOutParameters.ModuleHeight / 2;

		/* Implementation Note: Subdivided spline length will always be equal or greater than discretized linear length, depending on the curvature of the spline.
		 * For extremely long or curvy splines, this can result in the tokenized grammar being cut short. Alternative subdivision solutions may need to be explored.
		 */
		TArray<PCGSubdivisionBase::TModuleInstance<PCGGrammar::FTokenizedModule>> ModulesInstances;
		double RemainingSubdivide;
		if (!Subdivide(*CurrentTokenizedGrammar.ModuleGrammar, SplineLength, ModulesInstances, RemainingSubdivide, InOutParameters.Context, InOutParameters.AdditionalSeed))
		{
			if (!InOutParameters.Settings->bSilenceGrammarDoesNotFitErrors)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("GrammarDoesNotFit", "Grammar does not fit for this spline."), InOutParameters.Context);
			}

			return;
		}

		if (!InOutParameters.bAcceptIncompleteSubdivision && !FMath::IsNearlyZero(RemainingSubdivide))
		{
			PCGLog::LogWarningOnGraph(LOCTEXT("FailSubdivisionFullLength", "The spline has an incomplete subdivision (grammar doesn't fit the whole segment)."), InOutParameters.Context);
			return;
		}

		// Since we're essentially mapping spline-space modules onto linear-space modules (they aren't deformed),
		// Then we need to refit potentially, knowing that having scaled up some modules might make the total length too large.
		// We'll do a few passes to converge to a better fit.
		TArray<double> ModuleAlphas;
		int LengthConvergencePass = 0;
		const int MaxLengthConvergencePass = 10;

		constexpr double OvershootToleranceFactor = 0.02; // 2%
		constexpr double OvershootToleranceMax = 5.0; // 5 cm
		double MinimumModuleSize = std::numeric_limits<double>::max();
		for (const PCGSubdivisionBase::TModuleInstance<PCGGrammar::FTokenizedModule>& ModuleInstance : ModulesInstances)
		{
			const FName Symbol = ModuleInstance.Module->Descriptor->Symbol;
			const FPCGSubdivisionSubmodule& SubdivisionSubmodule = InOutParameters.ModulesInfo[Symbol];
			MinimumModuleSize = FMath::Min(MinimumModuleSize, SubdivisionSubmodule.Size);
		}

		const double OvershootTolerance = FMath::Min(MinimumModuleSize * OvershootToleranceFactor, OvershootToleranceMax);

		for(;LengthConvergencePass < MaxLengthConvergencePass; ++LengthConvergencePass)
		{
			ModuleAlphas.Reset(ModulesInstances.Num() + 1);
			ModuleAlphas.Emplace(0.0);

			double ScaledLength = 0.0;
			double OvershootDistance = 0.0;
			bool bAtSplineEnd = false;

			for (const PCGSubdivisionBase::TModuleInstance<PCGGrammar::FTokenizedModule>& ModuleInstance : ModulesInstances)
			{
				const FName Symbol = ModuleInstance.Module->Descriptor->Symbol;
				const FPCGSubdivisionSubmodule& SubdivisionSubmodule = InOutParameters.ModulesInfo[Symbol];
				const double SubmoduleSize = SubdivisionSubmodule.Size * (1.0 + ModuleInstance.ExtraScale);

				// TODO: modules that can be deformed can also have extra scale, but the move in spline space (so their real length needs to be measured)
				if (ModuleInstance.ExtraScale > 0)
				{
					ScaledLength += SubdivisionSubmodule.Size * ModuleInstance.ExtraScale;
				}

				if (!bAtSplineEnd)
				{
					const double PreviousAlpha = ModuleAlphas.Last();
					const double BisectionTolerance = FMath::Max(FMath::Min(InOutParameters.BisectionTolerance, SubmoduleSize), PCGSubdivideSplineConstants::MinimumBisectionTolerance);
					const double CurrentAlpha = FindRootAtLinearDistance_Bisection(InOutParameters.PolyLineData, SubmoduleSize, PreviousAlpha, BisectionTolerance);

					ModuleAlphas.Add(CurrentAlpha);

					if (FMath::IsNearlyEqual(CurrentAlpha, 1.0))
					{
						bAtSplineEnd = true;

						// In this case, we know that the end of the module might be really after the end of the spline, so we need to compare the distance and add it to the overshoot.
						const FVector SubmoduleStartPoint = InOutParameters.PolyLineData->GetLocationAtAlpha(ModuleAlphas.Last(1));
						const FVector SplineEndPoint = InOutParameters.PolyLineData->GetLocationAtAlpha(ModuleAlphas.Last());

						const double LastSegmentOvershoot = SubmoduleSize - (SplineEndPoint - SubmoduleStartPoint).Length();
						if (LastSegmentOvershoot > 0)
						{
							OvershootDistance += LastSegmentOvershoot;
						}
					}
				}
				else
				{
					OvershootDistance += SubmoduleSize;
				}
			}

			// Evaluate if we need another pass
			if (LengthConvergencePass < MaxLengthConvergencePass && OvershootDistance > OvershootTolerance && ScaledLength > 0)
			{
				double UpdateExtraScaleFactor = 1.0;

				// If the leeway we currently have (scaled length) is smaller than the overshoot distance, we are unlikely to converge, and in this instance it's probably best to set the scale to zero across the board.
				if (OvershootDistance > ScaledLength)
				{
					UpdateExtraScaleFactor = 0.0;
				}
				// Otherwise, we want to move towards a better solution, but not too quickly lest we undershoot (which we don't detect here).
				else
				{
					UpdateExtraScaleFactor = 0.25 * (3.0 + ((ScaledLength - OvershootDistance) / ScaledLength));
					check(UpdateExtraScaleFactor >= 0.0 && UpdateExtraScaleFactor <= 1.0);
				}

				for (PCGSubdivisionBase::TModuleInstance<PCGGrammar::FTokenizedModule>& ModuleInstance : ModulesInstances)
				{
					if (ModuleInstance.ExtraScale > 0)
					{
						ModuleInstance.ExtraScale *= UpdateExtraScaleFactor;
					}
				}
			}
			else
			{
				// TODO: possibly log an error here, but it's subtle to get to a place where it's not too much
				break;
			}
		}

		const bool bHasMetadata = InOutParameters.SymbolAttribute || InOutParameters.DebugColorAttribute || InOutParameters.ModuleIndexAttribute || InOutParameters.IsFirstPointAttribute || InOutParameters.IsFinalPointAttribute;

		FVector PreviousSegmentEndPoint = (ModuleAlphas.IsEmpty() ? FVector::Zero() : InOutParameters.PolyLineData->GetLocationAtAlpha(ModuleAlphas[0]));
		const int32 NumIterations = FMath::Min(ModulesInstances.Num(), ModuleAlphas.Num() - 1);

		const int32 WriteIndex = InOutParameters.OutputPointData->GetNumPoints();
		InOutParameters.OutputPointData->SetNumPoints(WriteIndex + NumIterations, /*bInitializeValues=*/false);
		InOutParameters.OutputPointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::MetadataEntry);
		FPCGPointValueRanges OutRanges(InOutParameters.OutputPointData, /*bAllocate=*/false);

		for (int32 ModuleInstanceIndex = 0; ModuleInstanceIndex < NumIterations; ModuleInstanceIndex++)
		{
			const PCGSubdivisionBase::TModuleInstance<PCGGrammar::FTokenizedModule>& ModuleInstance = ModulesInstances[ModuleInstanceIndex];
			const FName Symbol = ModuleInstance.Module->Descriptor->Symbol;

			const FPCGSubdivisionSubmodule& SubdivisionSubmodule = InOutParameters.ModulesInfo[Symbol];

			const double& PreviousAlpha = ModuleAlphas[ModuleInstanceIndex];
			const double& SplineAlpha = ModuleAlphas[ModuleInstanceIndex + 1];

			// Move to the next segment of the spline
			const FVector SegmentStartPoint = PreviousSegmentEndPoint;
			const FVector SegmentEndPoint = InOutParameters.PolyLineData->GetLocationAtAlpha(SplineAlpha);
			PreviousSegmentEndPoint = SegmentEndPoint;
			const FVector SubdivisionVector = SegmentEndPoint - SegmentStartPoint;
			const FVector SubdivisionDirection = SubdivisionVector.GetSafeNormal();

			// Since its discretized, we won't take the transform's position, but we'll use the up vector--to create the module rotation--and the scale
			FTransform CenterPointTransform = InOutParameters.PolyLineData->GetTransformAtAlpha((SplineAlpha + PreviousAlpha) * 0.5);

			const FVector Position = SegmentStartPoint + (SubdivisionVector * 0.5) + FVector(0, 0, InOutParameters.ModuleHeight * 0.5);
			const FRotator Rotation = FRotationMatrix::MakeFromXZ(SubdivisionDirection, CenterPointTransform.GetRotation().GetUpVector()).Rotator();
			const FVector Scale = FVector(1.0 + ModuleInstance.ExtraScale, 1.0, 1.0);

			FPCGPoint OutPoint(FTransform(Rotation, Position, Scale), /*InDensity=*/1, PCGHelpers::ComputeSeedFromPosition(Position));

			const double HalfSubmoduleSize = SubdivisionSubmodule.Size * 0.5;
			const double OffsetZ = InOutParameters.Settings->bCenterBounds ? 0.0 : HalfModuleHeight;
			OutPoint.SetLocalBounds(FBox(FVector(-HalfSubmoduleSize, 0, -HalfModuleHeight + OffsetZ), FVector(HalfSubmoduleSize, 1, HalfModuleHeight + OffsetZ)));

			// Now, handle the metadata attributes
			if (bHasMetadata)
			{
				InOutParameters.OutputPointData->Metadata->InitializeOnSet(OutPoint.MetadataEntry);
				if (InOutParameters.SymbolAttribute)
				{
					InOutParameters.SymbolAttribute->SetValue(OutPoint.MetadataEntry, Symbol);
				}

				if (InOutParameters.DebugColorAttribute)
				{
					InOutParameters.DebugColorAttribute->SetValue(OutPoint.MetadataEntry, FVector4(SubdivisionSubmodule.DebugColor, 1.0));
				}

				if (InOutParameters.ModuleIndexAttribute)
				{
					InOutParameters.ModuleIndexAttribute->SetValue(OutPoint.MetadataEntry, ModuleInstanceIndex);
				}

				const bool bIsFirstModule = (ModuleInstanceIndex == 0);
				if (bIsFirstModule && InOutParameters.IsFirstPointAttribute)
				{
					InOutParameters.IsFirstPointAttribute->SetValue(OutPoint.MetadataEntry, true);
				}

				const bool bIsFinalModule = (ModuleInstanceIndex == NumIterations - 1);
				if (bIsFinalModule && InOutParameters.IsFinalPointAttribute)
				{
					InOutParameters.IsFinalPointAttribute->SetValue(OutPoint.MetadataEntry, true);
				}
			}

			OutRanges.SetFromPoint(WriteIndex + ModuleInstanceIndex, OutPoint);
		}
	}
}

FPCGElementPtr UPCGSubdivideSplineSettings::CreateElement() const
{
	return MakeShared<FPCGSubdivideSplineElement>();
}

void UPCGSubdivideSplineSettings::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITOR
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGMeshSamplerDefaultColorPCGSubdivideSplineBoundsFixes)
	{
		// For all previous nodes, we'll force this option to false. for retro-compatibility
		bCenterBounds = false;
	}
#endif // WITH_EDITOR
}

TArray<FPCGPinProperties> UPCGSubdivideSplineSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::PolyLine);
	InputPin.SetRequiredPin();

	if (bModuleInfoAsInput)
	{
		FPCGPinProperties& ModuleInfoPin = PinProperties.Emplace_GetRef(PCGSubdivisionBase::Constants::ModulesInfoPinLabel, EPCGDataType::Param);
		ModuleInfoPin.SetRequiredPin();
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSubdivideSplineSettings::OutputPinProperties() const
{
	return Super::DefaultPointOutputPinProperties();
}

bool FPCGSubdivideSplineElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSubdivideSplineElement::Execute);

	const UPCGSubdivideSplineSettings* Settings = InContext->GetInputSettings<UPCGSubdivideSplineSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	const UPCGParamData* ModuleInfoParamData = nullptr;
	const FModuleInfoMap ModulesInfo = GetModulesInfoMap(InContext, Settings, ModuleInfoParamData);

	double ModuleHeight = Settings->bModuleHeightAsAttribute ? 0.0 : Settings->ModuleHeight;

	PCGSubdivideSplineHelpers::FParameters Parameters
	{
		.Context = InContext,
		.Settings = Settings,
		.ModulesInfo = ModulesInfo,
		.bAcceptIncompleteSubdivision = Settings->bAcceptIncompleteSubdivision,
		.BisectionTolerance = Settings->ModulePlacementTolerance
	};

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGPolyLineData* InputPolyLineData = Cast<const UPCGPolyLineData>(Input.Data);
		if (!InputPolyLineData)
		{
			continue;
		}

		if (Settings->bModuleHeightAsAttribute)
		{
			check(InputPolyLineData->Metadata);
			const FPCGAttributePropertyInputSelector Selector = Settings->ModuleHeightAttribute.CopyAndFixLast(InputPolyLineData);
			TUniquePtr<const IPCGAttributeAccessor> HeightAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputPolyLineData, Selector);
			TUniquePtr<const IPCGAttributeAccessorKeys> HeightAccessorKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputPolyLineData, Selector);

			if (!HeightAccessor || !HeightAccessorKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, InContext);
				continue;
			}

			if (!HeightAccessor->Get<double>(ModuleHeight, *HeightAccessorKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::Metadata::LogFailToGetAttributeError(Selector, InContext);
				continue;
			}
		}

		UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(InContext);
		OutputPointData->InitializeFromData(InputPolyLineData);
		OutputPointData->SetDensity(1.0f);

		FPCGTaggedData& Output = Outputs.Emplace_GetRef(Input);
		Output.Data = OutputPointData;

		// Update 'per-input' parameters
		Parameters.PolyLineData = InputPolyLineData;
		Parameters.OutputPointData = OutputPointData;
		Parameters.ModuleHeight = ModuleHeight;

		auto CreateAndValidateAttribute = [InContext, OutputMetadata = OutputPointData->Metadata]<typename T>(const FName AttributeName, const T DefaultValue, const bool bShouldCreate, FPCGMetadataAttribute<T>*& OutAttribute) -> bool
		{
			if (bShouldCreate)
			{
				OutAttribute = OutputMetadata->FindOrCreateAttribute<T>(AttributeName, DefaultValue, false, true);
				if (!OutAttribute)
				{
					PCGLog::Metadata::LogFailToCreateAttributeError<T>(AttributeName, InContext);
					return false;
				}
			}

			return true;
		};

		if (!CreateAndValidateAttribute(Settings->SymbolAttributeName, FName(NAME_None), true, Parameters.SymbolAttribute)
			|| !CreateAndValidateAttribute(Settings->DebugColorAttributeName, FVector4::Zero(), Settings->bOutputDebugColorAttribute, Parameters.DebugColorAttribute)
			|| !CreateAndValidateAttribute(Settings->ModuleIndexAttributeName, -1, Settings->bOutputModuleIndexAttribute, Parameters.ModuleIndexAttribute)
			|| !CreateAndValidateAttribute(Settings->IsFirstAttributeName, false, Settings->bOutputExtremityAttributes, Parameters.IsFirstPointAttribute)
			|| !CreateAndValidateAttribute(Settings->IsFinalAttributeName, false, Settings->bOutputExtremityAttributes, Parameters.IsFinalPointAttribute))
		{
			continue;
		}

		// Set seed if required
		Parameters.AdditionalSeed = 0;
		if (Settings->bUseSeedAttribute)
		{
			const FPCGAttributePropertyInputSelector Selector = Settings->SeedAttribute.CopyAndFixLast(InputPolyLineData);
			const TUniquePtr<const IPCGAttributeAccessor> SeedAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputPolyLineData, Selector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> SeedAccessorKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputPolyLineData, Selector);

			if (!SeedAccessor || !SeedAccessorKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, InContext);
			}
			// Otherwise, get the value, if it fails, the attribute wasn't compatible
			else if (!SeedAccessor->Get(Parameters.AdditionalSeed, *SeedAccessorKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::Metadata::LogFailToGetAttributeError<int32>(Selector, SeedAccessor.Get(), InContext);
			}
		}

		if (Settings->GrammarSelection.bGrammarAsAttribute)
		{
			const FPCGAttributePropertyInputSelector Selector = Settings->GrammarSelection.GrammarAttribute.CopyAndFixLast(InputPolyLineData);
			const TUniquePtr<const IPCGAttributeAccessor> GrammarAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputPolyLineData, Selector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> GrammarAccessorKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputPolyLineData, Selector);
			if (!GrammarAccessor || !GrammarAccessorKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, InContext);
				continue;
			}
			
			// Otherwise, get the value, if it fails, the attribute wasn't compatible
			FString Grammar;
			if (GrammarAccessor->Get(Grammar, *GrammarAccessorKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGSubdivideSplineHelpers::Process(Parameters, Grammar);
			}
			else
			{
				PCGLog::Metadata::LogFailToGetAttributeError<int32>(Selector, GrammarAccessor.Get(), InContext);
			}
		}
		else
		{
			PCGSubdivideSplineHelpers::Process(Parameters, Settings->GrammarSelection.GrammarString);
		}
	}

	if (Settings->bForwardAttributesFromModulesInfo && ModuleInfoParamData)
	{
		MatchAndSetAttributes(Inputs, Outputs, ModuleInfoParamData, Settings);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
