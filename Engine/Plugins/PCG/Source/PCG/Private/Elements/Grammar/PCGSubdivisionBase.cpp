// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Grammar/PCGSubdivisionBase.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Grammar/PCGGrammarParser.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSubdivisionBase)

#define LOCTEXT_NAMESPACE "PCGSubdivisionBaseElement"

namespace PCGSubdivisionBase
{
	static const FText DuplicatedSymbolText = LOCTEXT("SymbolDuplicate", "Symbol {0} is duplicated, ignored.");
}

void UPCGSubdivisionBaseSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bGrammarAsAttribute_DEPRECATED != false)
	{
		GrammarSelection.bGrammarAsAttribute = bGrammarAsAttribute_DEPRECATED;
		bGrammarAsAttribute_DEPRECATED = false;
	}

	if (!Grammar_DEPRECATED.IsEmpty())
	{
		GrammarSelection.GrammarString = Grammar_DEPRECATED;
		Grammar_DEPRECATED.Empty();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

FPCGSubdivisionBaseElement::FModuleInfoMap FPCGSubdivisionBaseElement::GetModulesInfoMap(FPCGContext* InContext, const TArray<FPCGSubdivisionSubmodule>& SubmodulesInfo, const UPCGParamData*& OutModuleInfoParamData) const
{
	PCGSubdivisionBase::FModuleInfoMap ModulesInfo;
	OutModuleInfoParamData = nullptr;

	ModulesInfo.Reserve(SubmodulesInfo.Num());
	for (const FPCGSubdivisionSubmodule& SubdivisionModule : SubmodulesInfo)
	{
		if (ModulesInfo.Contains(SubdivisionModule.Symbol))
		{
			PCGLog::LogWarningOnGraph(FText::Format(PCGSubdivisionBase::DuplicatedSymbolText, FText::FromName(SubdivisionModule.Symbol)), InContext);
			continue;
		}

		ModulesInfo.Emplace(SubdivisionModule.Symbol, SubdivisionModule);
	}

	return ModulesInfo;
}

FPCGSubdivisionBaseElement::FModuleInfoMap FPCGSubdivisionBaseElement::GetModulesInfoMap(FPCGContext* InContext, const FPCGSubdivisionModuleAttributeNames& InSubdivisionModuleAttributeNames, const UPCGParamData*& OutModuleInfoParamData) const
{
	PCGSubdivisionBase::FModuleInfoMap ModulesInfo;
	OutModuleInfoParamData = nullptr;

	const TArray<FPCGTaggedData> ModulesInfoInputs = InContext->InputData.GetInputsByPin(PCGSubdivisionBase::Constants::ModulesInfoPinLabel);

	if (ModulesInfoInputs.IsEmpty())
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("NoModuleInfo", "No data was found on the module info pin."), InContext);
		return ModulesInfo;
	}

	const UPCGParamData* ParamData = Cast<UPCGParamData>(ModulesInfoInputs[0].Data);
	if (!ParamData)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("ModuleInfoWrongType", "Module info input is not of type attribute set."), InContext);
		return ModulesInfo;
	}

	TMap<FName, TTuple<FName, bool>> PropertyNameMapping;
	PropertyNameMapping.Emplace(GET_MEMBER_NAME_CHECKED(FPCGSubdivisionSubmodule, Symbol), {InSubdivisionModuleAttributeNames.SymbolAttributeName, /*bCanBeDefaulted=*/false});
	PropertyNameMapping.Emplace(GET_MEMBER_NAME_CHECKED(FPCGSubdivisionSubmodule, Size), {InSubdivisionModuleAttributeNames.SizeAttributeName, /*bCanBeDefaulted=*/false});
	PropertyNameMapping.Emplace(GET_MEMBER_NAME_CHECKED(FPCGSubdivisionSubmodule, bScalable), {InSubdivisionModuleAttributeNames.ScalableAttributeName, /*bCanBeDefaulted=*/!InSubdivisionModuleAttributeNames.bProvideScalable});
	PropertyNameMapping.Emplace(GET_MEMBER_NAME_CHECKED(FPCGSubdivisionSubmodule, DebugColor), {InSubdivisionModuleAttributeNames.DebugColorAttributeName, /*bCanBeDefaulted=*/!InSubdivisionModuleAttributeNames.bProvideDebugColor});

	const TArray<FPCGSubdivisionSubmodule> AllModules = PCGPropertyHelpers::ExtractAttributeSetAsArrayOfStructs<FPCGSubdivisionSubmodule>(ParamData, &PropertyNameMapping, InContext);

	ModulesInfo.Reserve(AllModules.Num());

	for (int32 i = 0; i < AllModules.Num(); ++i)
	{
		if (ModulesInfo.Contains(AllModules[i].Symbol))
		{
			PCGLog::LogWarningOnGraph(FText::Format(PCGSubdivisionBase::DuplicatedSymbolText, FText::FromName(AllModules[i].Symbol)), InContext);
			continue;
		}

		ModulesInfo.Emplace(AllModules[i].Symbol, std::move(AllModules[i]));
	}

	OutModuleInfoParamData = ParamData;

	return ModulesInfo;
}

PCGSubdivisionBase::FModuleInfoMap FPCGSubdivisionBaseElement::GetModulesInfoMap(FPCGContext* InContext, const UPCGSubdivisionBaseSettings* InSettings, const UPCGParamData*& OutModuleInfoParamData) const
{
	if (InSettings->bModuleInfoAsInput)
	{
		return GetModulesInfoMap(InContext, InSettings->ModulesInfoAttributeNames, OutModuleInfoParamData);
	}
	else
	{
		return GetModulesInfoMap(InContext, InSettings->ModulesInfo, OutModuleInfoParamData);
	}
}

PCGGrammar::FTokenizedGrammar FPCGSubdivisionBaseElement::GetTokenizedGrammar(FPCGContext* InContext, const UPCGData* InputData, const UPCGSubdivisionBaseSettings* InSettings, const FModuleInfoMap& InModulesInfo, double& OutMinSize) const
{
	FString Grammar = InSettings->GrammarSelection.GrammarString;

	if (InSettings->GrammarSelection.bGrammarAsAttribute)
	{
		const FPCGAttributePropertyInputSelector Selector = InSettings->GrammarSelection.GrammarAttribute.CopyAndFixLast(InputData);
		const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, Selector);
		const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, Selector);
		if (!Accessor || !Keys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(Selector, InContext);
			return {};
		}

		if (!Accessor->Get(Grammar, *Keys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
		{
			PCGLog::Metadata::LogFailToGetAttributeError<FString>(Selector, Accessor.Get(), InContext);
			return {};
		}
	}

	return PCGSubdivisionBase::GetTokenizedGrammar(InContext, Grammar, InModulesInfo, OutMinSize);
}

PCGGrammar::FTokenizedGrammar PCGSubdivisionBase::GetTokenizedGrammar(FPCGContext* InContext, const FString& InGrammar, const FModuleInfoMap& InModulesInfo, double& OutMinSize)
{
	FPCGGrammarResult Result = PCGGrammar::Parse(InGrammar);

	// TODO: Add quiet mode
	{
		if (!Result.bSuccess)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("GrammarParseFail", "Problem while parsing grammar:"), InContext);
		}

		for (const FPCGGrammarResult::FLog& Log : Result.GetLogs())
		{
			switch (Log.Verbosity)
			{
			case FPCGGrammarResult::ELogType::Error:
				PCGLog::LogErrorOnGraph(Log.Message, InContext);
				break;
			case FPCGGrammarResult::ELogType::Warning:
				PCGLog::LogWarningOnGraph(Log.Message, InContext);
				break;
			default:
				UE_LOGF(LogPCG, Log, "%ls", *Log.Message.ToString());
				break;
			}
		}
	}

	if (!Result.bSuccess)
	{
		return {};
	}

	// Build equivalent grammar tree with the size information.
	TSet<FName> UnmatchedTokens;
	auto BuildNode = [&UnmatchedTokens, &InModulesInfo](PCGGrammar::FTokenizedModule& Module, const PCGGrammar::FModuleDescriptor& Descriptor, auto& RecursiveFn) -> void
	{
		if(Descriptor.Type == PCGGrammar::EModuleType::Literal)
		{
			if (const FPCGSubdivisionSubmodule* It = InModulesInfo.Find(Descriptor.Symbol))
			{
				Module.UnitSize = It->Size;
				Module.ConcreteUnitSize = It->Size;
				Module.bScalable = It->bScalable;
				Module.bIsValid = true;
			}
			else
			{
				UnmatchedTokens.Add(Descriptor.Symbol);
			}
		}
		else
		{
			// Go through submodules
			Module.Submodules.Reserve(Descriptor.Submodules.Num());

			// In any of the modules that can be expanded, the minimum concrete size is going to be either the MinSize if it's greater than 0, otherwise it will be the minimum of the minimum concrete sizes of all submodules
			double MinSubmoduleConcreteSize = 0.0;

			for (const PCGGrammar::FModuleDescriptor& SubDescriptor : Descriptor.Submodules)
			{
				PCGGrammar::FTokenizedModule& Submodule = Module.Submodules.Emplace_GetRef(&SubDescriptor);
				RecursiveFn(Submodule, SubDescriptor, RecursiveFn);

				if (Submodule.bIsValid)
				{
					// Update size/scalable, with Sequence/Root: sum and Stochastic/Priority : min
					if (Descriptor.Type == PCGGrammar::EModuleType::Root || Descriptor.Type == PCGGrammar::EModuleType::Sequence)
					{
						Module.UnitSize += Submodule.GetMinSize();
					}
					else if (Descriptor.Type == PCGGrammar::EModuleType::Stochastic || Descriptor.Type == PCGGrammar::EModuleType::Priority)
					{
						if (Module.bIsValid)
						{
							Module.UnitSize = FMath::Min(Module.UnitSize, Submodule.GetMinSize());
						}
						else
						{
							Module.UnitSize = Submodule.GetMinSize();
						}
					}

					if (Module.bIsValid)
					{
						MinSubmoduleConcreteSize = FMath::Min(MinSubmoduleConcreteSize, Submodule.GetMinConcreteSize());
					}
					else
					{
						MinSubmoduleConcreteSize = Submodule.GetMinConcreteSize();
					}

					Module.bScalable |= Submodule.bScalable;
					Module.bIsValid = true;
				}
			}

			// If all expansions of the module lead to nothing, we might as well remove it.
			if (Module.bIsValid && MinSubmoduleConcreteSize > 0)
			{
				Module.ConcreteUnitSize = FMath::Max(Module.UnitSize, MinSubmoduleConcreteSize);
			}
			else
			{
				Module.bIsValid = false;
			}
		}
	};

	if (Result.Root.Submodules.IsEmpty())
	{
		return {};
	}

	PCGGrammar::FTokenizedGrammar TokenizedGrammar;
	TokenizedGrammar.ParsedGrammar = MakeShared<PCGGrammar::FModuleDescriptor>(MoveTemp(Result.Root));
	TokenizedGrammar.ModuleGrammar = MakeShared<PCGGrammar::FTokenizedModule>(TokenizedGrammar.ParsedGrammar.Get());
	BuildNode(*TokenizedGrammar.ModuleGrammar, *TokenizedGrammar.ParsedGrammar, BuildNode);
	OutMinSize = TokenizedGrammar.ModuleGrammar->GetMinSize();

	// TODO Add quiet mode
	for (FName UnmatchedToken : UnmatchedTokens)
	{
		FText WarningMessage = FText::Format(LOCTEXT("UnmatchedTokensInGrammar", "Unmatched token found in grammar: {0}."), FText::FromName(UnmatchedToken));
		PCGLog::LogWarningOnGraph(WarningMessage, InContext);
	}

	return TokenizedGrammar;
}

TMap<FString, PCGGrammar::FTokenizedGrammar> FPCGSubdivisionBaseElement::GetTokenizedGrammarForPoints(FPCGContext* InContext, const UPCGBasePointData* InputData, const UPCGSubdivisionBaseSettings* InSettings, const FModuleInfoMap& InModulesInfo, double& OutMinSize) const
{
	TMap<FString, PCGGrammar::FTokenizedGrammar> Result;

	if (InSettings->GrammarSelection.bGrammarAsAttribute)
	{
		const FPCGAttributePropertyInputSelector Selector = InSettings->GrammarSelection.GrammarAttribute.CopyAndFixLast(InputData);
		const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, Selector);
		const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, Selector);
		if (!Accessor || !Keys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(Selector, InContext);
			return Result;
		}

		const bool bSuccess = PCGMetadataElementCommon::ApplyOnAccessor<FString>(*Keys, *Accessor, [&Result](const FString& InValue, int32)
		{
			if (!Result.Contains(InValue))
			{
				Result.Emplace(InValue);
			}
		});

		if (!bSuccess)
		{
			PCGLog::Metadata::LogFailToGetAttributeError<FString>(Selector, Accessor.Get(), InContext);
			return Result;
		}
	}
	else
	{
		Result.Emplace(InSettings->GrammarSelection.GrammarString);
	}

	for (auto& [Grammar, TokenizeGrammar] : Result)
	{
		TokenizeGrammar = PCGSubdivisionBase::GetTokenizedGrammar(InContext, Grammar, InModulesInfo, OutMinSize);
	}

	return Result;
}

bool FPCGSubdivisionBaseElement::MatchAndSetAttributes(const TArray<FPCGTaggedData>& InputData, TArray<FPCGTaggedData>& OutputData, const UPCGParamData* InModuleInfoParamData, const UPCGSubdivisionBaseSettings* InSettings, const FPCGMetadataDomainID& InTargetDomain) const
{
	check(InModuleInfoParamData && InModuleInfoParamData->Metadata);

	const UPCGMetadata* InputMetadata = InModuleInfoParamData->Metadata;

	// We prepare everything to process all the output data afterward.
	// Build the Symbol -> EntryKey mapping
	// Since we don't know if it is a Name or a String, we need to get both.
	TMap<FName, PCGMetadataEntryKey> SymbolToEntryKeyMapping;
	if (const FPCGMetadataAttribute<FName>* InSymbolNameAttribute = InputMetadata->GetConstTypedAttribute<FName>(PCGSubdivisionBase::Constants::SymbolAttributeName))
	{
		for (PCGMetadataEntryKey EntryKey = InputMetadata->GetItemKeyCountForParent(); EntryKey < InputMetadata->GetItemCountForChild(); ++EntryKey)
		{
			const FName Symbol = InSymbolNameAttribute->GetValueFromItemKey(EntryKey);
			if (!SymbolToEntryKeyMapping.Contains(Symbol))
			{
				SymbolToEntryKeyMapping.Emplace(Symbol, EntryKey);
			}
		}
	}
	else if (const FPCGMetadataAttribute<FString>* InSymbolStrAttribute = InputMetadata->GetConstTypedAttribute<FString>(PCGSubdivisionBase::Constants::SymbolAttributeName))
	{
		for (PCGMetadataEntryKey EntryKey = InputMetadata->GetItemKeyCountForParent(); EntryKey < InputMetadata->GetItemCountForChild(); ++EntryKey)
		{
			const FName Symbol(InSymbolStrAttribute->GetValueFromItemKey(EntryKey));
			if (!SymbolToEntryKeyMapping.Contains(Symbol))
			{
				SymbolToEntryKeyMapping.Emplace(Symbol, EntryKey);
			}
		}
	}
	else
	{
		return false;
	}

	// Also gather all the attribute in the input metadata
	TArray<FName> AttributeNames;
	TArray<EPCGMetadataTypes> AttributeTypes;
	InModuleInfoParamData->Metadata->GetAttributes(AttributeNames, AttributeTypes);
	check(AttributeNames.Num() == AttributeTypes.Num());

	for (FPCGTaggedData& TaggedData : OutputData)
	{
		// Be careful with the const cast, only allow it if the data is not present in the input too (forwarded)
		if (InputData.ContainsByPredicate([TaggedData](const FPCGTaggedData& InData) { return InData.Data == TaggedData.Data; }))
		{
			continue;
		}

		UPCGData* OutData = const_cast<UPCGData*>(TaggedData.Data.Get());
		check(OutData && OutData->MutableMetadata());

		UPCGMetadata* OutputMetadata = OutData->MutableMetadata();
		check(OutputMetadata);

		FPCGMetadataDomain* OutputMetadataDomain = OutputMetadata->GetMetadataDomain(InTargetDomain);

		if (!OutputMetadataDomain)
		{
			continue;
		}

		const bool bSupportMultipleValues = OutputMetadataDomain->SupportsMultiEntries();

		// Look for the Symbol Attribute in the output metadata to query its value.
		const FPCGMetadataAttribute<FName>* OutSymbolAttribute = OutputMetadataDomain->GetConstTypedAttribute<FName>(InSettings->SymbolAttributeName);
		if (!OutSymbolAttribute)
		{
			continue;
		}

		if (UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(OutData))
		{
			TArray<const FPCGMetadataAttributeBase*> InAttributes;
			TArray<FPCGMetadataAttributeBase*> OutAttributes;

			InAttributes.Reserve(AttributeNames.Num());
			OutAttributes.Reserve(AttributeNames.Num());

			// Copy all the attributes in the output metadata
			for (const FName AttributeName : AttributeNames)
			{
				// Skip the symbol, it already exists, but perhaps with a different attribute name.
				if (AttributeName == PCGSubdivisionBase::Constants::SymbolAttributeName)
				{
					continue;
				}

				// If the attribute already exists, override it
				if (OutputMetadataDomain->HasAttribute(AttributeName))
				{
					OutputMetadataDomain->DeleteAttribute(AttributeName);
				}

				InAttributes.Add(InputMetadata->GetConstAttribute(AttributeName));
				OutAttributes.Add(OutputMetadataDomain->CopyAttribute(InAttributes.Last(), AttributeName, /*bKeepParent=*/false, /*bCopyEntries=*/false, /*bCopyValues=*/bSupportMultipleValues));
			}

			// @todo_pcg: potential optimization GetValuesFromItemKeys / SetValuesFromValueKeys. Would need support for TPCGValueRange on those.
			if (bSupportMultipleValues)
			{
				TPCGValueRange<int64> MetadataEntryRange = OutPointData->GetMetadataEntryValueRange(/*bAllocate=*/true);
				for (int32 Index = 0; Index < OutPointData->GetNumPoints(); ++Index)
				{
					const FName Symbol = OutSymbolAttribute->GetValueFromItemKey(MetadataEntryRange[Index]);

					if (!SymbolToEntryKeyMapping.Contains(Symbol))
					{
						continue;
					}

					const PCGMetadataEntryKey InputEntryKey = SymbolToEntryKeyMapping[Symbol];

					if (InputEntryKey == PCGInvalidEntryKey)
					{
						continue;
					}

					for (int32 i = 0; i < InAttributes.Num(); ++i)
					{
						PCGMetadataValueKey InputValueKey = InAttributes[i]->GetValueKey(InputEntryKey);
						OutAttributes[i]->SetValueFromValueKey(MetadataEntryRange[Index], InputValueKey);
					}
				}
			}
			else
			{
				const FName Symbol = OutSymbolAttribute->GetValueFromItemKey(PCGFirstEntryKey);

				if (!SymbolToEntryKeyMapping.Contains(Symbol))
				{
					continue;
				}

				const PCGMetadataEntryKey InputEntryKey = SymbolToEntryKeyMapping[Symbol];

				if (InputEntryKey == PCGInvalidEntryKey)
				{
					continue;
				}
				
				if (OutputMetadataDomain->GetItemCountForChild() == 0)
				{
					OutputMetadataDomain->AddEntry();
				}

				for (int32 i = 0; i < InAttributes.Num(); ++i)
				{
					OutAttributes[i]->SetValue(PCGFirstEntryKey, InAttributes[i], InputEntryKey);
				}
			}
		}
		else
		{
			// If we have a spatial data that is not a point data, we can only operate on Default values.
			// So add all the attributes, and set the default value to the same value for the chosen entry key.
			const FName Symbol = OutSymbolAttribute->GetValue(PCGDefaultValueKey);
			const PCGMetadataEntryKey InputEntryKey = SymbolToEntryKeyMapping[Symbol];

			auto AddAttribute = [InputMetadata, OutputMetadataDomain, InputEntryKey]<typename T>(T Dummy, const FName AttributeName)
			{
				const FPCGMetadataAttribute<T>* InAttribute = InputMetadata->GetConstTypedAttribute<T>(AttributeName);
				check(InAttribute);

				// Overwrite any existing attribute
				if (OutputMetadataDomain->HasAttribute(AttributeName))
				{
					OutputMetadataDomain->DeleteAttribute(AttributeName);
				}

				OutputMetadataDomain->CreateAttribute<T>(AttributeName, InAttribute->GetValueFromItemKey(InputEntryKey), InAttribute->AllowsInterpolation(), /*bOverrideParent=*/true);
			};

			for (int32 i = 0; i < AttributeNames.Num(); ++i)
			{
				// Skip the symbol attribute, as it is already in the OutData (with perhaps a different name)
				if (AttributeNames[i] == PCGSubdivisionBase::Constants::SymbolAttributeName)
				{
					continue;
				}

				// Skip any attribute that already exists in the output data
				if (OutputMetadataDomain->HasAttribute(AttributeNames[i]))
				{
					continue;
				}

				PCGMetadataAttribute::CallbackWithRightType(static_cast<int16>(AttributeTypes[i]), AddAttribute, AttributeNames[i]);
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
