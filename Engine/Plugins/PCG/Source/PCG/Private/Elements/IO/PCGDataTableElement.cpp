// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/IO/PCGDataTableElement.h"

#include "PCGComponent.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Elements/IO/PCGExternalDataContext.h"

#if WITH_EDITOR
#include "Helpers/PCGDynamicTrackingHelpers.h"
#endif // WITH_EDITOR

#include "Engine/DataTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataTableElement)

#define LOCTEXT_NAMESPACE "PCGLoadDataTable"

#if WITH_EDITOR
void UPCGLoadDataTableSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (DataTable.IsNull() || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGLoadDataTableSettings, DataTable)))
	{
		return;
	}

	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(DataTable.ToSoftObjectPath());

	OutKeysToSettings.FindOrAdd(Key).Emplace(this, /*bCulling=*/false);
}

FText UPCGLoadDataTableSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Load Data Table");
}

FText UPCGLoadDataTableSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Loads data from DataTable asset");
}

#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGLoadDataTableSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGDataTypeIdentifier UPCGLoadDataTableSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	check(InPin);
	if (!InPin->IsOutputPin())
	{
		return Super::GetCurrentPinTypesID(InPin);
	}

	FPCGDataTypeIdentifier Result = UPCGDataTypeIdentifierHelpers::GetIdentifierFromLegacyType(OutputType);
	return Result.IsValid() ? Result : Super::GetCurrentPinTypesID(InPin); 
}

FPCGElementPtr UPCGLoadDataTableSettings::CreateElement() const
{
	return MakeShared<FPCGLoadDataTableElement>();
}

bool FPCGLoadDataTableElement::PrepareLoad(FPCGExternalDataContext* Context) const
{
	check(Context);
	const UPCGLoadDataTableSettings* Settings = Context->GetInputSettings<UPCGLoadDataTableSettings>();
	check(Settings);

	// Nothing to do if we have no data table.
	if (Settings->DataTable.IsNull())
	{
		return true;
	}

	// 1. Request load data table. Return false if we need to wait, otherwise continue.
	if (!Context->WasLoadRequested())
	{
		if (!Context->RequestResourceLoad(Context, { Settings->DataTable.ToSoftObjectPath() }, !Settings->bSynchronousLoad))
		{
			return false;
		}
	}

	UDataTable* DataTable = Settings->DataTable.Get();
	if (!DataTable)
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("DataTableNotLoaded", "Provided Data Table ('{0}') could not be loaded."), FText::FromString(Settings->DataTable.ToString())));
		return true;
	}

	// 2. Make sure it has a row struct
	const UScriptStruct* RowStruct = DataTable->GetRowStruct();
	if (!RowStruct)
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("DataTableNoRowStruct", "Provided Data Table ('{0}') does not have a valid row definition."), FText::FromString(Settings->DataTable.ToString())));
		return true;
	}

	// 2.1 Validate that there are rows too
	if (DataTable->GetRowMap().IsEmpty())
	{
		PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("DataTableIsEmpty", "Provided Data Table ('{0}') is empty."), FText::FromString(Settings->DataTable.ToString())));
		return true;
	}

	// 3-e. Create point data, and the mapping object
	UPCGData* OutData = nullptr;
	UPCGMetadata* OutMetadata = nullptr;

	if (Settings->OutputType == EPCGExclusiveDataType::Point)
	{
		UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(Context);
		// Dummy implementation - note that we do initialize the points here in cases where some of the properties would not be set
		// Set num points before creating accessors
		PointData->SetNumPoints(DataTable->GetRowMap().Num());

		check(PointData);
		OutData = PointData;
		OutMetadata = PointData->MutableMetadata();
	}
	else if(Settings->OutputType == EPCGExclusiveDataType::Param)
	{
		UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		check(ParamData);
		OutData = ParamData;
		OutMetadata = ParamData->MutableMetadata();
	}
	else
	{
		ensure(0);
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("Output format not supported", "Unable to load data table to selected output type"));
		return true;
	}

	FPCGExternalDataContext::FPointDataAccessorsMapping& PointDataAccessorMapping = Context->PointDataAccessorsMapping.Emplace_GetRef();
	PointDataAccessorMapping.Data = OutData;
	PointDataAccessorMapping.Metadata = OutMetadata;

	// 3. Build property to accessor mappings, including renamings, and create new attributes as needed
	for (TFieldIterator<const FProperty> FieldIt(RowStruct, EFieldIterationFlags::IncludeSuper); FieldIt; ++FieldIt)
	{
		FString FieldName = RowStruct->GetAuthoredNameForField(*FieldIt);

		// Try to create the read accessor
		TUniquePtr<IPCGAttributeAccessor> DataAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(*FieldIt);

		if (!DataAccessor)
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("DataTableFieldTypeNotSupported", "Provided Data Table ('{0}') property '{1}' is not of a supported type."), FText::FromString(Settings->DataTable.ToString()), FText::FromString(RowStruct->GetAuthoredNameForField(*FieldIt))));
			continue;
		}

		// Try to match to a property (if the name maps to a property) or create attribute as needed
		FPCGAttributePropertyOutputSelector PointPropertySelector;

		if (const FPCGAttributePropertySelector* MappedField = Settings->AttributeMapping.Find(FieldName))
		{
			PointPropertySelector.ImportFromOtherSelector(*MappedField);
			FieldName = PointPropertySelector.GetName().ToString();
		}
		else
		{
			PointPropertySelector.Update(FieldName);
		}

 		if (PointPropertySelector.GetSelection() == EPCGAttributePropertySelection::Attribute)
		{
			// Create attribute with the appropriate type
			const void* FirstRow = DataTable->GetRowMap().CreateConstIterator().Value();
			OutMetadata->CreateAttributeFromDataProperty(FName(FieldName), FirstRow, *FieldIt);
		}

		TUniquePtr<IPCGAttributeAccessor> PointPropertyAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutData, PointPropertySelector);

		if (!PointPropertyAccessor)
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("UnableToCreatePointProperty", "Unable to create/access point property/attribute '{0}'."), FText::FromString(FieldName)));
			continue;
		}

		// Final verification, if we can put the value of input into output
		if (!PCG::Private::IsBroadcastable(DataAccessor->GetUnderlyingType(), PointPropertyAccessor->GetUnderlyingType()))
		{
			FText InputTypeName = FText::FromString(PCG::Private::GetTypeName(DataAccessor->GetUnderlyingType()));
			FText OutputTypeName = FText::FromString(PCG::Private::GetTypeName(PointPropertyAccessor->GetUnderlyingType()));

			PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("CannotBroadcastTypes", "Cannot convert input type '{0}' into output type '{1}'"), InputTypeName, OutputTypeName));
			continue;
		}

		PointDataAccessorMapping.RowToPointAccessors.Emplace(MoveTemp(DataAccessor), MoveTemp(PointPropertyAccessor), PointPropertySelector);
	}

	if (!Context->PointDataAccessorsMapping.IsEmpty())
	{
		FPCGTaggedData& OutTaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
		OutTaggedData.Data = OutData;

		Context->bDataPrepared = true;
	}

	return true;
}


bool FPCGLoadDataTableElement::ExecuteLoad(FPCGExternalDataContext* Context) const
{
	check(Context);

	if (Context->PointDataAccessorsMapping.IsEmpty())
	{
		return true;
	}

	const UPCGLoadDataTableSettings* Settings = Context->GetInputSettings<UPCGLoadDataTableSettings>();
	check(Settings);

	//TODO: instead of trying to reload here, it would be better to make sure that this is rooted properly
	UDataTable* DataTable = Settings->DataTable.LoadSynchronous();
	check(DataTable);

	// Note that while this is prepared for execution in multiple calls, we need to make sure the data table is kept in memory at all times otherwise the row keys could become stale
	check(Context->PointDataAccessorsMapping.Num() == 1);
	FPCGExternalDataContext::FPointDataAccessorsMapping& PointDataAccessorMapping = Context->PointDataAccessorsMapping[0];

	// Preallocate the point array, and create the row keys
	UPCGData* Data = PointDataAccessorMapping.Data;
	check(Data);

	if (UPCGParamData* ParamData = Cast<UPCGParamData>(Data))
	{
		check(ParamData->Metadata);
		TArray<FName> AttributeNames;
		TArray<EPCGMetadataTypes> AttributeTypes;
		ParamData->Metadata->GetAttributes(AttributeNames, AttributeTypes);

		TArray<FPCGMetadataAttributeBase*> Attributes;
		for (const FName AttributeName : AttributeNames)
		{
			Attributes.Add(ParamData->Metadata->GetMutableAttribute(AttributeName));
		}

		for (int32 EntryIndex = 0; EntryIndex < DataTable->GetRowMap().Num(); ++EntryIndex)
		{
			const PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();

			// Need to create a value entry for each newly created entry in the attributes, otherwise we will not write properly to the output
			for (FPCGMetadataAttributeBase* Attribute : Attributes)
			{
				Attribute->SetValueFromValueKey(EntryKey, PCGDefaultValueKey);
			}
		}
	}

	TArray<void*> AllRows;
	AllRows.Reserve(DataTable->GetRowMap().Num());
	for (TMap<FName, uint8*>::TConstIterator RowMapIter(DataTable->GetRowMap().CreateConstIterator()); RowMapIter; ++RowMapIter)
	{
		AllRows.Add(RowMapIter.Value());
	}

	PointDataAccessorMapping.RowKeys = MakeUnique<FPCGAttributeAccessorKeysGenericPtrs>(AllRows);

#if WITH_EDITOR
	// If we have an override, register for dynamic tracking.
	if (Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGLoadDataTableSettings, DataTable)))
	{
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, FPCGSelectionKey::CreateFromPath(DataTable), /*bIsCulled=*/false);
	}
#endif // WITH_EDITOR
	
	return FPCGExternalDataElement::ExecuteLoad(Context);
}

#undef LOCTEXT_NAMESPACE
