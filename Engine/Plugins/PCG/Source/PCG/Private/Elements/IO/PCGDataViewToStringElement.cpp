// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/IO/PCGDataViewToStringElement.h"

#include "Data/DataView/PCGDataViewCSVConverter.h"
#include "Data/DataView/PCGDataViewData.h"
#include "Helpers/PCGSettingsHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataViewToStringElement)

#define LOCTEXT_NAMESPACE "PCGDataViewToStringElement"

UPCGDataViewToStringSettings::UPCGDataViewToStringSettings()
{
	ConverterParameters.InitializeAs(FPCGDataViewCSVParameters::StaticStruct());
}

#if WITH_EDITOR
EPCGChangeType UPCGDataViewToStringSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGDataViewToStringSettings, ConverterClass))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}

void UPCGDataViewToStringSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGDataViewToStringSettings, ConverterClass))
	{
		if (const UPCGDataViewConverterBase* ConverterCDO = ConverterClass ? ConverterClass->GetDefaultObject<UPCGDataViewConverterBase>() : nullptr)
		{
			ConverterParameters.InitializeAs(ConverterCDO->GetParameterStruct());
		}
		else
		{
			ConverterParameters.Reset();
		}
	}
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGDataViewToStringSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, FPCGDataTypeInfoDataView::AsId()).SetRequiredPin();
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGDataViewToStringSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGDataTypeIdentifier StringType = EPCGDataType::Param;
	StringType.CustomSubtype = static_cast<int32>(EPCGMetadataTypes::String);
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, StringType);
	return PinProperties;
}

FPCGElementPtr UPCGDataViewToStringSettings::CreateElement() const
{
	return MakeShared<FPCGDataViewToStringElement>();
}

#if WITH_EDITOR
TArray<FPCGSettingsOverridableParam> UPCGDataViewToStringSettings::GatherOverridableParams() const
{
	TArray<FPCGSettingsOverridableParam> OverridableParams = Super::GatherOverridableParams();

	if (ConverterParameters.IsValid())
	{
		PCGSettingsHelpers::FPCGGetAllOverridableParamsConfig Config{};
		Config.bExtractArrays = true;
		return PCGSettingsHelpers::GetAllOverridableParams(ConverterParameters.GetScriptStruct(), Config);
	}

	return {};
}
#endif // WITH_EDITOR

void UPCGDataViewToStringSettings::FixingOverridableParamPropertyClass(FPCGSettingsOverridableParam& Param) const
{
	bool bFound = false;

	if (!Param.PropertiesNames.IsEmpty() && !Param.Properties.IsEmpty())
	{
		const UScriptStruct* ScriptStruct = ConverterParameters.GetScriptStruct();
		const FProperty* Property = ScriptStruct ? ScriptStruct->FindPropertyByName(Param.PropertiesNames[0]) : nullptr;
		if (Property && ensure(Param.Properties[0]->GetOwnerStruct() == ScriptStruct))
		{
			Param.PropertyClass = ScriptStruct;
			bFound = true;
		}
	}

	if (!bFound)
	{
		Super::FixingOverridableParamPropertyClass(Param);
	}
}

FStructView FPCGDataViewToStringExecutionContext::GetExternalStructContainerForOverridableParam(const FPCGSettingsOverridableParam& InParam)
{
	UPCGDataViewToStringSettings* Settings = GetMutableInputSettings<UPCGDataViewToStringSettings>();
	if (Settings && Settings->ConverterParameters.IsValid() && !InParam.PropertiesNames.IsEmpty() && Settings->ConverterParameters.GetScriptStruct()->FindPropertyByName(InParam.PropertiesNames[0]))
	{
		return Settings->ConverterParameters;
	}
	else
	{
		return nullptr;
	}
}

bool FPCGDataViewToStringElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataViewToStringElement::Execute);

	const UPCGDataViewToStringSettings* Settings = InContext->GetInputSettings<UPCGDataViewToStringSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	if (Inputs.IsEmpty())
	{
		return true;
	}

	if (!Settings->ConverterClass)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("InvalidConverterClass", "Converter class is invalid."), InContext);
		return true;
	}

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGDataViewData* DataViewData = Cast<UPCGDataViewData>(Input.Data);
		if (!DataViewData)
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}

		const FPCGDataView& DataView = DataViewData->GetDataView();
		if (!DataView.IsValid())
		{
			PCGLog::LogWarningOnGraph(LOCTEXT("InvalidDataView", "Data View selection is empty or internal data is no longer valid."));
			return true;
		}

		const UPCGDataViewConverterBase* ConverterCDO = Settings->ConverterClass->GetDefaultObject<UPCGDataViewConverterBase>();
		TValueOrError<FString, FText> Result = ConverterCDO->SerializeToString(DataView, Settings->ConverterParameters);
		if (Result.HasError())
		{
			PCGLog::LogErrorOnGraph(Result.GetError(), InContext);
			return true;
		}

		UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InContext);
		FPCGMetadataAttribute<FString>* Attribute = ParamData->Metadata->CreateAttribute<FString>(DataView.ViewedData->GetFName(), Result.GetValue(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		check(Attribute);
		ParamData->Metadata->AddEntry();

		FPCGTaggedData& Out = InContext->OutputData.TaggedData.Emplace_GetRef(Input);
		Out.Data = ParamData;
	}

	return true;
}

void FPCGDataViewToStringElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	const UPCGDataViewToStringSettings* Settings = CastChecked<UPCGDataViewToStringSettings>(InParams.Settings);
	Crc.Combine(Settings->GetOrComputeCrc(/*bFullDataCrc=*/false));

	OutCrc = Crc;
}

#undef LOCTEXT_NAMESPACE
