// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/IO/PCGExportSelectedAttributes.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Helpers/IO/PCGJsonHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Utils/PCGLogErrors.h"

#include "DesktopPlatformModule.h"
#include "IDesktopPlatform.h"
#include "Data/PCGSplineStruct.h"
#include "HAL/FileManager.h"
#include "HAL/FileManagerGeneric.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Helpers/IO/PCGIOHelpers.h"

#if WITH_EDITOR
#include "IContentBrowserSingleton.h"
#include "ScopedTransaction.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGExportSelectedAttributes)

#define LOCTEXT_NAMESPACE "PCGExportSelectedAttributesElement"

// Register the custom version with core
namespace PCGExportSelectedAttributes
{
	struct FAccessorData
	{
		FPCGAttributePropertySelector Selector;
		TUniquePtr<const IPCGAttributeAccessor> Accessor = nullptr;
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = nullptr;
	};

	namespace Constants
	{
		static constexpr int32 DefaultInlineSourceCount = 16;
	}

	namespace Helpers
	{
		void AddAllPointPropertySelectors(TArray<FPCGAttributePropertySelector>& InOutSelectors)
		{
			// Some of the point properties are redundant, so pick these by hand for now
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Transform));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Density));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::BoundsMin));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::BoundsMax));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Color));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Steepness));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Seed));
		}

		void AddAllSplinePropertySelectors(TArray<FPCGAttributePropertySelector>& InOutSelectors)
		{
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePropertySelector(TEXT("Position")));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePropertySelector(TEXT("Rotation")));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePropertySelector(TEXT("Scale")));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePropertySelector(TEXT("ArriveTangent")));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePropertySelector(TEXT("LeaveTangent")));
			InOutSelectors.Add(FPCGAttributePropertySelector::CreatePropertySelector(TEXT("InterpType")));
		}

		void AddAllAttributeSelectors(const UPCGData* InData, TArray<FPCGAttributePropertySelector>& InOutSelectors)
		{
			check(InData);
			check(InData->ConstMetadata());

			TArray<FPCGAttributeIdentifier> AttributeNames;
			TArray<EPCGMetadataTypes> AttributeTypes;
			InData->ConstMetadata()->GetAllAttributes(AttributeNames, AttributeTypes);
			check(AttributeNames.Num() == AttributeTypes.Num());

			for (int32 Index = 0; Index < AttributeNames.Num(); ++Index)
			{
				InOutSelectors.Add(FPCGAttributePropertySelector::CreateAttributeSelector(AttributeNames[Index].Name));
			}
		}

		namespace String
		{
			FString MergeFileName(const FString& Path, const FString& File, const FString& Ext)
			{
				if (Path.IsEmpty() || File.IsEmpty() || Ext.IsEmpty())
				{
					return FString();
				}

				return FString::Format(TEXT("{0}/{1}{2}"), {Path, File, Ext});
			}

			void ToPrecisionString(const double Value, const int32 Precision, FString& OutString)
			{
				OutString = FString::SanitizeFloat(Value, Precision);
			}
		}
	}

	using FSourceDataArray = TArray<FAccessorData, TInlineAllocator<Constants::DefaultInlineSourceCount>>;
}

UPCGExportSelectedAttributesSettings::UPCGExportSelectedAttributesSettings()
{
	// Initialize with one @Last source.
	AttributeSelectors.Emplace();
}

TArray<FPCGPinProperties> UPCGExportSelectedAttributesSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/false, /*bInAllowMultipleData=*/false).SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGExportSelectedAttributesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DepPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any);
#if WITH_EDITOR
	DepPin.Tooltip = PCGPinConstants::Tooltips::ExecutionDependencyTooltip;
#endif // WITH_EDITOR
	DepPin.Usage = EPCGPinUsage::DependencyOnly;

	return PinProperties;
}

FPCGElementPtr UPCGExportSelectedAttributesSettings::CreateElement() const
{
	return MakeShared<FPCGExportSelectedAttributesElement>();
}

bool FPCGExportSelectedAttributesElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGExportSelectedAttributesElement::Execute);
	using namespace PCGExportSelectedAttributes;

	// Since this generates data on disk, only allow execution on editor approved platforms.
#if PLATFORM_WINDOWS || PLATFORM_LINUX || PLATFORM_MAC

	const UPCGExportSelectedAttributesSettings* Settings = Context->GetInputSettings<UPCGExportSelectedAttributesSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	if (Inputs.IsEmpty() || Settings->AttributeSelectors.IsEmpty())
	{
		return true;
	}

	// @todo_pcg: In the future, could be extended to support multiple inputs, but should also support separate file names for each, etc.
	if (Inputs.Num() > 1)
	{
		PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGPinConstants::DefaultInputLabel, Context);
	}

	// Take the property override (FString) if it exists, otherwise take the user property (FDirectoryPath).
	FString OutputDirectory = Settings->Path.Path;
	if (OutputDirectory.IsEmpty())
	{
#if WITH_EDITOR
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		const bool bSuccess = DesktopPlatform->OpenDirectoryDialog(nullptr, TEXT("Choose output directory"), FPaths::GameUserDeveloperDir(), OutputDirectory);

		if (!bSuccess || OutputDirectory.IsEmpty())
#endif // WITH_EDITOR
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidDirectoryError", "Invalid or empty export directory: {0}"), FText::FromString(OutputDirectory)), Context);
			return true;
		}
	}

	if (Settings->FileName.IsEmpty())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("EmptyFileName", "A valid file name is required."), Context);
		return true;
	}

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("ExportingSelectedAttributes", "Exporting Selected Attributes from PCG"), Context->ExecutionSource.Get() && Context->ExecutionSource->GetExecutionState().UseTransactions());
#endif

	// Create the serialization object pointers to be utilized and cleaned up later.
	FArchive* Archive = nullptr;
	TSharedPtr<FJsonObject> RootJsonObject = nullptr;

	using namespace PCG::IO::Constants;
	// Initialize the serialization object needed.
	if (Settings->Format == EPCGExportAttributesFormat::Binary)
	{
		FString FinalPath = *Helpers::String::MergeFileName(OutputDirectory, Settings->FileName, PCG::IO::Binary::Constants::Extension);
		Archive = IFileManager::Get().CreateFileWriter(*FinalPath);
		if (!Archive)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FileWriterInvalidPath", "Could not create a valid archive writer with path: {0}"), FText::FromString(FinalPath)), Context);
			return true;
		}

		// Set up the custom export version
		Archive->SetCustomVersion(FCustomExportVersion::GUID, FCustomExportVersion::LatestVersion, FCustomExportVersion::GetFriendlyName());

		// Set up the user's custom data version
		if (Settings->bAddCustomDataVersion)
		{
			int32 CustomVersion = Settings->CustomVersion;
			*Archive << CustomVersion;
		}
	}
	else if (Settings->Format == EPCGExportAttributesFormat::Json)
	{
		RootJsonObject = MakeShared<FJsonObject>();

		// Set up the Json header to include custom versions
		TSharedPtr<FJsonObject> JsonHeaderObject = MakeShared<FJsonObject>();
		PCG::IO::Json::Helpers::AppendVersionToHeader(JsonHeaderObject, FCustomExportVersion::LatestVersion);
		JsonHeaderObject->SetNumberField(TEXT("custom_data_version"), Settings->bAddCustomDataVersion ? Settings->CustomVersion : -1);
		RootJsonObject->SetObjectField(TEXT("Header"), std::move(JsonHeaderObject));
	}

	const UPCGData* InputData = Inputs[0].Data;
	const UPCGMetadata* Metadata = InputData ? InputData->ConstMetadata() : nullptr;
	if (!InputData || !Metadata)
	{
		PCGLog::InputOutput::LogInvalidInputDataError(Context);
		return true;
	}

	TArray<FPCGAttributePropertySelector> AttributeSelectors;

	if (Settings->bExportAllAttributes)
	{
		using namespace Helpers;

		/** @todo_pcg: Eventually, it would be useful to be able to iterate on all attributes and properties at once,
		* to include all/some domains established by the Data. This can be utilized once the logic here is abstracted out
		* into an API. */
		if (InputData->IsA<UPCGBasePointData>())
		{
			AddAllPointPropertySelectors(AttributeSelectors);
		}
		else if (InputData->IsA<UPCGSplineData>())
		{
			AddAllSplinePropertySelectors(AttributeSelectors);
		}

		AddAllAttributeSelectors(InputData, AttributeSelectors);
	}
	else
	{
		for (const FPCGAttributePropertyInputSelector& Selector : Settings->AttributeSelectors)
		{
			AttributeSelectors.Emplace(Selector.CopyAndFixLast(InputData));
		}
	}

	// Create and pre-process the attribute source data to be consumed later. Confirmed non-empty earlier.
	FSourceDataArray SourceData;
	SourceData.Reserve(Settings->AttributeSelectors.Num());

	for (FPCGAttributePropertySelector& Selector : AttributeSelectors)
	{
		FAccessorData Data;
		Data.Selector = std::move(Selector);
		Data.Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, Data.Selector);
		Data.Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, Data.Selector);
		if (Data.Accessor && Data.Keys)
		{
			SourceData.Emplace(std::move(Data));
		}
		else
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(Data.Selector, Context);
		}
	}

	if (SourceData.IsEmpty())
	{
		return true;
	}

	if (Settings->Format == EPCGExportAttributesFormat::Binary)
	{
		check(Archive);

		// Arrays of values can be consumed by the archive directly and should be consistent. Note: Archive serializes by attribute -only-.
		for (const FAccessorData& Data : SourceData)
		{
			PCGMetadataAttribute::CallbackWithRightType(Data.Accessor->GetUnderlyingType(), [Archive, &Data, Context]<typename T>(const T&)
			{
				TArray<T> Values;
				if (PCGAttributeAccessorHelpers::ExtractAllValues(Data.Accessor.Get(), Data.Keys.Get(), Values, EPCGAttributeAccessorFlags::StrictType))
				{
					*Archive << Values;
				}
				else
				{
					PCGLog::Metadata::LogFailToGetAttributeError(Data.Selector, Context);
				}
			});
		}
	}
	else if (Settings->Format == EPCGExportAttributesFormat::Json)
	{
		check(RootJsonObject);

		TArray<TTuple<FString, TArray<TSharedPtr<FJsonValue>>>> AllSourceJsonValues;
		AllSourceJsonValues.SetNum(SourceData.Num());
		// Pre-process the source data into Json values.
		for (int32 SourceIndex = 0; SourceIndex < SourceData.Num(); ++SourceIndex)
		{
			FAccessorData& Data = SourceData[SourceIndex];
			FString& SourceName = AllSourceJsonValues[SourceIndex].Get<0>();
			SourceName = Data.Selector.ToString();

			TArray<TSharedPtr<FJsonValue>>& SourceJsonValues = AllSourceJsonValues[SourceIndex].Get<1>();

			PCGMetadataAttribute::CallbackWithRightType(Data.Accessor->GetUnderlyingType(), [&Data, &SourceJsonValues]<typename T>(const T&)
			{
				// For floating point values and containers (FVector, FRotator, FTransform, etc.), use the Json numerical to maintain precision.
				if constexpr (PCG::Private::MetadataTraits<T>::IsFloatingPoint)
				{
					TArray<T> Values;
					PCGAttributeAccessorHelpers::ExtractAllValues(Data.Accessor.Get(), Data.Keys.Get(), Values, EPCGAttributeAccessorFlags::StrictType);

					SourceJsonValues.Reserve(Values.Num());
					Algo::Transform(Values, SourceJsonValues, [&Data](const T& Value)
					{
						return PCG::IO::Json::Helpers::ConvertFloatingPointType(Value);
					});
				}
				else // For everything else, automatically broadcast it to string
				{
					TArray<FString> Values;
					PCGAttributeAccessorHelpers::ExtractAllValues(Data.Accessor.Get(), Data.Keys.Get(), Values, EPCGAttributeAccessorFlags::AllowBroadcast);

					SourceJsonValues.Reserve(Values.Num());
					Algo::Transform(Values, SourceJsonValues, [](const FString& Value)
					{
						return MakeShared<FJsonValueString>(Value);
					});
				}
			});
		}

		// Iterate over the pre-processed source data by element index, to get the attribute values together "by element".
		if (Settings->Layout == EPCGExportAttributesLayout::ByElement)
		{
			check(!SourceData.IsEmpty());

			const int32 ElementNum = SourceData[0].Keys->GetNum();
			for (int32 ElementIndex = 0; ElementIndex < ElementNum; ++ElementIndex)
			{
				TSharedPtr<FJsonObject> ElementJsonObject = MakeShared<FJsonObject>();

				for (int32 SourceIndex = 0; SourceIndex < SourceData.Num(); ++SourceIndex)
				{
					check(AllSourceJsonValues.IsValidIndex(SourceIndex));

					TArray<TSharedPtr<FJsonValue>>& SourceValues = AllSourceJsonValues[SourceIndex].Get<1>();
					const FString& SourceName = AllSourceJsonValues[SourceIndex].Get<0>();

					check(SourceValues.IsValidIndex(ElementIndex));
					// Currently only support the following Json types:
					check(SourceValues[ElementIndex]->Type == EJson::Array
						|| SourceValues[ElementIndex]->Type == EJson::Number
						|| SourceValues[ElementIndex]->Type == EJson::String
						|| SourceValues[ElementIndex]->Type == EJson::Object);

					ElementJsonObject->SetField(SourceName, std::move(SourceValues[ElementIndex]));
				}

				RootJsonObject->SetObjectField(
					FText::Format(LOCTEXT("JsonElementPrefixFormat", "Element [{0}]"), FText::AsNumber(ElementIndex)).ToString(),
					std::move(ElementJsonObject));
			}
		}
		else if (Settings->Layout == EPCGExportAttributesLayout::ByAttribute)
		{
			for (int32 SourceIndex = 0; SourceIndex < SourceData.Num(); ++SourceIndex)
			{
				check(AllSourceJsonValues.IsValidIndex(SourceIndex));

				TArray<TSharedPtr<FJsonValue>>& SourceValues = AllSourceJsonValues[SourceIndex].Get<1>();
				const FString& SourceName = AllSourceJsonValues[SourceIndex].Get<0>();

				RootJsonObject->SetArrayField(SourceName, std::move(SourceValues));
			}
		}
		else
		{
			checkNoEntry();
			return true;
		}
	}

	// Clean up the serialization objects that exist.
	if (Archive)
	{
		Archive->Close();
	}
	else if (RootJsonObject) // Conclude the Json serialization.
	{
		FString JsonString;
		// @todo_pcg: Allow the user to set the final object name
		const TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonString);
		if (!FJsonSerializer::Serialize(RootJsonObject.ToSharedRef(), JsonWriter))
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("JsonSerializationError", "Serialization of Json data failed."), Context);
			return true;
		}

		const FString FinalPath = Helpers::String::MergeFileName(OutputDirectory, Settings->FileName, TEXT(".json"));
		if (!FFileHelper::SaveStringToFile(JsonString, *FinalPath))
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("JsonInvalidPath", "Could not create a valid Json file with path: {0}"), FText::FromString(FinalPath)), Context);
			return true;
		}
	}
	else
	{
		// There was no export data. Was a new format created?
		checkNoEntry();
	}

#else  // Not Windows, Linux, or Mac
	UE_LOGF(LogPCG, Verbose, "Running 'Export Selected Attributes' node, which has been disabled on this platform.");
#endif // PLATFORM_WINDOWS || PLATFORM_LINUX || PLATFORM_MAC

	return true;
}

#undef LOCTEXT_NAMESPACE
