// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/IO/PCGExternalData.h"
#include "Elements/IO/PCGExternalDataContext.h"

#include "PCGComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGExternalData)

#define LOCTEXT_NAMESPACE "PCGExternalData"

FPCGContext* FPCGExternalDataElement::CreateContext()
{
	return new FPCGExternalDataContext();
}

bool FPCGExternalDataElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGExternalDataElement::PrepareData);
	FPCGExternalDataContext* Context = static_cast<FPCGExternalDataContext*>(InContext);
	check(Context);

	return PrepareLoad(Context);
}

bool FPCGExternalDataElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGExternalDataElement::Execute);
	FPCGExternalDataContext* Context = static_cast<FPCGExternalDataContext*>(InContext);
	check(Context);

	if (!Context->bDataPrepared)
	{
		return true;
	}

	return ExecuteLoad(Context);
}

bool FPCGExternalDataElement::ExecuteLoad(FPCGExternalDataContext* Context) const
{
	check(Context);

	// Do in parallel according to the resources given in the context
	for (FPCGExternalDataContext::FPointDataAccessorsMapping& PointDataAccessorMapping : Context->PointDataAccessorsMapping)
	{
		UPCGData* Data = PointDataAccessorMapping.Data;
		TUniquePtr<const IPCGAttributeAccessorKeys>& RowKeys = PointDataAccessorMapping.RowKeys;

		for (FPCGExternalDataContext::FRowToPointAccessors& RowToPointAccessor : PointDataAccessorMapping.RowToPointAccessors)
		{
			TUniquePtr<IPCGAttributeAccessorKeys> PointKeys = PCGAttributeAccessorHelpers::CreateKeys(Data, RowToPointAccessor.Selector);

			auto Operation = [&RowKeys, &RowToPointAccessor, &PointKeys](auto Dummy)
			{
				using OutputType = decltype(Dummy);
				OutputType Value{};

				EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::AllowBroadcast;

				const int32 NumberOfElements = PointKeys->GetNum();
				constexpr int32 ChunkSize = 256;

				TArray<OutputType, TInlineAllocator<ChunkSize>> TempValues;
				TempValues.SetNum(ChunkSize);

				const int32 NumberOfIterations = (NumberOfElements + ChunkSize - 1) / ChunkSize;

				for (int32 i = 0; i < NumberOfIterations; ++i)
				{
					const int32 StartIndex = i * ChunkSize;
					const int32 Range = FMath::Min(NumberOfElements - StartIndex, ChunkSize);
					TArrayView<OutputType> View(TempValues.GetData(), Range);

					RowToPointAccessor.RowAccessor->GetRange<OutputType>(View, StartIndex, *RowKeys, Flags);
					RowToPointAccessor.PointAccessor->SetRange<OutputType>(View, StartIndex, *PointKeys, Flags);
				}

				return true;
			};

			if (!PCGMetadataAttribute::CallbackWithRightType(RowToPointAccessor.PointAccessor->GetUnderlyingType(), Operation))
			{
				PCGE_LOG(Warning, GraphAndLog, LOCTEXT("ErrorGettingSettingValues", "Error while getting/setting values"));
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
