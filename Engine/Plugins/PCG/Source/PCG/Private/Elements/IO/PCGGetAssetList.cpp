// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/IO/PCGGetAssetList.h"

#include "PCGParamData.h"
#include "Utils/PCGLogErrors.h"

#include "Algo/Transform.h"
#include "Blueprint/BlueprintSupport.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "PCGGetAssetListElement"

namespace PCGGetAsset
{
	const FLazyName AssetReferenceLabel = TEXT("AssetReference");
	const FLazyName ClassPathLabel = TEXT("ClassPath");
	const FLazyName BlueprintGeneratedClassPathLabel = TEXT("BlueprintGeneratedClassPath");
}

TArray<FPCGPinProperties> UPCGGetAssetListSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

bool FPCGGetAssetListElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetAssetListElement::Execute);

	check(Context);

	const UPCGGetAssetListSettings* Settings = Context->GetInputSettings<UPCGGetAssetListSettings>();
	check(Settings);

#if WITH_EDITOR
	TArray<FAssetData> Assets;

	FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry"));
	if (!AssetRegistryModule)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("AssetRegistryNotLoaded", "The asset registry module is not loaded."), Context);
		return true;
	}

	if (Settings->AssetListSource == EPCGAssetListSource::Folder)
	{
		if (!AssetRegistryModule->Get().GetAssetsByPath(FName(*Settings->Directory.Path), Assets, /*bRecursive=*/true))
		{
			if (!Settings->bQuiet)
			{
				PCGLog::LogWarningOnGraph(LOCTEXT("FolderInvalid", "Unable to retrieve assets from folder."), Context);
			}
		}
	}
	else if(Settings->AssetListSource == EPCGAssetListSource::Collection)
	{
		FCollectionManagerModule* CollectionManagerPtr = FModuleManager::GetModulePtr<FCollectionManagerModule>("CollectionManager");
		if (!CollectionManagerPtr)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("CollectionManagerModuleNotLoaded", "The collection manager module isn't loaded."), Context);
			return true;
		}

		TSharedPtr<ICollectionContainer> CollectionContainer = CollectionManagerPtr->Get().GetProjectCollectionContainer();
		if (CollectionContainer)
		{
			TArray<FSoftObjectPath> AssetPaths;
			if (!CollectionContainer->GetAssetsInCollection(Settings->Collection, ECollectionShareType::CST_All, AssetPaths, ECollectionRecursionFlags::SelfAndChildren))
			{
				// Collection doesn't exist or is empty.
				if(!Settings->bQuiet)
				{
					PCGLog::LogWarningOnGraph(LOCTEXT("CollectionEmptyOrInexistant", "Unable to retrieve collection or it is empty."), Context);
				}
			
				return true;
			}
			else
			{
				Algo::Transform(AssetPaths, Assets, [AssetRegistryModule](const FSoftObjectPath& AssetPath) { return AssetRegistryModule->Get().GetAssetByObjectPath(AssetPath); });
			}
		}
		else
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("UnableToGetCollectionContainer", "Unable to retrieve project collection container."), Context);
			return true;
		}
	}
	else
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("InvalidAssetListSource", "Invalid asset list source."), Context);
		return true;
	}

	UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	check(ParamData && ParamData->Metadata);

	FPCGMetadataAttribute<FSoftObjectPath>* AssetReferenceAttribute = ParamData->MutableMetadata()->CreateAttribute<FSoftObjectPath>(PCGGetAsset::AssetReferenceLabel, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
	FPCGMetadataAttribute<FSoftClassPath>* ClassPathAttribute = Settings->bGetClassPath ? ParamData->MutableMetadata()->CreateAttribute<FSoftClassPath>(PCGGetAsset::ClassPathLabel, FSoftClassPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false) : nullptr;
	FPCGMetadataAttribute<FSoftClassPath>* BPClassPathAttribute = Settings->bGetClassPath ? ParamData->MutableMetadata()->CreateAttribute<FSoftClassPath>(PCGGetAsset::BlueprintGeneratedClassPathLabel, FSoftClassPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false) : nullptr;

	for(const FAssetData& Asset : Assets)
	{
		PCGMetadataEntryKey Entry = ParamData->MutableMetadata()->AddEntry();
		AssetReferenceAttribute->SetValue(Entry, Asset.GetSoftObjectPath());

		if (ClassPathAttribute)
		{
			ClassPathAttribute->SetValue(Entry, FSoftClassPath(Asset.AssetClassPath.ToString()));
		}

		FString BPClassPath;
		if (BPClassPathAttribute && Asset.GetTagValue(FBlueprintTags::GeneratedClassPath, BPClassPath))
		{
			BPClassPathAttribute->SetValue(Entry, FSoftClassPath(BPClassPath));
		}
	}

	FPCGTaggedData& OutputData = Context->OutputData.TaggedData.Emplace_GetRef();
	OutputData.Data = ParamData;
#else
	PCGLog::LogErrorOnGraph(LOCTEXT("NodeWorksOnlyInEdior", "The Get Asset List node works only in editor contexts."), Context);
#endif // WITH_EDITOR

	return true;
}

#undef LOCTEXT_NAMESPACE