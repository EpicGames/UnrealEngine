// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/IO/PCGLoadAssetElement.h"

#include "PCGModule.h"
#include "PCGData.h"
#include "PCGParamData.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#if WITH_EDITOR
#include "Helpers/PCGDynamicTrackingHelpers.h"
#endif // WITH_EDITOR

#include "Algo/Transform.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/ObjectSaveContext.h"

#if WITH_EDITOR
#include "UObject/UObjectIterator.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGLoadAssetElement)

#define LOCTEXT_NAMESPACE "PCGLoadDataAssetElement"

namespace PCGLoadDataAsset
{
	const FName DefaultProviderPinLabel = TEXT("DefaultAttributeOverridesIn");
}

UPCGLoadDataAssetSettings::UPCGLoadDataAssetSettings()
{
	Pins = Super::OutputPinProperties();
	bTagOutputsBasedOnOutputPins = true;
}

void UPCGLoadDataAssetSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	UpdateFromData(/*bShouldLoadAsset=*/false);
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UPCGLoadDataAssetSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (Asset.IsNull() || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGLoadDataAssetSettings, Asset)))
	{
		return;
	}

	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(Asset.ToSoftObjectPath());

	OutKeysToSettings.FindOrAdd(Key).Emplace(this, /*bCulling=*/false);
}

void UPCGLoadDataAssetSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGLoadDataAssetSettings, Asset))
	{
		UpdateFromData();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

EPCGChangeType UPCGLoadDataAssetSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGLoadDataAssetSettings, bLoadFromInput) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGLoadDataAssetSettings, Asset) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGLoadDataAssetSettings, bSetDefaultAttributeOverridesFromInput))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGLoadDataAssetSettings::CreateElement() const
{
	return MakeShared<FPCGLoadDataAssetElement>();
}

TArray<FPCGPinProperties> UPCGLoadDataAssetSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	if (bLoadFromInput)
	{
		FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Param);
		InputPin.SetRequiredPin();
	}

	if (bSetDefaultAttributeOverridesFromInput)
	{
		FPCGPinProperties& DefaultsPin = PinProperties.Emplace_GetRef(PCGLoadDataAsset::DefaultProviderPinLabel, EPCGDataType::Param);
		DefaultsPin.SetNormalPin();
	}
	
	return PinProperties;
}

FString UPCGLoadDataAssetSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	if (bLoadFromInput || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGLoadDataAssetSettings, Asset)))
	{
		// If loading data from a specified input or from an overriden value, we shouldn't show the template asset name.
		return FString();
	}
	else
#endif // WITH_EDITOR
	{
		return AssetName.IsEmpty() ? Asset.ToSoftObjectPath().GetAssetName() : AssetName;
	}
}

FPCGDataTypeIdentifier UPCGLoadDataAssetSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	// Implementation notes: the output pin types don't depend on the input pin types,
	// but they can change based on the asset selected, hence why they are dynamic, but we need to return the pin type as-is.
	check(InPin);
	return InPin->Properties.AllowedTypes;
}

void UPCGLoadDataAssetSettings::SetFromAsset(const FAssetData& InAsset)
{
	// First verify that the asset if the right class, in the very unlikely possibility that the same path would now point to a different asset class.
	const UClass* AssetClass = InAsset.GetClass();
	const bool bIsRightClass = AssetClass && AssetClass->IsChildOf(UPCGDataAsset::StaticClass());

	FSoftObjectPath InAssetPath = InAsset.GetSoftObjectPath();
	if (bIsRightClass && InAssetPath == Asset.ToSoftObjectPath())
	{
		// Nothing to do
		return;
	}

	// If it is not the right class, reset.
	Asset = bIsRightClass ? TSoftObjectPtr<UPCGDataAsset>(MoveTemp(InAssetPath)) : TSoftObjectPtr<UPCGDataAsset>{};

	UpdateFromData();
#if WITH_EDITOR
	OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Structural);
#endif // WITH_EDITOR
}

void UPCGLoadDataAssetSettings::UpdateFromData(bool bShouldLoadAsset)
{
	Modify(/*bAlwaysMarkDirty=*/bShouldLoadAsset);

	if (Asset.IsNull())
	{
		Pins = Super::OutputPinProperties();
		bTagOutputsBasedOnOutputPins = true;

		AssetName = FString();
#if WITH_EDITOR
		AssetDescription = FText::GetEmpty();
		AssetColor = FLinearColor::White;
#endif // WITH_EDITOR
	}
	else
	{
		PCGDataAsset::FGetAssetDataOutput AssetRegistryData = UPCGDataAsset::GetAssetRegistryData(Asset, bShouldLoadAsset);
		
		bTagOutputsBasedOnOutputPins = false;

		// Update rest of cached data (name, tooltip, color, ...)
		AssetName = MoveTemp(AssetRegistryData.Name);
		
#if WITH_EDITOR
		// We don't have any info, so we can't update.
		if (AssetRegistryData.CachedPins.bHasCachedPins)
		{
			Pins = MoveTemp(AssetRegistryData.CachedPins.CachedPins);
		}
		
		AssetDescription = MoveTemp(AssetRegistryData.Description);
		if (AssetRegistryData.Color.IsSet())
		{
			AssetColor = MoveTemp(AssetRegistryData.Color.GetValue());
		}
#endif
	}
}

#if WITH_EDITOR
void UPCGLoadDataAssetSettings::OnPreObjectSaved(UObject* Object, FObjectPreSaveContext SaveContext)
{
	if (Object && Asset == Object)
	{
		UpdateFromData();
		OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Structural);
	}
}
#endif // WITH_EDITOR

bool FPCGLoadDataAssetElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGLoadDataAssetElement::PrepareData);

	check(InContext);
	FPCGLoadDataAssetContext* Context = static_cast<FPCGLoadDataAssetContext*>(InContext);

	const UPCGLoadDataAssetSettings* Settings = Context->GetInputSettings<UPCGLoadDataAssetSettings>();
	check(Settings);

	// Additional validation when we're asking to set default from the default pin -
	// Basically we want that the number of data on the pin is either 0, 1 or the same cardinality as the actual input.
	if (Settings->bSetDefaultAttributeOverridesFromInput)
	{
		const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
		const TArray<FPCGTaggedData> DefaultProviders = Context->InputData.GetInputsByPin(PCGLoadDataAsset::DefaultProviderPinLabel);

		if (DefaultProviders.Num() != 0 && DefaultProviders.Num() != 1 && DefaultProviders.Num() != Inputs.Num())
		{
			PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, PCGLoadDataAsset::DefaultProviderPinLabel, Context);
			return true;
		}

		if (!DefaultProviders.IsEmpty())
		{
			Algo::Transform(DefaultProviders, Context->DefaultProviders, [](const FPCGTaggedData& TaggedData) { return Cast<UPCGParamData>(TaggedData.Data); });
		}
	}
	else if(!Settings->CommaSeparatedDefaultAttributeOverrides.IsEmpty() || !Settings->DefaultAttributeOverrides.IsEmpty())
	{
		TArray<FString> OverriddenDefaultValues;
		const TArray<FString>* DefaultValues = nullptr;

		if (!Settings->CommaSeparatedDefaultAttributeOverrides.IsEmpty())
		{
			OverriddenDefaultValues = PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->CommaSeparatedDefaultAttributeOverrides);
			DefaultValues = &OverriddenDefaultValues;
		}
		else
		{
			DefaultValues = &Settings->DefaultAttributeOverrides;
		}

		check(DefaultValues);
		Context->DefaultValueTags.Reserve(DefaultValues->Num());
		Algo::Transform(*DefaultValues, Context->DefaultValueTags, [](const FString& InTagValue) { return PCG::Private::FParseTagResult(InTagValue); });
	}

	Context->bDefaultsMatchInput = true;

	return Context->InitializeAndRequestLoad(PCGPinConstants::DefaultInputLabel,
		Settings->AssetReferenceSelector,
		{ Settings->Asset.ToSoftObjectPath() },
		/*bPersistAllData=*/false,
		/*bSilenceErrorOnEmptyObjectPath*/!Settings->bWarnIfNoAsset,
		/*bSynchronousLoad=*/Settings->bSynchronousLoad);
}

bool FPCGLoadDataAssetElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataAssetElement::Execute);

	check(InContext);
	FPCGLoadDataAssetContext* Context = static_cast<FPCGLoadDataAssetContext*>(InContext);
	const UPCGLoadDataAssetSettings* Settings = Context->GetInputSettings<UPCGLoadDataAssetSettings>();
	check(Settings);

	// Early out if we have a data matching issue
	if (!Context->bDefaultsMatchInput)
	{
		return true;
	}

#if WITH_EDITOR
	FPCGDynamicTrackingHelper DynamicTracking;
	const bool bRequiresDynamicTracking = Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGLoadDataAssetSettings, Asset)) || (Settings->bLoadFromInput && !Context->PathsToObjectsAndDataIndex.IsEmpty());
	if (bRequiresDynamicTracking)
	{
		DynamicTracking.EnableAndInitialize(Context, Context->PathsToObjectsAndDataIndex.Num());
	}
#endif

	// At this point, the data should already be loaded
	for(const TTuple<FSoftObjectPath, int32, int32>& AssetPath : Context->PathsToObjectsAndDataIndex)
	{
		TSoftObjectPtr<UPCGDataAsset> Asset(AssetPath.Get<0>());
		UPCGDataAsset* AssetData = Asset.LoadSynchronous();
		if (AssetData)
		{
			const int TaggedDataOffset = Context->OutputData.TaggedData.Num();
			Context->OutputData.TaggedData.Append(AssetData->Data.TaggedData);

			// Apply defaults if needed
			if (!Context->DefaultProviders.IsEmpty())
			{
				// Get matching param data based on input index (AssetPath.Get<1>())
				const int32 InputIndex = AssetPath.Get<1>();
				TObjectPtr<const UPCGParamData> DefaultProvider = Context->DefaultProviders[InputIndex % Context->DefaultProviders.Num()];
				const UPCGMetadata* DefaultMetadata = DefaultProvider ? DefaultProvider->ConstMetadata() : nullptr;

				// Skip applying defaults if the provider isn't valid or has no entries.
				if (DefaultProvider && DefaultMetadata && DefaultMetadata->GetItemCountForChild() > 0)
				{
					TArray<FPCGAttributeIdentifier> DefaultAttributesNames;
					TArray<EPCGMetadataTypes> DefaultAttributesTypes;
					DefaultMetadata->GetAllAttributes(DefaultAttributesNames, DefaultAttributesTypes);

					for (int TaggedDataIndex = TaggedDataOffset; TaggedDataIndex < Context->OutputData.TaggedData.Num(); ++TaggedDataIndex)
					{
						const UPCGData* OriginalData = Context->OutputData.TaggedData[TaggedDataIndex].Data.Get();
						if (!OriginalData)
						{
							continue;
						}

						UPCGData* DuplicateData = OriginalData->DuplicateData(Context);
						UPCGMetadata* Metadata = DuplicateData->MutableMetadata();

						Context->OutputData.TaggedData[TaggedDataIndex].Data = DuplicateData;

						for (const FPCGAttributeIdentifier& DefaultAttributeName : DefaultAttributesNames)
						{
							const FPCGMetadataAttributeBase* DefaultAttribute = DefaultMetadata->GetConstAttribute(DefaultAttributeName);
							check(DefaultAttribute);

							FPCGMetadataAttributeBase* Attribute = Metadata->GetMutableAttribute(DefaultAttributeName);
							if (!Attribute)
							{
								// This already sets up the default value, so we will have nothing else to do.
								Metadata->AddAttribute(DefaultMetadata, DefaultAttributeName);
								Attribute = Metadata->GetMutableAttribute(DefaultAttributeName);
							}
							else
							{
								if (!PCG::Private::IsBroadcastableOrConstructible(DefaultAttribute->GetTypeId(), Attribute->GetTypeId()))
								{
									PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("DefaultTypeDoesNotMatchAssetType", "Default value '{0}' does not have a compatible type ('{1}') to its original type ('{2}') in the asset. Will be skipped."), FText::FromName(DefaultAttributeName.Name), PCG::Private::GetTypeNameText(DefaultAttribute->GetTypeId()), PCG::Private::GetTypeNameText(Attribute->GetTypeId())), Context);
									continue;
								}

								// Create accessor on mutable metadata for the current attribute
								TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(Attribute, Metadata);
								auto SetDefaultValueOperation = [&Accessor, DefaultMetadata, DefaultAttribute](auto Dummy)
								{
									using AttributeType = decltype(Dummy);
									const FPCGMetadataAttribute<AttributeType>* TypedDefaultAttribute = static_cast<const FPCGMetadataAttribute<AttributeType>*>(DefaultAttribute);
									const AttributeType DefaultValue = TypedDefaultAttribute->GetValueFromItemKey(0); // first key should always be zero.
									FPCGAttributeAccessorKeysEntries DefaultEntry(PCGInvalidEntryKey);

									Accessor->Set(DefaultValue, DefaultEntry, EPCGAttributeAccessorFlags::AllowSetDefaultValue | EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
								};

								PCGMetadataAttribute::CallbackWithRightType(DefaultAttribute->GetTypeId(), SetDefaultValueOperation);
							}
						}
					}
				}
			}
			// Otherwise, normal Tag:Value case
			else if (!Context->DefaultValueTags.IsEmpty())
			{
				for (int TaggedDataIndex = TaggedDataOffset; TaggedDataIndex < Context->OutputData.TaggedData.Num(); ++TaggedDataIndex)
				{
					const UPCGData* OriginalData = Context->OutputData.TaggedData[TaggedDataIndex].Data.Get();
					if (!OriginalData)
					{
						continue;
					}

					UPCGData* DuplicateData = OriginalData->DuplicateData(Context);
					UPCGMetadata* Metadata = DuplicateData->MutableMetadata();

					Context->OutputData.TaggedData[TaggedDataIndex].Data = DuplicateData;

					for (const PCG::Private::FParseTagResult& TagData : Context->DefaultValueTags)
					{
						if (!PCG::Private::SetAttributeFromTag(TagData, Metadata, PCGInvalidEntryKey, PCG::Private::ESetAttributeFromTagFlags::CreateAttribute | PCG::Private::ESetAttributeFromTagFlags::SetDefaultValue))
						{
							PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("TagFailedToSetDefaultValue", "Default Tag Value '{0}' failed to set its value to the target asset data, most likely due to type mismatch."), FText::FromString(TagData.GetOriginalAttribute())), Context);
						}
					}
				}
			}

			if (Settings->bTagOutputsBasedOnOutputPins || Settings->InputIndexTag != NAME_None || Settings->DataIndexTag != NAME_None)
			{
				for (int TaggedDataIndex = TaggedDataOffset; TaggedDataIndex < Context->OutputData.TaggedData.Num(); ++TaggedDataIndex)
				{
					FPCGTaggedData& TaggedData = Context->OutputData.TaggedData[TaggedDataIndex];

					if (Settings->bTagOutputsBasedOnOutputPins && TaggedData.Pin != NAME_None)
					{
						TaggedData.Tags.Add(TaggedData.Pin.ToString());
					}

					if (Settings->InputIndexTag != NAME_None)
					{
						TaggedData.Tags.Add(FString::Format(TEXT("{0}:{1}"), { Settings->InputIndexTag.ToString(), AssetPath.Get<1>() }));
					}

					if (Settings->DataIndexTag != NAME_None)
					{
						TaggedData.Tags.Add(FString::Format(TEXT("{0}:{1}"), { Settings->DataIndexTag.ToString(), AssetPath.Get<2>() }));
					}
				}
			}

#if WITH_EDITOR
			if (bRequiresDynamicTracking)
			{
				DynamicTracking.AddToTracking(FPCGSelectionKey::CreateFromPath(AssetPath.Get<0>()), /*bIsCulled=*/false);
			}
#endif
		}
	}

#if WITH_EDITOR
	if (bRequiresDynamicTracking)
	{
		DynamicTracking.Finalize(Context);
	}
#endif

	return true;
}

#if WITH_EDITOR
FPCGLoadDataAssetChangeTracker::FPCGLoadDataAssetChangeTracker()
{
	Handle = FCoreUObjectDelegates::OnObjectPreSave.AddStatic(&FPCGLoadDataAssetChangeTracker::OnPreObjectSaved);
}

FPCGLoadDataAssetChangeTracker::~FPCGLoadDataAssetChangeTracker()
{
	FCoreUObjectDelegates::OnObjectPreSave.Remove(Handle);
}

void FPCGLoadDataAssetChangeTracker::OnPreObjectSaved(UObject* InObject, FObjectPreSaveContext InSaveContext)
{
	if (InObject->IsA<UPCGDataAsset>())
	{
		for (TObjectIterator<UPCGLoadDataAssetSettings> It; It; ++It)
		{
			It->OnPreObjectSaved(InObject, InSaveContext);
		}
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
