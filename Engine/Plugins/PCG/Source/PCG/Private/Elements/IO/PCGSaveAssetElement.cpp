// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/IO/PCGSaveAssetElement.h"

#include "PCGAssetExporterUtils.h"
#include "PCGModule.h"
#include "PCGParamData.h"

#include "AssetRegistry/AssetData.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSaveAssetElement)

bool UPCGDataCollectionExporter::ExportAsset(const FString& PackageName, UPCGDataAsset* Asset)
{
	if (!ShouldSave(Asset))
	{
		return false;
	}

	// Relies on default behavior to duplicate if needed
	Asset->Data = Data;
#if WITH_EDITOR
	Asset->Description = FText::FromString(AssetDescription);
	Asset->Color = AssetColor;
#endif
	return true;
}

UPackage* UPCGDataCollectionExporter::UpdateAsset(const FAssetData& PCGAsset)
{
	return nullptr;
}

bool UPCGDataCollectionExporter::ShouldSave(const UPCGDataAsset* Asset)
{
	Data.ComputeCrcs(/*bFullDataCrc=*/true);
	
#if WITH_EDITOR
	return Data.DataCrcs != Asset->Data.DataCrcs || AssetColor != Asset->Color || AssetDescription != Asset->Description.ToString();
#else
	return Data.DataCrcs != Asset->Data.DataCrcs;
#endif // WITH_EDITOR
}

UPCGSaveDataAssetSettings::UPCGSaveDataAssetSettings()
{
	Pins = Super::InputPinProperties();
}

TArray<FPCGPinProperties> UPCGSaveDataAssetSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(TEXT("AssetPath"), EPCGDataType::Param, false, false);
	return Properties;
}

FPCGElementPtr UPCGSaveDataAssetSettings::CreateElement() const
{
	return MakeShared<FPCGSaveDataAssetElement>();
}

bool FPCGSaveDataAssetElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSaveDataAssetElement::Execute);
	check(Context);
	const UPCGSaveDataAssetSettings* Settings = Context->GetInputSettings<UPCGSaveDataAssetSettings>();
	check(Settings);

	UPCGDataCollectionExporter* Exporter = nullptr;

	if (Settings->CustomDataCollectionExporterClass)
	{
		Exporter = NewObject<UPCGDataCollectionExporter>(GetTransientPackage(), Settings->CustomDataCollectionExporterClass);
	}
	
	if (!Exporter)
	{
		Exporter = NewObject<UPCGDataCollectionExporter>();
	}

	TArray<FPCGPinProperties> InputPins = Settings->InputPinProperties();

	check(Exporter);
	// Implementation note: we can't simply copy the input wholesale because this will also gather overrides if any.
	for (const FPCGTaggedData& TaggedData : Context->InputData.TaggedData)
	{
		// Accept data if its pin matches any of the actual input pins
		if (InputPins.FindByPredicate([&TaggedData](const FPCGPinProperties& Pin) { return TaggedData.Pin == Pin.Label; }))
		{
			Exporter->Data.TaggedData.Add(TaggedData);
		}
	}

#if WITH_EDITOR
	Exporter->AssetDescription = Settings->AssetDescription;
	Exporter->AssetColor = Settings->AssetColor;
#endif

#if WITH_EDITOR
	FScopedTransaction Transaction(NSLOCTEXT("PCGSaveDataAssetSettings", "ExportingAsset", "Exporting data asset from PCG"), Context->ExecutionSource.Get() && Context->ExecutionSource->GetExecutionState().UseTransactions());
#endif
	UPackage* OutPackage = UPCGAssetExporterUtils::CreateAsset(Exporter, Settings->Params, Context);
	UObject* OutObject = OutPackage ? OutPackage->FindAssetInPackage() : nullptr;

	if (!OutPackage || !OutObject)
	{
		return true;
	}

	UPCGParamData* OutParamData = NewObject<UPCGParamData>();
	OutParamData->Metadata->CreateAttribute<FSoftObjectPath>(TEXT("AssetPath"), FSoftObjectPath(OutObject), false, false);
	OutParamData->Metadata->AddEntry();

	Context->OutputData.TaggedData.Emplace_GetRef().Data = OutParamData;

	return true;
}