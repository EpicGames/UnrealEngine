// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/IO/PCGSaveDataViewElement.h"

#include "Data/DataView/PCGDataViewData.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"

#if WITH_EDITOR
#include "DesktopPlatformModule.h"
#include "IContentBrowserSingleton.h"
#include "ScopedTransaction.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSaveDataViewElement)

#define LOCTEXT_NAMESPACE "PCGSaveDataViewElement"

UPCGSaveDataViewSettings::UPCGSaveDataViewSettings()
{
	ConverterParameters.InitializeAs(FPCGDataViewJsonParameters::StaticStruct());
}

#if WITH_EDITOR
EPCGChangeType UPCGSaveDataViewSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSaveDataViewSettings, ConverterClass))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}

void UPCGSaveDataViewSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGSaveDataViewSettings, ConverterClass))
	{
		if (const UPCGDataViewConverterBase* ConverterCDO = ConverterClass ? ConverterClass->GetDefaultObject<UPCGDataViewConverterBase>() : nullptr)
		{
			ConverterParameters.InitializeAs(ConverterCDO->GetParameterStruct());
		}
		else
		{
			ConverterParameters.Reset();
		}
	}
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGSaveDataViewSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	// @todo_pcg: Allow iterating on inputs and optionally save paths as inputs N:N.
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, FPCGDataTypeInfoDataView::AsId(), /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false).SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSaveDataViewSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DepPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any);
#if WITH_EDITOR
	DepPin.Tooltip = PCGPinConstants::Tooltips::ExecutionDependencyTooltip;
#endif // WITH_EDITOR
	DepPin.Usage = EPCGPinUsage::DependencyOnly;

	return PinProperties;
}

FPCGElementPtr UPCGSaveDataViewSettings::CreateElement() const
{
	return MakeShared<FPCGSaveDataViewElement>();
}

#if WITH_EDITOR
TArray<FPCGSettingsOverridableParam> UPCGSaveDataViewSettings::GatherOverridableParams() const
{
	TArray<FPCGSettingsOverridableParam> OverridableParams = Super::GatherOverridableParams();

	if (ConverterParameters.IsValid())
	{
		const PCGSettingsHelpers::FPCGGetAllOverridableParamsConfig Config;
		OverridableParams.Append(PCGSettingsHelpers::GetAllOverridableParams(ConverterParameters.GetScriptStruct(), Config));
	}

	return OverridableParams;
}
#endif // WITH_EDITOR

void UPCGSaveDataViewSettings::FixingOverridableParamPropertyClass(FPCGSettingsOverridableParam& Param) const
{
	bool bFound = false;

	if (!Param.PropertiesNames.IsEmpty())
	{
		const UScriptStruct* ScriptStruct = ConverterParameters.GetScriptStruct();
		if (ScriptStruct && ScriptStruct->FindPropertyByName(Param.PropertiesNames[0]))
		{
			Param.PropertyClass = ScriptStruct;
			bFound = true;
		}
	}

	if (!bFound)
	{
		Super::FixingOverridableParamPropertyClass(Param);
	}
}

FStructView FPCGSaveDataViewExecutionContext::GetExternalStructContainerForOverridableParam(const FPCGSettingsOverridableParam& InParam)
{
	UPCGSaveDataViewSettings* Settings = GetMutableInputSettings<UPCGSaveDataViewSettings>();
	if (Settings && Settings->ConverterParameters.IsValid() && !InParam.PropertiesNames.IsEmpty() && Settings->ConverterParameters.GetScriptStruct()->FindPropertyByName(InParam.PropertiesNames[0]))
	{
		return Settings->ConverterParameters;
	}
	else
	{
		return nullptr;
	}
}

FPCGContext* FPCGSaveDataViewElement::CreateContext()
{
	return new FPCGSaveDataViewExecutionContext();
}

bool FPCGSaveDataViewElement::ExecuteInternal(FPCGContext* InContext) const
{
	using namespace PCG::IO;
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSaveDataViewElement::Execute);

	// Since this generates data on disk, only allow execution on editor approved platforms.
#if PLATFORM_WINDOWS || PLATFORM_LINUX || PLATFORM_MAC
	const UPCGSaveDataViewSettings* Settings = InContext->GetInputSettings<UPCGSaveDataViewSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	if (Inputs.IsEmpty())
	{
		return true;
	}

	// @todo_pcg: Allow iterating on inputs and optionally save paths as inputs N:N. For now, just one input.
	if (Inputs.Num() > 1)
	{
		PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGPinConstants::DefaultInputLabel, InContext);
	}

	const FPCGTaggedData& Input = Inputs[0];

	const UPCGDataViewData* DataViewData = Cast<UPCGDataViewData>(Input.Data);
	if (!DataViewData)
	{
		PCGLog::InputOutput::LogInvalidInputDataError(InContext);
		return true;
	}

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("SavingDataView", "Saving Data View from PCG"), InContext->ExecutionSource.Get() && InContext->ExecutionSource->GetExecutionState().UseTransactions());
#endif

	bool bOverwriteFile = Settings->bOverwriteExistingFile;

	FString FilePath = Settings->FilePath.FilePath;
#if WITH_EDITOR
	if (Settings->FilePath.FilePath.IsEmpty() && Settings->bOpenDialogueOnEmptyPath)
	{
		TArray<FString> OutFiles;
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (DesktopPlatform->SaveFileDialog(
			nullptr,
			LOCTEXT("OpenFileDialogue", "Choose output file path").ToString(),
			FPaths::GameUserDeveloperDir(),
			Settings->FilePath.FilePath.IsEmpty() ? FPaths::SetExtension(InContext->GetTaskName(), Json::Constants::Extension) : Settings->FilePath.FilePath,
			Json::Constants::Extension,
			EFileDialogFlags::None,
			OutFiles))
		{
			check(!OutFiles.IsEmpty())
			FilePath = OutFiles[0];
			// Explicit decision to choose this file. Overwrite is now okay.
			bOverwriteFile = true;
		}
	}
#endif // WITH_EDITOR

	if (FilePath.IsEmpty())
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidFilePathError", "Invalid or empty export file path: {0}"), FilePath.IsEmpty() ? LOCTEXT("Empty", "Empty") : FText::FromString(FilePath)), InContext);
		return true;
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const bool bFileExists = PlatformFile.FileExists(*FilePath);

	if (!bOverwriteFile && bFileExists)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NoOverwriteFile", "File would be overwritten. Enable 'Overwrite Existing File' to bypass this check."));
		return true;
	}

	if (bFileExists && PlatformFile.IsReadOnly(*FilePath))
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ReadOnlyFile", "Cannot save to read-only file '{0}'. If it is under source control, check it out (or clear the read-only flag) before saving."), FText::FromString(FilePath)), InContext);
		return true;
	}

	const FPCGDataView DataView = DataViewData->GetDataView();
	if (!DataView.IsValid())
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("InvalidDataView", "Data View selection is empty or internal data is no longer invalid."));
		return true;
	}

	if (Settings->ConverterClass)
	{
		const UPCGDataViewConverterBase* ConverterCDO = Settings->ConverterClass->GetDefaultObject<UPCGDataViewConverterBase>();
		TValueOrError<void, FText> Result = ConverterCDO->SerializeToFile(DataView, Settings->ConverterParameters, FilePath);
		if (Result.HasError())
		{
			PCGLog::LogErrorOnGraph(Result.GetError(), InContext);
			return true;
		}
	}

#else  // PLATFORM_WINDOWS || PLATFORM_LINUX || PLATFORM_MAC - This is not Windows, Linux, or Mac
	UE_LOGF(LogPCG, Verbose, "The 'Save Data View' node has been disabled on this platform.");
#endif // PLATFORM_WINDOWS || PLATFORM_LINUX || PLATFORM_MAC
	return true;
}

#undef LOCTEXT_NAMESPACE
