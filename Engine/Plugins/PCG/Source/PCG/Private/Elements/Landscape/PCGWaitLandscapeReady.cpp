// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Landscape/PCGWaitLandscapeReady.h"

#include "PCGSubsystem.h"
#include "Elements/PCGActorSelector.h"

#include "Landscape.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGWaitLandscapeReady)

TArray<FPCGPinProperties> UPCGWaitLandscapeReadySettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any);
	InputPinProperty.SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGWaitLandscapeReadySettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGWaitLandscapeReadySettings::CreateElement() const
{
	return MakeShared<FPCGWaitLandscapeReadyElement>();
}

bool FPCGWaitLandscapeReadyElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWaitLandscapeReadyElement::Execute);
	check(InContext);
	FPCGWaitLandscapeReadyElementContext* Context = static_cast<FPCGWaitLandscapeReadyElementContext*>(InContext);
	const UPCGWaitLandscapeReadySettings* Settings = Context->GetInputSettings<UPCGWaitLandscapeReadySettings>();
	check(Settings);

	if (!Context->bLandscapeQueryDone)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWaitLandscapeReadyElement::Execute::LandscapeQuery);
		check(Context->Landscapes.IsEmpty());
		Context->bLandscapeQueryDone = true;

		// Get landscape(s)
		FPCGActorSelectorSettings ActorSelector;
		ActorSelector.ActorFilter = EPCGActorFilter::AllWorldActors;
		ActorSelector.ActorSelection = EPCGActorSelection::ByClass;
		ActorSelector.ActorSelectionClass = ALandscapeProxy::StaticClass();
		ActorSelector.bSelectMultiple = true;

		auto BoundsCheck = [](const AActor*) -> bool { return true; };
		auto SelfIgnoreCheck = [](const AActor*) -> bool { return true; };

		TArray<AActor*> FoundLandscapeProxies = PCGActorSelector::FindActors(ActorSelector, Context->ExecutionSource.Get(), BoundsCheck, SelfIgnoreCheck);

		for (AActor* Proxy : FoundLandscapeProxies)
		{
			if (ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(Proxy))
			{
				TWeakObjectPtr<ALandscape> Landscape(LandscapeProxy->GetLandscapeActor());
				Context->Landscapes.AddUnique(Landscape);
			}
		}
	}

	if (!Context->bLandscapeReady)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWaitLandscapeReadyElement::Execute::CheckingIfLandscapeAreReady);
		check(Context->bLandscapeQueryDone);

		bool bAllLandscapesAreReady = true;
		for (const TWeakObjectPtr<ALandscape>& Landscape : Context->Landscapes)
		{
			if (Landscape.IsValid() && !Landscape->IsUpToDate())
			{
				bAllLandscapesAreReady = false;
				break;
			}
		}

		if (bAllLandscapesAreReady)
		{
			Context->bLandscapeReady = true;
		}
	}

	// We've validated the landscape(s) are ready - we`re done.
	if (Context->bLandscapeReady || Context->OutputData.bCancelExecution)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWaitLandscapeReadyElement::Execute::FinalizeExecution);
		Context->OutputData = Context->InputData;
		return true;
	}

	// At this point, we need to go to sleep for at least the remainder of the frame
	Context->bIsPaused = true;

	Context->ScheduleGeneric(FPCGScheduleGenericParams(
		[ContextHandle = Context->GetOrCreateHandle()](FPCGContext*) // Normal execution: wake up the current task
		{
			FPCGContext::FSharedContext<FPCGWaitLandscapeReadyElementContext> SharedContext(ContextHandle);
			if (FPCGWaitLandscapeReadyElementContext* ContextPtr = SharedContext.Get())
			{
				ContextPtr->bIsPaused = false;
			}
			return true;
		},
		[ContextHandle = Context->GetOrCreateHandle()](FPCGContext*) // On abort: wakeup and cancel
		{
			FPCGContext::FSharedContext<FPCGWaitLandscapeReadyElementContext> SharedContext(ContextHandle);
			if (FPCGWaitLandscapeReadyElementContext* ContextPtr = SharedContext.Get())
			{
				ContextPtr->bIsPaused = false;
				ContextPtr->OutputData.bCancelExecution = true;
			}
		},
		Context->ExecutionSource.Get(),
		{}));

	return false;
}

