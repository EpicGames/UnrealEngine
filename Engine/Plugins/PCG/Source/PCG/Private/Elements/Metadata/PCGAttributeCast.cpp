// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGAttributeCast.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeCast)

#define LOCTEXT_NAMESPACE "PCGAttributeCastElement"

TArray<FPCGPinProperties> UPCGAttributeCastSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGAttributeCastSettings::CreateElement() const
{
	return MakeShared<FPCGAttributeCastElement>();
}

#if WITH_EDITOR
TArray<FPCGPreConfiguredSettingsInfo> UPCGAttributeCastSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataTypes>({ EPCGMetadataTypes::Count, EPCGMetadataTypes::Unknown }, FTextFormat(LOCTEXT("PreconfigureInfo", "Cast to {0}")));
}
#endif

FString UPCGAttributeCastSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	const bool bIsTypeOverridden = IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGAttributeCastSettings, OutputType));
	const bool bSelectorOverridden = IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGAttributeCastSettings, InputSource)) || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGAttributeCastSettings, OutputTarget));

	if (bIsTypeOverridden)
	{
		return FString();
	}
	else
	{
		if (bSelectorOverridden)
		{
			return FText::Format(LOCTEXT("CastType", "To {0}"), PCG::Private::GetTypeNameText((uint16)OutputType)).ToString();
		}
	}
#endif

	return FText::Format(LOCTEXT("FullInfo", "{0} -> {1} as {2}"), InputSource.GetDisplayText(), OutputTarget.GetDisplayText(), PCG::Private::GetTypeNameText((uint16)OutputType)).ToString();
}

FName UPCGAttributeCastSettings::AdditionalTaskName() const
{
	return FName(*FText::Format(LOCTEXT("AdditionalTaskName", "Attribute Cast: {0}"), PCG::Private::GetTypeNameText(static_cast<uint16>(OutputType))).ToString());
}

void UPCGAttributeCastSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataTypes>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			OutputType = EPCGMetadataTypes(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

bool FPCGAttributeCastElement::ExecuteInternal(FPCGContext* Context) const
{
	check(Context);

	const UPCGAttributeCastSettings* Settings = Context->GetInputSettings<UPCGAttributeCastSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	for (int32 i = 0; i < Inputs.Num(); ++i)
	{
		const FPCGTaggedData& Input = Inputs[i];

		// Only support types that supports metadata
		if (!ensure(Input.Data) || !Input.Data->ConstMetadata())
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InvalidData", "Only support Spatial and Attribute Set, Input {0} is neither of them. Skipped"), FText::AsNumber(i)));
			continue;
		}

		FPCGTaggedData& Output = Context->OutputData.TaggedData.Add_GetRef(Input);

		UPCGData* OutputData = Input.Data->DuplicateData(Context);
		check(OutputData);

		PCGMetadataHelpers::FPCGCopyAttributeParams Params{};
		Params.SourceData = Input.Data;
		Params.TargetData = OutputData;
		Params.InputSource = Settings->InputSource;
		Params.OutputTarget = Settings->OutputTarget;
		Params.OptionalContext = Context;
		Params.bSameOrigin = true;
		Params.OutputType = Settings->OutputType;

		if (PCGMetadataHelpers::CopyAttribute(Params))
		{
			Output.Data = OutputData;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
