// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "StructUtils/PropertyBag.h"

#include "PCGCreateComplexConstantElement.generated.h"

/**
* Create a complex constant using the defined property bag
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCreateComplexConstantSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif
	
	virtual bool HasDynamicPins() const override;
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual FPCGElementPtr CreateElement() const override;
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(EditAnywhere, Category = Settings)
	FInstancedPropertyBag Properties;
	
	UPROPERTY(EditAnywhere, Category = Settings)
	FName TargetMetadataDomain;
};

/**
* Add a complex constant using the defined property bag, re-use most of UPCGCreateComplexConstantSettings
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAddComplexConstantSettings : public UPCGCreateComplexConstantSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface
};

class FPCGCreateComplexConstantElement : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return false; };
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override;
};

