// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGDataAttributesAndTags.h"

#include "PCGContext.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGTagHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataAttributesAndTags)

#define LOCTEXT_NAMESPACE "PCGDataAttributesAndTagsElement"

TArray<FPCGPinProperties> UPCGDataAttributesAndTagsSettingsBase::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any).SetRequiredPin();	
	return Properties;
}

TArray<FPCGPinProperties> UPCGDataAttributesAndTagsSettingsBase::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);	
	return Properties;
}

///////////////////////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
FName UPCGDataAttributesToTagsSettings::GetDefaultNodeName() const
{
	return FName(TEXT("DataAttributesToTags"));
}

FText UPCGDataAttributesToTagsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitleDAToTags", "Data Attributes To Tags");
}

FText UPCGDataAttributesToTagsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltipDAToTags", "Copy data attributes and their values to tags.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGDataAttributesToTagsSettings::CreateElement() const
{
	return MakeShared<FPCGDataAttributesToTagsElement>();
}

///////////////////////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
FName UPCGTagsToDataAttributesSettings::GetDefaultNodeName() const
{
	return FName(TEXT("TagsToDataAttributes"));
}

FText UPCGTagsToDataAttributesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitleTagsToDA", "Tags to Data Attributes");
}

FText UPCGTagsToDataAttributesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltipTagsToDA", "Parse tags and create data attributes from it.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGTagsToDataAttributesSettings::CreateElement() const
{
	return MakeShared<FPCGTagsToDataAttributesElement>();
}

///////////////////////////////////////////////////////////////////////////////////////////////

bool FPCGDataAttributesToTagsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataAttributesToTagsElement::Execute);

	check(InContext);

	const UPCGDataAttributesToTagsSettings* Settings = InContext->GetInputSettings<UPCGDataAttributesToTagsSettings>();
	check(Settings);
	
	TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const FPCGMetadataDomain* DataDomain = (Input.Data && Input.Data->ConstMetadata()) ? Input.Data->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data) : nullptr;
		if (!Input.Data)
		{
			continue;
		}

		if (!DataDomain)
		{
			// No data domain, just forward the input
			Outputs.Emplace(Input);
			continue;
		}

		TArray<FName> AttributeNames;

		if (Settings->AttributesTagsMapping.IsEmpty())
		{
			TArray<EPCGMetadataTypes> AttributeTypes;
			DataDomain->GetAttributes(AttributeNames, AttributeTypes);
		}
		else
		{
			Algo::Transform(Settings->AttributesTagsMapping, AttributeNames, [](const auto& It) -> FName { return *It.Key; });
		}

		FPCGTaggedData& OutputTaggedData = Outputs.Emplace_GetRef(Input);
		FPCGMetadataDomain* OutputMetadata = nullptr;

		if (Settings->bDeleteInputsAfterOperation)
		{
			UPCGData* OutputData = Input.Data->DuplicateData(InContext);
			check(OutputData && OutputData->MutableMetadata());
			OutputMetadata = OutputData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data);
			check(OutputMetadata);

			OutputTaggedData.Data = OutputData;
		}

		for (const FName& AttributeName : AttributeNames)
		{
			const FPCGMetadataAttributeBase* Attribute = DataDomain->GetConstAttribute(AttributeName);
			if (!Attribute)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("AttributeNotFound", "Attribute '{0} does not exist on the data domain."), FText::FromName(AttributeName)), InContext);
				continue;
			}

			FString TagName = Attribute->Name.ToString();
			const FPCGAttributePropertyOutputSelector* OutSelector = Settings->AttributesTagsMapping.Find(TagName);

			// If we have a mapping, we only process the attributes that are in the mapping. Otherwise we process them all.
			if (!Settings->AttributesTagsMapping.IsEmpty() && !OutSelector)
			{
				continue;
			}

			if (OutSelector)
			{
				const FPCGAttributePropertyInputSelector InSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(*TagName);
				const FPCGAttributePropertyOutputSelector FixedOutSelector = OutSelector->CopyAndFixSource(&InSelector, Input.Data);
				
				TagName = FixedOutSelector.GetName().ToString();
			}

			// Returns true if it was added to the tags
			auto SetTag = [Attribute, Settings, &OutputTaggedData, &TagName]<typename T>(T) -> bool
			{
				if constexpr (!PCG::Private::IsOfTypes<T, float, double, int32, int64, bool, FName, FString>())
				{
					if (!Settings->bDiscardNonParseableAttributeTypes)
					{
						OutputTaggedData.Tags.Add(std::move(TagName));
						return true;
					}
					else
					{
						return false;
					}
				}
				else
				{
					if (Settings->bDiscardAttributeValue)
					{
						OutputTaggedData.Tags.Add(std::move(TagName));
					}
					else
					{
						const T Value = static_cast<const FPCGMetadataAttribute<T>*>(Attribute)->GetValueFromItemKey(PCGFirstEntryKey);

						if constexpr (PCG::Private::IsOfTypes<T, FString, FName>())
						{
							if (Settings->bTreatStringAttributesAsTags)
							{
								FString StringValue = PCG::Private::MetadataTraits<T>::ToString(Value);
								const TArray<FString> TagAndValues = PCGHelpers::GetStringArrayFromCommaSeparatedList(StringValue);

								for (const FString& TagAndValue : TagAndValues)
								{
									OutputTaggedData.Tags.Add(TagAndValue);
								}

								return true;
							}
						}

						// Otherwise/default case
						OutputTaggedData.Tags.Add(FString::Format(TEXT("{0}:{1}"), {std::move(TagName), PCG::Private::MetadataTraits<T>::ToString(Value)}));
					}
					
					return true;
				}
			};
			
			const bool bWasAdded = PCGMetadataAttribute::CallbackWithRightType(Attribute->GetTypeId(), SetTag);

			if (bWasAdded && Settings->bDeleteInputsAfterOperation)
			{
				check(OutputMetadata);
				OutputMetadata->DeleteAttribute(Attribute->Name);
			}
		}
	}
	
	return true;
}

///////////////////////////////////////////////////////////////////////////////////////////////

bool FPCGTagsToDataAttributesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTagsToDataAttributesElement::Execute);

	check(InContext);

	const UPCGTagsToDataAttributesSettings* Settings = InContext->GetInputSettings<UPCGTagsToDataAttributesSettings>();
	check(Settings);
	
	TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		if (!Input.Data)
		{
			continue;
		}

		if (!Input.Data->IsSupportedMetadataDomainID(PCGMetadataDomainID::Data))
		{
			// No data domain, just forward the input
			Outputs.Emplace(Input);
			continue;
		}
		
		// Parse the tags first. Need to keep a mapping between original tags and their parsed result, if we need to remove the tag at the end.
		TMap<const FString*, PCG::Private::FParseTagResult> ParsedTags;
		ParsedTags.Reserve(Input.Tags.Num());
		Algo::Transform(Input.Tags, ParsedTags, [](const FString& Tag) { return MakeTuple(&Tag, PCG::Private::ParseTag(Tag));});
		
		FPCGTaggedData& OutputTaggedData = Outputs.Emplace_GetRef(Input);
		UPCGData* OutputData = Input.Data->DuplicateData(InContext);
		check(OutputData && OutputData->MutableMetadata());
		FPCGMetadataDomain* OutputMetadata = OutputData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data);
		check(OutputMetadata);

		if (OutputMetadata->GetItemCountForChild() == 0)
		{
			OutputMetadata->AddEntry();
		}

		OutputTaggedData.Data = OutputData;

		for (const auto& It : ParsedTags)
		{
			const PCG::Private::FParseTagResult& ParsedResult = It.Value;
			const FString& Tag = ParsedResult.GetOriginalAttribute();
			const FPCGAttributePropertyOutputSelector* OutSelector = Settings->AttributesTagsMapping.Find(Tag);

			// If we have a mapping, we only process the tags that are in the mapping. Otherwise we process them all.
			if (!Settings->AttributesTagsMapping.IsEmpty() && !OutSelector)
			{
				continue;
			}
			
			FName AttributeName = *ParsedResult.Attribute;

			if (OutSelector)
			{
				const FPCGAttributePropertyInputSelector InSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(AttributeName);
				const FPCGAttributePropertyOutputSelector FixedOutSelector = OutSelector->CopyAndFixSource(&InSelector, Input.Data);
				
				AttributeName = FixedOutSelector.GetName();
			}

			if (!PCG::Private::SetAttributeFromTag(ParsedResult, OutputMetadata, PCGFirstEntryKey, PCG::Private::ESetAttributeFromTagFlags::CreateAttribute, AttributeName))
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidTag", "Failed to parse tag {0}"), FText::FromString(Tag)), InContext);
				continue;
			}

			if (Settings->bDeleteInputsAfterOperation)
			{
				check(It.Key);
				OutputTaggedData.Tags.Remove(*It.Key);
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
