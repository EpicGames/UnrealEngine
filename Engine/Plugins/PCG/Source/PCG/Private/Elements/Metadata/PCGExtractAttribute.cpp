// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGExtractAttribute.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Utils/PCGLogErrors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGExtractAttribute)

#define LOCTEXT_NAMESPACE "PCGExtractAttributeElement"

void PCGExtractAttribute::ExtractAttribute(const PCGExtractAttribute::FExtractAttributeParams& Params)
{
	check(Params.Context && Params.InputSource && Params.OutputAttributeName)
	TArray<FPCGTaggedData> Inputs = Params.Context->InputData.GetInputsByPin(Params.InputLabel);

	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		const FPCGTaggedData& Input = Inputs[InputIndex];

		if (!Input.Data || (Params.OptionalClassRequirement.IsSet() && *Params.OptionalClassRequirement && !Input.Data->IsA(*Params.OptionalClassRequirement)))
		{
			PCGLog::InputOutput::LogInvalidInputDataError(Params.Context);
			continue;
		}

		TArray<FPCGTaggedData>& Outputs = Params.Context->OutputData.TaggedData;

		const FPCGAttributePropertyInputSelector InputSource = Params.InputSource->CopyAndFixLast(Input.Data);
		FPCGAttributePropertyOutputSelector OutputTarget = Params.OutputAttributeName->CopyAndFixSource(&InputSource);
		// Make sure the OutputTarget is an attribute, and also discard the extra names as this is a new attribute
		OutputTarget.SetAttributeName(OutputTarget.GetName(), /*bResetExtraNames=*/true);

		if (Params.bForceOutputAttributeToBeInElementsDomain)
		{
			OutputTarget.SetDomainName(NAME_None);
		}

		TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, InputSource);
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, InputSource);

		if (!Accessor || !Keys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(InputSource, Params.Context);
			continue;
		}

		const int32 Index = Params.Index;

		if (Index < 0 || Index >= Keys->GetNum())
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("IndexOutOfBounds", "Index for input {0} is out of bounds. Index: {1}; Number of Elements: {2}"), InputIndex, Index, Keys->GetNum()), Params.Context);
			continue;
		}
		
		UPCGParamData* OutputParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Params.Context);
		FPCGMetadataDomain* Domain = OutputParamData->MutableMetadata()->GetMetadataDomainFromSelector(OutputTarget);
		
		if (!Domain)
		{
			PCGLog::Metadata::LogInvalidMetadataDomain(OutputTarget);
			continue;
		}
		
		Domain->AddEntry();

		PCGAttributeAccessorHelpers::FPCGCreateAccessorWithAttributeCreationParams CreationParams =
		{
			.InData = OutputParamData,
			.InSelector = &OutputTarget,
			.InExpectedDesc = Accessor->GetUnderlyingDesc()
		};

		TUniquePtr<IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateAccessorWithAttributeCreation(CreationParams);
		FPCGAttributeAccessorKeysEntries AccessorKeys(PCGInvalidEntryKey);

		if (!AttributeAccessor)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(OutputTarget);
			continue;
		}

		if (Accessor->CopyTo(*Keys, *AttributeAccessor, AccessorKeys, /*InputIndex=*/Index, /*OutputIndex=*/0, /*Count=*/1, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible | EPCGAttributeAccessorFlags::AllowSetDefaultValue))
		{
			FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
			Output.Data = OutputParamData;
			Output.Pin = Params.OutputLabel;

			if (Params.OnSuccessExtractionCallback.IsSet())
			{
				Params.OnSuccessExtractionCallback(Input);
			}
		}
	}
}

UPCGExtractAttributeSettings::UPCGExtractAttributeSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		bForceOutputAttributeToBeInElementsDomain = false;
	}
}

#if WITH_EDITOR
FName UPCGExtractAttributeSettings::GetDefaultNodeName() const
{
	return FName(TEXT("ExtractAttributeAtIndex"));
}

FText UPCGExtractAttributeSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Extract Attribute at Index");
}

FText UPCGExtractAttributeSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Extract an attribute at a given index into a new attribute set.\n"
		"Support any domain. Index needs to be in range of valid indexes for the given domain.");
}
#endif // WITH_EDITOR

FString UPCGExtractAttributeSettings::GetAdditionalTitleInformation() const
{
	return FString::Printf(TEXT("%s -> %s"), *InputSource.ToString(), *OutputAttributeName.ToString());
}

FPCGElementPtr UPCGExtractAttributeSettings::CreateElement() const
{
	return MakeShared<FPCGExtractAttributeElement>();
}

TArray<FPCGPinProperties> UPCGExtractAttributeSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any).SetRequiredPin();
	return Properties;
}

TArray<FPCGPinProperties> UPCGExtractAttributeSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);
	return Properties;
}

bool FPCGExtractAttributeElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGExtractAttributeElement::Execute);

	check(Context);

	const UPCGExtractAttributeSettings* Settings = Context->GetInputSettings<UPCGExtractAttributeSettings>();
	check(Settings);

	PCGExtractAttribute::FExtractAttributeParams Params =
	{
		.Context = Context,
		.InputSource = &Settings->InputSource,
		.Index = Settings->Index,
		.OutputAttributeName = &Settings->OutputAttributeName,
		.bForceOutputAttributeToBeInElementsDomain = Settings->bForceOutputAttributeToBeInElementsDomain
	};

	PCGExtractAttribute::ExtractAttribute(Params);

	return true;
}

#undef LOCTEXT_NAMESPACE
