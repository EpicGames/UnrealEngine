// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGExtractMemberFromStruct.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Utils/PCGLogErrors.h"

#include "Algo/AnyOf.h"
#include "Helpers/PCGSettingsHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGExtractMemberFromStruct)

#define LOCTEXT_NAMESPACE "PCGExtractMemberFromStructElement"

void PCGExtractMemberFromStruct::ExtractMemberFromStruct(const PCGExtractMemberFromStruct::FExtractMemberFromStructParams& Params)
{
	check(Params.Context && Params.InputSource && Params.OutputAttributeName)
	TArray<FPCGTaggedData> Inputs = Params.Context->InputData.GetInputsByPin(Params.InputLabel);

	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		const FPCGTaggedData& Input = Inputs[InputIndex];

		if (!Input.Data || (Params.OptionalClassRequirement.IsSet() && *Params.OptionalClassRequirement && !Input.Data->IsA(*Params.OptionalClassRequirement)))
		{
			PCGLog::InputOutput::LogInvalidInputDataError(Params.Context);
			continue;
		}

		TArray<FPCGTaggedData>& Outputs = Params.Context->OutputData.TaggedData;

		const FPCGAttributePropertyInputSelector InputSource = Params.InputSource->CopyAndFixLast(Input.Data);
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, InputSource);

		if (!Keys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(InputSource, Params.Context);
			continue;
		}

		TUniquePtr<const IPCGAttributeAccessor> Accessor;
		TArray<const void*> StructPtrs;

		// We need to have extra names, to extract members
		if (InputSource.GetExtraNames().IsEmpty() && !Params.bExtractAll)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidExtraName", "No member specified to extract in attribute {0}."), InputSource.GetDisplayText()));
			continue;
		}

		const int32 Count = Keys->GetNum();
		const int32 StartIndex = 0;

		StructPtrs.SetNumUninitialized(Count);
		const FPCGAttributePropertySelector RootSelector = FPCGAttributePropertySelector::CreateAttributeSelector(InputSource.GetName(), InputSource.GetDomainName());
		Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, RootSelector);
		if (!Accessor)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(InputSource);
			continue;
		}

		const FPCGMetadataAttributeDesc& AccessorDesc = Accessor->GetUnderlyingDesc();

		if (!AccessorDesc.IsSingleValue() || AccessorDesc.ValueType != EPCGMetadataTypes::Struct || AccessorDesc.ValueTypeObject == nullptr)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidStruct", "Attribute '{0}' is not a valid struct"), RootSelector.GetDisplayText()));
			continue;
		}

		PCG::Private::FOutValuesByPtr OutValues =
		{
			.OutValues = StructPtrs,
			.UnderlyingDesc = &AccessorDesc
		};

		if (!Accessor->GetRange(PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByPtr>{}, MoveTemp(OutValues)}, AccessorDesc, Count, /*Index=*/0, *Keys, EPCGAttributeAccessorFlags::StrictType))
		{
			PCGLog::Metadata::LogFailToGetAttributeError(InputSource);
			continue;
		}

		Keys = MakeUnique<FPCGAttributeAccessorKeysGenericPtrs>(StructPtrs);
		TArray<FName> PropertyNames;
		TArray<const FProperty*> PropertyChain;
		PropertyNames.Reserve(InputSource.GetExtraNames().Num());
		Algo::Transform(InputSource.GetExtraNames(), PropertyNames, [](const FString& Item) { return FName{Item}; });
		if (!PCGAttributeAccessorHelpers::GetPropertyChain(PropertyNames, Cast<UScriptStruct>(AccessorDesc.ValueTypeObject), PropertyChain))
		{
			continue;
		}

		// Get Property Chain extracts an array property into 2 properties (array and inner).
		// So search for the first array property.
		// If the array property is the penultimate value, then pop the inner property (to keep the array property only)
		// If it is not, then we have an array property in the middle and this is unsupported.
		const int32 ArrayPropertyIndex = PropertyChain.IndexOfByPredicate([](const FProperty* Property) { return Property && Property->IsA<FArrayProperty>(); });
		if (ArrayPropertyIndex != INDEX_NONE)
		{
			if (ArrayPropertyIndex == PropertyChain.Num() - 2)
			{
				PropertyChain.Pop();
			}
			else
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ArrayPropertyFound", "There is an array property in the chain '{0}', this is unsupported."), InputSource.GetDisplayText()));
				continue;
			}
		}
		
		TArray<TTuple<TUniquePtr<const IPCGAttributeAccessor>, FPCGAttributePropertyOutputSelector>> PropertyAccessorsAndSelectors;
		
		// Only support extract all on struct property otherwise will go with the default flow.
		if (Params.bExtractAll && (PropertyChain.IsEmpty() || PropertyChain.Last()->IsA<FStructProperty>()))
		{
			PCGSettingsHelpers::FPCGGetAllOverridableParamsConfig Config{};
			Config.ShouldKeepPropertyFunc = [](const FProperty* InProperty, int32) -> bool { return InProperty && InProperty->HasAnyPropertyFlags(CPF_BlueprintVisible); };
			Config.MaxStructDepth = 0;
			Config.bExtractArrays = true;
			Config.bDiscardLeafStructProperty = false;
			
			const UScriptStruct* Struct = PropertyChain.IsEmpty() ? CastChecked<UScriptStruct>(AccessorDesc.ValueTypeObject) : CastFieldChecked<FStructProperty>(PropertyChain.Last())->Struct;

			TArray<FPCGSettingsOverridableParam> ExtractedProperties = PCGSettingsHelpers::GetAllOverridableParams(Struct, Config);
			
			for (const FPCGSettingsOverridableParam& ExtractedProperty : ExtractedProperties)
			{
				// Can have 2 properties if the first one is an array, but we'll still use the first property anyway.
				check(ExtractedProperty.Properties.Num() <= 2);
				const FProperty* ThisProperty = ExtractedProperty.Properties[0];
				check(ThisProperty);
				TArray<const FProperty*> ExtendedPropertyChain = PropertyChain;
				ExtendedPropertyChain.Add(ThisProperty);
				
				// First try with old accessors
				Accessor = PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(TArray<const FProperty*>(ExtendedPropertyChain), /*bUseGenericAccessor=*/false);
				if (!Accessor)
				{
					Accessor = PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(MoveTemp(ExtendedPropertyChain), /*bUseGenericAccessor=*/true);
				}
			
				if (!Accessor)
				{
					continue;
				}
			
				// Fix the output attribute name for source, to extract the data domain of the source. Then force it to the extracted property name.
				const FPCGAttributePropertyInputSelector LeafSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(ExtractedProperty.PropertiesNames[0], InputSource.GetDomainName());
				FPCGAttributePropertyOutputSelector OutputTarget = Params.OutputAttributeName->CopyAndFixSource(&LeafSelector);
				OutputTarget.SetAttributeName(ExtractedProperty.PropertiesNames[0]);
				PropertyAccessorsAndSelectors.Emplace(MoveTemp(Accessor), MoveTemp(OutputTarget));
			}
		}
		else
		{
			// First try with old accessors
			Accessor = PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(TArray<const FProperty*>(PropertyChain), /*bUseGenericAccessor=*/false);
			if (!Accessor)
			{
				Accessor = PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(MoveTemp(PropertyChain), /*bUseGenericAccessor=*/true);
			}
			
			if (!Accessor)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(InputSource, Params.Context);
				continue;
			}
			
			const FPCGAttributePropertyInputSelector LeafSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(PropertyNames.Last(), InputSource.GetDomainName());
			PropertyAccessorsAndSelectors.Emplace(MoveTemp(Accessor), Params.OutputAttributeName->CopyAndFixSource(&LeafSelector));
		}
		
		if (PropertyAccessorsAndSelectors.IsEmpty())
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("NoPropertiesFound", "No properties found to extract."));
			continue;
		}
		
		UPCGData* OutputData = Input.Data->DuplicateData(Params.Context);
		if (!OutputData)
		{
			continue;
		}

		TArray<PCGMetadataEntryKey> EntryKeys;

		FPCGMetadataDomain* Domain = OutputData->MutableMetadata()->GetMetadataDomainFromSelector(PropertyAccessorsAndSelectors[0].Get<1>());
		if (!Domain)
		{
			PCGLog::Metadata::LogInvalidMetadataDomain(PropertyAccessorsAndSelectors[0].Get<1>());
			continue;
		}
		
		if (Params.bDeleteSourceAttribute)
		{
			// Make sure we delete on the right domain 
			if (FPCGMetadataDomain* InputDomain = OutputData->MutableMetadata()->GetMetadataDomainFromSelector(InputSource))
			{
				InputDomain->DeleteAttribute(InputSource.GetAttributeName());
			}
		}
		
		for (const auto& [PropertyAccessor, OutputTarget] : PropertyAccessorsAndSelectors)
		{
			FPCGMetadataAttributeDesc OutputDesc = PropertyAccessor->GetUnderlyingDesc();
			OutputDesc.Name = OutputTarget.GetAttributeName();
		
			// Delete attribute if it exists.
			if (Domain->HasAttribute(OutputDesc.Name))
			{
				Domain->DeleteAttribute(OutputDesc.Name);
			}

			PCGAttributeAccessorHelpers::FPCGCreateAccessorWithAttributeCreationParams CreationParams =
			{
				.InData = OutputData,
				.InSelector = &OutputTarget,
				.InExpectedDesc = OutputDesc
			};

			TUniquePtr<IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateAccessorWithAttributeCreation(CreationParams);
			TUniquePtr<IPCGAttributeAccessorKeys> AccessorKeys = PCGAttributeAccessorHelpers::CreateKeys(OutputData, OutputTarget);

			if (!AttributeAccessor || !AccessorKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(OutputTarget);
				break;
			}
			
			if (!PropertyAccessor->CopyTo(*Keys, *AttributeAccessor, *AccessorKeys, /*InputIndex=*/StartIndex, /*OutputIndex=*/0, /*Count=*/Count, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("FailCopy", "Failed to copy attribute {0}."), OutputTarget.GetDisplayText()));
			}
		}
		
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
		Output.Data = OutputData;
		Output.Pin = Params.OutputLabel;

		if (Params.OnSuccessExtractionCallback.IsSet())
		{
			Params.OnSuccessExtractionCallback(Input);
		}
	}
}

#if WITH_EDITOR
FName UPCGExtractMemberFromStructSettings::GetDefaultNodeName() const
{
	return FName(TEXT("ExtractMemberFromStruct"));
}

FText UPCGExtractMemberFromStructSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Extract Member From Struct");
}

FText UPCGExtractMemberFromStructSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Extract a member (or all members) from a struct attribute into a new attribute on the input data. Support any domain.");
}
#endif // WITH_EDITOR

FString UPCGExtractMemberFromStructSettings::GetAdditionalTitleInformation() const
{
	return FString::Printf(TEXT("%s -> %s"), *InputSource.ToString(), *OutputAttributeName.ToString());
}

FPCGElementPtr UPCGExtractMemberFromStructSettings::CreateElement() const
{
	return MakeShared<FPCGExtractMemberFromStructElement>();
}

TArray<FPCGPinProperties> UPCGExtractMemberFromStructSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any).SetRequiredPin();
	return Properties;
}

TArray<FPCGPinProperties> UPCGExtractMemberFromStructSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
	return Properties;
}

bool FPCGExtractMemberFromStructElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGExtractMemberFromStructElement::Execute);

	check(Context);

	const UPCGExtractMemberFromStructSettings* Settings = Context->GetInputSettings<UPCGExtractMemberFromStructSettings>();
	check(Settings);

	PCGExtractMemberFromStruct::FExtractMemberFromStructParams Params =
	{
		.Context = Context,
		.InputSource = &Settings->InputSource,
		.bExtractAll = Settings->bExtractAll,
		.OutputAttributeName = &Settings->OutputAttributeName,
		.bDeleteSourceAttribute = Settings->bDeleteSourceAttribute
	};

	PCGExtractMemberFromStruct::ExtractMemberFromStruct(Params);

	return true;
}

#undef LOCTEXT_NAMESPACE
