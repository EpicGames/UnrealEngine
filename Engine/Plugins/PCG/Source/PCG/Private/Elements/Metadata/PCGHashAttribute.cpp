// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGHashAttribute.h"

#include "PCGParamData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGHashAttribute)

#define LOCTEXT_NAMESPACE "PCGHashAttributeElement"

FPCGAttributePropertyInputSelector UPCGHashAttributeSettings::GetInputSource(uint32 Index) const
{
	ensure(Index == 0);
	return InputSource1;
}

FPCGElementPtr UPCGHashAttributeSettings::CreateElement() const
{
	return MakeShared<FPCGHashAttributeElement>();
}

bool FPCGHashAttributeElement::DoOperation(PCGMetadataOps::FOperationData& InOperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGHashAttributeElement::Execute);

	return PCGMetadataAttribute::CallbackWithRightType(InOperationData.MostComplexInputType, [this, &InOperationData]<typename AttributeType>(AttributeType) -> bool
	{
		return DoUnaryOp<AttributeType>(InOperationData, [](const AttributeType& Value) -> int32
		{
			return static_cast<int32>(PCG::Private::MetadataTraits<AttributeType>::Hash(Value));
		});
	});
}

#undef LOCTEXT_NAMESPACE
