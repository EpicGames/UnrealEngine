// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataArrayOperation.h"

#include "PCGContext.h"
#include "PCGData.h"
#include "PCGParamData.h"
#include "Elements/Metadata/PCGMetadataOpElementBase.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#define LOCTEXT_NAMESPACE "PCGMetadataArrayOperation"

namespace PCGMetadataArrayOperation
{
	static const FTextFormat InvalidOutputTypeError = LOCTEXT("InvalidOutputType", "Property on output pin is of unsupported type {1}.");

	namespace Helpers
	{
		FName GetDefaultValuePropertyName(const int32 Index)
		{
			static const TStaticArray<FName, 3> InputDefaultValuePropertyNames = {TEXT("DefaultValue1"), TEXT("DefaultValue2"), TEXT("DefaultValue3"),};
			const bool bValidIndex = Index > INDEX_NONE && Index < InputDefaultValuePropertyNames.Num();
			return ensure(bValidIndex) ? InputDefaultValuePropertyNames[Index] : NAME_None;
		}
	}

	int32 FindIndex(const FProperty* Property, const void* DataPtr, int32 Count, const void* ValueToFind)
	{
		// Iterate runtime array elements with element-size pointer arithmetic.
		// GetValueAddressAtIndex_Direct addresses CPP-level static arrays (ArrayDim), not
		// runtime TArray contents — using it here would only ever return the i=0 address.
		const size_t ElementSize = Property->GetElementSize();
		const uint8* ArrayBytes = static_cast<const uint8*>(DataPtr);

		if (Property->HasAllPropertyFlags(CPF_HasGetValueTypeHash))
		{
			const int32 ValueHash = Property->GetValueTypeHash(ValueToFind);

			for (int32 i = 0; i < Count; ++i)
			{
				const void* ArrayValue = ArrayBytes + i * ElementSize;
				const int32 ArrayValueHash = Property->GetValueTypeHash(ArrayValue);
				if (ArrayValueHash == ValueHash && Property->Identical(ValueToFind, ArrayValue))
				{
					return i;
				}
			}
		}
		else
		{
			for (int32 i = 0; i < Count; ++i)
			{
				const void* ArrayValue = ArrayBytes + i * ElementSize;
				if (Property->Identical(ValueToFind, ArrayValue))
				{
					return i;
				}
			}
		}

		return INDEX_NONE;
	}
	
	struct FValueExtractionResult
	{
		FValueExtractionResult(FPCGAttributeProperty&& InProperty)
			: Property(MoveTemp(InProperty))
			, Buffer()
			, Helper(Property.GetProperty(), &Buffer)
		{}
		
		// Be sure to no allow to copy (as FScriptArray is not copyable)
		FValueExtractionResult(const FValueExtractionResult&) = delete;
		FValueExtractionResult& operator=(const FValueExtractionResult&) = delete;
		
		// Need to explicitly move the buffer
		FValueExtractionResult(FValueExtractionResult&& Other)
			: Property(MoveTemp(Other.Property))
			, Buffer()
			, Helper(Property.GetProperty(), &Buffer)
		{
			Helper.MoveAssign(&Other.Buffer);
		}
		
		FValueExtractionResult& operator=(FValueExtractionResult&& Other)
		{
			Helper.EmptyValues();
			Property = MoveTemp(Other.Property);
			Helper = FScriptArrayHelper(Property.GetProperty(), &Buffer);
			Helper.MoveAssign(&Other.Buffer);
			
			return *this;
		}
		
		FPCGAttributeProperty Property;
		FScriptArray Buffer;
		FScriptArrayHelper Helper;
	};
	
	struct FArrayExtractionResult
	{
		FArrayExtractionResult() = default;
		
		// Be sure to no allow to copy
		FArrayExtractionResult(const FArrayExtractionResult&) = delete;
		FArrayExtractionResult& operator=(const FArrayExtractionResult&) = delete;
		FArrayExtractionResult(FArrayExtractionResult&& Other) = default;
		FArrayExtractionResult& operator=(FArrayExtractionResult&& Other) = default;
		
		TArray<TTuple<const void*, int32>> ArrayValues;
		TArray<PCG::FPCGAccessorBuffer> Buffers;
	};
	
	TOptional<FValueExtractionResult> ExtractValues(FPCGMetadataArrayOperationIterState& IterState, int32 PinIndex, const FPCGMetadataAttributeDesc& WantedDesc)
	{
		const FPCGMetadataAttributeDesc WantedDescSingleValue = WantedDesc.ConvertToSingleValue();

		FPCGAttributeProperty Property(WantedDescSingleValue);
		if (!Property.IsValid())
		{
			const FName PinName = IterState.Settings ? IterState.Settings->GetInputPinName(PinIndex) : "Unknown";
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidInputType", "Property on pin {0} is of unsupported type {1}."), FText::FromName(PinName), WantedDesc.GetTypeText()));
			return {};
		}

		const int32 Count = IterState.NumberOfElementsToProcess;

		FValueExtractionResult Result{MoveTemp(Property)};
		Result.Helper.AddValues(Count);

		if (!IterState.InputAccessors[PinIndex]->GetRange(PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, Result.Helper.GetElementPtr(0)}, WantedDescSingleValue, Count, /*Index=*/0, *IterState.InputKeys[PinIndex], EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
		{
			PCGLog::Metadata::LogFailToGetAttributeError(IterState.InputSources[PinIndex]);
			return {};
		}
		
		return TOptional<FValueExtractionResult>{MoveTemp(Result)};
	}
	
	TOptional<FArrayExtractionResult> ExtractArray(FPCGMetadataArrayOperationIterState& IterState, int32 PinIndex = 0)
	{
		if (!IterState.WorkingType.IsArray())
		{
			return {};
		}

		FArrayExtractionResult Result{};
		Result.ArrayValues.SetNumUninitialized(IterState.NumberOfElementsToProcess);
		if (!IterState.InputAccessors[PinIndex]->GetUnderlyingDesc().IsSameType(IterState.WorkingType))
		{
			// We are extracting an array with a different type, we need to allocate buffers
			Result.Buffers.SetNum(IterState.NumberOfElementsToProcess);
		}
		
		PCG::Private::FOutValuesAsArray OutValuesAsArray =
		{
			.OutValues = Result.ArrayValues,
			.Buffers = Result.Buffers,
			.UnderlyingDesc = &IterState.WorkingType
		};
		
		if (!IterState.InputAccessors[PinIndex]->GetRange(PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, MoveTemp(OutValuesAsArray)}, IterState.WorkingType, IterState.NumberOfElementsToProcess, /*Index=*/0, *IterState.InputKeys[PinIndex], EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
		{
			PCGLog::Metadata::LogFailToGetAttributeError(IterState.InputSources[PinIndex]);
			return {};
		}
		
		return Result;
	}
	
	void WriteResult(FPCGMetadataArrayOperationIterState& IterState, TArrayView<TTuple<const void*, int32>> InValues)
	{
		IterState.OutputAccessor->SetRange(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, InValues, &IterState.WorkingType}, IterState.OutputType, IterState.NumberOfElementsToProcess, /*Index=*/0, *IterState.OutputKeys, EPCGAttributeAccessorFlags::StrictType);
	}
	
	bool FindOrContains(FPCGMetadataArrayOperationIterState& IterState, bool bContains)
	{
		// First extract values to find in second accessor
		TOptional<FValueExtractionResult> ExtractedValues = ExtractValues(IterState, 1, IterState.InputAccessors[0]->GetUnderlyingDesc());
		if (!ExtractedValues)
		{
			return true;
		}
		
		TOptional<FArrayExtractionResult> ExtractedArray = ExtractArray(IterState);
		if (!ExtractedArray)
		{
			return true;
		}
		
		// ExtractedValues->Property is built from the single-value desc, so its outer
		// FArrayProperty's Inner is the element FProperty directly.
		const FProperty* InnerProperty = ExtractedValues->Property.GetProperty()->Inner;
		check(InnerProperty);

		TArray<int32> Indices;
		TArray<bool> Contains;
		
		if (bContains)
		{
			Contains.SetNumUninitialized(IterState.NumberOfElementsToProcess);
		}
		else
		{
			Indices.SetNumUninitialized(IterState.NumberOfElementsToProcess);
		}
		
		for (int32 i = 0; i < IterState.NumberOfElementsToProcess; ++i)
		{
			const auto& [DataPtr, ArrayNum] = ExtractedArray->ArrayValues[i];
			const int32 FoundIndex = FindIndex(InnerProperty, DataPtr, ArrayNum, ExtractedValues->Helper.GetElementPtr(i));
			if (bContains)
			{
				Contains[i] = FoundIndex != INDEX_NONE;
			}
			else
			{
				Indices[i] = FoundIndex;
			}
		}
		
		if (bContains)
		{
			IterState.OutputAccessor->SetRange<bool>(Contains, /*Index=*/0, *IterState.OutputKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
		}
		else
		{
			IterState.OutputAccessor->SetRange<int32>(Indices, /*Index=*/0, *IterState.OutputKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
		}

		return true;
	}
	
	bool Length(FPCGMetadataArrayOperationIterState& IterState)
	{
		TOptional<FArrayExtractionResult> ExtractedArray = ExtractArray(IterState);
		if (!ExtractedArray)
		{
			return true;
		}
		
		TArray<int32> Length;
		Length.SetNumUninitialized(IterState.NumberOfElementsToProcess);
		
		for (int32 i = 0; i < IterState.NumberOfElementsToProcess; ++i)
		{
			Length[i] = ExtractedArray->ArrayValues[i].Get<1>();
		}
		
		IterState.OutputAccessor->SetRange<int32>(Length, /*Index=*/0, *IterState.OutputKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
		return true;
	}
	
	bool ConvertToArray(FPCGMetadataArrayOperationIterState& IterState)
	{
		TOptional<FValueExtractionResult> ExtractedValues = ExtractValues(IterState, 0, IterState.InputAccessors[0]->GetUnderlyingDesc());
		if (!ExtractedValues)
		{
			return true;
		}
		
		TArray<TTuple<const void*, int32>> ArrayValues;
		ArrayValues.SetNumUninitialized(IterState.NumberOfElementsToProcess);

		for (int32 i = 0; i < IterState.NumberOfElementsToProcess; ++i)
		{
			auto& [DataPtr, ArrayNum] = ArrayValues[i];
			DataPtr = ExtractedValues->Helper.GetElementPtr(i);
			ArrayNum = DataPtr ? 1 : 0;
		}
		
		WriteResult(IterState, ArrayValues);
		return true;
	}
	
	bool Flatten(FPCGMetadataArrayOperationIterState& IterState)
	{
		TOptional<FArrayExtractionResult> ExtractedArray = ExtractArray(IterState);
		if (!ExtractedArray)
		{
			return true;
		}

		int32 CurrentIndex = 0;
		for (const auto& [DataPtr, ArrayNum] : ExtractedArray->ArrayValues)
		{
			IterState.OutputAccessor->SetRange(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, DataPtr, ArrayNum}, IterState.OutputType, ArrayNum, /*Index=*/CurrentIndex, *IterState.OutputKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
			CurrentIndex += ArrayNum;
		}

		return true;
	}
	
	bool MakeArray(FPCGMetadataArrayOperationIterState& IterState)
	{
		TOptional<FValueExtractionResult> ExtractedValues = ExtractValues(IterState, 0, IterState.InputAccessors[0]->GetUnderlyingDesc());
		if (!ExtractedValues)
		{
			return true;
		}
		
		TTuple<const void*, int32> ArrayTuple = {ExtractedValues->Helper.GetElementPtr(), ExtractedValues->Helper.Num()};

		// MakeArray packs every input value into a single output array, so the output has 1
		// entry regardless of input count. Use Count=1 instead of going through WriteResult
		// (which writes NumberOfElementsToProcess entries).
		IterState.OutputAccessor->SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, MakeArrayView(&ArrayTuple, 1), &IterState.WorkingType},
			IterState.OutputType, /*Count=*/1, /*Index=*/0, *IterState.OutputKeys, EPCGAttributeAccessorFlags::StrictType);
		return true;
	}
	
	bool Insert(FPCGMetadataArrayOperationIterState& IterState, TOptional<int32> ForcedIndex = {})
	{
		// First extract indices in third accessor
		TArray<int32> Indices;
		if (!ForcedIndex)
		{
			Indices.SetNumUninitialized(IterState.NumberOfElementsToProcess);
		
			if (!IterState.InputAccessors[2]->GetRange<int32>(Indices, /*Index=*/0, *IterState.InputKeys[2], EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::Metadata::LogFailToGetAttributeError(IterState.InputSources[2]);
				return true;
			}
		}
		
		TOptional<FValueExtractionResult> ExtractedValues = ExtractValues(IterState, 1, IterState.InputAccessors[0]->GetUnderlyingDesc());
		if (!ExtractedValues)
		{
			return true;
		}
		
		// Finally pull the array
		TOptional<FArrayExtractionResult> ExtractedArray = ExtractArray(IterState);
		if (!ExtractedArray)
		{
			return true;
		}
		
		const FPCGAttributeProperty OutProperty(IterState.OutputAccessor->GetUnderlyingDesc());
		if (!OutProperty.IsValid())
		{
			PCGLog::LogErrorOnGraph(FText::Format(InvalidOutputTypeError, IterState.OutputAccessor->GetUnderlyingDesc().GetTypeText()));
			return true;
		}
		
		const FArrayProperty* ArrayProperty = OutProperty.GetProperty();
		const FArrayProperty* InnerArrayProperty = CastFieldChecked<FArrayProperty>(ArrayProperty->Inner);
		
		check(ArrayProperty && InnerArrayProperty && InnerArrayProperty->Inner);
		
		const int32 ElementSize = InnerArrayProperty->Inner->GetElementSize();

		// @todo_pcg Do by pointer to be more efficient for complex types.
		FScriptArray OutBuffer;
		FScriptArrayHelper OutHelper(ArrayProperty, &OutBuffer);
		
		OutHelper.EmptyAndAddValues(IterState.NumberOfElementsToProcess);
		
		for (int32 i = 0; i < IterState.NumberOfElementsToProcess; ++i)
		{
			const int32 OriginalIndex = ForcedIndex ? *ForcedIndex : Indices[i];
			int32 Index = OriginalIndex;
			auto& [DataPtr, ArrayNum] = ExtractedArray->ArrayValues[i];
			if (Index < 0)
			{
				Index = ArrayNum + Index + 1;
			}
			
			if (Index < 0 || Index > ArrayNum)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidInsertIndex", "Tried to insert at index {0} in array of size {1} at element {2}."), OriginalIndex, ArrayNum, i));
			}
			else
			{
				FScriptArrayHelper OutInnerHelper(InnerArrayProperty, OutHelper.GetElementPtr(i));
				OutInnerHelper.AddValues(ArrayNum + 1);
				// Copy the values until the index to insert
				PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(), DataPtr, Index);
				// Copy the value to insert
				PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(Index), ExtractedValues->Helper.GetElementPtr(i), 1);
				// Copy the rest
				PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(Index+1), static_cast<const uint8*>(DataPtr) + ElementSize * Index, ArrayNum - Index);
				
				DataPtr = OutInnerHelper.GetElementPtr();
				ArrayNum++;
			}
		}
		
		WriteResult(IterState, ExtractedArray->ArrayValues);
		return true;
	}
	
	bool ReplaceAtIndex(FPCGMetadataArrayOperationIterState& IterState)
	{
		// First extract indices in third accessor
		TArray<int32> Indices;
		Indices.SetNumUninitialized(IterState.NumberOfElementsToProcess);
	
		if (!IterState.InputAccessors[2]->GetRange<int32>(Indices, /*Index=*/0, *IterState.InputKeys[2], EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
		{
			PCGLog::Metadata::LogFailToGetAttributeError(IterState.InputSources[2]);
			return true;
		}
		
		TOptional<FValueExtractionResult> ExtractedValues = ExtractValues(IterState, 1, IterState.InputAccessors[0]->GetUnderlyingDesc());
		if (!ExtractedValues)
		{
			return true;
		}
		
		// Finally pull the array
		TOptional<FArrayExtractionResult> ExtractedArray = ExtractArray(IterState);
		if (!ExtractedArray)
		{
			return true;
		}
		
		const FPCGAttributeProperty OutProperty(IterState.WorkingType);
		if (!OutProperty.IsValid())
		{
			PCGLog::LogErrorOnGraph(FText::Format(InvalidOutputTypeError, IterState.OutputAccessor->GetUnderlyingDesc().GetTypeText()));
			return true;
		}
		
		const FArrayProperty* ArrayProperty = OutProperty.GetProperty();
		const FArrayProperty* InnerArrayProperty = CastFieldChecked<FArrayProperty>(ArrayProperty->Inner);
		
		check(ArrayProperty && InnerArrayProperty && InnerArrayProperty->Inner);
		
		FScriptArray OutBuffer;
		FScriptArrayHelper OutHelper(ArrayProperty, &OutBuffer);
		
		const int32 ElementSize = InnerArrayProperty->Inner->GetElementSize();
		
		OutHelper.EmptyAndAddValues(IterState.NumberOfElementsToProcess);
		
		for (int32 i = 0; i < IterState.NumberOfElementsToProcess; ++i)
		{
			const int32 OriginalIndex = Indices[i];
			int32 Index = OriginalIndex;
			auto& [DataPtr, ArrayNum] = ExtractedArray->ArrayValues[i];
			if (Index < 0)
			{
				Index = ArrayNum + Index;
			}

			if (Index < 0 || Index >= ArrayNum)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidReplaceAtIndex", "Tried to replace at index {0} in array of size {1} at element {2}."), OriginalIndex, ArrayNum, i));
			}
			else
			{
				FScriptArrayHelper OutInnerHelper(InnerArrayProperty, OutHelper.GetElementPtr(i));
				OutInnerHelper.AddValues(ArrayNum);
				// Copy the values until the index to replace
				PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(), DataPtr, Index);
				// Copy the value to replace
				PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(Index), ExtractedValues->Helper.GetElementPtr(i), 1);
				// Copy the rest
				PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(Index+1), static_cast<const uint8*>(DataPtr) + ElementSize * (Index + 1), ArrayNum - Index - 1);
				
				DataPtr = OutInnerHelper.GetElementPtr();
			}
		}
		
		WriteResult(IterState, ExtractedArray->ArrayValues);
		return true;
	}
	
	bool RemoveAt(FPCGMetadataArrayOperationIterState& IterState, TOptional<int32> ForcedIndex = {})
	{
		// First extract indices in second accessor
		TArray<int32> Indices;
		if (!ForcedIndex)
		{
			Indices.SetNumUninitialized(IterState.NumberOfElementsToProcess);
		
			if (!IterState.InputAccessors[1]->GetRange<int32>(Indices, /*Index=*/0, *IterState.InputKeys[1], EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::Metadata::LogFailToGetAttributeError(IterState.InputSources[1]);
				return true;
			}
		}
		
		// Finally pull the array
		TOptional<FArrayExtractionResult> ExtractedArray = ExtractArray(IterState);
		if (!ExtractedArray)
		{
			return true;
		}
		
		const FPCGAttributeProperty OutProperty(IterState.OutputAccessor->GetUnderlyingDesc());
		if (!OutProperty.IsValid())
		{
			PCGLog::LogErrorOnGraph(FText::Format(InvalidOutputTypeError, IterState.OutputAccessor->GetUnderlyingDesc().GetTypeText()));
			return true;
		}
		
		const FArrayProperty* ArrayProperty = OutProperty.GetProperty();
		const FArrayProperty* InnerArrayProperty = CastFieldChecked<FArrayProperty>(ArrayProperty->Inner);

		check(ArrayProperty && InnerArrayProperty && InnerArrayProperty->Inner);

		const int32 ElementSize = InnerArrayProperty->Inner->GetElementSize();

		// @todo_pcg Do by pointer to be more efficient for complex types.
		FScriptArray OutBuffer;
		FScriptArrayHelper OutHelper(ArrayProperty, &OutBuffer);

		if (!ForcedIndex || *ForcedIndex != -1)
		{
			OutHelper.EmptyAndAddValues(IterState.NumberOfElementsToProcess);
		}
		
		for (int32 i = 0; i < IterState.NumberOfElementsToProcess; ++i)
		{
			const int32 OriginalIndex = ForcedIndex ? *ForcedIndex : Indices[i];
			int32 Index = OriginalIndex;
			auto& [DataPtr, ArrayNum] = ExtractedArray->ArrayValues[i];

			if (Index < 0)
			{
				Index = ArrayNum + Index;
			}

			if (Index < 0 || Index >= ArrayNum)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidRemoveAtIndex", "Tried to remove at index {0} in array of size {1} at element {2}."), OriginalIndex, ArrayNum, i));
			}
			else if (Index == (ArrayNum - 1))
			{
				--ArrayNum;
			}
			else
			{
				FScriptArrayHelper OutInnerHelper(InnerArrayProperty, OutHelper.GetElementPtr(i));
				OutInnerHelper.AddValues(ArrayNum - 1);
				// Copy the values until the index to remove
				PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(), DataPtr, Index);
				// Copy the rest
				PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(Index), static_cast<const uint8*>(DataPtr) + ElementSize * (Index+1), ArrayNum - Index - 1);
				
				DataPtr = OutInnerHelper.GetElementPtr();
				--ArrayNum;
			}
		}
		
		WriteResult(IterState, ExtractedArray->ArrayValues);
		return true;
	}
	
	bool AddUnique(FPCGMetadataArrayOperationIterState& IterState)
	{
		TOptional<FValueExtractionResult> ExtractedValues = ExtractValues(IterState, 1, IterState.InputAccessors[0]->GetUnderlyingDesc());
		if (!ExtractedValues)
		{
			return true;
		}
		
		// Finally pull the array
		TOptional<FArrayExtractionResult> ExtractedArray = ExtractArray(IterState);
		if (!ExtractedArray)
		{
			return true;
		}
		
		const FPCGAttributeProperty OutProperty(IterState.OutputAccessor->GetUnderlyingDesc());
		if (!OutProperty.IsValid())
		{
			PCGLog::LogErrorOnGraph(FText::Format(InvalidOutputTypeError, IterState.OutputAccessor->GetUnderlyingDesc().GetTypeText()));
			return true;
		}
		
		const FArrayProperty* ArrayProperty = OutProperty.GetProperty();
		const FArrayProperty* InnerArrayProperty = CastFieldChecked<FArrayProperty>(ArrayProperty->Inner);
		
		check(ArrayProperty && InnerArrayProperty);
		
		// @todo_pcg Do by pointer to be more efficient for complex types.
		FScriptArray OutBuffer;
		FScriptArrayHelper OutHelper(ArrayProperty, &OutBuffer);
		
		OutHelper.EmptyAndAddValues(IterState.NumberOfElementsToProcess);
		
		const FProperty* InnerProperty = InnerArrayProperty->Inner;
		check(InnerProperty);

		for (int32 i = 0; i < IterState.NumberOfElementsToProcess; ++i)
		{
			auto& [DataPtr, ArrayNum] = ExtractedArray->ArrayValues[i];

			if (FindIndex(InnerProperty, DataPtr, ArrayNum, ExtractedValues->Helper.GetElementPtr(i)) == INDEX_NONE)
			{
				FScriptArrayHelper OutInnerHelper(InnerArrayProperty, OutHelper.GetElementPtr(i));
				OutInnerHelper.AddValues(ArrayNum + 1);

				PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(), DataPtr, ArrayNum);
				PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(ArrayNum), ExtractedValues->Helper.GetElementPtr(i), 1);
				DataPtr = OutInnerHelper.GetElementPtr();
				ArrayNum++;
			}
		}
		
		WriteResult(IterState, ExtractedArray->ArrayValues);
		return true;
	}
	
	bool Append(FPCGMetadataArrayOperationIterState& IterState)
	{
		TOptional<FArrayExtractionResult> ExtractedWorkingArray = ExtractArray(IterState);
		if (!ExtractedWorkingArray)
		{
			return true;
		}
		
		TOptional<FArrayExtractionResult> ExtractedArrayToAppend = ExtractArray(IterState, /*PinIndex=*/1);
		if (!ExtractedArrayToAppend)
		{
			return true;
		}
		
		const FPCGAttributeProperty OutProperty(IterState.OutputAccessor->GetUnderlyingDesc());
		if (!OutProperty.IsValid())
		{
			PCGLog::LogErrorOnGraph(FText::Format(InvalidOutputTypeError, IterState.OutputAccessor->GetUnderlyingDesc().GetTypeText()));
			return true;
		}
		
		const FArrayProperty* ArrayProperty = OutProperty.GetProperty();
		const FArrayProperty* InnerArrayProperty = CastFieldChecked<FArrayProperty>(ArrayProperty->Inner);
		
		check(ArrayProperty && InnerArrayProperty);

		FScriptArray OutBuffer;
		FScriptArrayHelper OutHelper(ArrayProperty, &OutBuffer);
		
		OutHelper.EmptyAndAddValues(IterState.NumberOfElementsToProcess);
		
		for (int32 i = 0; i < IterState.NumberOfElementsToProcess; ++i)
		{
			auto& [DataPtr1, ArrayNum1] = ExtractedWorkingArray->ArrayValues[i];
			const auto& [DataPtr2, ArrayNum2] = ExtractedArrayToAppend->ArrayValues[i];
			
			if (!DataPtr2 || ArrayNum2 == 0)
			{
				continue;
			}
			
			FScriptArrayHelper OutInnerHelper(InnerArrayProperty, OutHelper.GetElementPtr(i));
			OutInnerHelper.AddValues(ArrayNum1 + ArrayNum2);
				
			PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(), DataPtr1, ArrayNum1);
			PCG::Private::CopyArray(InnerArrayProperty, OutInnerHelper.GetElementPtr(ArrayNum1), DataPtr2, ArrayNum2);
			DataPtr1 = OutInnerHelper.GetElementPtr();
			ArrayNum1 += ArrayNum2;
		}
		
		WriteResult(IterState, ExtractedWorkingArray->ArrayValues);
		return true;
	}
	
	bool Get(FPCGMetadataArrayOperationIterState& IterState)
	{
		// First extract indices in second accessor
		TArray<int32> Indices;
		Indices.SetNumUninitialized(IterState.NumberOfElementsToProcess);
		
		if (!IterState.InputAccessors[1]->GetRange<int32>(Indices, /*Index=*/0, *IterState.InputKeys[1], EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
		{
			PCGLog::Metadata::LogFailToGetAttributeError(IterState.InputSources[1]);
			return true;
		}
		
		TOptional<FArrayExtractionResult> ExtractedArray = ExtractArray(IterState);
		if (!ExtractedArray)
		{
			return true;
		}
		
		const FPCGAttributeProperty OutProperty(IterState.OutputAccessor->GetUnderlyingDesc());
		if (!OutProperty.IsValid())
		{
			PCGLog::LogErrorOnGraph(FText::Format(InvalidOutputTypeError, IterState.OutputAccessor->GetUnderlyingDesc().GetTypeText()));
			return true;
		}
		
		const FArrayProperty* ArrayProperty = OutProperty.GetProperty();
		const FProperty* InnerProperty = ArrayProperty->Inner;
		
		check(ArrayProperty && InnerProperty);
		
		const int32 ElementSize = InnerProperty->GetElementSize();

		// @todo_pcg Do by pointer to be more efficient for complex types.
		FScriptArray OutBuffer;
		FScriptArrayHelper Helper(OutProperty.GetProperty(), &OutBuffer);
		Helper.AddValues(IterState.NumberOfElementsToProcess);
		
		for (int32 i = 0; i < IterState.NumberOfElementsToProcess; ++i)
		{
			int32 Index = Indices[i];
			const auto& [DataPtr, ArrayNum] = ExtractedArray->ArrayValues[i];
			if (Index < 0)
			{
				Index = ArrayNum + Index;
			}
			
			if (Index < 0 || Index >= ArrayNum)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidGetIndex", "Try to get index {0} on array of size {1} at element {2}."), Indices[i], ArrayNum, i));
			}
			else
			{
				OutProperty.GetProperty()->Inner->CopyCompleteValue(Helper.GetElementPtr(i), static_cast<const uint8*>(DataPtr) + ElementSize * Index);
			}
		}
		
		IterState.OutputAccessor->SetRange(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, OutBuffer.GetData(), OutBuffer.Num()}, IterState.OutputAccessor->GetUnderlyingDesc(), IterState.NumberOfElementsToProcess, /*Index=*/0, *IterState.OutputKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
		return true;
	}
}

#if WITH_EDITOR
FName UPCGMetadataArrayOperationSettings::GetDefaultNodeName() const
{
	return "MetadataArrayOperation";
}

FText UPCGMetadataArrayOperationSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Metadata Array Operation");
}

EPCGChangeType UPCGMetadataArrayOperationSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);
	
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGMetadataArrayOperationSettings, Operation))
	{
		ChangeType |= EPCGChangeType::Structural;
	}
	
	return ChangeType;
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataArrayOperationSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataArrayOperation>();
}
#endif // WITH_EDITOR

bool UPCGMetadataArrayOperationSettings::IsPinDefaultValueEnabled(const FName PinLabel) const
{
	return DefaultValuesAreEnabled() && GetInputPinIndex(PinLabel) != INDEX_NONE && PCGMetadataHelpers::MetadataTypeSupportsDefaultValues(GetPinDefaultValueType(PinLabel));
}

bool UPCGMetadataArrayOperationSettings::IsPinDefaultValueActivated(const FName PinLabel) const
{
	if (!IsPinDefaultValueEnabled(PinLabel))
	{
		return false;
	}

	const uint32 PinIndex = GetInputPinIndex(PinLabel);
	if (PinIndex != static_cast<uint32>(INDEX_NONE))
	{
		const FName PropertyName = PCGMetadataArrayOperation::Helpers::GetDefaultValuePropertyName(PinIndex);
		return DefaultValues.IsPropertyActivated(PropertyName);
	}
	else
	{
		return false;
	}
}

EPCGMetadataTypes UPCGMetadataArrayOperationSettings::GetPinDefaultValueType(const FName PinLabel) const
{
	const FName PropertyName = PCGMetadataArrayOperation::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
	if (PropertyName != NAME_None)
	{
		if (DefaultValues.FindProperty(PropertyName))
		{
			return DefaultValues.GetCurrentPropertyType(PropertyName);
		}
		else
		{
			return GetPinInitialDefaultValueType(PinLabel);
		}
	}

	return EPCGMetadataTypes::Unknown;
}

#if WITH_EDITOR
bool UPCGMetadataArrayOperationSettings::IsPinDefaultValueMetadataTypeValid(const FName PinLabel, EPCGMetadataTypes DataType) const
{
	FPCGMetadataAttributeDesc Desc;
	Desc.ValueType = DataType;
	return IsValidType(GetInputPinIndex(PinLabel), Desc, Desc);
}

void UPCGMetadataArrayOperationSettings::ResetDefaultValues()
{
	DefaultValues.Reset();
	OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Settings | EPCGChangeType::Edge);
}

FString UPCGMetadataArrayOperationSettings::GetPinDefaultValueAsString(const FName PinLabel) const
{
	if (ensure(IsPinDefaultValueActivated(PinLabel)))
	{
		const FName PropertyName = PCGMetadataArrayOperation::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
		if (PropertyName != NAME_None)
		{
			if (DefaultValues.FindProperty(PropertyName))
			{
				return DefaultValues.GetPropertyValueAsString(PropertyName);
			}
			else
			{
				return GetPinInitialDefaultValueString(PinLabel);
			}
		}
	}

	return FString();
}

void UPCGMetadataArrayOperationSettings::ResetDefaultValue(const FName PinLabel)
{
	const FName PropertyName = PCGMetadataArrayOperation::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
	if (PropertyName != NAME_None && DefaultValues.FindProperty(PropertyName))
	{
		Modify();
		const EPCGMetadataTypes CurrentType = DefaultValues.GetCurrentPropertyType(PropertyName);
		DefaultValues.RemoveProperty(PropertyName);
		DefaultValues.CreateNewProperty(PropertyName, CurrentType);
	}
}
#endif // WITH_EDITOR

EPCGMetadataTypes UPCGMetadataArrayOperationSettings::GetPinInitialDefaultValueType(const FName PinLabel) const
{
	if (PinLabel == PCGPinConstants::DefaultInputLabel)
	{
		if (Operation == EPCGMetadataArrayOperation::ConvertToArray)
		{
			return EPCGMetadataTypes::Double;
		}
	}
	if (PinLabel == PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel)
	{
		if (Operation == EPCGMetadataArrayOperation::Get)
		{
			return EPCGMetadataTypes::Integer32;
		}
		else if (Operation == EPCGMetadataArrayOperation::Append)
		{
			// Not supported as it is an array.
			return EPCGMetadataTypes::Unknown;
		}
		else
		{
			return EPCGMetadataTypes::Double;
		}
	}
	else if (PinLabel == PCGMetadataArrayOperationConstants::IndexPinLabel)
	{
		return EPCGMetadataTypes::Integer32; 
	}
	else if (PinLabel == PCGMetadataArrayOperationConstants::ValuePinLabel)
	{
		return EPCGMetadataTypes::Double; 
	}
	
	return EPCGMetadataTypes::Unknown;
}

#if WITH_EDITOR
void UPCGMetadataArrayOperationSettings::SetPinDefaultValue(const FName PinLabel, const FString& DefaultValue, const bool bCreateIfNeeded)
{
	const FName PropertyName = PCGMetadataArrayOperation::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
	if (PropertyName != NAME_None)
	{
		Modify();

		if (!DefaultValues.FindProperty(PropertyName) && bCreateIfNeeded)
		{
			const EPCGMetadataTypes Type = GetPinInitialDefaultValueType(PinLabel);
			DefaultValues.CreateNewProperty(PropertyName, Type);
		}

		if (DefaultValues.SetPropertyValueFromString(PropertyName, DefaultValue))
		{
			OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Node | EPCGChangeType::Settings);
		}
	}
}

void UPCGMetadataArrayOperationSettings::ConvertPinDefaultValueMetadataType(const FName PinLabel, const EPCGMetadataTypes DataType)
{
	if (ensure(IsPinDefaultValueActivated(PinLabel)))
	{
		const FName PropertyName = PCGMetadataArrayOperation::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
		if (PropertyName != NAME_None && IsPinDefaultValueMetadataTypeValid(PinLabel, DataType))
		{
			Modify();
			DefaultValues.ConvertPropertyType(PropertyName, DataType);
			OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Node | EPCGChangeType::Edge | EPCGChangeType::Settings);
		}
	}
}

void UPCGMetadataArrayOperationSettings::SetPinDefaultValueIsActivated(const FName PinLabel, const bool bIsActivated, const bool bDirtySettings)
{
	if (ensure(IsPinDefaultValueEnabled(PinLabel)))
	{
		const FName PropertyName = PCGMetadataArrayOperation::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
		if (DefaultValues.IsPropertyActivated(PropertyName) != bIsActivated)
		{
			if (bDirtySettings)
			{
				Modify();
			}

			DefaultValues.SetPropertyActivated(PropertyName, bIsActivated);
			if (bDirtySettings)
			{
				OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Node | EPCGChangeType::Edge | EPCGChangeType::Settings);
			}
		}
	}
}
#endif // WITH_EDITOR

FString UPCGMetadataArrayOperationSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataArrayOperation>())
	{
		return EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation)).ToString();
	}
	else
	{
		return {};
	}
}

bool UPCGMetadataArrayOperationSettings::IsInputPinRequiredByExecution(const UPCGPin* InPin) const
{
	// The pin maintains 'required' status unless the default value is both enabled and activated.
	return InPin && (InPin->IsConnected() || !IsPinDefaultValueEnabled(InPin->Properties.Label) || !IsPinDefaultValueActivated(InPin->Properties.Label));
}

int32 UPCGMetadataArrayOperationSettings::GetInputPinIndex(FName InPinLabel) const
{
	for (int32 i = 0; i < GetNumOperands(); ++i)
	{
		if (InPinLabel == GetInputPinName(i))
		{
			return i;
		}
	}
	
	return INDEX_NONE;
}

const UPCGParamData* UPCGMetadataArrayOperationSettings::CreateDefaultValueParamData(FPCGContext* Context, FName PinLabel) const
{
	const FName PropertyName = PCGMetadataArrayOperation::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));

	const UPCGParamData* Data = DefaultValues.CreateParamData(Context, PropertyName);

	// Default Value Container did not have a value. Try getting the node's 'reset' default value.
	if (!Data)
	{
		const TObjectPtr<UPCGParamData> NewParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		if (CreateInitialDefaultValueAttribute(PinLabel, NewParamData->Metadata))
		{
			Data = NewParamData;
		}
	}

	return Data;
}

bool UPCGMetadataArrayOperationSettings::SupportsSingleEntryDomains(int32 NumberOfElementsToProcess) const
{
	if (Operation == EPCGMetadataArrayOperation::Flatten)
	{
		// Flatten is never possible
		return false;
	}
	else if (Operation == EPCGMetadataArrayOperation::MakeArray)
	{
		// MakeArray is always possible
		return true;
	}
	else
	{
		// For all other operations, it is only if the number of elements to process is less than one.
		return NumberOfElementsToProcess <= 1;
	}
}

void UPCGMetadataArrayOperationSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataArrayOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfigureInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataArrayOperation(PreconfigureInfo.PreconfiguredIndex);
		}
	}
}

TArray<FPCGPinProperties> UPCGMetadataArrayOperationSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Result;
	for (int32 i = 0; i < GetNumOperands(); ++i)
	{
		Result.Emplace_GetRef(GetInputPinName(i), EPCGDataType::Any).SetRequiredPin();
	}
	
	return Result;
}

TArray<FPCGPinProperties> UPCGMetadataArrayOperationSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Result;
	Result.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
	return Result;
}

FPCGElementPtr UPCGMetadataArrayOperationSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataArrayOperationElement>();
}

FName UPCGMetadataArrayOperationSettings::GetInputPinName(int32 PinIndex) const
{
	const int32 NumOperands = GetNumOperands();
	if (PinIndex == 0)
	{
		return NumOperands == 1 ? PCGPinConstants::DefaultInputLabel : PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
	}
	else if (PinIndex == 1)
	{
		return NumOperands == 3 ? PCGMetadataArrayOperationConstants::ValuePinLabel : PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
	}
	else if (PinIndex == 2)
	{
		return NumOperands == 3 ? PCGMetadataArrayOperationConstants::IndexPinLabel : PCGMetadataSettingsBaseConstants::DoubleInputThirdLabel;
	}
	else
	{
		checkNoEntry();
		return NAME_None;
	}
}

bool UPCGMetadataArrayOperationSettings::IsValidType(int32 PinIndex, const FPCGMetadataAttributeDesc& AttributePinType, const FPCGMetadataAttributeDesc& WorkingType) const
{
	if (PinIndex == 0)
	{
		if (Operation == EPCGMetadataArrayOperation::ConvertToArray || Operation == EPCGMetadataArrayOperation::MakeArray)
		{
			return AttributePinType.IsSingleValue();
		}
		else
		{
			return AttributePinType.IsArray();
		}
	}
	else if (PinIndex == 1)
	{
		if (Operation == EPCGMetadataArrayOperation::Get || Operation == EPCGMetadataArrayOperation::RemoveAtIndex)
		{
			return PCG::Private::IsBroadcastableOrConstructible(AttributePinType, PCG::Private::GetDefaultAttributeDesc<int32>());
		}
		else if (Operation == EPCGMetadataArrayOperation::Append)
		{
			return AttributePinType.IsArray() && PCG::Private::IsBroadcastableOrConstructible(AttributePinType, WorkingType);
		}
		else
		{
			return AttributePinType.IsSingleValue() && PCG::Private::IsBroadcastableOrConstructible(AttributePinType, WorkingType.ConvertToSingleValue());
		}
	}
	else if (PinIndex == 2)
	{
		if (Operation == EPCGMetadataArrayOperation::Insert || Operation == EPCGMetadataArrayOperation::ReplaceAtIndex)
		{
			return PCG::Private::IsBroadcastableOrConstructible(AttributePinType, PCG::Private::GetDefaultAttributeDesc<int32>());
		}
		else
		{
			// Update this if we have new operators
			checkNoEntry();
		}
	}
	
	return false;
}

FPCGMetadataAttributeDesc UPCGMetadataArrayOperationSettings::GetOutputType(const FPCGMetadataAttributeDesc& WorkingType) const
{
	if (Operation == EPCGMetadataArrayOperation::Length || Operation == EPCGMetadataArrayOperation::Find)
	{
		return PCG::Private::GetDefaultAttributeDesc<int32>();
	}
	else if (Operation == EPCGMetadataArrayOperation::Contains)
	{
		return PCG::Private::GetDefaultAttributeDesc<bool>();
	}
	
	if (Operation == EPCGMetadataArrayOperation::Get || Operation == EPCGMetadataArrayOperation::Flatten)
	{
		return WorkingType.ConvertToSingleValue();
	}
	else if (Operation == EPCGMetadataArrayOperation::ConvertToArray || Operation == EPCGMetadataArrayOperation::MakeArray)
	{
		return WorkingType.ConvertToArray();
	}
	else
	{
		return WorkingType;
	}
}

int32 UPCGMetadataArrayOperationSettings::GetNumOperands() const
{
	if (Operation > EPCGMetadataArrayOperation::ThreeInputs)
	{
		return 3;
	}
	else if (Operation > EPCGMetadataArrayOperation::TwoInputs)
	{
		return 2;
	}
	else
	{
		return 1;
	}
}

bool UPCGMetadataArrayOperationSettings::ValidateInputData(int32 PinIndex, const UPCGData* InData) const
{
	if (!InData)
	{
		PCGLog::InputOutput::LogInvalidInputDataError();
		return false;
	}
	
	// Flatten and MakeArray are only available for point data and attribute set
	if (Operation == EPCGMetadataArrayOperation::Flatten || Operation == EPCGMetadataArrayOperation::MakeArray)
	{
		if (!InData->IsA<UPCGParamData>() && !InData->IsA<UPCGBasePointData>())
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidDataType", "Flatten/MakeArray only work for AttributeSet/Points"));
			return false;
		}
	}

	return true;
}

#if WITH_EDITOR
bool UPCGMetadataArrayOperationSettings::IsInputSource2Visible() const
{
	return GetNumOperands() >= 2;
}

bool UPCGMetadataArrayOperationSettings::IsInputSource3Visible() const
{
	return GetNumOperands() >= 3;
}
#endif //WITH_EDITOR

bool FPCGMetadataArrayOperationElement::PrepareDataInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataArrayOperationElement::PrepareDataInternal);

	FPCGMetadataArrayOperationElement::ContextType* TimeSlicedContext = static_cast<FPCGMetadataArrayOperationElement::ContextType*>(Context);
	check(TimeSlicedContext);

	const UPCGMetadataArrayOperationSettings* Settings = Context->GetInputSettings<UPCGMetadataArrayOperationSettings>();
	check(Settings);

	const uint32 OperandNum = Settings->GetNumOperands();

	check(OperandNum > 0);
	
	int32 OperandInputNumMax = 0;

	// Initiialize execution state, which will setup default data as need and perform early validation.
	EPCGTimeSliceInitResult ExecStateInitResult = TimeSlicedContext->InitializePerExecutionState([this, Settings, OperandNum, &OperandInputNumMax](ContextType* Context, ExecStateType& OutState) -> EPCGTimeSliceInitResult
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataArrayOperationElement::PrepareDataInternal::InitializePerExecutionState);
		FPCGMetadataArrayOperationElement::ContextType* TimeSlicedContext = static_cast<FPCGMetadataArrayOperationElement::ContextType*>(Context);
		OutState.DefaultValueOverriddenPins.AddZeroed(OperandNum);
		OutState.OperandPinData.SetNum(OperandNum);

		for (uint32 OperandPinIndex = 0; OperandPinIndex < OperandNum; ++OperandPinIndex)
		{
			const FName CurrentPinLabel = Settings->GetInputPinName(OperandPinIndex);
			const bool bIsInputConnected = Context->Node && Context->Node->IsInputPinConnected(CurrentPinLabel);
			TArray<FPCGTaggedData>& CurrentPinInputData = OutState.OperandPinData[OperandPinIndex];
			CurrentPinInputData = Context->InputData.GetInputsByPin(CurrentPinLabel);
			
			if (Settings->DefaultValuesAreEnabled() && !bIsInputConnected && CurrentPinInputData.IsEmpty() && Settings->IsPinDefaultValueActivated(CurrentPinLabel))
			{
				FPCGTaggedData& DefaultData = CurrentPinInputData.Emplace_GetRef();
				DefaultData.Pin = CurrentPinLabel;

				// @todo_pcg: Future optimizations/refactors - cache the param data on the settings, or use accessors on the default value struct, etc.
				// Create from the Default Value Container if it exists
				DefaultData.Data = Settings->CreateDefaultValueParamData(Context, CurrentPinLabel);

				// Couldn't create a default value
				if (!DefaultData.Data)
				{
					PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("CantCreateDefaultValue", "Pin '{0}' supports default value but we could not create it."), FText::FromName(CurrentPinLabel)));
					return EPCGTimeSliceInitResult::AbortExecution;
				}
				else
				{
					// Need to make sure the param data is properly tracked by the context to prevent garbage collection
					TimeSlicedContext->TrackObject(DefaultData.Data);

					UPCGMetadata* DefaultParamMetadata = CastChecked<UPCGParamData>(DefaultData.Data)->Metadata;
					if (DefaultParamMetadata->GetLocalItemCount() == 0)
					{
						DefaultParamMetadata->AddEntry();
					}

					OutState.DefaultValueOverriddenPins[OperandPinIndex] = true;
				}
			}
			
			const int32 CurrentInputNum = CurrentPinInputData.Num();

			// For the current input, no input (0) could be default value and we support N:1 and 1:N
			if (CurrentInputNum > 1 && OperandInputNumMax > 1 && CurrentInputNum != OperandInputNumMax)
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("MismatchedOperandDataCount", "Number of data elements provided on inputs must be 1:N, N:1, or N:N."));
				return EPCGTimeSliceInitResult::AbortExecution;
			}

			if (CurrentPinInputData.IsEmpty())
			{
				// If we have no data, there is no operation
				PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("MissingInputDataForPin", "No data provided on pin '{0}'."), FText::FromName(CurrentPinLabel)));
				return EPCGTimeSliceInitResult::NoOperation;
			}
			else if (CurrentPinInputData.Num() != 1 && CurrentPinInputData.Num() != OperandInputNumMax)
			{
				PCGE_LOG(Error, GraphAndLog,
					FText::Format(LOCTEXT("MismatchedDataCountForPin", "Number of data elements ({0}) provided on pin '{1}' doesn't match number of expected elements ({2}). Only 1 input or {2} are supported."),
						CurrentPinInputData.Num(),
						FText::FromName(CurrentPinLabel),
						OperandInputNumMax));
				return EPCGTimeSliceInitResult::AbortExecution;
			}
			
			if (!Algo::AllOf(CurrentPinInputData, [Settings, OperandPinIndex](const FPCGTaggedData& Data) { return Settings->ValidateInputData(OperandPinIndex, Data.Data); }))
			{
				return EPCGTimeSliceInitResult::AbortExecution; 
			}
			
			OperandInputNumMax = FMath::Max(OperandInputNumMax, CurrentInputNum);
		}

		return EPCGTimeSliceInitResult::Success;
	});

	if (ExecStateInitResult == EPCGTimeSliceInitResult::AbortExecution)
	{
		return true;
	}
	else if (ExecStateInitResult == EPCGTimeSliceInitResult::NoOperation)
	{
		// Passthrough all inputs
		if (!TimeSlicedContext->GetPerExecutionState().OperandPinData.IsEmpty())
		{
			Context->OutputData.TaggedData = TimeSlicedContext->GetPerExecutionState().OperandPinData[0];
		}

		return true;
	}

	// Set up the iterations on the multiple inputs of the primary pin
	TimeSlicedContext->InitializePerIterationStates(OperandInputNumMax, [this, Context, OperandNum, Settings](IterStateType& OutState, const ExecStateType& ExecState, const uint32 IterationIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataElementBase::PrepareDataInternal::InitializePerIterationStates);
		FPCGMetadataElementBase::ContextType* TimeSlicedContext = static_cast<FPCGMetadataElementBase::ContextType*>(Context);
		TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

		// Gathering all the inputs metadata
		TArray<const UPCGMetadata*> SourceMetadata;
		TArray<const FPCGMetadataAttributeBase*> SourceAttribute;
		TArray<FPCGTaggedData> InputTaggedData;
		SourceMetadata.SetNum(OperandNum);
		SourceAttribute.SetNum(OperandNum);
		InputTaggedData.SetNum(OperandNum);

		const uint32 PrimaryPinIndex = 0;

		// Iterate over the inputs and validate
		for (uint32 OperandPinIndex = 0; OperandPinIndex < OperandNum; ++OperandPinIndex)
		{
			const TArray<FPCGTaggedData>& CurrentPinInputData = ExecState.OperandPinData[OperandPinIndex];

			// The operand inputs must either be N:1 or N:N or 1:N
			InputTaggedData[OperandPinIndex] = CurrentPinInputData.Num() == 1 ? CurrentPinInputData[0] : CurrentPinInputData[IterationIndex];

			SourceMetadata[OperandPinIndex] = InputTaggedData[OperandPinIndex].Data->ConstMetadata();
			if (!SourceMetadata[OperandPinIndex])
			{
				PCGLog::Metadata::LogInvalidMetadata(Context);
				return EPCGTimeSliceInitResult::AbortExecution;
			}
		}

		OutState.InputAccessors.SetNum(OperandNum);
		OutState.InputKeys.SetNum(OperandNum);
		OutState.InputSources.SetNum(OperandNum);
		OutState.WorkingType = FPCGMetadataAttributeDesc{};

		const FPCGTaggedData& PrimaryPinData = InputTaggedData[PrimaryPinIndex];
		if (!CreateAndValidateAccessors(Settings, ExecState, OutState, PrimaryPinIndex, PrimaryPinData.Data, Settings->InputSource1))
		{
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		OutState.WorkingType = OutState.InputAccessors[PrimaryPinIndex]->GetUnderlyingDesc();

		OutState.NumberOfElementsToProcess = OutState.InputKeys[PrimaryPinIndex]->GetNum();
		if (OutState.NumberOfElementsToProcess == 0)
		{
			PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("NoElementsInForwardedInput", "No elements in data from forwarded pin '{0}'."), FText::FromName(PrimaryPinData.Pin)));
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Create the accessors and validate them for each of the other operands
		for (uint32 Index = 0; Index < OperandNum; ++Index)
		{
			if (Index == PrimaryPinIndex)
			{
				continue;
			}

			const FPCGAttributePropertyInputSelector& Selector = Index == 1 ? Settings->InputSource2 : Settings->InputSource3;
			if (!CreateAndValidateAccessors(Settings, ExecState, OutState, Index, InputTaggedData[Index].Data, Selector))
			{
				return EPCGTimeSliceInitResult::AbortExecution;
			}

			const int32 ElementNum = OutState.InputKeys[Index]->GetNum();

			// Verify that the number of elements makes sense
			if (ElementNum == 0 || OutState.NumberOfElementsToProcess % ElementNum != 0)
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("MismatchInNumberOfElements", "Mismatch between the number of elements from pin '{0}' ({1}) and from pin '{2}' ({3})."), FText::FromName(PrimaryPinData.Pin), OutState.NumberOfElementsToProcess, FText::FromName(InputTaggedData[Index].Pin), ElementNum));
				return EPCGTimeSliceInitResult::AbortExecution;
			}
		}

		UPCGData* OutputData = CreateOutputData(Settings, InputTaggedData[PrimaryPinIndex].Data, PrimaryPinIndex, OutState, Context);
		if (!OutputData)
		{
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		Outputs.Add_GetRef(InputTaggedData[PrimaryPinIndex]).Data = OutputData;
			
		const FPCGAttributePropertyOutputSelector OutputTarget = Settings->OutputTarget.CopyAndFixSource(&OutState.InputSources[PrimaryPinIndex]);
		OutState.OutputType = Settings->GetOutputType(OutState.WorkingType);
			
		// If the output target is an attribute, make sure to delete it before, as this is what it is done in PCG Metadata Op Element Base
		const FPCGMetadataDomainID DomainID = OutputData->GetMetadataDomainIDFromSelector(OutputTarget);
		if (OutputTarget.IsBasicAttribute())
		{
			const FName AttributeName = OutputTarget.GetAttributeName();
			FPCGMetadataDomain* Domain = OutputData->MutableMetadata()->GetMetadataDomain(DomainID);
			if (Domain && Domain->HasAttribute(OutputTarget.GetAttributeName()))
			{
				Domain->DeleteAttribute(AttributeName);
			}
		}

		PCGAttributeAccessorHelpers::FPCGCreateAccessorWithAttributeCreationParams Params =
		{
			.InData = OutputData,
			.InSelector = &OutputTarget,
			.InMatchingAccessor = OutState.OutputType.IsSameType(OutState.WorkingType) ? OutState.InputAccessors[PrimaryPinIndex].Get() : nullptr,
			.InExpectedDesc = OutState.OutputType
		};

		OutState.OutputAccessor = PCGAttributeAccessorHelpers::CreateAccessorWithAttributeCreation(Params);
		OutState.OutputKeys = PCGAttributeAccessorHelpers::CreateKeys(OutputData, OutputTarget);
		if (!OutState.OutputAccessor || !OutState.OutputKeys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(OutputTarget, Context);
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		if (OutState.OutputAccessor->IsAttribute()
			&& !OutputData->ConstMetadata()->MetadataDomainSupportsMultiEntries(DomainID)
			&& !Settings->SupportsSingleEntryDomains(OutState.NumberOfElementsToProcess))
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("OutputAccessorIsNotSupportingMultiEntries", "Output attribute '{0}' is on a domain that doesn't support multi entries, but we try to process multiple elements ({1}) or to flatten. It's invalid."), OutputTarget.GetDisplayText(), OutState.NumberOfElementsToProcess));
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		OutState.Settings = Settings;

		return EPCGTimeSliceInitResult::Success;
	});

	return true;
}

bool FPCGMetadataArrayOperationElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataElementBase::Execute);
	ContextType* TimeSlicedContext = static_cast<ContextType*>(Context);
	check(TimeSlicedContext);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	
	const UPCGMetadataArrayOperationSettings* Settings = Context->GetInputSettings<UPCGMetadataArrayOperationSettings>();
	check(Settings);

	// Prepare data failed, no need to execute. Return an empty output
	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		return true;
	}

	return ExecuteSlice(TimeSlicedContext, [this, &Outputs, Settings](ContextType* Context, const ExecStateType& ExecState, IterStateType& IterState, const uint32 IterationIndex) -> bool
	{
		// No operation, so skip the iteration.
		if (Context->GetIterationStateResult(IterationIndex) == EPCGTimeSliceInitResult::NoOperation)
		{
			return true;
		}

		bool bIsDone = true;
		
		switch (Settings->Operation)
		{
		case EPCGMetadataArrayOperation::ConvertToArray:
			bIsDone = PCGMetadataArrayOperation::ConvertToArray(IterState);
			break;
		case EPCGMetadataArrayOperation::Flatten:
			bIsDone = PCGMetadataArrayOperation::Flatten(IterState);
			break;
		case EPCGMetadataArrayOperation::MakeArray:
			bIsDone = PCGMetadataArrayOperation::MakeArray(IterState);
			break;
		case EPCGMetadataArrayOperation::Length:
			bIsDone = PCGMetadataArrayOperation::Length(IterState);
			break;
		case EPCGMetadataArrayOperation::Insert:
			bIsDone = PCGMetadataArrayOperation::Insert(IterState);
			break;
		case EPCGMetadataArrayOperation::Add:
			bIsDone = PCGMetadataArrayOperation::Insert(IterState, -1);
			break;
		case EPCGMetadataArrayOperation::AddUnique:
			bIsDone = PCGMetadataArrayOperation::AddUnique(IterState);
			break;
		case EPCGMetadataArrayOperation::Get:
			bIsDone = PCGMetadataArrayOperation::Get(IterState);
			break;
		case EPCGMetadataArrayOperation::Append:
			bIsDone = PCGMetadataArrayOperation::Append(IterState);
			break;
		case EPCGMetadataArrayOperation::Find:
			bIsDone = PCGMetadataArrayOperation::FindOrContains(IterState, /*bContains=*/false);
			break;
		case EPCGMetadataArrayOperation::Contains:
			bIsDone = PCGMetadataArrayOperation::FindOrContains(IterState, /*bContains=*/true);
			break;
		case EPCGMetadataArrayOperation::ReplaceAtIndex:
			bIsDone = PCGMetadataArrayOperation::ReplaceAtIndex(IterState);
			break;
		case EPCGMetadataArrayOperation::Pop:
			bIsDone = PCGMetadataArrayOperation::RemoveAt(IterState, -1);
			break;
		case EPCGMetadataArrayOperation::RemoveAtIndex:
			bIsDone = PCGMetadataArrayOperation::RemoveAt(IterState);
			break;
		default:
			break;
		}

		return bIsDone;
	});
}

bool FPCGMetadataArrayOperationElement::CreateAndValidateAccessors(const UPCGMetadataArrayOperationSettings* Settings, const FPCGMetadataArrayOperationExecState& ExecState, FPCGMetadataArrayOperationIterState& OutState, int32 Index, const UPCGData* InputData, const FPCGAttributePropertyInputSelector& InputSelector) const
{
	static const FPCGAttributePropertyInputSelector EmptySelector{};
	OutState.InputSources[Index] = (!ExecState.DefaultValueOverriddenPins[Index] ? InputSelector : EmptySelector).CopyAndFixLast(InputData);
			
	OutState.InputAccessors[Index] = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, OutState.InputSources[Index]);
	OutState.InputKeys[Index] = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, OutState.InputSources[Index]);
						
	if (!OutState.InputAccessors[Index] || !OutState.InputKeys[Index])
	{
		PCGLog::Metadata::LogFailToCreateAccessorError(OutState.InputSources[Index]);
		return false;
	}
			
	if (!Settings->IsValidType(Index, OutState.InputAccessors[Index]->GetUnderlyingDesc(), OutState.WorkingType))
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidType", "Attribute {0} on pin {1} is of type {2} which is not convertible to the required type {3}."),
			OutState.InputSources[Index].GetDisplayText(),
			FText::FromName(Settings->GetInputPinName(Index)),
			OutState.InputAccessors[Index]->GetUnderlyingDesc().GetTypeText(),
			OutState.WorkingType.GetTypeText()));
		return false;
	}
	
	return true;
}

UPCGData* FPCGMetadataArrayOperationElement::CreateOutputData(const UPCGMetadataArrayOperationSettings* Settings, const UPCGData* InData, const int32 InputPinIndex, FPCGMetadataArrayOperationIterState& OutState, FPCGContext* InContext) const
{
	const FPCGAttributePropertyInputSelector& InputSource = OutState.InputSources[InputPinIndex];

	if (Settings->Operation == EPCGMetadataArrayOperation::Flatten)
	{
		// When flattening, we will need to copy all the attributes and remap them
		UPCGData* OutputData = nullptr;
		
		TArray<PCGMetadataEntryKey> DuplicatedEntries;
		TArray<TTuple<const void*, int32>> ArrayValues;
		ArrayValues.SetNumUninitialized(OutState.NumberOfElementsToProcess);
		
		// Gather all the values
		if (!OutState.InputAccessors[InputPinIndex]->GetRangeVirtual(PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, ArrayValues}, OutState.NumberOfElementsToProcess, /*Index=*/0, *OutState.InputKeys[InputPinIndex]))
		{
			PCGLog::Metadata::LogFailToGetAttributeError(InputSource, InContext);
			return nullptr;
		}
		
		int32 Sum = 0;
		for (const auto& [DataPtr, ArrayNum] : ArrayValues)
		{
			Sum += ArrayNum;
		}
		
		DuplicatedEntries.Reserve(Sum);

		if (const UPCGParamData* ParamData = Cast<UPCGParamData>(InData))
		{
			UPCGParamData* OutputParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InContext);
			// Copy the data domain
			OutputParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->InitializeAsCopy(ParamData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data));
			// Initialize the elements domain with all the input attributes except the one we target
			FPCGMetadataDomainInitializeParams Params{ParamData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements)};
			Params.FilteredAttributes = {InputSource.GetName()};
			
			FPCGMetadataDomain* OutDomain = OutputParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements);
			OutDomain->AddAttributes(Params);
			
			TArray<PCGMetadataEntryKey> OutKeys;
			OutKeys.Reserve(Sum);
			for (int32 i = 0; i < ArrayValues.Num(); ++i)
			{
				for (int32 j = 0; j < ArrayValues[i].Get<1>(); ++j)
				{
					DuplicatedEntries.Add(i);
					OutKeys.Add(PCGInvalidEntryKey);
				}
			}
			
			OutKeys = OutDomain->AddEntries(OutKeys);
			OutDomain->SetAttributes(DuplicatedEntries, ParamData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements), OutKeys, InContext);
			
			OutputData = OutputParamData;
		}
		else if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(InData))
		{
			UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(InContext);
			FPCGInitializeFromDataParams Params{PointData};
			Params.bInheritSpatialData = false;
			
			OutputPointData->InitializeFromDataWithParams(Params);
			
			// Remove the input source attribute
			OutputPointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements)->DeleteAttribute(InputSource.GetName());
			// Copy all the points
			OutputPointData->SetNumPoints(Sum);
			
			TArray<int32> ReadPointIndices;
			TArray<int32> WritePointIndices;
			int32 CurrentIndex = 0;
			for (int32 i = 0; i < ArrayValues.Num(); ++i)
			{
				const int32 ArrayNum = ArrayValues[i].Get<1>();
				ReadPointIndices.SetNumUninitialized(ArrayNum, EAllowShrinking::No);
				WritePointIndices.SetNumUninitialized(ArrayNum, EAllowShrinking::No);
				for (int32 j = 0; j < ArrayNum; ++j)
				{
					ReadPointIndices[j] = i;
					WritePointIndices[j] = CurrentIndex++;
				}
				
				PointData->CopyPointsTo(OutputPointData, ReadPointIndices, WritePointIndices);
			}
			
			OutputData = OutputPointData;
		}
		else
		{
			checkNoEntry();
		}
		
		return OutputData;
	}
	else if (Settings->Operation == EPCGMetadataArrayOperation::MakeArray)
	{
		UPCGData* OutputData = nullptr;
		// Take the first entry only
		if (const UPCGParamData* ParamData = Cast<UPCGParamData>(InData))
		{
			UPCGParamData* OutputParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InContext);
			// Copy the data domain
			OutputParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->InitializeAsCopy(ParamData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data));
			// Initialize the elements domain with all the input attributes except the one we target
			FPCGMetadataDomainInitializeParams Params{ParamData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements)};
			Params.FilteredAttributes = {InputSource.GetName()};
			
			FPCGMetadataDomain* OutDomain = OutputParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements);
			OutDomain->AddAttributes(Params);
			// Copy the first entry
			PCGMetadataEntryKey OutKey = OutDomain->AddEntry();
			OutDomain->SetAttributes(PCGFirstEntryKey, ParamData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements), OutKey);
			
			OutputData = OutputParamData;
		}
		else if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(InData))
		{
			UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(InContext);
			FPCGInitializeFromDataParams Params{PointData};
			Params.bInheritSpatialData = false;
			
			OutputPointData->InitializeFromDataWithParams(Params);
			
			// Remove the input source attribute
			OutputPointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements)->DeleteAttribute(InputSource.GetName());
			// Copy the first point
			OutputPointData->SetNumPoints(1);
			OutputPointData->SetPointsFrom(PointData, {0});
			
			OutputData = OutputPointData;
		}
		else
		{
			checkNoEntry();
		}
		
		return OutputData;
	}
	else
	{
		return InData->DuplicateData(InContext);
	}
}


#undef LOCTEXT_NAMESPACE
