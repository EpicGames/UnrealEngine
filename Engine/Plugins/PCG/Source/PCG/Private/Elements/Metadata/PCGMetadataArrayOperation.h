// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Elements/PCGTimeSlicedElementBase.h"
#include "Helpers/PCGDefaultValueContainer.h"
#include "Metadata/PCGDefaultValueInterface.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "PCGMetadataArrayOperation.generated.h"

class FPCGMetadataArrayOperationElement;

namespace PCGMetadataArrayOperationConstants
{
	constexpr FLazyName ValuePinLabel = "Value";
	constexpr FLazyName IndexPinLabel = "Index";
}

UENUM()
enum class EPCGMetadataArrayOperation : uint16
{
	OneInput = 1 << 10 UMETA(Hidden),
	/** Convert a single value into an array. */
	ConvertToArray,
	/** Flatten the content of the array into multiple elements (Attribute sets and Points only). */
	Flatten,
	/** Create an array from a list of elements (Attribute set and Points only). */
	MakeArray,
	/** Remove the last element of the array. Will error out if there are no elements. */
	Pop,
	/** Return the length the array */
	Length,
	
	TwoInputs = 1 << 11 UMETA(Hidden),
	/** Add an element at the end of an array. */
	Add UMETA(DisplayName = "Add To Array"),
	/** Add an element at the end of the array if it doesn't exist already in this array. */
	AddUnique UMETA(DisplayName = "Add Unique To Array"),
	/** Get the element of the array at the given index. Negative index start from the end (-1 = last element). Will error out if the index is out of bounds. */
	Get UMETA(DisplayName = "Get At Index"),
	/** Append the content of another array. */
	Append,
	/** Get the index of a given value. Return -1 if not found. */
	Find,
	/** Returns true if the array contains the given value. */
	Contains,
	/** Remove element at the given index. Negative index start from the end (-1 = last element). Will error out if the index is out of bounds. */
	RemoveAtIndex,
	
	ThreeInputs = 1 << 12 UMETA(Hidden),
	/** Insert an element at the given index. Negative index start from the end (-1 = last element, insert at the end). Will error out if the index is out of bounds. */
	Insert UMETA(DisplayName = "Insert At Index"),
	/** Replace the value at the given index. Will error out if the index is out of bounds. */
	ReplaceAtIndex,
};

/**
* Operations to manipulate arrays in metadata
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataArrayOperationSettings : public UPCGSettings, public IPCGSettingsDefaultValueProvider
{
	GENERATED_BODY()
	
	friend FPCGMetadataArrayOperationElement;

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual bool HasFlippedTitleLines() const override { return true; };
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
#endif
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;
	virtual FString GetAdditionalTitleInformation() const override;
	
	virtual bool CanCullTaskIfUnwired() const override { return false; }
	virtual bool IsInputPinRequiredByExecution(const UPCGPin* InPin) const override;
	
	virtual bool HasDynamicPins() const override { return true; }
	
	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual bool DefaultValuesAreEnabled() const override { return true; }
	virtual bool IsPinDefaultValueEnabled(FName PinLabel) const override;
	virtual bool IsPinDefaultValueActivated(FName PinLabel) const override;
	virtual EPCGMetadataTypes GetPinDefaultValueType(FName PinLabel) const override;
#if WITH_EDITOR
	virtual bool IsPinDefaultValueMetadataTypeValid(FName PinLabel, EPCGMetadataTypes DataType) const override;
	virtual void SetPinDefaultValue(FName PinLabel, const FString& DefaultValue, bool bCreateIfNeeded = false) override;
	virtual void ConvertPinDefaultValueMetadataType(FName PinLabel, EPCGMetadataTypes DataType) override;
	virtual void SetPinDefaultValueIsActivated(FName PinLabel, bool bIsActivated, bool bDirtySettings = true) override;
	virtual void ResetDefaultValues() override;
	virtual FString GetPinInitialDefaultValueString(FName PinLabel) const override { return PCG::Private::MetadataTraits<double>::ZeroValueString(); }
	virtual FString GetPinDefaultValueAsString(FName PinLabel) const override;
	virtual void ResetDefaultValue(FName PinLabel) override;
#endif // WITH_EDITOR

protected:
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	//~End IPCGSettingsDefaultValueProvider interface

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	bool IsValidType(int32 PinIndex, const FPCGMetadataAttributeDesc& AttributePinType, const FPCGMetadataAttributeDesc& WorkingType) const;
	FPCGMetadataAttributeDesc GetOutputType(const FPCGMetadataAttributeDesc& WorkingType) const;
	bool ValidateInputData(int32 PinIndex, const UPCGData* InData) const;
	
	/* Return the index of the given input pin label. INDEX_NONE if not found*/
	PCG_API int32 GetInputPinIndex(FName InPinLabel) const;
	
	/** Creates a Param Data with the inline constant default value properties inserted as metadata. */
	const UPCGParamData* CreateDefaultValueParamData(FPCGContext* Context, FName PinLabel) const;
	
	/** For single entry domains like the data domain, not all operations are possible. */
	bool SupportsSingleEntryDomains(int32 NumberOfElementsToProcess) const;

public:
	FName GetInputPinName(int32 PinIndex) const;

	UFUNCTION()
	int32 GetNumOperands() const;
	
#if WITH_EDITOR
	UFUNCTION()
	bool IsInputSource2Visible() const;
	
	UFUNCTION()
	bool IsInputSource3Visible() const;
#endif // WITH_EDITOR
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGMetadataArrayOperation Operation = EPCGMetadataArrayOperation::Get;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource1;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "IsInputSource2Visible", EditConditionHides))
	FPCGAttributePropertyInputSelector InputSource2;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "IsInputSource3Visible", EditConditionHides))
	FPCGAttributePropertyInputSelector InputSource3;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyOutputSelector OutputTarget;
	
private:
	/** Stores the default values for the pins to be used as inline constants. */
	UPROPERTY()
	FPCGDefaultValueContainer DefaultValues;
};

struct FPCGMetadataArrayOperationExecState
{
	TArray<TArray<FPCGTaggedData>> OperandPinData;
	TArray<bool, TFixedAllocator<3>> DefaultValueOverriddenPins;
};

struct FPCGMetadataArrayOperationIterState
{
	TArray<TUniquePtr<const IPCGAttributeAccessor>> InputAccessors;
	TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> InputKeys;
	TArray<FPCGAttributePropertyInputSelector> InputSources;
	
	TUniquePtr<IPCGAttributeAccessor> OutputAccessor;
	TUniquePtr<IPCGAttributeAccessorKeys> OutputKeys;
	FPCGAttributePropertyOutputSelector OutputTarget;
	
	FPCGMetadataAttributeDesc WorkingType;
	FPCGMetadataAttributeDesc OutputType;
	int32 NumberOfElementsToProcess = 0;

	const UPCGMetadataArrayOperationSettings* Settings = nullptr;
};

class FPCGMetadataArrayOperationElement : public TPCGTimeSlicedElementBase<FPCGMetadataArrayOperationExecState, FPCGMetadataArrayOperationIterState>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	
	bool CreateAndValidateAccessors(const UPCGMetadataArrayOperationSettings* Settings, const FPCGMetadataArrayOperationExecState& ExecState, FPCGMetadataArrayOperationIterState& OutState, int32 Index, const UPCGData* InputData, const FPCGAttributePropertyInputSelector& InputSelector) const;
	UPCGData* CreateOutputData(const UPCGMetadataArrayOperationSettings* Settings, const UPCGData* InData, const int32 InputPinIndex, FPCGMetadataArrayOperationIterState& OutState, FPCGContext* InContext) const;
};

