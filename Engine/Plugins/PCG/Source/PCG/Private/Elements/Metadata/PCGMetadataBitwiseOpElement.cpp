// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataBitwiseOpElement.h"

#include "Elements/Metadata/PCGMetadataElementCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataBitwiseOpElement)

namespace PCGMetadataBitwiseSettings
{
	inline int64 UnaryOp(const int64& Value)
	{
		// EPCGMetadataBitwiseOperation::Not
		return ~Value;
	}

	inline int64 BinaryOp(const int64& Value1, const int64& Value2, EPCGMetadataBitwiseOperation Operation)
	{
		switch (Operation)
		{
		case EPCGMetadataBitwiseOperation::And:
			return (Value1 & Value2);
		case EPCGMetadataBitwiseOperation::Or:
			return (Value1 | Value2);
		case EPCGMetadataBitwiseOperation::Xor:
			return (Value1 ^ Value2);
		case EPCGMetadataBitwiseOperation::ShiftLeft:
			return Value1 << Value2;
		case EPCGMetadataBitwiseOperation::ShiftRight:
			return Value1 >> Value2;
		default:
			return 0;
		}
	}
}

void UPCGMetadataBitwiseSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (Input1AttributeName_DEPRECATED != NAME_None)
	{
		InputSource1.SetAttributeName(Input1AttributeName_DEPRECATED);
		Input1AttributeName_DEPRECATED = NAME_None;
	}

	if (Input2AttributeName_DEPRECATED != NAME_None)
	{
		InputSource2.SetAttributeName(Input2AttributeName_DEPRECATED);
		Input2AttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

FName UPCGMetadataBitwiseSettings::GetInputPinLabel(uint32 Index) const
{
	const bool bIsShiftOp = Operation == EPCGMetadataBitwiseOperation::ShiftLeft ||
			Operation == EPCGMetadataBitwiseOperation::ShiftRight;
	
	switch (Index)
	{
	case 0:
		return (Operation == EPCGMetadataBitwiseOperation::Not || bIsShiftOp) ? PCGPinConstants::DefaultInputLabel : PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
	case 1:
		return bIsShiftOp ? PCGMetadataBitwiseConstants::ShiftInputLabel : PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataBitwiseSettings::GetOperandNum() const
{
	return (Operation == EPCGMetadataBitwiseOperation::Not) ? 1 : 2;
}

bool UPCGMetadataBitwiseSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;
	return PCG::Private::IsOfTypes<int32, int64>(TypeId);
}

FPCGAttributePropertyInputSelector UPCGMetadataBitwiseSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

FString UPCGMetadataBitwiseSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataBitwiseOperation>())
	{
		return FText::Format(NSLOCTEXT("PCGMetadataBitwiseSettings", "BitwiseOperation", "Bitwise: {0}"), EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation))).ToString();
	}
	else
	{
		return FString();
	}
}

#if WITH_EDITOR
FName UPCGMetadataBitwiseSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeBitwiseOp");
}

FText UPCGMetadataBitwiseSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataBitwiseSettings", "NodeTitle", "Attribute Bitwise Op");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataBitwiseSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataBitwiseOperation>();
}
#endif // WITH_EDITOR

void UPCGMetadataBitwiseSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataBitwiseOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataBitwiseOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

FPCGElementPtr UPCGMetadataBitwiseSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataBitwiseElement>();
}

uint16 UPCGMetadataBitwiseSettings::GetOutputType(uint16 InputTypeId) const
{
	return (uint16)EPCGMetadataTypes::Integer64;
}

bool FPCGMetadataBitwiseElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataBitwiseElement::Execute);

	const UPCGMetadataBitwiseSettings* Settings = CastChecked<UPCGMetadataBitwiseSettings>(OperationData.Settings);

	if (Settings->Operation == EPCGMetadataBitwiseOperation::Not)
	{
		return DoUnaryOp<int64>(OperationData, [](const int64& Value) -> int64 { return PCGMetadataBitwiseSettings::UnaryOp(Value); });
	}
	else
	{
		return DoBinaryOp<int64, int64>(OperationData, [Operation = Settings->Operation](const int64& Value1, const int64& Value2) -> int64 { return PCGMetadataBitwiseSettings::BinaryOp(Value1, Value2, Operation); });
	}
}
