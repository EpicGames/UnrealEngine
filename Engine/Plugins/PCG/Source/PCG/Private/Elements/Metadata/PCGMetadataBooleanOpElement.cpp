// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataBooleanOpElement.h"

#include "Elements/Metadata/PCGMetadataElementCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataBooleanOpElement)

namespace PCGMetadataBooleanSettings
{
	inline bool UnaryOp(const bool& Value)
	{
		// EPCGMetadataBooleanOperation::Not
		return !Value;
	}

	inline bool BinaryOp(const bool& Value1, const bool& Value2, EPCGMetadataBooleanOperation Operation)
	{
		switch (Operation)
		{
		case EPCGMetadataBooleanOperation::And:
			return (Value1 && Value2);
		case EPCGMetadataBooleanOperation::Or:
			return (Value1 || Value2);
		case EPCGMetadataBooleanOperation::Xor:
			return (Value1 != Value2);
		case EPCGMetadataBooleanOperation::Nand:
			return !(Value1 && Value2);
		case EPCGMetadataBooleanOperation::Nor:
			return !(Value1 || Value2);
		case EPCGMetadataBooleanOperation::Xnor:
			return (Value1 == Value2);
		case EPCGMetadataBooleanOperation::Imply:
			return !Value1 || Value2;
		case EPCGMetadataBooleanOperation::Nimply:
			return Value1 && !Value2;
		default:
			return false;
		}
	}
}

void UPCGMetadataBooleanSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (Input1AttributeName_DEPRECATED != NAME_None)
	{
		InputSource1.SetAttributeName(Input1AttributeName_DEPRECATED);
		Input1AttributeName_DEPRECATED = NAME_None;
	}

	if (Input2AttributeName_DEPRECATED != NAME_None)
	{
		InputSource2.SetAttributeName(Input2AttributeName_DEPRECATED);
		Input2AttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

FName UPCGMetadataBooleanSettings::GetInputPinLabel(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return (Operation == EPCGMetadataBooleanOperation::Not) ? PCGPinConstants::DefaultInputLabel : PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
	case 1:
		return PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataBooleanSettings::GetOperandNum() const
{
	return (Operation == EPCGMetadataBooleanOperation::Not) ? 1 : 2;
}

bool UPCGMetadataBooleanSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;
	return PCG::Private::IsBroadcastableOrConstructible(TypeId, PCG::Private::MetadataTypes<bool>::Id);
}

FPCGAttributePropertyInputSelector UPCGMetadataBooleanSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

uint16 UPCGMetadataBooleanSettings::GetOutputType(uint16 InputTypeId) const
{
	return (uint16)EPCGMetadataTypes::Boolean;
}

FString UPCGMetadataBooleanSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataBooleanOperation>())
	{
		return FText::Format(NSLOCTEXT("PCGMetadataBooleanSettings", "BooleanOperation", "Boolean: {0}"), EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation))).ToString();
	}
	else
	{
		return FString();
	}
}

#if WITH_EDITOR
FName UPCGMetadataBooleanSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeBooleanOp");
}

FText UPCGMetadataBooleanSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataBooleanSettings", "NodeTitle", "Attribute Boolean Op");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataBooleanSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataBooleanOperation>();
}
#endif // WITH_EDITOR

void UPCGMetadataBooleanSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataBooleanOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataBooleanOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

FPCGElementPtr UPCGMetadataBooleanSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataBooleanElement>();
}

bool FPCGMetadataBooleanElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataBooleanElement::Execute);

	const UPCGMetadataBooleanSettings* Settings = CastChecked<UPCGMetadataBooleanSettings>(OperationData.Settings);

	if (Settings->Operation == EPCGMetadataBooleanOperation::Not)
	{
		return DoUnaryOp<bool>(OperationData, [](const bool& Value) -> bool { return PCGMetadataBooleanSettings::UnaryOp(Value); });
	}
	else
	{
		return DoBinaryOp<bool, bool>(OperationData, [Operation = Settings->Operation](const bool& Value1, const bool& Value2) -> bool { return PCGMetadataBooleanSettings::BinaryOp(Value1, Value2, Operation); });
	}
}
