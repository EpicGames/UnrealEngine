// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataBreakTransform.h"

#include "Elements/Metadata/PCGMetadataMakeTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataBreakTransform)

namespace PCGMetadataBreakTransformSettings
{
	inline FVector GetTranslation(const FTransform& Transform)
	{
		return Transform.GetTranslation();
	}

	inline FVector GetScale(const FTransform& Transform)
	{
		return Transform.GetScale3D();
	}

	inline FQuat GetRotation(const FTransform& Transform)
	{
		return Transform.GetRotation();
	}
}

void UPCGMetadataBreakTransformSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (InputAttributeName_DEPRECATED != NAME_None)
	{
		InputSource.SetAttributeName(InputAttributeName_DEPRECATED);
		InputAttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
  bool UPCGMetadataBreakTransformSettings::CanEditChange(const FProperty* InProperty) const
{
	if (!InProperty || !UPCGSettings::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGMetadataBreakTransformSettings, InputSource))
	{
		if (const UPCGNode* Node = Cast<UPCGNode>(GetOuter()))
		{
			return Node->IsInputPinConnected(PCGPinConstants::DefaultInputLabel);
		}
	}

	return false;
}
#endif // WITH_EDITOR

FPCGAttributePropertyInputSelector UPCGMetadataBreakTransformSettings::GetInputSource(uint32 Index) const
{
	return InputSource;
}

FName UPCGMetadataBreakTransformSettings::GetOutputPinLabel(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return PCGMetadataTransformConstants::Translation;
	case 1:
		return PCGMetadataTransformConstants::Rotation;
	case 2:
		return PCGMetadataTransformConstants::Scale;
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataBreakTransformSettings::GetResultNum() const
{
	return 3;
}

bool UPCGMetadataBreakTransformSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;
	return PCG::Private::IsOfTypes<FTransform>(TypeId);
}

bool UPCGMetadataBreakTransformSettings::HasDifferentOutputTypes() const
{
	return true;
}

TArray<uint16> UPCGMetadataBreakTransformSettings::GetAllOutputTypes() const
{
	return { (uint16)EPCGMetadataTypes::Vector, (uint16)EPCGMetadataTypes::Quaternion, (uint16)EPCGMetadataTypes::Vector };
}

FName UPCGMetadataBreakTransformSettings::GetOutputAttributeName(FName BaseName, uint32 Index) const
{
	if (BaseName != NAME_None)
	{
		return FName(BaseName.ToString() + "." + GetOutputPinLabel(Index).ToString());
	}
	else
	{
		return NAME_None;
	}
}

#if WITH_EDITOR
FName UPCGMetadataBreakTransformSettings::GetDefaultNodeName() const
{
	return TEXT("BreakTransformAttribute");
}

FText UPCGMetadataBreakTransformSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataBreakTransformSettings", "NodeTitle", "Break Transform Attribute");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGMetadataBreakTransformSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataBreakTransformElement>();
}

bool FPCGMetadataBreakTransformElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataBreakTransformElement::Execute);

	const UPCGMetadataBreakTransformSettings* Settings = static_cast<const UPCGMetadataBreakTransformSettings*>(OperationData.Settings);
	check(Settings);

	return DoUnaryOp<FTransform>(OperationData,
		PCGMetadataBreakTransformSettings::GetTranslation,
		PCGMetadataBreakTransformSettings::GetRotation,
		PCGMetadataBreakTransformSettings::GetScale);
}

