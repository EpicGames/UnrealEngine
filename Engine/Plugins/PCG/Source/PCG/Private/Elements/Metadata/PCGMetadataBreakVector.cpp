// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataBreakVector.h"

#include "Metadata/PCGMetadataAttributeTpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataBreakVector)

namespace PCGMetadataBreakVectorSettings
{
	template <typename InType>
	inline double GetFirstIndex(const InType& Value)
	{
		return Value.X;
	}

	template<>
	inline double GetFirstIndex<FRotator>(const FRotator& Value)
	{
		return Value.Roll;
	}

	template <typename InType>
	inline double GetSecondIndex(const InType& Value)
	{
		return Value.Y;
	}

	template<>
	inline double GetSecondIndex<FRotator>(const FRotator& Value)
	{
		return Value.Pitch;
	}

	template <typename InType>
	inline double GetThirdIndex(const InType& Value)
	{
		return Value.Z;
	}

	template<>
	inline double GetThirdIndex<FRotator>(const FRotator& Value)
	{
		return Value.Yaw;
	}

	template<>
	inline double GetThirdIndex<FVector2D>(const FVector2D& Value)
	{
		return 0.0;
	}

	template <typename InType>
	inline double GetFourthIndex(const InType& Value)
	{
		return 0.0;
	}

	template<>
	inline double GetFourthIndex<FVector4>(const FVector4& Value)
	{
		return Value.W;
	}

	inline constexpr bool IsValidType(uint16 TypeId)
	{
		return PCG::Private::IsOfTypes<FVector2D, FVector, FVector4, FRotator>(TypeId);
	}

	template <typename T>
	inline constexpr bool IsValidType()
	{
		return IsValidType(PCG::Private::MetadataTypes<T>::Id);
	}
}

void UPCGMetadataBreakVectorSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (InputAttributeName_DEPRECATED != NAME_None)
	{
		InputSource.SetAttributeName(InputAttributeName_DEPRECATED);
		InputAttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
  bool UPCGMetadataBreakVectorSettings::CanEditChange(const FProperty* InProperty) const
{
	if (!InProperty || !UPCGSettings::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGMetadataBreakVectorSettings, InputSource))
	{
		if (const UPCGNode* Node = Cast<UPCGNode>(GetOuter()))
		{
			return Node->IsInputPinConnected(PCGPinConstants::DefaultInputLabel);
		}
	}

	return false;
}
#endif // WITH_EDITOR

FPCGAttributePropertyInputSelector UPCGMetadataBreakVectorSettings::GetInputSource(uint32 Index) const
{
	return InputSource;
}

FName UPCGMetadataBreakVectorSettings::GetOutputPinLabel(uint32 Index) const 
{
	switch (Index)
	{
	case 0:
		return PCGMetadataBreakVectorConstants::XLabel;
	case 1:
		return PCGMetadataBreakVectorConstants::YLabel;
	case 2:
		return PCGMetadataBreakVectorConstants::ZLabel;
	default:
		return PCGMetadataBreakVectorConstants::WLabel;
	}
}

uint32 UPCGMetadataBreakVectorSettings::GetResultNum() const
{
	return 4;
}

bool UPCGMetadataBreakVectorSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;
	return PCGMetadataBreakVectorSettings::IsValidType(TypeId);
}

uint16 UPCGMetadataBreakVectorSettings::GetOutputType(uint16 InputTypeId) const
{
	return (uint16)EPCGMetadataTypes::Double;
}

FName UPCGMetadataBreakVectorSettings::GetOutputAttributeName(FName BaseName, uint32 Index) const
{
	if (BaseName == NAME_None)
	{
		return NAME_None;
	}

	switch (Index)
	{
	case 0:
		return FName(BaseName.ToString() + "." + PCGMetadataBreakVectorConstants::XLabel.ToString());
	case 1:
		return FName(BaseName.ToString() + "." + PCGMetadataBreakVectorConstants::YLabel.ToString());
	case 2:
		return FName(BaseName.ToString() + "." + PCGMetadataBreakVectorConstants::ZLabel.ToString());
	default:
		return FName(BaseName.ToString() + "." + PCGMetadataBreakVectorConstants::WLabel.ToString());
	}
}

#if WITH_EDITOR
FName UPCGMetadataBreakVectorSettings::GetDefaultNodeName() const
{
	return TEXT("BreakVectorAttribute");
}

FText UPCGMetadataBreakVectorSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataBreakVectorSettings", "NodeTitle", "Break Vector Attribute");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGMetadataBreakVectorSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataBreakVectorElement>();
}

bool FPCGMetadataBreakVectorElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataBreakVectorElement::Execute);

	const UPCGMetadataBreakVectorSettings* Settings = static_cast<const UPCGMetadataBreakVectorSettings*>(OperationData.Settings);
	check(Settings);

	auto BreakFunc = [this, &OperationData]<typename AttributeType>(AttributeType) -> bool
	{
		if constexpr (!PCGMetadataBreakVectorSettings::IsValidType<AttributeType>())
		{
			ensure(false);
			return true;
		}
		else
		{
			return DoUnaryOp<AttributeType>(OperationData, 
				PCGMetadataBreakVectorSettings::GetFirstIndex<AttributeType>,
				PCGMetadataBreakVectorSettings::GetSecondIndex<AttributeType>,
				PCGMetadataBreakVectorSettings::GetThirdIndex<AttributeType>,
				PCGMetadataBreakVectorSettings::GetFourthIndex<AttributeType>);
		}
	};

	return PCGMetadataAttribute::CallbackWithRightType(OperationData.MostComplexInputType, BreakFunc);
}

