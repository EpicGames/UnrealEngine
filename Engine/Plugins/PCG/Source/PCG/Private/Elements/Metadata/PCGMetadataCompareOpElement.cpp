// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataCompareOpElement.h"

#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataCompareOpElement)

namespace PCGMetadataCompareSettings
{
	template <typename T>
	bool ApplyCompare(const T& Input1, const T& Input2, EPCGMetadataCompareOperation Operation, double Tolerance = UE_DOUBLE_SMALL_NUMBER)
	{
		if (Operation == EPCGMetadataCompareOperation::Equal)
		{
			if constexpr (PCG::Private::MetadataTraits<T>::IsFloatingPoint)
			{
				return PCG::Private::MetadataTraits<T>::Equal(Input1, Input2, Tolerance);
			}
			else
			{
				return PCG::Private::MetadataTraits<T>::Equal(Input1, Input2);
			}
		}
		else if (Operation == EPCGMetadataCompareOperation::NotEqual)
		{
			if constexpr (PCG::Private::MetadataTraits<T>::IsFloatingPoint)
			{
				return !PCG::Private::MetadataTraits<T>::Equal(Input1, Input2, Tolerance);
			}
			else
			{
				return !PCG::Private::MetadataTraits<T>::Equal(Input1, Input2);
			}
		}

		if constexpr (PCG::Private::MetadataTraits<T>::CanCompare)
		{
			switch (Operation)
			{
			case EPCGMetadataCompareOperation::Greater:
				return PCG::Private::MetadataTraits<T>::Greater(Input1, Input2);
			case EPCGMetadataCompareOperation::GreaterOrEqual:
				return PCG::Private::MetadataTraits<T>::GreaterOrEqual(Input1, Input2);
			case EPCGMetadataCompareOperation::Less:
				return PCG::Private::MetadataTraits<T>::Less(Input1, Input2);
			case EPCGMetadataCompareOperation::LessOrEqual:
				return PCG::Private::MetadataTraits<T>::LessOrEqual(Input1, Input2);
			default:
				return false;
			}
		}
		else
		{
			return false;
		}
	}
}

void UPCGMetadataCompareSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (Input1AttributeName_DEPRECATED != NAME_None)
	{
		InputSource1.SetAttributeName(Input1AttributeName_DEPRECATED);
		Input1AttributeName_DEPRECATED = NAME_None;
	}

	if (Input2AttributeName_DEPRECATED != NAME_None)
	{
		InputSource2.SetAttributeName(Input2AttributeName_DEPRECATED);
		Input2AttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

FName UPCGMetadataCompareSettings::GetInputPinLabel(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
	case 1:
		return PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataCompareSettings::GetOperandNum() const
{
	return 2;
}

bool UPCGMetadataCompareSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;
	if (Operation == EPCGMetadataCompareOperation::Equal || Operation == EPCGMetadataCompareOperation::NotEqual)
	{
		return PCG::Private::IsPCGType(TypeId);
	}
	else
	{
		auto IsValid = [](auto Dummy) -> bool
		{
			return PCG::Private::MetadataTraits<decltype(Dummy)>::CanCompare;
		};

		return PCGMetadataAttribute::CallbackWithRightType(TypeId, IsValid);
	}
}

FPCGAttributePropertyInputSelector UPCGMetadataCompareSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

uint16 UPCGMetadataCompareSettings::GetOutputType(uint16 InputTypeId) const
{
	return (uint16)EPCGMetadataTypes::Boolean;
}

FString UPCGMetadataCompareSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataCompareOperation>())
	{
		return FText::Format(NSLOCTEXT("PCGMetadataCompareSettings", "CompareWithOperation", "Compare: {0}"), EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation))).ToString();
	}
	else
	{
		return FString();
	}
}

#if WITH_EDITOR
FName UPCGMetadataCompareSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeCompareOp");
}

FText UPCGMetadataCompareSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataCompareSettings", "NodeTitle", "Attribute Compare Op");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataCompareSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataCompareOperation>();
}
#endif // WITH_EDITOR

void UPCGMetadataCompareSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataCompareOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataCompareOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

FPCGElementPtr UPCGMetadataCompareSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataCompareElement>();
}

bool FPCGMetadataCompareElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataCompareElement::Execute);

	const UPCGMetadataCompareSettings* Settings = CastChecked<UPCGMetadataCompareSettings>(OperationData.Settings);

	auto CompareFunc = [this, Operation = Settings->Operation, Tolerance = Settings->Tolerance, &OperationData]<typename AttributeType>(AttributeType) -> bool
	{
		return DoBinaryOp<AttributeType, AttributeType>(OperationData, 
			[Operation, Tolerance](const AttributeType& Value1, const AttributeType& Value2) -> bool { 
				return PCGMetadataCompareSettings::ApplyCompare(Value1, Value2, Operation, Tolerance); 
			});
	};

	return PCGMetadataAttribute::CallbackWithRightType(OperationData.MostComplexInputType, CompareFunc);
}
