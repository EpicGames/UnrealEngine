// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataElementCommon.h"

#include "PCGParamData.h"
#include "Data/PCGSpatialData.h"

namespace PCGMetadataElementCommon
{
	void DuplicateTaggedData(const FPCGTaggedData& InTaggedData, FPCGTaggedData& OutTaggedData, UPCGMetadata*& OutMetadata)
	{
		DuplicateTaggedData(nullptr, InTaggedData, OutTaggedData, OutMetadata);
	}

	void DuplicateTaggedData(FPCGContext* InContext, const FPCGTaggedData& InTaggedData, FPCGTaggedData& OutTaggedData, UPCGMetadata*& OutMetadata)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataElementCommon::DuplicateTaggedData);
		if (InTaggedData.Data)
		{
			UPCGData* NewData = InTaggedData.Data->DuplicateData(InContext);
			check(NewData);
			OutTaggedData.Data = NewData;
			OutMetadata = NewData->MutableMetadata();
		}
	}

	void CopyEntryToValueKeyMap(const UPCGMetadata* MetadataToCopy, const FPCGMetadataAttributeBase* AttributeToCopy, FPCGMetadataAttributeBase* OutAttribute)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataElementCommon::CopyEntryToValueKeyMap);

		if (!OutAttribute)
		{
			UE_LOGF(LogPCG, Error, "Failed to create output attribute");
			return;
		}

		const PCGMetadataEntryKey EntryKeyCount = MetadataToCopy->GetItemCountForChild();
		for (PCGMetadataEntryKey EntryKey = 0; EntryKey < EntryKeyCount; ++EntryKey)
		{
			const PCGMetadataValueKey ValueKey = AttributeToCopy->GetValueKey(EntryKey);
			OutAttribute->SetValueFromValueKey(EntryKey, ValueKey);
		}
	}

	bool CopyFromAccessorToAccessor(FCopyFromAccessorToAccessorParams& Params)
	{
		check(Params.InAccessor && Params.OutAccessor && Params.InKeys && Params.OutKeys);

		int32 Count = 0;
		switch (Params.IterationCount)
		{
		case FCopyFromAccessorToAccessorParams::EIterationCount::In:
			Count = Params.InKeys->GetNum();
			break;
		case FCopyFromAccessorToAccessorParams::EIterationCount::Out:
			Count = Params.OutKeys->GetNum();
			break;
		case FCopyFromAccessorToAccessorParams::EIterationCount::Min:
			Count = FMath::Min(Params.InKeys->GetNum(), Params.OutKeys->GetNum());
			break;
		case FCopyFromAccessorToAccessorParams::EIterationCount::Max:
			Count = FMath::Max(Params.InKeys->GetNum(), Params.OutKeys->GetNum());
			break;
		default:
			checkNoEntry();
			return false;
		}
		
		const int32 NumberOfIterations = (Count + Params.ChunkSize - 1) / Params.ChunkSize;
		
		for (int32 i = 0; i < NumberOfIterations; ++i)
		{
			const int32 StartIndex = i * Params.ChunkSize;
			const int32 Range = FMath::Min(Count - StartIndex, Params.ChunkSize);
			if (!Params.InAccessor->CopyTo(*Params.InKeys, *Params.OutAccessor, *Params.OutKeys, StartIndex, Range, Params.Flags))
			{
				return false;
			}
		}
		
		return true;
	}
}
