// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataMakeRotator.h"

#include "PCGParamData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadata.h"

#include "Algo/Transform.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataMakeRotator)

#define LOCTEXT_NAMESPACE "PCGMetadataMakeRotatorSettings"

namespace PCGMetadataMakeRotator::Helpers
{
	FVector GetAxisAlignedVector(const FName PinLabel)
	{
		if (PinLabel == PCGMetadataMakeRotatorConstants::XLabel || PinLabel == PCGMetadataMakeRotatorConstants::ForwardLabel)
		{
			return FVector::ForwardVector;
		}
		else if (PinLabel == PCGMetadataMakeRotatorConstants::YLabel || PinLabel == PCGMetadataMakeRotatorConstants::RightLabel)
		{
			return FVector::RightVector;
		}
		else if (PinLabel == PCGMetadataMakeRotatorConstants::ZLabel || PinLabel == PCGMetadataMakeRotatorConstants::UpLabel)
		{
			return FVector::UpVector;
		}
		else
		{
			return FVector::ZeroVector;
		}
	}

	FString GetAxisAlignedVectorString(const FName PinLabel)
	{
		return PCG::Private::MetadataTraits<FVector>::ToString(GetAxisAlignedVector(PinLabel));
	}
}

FName UPCGMetadataMakeRotatorSettings::GetInputPinLabel(uint32 Index) const
{
	switch (Operation)
	{
	case EPCGMetadataMakeRotatorOp::MakeRotFromX:
		return PCGMetadataMakeRotatorConstants::XLabel;
	case EPCGMetadataMakeRotatorOp::MakeRotFromY:
		return PCGMetadataMakeRotatorConstants::YLabel;
	case EPCGMetadataMakeRotatorOp::MakeRotFromZ:
		return PCGMetadataMakeRotatorConstants::ZLabel;
	case EPCGMetadataMakeRotatorOp::MakeRotFromXY:
		return (Index == 0 ? PCGMetadataMakeRotatorConstants::XLabel : PCGMetadataMakeRotatorConstants::YLabel);
	case EPCGMetadataMakeRotatorOp::MakeRotFromYX:
		return (Index == 1 ? PCGMetadataMakeRotatorConstants::XLabel : PCGMetadataMakeRotatorConstants::YLabel);
	case EPCGMetadataMakeRotatorOp::MakeRotFromXZ:
		return (Index == 0 ? PCGMetadataMakeRotatorConstants::XLabel : PCGMetadataMakeRotatorConstants::ZLabel);
	case EPCGMetadataMakeRotatorOp::MakeRotFromZX:
		return (Index == 1 ? PCGMetadataMakeRotatorConstants::XLabel : PCGMetadataMakeRotatorConstants::ZLabel);
	case EPCGMetadataMakeRotatorOp::MakeRotFromYZ:
		return (Index == 0 ? PCGMetadataMakeRotatorConstants::YLabel : PCGMetadataMakeRotatorConstants::ZLabel);
	case EPCGMetadataMakeRotatorOp::MakeRotFromZY:
		return (Index == 1 ? PCGMetadataMakeRotatorConstants::YLabel : PCGMetadataMakeRotatorConstants::ZLabel);
	case EPCGMetadataMakeRotatorOp::MakeRotFromAxes:
		return (Index == 0 ? PCGMetadataMakeRotatorConstants::ForwardLabel : (Index == 1 ? PCGMetadataMakeRotatorConstants::RightLabel : PCGMetadataMakeRotatorConstants::UpLabel));
	case EPCGMetadataMakeRotatorOp::MakeRotFromAngles:
		return (Index == 0 ? PCGMetadataMakeRotatorConstants::RollLabel : (Index == 1 ? PCGMetadataMakeRotatorConstants::PitchLabel : PCGMetadataMakeRotatorConstants::YawLabel));
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataMakeRotatorSettings::GetOperandNum() const
{
	switch (Operation)
	{
	case EPCGMetadataMakeRotatorOp::MakeRotFromX: // fall-through
	case EPCGMetadataMakeRotatorOp::MakeRotFromY: // fall-through
	case EPCGMetadataMakeRotatorOp::MakeRotFromZ:
		return 1;
	case EPCGMetadataMakeRotatorOp::MakeRotFromXY: // fall-through
	case EPCGMetadataMakeRotatorOp::MakeRotFromYX: // fall-through
	case EPCGMetadataMakeRotatorOp::MakeRotFromXZ: // fall-through
	case EPCGMetadataMakeRotatorOp::MakeRotFromZX: // fall-through
	case EPCGMetadataMakeRotatorOp::MakeRotFromYZ: // fall-through
	case EPCGMetadataMakeRotatorOp::MakeRotFromZY:
		return 2;
	case EPCGMetadataMakeRotatorOp::MakeRotFromAxes: // fall-through
	case EPCGMetadataMakeRotatorOp::MakeRotFromAngles:
	default:
		return 3;
	}
}

bool UPCGMetadataMakeRotatorSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;
	if (Operation == EPCGMetadataMakeRotatorOp::MakeRotFromAngles)
	{
		return PCG::Private::IsOfTypes<float, double, int32, int64>(TypeId);
	}
	else
	{
		return PCG::Private::IsOfTypes<FVector, FVector2D, float, double, int32, int64>(TypeId);
	}
}

FPCGAttributePropertyInputSelector UPCGMetadataMakeRotatorSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	case 2:
		return InputSource3;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

uint16 UPCGMetadataMakeRotatorSettings::GetOutputType(uint16 InputTypeId) const
{
	return (uint16)EPCGMetadataTypes::Rotator;
}

#if WITH_EDITOR
void UPCGMetadataMakeRotatorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(UPCGMetadataMakeRotatorSettings, Operation))
	{
		// @todo_pcg: It doesn't need to be reset in most cases.
		ResetDefaultValues();
	}
}

FName UPCGMetadataMakeRotatorSettings::GetDefaultNodeName() const
{
	return TEXT("MakeRotatorAttribute");
}

FText UPCGMetadataMakeRotatorSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Make Rotator Attribute");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataMakeRotatorSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataMakeRotatorOp>();
}

void UPCGMetadataMakeRotatorSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	// Supported default values on Make Rot From Angles
	if (InOutNode
		&& Operation == EPCGMetadataMakeRotatorOp::MakeRotFromAngles
		&& GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGInlineConstantDefaultValues)
	{
		// Gather the labels instead of operating on the pins directly, because converting the pin types will trigger an update.
		TArray<FName> PinLabels;
		PinLabels.Reserve(InputPins.Num());

		// Only transform the pins that aren't connected.
		Algo::TransformIf(InputPins, PinLabels,
			/*Predicate=*/[](const UPCGPin* Pin)
			{
				return !Pin->IsConnected();
			},
			/*Trans=*/[](const UPCGPin* Pin)
			{
				return Pin->Properties.Label;
			});

		// Activate the remaining unconnected pins and for MakeRotFromAngles, change the type from vector to double.
		for (const FName Label : PinLabels)
		{
			if (IsPinDefaultValueEnabled(Label))
			{
				SetPinDefaultValueIsActivated(Label, /*bIsActivated=*/true, /*bDirtySettings=*/false);
				ConvertPinDefaultValueMetadataType(Label, EPCGMetadataTypes::Double);
			}
		}
	}
}

FString UPCGMetadataMakeRotatorSettings::GetPinInitialDefaultValueString(FName PinLabel) const
{
	switch (Operation)
	{
		case EPCGMetadataMakeRotatorOp::MakeRotFromAxes:
		case EPCGMetadataMakeRotatorOp::MakeRotFromX:  // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromY:  // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromZ:  // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromXY: // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromYX: // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromXZ: // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromZX: // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromYZ: // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromZY: // fall-through
			return PCGMetadataMakeRotator::Helpers::GetAxisAlignedVectorString(PinLabel);
		case EPCGMetadataMakeRotatorOp::MakeRotFromAngles:
			return PCG::Private::MetadataTraits<double>::ZeroValueString();
		default:
			checkNoEntry();
			return FString();
	}
}
#endif // WITH_EDITOR

bool UPCGMetadataMakeRotatorSettings::CreateInitialDefaultValueAttribute(const FName PinLabel, UPCGMetadata* OutMetadata) const
{
	if (Operation != EPCGMetadataMakeRotatorOp::MakeRotFromAngles)
	{
		const FVector Value = PCGMetadataMakeRotator::Helpers::GetAxisAlignedVector(PinLabel);
		return nullptr != OutMetadata->CreateAttribute(NAME_None, Value, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
	}
	else
	{
		return Super::CreateInitialDefaultValueAttribute(PinLabel, OutMetadata);
	}
}

EPCGMetadataTypes UPCGMetadataMakeRotatorSettings::GetPinInitialDefaultValueType(FName PinLabel) const
{
	switch (Operation)
	{
		case EPCGMetadataMakeRotatorOp::MakeRotFromAxes: // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromX:    // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromY:    // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromZ:    // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromXY:   // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromYX:   // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromXZ:   // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromZX:   // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromYZ:   // fall-through
		case EPCGMetadataMakeRotatorOp::MakeRotFromZY:
			return EPCGMetadataTypes::Vector;
		case EPCGMetadataMakeRotatorOp::MakeRotFromAngles:
			return EPCGMetadataTypes::Double;
		default:
			checkNoEntry();
			return EPCGMetadataTypes::Unknown;
	}
}

void UPCGMetadataMakeRotatorSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataMakeRotatorOp>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataMakeRotatorOp(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

FPCGElementPtr UPCGMetadataMakeRotatorSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataMakeRotatorElement>();
}

bool FPCGMetadataMakeRotatorElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataMakeRotatorElement::Execute);

	const UPCGMetadataMakeRotatorSettings* Settings = CastChecked<UPCGMetadataMakeRotatorSettings>(OperationData.Settings);

	switch (Settings->Operation)
	{
	case EPCGMetadataMakeRotatorOp::MakeRotFromX:
		return DoUnaryOp<FVector>(OperationData, [](const FVector& X) -> FRotator { return FRotationMatrix::MakeFromX(X).Rotator(); });
	case EPCGMetadataMakeRotatorOp::MakeRotFromY:
		return DoUnaryOp<FVector>(OperationData, [](const FVector& Y) -> FRotator { return FRotationMatrix::MakeFromY(Y).Rotator(); });
	case EPCGMetadataMakeRotatorOp::MakeRotFromZ:
		return DoUnaryOp<FVector>(OperationData, [](const FVector& Z) -> FRotator { return FRotationMatrix::MakeFromZ(Z).Rotator(); });
	case EPCGMetadataMakeRotatorOp::MakeRotFromXY:
		return DoBinaryOp<FVector, FVector>(OperationData, [](const FVector& X, const FVector& Y) -> FRotator { return FRotationMatrix::MakeFromXY(X, Y).Rotator(); });
	case EPCGMetadataMakeRotatorOp::MakeRotFromYX:
		return DoBinaryOp<FVector, FVector>(OperationData, [](const FVector& Y, const FVector& X) -> FRotator { return FRotationMatrix::MakeFromYX(Y, X).Rotator(); });
	case EPCGMetadataMakeRotatorOp::MakeRotFromXZ:
		return DoBinaryOp<FVector, FVector>(OperationData, [](const FVector& X, const FVector& Z) -> FRotator { return FRotationMatrix::MakeFromXZ(X, Z).Rotator(); });
	case EPCGMetadataMakeRotatorOp::MakeRotFromZX:
		return DoBinaryOp<FVector, FVector>(OperationData, [](const FVector& Z, const FVector& X) -> FRotator { return FRotationMatrix::MakeFromZX(Z, X).Rotator(); });
	case EPCGMetadataMakeRotatorOp::MakeRotFromYZ:
		return DoBinaryOp<FVector, FVector>(OperationData, [](const FVector& Y, const FVector& Z) -> FRotator { return FRotationMatrix::MakeFromYZ(Y, Z).Rotator(); });
	case EPCGMetadataMakeRotatorOp::MakeRotFromZY:
		return DoBinaryOp<FVector, FVector>(OperationData, [](const FVector& Z, const FVector& Y) -> FRotator { return FRotationMatrix::MakeFromZY(Z, Y).Rotator(); });
	case EPCGMetadataMakeRotatorOp::MakeRotFromAxes:
		return DoTernaryOp<FVector, FVector, FVector>(OperationData, [](const FVector& X, const FVector& Y, const FVector& Z) -> FRotator
		{
			return FMatrix(X.GetSafeNormal(), Y.GetSafeNormal(), Z.GetSafeNormal(), FVector::ZeroVector).Rotator();
		});
	case EPCGMetadataMakeRotatorOp::MakeRotFromAngles:
		return DoTernaryOp<double, double, double>(OperationData, [](const double& Roll, const double& Pitch, const double& Yaw) -> FRotator
		{
			return FRotator{ Pitch, Yaw, Roll };
		});
	default:
		ensure(false);
		return true;
	}
}

#undef LOCTEXT_NAMESPACE
