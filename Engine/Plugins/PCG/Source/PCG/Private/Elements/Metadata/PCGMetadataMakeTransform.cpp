// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataMakeTransform.h"

#include "PCGParamData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataMakeTransform)

namespace PCGMetadataMakeTransformSettings
{
	template <typename VectorType = FVector4>
	FTransform MakeTransform(const VectorType& Translation, const FQuat& Rotation, const VectorType& Scale)
	{
		return FTransform(Rotation, FVector(Translation), FVector(Scale));
	}

	template <>
	FTransform MakeTransform<FVector>(const FVector& Translation, const FQuat& Rotation, const FVector& Scale)
	{
		return FTransform(Rotation, Translation, Scale);
	}

	template <>
	FTransform MakeTransform<FVector2D>(const FVector2D& Translation, const FQuat& Rotation, const FVector2D& Scale)
	{
		return FTransform(Rotation, FVector(Translation, 0.0), FVector(Scale, 1.0));
	}
}

void UPCGMetadataMakeTransformSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (Input1AttributeName_DEPRECATED != NAME_None)
	{
		InputSource1.SetAttributeName(Input1AttributeName_DEPRECATED);
		Input1AttributeName_DEPRECATED = NAME_None;
	}

	if (Input2AttributeName_DEPRECATED != NAME_None)
	{
		InputSource2.SetAttributeName(Input2AttributeName_DEPRECATED);
		Input2AttributeName_DEPRECATED = NAME_None;
	}

	if (Input3AttributeName_DEPRECATED != NAME_None)
	{
		InputSource3.SetAttributeName(Input3AttributeName_DEPRECATED);
		Input3AttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

FName UPCGMetadataMakeTransformSettings::GetInputPinLabel(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return PCGMetadataTransformConstants::Translation;
	case 1:
		return PCGMetadataTransformConstants::Rotation;
	case 2:
		return PCGMetadataTransformConstants::Scale;
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataMakeTransformSettings::GetOperandNum() const
{
	return 3;
}

bool UPCGMetadataMakeTransformSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	if (InputIndex == 1)
	{
		bHasSpecialRequirement = true;
		return PCG::Private::IsOfTypes<FRotator, FQuat>(TypeId);
	}
	else
	{
		bHasSpecialRequirement = false;
		return PCG::Private::IsOfTypes<FVector2D, FVector, FVector4, int32, int64, float , double>(TypeId);
	}
}

FPCGAttributePropertyInputSelector UPCGMetadataMakeTransformSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	case 2:
		return InputSource3;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

uint16 UPCGMetadataMakeTransformSettings::GetOutputType(uint16 InputTypeId) const
{
	return (uint16)EPCGMetadataTypes::Transform;
}

#if WITH_EDITOR
FName UPCGMetadataMakeTransformSettings::GetDefaultNodeName() const
{
	return TEXT("MakeTransformAttribute");
}

FText UPCGMetadataMakeTransformSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataMakeTransformSettings", "NodeTitle", "Make Transform Attribute");
}

void UPCGMetadataMakeTransformSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	// Supported default values on all pins
	if (this != GetClass()->GetDefaultObject()
		&& GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGInlineConstantDefaultValues)
	{
		for (const UPCGPin* Pin : InputPins)
		{
			if (IsPinDefaultValueEnabled(Pin->Properties.Label) && !Pin->IsConnected())
			{
				SetPinDefaultValueIsActivated(Pin->Properties.Label, /*bIsActivated=*/true, /*bDirtySettings=*/false);
			}
		}
	}
}

FString UPCGMetadataMakeTransformSettings::GetPinInitialDefaultValueString(const FName PinLabel) const
{
	if (PinLabel == PCGMetadataTransformConstants::Translation)
	{
		// Location -> Default is Zero vector
		return PCG::Private::MetadataTraits<FVector>::ZeroValueString();
	}
	else if (PinLabel == PCGMetadataTransformConstants::Rotation)
	{
		// Rotation -> Default is Zero rotator
		return PCG::Private::MetadataTraits<FRotator>::ZeroValueString();
	}
	else if (PinLabel == PCGMetadataTransformConstants::Scale)
	{
		// Scale -> Default is Vector (1, 1, 1)
		return FVector::OneVector.ToString();
	}
	else
	{
		return FString();
	}
}
#endif // WITH_EDITOR

EPCGMetadataTypes UPCGMetadataMakeTransformSettings::GetPinInitialDefaultValueType(const FName PinLabel) const
{
	if (PinLabel == PCGMetadataTransformConstants::Translation
		|| PinLabel == PCGMetadataTransformConstants::Scale)
	{
		return EPCGMetadataTypes::Vector;
	}
	else if (PinLabel == PCGMetadataTransformConstants::Rotation)
	{
		return EPCGMetadataTypes::Rotator;
	}
	else
	{
		return EPCGMetadataTypes::Unknown;
	}
}

bool UPCGMetadataMakeTransformSettings::CreateInitialDefaultValueAttribute(const FName PinLabel, UPCGMetadata* OutMetadata) const
{
	if (PinLabel == PCGMetadataTransformConstants::Scale)
	{
		return nullptr != OutMetadata->CreateAttribute(NAME_None, FVector::OneVector, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
	}
	else
	{
		return Super::CreateInitialDefaultValueAttribute(PinLabel, OutMetadata);
	}
}

FPCGElementPtr UPCGMetadataMakeTransformSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataMakeTransformElement>();
}

bool FPCGMetadataMakeTransformElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataMakeTransformElement::Execute);

	const UPCGMetadataMakeTransformSettings* Settings = CastChecked<UPCGMetadataMakeTransformSettings>(OperationData.Settings);

	auto TransformFunc = [this, &OperationData]<typename AttributeType>(AttributeType) -> bool
	{
		if constexpr (PCG::Private::IsOfTypes<AttributeType, FVector2D, FVector, FVector4>())
		{
			return DoTernaryOp<AttributeType, FQuat, AttributeType>(OperationData, PCGMetadataMakeTransformSettings::MakeTransform<AttributeType>);
		}
		else if constexpr (PCG::Private::IsOfTypes<AttributeType, int32, int64, float, double>())
		{
			return DoTernaryOp<FVector, FQuat, FVector>(OperationData, PCGMetadataMakeTransformSettings::MakeTransform<FVector>);
		}
		else
		{
			ensure(false);
			return true;
		}
	};

	return PCGMetadataAttribute::CallbackWithRightType(OperationData.MostComplexInputType, TransformFunc);
}
