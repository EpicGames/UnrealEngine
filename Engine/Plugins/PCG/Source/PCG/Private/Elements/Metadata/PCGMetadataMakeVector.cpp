// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataMakeVector.h"

#include "PCGParamData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadata.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataMakeVector)

namespace PCGMetadataMakeVectorSettings
{
	FVector2D MakeVector2(const double& X, const double& Y)
	{
		return FVector2D(X, Y);
	}

	FVector MakeVector3(const double& X, const double& Y, const double& Z)
	{
		return FVector(X, Y, Z);
	}

	FVector MakeVector3Vec2(const FVector2D& XY, const double& Z)
	{
		return FVector(XY, Z);
	}

	FVector4 MakeVector4(const double& X, const double& Y, const double& Z, const double& W)
	{
		return FVector4(X, Y, Z, W);
	}

	FVector4 MakeVector4Vec2(const FVector2D& XY, const double& Z, const double W)
	{
		return FVector4(XY.X, XY.Y, Z, W);
	}

	FVector4 MakeVector4TwoVec2(const FVector2D& XY, const FVector2D& ZW)
	{
		return FVector4(XY, ZW);
	}

	FVector4 MakeVector4Vec3(const FVector& XYZ, const double& W)
	{
		return FVector4(XYZ, W);
	}
}

void UPCGMetadataMakeVectorSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (Input1AttributeName_DEPRECATED != NAME_None)
	{
		InputSource1.SetAttributeName(Input1AttributeName_DEPRECATED);
		Input1AttributeName_DEPRECATED = NAME_None;
	}

	if (Input2AttributeName_DEPRECATED != NAME_None)
	{
		InputSource2.SetAttributeName(Input2AttributeName_DEPRECATED);
		Input2AttributeName_DEPRECATED = NAME_None;
	}

	if (Input3AttributeName_DEPRECATED != NAME_None)
	{
		InputSource3.SetAttributeName(Input3AttributeName_DEPRECATED);
		Input3AttributeName_DEPRECATED = NAME_None;
	}

	if (Input4AttributeName_DEPRECATED != NAME_None)
	{
		InputSource4.SetAttributeName(Input4AttributeName_DEPRECATED);
		Input4AttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UPCGMetadataMakeVectorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(UPCGMetadataMakeVectorSettings, MakeVector3Op)
		|| PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(UPCGMetadataMakeVectorSettings, MakeVector4Op)
		|| PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(UPCGMetadataMakeVectorSettings, OutputType))
	{
		// @todo_pcg: It doesn't need to be reset in most cases.
		ResetDefaultValues();
	}
}
#endif // WITH_EDITOR

FName UPCGMetadataMakeVectorSettings::GetInputPinLabel(uint32 Index) const
{
	static const FName DefaultNames[4] = {
		PCGMetadataMakeVectorConstants::XLabel,
		PCGMetadataMakeVectorConstants::YLabel,
		PCGMetadataMakeVectorConstants::ZLabel,
		PCGMetadataMakeVectorConstants::WLabel
	};

	if (OutputType == EPCGMetadataTypes::Vector2)
	{
		return DefaultNames[Index];
	}
	else if (OutputType == EPCGMetadataTypes::Vector)
	{
		if (MakeVector3Op == EPCGMetadataMakeVector3::ThreeValues)
		{
			return DefaultNames[Index];
		}
		else
		{
			return Index == 0 ? PCGMetadataMakeVectorConstants::XYLabel : PCGMetadataMakeVectorConstants::ZLabel;
		}
	}
	else
	{
		if (MakeVector4Op == EPCGMetadataMakeVector4::FourValues)
		{
			return DefaultNames[Index];
		}
		else if (MakeVector4Op == EPCGMetadataMakeVector4::Vector2AndTwoValues)
		{
			return Index == 0 ? PCGMetadataMakeVectorConstants::XYLabel : (Index == 1 ? PCGMetadataMakeVectorConstants::ZLabel : PCGMetadataMakeVectorConstants::WLabel);
		}
		else if (MakeVector4Op == EPCGMetadataMakeVector4::TwoVector2)
		{
			return Index == 0 ? PCGMetadataMakeVectorConstants::XYLabel : PCGMetadataMakeVectorConstants::ZWLabel;
		}
		else
		{
			return Index == 0 ? PCGMetadataMakeVectorConstants::XYZLabel : PCGMetadataMakeVectorConstants::WLabel;
		}
	}
}

uint32 UPCGMetadataMakeVectorSettings::GetOperandNum() const
{
	if (OutputType == EPCGMetadataTypes::Vector2 ||
		(OutputType == EPCGMetadataTypes::Vector && MakeVector3Op == EPCGMetadataMakeVector3::Vector2AndValue) ||
		(OutputType == EPCGMetadataTypes::Vector4 &&
			(MakeVector4Op == EPCGMetadataMakeVector4::TwoVector2 || MakeVector4Op == EPCGMetadataMakeVector4::Vector3AndValue)))
	{
		return 2;
	}
	else if (OutputType == EPCGMetadataTypes::Vector ||
		(OutputType == EPCGMetadataTypes::Vector4 && MakeVector4Op == EPCGMetadataMakeVector4::Vector2AndTwoValues))
	{
		return 3;
	}
	else
	{
		return 4;
	}
}

bool UPCGMetadataMakeVectorSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	// Use labels since the logic is already done there.
	bHasSpecialRequirement = false;
	FName Label = GetInputPinLabel(InputIndex);

	if (Label == PCGMetadataMakeVectorConstants::XYZLabel)
	{
		return PCG::Private::IsOfTypes<FVector, FVector2D, float, double, int32, int64>(TypeId);
	}
	else if (Label == PCGMetadataMakeVectorConstants::XYLabel || Label == PCGMetadataMakeVectorConstants::ZWLabel)
	{
		return PCG::Private::IsOfTypes<FVector2D, float, double, int32, int64>(TypeId);
	}
	else
	{
		return PCG::Private::IsOfTypes<float, double, int32, int64>(TypeId);
	}
}

FPCGAttributePropertyInputSelector UPCGMetadataMakeVectorSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	case 2:
		return InputSource3;
	case 3:
		return InputSource4;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

uint16 UPCGMetadataMakeVectorSettings::GetOutputType(uint16 InputTypeId) const
{
	return (uint16)OutputType;
}

#if WITH_EDITOR
FName UPCGMetadataMakeVectorSettings::GetDefaultNodeName() const
{
	return TEXT("MakeVectorAttribute");
}

FText UPCGMetadataMakeVectorSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataMakeVectorSettings", "NodeTitle", "Make Vector Attribute");
}

void UPCGMetadataMakeVectorSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	// Supported default values on all pins
	if (InOutNode && GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGInlineConstantDefaultValues)
	{
		for (const UPCGPin* Pin : InputPins)
		{
			if (IsPinDefaultValueEnabled(Pin->Properties.Label) && !Pin->IsConnected())
			{
				SetPinDefaultValueIsActivated(Pin->Properties.Label, /*bIsActivated=*/true, /*bDirtySettings=*/false);
			}
		}
	}
}

FString UPCGMetadataMakeVectorSettings::GetPinInitialDefaultValueString(const FName PinLabel) const
{
	if (PinLabel == PCGMetadataMakeVectorConstants::XYZLabel)
	{
		return PCG::Private::MetadataTraits<FVector>::ZeroValueString();
	}
	else if (PinLabel == PCGMetadataMakeVectorConstants::XYLabel || PinLabel == PCGMetadataMakeVectorConstants::ZWLabel)
	{
		return PCG::Private::MetadataTraits<FVector2D>::ZeroValueString();
	}
	else
	{
		return PCG::Private::MetadataTraits<double>::ZeroValueString();
	}
}
#endif // WITH_EDITOR

EPCGMetadataTypes UPCGMetadataMakeVectorSettings::GetPinInitialDefaultValueType(FName PinLabel) const
{
	if (PinLabel == PCGMetadataMakeVectorConstants::XYZLabel)
	{
		return EPCGMetadataTypes::Vector;
	}
	else if (PinLabel == PCGMetadataMakeVectorConstants::XYLabel || PinLabel == PCGMetadataMakeVectorConstants::ZWLabel)
	{
		return EPCGMetadataTypes::Vector2;
	}
	else
	{
		return EPCGMetadataTypes::Double;
	}
}

FPCGElementPtr UPCGMetadataMakeVectorSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataMakeVectorElement>();
}

bool FPCGMetadataMakeVectorElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataMakeVectorElement::Execute);

	const UPCGMetadataMakeVectorSettings* Settings = CastChecked<UPCGMetadataMakeVectorSettings>(OperationData.Settings);

	if (Settings->OutputType == EPCGMetadataTypes::Vector2)
	{
		return DoBinaryOp<double, double>(OperationData, PCGMetadataMakeVectorSettings::MakeVector2);
	}
	else if (Settings->OutputType == EPCGMetadataTypes::Vector)
	{
		if (Settings->MakeVector3Op == EPCGMetadataMakeVector3::ThreeValues)
		{
			return DoTernaryOp<double, double, double>(OperationData, PCGMetadataMakeVectorSettings::MakeVector3);
		}
		else
		{
			return DoBinaryOp<FVector2D, double>(OperationData, PCGMetadataMakeVectorSettings::MakeVector3Vec2);
		}
	}
	else
	{
		switch (Settings->MakeVector4Op)
		{
		case EPCGMetadataMakeVector4::FourValues:
			return DoQuaternaryOp<double, double, double, double>(OperationData, PCGMetadataMakeVectorSettings::MakeVector4);
		case EPCGMetadataMakeVector4::TwoVector2:
			return DoBinaryOp<FVector2D, FVector2D>(OperationData, PCGMetadataMakeVectorSettings::MakeVector4TwoVec2);
		case EPCGMetadataMakeVector4::Vector2AndTwoValues:
			return DoTernaryOp<FVector2D, double, double>(OperationData, PCGMetadataMakeVectorSettings::MakeVector4Vec2);
		case EPCGMetadataMakeVector4::Vector3AndValue:
			return DoBinaryOp<FVector, double>(OperationData, PCGMetadataMakeVectorSettings::MakeVector4Vec3);
		default:
			ensure(false);
			return true;
		}
	}
}
