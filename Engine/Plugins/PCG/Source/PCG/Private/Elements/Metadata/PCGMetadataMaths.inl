// Copyright Epic Games, Inc. All Rights Reserved.

namespace PCGMetadataMaths
{
	////////////////////
	// Sign
	////////////////////
	template <typename T>
	inline T Sign(const T& Value)
	{
		double Temp(Value);
		return Temp > 0.0 ? static_cast<T>(1) : (Temp < 0.0 ? static_cast<T>(-1) : static_cast<T>(0));
	}

	template<>
	inline FVector2D Sign(const FVector2D& Value)
	{
		return FVector2D(Sign(Value.X), Sign(Value.Y));
	}

	template<>
	inline FVector Sign(const FVector& Value)
	{
		return FVector(Sign(Value.X), Sign(Value.Y), Sign(Value.Z));
	}

	template<>
	inline FVector4 Sign(const FVector4& Value)
	{
		return FVector4(Sign(Value.X), Sign(Value.Y), Sign(Value.Z), Sign(Value.W));
	}

	////////////////////
	// Frac
	////////////////////
	template <typename T>
	inline T Frac(const T& Value)
	{
		return static_cast<T>(FMath::Frac(static_cast<double>(Value)));
	}

	template<>
	inline FVector2D Frac(const FVector2D& Value)
	{
		return FVector2D(Frac(Value.X), Frac(Value.Y));
	}

	template<>
	inline FVector Frac(const FVector& Value)
	{
		return FVector(Frac(Value.X), Frac(Value.Y), Frac(Value.Z));
	}

	template<>
	inline FVector4 Frac(const FVector4& Value)
	{
		return FVector4(Frac(Value.X), Frac(Value.Y), Frac(Value.Z), Frac(Value.W));
	}

	////////////////////
	// Truncate
	////////////////////
	template <typename T>
	inline T Truncate(const T& Value)
	{
		return static_cast<T>(FMath::TruncToDouble(static_cast<double>(Value)));
	}

	template<>
	inline FVector2D Truncate(const FVector2D& Value)
	{
		return FVector2D(Truncate(Value.X), Truncate(Value.Y));
	}

	template<>
	inline FVector Truncate(const FVector& Value)
	{
		return FVector(Truncate(Value.X), Truncate(Value.Y), Truncate(Value.Z));
	}

	template<>
	inline FVector4 Truncate(const FVector4& Value)
	{
		return FVector4(Truncate(Value.X), Truncate(Value.Y), Truncate(Value.Z), Truncate(Value.W));
	}

	////////////////////
	// Round
	////////////////////
	template <typename T>
	inline T Round(const T& Value)
	{
		return static_cast<T>(FMath::RoundToDouble(static_cast<double>(Value)));
	}

	template<>
	inline FVector2D Round(const FVector2D& Value)
	{
		return FVector2D(Round(Value.X), Round(Value.Y));
	}

	template<>
	inline FVector Round(const FVector& Value)
	{
		return FVector(Round(Value.X), Round(Value.Y), Round(Value.Z));
	}

	template<>
	inline FVector4 Round(const FVector4& Value)
	{
		return FVector4(Round(Value.X), Round(Value.Y), Round(Value.Z), Round(Value.W));
	}

	////////////////////
	// Floor
	////////////////////
	template <typename T>
	inline T Floor(const T& Value)
	{
		return static_cast<T>(FMath::FloorToDouble(static_cast<double>(Value)));
	}

	template<>
	inline FVector2D Floor(const FVector2D& Value)
	{
		return FVector2D(Floor(Value.X), Floor(Value.Y));
	}

	template<>
	inline FVector Floor(const FVector& Value)
	{
		return FVector(Floor(Value.X), Floor(Value.Y), Floor(Value.Z));
	}

	template<>
	inline FVector4 Floor(const FVector4& Value)
	{
		return FVector4(Floor(Value.X), Floor(Value.Y), Floor(Value.Z), Floor(Value.W));
	}

	////////////////////
	// Ceil
	////////////////////
	template <typename T>
	inline T Ceil(const T& Value)
	{
		return static_cast<T>(FMath::CeilToDouble(static_cast<double>(Value)));
	}

	template<>
	inline FVector2D Ceil(const FVector2D& Value)
	{
		return FVector2D(Ceil(Value.X), Ceil(Value.Y));
	}

	template<>
	inline FVector Ceil(const FVector& Value)
	{
		return FVector(Ceil(Value.X), Ceil(Value.Y), Ceil(Value.Z));
	}

	template<>
	inline FVector4 Ceil(const FVector4& Value)
	{
		return FVector4(Ceil(Value.X), Ceil(Value.Y), Ceil(Value.Z), Ceil(Value.W));
	}

	////////////////////
	// Sqrt
	////////////////////
	template <typename T>
	inline T Sqrt(const T& Value)
	{
		return static_cast<T>(FMath::Sqrt(static_cast<double>(Value)));
	}

	template<>
	inline FVector2D Sqrt(const FVector2D& Value)
	{
		return FVector2D(Sqrt(Value.X), Sqrt(Value.Y));
	}

	template<>
	inline FVector Sqrt(const FVector& Value)
	{
		return FVector(Sqrt(Value.X), Sqrt(Value.Y), Sqrt(Value.Z));
	}

	template<>
	inline FVector4 Sqrt(const FVector4& Value)
	{
		return FVector4(Sqrt(Value.X), Sqrt(Value.Y), Sqrt(Value.Z), Sqrt(Value.W));
	}

	////////////////////
	// Abs
	////////////////////
	template <typename T>
	inline T Abs(const T& Value)
	{
		return FMath::Abs(Value);
	}

	template<>
	inline FVector2D Abs(const FVector2D& Value)
	{
		return Value.GetAbs();
	}

	template<>
	inline FVector Abs(const FVector& Value)
	{
		return Value.GetAbs();
	}

	template<>
	inline FVector4 Abs(const FVector4& Value)
	{
		return FVector4(FMath::Abs(Value.X), FMath::Abs(Value.Y), FMath::Abs(Value.Z), FMath::Abs(Value.W));
	}

	////////////////////
	// Max
	////////////////////
	template <typename T>
	inline T Max(const T& Value1, const T& Value2)
	{
		return FMath::Max(Value1, Value2);
	}

	template<>
	inline FVector2D Max(const FVector2D& Value1, const FVector2D& Value2)
	{
		return FVector2D::Max(Value1, Value2);
	}

	template<>
	inline FVector Max(const FVector& Value1, const FVector& Value2)
	{
		return FVector::Max(Value1, Value2);
	}

	template<>
	inline FVector4 Max(const FVector4& Value1, const FVector4& Value2)
	{
		return FVector4(FMath::Max(Value1.X, Value2.X), FMath::Max(Value1.Y, Value2.Y), FMath::Max(Value1.Z, Value2.Z), FMath::Max(Value1.W, Value2.W));
	}

	////////////////////
	// Min
	////////////////////
	template <typename T>
	inline T Min(const T& Value1, const T& Value2)
	{
		return FMath::Min(Value1, Value2);
	}

	template<>
	inline FVector2D Min(const FVector2D& Value1, const FVector2D& Value2)
	{
		return FVector2D::Min(Value1, Value2);
	}

	template<>
	inline FVector Min(const FVector& Value1, const FVector& Value2)
	{
		return FVector::Min(Value1, Value2);
	}

	template<>
	inline FVector4 Min(const FVector4& Value1, const FVector4& Value2)
	{
		return FVector4(FMath::Min(Value1.X, Value2.X), FMath::Min(Value1.Y, Value2.Y), FMath::Min(Value1.Z, Value2.Z), FMath::Min(Value1.W, Value2.W));
	}

	////////////////////
	// Pow
	////////////////////
	template <typename T>
	inline T Pow(const T& Value, const T& Power)
	{
		return static_cast<T>(FMath::Pow(static_cast<double>(Value), static_cast<double>(Power)));
	}

	template<>
	inline FVector2D Pow(const FVector2D& Value, const FVector2D& Power)
	{
		return FVector2D(FMath::Pow(Value.X, Power.X), FMath::Pow(Value.Y, Power.Y));
	}

	template<>
	inline FVector Pow(const FVector& Value, const FVector& Power)
	{
		return FVector(FMath::Pow(Value.X, Power.X), FMath::Pow(Value.Y, Power.Y), FMath::Pow(Value.Z, Power.Z));
	}

	template<>
	inline FVector4 Pow(const FVector4& Value, const FVector4& Power)
	{
		return FVector4(FMath::Pow(Value.X, Power.X), FMath::Pow(Value.Y, Power.Y), FMath::Pow(Value.Z, Power.Z), FMath::Pow(Value.W, Power.W));
	}

	////////////////////
	// Clamp
	////////////////////
	template <typename T>
	inline T Clamp(const T& Value, const T& MinValue, const T& MaxValue)
	{
		return FMath::Clamp(Value, MinValue, MaxValue);
	}

	template<>
	inline FVector2D Clamp(const FVector2D& Value, const FVector2D& MinValue, const FVector2D& MaxValue)
	{
		return FVector2D(Clamp(Value.X, MinValue.X, MaxValue.X), Clamp(Value.Y, MinValue.Y, MaxValue.Y));
	}

	template<>
	inline FVector Clamp(const FVector& Value, const FVector& MinValue, const FVector& MaxValue)
	{
		return Value.BoundToBox(MinValue, MaxValue);
	}

	template<>
	inline FVector4 Clamp(const FVector4& Value, const FVector4& MinValue, const FVector4& MaxValue)
	{
		return FVector4(
			Clamp(Value.X, MinValue.X, MaxValue.X),
			Clamp(Value.Y, MinValue.Y, MaxValue.Y),
			Clamp(Value.Z, MinValue.Z, MaxValue.Z),
			Clamp(Value.W, MinValue.W, MaxValue.W));
	}

	////////////////////
	// Lerp
	////////////////////
	template <typename T, typename U>
	inline T Lerp(const T& Value1, const T& Value2, const U& Ratio)
	{
		return static_cast<T>(FMath::Lerp(Value1, Value2, Ratio));
	}

	template<typename U>
	inline FVector2D Lerp(const FVector2D& Value1, const FVector2D& Value2, const U& Ratio)
	{
		return FVector2D(
			Lerp(Value1.X, Value2.X, Ratio),
			Lerp(Value1.Y, Value2.Y, Ratio));
	}

	template<>
	inline FVector2D Lerp(const FVector2D& Value1, const FVector2D& Value2, const FVector2D& Ratio)
	{
		return FVector2D(
			Lerp(Value1.X, Value2.X, Ratio.X),
			Lerp(Value1.Y, Value2.Y, Ratio.Y));
	}

	template<typename U>
	inline FVector Lerp(const FVector& Value1, const FVector& Value2, const U& Ratio)
	{
		return FVector(
			Lerp(Value1.X, Value2.X, Ratio),
			Lerp(Value1.Y, Value2.Y, Ratio),
			Lerp(Value1.Z, Value2.Z, Ratio));
	}

	template<>
	inline FVector Lerp(const FVector& Value1, const FVector& Value2, const FVector& Ratio)
	{
		return FVector(
			Lerp(Value1.X, Value2.X, Ratio.X), 
			Lerp(Value1.Y, Value2.Y, Ratio.Y), 
			Lerp(Value1.Z, Value2.Z, Ratio.Z));
	}

	template<typename U>
	inline FVector4 Lerp(const FVector4& Value1, const FVector4& Value2, const U& Ratio)
	{
		return FVector4(
			Lerp(Value1.X, Value2.X, Ratio),
			Lerp(Value1.Y, Value2.Y, Ratio),
			Lerp(Value1.Z, Value2.Z, Ratio),
			Lerp(Value1.W, Value2.W, Ratio));
	}

	template<>
	inline FVector4 Lerp(const FVector4& Value1, const FVector4& Value2, const FVector4& Ratio)
	{
		return FVector4(
			Lerp(Value1.X, Value2.X, Ratio.X),
			Lerp(Value1.Y, Value2.Y, Ratio.Y),
			Lerp(Value1.Z, Value2.Z, Ratio.Z),
			Lerp(Value1.W, Value2.W, Ratio.W));
	}

	////////////////////
	// Modulo
	////////////////////
	template <typename T>
	inline T Modulo(const T& Value1, const T& Value2)
	{
		return (Value2 != 0) ? (Value1 % Value2) : 0;
	}

	template<>
	inline float Modulo(const float& Value1, const float& Value2)
	{
		return static_cast<float>(Modulo(FMath::RoundToInt(Value1), FMath::RoundToInt(Value2)));
	}

	template<>
	inline double Modulo(const double& Value1, const double& Value2)
	{
		return static_cast<double>(Modulo(FMath::RoundToInt(Value1), FMath::RoundToInt(Value2)));
	}

	template<>
	inline FVector2D Modulo(const FVector2D& Value1, const FVector2D& Value2)
	{
		return FVector2D(
			Modulo(Value1.X, Value2.X),
			Modulo(Value1.Y, Value2.Y));
	}

	template<>
	inline FVector Modulo(const FVector& Value1, const FVector& Value2)
	{
		return FVector(
			Modulo(Value1.X, Value2.X),
			Modulo(Value1.Y, Value2.Y),
			Modulo(Value1.Z, Value2.Z));
	}

	template<>
	inline FVector4 Modulo(const FVector4& Value1, const FVector4& Value2)
	{
		return FVector4(
			Modulo(Value1.X, Value2.X),
			Modulo(Value1.Y, Value2.Y),
			Modulo(Value1.Z, Value2.Z),
			Modulo(Value1.W, Value2.W));
	}

	////////////////////
	// One Minus
	////////////////////
	template <typename T>
	inline T OneMinus(const T& Value)
	{
		return T{ 1 } - Value;
	}

	template<>
	inline FVector2D OneMinus(const FVector2D& Value)
	{
		return FVector2D::One() - Value;
	}

	template<>
	inline FVector OneMinus(const FVector& Value)
	{
		return FVector::One() - Value;
	}

	template<>
	inline FVector4 OneMinus(const FVector4& Value)
	{
		return FVector4::One() - Value;
	}

	////////////////////
	// Inc/Dec
	////////////////////
	template <typename T>
	inline T Inc(const T& Value)
	{
		return Value + T{1};
	}

	template <>
	inline FVector4 Inc(const FVector4& Value)
	{
		return Value + FVector4::One();
	}

	template <typename T>
	inline T Dec(const T& Value)
	{
		return Value - T{1};
	}

	template <>
	inline FVector4 Dec(const FVector4& Value)
	{
		return Value - FVector4::One();
	}

	////////////////////
	// Negate
	////////////////////
	template <typename T>
	inline T Negate(const T& Value)
	{
		return -Value;
	}

	////////////////////
	// AddModulo
	////////////////////
	template <typename T>
	inline T AddModulo(const T& Value1, const T& Value2, const T& Value3)
	{
		// If Result is negative, add Value3. (i.e. A=0, B=-1, C=5 Result=-1, returns 4.
		const T Result = Modulo(Value1 + Value2, Value3);
		return Result >= 0 ? Result : (Result + Value3);
	}

	template<>
	inline FVector2D AddModulo(const FVector2D& Value1, const FVector2D& Value2, const FVector2D& Value3)
	{
		return FVector2D(
			AddModulo(Value1.X, Value2.X, Value3.X),
			AddModulo(Value1.Y, Value2.Y, Value3.X));
	}

	template<>
	inline FVector AddModulo(const FVector& Value1, const FVector& Value2, const FVector& Value3)
	{
		return FVector(
			AddModulo(Value1.X, Value2.X, Value3.X),
			AddModulo(Value1.Y, Value2.Y, Value3.Y),
			AddModulo(Value1.Z, Value2.Z, Value3.Z));
	}

	template<>
	inline FVector4 AddModulo(const FVector4& Value1, const FVector4& Value2, const FVector4& Value3)
	{
		return FVector4(
			AddModulo(Value1.X, Value2.X, Value3.X),
			AddModulo(Value1.Y, Value2.Y, Value3.Y),
			AddModulo(Value1.Z, Value2.Z, Value3.Z),
			AddModulo(Value1.W, Value2.W, Value3.W));
	}
}
