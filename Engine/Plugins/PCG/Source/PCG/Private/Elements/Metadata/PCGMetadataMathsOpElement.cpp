// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataMathsOpElement.h"

#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include "Elements/Metadata/PCGMetadataMaths.inl"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataMathsOpElement)

namespace PCGMetadataMathsSettings
{
	inline constexpr bool IsUnaryOp(EPCGMetadataMathsOperation Operation)
	{
		return !!(Operation & EPCGMetadataMathsOperation::UnaryOp);
	}

	inline constexpr bool IsBinaryOp(EPCGMetadataMathsOperation Operation)
	{
		return !!(Operation & EPCGMetadataMathsOperation::BinaryOp);
	}

	inline constexpr bool IsTernaryOp(EPCGMetadataMathsOperation Operation)
	{
		return !!(Operation & EPCGMetadataMathsOperation::TernaryOp);
	}

	inline FName GetFirstPinLabel(EPCGMetadataMathsOperation Operation)
	{
		if (PCGMetadataMathsSettings::IsUnaryOp(Operation)
			|| Operation == EPCGMetadataMathsOperation::Clamp
			|| Operation == EPCGMetadataMathsOperation::ClampMin
			|| Operation == EPCGMetadataMathsOperation::ClampMax)
		{
			return PCGPinConstants::DefaultInputLabel;
		}

		if (PCGMetadataMathsSettings::IsBinaryOp(Operation)
			|| Operation == EPCGMetadataMathsOperation::Lerp
			|| Operation == EPCGMetadataMathsOperation::MulAdd
			|| Operation == EPCGMetadataMathsOperation::AddModulo)
		{
			return PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
		}

		return NAME_None;
	}

	inline FName GetSecondPinLabel(EPCGMetadataMathsOperation Operation)
	{
		if (Operation == EPCGMetadataMathsOperation::ClampMin || Operation == EPCGMetadataMathsOperation::Clamp)
		{
			return PCGMetadataSettingsBaseConstants::ClampMinLabel;
		}

		if (Operation == EPCGMetadataMathsOperation::ClampMax)
		{
			return PCGMetadataSettingsBaseConstants::ClampMaxLabel;
		}

		if (PCGMetadataMathsSettings::IsBinaryOp(Operation) || PCGMetadataMathsSettings::IsTernaryOp(Operation))
		{
			return PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
		}

		return NAME_None;
	}

	inline FName GetThirdPinLabel(EPCGMetadataMathsOperation Operation)
	{
		if (Operation == EPCGMetadataMathsOperation::Clamp)
		{
			return PCGMetadataSettingsBaseConstants::ClampMaxLabel;
		}

		if (Operation == EPCGMetadataMathsOperation::Lerp)
		{
			return PCGMetadataSettingsBaseConstants::LerpRatioLabel;
		}

		if (PCGMetadataMathsSettings::IsTernaryOp(Operation))
		{
			return PCGMetadataSettingsBaseConstants::DoubleInputThirdLabel;
		}

		return NAME_None;
	}

	template <typename T>
	T UnaryOp(const T& Value, EPCGMetadataMathsOperation Op)
	{
		switch (Op)
		{
		case EPCGMetadataMathsOperation::Sign:
			return PCGMetadataMaths::Sign(Value);
		case EPCGMetadataMathsOperation::Frac:
			return PCGMetadataMaths::Frac(Value);
		case EPCGMetadataMathsOperation::Truncate:
			return PCGMetadataMaths::Truncate(Value);
		case EPCGMetadataMathsOperation::Round:
			return PCGMetadataMaths::Round(Value);
		case EPCGMetadataMathsOperation::Sqrt:
			return PCGMetadataMaths::Sqrt(Value);
		case EPCGMetadataMathsOperation::Abs:
			return PCGMetadataMaths::Abs(Value);
		case EPCGMetadataMathsOperation::Floor:
			return PCGMetadataMaths::Floor(Value);
		case EPCGMetadataMathsOperation::Ceil:
			return PCGMetadataMaths::Ceil(Value);
		case EPCGMetadataMathsOperation::OneMinus:
			return PCGMetadataMaths::OneMinus(Value);
		case EPCGMetadataMathsOperation::Inc:
			return PCGMetadataMaths::Inc(Value);
		case EPCGMetadataMathsOperation::Dec:
			return PCGMetadataMaths::Dec(Value);
		case EPCGMetadataMathsOperation::Negate:
			return PCGMetadataMaths::Negate(Value);
		default:
			return T{};
		}
	}

	template <typename T>
	T BinaryOp(const T& Value1, const T& Value2, EPCGMetadataMathsOperation Op)
	{
		switch (Op)
		{
		case EPCGMetadataMathsOperation::Add:
			return Value1 + Value2;
		case EPCGMetadataMathsOperation::Subtract:
			return Value1 - Value2;
		case EPCGMetadataMathsOperation::Multiply:
			return Value1 * Value2;
		case EPCGMetadataMathsOperation::Divide:
		{
			static const T ZeroValue = PCG::Private::MetadataTraits<T>::ZeroValue();
			return !PCG::Private::MetadataTraits<T>::Equal(Value2, ZeroValue) ? (Value1 / Value2) : ZeroValue; // To mirror FMath
		}
		case EPCGMetadataMathsOperation::Max: // fall-through
		case EPCGMetadataMathsOperation::ClampMin:
			return PCGMetadataMaths::Max(Value1, Value2);
		case EPCGMetadataMathsOperation::Min: // fall-through
		case EPCGMetadataMathsOperation::ClampMax:
			return PCGMetadataMaths::Min(Value1, Value2);
		case EPCGMetadataMathsOperation::Pow:
			return PCGMetadataMaths::Pow(Value1, Value2);
		case EPCGMetadataMathsOperation::Modulo:
			return PCGMetadataMaths::Modulo(Value1, Value2);
		default:
			return T{};
		}
	}

	template <typename T>
	T TernaryOp(const T& Value1, const T& Value2, const T& Value3, EPCGMetadataMathsOperation Op)
	{
		switch (Op)
		{
		case EPCGMetadataMathsOperation::Clamp:
			return PCGMetadataMaths::Clamp(Value1, Value2, Value3);
		case EPCGMetadataMathsOperation::Lerp:
			return PCGMetadataMaths::Lerp(Value1, Value2, Value3);
		case EPCGMetadataMathsOperation::MulAdd:
			return Value1 + Value2 * Value3;
		case EPCGMetadataMathsOperation::AddModulo:
			return PCGMetadataMaths::AddModulo(Value1, Value2, Value3);
		default:
			return T{};
		}
	}

	// Specialize bool to int32, as some math operations won't compile with boolean values.
	template <>
	bool UnaryOp(const bool& Value, EPCGMetadataMathsOperation Op)
	{
		return static_cast<bool>(UnaryOp(static_cast<int32>(Value), Op));
	}

	template <>
	bool BinaryOp(const bool& Value1, const bool& Value2, EPCGMetadataMathsOperation Op)
	{
		return static_cast<bool>(BinaryOp(static_cast<int32>(Value1), static_cast<int32>(Value2), Op));
	}

	template <>
	bool TernaryOp(const bool& Value1, const bool& Value2, const bool& Value3, EPCGMetadataMathsOperation Op)
	{
		return static_cast<bool>(TernaryOp(static_cast<int32>(Value1), static_cast<int32>(Value2), static_cast<int32>(Value3), Op));
	}
}

void UPCGMetadataMathsSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (Input1AttributeName_DEPRECATED != NAME_None)
	{
		InputSource1.SetAttributeName(Input1AttributeName_DEPRECATED);
		Input1AttributeName_DEPRECATED = NAME_None;
	}

	if (Input2AttributeName_DEPRECATED != NAME_None)
	{
		InputSource2.SetAttributeName(Input2AttributeName_DEPRECATED);
		Input2AttributeName_DEPRECATED = NAME_None;
	}

	if (Input3AttributeName_DEPRECATED != NAME_None)
	{
		InputSource3.SetAttributeName(Input3AttributeName_DEPRECATED);
		Input3AttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

FName UPCGMetadataMathsSettings::GetInputPinLabel(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return PCGMetadataMathsSettings::GetFirstPinLabel(Operation);
	case 1:
		return PCGMetadataMathsSettings::GetSecondPinLabel(Operation);
	case 2:
		return PCGMetadataMathsSettings::GetThirdPinLabel(Operation);
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataMathsSettings::GetOperandNum() const
{
	if (PCGMetadataMathsSettings::IsUnaryOp(Operation))
	{
		return 1;
	}

	if (PCGMetadataMathsSettings::IsBinaryOp(Operation))
	{
		return 2;
	}

	if (PCGMetadataMathsSettings::IsTernaryOp(Operation))
	{
		return 3;
	}

	return 0;
}

// By default: Float/Double, Int32/Int64, Vector2, Vector, Vector4
bool UPCGMetadataMathsSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;

	if (Operation == EPCGMetadataMathsOperation::Set)
	{
		return PCG::Private::IsPCGType(TypeId);
	}
	else
	{
		return PCG::Private::IsOfTypes<bool, float, double, int32, int64, FVector2D, FVector, FVector4>(TypeId);
	}
}

bool UPCGMetadataMathsSettings::ShouldForceOutputToInt(uint16 InputTypeId) const
{
	return PCG::Private::IsOfTypes<float, double>(InputTypeId) && bForceRoundingOpToInt &&
		(Operation == EPCGMetadataMathsOperation::Round ||
		Operation == EPCGMetadataMathsOperation::Truncate ||
		Operation == EPCGMetadataMathsOperation::Floor ||
		Operation == EPCGMetadataMathsOperation::Ceil);
}

bool UPCGMetadataMathsSettings::ShouldForceOutputToDouble(uint16 InputTypeId) const
{
	return PCG::Private::IsOfTypes<int32, int64>(InputTypeId) && bForceOpToDouble &&
		(Operation == EPCGMetadataMathsOperation::Divide ||
		Operation == EPCGMetadataMathsOperation::Sqrt ||
		Operation == EPCGMetadataMathsOperation::Pow ||
		Operation == EPCGMetadataMathsOperation::Lerp);
}

uint16 UPCGMetadataMathsSettings::GetOutputType(uint16 InputTypeId) const
{
	// If attribute Type is a float or double, can convert to int if it is a rounding op.
	if (ShouldForceOutputToInt(InputTypeId))
	{
		return PCG::Private::MetadataTypes<int64>::Id;
	}
	// If attribute Type is an integer, can convert to double if it is an operation that can yield a floating point value.
	else if (ShouldForceOutputToDouble(InputTypeId))
	{
		return PCG::Private::MetadataTypes<double>::Id;
	}
	else
	{
		return InputTypeId;
	}
}

FPCGAttributePropertyInputSelector UPCGMetadataMathsSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	case 2:
		return InputSource3;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

FString UPCGMetadataMathsSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataMathsOperation>())
	{
		return EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation)).ToString();
	}
	else
	{
		return FString();
	}
}

#if WITH_EDITOR
bool UPCGMetadataMathsSettings::IsPinDefaultValueMetadataTypeValid(const FName PinLabel, const EPCGMetadataTypes DataType) const
{
	return (Operation == EPCGMetadataMathsOperation::Set) || Super::IsPinDefaultValueMetadataTypeValid(PinLabel, DataType);
}

FName UPCGMetadataMathsSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeMathsOp");
}

FText UPCGMetadataMathsSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataMathsSettings", "NodeTitle", "Attribute Maths Op");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataMathsSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataMathsOperation>({ EPCGMetadataMathsOperation::UnaryOp, EPCGMetadataMathsOperation::BinaryOp, EPCGMetadataMathsOperation::TernaryOp });
}
#endif // WITH_EDITOR

void UPCGMetadataMathsSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataMathsOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataMathsOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

FPCGElementPtr UPCGMetadataMathsSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataMathsElement>();
}

bool FPCGMetadataMathsElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataMathsElement::Execute);

	const UPCGMetadataMathsSettings* Settings = CastChecked<UPCGMetadataMathsSettings>(OperationData.Settings);

	auto MathFunc = [this, Operation = Settings->Operation, &OperationData]<typename AttributeType>(AttributeType) -> bool
	{
		if constexpr (PCG::Private::IsPCGType<AttributeType>()) // Types that wouldn't compile with operations. Only works with Set operation.
		{
			if (Operation == EPCGMetadataMathsOperation::Set)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataMathsElement::ExecuteInternal::BinaryOp);
				return DoBinaryOp<AttributeType, AttributeType>(OperationData, [](const AttributeType&, const AttributeType& Value2) -> AttributeType { return Value2; });
			}
		}

		if constexpr (PCG::Private::IsOfTypes<AttributeType, bool, float, double, int32, int64, FVector2D, FVector, FVector4>())
		{
			if (PCGMetadataMathsSettings::IsUnaryOp(Operation))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataMathsElement::ExecuteInternal::UnaryOp);
				// For int64 as output of the lambda if the output type is int64, as AttributeType might be different (cf GetOutputType)
				using OverriddenOutputType = typename std::conditional_t<PCG::Private::IsOfTypes<AttributeType, float, double>(), int64, AttributeType>;

				if (OperationData.OutputType == PCG::Private::MetadataTypes<int64>::Id)
				{
					return DoUnaryOp<AttributeType>(OperationData, [Operation](const AttributeType& Value) -> OverriddenOutputType { return static_cast<OverriddenOutputType>(PCGMetadataMathsSettings::UnaryOp<AttributeType>(Value, Operation)); });
				}
				else
				{
					return DoUnaryOp<AttributeType>(OperationData, [Operation](const AttributeType& Value) -> AttributeType { return PCGMetadataMathsSettings::UnaryOp(Value, Operation); });
				}
			}
			else if (PCGMetadataMathsSettings::IsBinaryOp(Operation))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataMathsElement::ExecuteInternal::BinaryOp);
				return DoBinaryOp<AttributeType, AttributeType>(OperationData, [Operation](const AttributeType& Value1, const AttributeType& Value2) -> AttributeType { return PCGMetadataMathsSettings::BinaryOp(Value1, Value2, Operation); });
			}
			else if (PCGMetadataMathsSettings::IsTernaryOp(Operation))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataMathsElement::ExecuteInternal::TernaryOp);
				return DoTernaryOp<AttributeType, AttributeType, AttributeType>(OperationData, [Operation](const AttributeType& Value1, const AttributeType& Value2, const AttributeType& Value3) -> AttributeType { return PCGMetadataMathsSettings::TernaryOp(Value1, Value2, Value3, Operation); });
			}
			else
			{
				ensure(false);
				return true;
			}
		}
		else // Some other type not supported
		{
			ensure(false);
			return true;
		}
	};

	// If the output is double, force all to double.
	if (OperationData.OutputType == PCG::Private::MetadataTypes<double>::Id)
	{
		return MathFunc(double{});
	}
	else
	{
		return PCGMetadataAttribute::CallbackWithRightType(OperationData.MostComplexInputType, MathFunc);
	}
}
