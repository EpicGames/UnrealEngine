// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Metadata/PCGMetadataTypesConstantStruct.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataOpElementBase)

#define LOCTEXT_NAMESPACE "PCGMetadataElementBaseElement"

namespace PCGMetadataBase
{
	TAutoConsoleVariable<bool> CVarMetadataOperationInMT(
		TEXT("pcg.MetadataOperationInMT"),
		true,
		TEXT("Metadata operations are now multithreaded."));

	TAutoConsoleVariable<int> CVarMetadataOperationChunkSize(
		TEXT("pcg.MetadataOperationChunkSize"),
		256,
		TEXT("Metadata operations chunk size."));

	TAutoConsoleVariable<bool> CVarMetadataOperationReserveValues(
		TEXT("pcg.MetadataOperationReserveValues"),
		true,
		TEXT("Metadata operations reserve values."));

	namespace Helpers
	{
		FName GetInputSourceName(const int32 Index)
		{
			static const TStaticArray<FName, UPCGMetadataSettingsBase::MaxNumberOfInputs> InputSourceNames = {TEXT("InputSource1"), TEXT("InputSource2"), TEXT("InputSource3"), TEXT("InputSource4")};
			const bool bValidIndex = Index > INDEX_NONE && Index < InputSourceNames.Num();
			return ensure(bValidIndex) ? InputSourceNames[Index] : NAME_None;
		}

		FName GetDefaultValuePropertyName(const int32 Index)
		{
			static const TStaticArray<FName, UPCGMetadataSettingsBase::MaxNumberOfInputs> InputDefaultValuePropertyNames = {TEXT("DefaultValue1"), TEXT("DefaultValue2"), TEXT("DefaultValue3"), TEXT("DefaultValue4")};
			const bool bValidIndex = Index > INDEX_NONE && Index < InputDefaultValuePropertyNames.Num();
			return ensure(bValidIndex) ? InputDefaultValuePropertyNames[Index] : NAME_None;
		}
	}
}

void UPCGMetadataSettingsBase::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (OutputAttributeName_DEPRECATED != NAME_None)
	{
		OutputTarget.SetAttributeName(OutputAttributeName_DEPRECATED);
		OutputAttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

EPCGDataType UPCGMetadataSettingsBase::GetInputPinType(uint32 Index) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetInputPinTypeID(Index);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FPCGDataTypeIdentifier UPCGMetadataSettingsBase::GetInputPinTypeID(uint32 Index) const
{
	const FName PinLabel = GetInputPinLabel(Index);
	const FPCGDataTypeIdentifier FirstPinTypeUnion = GetTypeUnionIDOfIncidentEdges(PinLabel);

	// If the pin is not connected but supports default values, treat it as a param
	if (FirstPinTypeUnion == EPCGDataType::None && IsPinDefaultValueActivated(PinLabel))
	{
		return FPCGDataTypeIdentifier{EPCGDataType::Param};
	}

	return FirstPinTypeUnion.IsValid() ? FirstPinTypeUnion : FPCGDataTypeIdentifier{EPCGDataType::Any};
}

TArray<FName> UPCGMetadataSettingsBase::GetOutputDataFromPinOptions() const
{
	const uint32 OperandNum = GetOperandNum();

	TArray<FName> AllOptions;
	AllOptions.SetNum(OperandNum + 1);
	AllOptions[0] = PCGMetadataSettingsBaseConstants::DefaultOutputDataFromPinName;

	for (uint32 Index = 0; Index < OperandNum; ++Index)
	{
		AllOptions[Index + 1] = GetInputPinLabel(Index);
	}

	return AllOptions;
}

uint32 UPCGMetadataSettingsBase::GetInputPinIndex(const FName InPinLabel) const
{
	if (InPinLabel != PCGMetadataSettingsBaseConstants::DefaultOutputDataFromPinName)
	{
		for (uint32 Index = 0; Index < GetOperandNum(); ++Index)
		{
			if (InPinLabel == GetInputPinLabel(Index))
			{
				return Index;
			}
		}
	}

	return static_cast<uint32>(INDEX_NONE);
}

bool UPCGMetadataSettingsBase::IsPinDefaultValueEnabled(const FName PinLabel) const
{
	return DefaultValuesAreEnabled() && (GetInputPinIndex(PinLabel) != INDEX_NONE) && PCGMetadataHelpers::MetadataTypeSupportsDefaultValues(GetPinDefaultValueType(PinLabel));
}

bool UPCGMetadataSettingsBase::IsPinDefaultValueActivated(const FName PinLabel) const
{
	if (!IsPinDefaultValueEnabled(PinLabel))
	{
		return false;
	}

	const uint32 PinIndex = GetInputPinIndex(PinLabel);
	if (PinIndex != static_cast<uint32>(INDEX_NONE))
	{
		const FName PropertyName = PCGMetadataBase::Helpers::GetDefaultValuePropertyName(PinIndex);
		return DefaultValues.IsPropertyActivated(PropertyName);
	}
	else
	{
		return false;
	}
}

uint32 UPCGMetadataSettingsBase::GetInputPinToForward() const
{
	const uint32 OperandNum = GetOperandNum();
	uint32 InputPinToForward = 0;

	// If there is only one input, it is trivial
	if (OperandNum != 1)
	{
		// Heuristic:
		//	* If OutputDataFromPin is set, use this value
		//	* If there are connected pins, use the first spatial input (not Any)
		//	* Otherwise, take the first pin
		const uint32 OutputDataFromPinIndex = GetInputPinIndex(OutputDataFromPin);
		if (OutputDataFromPinIndex != static_cast<uint32>(INDEX_NONE))
		{
			InputPinToForward = OutputDataFromPinIndex;
		}
		else
		{
			// Implementation note: here we use the pin type scope helper so that all type queries are cached in the scope of this call.
			PCGSettingsHelpers::FPinTypeScopeHelper PinTypeHelper;

			for (uint32 InputPinIndex = 0; InputPinIndex < OperandNum; ++InputPinIndex)
			{
				const FPCGDataTypeIdentifier PinType = GetInputPinTypeID(InputPinIndex);

				if (!PinType.IsSameType(FPCGDataTypeIdentifier{EPCGDataType::Any}) && (PinType & EPCGDataType::Spatial).IsValid())
				{
					InputPinToForward = InputPinIndex;
					break;
				}
			}
		}
	}

	return InputPinToForward;
}

const UPCGParamData* UPCGMetadataSettingsBase::CreateDefaultValueParamData(FPCGContext* Context, const FName PinLabel) const
{
	const FName PropertyName = PCGMetadataBase::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));

	const UPCGParamData* Data = DefaultValues.CreateParamData(Context, PropertyName);

	// Default Value Container did not have a value. Try getting the node's 'reset' default value.
	if (!Data)
	{
		const TObjectPtr<UPCGParamData> NewParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		if (CreateInitialDefaultValueAttribute(PinLabel, NewParamData->Metadata))
		{
			Data = NewParamData;
		}
	}

	return Data;
}

FPCGDataTypeIdentifier UPCGMetadataSettingsBase::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	check(InPin);
	const uint32 OperandNum = GetOperandNum();
	if (!InPin->IsOutputPin() || OperandNum == 0)
	{
		// Fall back to default for input pins, or if no input pins present from which to obtain type
		return Super::GetCurrentPinTypesID(InPin);
	}

	// Output pin narrows to union of inputs on pin to forward
	return GetInputPinTypeID(GetInputPinToForward());
}

#if WITH_EDITOR
void UPCGMetadataSettingsBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Make sure the output data from pin value is always valid. Reset it otherwise.
	if (GetInputPinIndex(OutputDataFromPin) == static_cast<uint32>(INDEX_NONE))
	{
		OutputDataFromPin = PCGMetadataSettingsBaseConstants::DefaultOutputDataFromPinName;
	}
}

bool UPCGMetadataSettingsBase::CanEditChange(const FProperty* InProperty) const
{
	return InProperty
		&& Super::CanEditChange(InProperty)
		&& CanEditInputSource(InProperty, GetOperandNum())
		&& ((InProperty->GetFName() != GET_MEMBER_NAME_CHECKED(UPCGMetadataSettingsBase, OutputDataFromPin)) || (GetOperandNum() != 1));
}

bool UPCGMetadataSettingsBase::GetPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip) const
{
	// Only set the arrow if the output data from pin is forced.
	if (GetOperandNum() > 1 && InPin && OutputDataFromPin == InPin->Properties.Label)
	{
		OutExtraIcon = TEXT("Icons.ArrowRight");
		return true;
	}
	else
	{
		return false;
	}
}

FText UPCGMetadataSettingsBase::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Metadata operation between Points/Spatial/AttributeSet data.\n"
		"Output data will be taken from the first spatial data by default, or first pin if all are attribute sets.\n"
		"It can be overridden in the settings.");
}

void UPCGMetadataSettingsBase::ApplyDeprecation(UPCGNode* InOutNode)
{
	if (DataVersion < FPCGCustomVersion::UpdateAttributePropertyInputSelector
		&& OutputTarget.GetSelection() == EPCGAttributePropertySelection::Attribute
		&& OutputTarget.GetAttributeName() == PCGMetadataAttributeConstants::SourceAttributeName)
	{
		// Previous behavior of the output target for this node was:
		// - If the input to forward is an attribute -> SourceName
		// - If the input to forward was not an attribute -> None
		const FPCGAttributePropertyInputSelector& InputSource = GetInputSource(GetInputPinToForward());
		if (InputSource.GetSelection() == EPCGAttributePropertySelection::Attribute)
		{
			OutputTarget.SetAttributeName(PCGMetadataAttributeConstants::SourceNameAttributeName);
		}
		else
		{
			OutputTarget.SetAttributeName(NAME_None);
		}
	}

	Super::ApplyDeprecation(InOutNode);
}

void UPCGMetadataSettingsBase::SetPinDefaultValue(const FName PinLabel, const FString& DefaultValue, const bool bCreateIfNeeded)
{
	const FName PropertyName = PCGMetadataBase::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
	if (PropertyName != NAME_None)
	{
		Modify();

		if (!DefaultValues.FindProperty(PropertyName) && bCreateIfNeeded)
		{
			const EPCGMetadataTypes Type = GetPinInitialDefaultValueType(PinLabel);
			DefaultValues.CreateNewProperty(PropertyName, Type);
		}

		if (DefaultValues.SetPropertyValueFromString(PropertyName, DefaultValue))
		{
			OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Node | EPCGChangeType::Settings);
		}
	}
}

void UPCGMetadataSettingsBase::ConvertPinDefaultValueMetadataType(const FName PinLabel, const EPCGMetadataTypes DataType)
{
	if (ensure(IsPinDefaultValueActivated(PinLabel)))
	{
		const FName PropertyName = PCGMetadataBase::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
		if (PropertyName != NAME_None && IsPinDefaultValueMetadataTypeValid(PinLabel, DataType))
		{
			Modify();
			DefaultValues.ConvertPropertyType(PropertyName, DataType);
			OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Node | EPCGChangeType::Edge | EPCGChangeType::Settings);
		}
	}
}

void UPCGMetadataSettingsBase::SetPinDefaultValueIsActivated(const FName PinLabel, const bool bIsActivated, const bool bDirtySettings)
{
	if (ensure(IsPinDefaultValueEnabled(PinLabel)))
	{
		const FName PropertyName = PCGMetadataBase::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
		if (DefaultValues.IsPropertyActivated(PropertyName) != bIsActivated)
		{
			if (bDirtySettings)
			{
				Modify();
			}

			DefaultValues.SetPropertyActivated(PropertyName, bIsActivated);
			if (bDirtySettings)
			{
				OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Node | EPCGChangeType::Edge | EPCGChangeType::Settings);
			}
		}
	}
}
#endif // WITH_EDITOR

bool UPCGMetadataSettingsBase::DoesPinSupportPassThrough(UPCGPin* InPin) const
{
	return InPin && !InPin->IsOutputPin() && GetInputPinIndex(InPin->Properties.Label) == GetInputPinToForward();
}

EPCGMetadataTypes UPCGMetadataSettingsBase::GetPinDefaultValueType(const FName PinLabel) const
{
	const FName PropertyName = PCGMetadataBase::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
	if (PropertyName != NAME_None)
	{
		if (DefaultValues.FindProperty(PropertyName))
		{
			return DefaultValues.GetCurrentPropertyType(PropertyName);
		}
		else
		{
			return GetPinInitialDefaultValueType(PinLabel);
		}
	}

	return EPCGMetadataTypes::Unknown;
}

#if WITH_EDITOR
bool UPCGMetadataSettingsBase::IsPinDefaultValueMetadataTypeValid(const FName PinLabel, EPCGMetadataTypes DataType) const
{
	bool bHasSpecialRequirement;
	return IsSupportedInputType(static_cast<uint16>(DataType), GetInputPinIndex(PinLabel), bHasSpecialRequirement);
}

void UPCGMetadataSettingsBase::ResetDefaultValues()
{
	DefaultValues.Reset();
	OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Settings | EPCGChangeType::Edge);
}

FString UPCGMetadataSettingsBase::GetPinDefaultValueAsString(const FName PinLabel) const
{
	if (ensure(IsPinDefaultValueActivated(PinLabel)))
	{
		const FName PropertyName = PCGMetadataBase::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
		if (PropertyName != NAME_None)
		{
			if (DefaultValues.FindProperty(PropertyName))
			{
				return DefaultValues.GetPropertyValueAsString(PropertyName);
			}
			else
			{
				return GetPinInitialDefaultValueString(PinLabel);
			}
		}
	}

	return FString();
}

void UPCGMetadataSettingsBase::ResetDefaultValue(const FName PinLabel)
{
	const FName PropertyName = PCGMetadataBase::Helpers::GetDefaultValuePropertyName(GetInputPinIndex(PinLabel));
	if (PropertyName != NAME_None && DefaultValues.FindProperty(PropertyName))
	{
		Modify();
		const EPCGMetadataTypes CurrentType = DefaultValues.GetCurrentPropertyType(PropertyName);
		DefaultValues.RemoveProperty(PropertyName);
		DefaultValues.CreateNewProperty(PropertyName, CurrentType);
	}
}
#endif // WITH_EDITOR

EPCGMetadataTypes UPCGMetadataSettingsBase::GetPinInitialDefaultValueType(const FName PinLabel) const
{
	// All overrides should exist for clarity and robustness.
#if WITH_EDITOR
	ensure(false);
	PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("GetPinInitialDefaultValueTypeNotOverridden", "GetPinInitialDefaultValueType not overridden by node: {0}"), FText::FromName(GetDefaultNodeName())));
#endif // WITH_EDITOR

	return EPCGMetadataTypes::Unknown;
}


bool UPCGMetadataSettingsBase::IsInputPinRequiredByExecution(const UPCGPin* InPin) const
{
	// The pin maintains 'required' status unless the default value is both enabled and activated.
	return InPin && (InPin->IsConnected() || !IsPinDefaultValueEnabled(InPin->Properties.Label) || !IsPinDefaultValueActivated(InPin->Properties.Label));
}

TArray<FPCGPinProperties> UPCGMetadataSettingsBase::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	for (uint32 InputPinIndex = 0; InputPinIndex < GetOperandNum(); ++InputPinIndex)
	{
		const FName PinLabel = GetInputPinLabel(InputPinIndex);
		if (PinLabel != NAME_None)
		{
			FPCGPinProperties& PinProperty = PinProperties.Emplace_GetRef(PinLabel, EPCGDataType::Any);
			PinProperty.SetRequiredPin();

#if WITH_EDITOR
			TArray<FText> AllTooltips;
			TArray<FString> SupportedTypes;

			for (uint8 TypeId = 0; TypeId < static_cast<uint8>(EPCGMetadataTypes::Count); ++TypeId)
			{
				bool DummyBool;
				if (IsSupportedInputType(TypeId, InputPinIndex, DummyBool))
				{
					SupportedTypes.Add(PCG::Private::GetTypeName(TypeId));
				}
			}

			if (!SupportedTypes.IsEmpty())
			{
				AllTooltips.Add(FText::Format(LOCTEXT("PinTooltipSupportedTypes", "Supported types: {0}"), FText::FromString(FString::Join(SupportedTypes, TEXT(", ")))));
			}

			if (GetOperandNum() > 1 && OutputDataFromPin == PinLabel)
			{
				AllTooltips.Add(LOCTEXT("PinTooltipForwardInput", "This input will be forwarded to the output."));
			}

			if (DefaultValuesAreEnabled() && IsPinDefaultValueEnabled(PinLabel))
			{
				EPCGMetadataTypes TypeValue = GetPinDefaultValueType(PinLabel);
				const UEnum* TypeEnum = StaticEnum<EPCGMetadataTypes>();
				const FText TypeString = TypeEnum ? TypeEnum->GetDisplayNameTextByValue(static_cast<int64>(TypeValue)) : FText();
				AllTooltips.Add(FText::Format(LOCTEXT("PinTooltipDefaultValue", "Pin is using default value of type: {0}"), TypeString));
			}

			if (!AllTooltips.IsEmpty())
			{
				PinProperty.Tooltip = FText::Join(FText::FromString(TEXT("\n")), AllTooltips);
			}
#endif // WITH_EDITOR
		}
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGMetadataSettingsBase::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	for (uint32 OutputPinIndex = 0; OutputPinIndex < GetResultNum(); ++OutputPinIndex)
	{
		const FName PinLabel = GetOutputPinLabel(OutputPinIndex);
		if (PinLabel != NAME_None)
		{
			PinProperties.Emplace(PinLabel, EPCGDataType::Any);
		}
	}

	return PinProperties;
}

#if WITH_EDITOR
bool UPCGMetadataSettingsBase::CanEditInputSource(const FProperty* InProperty, const int32 NumSources) const
{
	check(InProperty && NumSources <= MaxNumberOfInputs);

	for (int32 SourceIndex = 0; SourceIndex < NumSources; ++SourceIndex)
	{
		if (InProperty->GetFName() == PCGMetadataBase::Helpers::GetInputSourceName(SourceIndex)
			|| InProperty->GetFName() == TEXT("InputSource"))
		{
			if (const UPCGNode* Node = Cast<UPCGNode>(GetOuter()))
			{
				return !IsPinDefaultValueActivated(GetInputPinLabel(SourceIndex)) || Node->IsInputPinConnected(GetInputPinLabel(SourceIndex));
			}
		}
	}

	return true;
}
#endif // WITH_EDITOR

void FPCGMetadataElementBase::PassthroughInput(FPCGContext* Context, TArray<FPCGTaggedData>& Outputs, const int32 Index) const
{
	check(Context);
	FPCGMetadataElementBase::ContextType* TimeSlicedContext = static_cast<FPCGMetadataElementBase::ContextType*>(Context);

	const UPCGMetadataSettingsBase* Settings = Context->GetInputSettings<UPCGMetadataSettingsBase>();
	check(Settings);

	const uint32 NumberOfOutputs = Settings->GetResultNum();
	const uint32 PrimaryPinIndex = Settings->GetInputPinToForward();

	// Cache the primary pin inputs if we haven't already done it.
	if (TimeSlicedContext->GetPerExecutionState().OperandPinData.IsEmpty())
	{
		TimeSlicedContext->GetPerExecutionState().OperandPinData.SetNum(Settings->GetOperandNum());
	}

	if (TimeSlicedContext->GetPerExecutionState().OperandPinData[PrimaryPinIndex].IsEmpty())
	{
		TimeSlicedContext->GetPerExecutionState().OperandPinData[PrimaryPinIndex] = Context->InputData.GetInputsByPin(Settings->GetInputPinLabel(PrimaryPinIndex));
	}

	const TArray<FPCGTaggedData>& InputsToForward = TimeSlicedContext->GetPerExecutionState().OperandPinData[PrimaryPinIndex];

	if (InputsToForward.IsEmpty())
	{
		return;
	}

	// Take the index of the iteration, except for the 1:N case, where we just grab the first index
	const int32 AdjustedIndex = (Index < InputsToForward.Num()) ? Index : 0;

	// Passthrough this single input to all of the outputs
	for (uint32 I = 0; I < NumberOfOutputs; ++I)
	{
		Outputs.Emplace_GetRef(InputsToForward[AdjustedIndex]).Pin = Settings->GetOutputPinLabel(I);
	}
}

namespace PCGMetadataOpPrivate
{
	using ContextType = FPCGMetadataElementBase::ContextType;
	using ExecStateType = FPCGMetadataElementBase::ExecStateType;

	void CreateAccessor(const FPCGAttributePropertyInputSelector& Selector, const FPCGTaggedData& InputData, PCGMetadataOps::FOperationData& OperationData, const int32 Index)
	{
		OperationData.InputSources[Index] = Selector.CopyAndFixLast(InputData.Data);
		const FPCGAttributePropertyInputSelector& InputSource = OperationData.InputSources[Index];

		OperationData.InputAccessors[Index] = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData.Data, InputSource);
		OperationData.InputKeys[Index] = PCGAttributeAccessorHelpers::CreateConstKeys(InputData.Data, InputSource);
	}

	bool ValidateAccessor(const FPCGContext* Context, const UPCGMetadataSettingsBase* Settings, const FPCGTaggedData& InputData, PCGMetadataOps::FOperationData& OperationData, int32 Index)
	{
		const FPCGAttributePropertyInputSelector& InputSource = OperationData.InputSources[Index];
		const FText InputSourceText = InputSource.GetDisplayText();

		if (!OperationData.InputAccessors[Index].IsValid() || !OperationData.InputKeys[Index].IsValid())
		{
			PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("AttributeDoesNotExist", "Attribute/Property '{0}' from pin {1} does not exist"), InputSourceText, FText::FromName(InputData.Pin)));
			return false;
		}

		const uint16 AttributeTypeId = OperationData.InputAccessors[Index]->GetUnderlyingType();

		// Then verify that the type is OK
		bool bHasSpecialRequirement = false;
		if (!Settings->IsSupportedInputType(AttributeTypeId, Index, bHasSpecialRequirement))
		{
			const FText AttributeTypeName = PCG::Private::GetTypeNameText(AttributeTypeId);
			PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("UnsupportedAttributeType", "Attribute/Property '{0}' from pin {1} is not a supported type ('{2}')"),
				InputSourceText,
				FText::FromName(InputData.Pin),
				AttributeTypeName));
			return false;
		}

		if (!bHasSpecialRequirement)
		{
			// In this case, check if we have a more complex type, or if we can broadcast to the most complex type.
			if (OperationData.MostComplexInputType == static_cast<uint16>(EPCGMetadataTypes::Unknown) || PCG::Private::IsMoreComplexType(AttributeTypeId, OperationData.MostComplexInputType))
			{
				OperationData.MostComplexInputType = AttributeTypeId;
			}
			else if (OperationData.MostComplexInputType != AttributeTypeId && !PCG::Private::IsBroadcastable(AttributeTypeId, OperationData.MostComplexInputType))
			{
				const FText AttributeTypeName = PCG::Private::GetTypeNameText(AttributeTypeId);
				const FText MostComplexTypeName = PCG::Private::GetTypeNameText(OperationData.MostComplexInputType);
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("AttributeCannotBeBroadcasted", "Attribute '{0}' (from pin {1}) of type '{2}' cannot be used for operation with type '{3}'"),
					InputSourceText,
					FText::FromName(InputData.Pin),
					AttributeTypeName,
					MostComplexTypeName));
				return false;
			}
		}

		return true;
	}

	bool ValidateSecondaryInputClassMatches(const FPCGTaggedData& PrimaryInputData, const FPCGTaggedData& SecondaryInputData)
	{
		// First, verify the input data matches the primary. If the pin to forward is not connected, behave like a param data
		const UClass* InputPinToForwardClass = (PrimaryInputData.Data ? PrimaryInputData.Data->GetClass() : UPCGParamData::StaticClass());

		// TODO: Consider updating this to check if its a child class instead to be more future proof. For now this is good.
		// Check for data mismatch between primary pin and current pin
		if (InputPinToForwardClass != SecondaryInputData.Data->GetClass() && !SecondaryInputData.Data->IsA<UPCGParamData>())
		{
			return false;
		}

		return true;
	}
}

bool FPCGMetadataElementBase::PrepareDataInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataElementBase::PrepareDataInternal);

	FPCGMetadataElementBase::ContextType* TimeSlicedContext = static_cast<FPCGMetadataElementBase::ContextType*>(Context);
	check(TimeSlicedContext);

	const UPCGMetadataSettingsBase* Settings = Context->GetInputSettings<UPCGMetadataSettingsBase>();
	check(Settings);

	const uint32 OperandNum = Settings->GetOperandNum();
	const uint32 ResultNum = Settings->GetResultNum();

	check(OperandNum > 0);
	check(ResultNum <= UPCGMetadataSettingsBase::MaxNumberOfOutputs);

	const FName PrimaryPinLabel = Settings->GetInputPinLabel(Settings->GetInputPinToForward());
	const TArray<FPCGTaggedData> PrimaryInputs = Context->InputData.GetInputsByPin(PrimaryPinLabel);

	// There are no inputs on the primary pin, so pass-through inputs if the primary pin is required
	if (!Settings->IsPinDefaultValueActivated(PrimaryPinLabel) && PrimaryInputs.IsEmpty())
	{
		return true;
	}

	int32 OperandInputNumMax = 0;

	// Initiialize execution state, which will setup default data as need and perform early validation.
	EPCGTimeSliceInitResult ExecStateInitResult = TimeSlicedContext->InitializePerExecutionState([this, Settings, &OperandInputNumMax, OperandNum](ContextType* Context, ExecStateType& OutState) -> EPCGTimeSliceInitResult
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataElementBase::PrepareDataInternal::InitializePerExecutionState);
		FPCGMetadataElementBase::ContextType* TimeSlicedContext = static_cast<FPCGMetadataElementBase::ContextType*>(Context);
		OutState.DefaultValueOverriddenPins.AddZeroed(OperandNum);
		OutState.OperandPinData.SetNum(OperandNum);

		for (uint32 OperandPinIndex = 0; OperandPinIndex < OperandNum; ++OperandPinIndex)
		{
			const FName CurrentPinLabel = Settings->GetInputPinLabel(OperandPinIndex);
			const bool bIsInputConnected = Context->Node ? Context->Node->IsInputPinConnected(CurrentPinLabel) : false;
			TArray<FPCGTaggedData>& CurrentPinInputData = OutState.OperandPinData[OperandPinIndex];
			// Override data regardless.
			CurrentPinInputData = Context->InputData.GetInputsByPin(CurrentPinLabel);

			const int32 CurrentInputNum = CurrentPinInputData.Num();

			// For the current input, no input (0) could be default value and we support N:1 and 1:N
			if (CurrentInputNum > 1 && OperandInputNumMax > 1 && CurrentInputNum != OperandInputNumMax)
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("MismatchedOperandDataCount", "Number of data elements provided on inputs must be 1:N, N:1, or N:N."));
				return EPCGTimeSliceInitResult::AbortExecution;
			}

			OperandInputNumMax = FMath::Max(OperandInputNumMax, FMath::Max(1, CurrentInputNum));

			if (Settings->DefaultValuesAreEnabled() && !bIsInputConnected && CurrentPinInputData.IsEmpty() && Settings->IsPinDefaultValueActivated(CurrentPinLabel))
			{
				FPCGTaggedData& DefaultData = CurrentPinInputData.Emplace_GetRef();
				DefaultData.Pin = CurrentPinLabel;

				// @todo_pcg: Future optimizations/refactors - cache the param data on the settings, or use accessors on the default value struct, etc.
				// Create from the Default Value Container if it exists
				DefaultData.Data = Settings->CreateDefaultValueParamData(Context, CurrentPinLabel);

				// Couldn't create a default value
				if (!DefaultData.Data)
				{
					PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("CantCreateDefaultValue", "Pin '{0}' supports default value but we could not create it."), FText::FromName(CurrentPinLabel)));
					return EPCGTimeSliceInitResult::AbortExecution;
				}
				else
				{
					// Need to make sure the param data is properly tracked by the context to prevent garbage collection
					TimeSlicedContext->TrackObject(DefaultData.Data);

					UPCGMetadata* DefaultParamMetadata = CastChecked<UPCGParamData>(DefaultData.Data)->Metadata;
					if (DefaultParamMetadata->GetLocalItemCount() == 0)
					{
						DefaultParamMetadata->AddEntry();
					}

					OutState.DefaultValueOverriddenPins[OperandPinIndex] = true;
				}
			}

			if (CurrentPinInputData.IsEmpty())
			{
				// If we have no data, there is no operation
				PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("MissingInputDataForPin", "No data provided on pin '{0}'."), FText::FromName(CurrentPinLabel)));
				return EPCGTimeSliceInitResult::NoOperation;
			}
			else if (CurrentPinInputData.Num() != 1 && CurrentPinInputData.Num() != OperandInputNumMax)
			{
				PCGE_LOG(Error, GraphAndLog,
					FText::Format(LOCTEXT("MismatchedDataCountForPin", "Number of data elements ({0}) provided on pin '{1}' doesn't match number of expected elements ({2}). Only 1 input or {2} are supported."),
						CurrentPinInputData.Num(),
						FText::FromName(CurrentPinLabel),
						OperandInputNumMax));
				return EPCGTimeSliceInitResult::AbortExecution;
			}
		}

		return EPCGTimeSliceInitResult::Success;
	});

	if (ExecStateInitResult == EPCGTimeSliceInitResult::AbortExecution)
	{
		return true;
	}
	else if (ExecStateInitResult == EPCGTimeSliceInitResult::NoOperation)
	{
		// Passthrough all inputs
		// @todo_pcg - this could be optimized somewhat by doing a version that takes the OperandInputNumMax here.
		TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
		for (int IterationIndex = 0; IterationIndex < OperandInputNumMax; ++IterationIndex)
		{
			PassthroughInput(Context, Outputs, IterationIndex);
		}

		return true;
	}

	// Set up the iterations on the multiple inputs of the primary pin
	TimeSlicedContext->InitializePerIterationStates(OperandInputNumMax, [this, Context, OperandNum, Settings, OperandInputNumMax](IterStateType& OutState, const ExecStateType& ExecState, const uint32 IterationIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataElementBase::PrepareDataInternal::InitializePerIterationStates);
		FPCGMetadataElementBase::ContextType* TimeSlicedContext = static_cast<FPCGMetadataElementBase::ContextType*>(Context);
		TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
		const uint32 NumberOfResults = Settings->GetResultNum();

		OutState.Context = Context;

		// Gathering all the inputs metadata
		TArray<const UPCGMetadata*> SourceMetadata;
		TArray<const FPCGMetadataAttributeBase*> SourceAttribute;
		TArray<FPCGTaggedData> InputTaggedData;
		SourceMetadata.SetNum(OperandNum);
		SourceAttribute.SetNum(OperandNum);
		InputTaggedData.SetNum(OperandNum);

		// Since we add the output data (in CreateAttribute below) if the operation is valid in the PrepareData, if we ever have a no-op, we have to passthrough the inputs now and not in the Execute. So that the order is respected
		// in the end. (ie. { Input1(Valid), Input2(No-Op), Input3(Valid) } will have in output { Output1, Input2, Output3 } and not { Output1, Output3, Input2 } if we do the passthrough in the Execute.)
		auto NoOperation = [this, Context, IterationIndex, &Outputs]() { PassthroughInput(Context, Outputs, IterationIndex); return EPCGTimeSliceInitResult::NoOperation; };

		const uint32 PrimaryPinIndex = Settings->GetInputPinToForward();

		// Iterate over the inputs and validate
		for (uint32 OperandPinIndex = 0; OperandPinIndex < OperandNum; ++OperandPinIndex)
		{
			const TArray<FPCGTaggedData>& CurrentPinInputData = ExecState.OperandPinData[OperandPinIndex];

			// The operand inputs must either be N:1 or N:N or 1:N
			InputTaggedData[OperandPinIndex] = CurrentPinInputData.Num() == 1 ? CurrentPinInputData[0] : CurrentPinInputData[IterationIndex];

			// Check if we have any points
			if (const UPCGBasePointData* PointInput = Cast<const UPCGBasePointData>(InputTaggedData[OperandPinIndex].Data))
			{
				if (PointInput->GetNumPoints() == 0)
				{
					const FName CurrentPinLabel = Settings->GetInputPinLabel(OperandPinIndex);
					// If we have no points, there is no operation
					PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("NoPointsForPin", "No points in point data provided on pin {0}"), FText::FromName(CurrentPinLabel)));
					return NoOperation();
				}
			}

			SourceMetadata[OperandPinIndex] = InputTaggedData[OperandPinIndex].Data->ConstMetadata();
			if (!SourceMetadata[OperandPinIndex])
			{
				const FName CurrentPinLabel = Settings->GetInputPinLabel(OperandPinIndex);
				// Since this aborts execution, and the user can fix it, it should be a node error
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InvalidInputDataTypeForPin", "Invalid data provided on pin '{0}', must be of type Spatial or Attribute Set."), FText::FromName(CurrentPinLabel)));
				return EPCGTimeSliceInitResult::AbortExecution;
			}
		}

		PCGMetadataOps::FOperationData& OperationData = OutState;
		OperationData.Settings = Settings;
		OperationData.InputAccessors.SetNum(OperandNum);
		OperationData.InputKeys.SetNum(OperandNum);
		OperationData.InputSources.SetNum(OperandNum);
		OperationData.MostComplexInputType = static_cast<uint16>(EPCGMetadataTypes::Unknown);

		const FPCGTaggedData& PrimaryPinData = InputTaggedData[PrimaryPinIndex];

		// First create an accessor for the input to forward (it's our control data)
		const FPCGAttributePropertyInputSelector PrimarySelector = !ExecState.DefaultValueOverriddenPins[PrimaryPinIndex] ? Settings->GetInputSource(PrimaryPinIndex) : FPCGAttributePropertyInputSelector{};

		PCGMetadataOpPrivate::CreateAccessor(PrimarySelector, PrimaryPinData, OperationData, PrimaryPinIndex);
		if (!PCGMetadataOpPrivate::ValidateAccessor(Context, Settings, PrimaryPinData, OperationData, PrimaryPinIndex))
		{
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		// Update the number of elements to process, it's OK to be 0 if it is an attribute, as we can do a default value operation.
		OperationData.NumberOfElementsToProcess = OperationData.InputKeys[PrimaryPinIndex]->GetNum();
		if (OperationData.NumberOfElementsToProcess == 0 && !OperationData.InputAccessors[PrimaryPinIndex]->IsAttribute())
		{
			PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("NoElementsInForwardedInput", "No elements in data from forwarded pin '{0}'."), FText::FromName(PrimaryPinData.Pin)));
			return NoOperation();
		}

		// Create the accessors and validate them for each of the other operands
		for (uint32 Index = 0; Index < OperandNum; ++Index)
		{
			if (Index != PrimaryPinIndex)
			{
				// Secondary input class should match the forwarded one
				if (!PCGMetadataOpPrivate::ValidateSecondaryInputClassMatches(PrimaryPinData, InputTaggedData[Index]))
				{
					PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InputTypeMismatch", "Data on pin '{0}' is not of the same type than on pin '{1}' and is not an Attribute Set. This is not supported."), FText::FromName(InputTaggedData[Index].Pin), FText::FromName(PrimaryPinData.Pin)));
					return EPCGTimeSliceInitResult::AbortExecution;
				}

				const FPCGAttributePropertyInputSelector Selector = !ExecState.DefaultValueOverriddenPins[Index] ? Settings->GetInputSource(Index) : FPCGAttributePropertyInputSelector{};

				PCGMetadataOpPrivate::CreateAccessor(Selector, InputTaggedData[Index], OperationData, Index);
				if (!PCGMetadataOpPrivate::ValidateAccessor(Context, Settings, InputTaggedData[Index], OperationData, Index))
				{
					return EPCGTimeSliceInitResult::AbortExecution;
				}

				const int32 ElementNum = OperationData.InputKeys[Index]->GetNum();

				// No elements on secondary pin, early out for no operation, only if it is not an attribute, as we could still do a default value operation
				if (ElementNum == 0 && !OperationData.InputAccessors[Index]->IsAttribute())
				{
					PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("NoElementsInInput", "No elements in data from secondary pin '{0}'."), FText::FromName(PrimaryPinData.Pin)));
					return NoOperation();
				}

				// Verify that the number of elements makes sense
				if (ElementNum != 0 && OperationData.NumberOfElementsToProcess % ElementNum != 0)
				{
					PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("MismatchInNumberOfElements", "Mismatch between the number of elements from pin '{0}' ({1}) and from pin '{2}' ({3})."), FText::FromName(PrimaryPinData.Pin), OperationData.NumberOfElementsToProcess, FText::FromName(InputTaggedData[Index].Pin), ElementNum));
					return EPCGTimeSliceInitResult::AbortExecution;
				}

				// If selection is an attribute, get it from the metadata
				FPCGAttributePropertyInputSelector& InputSource = OperationData.InputSources[Index];
				if (InputSource.GetSelection() == EPCGAttributePropertySelection::Attribute)
				{
					SourceAttribute[Index] = SourceMetadata[Index]->GetConstAttribute(InputSource.GetName());
				}
				else
				{
					SourceAttribute[Index] = nullptr;
				}
			}
		}

		// At this point, we verified everything, so we can go forward with the computation, depending on the most complex type
		// So first forward outputs and create the attribute
		OperationData.OutputAccessors.SetNum(Settings->GetResultNum());
		OperationData.OutputKeys.SetNum(Settings->GetResultNum());

		const FPCGAttributePropertyOutputSelector OutputTarget = Settings->OutputTarget.CopyAndFixSource(&OperationData.InputSources[PrimaryPinIndex]);

		// Use implicit capture, since we capture a lot
		auto CreateAttribute = [&]<typename AttributeType>(uint32 OutputIndex, AttributeType DummyOutValue) -> bool
		{
			FPCGTaggedData& OutputTaggedData = Outputs.Add_GetRef(InputTaggedData[PrimaryPinIndex]);
			OutputTaggedData.Pin = Settings->GetOutputPinLabel(OutputIndex);

			// In case of property or attribute with extra accessor, we need to validate that the property/attribute can accept the output type.
			// Verify this before duplicating, because an extra allocation is certainly less costly than duplicating the data.
			// Do it with a const accessor, since OutputTaggedData.Data is still pointing on the const input data.

			if (!OutputTarget.IsBasicAttribute())
			{
				const TUniquePtr<const IPCGAttributeAccessor> TempConstAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OutputTaggedData.Data.Get(), OutputTarget);

				if (!TempConstAccessor.IsValid())
				{
					PCGLog::Metadata::LogFailToCreateAccessorError(OutputTarget, Context);
					return false;
				}

				if (!PCG::Private::IsBroadcastable(PCG::Private::MetadataTypes<AttributeType>::Id, TempConstAccessor->GetUnderlyingType()))
				{
					PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("AttributeTypeBroadcastFailed_Updated", "Output Attribute/Property '{0}' ({1}) is not compatible with operation output type ({2})."),
						OutputTarget.GetDisplayText(),
						PCG::Private::GetTypeNameText(TempConstAccessor->GetUnderlyingType()),
						PCG::Private::GetTypeNameText<AttributeType>()));
					return false;
				}

				// We have no element to process but we try to write into a property, early out.
				if (OperationData.NumberOfElementsToProcess == 0 && !TempConstAccessor->IsAttribute())
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NoDefaultValue", "Operation is done on the default value, but output attribute '{0}' does not support default values"), OutputTarget.GetDisplayText()), Context);
					return false;
				}
			}

			check(InputTaggedData[PrimaryPinIndex].Data);
			UPCGData* OutputData = InputTaggedData[PrimaryPinIndex].Data->DuplicateData(Context);
			check(OutputData);
			OutputTaggedData.Data = OutputData;

			if (OutputTarget.IsBasicAttribute())
			{
				FPCGMetadataAttributeBase* OutputAttribute = PCGMetadataElementCommon::ClearOrCreateAttribute<AttributeType>(OutputData->MutableMetadata(), OutputTarget);
				if (!OutputAttribute)
				{
					return false;
				}
			}

			OperationData.OutputAccessors[OutputIndex] = PCGAttributeAccessorHelpers::CreateAccessor(OutputData, OutputTarget);

			if (!OperationData.OutputAccessors[OutputIndex].IsValid())
			{
				return false;
			}

			if (OperationData.OutputAccessors[OutputIndex]->IsReadOnly())
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("OutputAccessorIsReadOnly", "Attribute/Property '{0}' is read only."), OutputTarget.GetDisplayText()));
				return false;
			}

			if (OperationData.OutputAccessors[OutputIndex]->IsAttribute()
				&& OperationData.NumberOfElementsToProcess > 1
				&& !OutputData->ConstMetadata()->MetadataDomainSupportsMultiEntries(OutputData->GetMetadataDomainIDFromSelector(OutputTarget)))
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("OutputAccessorIsNotSupportingMultiEntries", "Output attribute '{0}' is on a domain that doesn't support multi entries, but we try to process multiple elements ({1}). It's invalid."), OutputTarget.GetDisplayText(), OperationData.NumberOfElementsToProcess));
				return false;
			}

			OperationData.OutputKeys[OutputIndex] = PCGAttributeAccessorHelpers::CreateKeys(OutputData, OutputTarget);

			return OperationData.OutputKeys[OutputIndex].IsValid();
		};

		auto CreateAllSameAttributes = [NumberOfResults, &CreateAttribute](auto DummyOutValue) -> bool
		{
			for (uint32 i = 0; i < NumberOfResults; ++i)
			{
				if (!CreateAttribute(i, DummyOutValue))
				{
					return false;
				}
			}

			return true;
		};

		OperationData.OutputType = Settings->GetOutputType(OperationData.MostComplexInputType);

		bool bCreateAttributeSucceeded = true;

		if (!Settings->HasDifferentOutputTypes())
		{
			bCreateAttributeSucceeded = PCGMetadataAttribute::CallbackWithRightType(OperationData.OutputType, CreateAllSameAttributes);
		}
		else
		{
			TArray<uint16> OutputTypes = Settings->GetAllOutputTypes();
			check(OutputTypes.Num() == NumberOfResults);

			for (uint32 i = 0; i < NumberOfResults && bCreateAttributeSucceeded; ++i)
			{
				bCreateAttributeSucceeded &= PCGMetadataAttribute::CallbackWithRightType(OutputTypes[i], [&CreateAttribute, i](auto DummyOutValue) -> bool
				{
					return CreateAttribute(i, DummyOutValue);
				});
			}
		}

		if (!bCreateAttributeSucceeded)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("ErrorCreatingOutputAttributes", "Error while creating output attributes"));
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		OperationData.Settings = Settings;

		return EPCGTimeSliceInitResult::Success;
	});

	return true;
}

bool FPCGMetadataElementBase::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataElementBase::Execute);
	ContextType* TimeSlicedContext = static_cast<ContextType*>(Context);
	check(TimeSlicedContext);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	// Prepare data failed, no need to execute. Return an empty output
	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		return true;
	}

	return ExecuteSlice(TimeSlicedContext, [this, &Outputs](ContextType* Context, const ExecStateType& ExecState, IterStateType& IterState, const uint32 IterationIndex) -> bool
	{
		// No operation, so skip the iteration.
		if (Context->GetIterationStateResult(IterationIndex) == EPCGTimeSliceInitResult::NoOperation)
		{
			return true;
		}

		const bool bIsDone = DoOperation(IterState);
		
		if (bIsDone)
		{
			// Make sure the async state is reset, otherwise it means the metadata op is not taking into account time-slicing correctly
			ensureMsgf(!Context->AsyncState.bStarted,
				TEXT("Metadata operation has not finish processing the previous data and is starting a new processing.\n"
				"Make sure that the DoOperation is returning true only when the async processing is done."));
		}

		return bIsDone;
	});
}

#undef LOCTEXT_NAMESPACE
