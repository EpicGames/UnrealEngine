// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataPartition.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGKernelHelpers.h"
#include "Compute/BuiltInKernels/PCGCountUniqueAttributeValuesKernel.h"
#include "Elements/Metadata/PCGMetadataPartitionKernel.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/PCGMetadataPartitionCommon.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Utils/PCGLogErrors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataPartition)

#define LOCTEXT_NAMESPACE "PCGMetadataPartitionElement"

TArray<FPCGPinProperties> UPCGMetadataPartitionSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGMetadataPartitionSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataPartitionElement>();
}

void UPCGMetadataPartitionSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (PartitionAttribute_DEPRECATED != NAME_None)
	{
		PartitionAttributeSelectors.Empty();
		PartitionAttributeSelectors.Emplace_GetRef().SetAttributeName(PartitionAttribute_DEPRECATED);
		PartitionAttribute_DEPRECATED = NAME_None;
	}

	if (PartitionAttributeSource_DEPRECATED != FPCGAttributePropertyInputSelector())
	{
		PartitionAttributeSelectors.Empty();
		PartitionAttributeSelectors.Emplace(PartitionAttributeSource_DEPRECATED);
		PartitionAttributeSource_DEPRECATED = FPCGAttributePropertyInputSelector();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UPCGMetadataPartitionSettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	if (PartitionAttributeSelectors.IsEmpty())
	{
		ensure(false);
		return;
	}

	PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);

	UPCGMetadataPartitionKernel* PartitionKernel = PCGKernelHelpers::CreateKernel<UPCGMetadataPartitionKernel>(InOutContext, CreateParams, OutKernels, OutEdges);

	// Don't wire analysis kernel to node output pin, wire manually to the copy kernel below.
	CreateParams.NodeOutputPinsToWire.Empty();

	UPCGCountUniqueAttributeValuesKernel* CountKernel = PCGKernelHelpers::CreateKernel<UPCGCountUniqueAttributeValuesKernel>(InOutContext, CreateParams, OutKernels, OutEdges);
	CountKernel->SetAttributeName(PartitionAttributeSelectors[0].GetAttributeName());
	CountKernel->SetEmitPerDataCounts(true);
	CountKernel->SetMaxNumUniqueValuesSource(EPCGMaxNumUniqueValuesSource::StringKeyValues);

	// Wire count kernel to copy partition kernel.
	OutEdges.Emplace(FPCGPinReference(CountKernel, PCGPinConstants::DefaultOutputLabel), FPCGPinReference(PartitionKernel, PCGMetadataPartitionConstants::ElementCountsPinLabel));
}

void UPCGMetadataPartitionSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DataVersion < FPCGCustomVersion::AttributesAndTagsCanContainSpaces)
	{
		bTokenizeOnWhiteSpace = true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Super::ApplyDeprecation(InOutNode);
}
#endif // WITH_EDITOR

FString UPCGMetadataPartitionSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGMetadataPartitionSettings, PartitionAttributeNames)))
	{
		return FString();
	}
	else
#endif
	{
		if (PartitionAttributeSelectors.IsEmpty())
		{
			return FString();
		}

		FString OutString = PartitionAttributeSelectors[0].GetDisplayText().ToString();
		for (int I = 1; I < PartitionAttributeSelectors.Num(); ++I)
		{
			OutString += ", " + PartitionAttributeSelectors[I].GetDisplayText().ToString();
		}

		return OutString;
	}
}

bool FPCGMetadataPartitionElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataPartitionElement::Execute);
	check(Context);

	const UPCGMetadataPartitionSettings* Settings = Context->GetInputSettings<UPCGMetadataPartitionSettings>();
	check(Settings);

	// TODO: This is a temporary solution for overrides until arrays are supported
	TArray<FPCGAttributePropertyInputSelector> OverriddenSelectors;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FString> AttributeNames = Settings->bTokenizeOnWhiteSpace
		? PCGHelpers::GetStringArrayFromCommaSeparatedString(Settings->PartitionAttributeNames, Context)
		: PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->PartitionAttributeNames);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// If the names are overridden by the user, generate the selectors with them
	OverriddenSelectors.Reserve(AttributeNames.Num());
	for (const FString& AttributeName : AttributeNames)
	{
		OverriddenSelectors.Emplace_GetRef().Update(AttributeName);
	}

	const TArray<FPCGAttributePropertyInputSelector>& ActiveSelectors = OverriddenSelectors.IsEmpty() ? Settings->PartitionAttributeSelectors : OverriddenSelectors;

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	TArray<FPCGAttributePropertySelector> PartitionAttributeSources;
	PartitionAttributeSources.SetNum(ActiveSelectors.Num());

	if (PartitionAttributeSources.IsEmpty())
	{
		Outputs = Inputs;
		return true;
	}

	const bool bAssignIndexPartition = Settings->bAssignIndexPartition;
	// bDoNotPartition is only exposed to the user if bAssignIndexPartition is true, so make sure that this condition is respected
	// as no partition and no assign index is a trivial operation.
	const bool bDoPartition = !bAssignIndexPartition || !Settings->bDoNotPartition;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGData* InData = Input.Data;

		for (int32 i = 0; i < PartitionAttributeSources.Num(); ++i)
		{
			PartitionAttributeSources[i] = static_cast<FPCGAttributePropertySelector>(ActiveSelectors[i].CopyAndFixLast(InData));
		}

		if (bDoPartition)
		{
			TArray<UPCGData*> PartitionDataArray = PCGMetadataPartitionCommon::AttributePartition(InData, PartitionAttributeSources, Context);

			for (int32 i = 0; i < PartitionDataArray.Num(); ++i)
			{
				UPCGData* PartitionData = PartitionDataArray[i];
				// By construction, it must be valid
				check(PartitionData);

				FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
				Output.Data = PartitionData;

				if (bAssignIndexPartition)
				{
					UPCGMetadata* Metadata = PartitionData->MutableMetadata();
					if (ensure(Metadata))
					{
						Metadata->CreateAttribute<int32>(Settings->PartitionIndexAttributeName, i, /*bAllowInterpolation=*/false, /*bOverrideParent=*/true);
					}
				}
			}
		}
		else
		{
			const TArray<TArray<int32>> PartitionDataArray = PCGMetadataPartitionCommon::AttributeGenericPartition(InData, PartitionAttributeSources, Context);
			FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

			if (PartitionDataArray.IsEmpty())
			{
				continue;
			}

			UPCGData* OutputData = InData->DuplicateData(Context);
			check(OutputData);
			UPCGMetadata* Metadata = OutputData->MutableMetadata();
			check(Metadata);

			FPCGMetadataAttributeBase* AttributePartitionIndex = Metadata->CreateAttribute<int32>(Settings->PartitionIndexAttributeName, 0, /*bAllowInterpolation=*/false, /*bOverrideParent=*/true);
			if (!AttributePartitionIndex)
			{
				PCGLog::Metadata::LogFailToCreateAttributeError<int32>(Settings->PartitionIndexAttributeName, Context);
				continue;
			}

			const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(Settings->PartitionIndexAttributeName);
			TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputData, Selector);
			TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(OutputData, Selector);

			// Attribute exists, so Accessor and keys should exist
			if (!ensure(Accessor && Keys))
			{
				continue;
			}

			// Flatten the array to write it in one go.
			TArray<int32> AllIndexes;
			AllIndexes.SetNumUninitialized(Keys->GetNum());

			for (int32 i = 0; i < PartitionDataArray.Num(); ++i)
			{
				for (int32 j : PartitionDataArray[i])
				{
					AllIndexes[j] = i;
				}
			}

			Accessor->SetRange<int32>(AllIndexes, 0, *Keys);
			Output.Data = OutputData;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
