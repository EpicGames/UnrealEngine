// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataPartitionKernel.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/BuiltInKernels/PCGCountUniqueAttributeValuesKernel.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/DataInterfaces/Elements/PCGMetadataPartitionDataInterface.h"
#include "Elements/Metadata/PCGMetadataPartition.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataPartitionKernel)

#define LOCTEXT_NAMESPACE "PCGMetadataPartitionKernel"

bool UPCGMetadataPartitionKernel::IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMetadataPartitionKernel::IsKernelDataValid);

	if (!Super::IsKernelDataValid(InDataBinding, InContext))
	{
		return false;
	}

	if (InDataBinding)
	{
		const UPCGMetadataPartitionSettings* MPSettings = CastChecked<UPCGMetadataPartitionSettings>(GetSettings());
		const FName AttributeName = MPSettings->PartitionAttributeSelectors[0].GetAttributeName();

		const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InDataBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);

		if (!ensure(InputDesc))
		{
			return false;
		}

		FPCGKernelAttributeDesc AttributeDesc;
		bool bConflictingTypesInData = false;
		bool bPresentOnAllData = false;
		InputDesc->GetAttributeDesc(AttributeName, AttributeDesc, bConflictingTypesInData, bPresentOnAllData);

		if (!bPresentOnAllData)
		{
			if (!InputDesc->GetDataDescriptions().IsEmpty())
			{
				PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), FText::Format(
					LOCTEXT("PartitionAttributeMissing", "Partition attribute was not present on all incoming data."),
					FText::FromName(AttributeName)));
			}

			return false;
		}

		if (bConflictingTypesInData)
		{
			PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), FText::Format(
				LOCTEXT("PartitionAttributeTypeConflict", "Attribute '{0}' encountered with multiple different types in input data."),
				FText::FromName(AttributeName)));
			return false;
		}

		if (AttributeDesc.GetAttributeKey().GetType() != EPCGKernelAttributeType::StringKey)
		{
			PCG_KERNEL_VALIDATION_ERR(InContext, GetSettings(), FText::Format(
				LOCTEXT("PartitionAttributeTypeInvalid", "Attribute '{0}' not usable for partitioning, only attributes of type String Key are currently supported."),
				FText::FromName(AttributeName)));
			return false;
		}
	}

	return true;
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGMetadataPartitionKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	const UPCGMetadataPartitionSettings* MPSettings = CastChecked<UPCGMetadataPartitionSettings>(GetSettings());

	// Code assumes single output pin.
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	const FPCGKernelPin InputKernelPin(GetKernelIndex(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	const TSharedPtr<const FPCGDataCollectionDesc> InputPinDesc = InBinding->ComputeKernelPinDataDesc(InputKernelPin);

	if (!ensure(InputPinDesc))
	{
		return nullptr;
	}

	if (InputPinDesc->GetDataDescriptions().IsEmpty())
	{
		UE_LOGF(LogPCG, Verbose, "UPCGMetadataPartitionKernel: No analysis data found from count kernel.");
		return nullptr;
	}

	const TArray<int32> AnalysisDataIndices = InBinding->GetPinInputDataIndices(this, PCGMetadataPartitionConstants::ElementCountsPinLabel);

	if (AnalysisDataIndices.Num() != InputPinDesc->GetDataDescriptions().Num())
	{
		UE_LOGF(LogPCG, Error, "UPCGMetadataPartitionKernel: Number of received analysis data items (%d) does not match input data count (%d).",
			AnalysisDataIndices.Num(), InputPinDesc->GetDataDescriptions().Num());
		return nullptr;
	}

	if (AnalysisDataIndices.IsEmpty())
	{
		return nullptr;
	}

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeShared();

	for (int32 DataIndex = 0; DataIndex < AnalysisDataIndices.Num(); ++DataIndex)
	{
		const int32 AnalysisDataIndex = AnalysisDataIndices[DataIndex];
		
		check(InBinding->GetInputDataCollection().TaggedData.IsValidIndex(AnalysisDataIndex));

		// We can expect the analysis results data to already have been readback from the GPU because data description caching runs after the pre-execute readback stage of the ComputeGraph execution.
		const UPCGParamData* AnalysisResultsData = Cast<UPCGParamData>(InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data);
		const UPCGMetadata* AnalysisMetadata = AnalysisResultsData ? AnalysisResultsData->ConstMetadata() : nullptr;

		const FPCGMetadataAttributeBase* AnalysisCountAttributeBase = AnalysisMetadata ? AnalysisMetadata->GetConstAttribute(PCGAttributeAnalysisKernelConstants::ValueCountAttributeName) : nullptr;

		TArray<uint32> AttributeValueToCount;

		if (AnalysisCountAttributeBase && AnalysisCountAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<int32>::Id)
		{
			const FPCGMetadataAttribute<int32>* CountAttribute = static_cast<const FPCGMetadataAttribute<int32>*>(AnalysisCountAttributeBase);

			const int32 NumElements = AnalysisMetadata->GetItemCountForChild();

			AttributeValueToCount.Reserve(NumElements);

			// TODO: Range based get would scale better.
			for (int64 MetadataKey = 0; MetadataKey < NumElements; ++MetadataKey)
			{
				AttributeValueToCount.Add(CountAttribute->GetValue(MetadataKey));
			}
		}
		else
		{
			UE_LOGF(LogPCG, Warning, "UPCGMetadataPartitionKernel: No analysis data received.");

			if (InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data->IsA<UPCGProxyForGPUData>())
			{
				UE_LOGF(LogPCG, Error, "Data was not read back.");
			}

			return nullptr;
		}

		const int32 PartitionAttributeId = InBinding->GetAttributeId(MPSettings->PartitionAttributeSelectors[0].GetAttributeName(), EPCGKernelAttributeType::StringKey);

		for (int32 Value = 0; Value < AttributeValueToCount.Num(); ++Value)
		{
			if (AttributeValueToCount[Value] > 0)
			{
				FPCGDataDesc& DataDesc = OutDataDesc->GetDataDescriptionsMutable().Emplace_GetRef(EPCGDataType::Point, AttributeValueToCount[Value]);
				DataDesc.GetAttributeDescriptionsMutable().Append(InputPinDesc->GetDataDescriptions()[DataIndex].GetAttributeDescriptions());
				DataDesc.AllocateProperties(InputPinDesc->GetDataDescriptions()[DataIndex].GetAllocatedProperties());

				for (FPCGKernelAttributeDesc& AttributeDesc : DataDesc.GetAttributeDescriptionsMutable())
				{
					if (AttributeDesc.GetAttributeId() == PartitionAttributeId)
					{
						// By definition each data emitted by the partition node will have a single value for this attribute.
						AttributeDesc.SetStringKeys(MakeConstArrayView(&Value, 1));
						break;
					}
				}
			}
		}
	}

	// Add output attribute (partition index).
	if (MPSettings->bAssignIndexPartition)
	{
		OutDataDesc->AddAttributeToAllData(FPCGKernelAttributeKey(MPSettings->PartitionIndexAttributeName, EPCGKernelAttributeType::Int), InBinding);
	}

	return OutDataDesc;
}

int UPCGMetadataPartitionKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const TSharedPtr<const FPCGDataCollectionDesc> OutputPinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultOutputLabel, /*bIsInput=*/false);
	return ensure(OutputPinDesc) ? OutputPinDesc->ComputeTotalElementCount() : 0;
}

#if WITH_EDITOR
void UPCGMetadataPartitionKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGMetadataPartitionDataInterface> NodeDI = InOutContext.NewObject_AnyThread<UPCGMetadataPartitionDataInterface>(InObjectOuter);
	NodeDI->SetProducerKernel(this);
	OutDataInterfaces.Add(NodeDI);
}
#endif

void UPCGMetadataPartitionKernel::GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const
{
	const UPCGMetadataPartitionSettings* MPSettings = CastChecked<UPCGMetadataPartitionSettings>(GetSettings());

	// Register the attributes this node reads or writes for which we know the attribute type. Currently
	// only StringKey attributes can be partitioned when executing on GPU, so declare the attribute here
	// rather than doing work to resolve at runtime.
	for (const FPCGAttributePropertyInputSelector& Selector : MPSettings->PartitionAttributeSelectors)
	{
		if (Selector.IsBasicAttribute())
		{
			OutKeys.Add(FPCGKernelAttributeKey(Selector.GetAttributeName(), EPCGKernelAttributeType::StringKey));
		}
	}

	// Register output attribute created by this node.
	if (MPSettings->bAssignIndexPartition)
	{
		OutKeys.AddUnique(FPCGKernelAttributeKey(MPSettings->PartitionIndexAttributeName, EPCGKernelAttributeType::Int));
	}
}

void UPCGMetadataPartitionKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	OutPins.Emplace(PCGMetadataPartitionConstants::ElementCountsPinLabel, EPCGDataType::Param);
}

void UPCGMetadataPartitionKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
}

#if WITH_EDITOR
bool UPCGMetadataPartitionKernel::PerformStaticValidation()
{
	if (!Super::PerformStaticValidation())
	{
		return false;
	}
	
	const UPCGMetadataPartitionSettings* MPSettings = CastChecked<UPCGMetadataPartitionSettings>(GetSettings());

	if (MPSettings->PartitionAttributeSelectors.Num() != 1)
	{
#if PCG_KERNEL_LOGGING_ENABLED
		AddStaticLogEntry(LOCTEXT("MustProvideOnePartitionAttribute", "GPU implementation currently only supports a single partition attribute."), EPCGKernelLogVerbosity::Error);
#endif
		return false;
	}

	if (!MPSettings->PartitionAttributeSelectors[0].IsBasicAttribute())
	{
#if PCG_KERNEL_LOGGING_ENABLED
		AddStaticLogEntry(LOCTEXT("OnlyBasicAttributesSupported", "GPU implementation currently only supports basic attributes."), EPCGKernelLogVerbosity::Error);
#endif
		return false;
	}

	return true;
}
#endif

#undef LOCTEXT_NAMESPACE
