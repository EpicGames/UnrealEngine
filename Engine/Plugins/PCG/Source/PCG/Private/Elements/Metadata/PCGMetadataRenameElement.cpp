// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataRenameElement.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGPin.h"
#include "PCGParamData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataRenameElement)

#define LOCTEXT_NAMESPACE "PCGMetadataRenameElement"

UPCGMetadataRenameSettings::UPCGMetadataRenameSettings(const FObjectInitializer& ObjectInitializer)
{
	AttributeToRename.SetAttributeName(NAME_None);
}

void UPCGMetadataRenameSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (DataVersion < FPCGCustomVersion::AttributeRenameSupportToSelectors && AttributeToRename.GetAttributeName() == NAME_None)
	{
		// Previous behavior was that None -> Last created.
		AttributeToRename.SetAttributeName(PCGMetadataAttributeConstants::LastCreatedAttributeName);
	}
#endif // WITH_EDITOR
}

FString UPCGMetadataRenameSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGMetadataRenameSettings, AttributeToRename)) || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGMetadataRenameSettings, NewAttributeName)))
	{
		return FString();
	}
	else
#endif
	{
		return FString::Printf(TEXT("%s -> %s"), *AttributeToRename.ToString(), *NewAttributeName.ToString());
	}
}

TArray<FPCGPinProperties> UPCGMetadataRenameSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGMetadataRenameSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataRenameElement>();
}

bool FPCGMetadataRenameElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataRenameElement::Execute);

	const UPCGMetadataRenameSettings* Settings = Context->GetInputSettings<UPCGMetadataRenameSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	
	const FName NewAttributeName = Settings->NewAttributeName;

	for (const FPCGTaggedData& Input : Inputs)
	{
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
		
		const UPCGMetadata* Metadata = Input.Data ? Input.Data->ConstMetadata() : nullptr;
		
		if (!Metadata)
		{
			PCGLog::Metadata::LogInvalidMetadata(Context);
			continue;
		}
		
		const FPCGAttributePropertyInputSelector Selector = Settings->AttributeToRename.CopyAndFixLast(Input.Data);

		if (!Selector.IsBasicAttribute())
		{
			PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidInputName", "Attribute to rename '{0}' is not an attribute."), Selector.GetDisplayText()));
			continue;
		}
		
		const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomainFromSelector(Selector);
		
		if (!MetadataDomain)
		{
			PCGLog::Metadata::LogInvalidMetadataDomain(Selector, Context);
			continue;
		}

		const FName LocalAttributeToRename = Selector.GetAttributeName();

		if (!MetadataDomain->HasAttribute(LocalAttributeToRename))
		{
			continue;
		}

		UPCGMetadata* NewMetadata = nullptr;
		PCGMetadataElementCommon::DuplicateTaggedData(Context, Input, Output, NewMetadata);
		FPCGMetadataDomain* NewMetadataDomain = NewMetadata ? NewMetadata->GetMetadataDomainFromSelector(Selector) : nullptr;

		if (!NewMetadataDomain || !NewMetadataDomain->RenameAttribute(LocalAttributeToRename, NewAttributeName))
		{
			PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("AttributeRenamedFailed", "Failed to rename attribute from '{0}' to '{1}'"),
				FText::FromName(LocalAttributeToRename), FText::FromName(NewAttributeName)));
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
