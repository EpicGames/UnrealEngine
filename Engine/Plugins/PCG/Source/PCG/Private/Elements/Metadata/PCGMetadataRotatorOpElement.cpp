// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataRotatorOpElement.h"

#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataRotatorOpElement)

// Taken from Kismet Math library
FRotator PCGMetadataRotatorHelpers::RLerp(const FRotator& A, const FRotator& B, double Alpha, bool bShortestPath)
{
	// if shortest path, we use Quaternion to interpolate instead of using FRotator
	if (bShortestPath)
	{
		FQuat AQuat(A);
		FQuat BQuat(B);

		FQuat Result = FQuat::Slerp(AQuat, BQuat, Alpha);

		return Result.Rotator();
	}

	const FRotator DeltaAngle = B - A;
	return A + Alpha * DeltaAngle;
}

namespace PCGMetadataRotatorSettings
{
	inline constexpr bool IsUnaryOp(EPCGMetadataRotatorOperation Operation)
	{
		return Operation == EPCGMetadataRotatorOperation::Invert ||
			Operation == EPCGMetadataRotatorOperation::Normalize;
	}

	inline constexpr bool IsTernaryOp(EPCGMetadataRotatorOperation Operation)
	{
		return Operation == EPCGMetadataRotatorOperation::Lerp;
	}

	inline constexpr bool IsTransformOp(EPCGMetadataRotatorOperation Operation)
	{
		return (uint16)Operation >= (uint16)EPCGMetadataRotatorOperation::TransformOp;
	}

	inline FRotator ApplyRotatorOperation(const FRotator& Input1, const FRotator& Input2, double Ratio, EPCGMetadataRotatorOperation Operation)
	{
		switch (Operation)
		{
		case EPCGMetadataRotatorOperation::Combine:
			return FRotator(Input2.Quaternion() * Input1.Quaternion());
		case EPCGMetadataRotatorOperation::Lerp:
			return PCGMetadataRotatorHelpers::RLerp(Input1, Input2, Ratio, false);
		case EPCGMetadataRotatorOperation::Invert:
			return Input1.GetInverse();
		case EPCGMetadataRotatorOperation::Normalize:
			return Input1.GetNormalized();
		default:
			return FRotator{};
		}
	}

	inline FQuat ApplyRotatorOperation(const FQuat& Input1, const FQuat& Input2, double Ratio, EPCGMetadataRotatorOperation Operation)
	{
		switch (Operation)
		{
		case EPCGMetadataRotatorOperation::Combine:
			return Input2 * Input1;
		case EPCGMetadataRotatorOperation::Lerp:
			return FQuat::Slerp(Input1, Input2, Ratio);
		case EPCGMetadataRotatorOperation::Invert:
			return Input1.Inverse();
		case EPCGMetadataRotatorOperation::Normalize:
			return Input1.GetNormalized();
		default:
			return FQuat{};
		}
	}

	inline FQuat ApplyTransformOperation(const FQuat& Input, const FTransform& Transform, EPCGMetadataRotatorOperation Operation)
	{
		switch (Operation)
		{
		case EPCGMetadataRotatorOperation::TransformRotation:
			return Transform.TransformRotation(Input);
		case EPCGMetadataRotatorOperation::InverseTransformRotation:
			return Transform.InverseTransformRotation(Input);
		default:
			return FQuat{};
		}
	}

	// In Kismet Math Library, they transform Rotators in Quaternions.
	inline FRotator ApplyTransformOperation(const FRotator& Input, const FTransform& Transform, EPCGMetadataRotatorOperation Operation)
	{
		return FRotator(ApplyTransformOperation(Input.Quaternion(), Transform, Operation));
	}
}

void UPCGMetadataRotatorSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (Input1AttributeName_DEPRECATED != NAME_None)
	{
		InputSource1.SetAttributeName(Input1AttributeName_DEPRECATED);
		Input1AttributeName_DEPRECATED = NAME_None;
	}

	if (Input2AttributeName_DEPRECATED != NAME_None)
	{
		InputSource2.SetAttributeName(Input2AttributeName_DEPRECATED);
		Input2AttributeName_DEPRECATED = NAME_None;
	}

	if (Input3AttributeName_DEPRECATED != NAME_None)
	{
		InputSource3.SetAttributeName(Input3AttributeName_DEPRECATED);
		Input3AttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

FName UPCGMetadataRotatorSettings::GetInputPinLabel(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		if (!PCGMetadataRotatorSettings::IsUnaryOp(Operation) && !PCGMetadataRotatorSettings::IsTransformOp(Operation))
		{
			return PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
		}
		else
		{
			return PCGPinConstants::DefaultInputLabel;
		}
	case 1:
		return PCGMetadataRotatorSettings::IsTransformOp(Operation) ? PCGMetadataSettingsBaseConstants::TransformLabel : PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
	case 2:
		return PCGMetadataSettingsBaseConstants::LerpRatioLabel;
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataRotatorSettings::GetOperandNum() const
{
	if (PCGMetadataRotatorSettings::IsUnaryOp(Operation))
	{
		return 1;
	}
	else if (PCGMetadataRotatorSettings::IsTernaryOp(Operation))
	{
		return 3;
	}
	else
	{
		return 2;
	}
}

bool UPCGMetadataRotatorSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	if (InputIndex == 2)
	{
		bHasSpecialRequirement = true;
		return PCG::Private::IsOfTypes<float, double>(TypeId);
	}
	else if (InputIndex == 1 && PCGMetadataRotatorSettings::IsTransformOp(Operation))
	{
		bHasSpecialRequirement = true;
		return PCG::Private::IsOfTypes<FTransform>(TypeId);
	}
	else
	{
		bHasSpecialRequirement = false;
		return PCG::Private::IsOfTypes<FRotator, FQuat>(TypeId);
	}
}

FPCGAttributePropertyInputSelector UPCGMetadataRotatorSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	case 2:
		return InputSource3;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

FString UPCGMetadataRotatorSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataRotatorOperation>())
	{
		return FText::Format(NSLOCTEXT("PCGMetadataRotatorSettings", "RotatorOperation", "Rotator: {0}"), EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation))).ToString();
	}
	else
	{
		return FString();
	}
}

#if WITH_EDITOR
bool UPCGMetadataRotatorSettings::IsPinDefaultValueMetadataTypeValid(FName PinLabel, const EPCGMetadataTypes DataType) const
{
	return DataType == EPCGMetadataTypes::Rotator
		|| DataType == EPCGMetadataTypes::Vector
		|| DataType == EPCGMetadataTypes::Double;
}

FName UPCGMetadataRotatorSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeRotatorOp");
}

FText UPCGMetadataRotatorSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataRotatorSettings", "NodeTitle", "Attribute Rotator Op");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataRotatorSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataRotatorOperation>({ EPCGMetadataRotatorOperation::RotatorOp, EPCGMetadataRotatorOperation::TransformOp });
}
#endif // WITH_EDITOR

void UPCGMetadataRotatorSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataRotatorOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataRotatorOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

EPCGMetadataTypes UPCGMetadataRotatorSettings::GetPinInitialDefaultValueType(const FName PinLabel) const
{
	if (PinLabel == PCGPinConstants::DefaultInputLabel
		|| PinLabel == PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel
		|| PinLabel == PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel)
	{
		return EPCGMetadataTypes::Rotator;
	}
	else if (PinLabel == PCGMetadataSettingsBaseConstants::TransformLabel)
	{
		return EPCGMetadataTypes::Transform;
	}
	else if (PinLabel == PCGMetadataSettingsBaseConstants::LerpRatioLabel)
	{
		return EPCGMetadataTypes::Double;
	}
	else
	{
		return EPCGMetadataTypes::Unknown;
	}
}

FPCGElementPtr UPCGMetadataRotatorSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataRotatorElement>();
}

bool FPCGMetadataRotatorElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataRotatorElement::Execute);

	const UPCGMetadataRotatorSettings* Settings = CastChecked<UPCGMetadataRotatorSettings>(OperationData.Settings);

	auto RotatorFunc = [this, &OperationData, Operation = Settings->Operation]<typename AttributeType>(AttributeType) -> bool
	{
		if constexpr (!PCG::Private::IsOfTypes<AttributeType, FQuat, FRotator>())
		{
			ensure(false);
			return true;
		}
		else
		{
			if (PCGMetadataRotatorSettings::IsTransformOp(Operation))
			{
				return DoBinaryOp<AttributeType, FTransform>(OperationData, [Operation](const AttributeType& Value, const FTransform& Transform)->AttributeType {
					return PCGMetadataRotatorSettings::ApplyTransformOperation(Value, Transform, Operation);
					});
			}
			else if (PCGMetadataRotatorSettings::IsUnaryOp(Operation))
			{
				return DoUnaryOp<AttributeType>(OperationData, [Operation](const AttributeType& Value)->AttributeType {
					double DummyDouble = 0.0;
					return PCGMetadataRotatorSettings::ApplyRotatorOperation(Value, AttributeType{}, DummyDouble, Operation);
				});
			}
			else if (PCGMetadataRotatorSettings::IsTernaryOp(Operation))
			{
				return DoTernaryOp<AttributeType, AttributeType, double>(OperationData, [Operation](const AttributeType& Value1, const AttributeType& Value2, const double& Ratio)->AttributeType {
					return PCGMetadataRotatorSettings::ApplyRotatorOperation(Value1, Value2, Ratio, Operation);
				});
			}
			else
			{
				return DoBinaryOp<AttributeType, AttributeType>(OperationData, [Operation](const AttributeType& Value1, const AttributeType& Value2)->AttributeType {
					double DummyDouble = 0.0;
					return PCGMetadataRotatorSettings::ApplyRotatorOperation(Value1, Value2, DummyDouble, Operation);
				});
			}
		}
	};

	return PCGMetadataAttribute::CallbackWithRightType(OperationData.OutputType, RotatorFunc);
}
