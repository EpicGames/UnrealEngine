// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataStringOpElement.h"

#include "Elements/Metadata/PCGMetadataElementCommon.h"

#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataStringOpElement)

namespace PCGMetadataStringOpConstants
{
	constexpr FLazyName ToFindLabel = "Search";
	constexpr FLazyName ToReplaceLabel = "Replace";
	constexpr FLazyName CountLabel = "Count";
	constexpr FLazyName StartLabel = "Start";
	
}

FName UPCGMetadataStringOpSettings::GetInputPinLabel(uint32 Index) const
{
	switch (Operation)
	{
	case EPCGMetadataStringOperation::ToLower:   // fall-through
	case EPCGMetadataStringOperation::ToUpper:   // fall-through
	case EPCGMetadataStringOperation::TrimStart: // fall-through
	case EPCGMetadataStringOperation::TrimEnd:   // fall-through
	case EPCGMetadataStringOperation::TrimStartAndEnd:
	{
		return PCGPinConstants::DefaultInputLabel;
	}

	case EPCGMetadataStringOperation::Append:          // fall-through
	case EPCGMetadataStringOperation::Substring:       // fall-through
	case EPCGMetadataStringOperation::Matches:         // fall-through
	case EPCGMetadataStringOperation::RemoveFromStart: // fall-through
	case EPCGMetadataStringOperation::RemoveFromEnd:   // fall-through
	case EPCGMetadataStringOperation::Find:            // fall-through
	case EPCGMetadataStringOperation::FindLast:
		switch (Index)
		{
		case 0: return PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
		case 1: return PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
		default: break;
		}
		
	case EPCGMetadataStringOperation::Left:     // fall-through
	case EPCGMetadataStringOperation::LeftChop: // fall-through
	case EPCGMetadataStringOperation::Right:    // fall-through
	case EPCGMetadataStringOperation::RightChop:
		switch (Index)
		{
		case 0: return PCGPinConstants::DefaultInputLabel;
		case 1: return PCGMetadataStringOpConstants::CountLabel;
		default: break;
		}
		
	case EPCGMetadataStringOperation::Mid:
		switch (Index)
		{
		case 0: return PCGPinConstants::DefaultInputLabel;
		case 1: return PCGMetadataStringOpConstants::StartLabel;
		case 2: return PCGMetadataStringOpConstants::CountLabel;
		default: break;
		}
		
	case EPCGMetadataStringOperation::Replace:
		switch (Index)
		{
		case 0: return PCGPinConstants::DefaultInputLabel;
		case 1: return PCGMetadataStringOpConstants::ToFindLabel;
		case 2: return PCGMetadataStringOpConstants::ToReplaceLabel;
		default: break;
		}

	default: break;
	}
	
	return NAME_None;
}

uint32 UPCGMetadataStringOpSettings::GetOperandNum() const
{
	if (HasThreeOperands())
	{
		return 3;
	}
	else if (HasTwoOperands())
	{
		return 2;
	}
	else
	{
		return 1;
	}
}

bool UPCGMetadataStringOpSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;
	
	switch (Operation)
	{
	case EPCGMetadataStringOperation::Left:    // fall-through
	case EPCGMetadataStringOperation::LeftChop: // fall-through
	case EPCGMetadataStringOperation::Right: // fall-through
	case EPCGMetadataStringOperation::RightChop: // fall-through
	case EPCGMetadataStringOperation::Mid:
	{
		bHasSpecialRequirement = true;
		if (InputIndex >= 1)
		{
			return PCG::Private::IsBroadcastableOrConstructible(TypeId, PCG::Private::MetadataTypes<int32>::Id);
		}
		
		break;
	}
	default:
		break;
	}
	
	return true; // all types support ToString()
}

FPCGAttributePropertyInputSelector UPCGMetadataStringOpSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0: return InputSource1;
	case 1: return InputSource2;
	case 2: return InputSource3;
	default: return FPCGAttributePropertyInputSelector();
	}
}

FString UPCGMetadataStringOpSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataStringOperation>())
	{
		return FText::Format(NSLOCTEXT("PCGMetadataStringOpSettings", "StringOperation", "String: {0}"), EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation))).ToString();
	}
	else
	{
		return FString();
	}
}

void UPCGMetadataStringOpSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGMetadataOpStringTypeChange)
	{
		// For all previous nodes, we'll force this option to true. for retro-compatibility
		OutputTypeAsString = true;
	}
#endif // WITH_EDITOR
}
 
#if WITH_EDITOR
FName UPCGMetadataStringOpSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeStringOp");
}

FText UPCGMetadataStringOpSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataStringOpSettings", "NodeTitle", "Attribute String Op");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataStringOpSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataStringOperation>();
}
#endif // WITH_EDITOR

void UPCGMetadataStringOpSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataStringOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataStringOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

FPCGElementPtr UPCGMetadataStringOpSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataStringOpElement>();
}

uint16 UPCGMetadataStringOpSettings::GetOutputType(uint16 InputTypeId) const
{
	if (Operation == EPCGMetadataStringOperation::Find || Operation == EPCGMetadataStringOperation::FindLast)
	{
		return (uint16)EPCGMetadataTypes::Integer32;
	}
	else if (OutputsBool() && !OutputTypeAsString)
	{
		return (uint16)EPCGMetadataTypes::Boolean;
	}
	else
	{
		return (uint16)EPCGMetadataTypes::String;
	}
}

EPCGMetadataTypes UPCGMetadataStringOpSettings::GetPinInitialDefaultValueType(FName PinLabel) const
{
	return (PinLabel == PCGMetadataStringOpConstants::CountLabel || PinLabel == PCGMetadataStringOpConstants::StartLabel) ? EPCGMetadataTypes::Integer32 : EPCGMetadataTypes::String;
}

#if WITH_EDITOR
FString UPCGMetadataStringOpSettings::GetPinInitialDefaultValueString(FName PinLabel) const
{
	return (PinLabel == PCGMetadataStringOpConstants::CountLabel || PinLabel == PCGMetadataStringOpConstants::StartLabel) ? PCG::Private::MetadataTraits<int32>::ZeroValueString() : PCG::Private::MetadataTraits<FString>::ZeroValueString();
}

bool UPCGMetadataStringOpSettings::HasSearchCase() const
{
	return Operation == EPCGMetadataStringOperation::Substring
		|| Operation == EPCGMetadataStringOperation::Matches
		|| Operation == EPCGMetadataStringOperation::Replace
		|| Operation == EPCGMetadataStringOperation::RemoveFromStart
		|| Operation == EPCGMetadataStringOperation::RemoveFromEnd
		|| Operation == EPCGMetadataStringOperation::Find
		|| Operation == EPCGMetadataStringOperation::FindLast;
}
#endif // WITH_EDITOR

bool UPCGMetadataStringOpSettings::HasTwoOperands() const
{
	return Operation == EPCGMetadataStringOperation::Append
		|| Operation == EPCGMetadataStringOperation::Substring
		|| Operation == EPCGMetadataStringOperation::Matches
		|| Operation == EPCGMetadataStringOperation::Left
		|| Operation == EPCGMetadataStringOperation::LeftChop
		|| Operation == EPCGMetadataStringOperation::Right
		|| Operation == EPCGMetadataStringOperation::RightChop
		|| Operation == EPCGMetadataStringOperation::RemoveFromStart
		|| Operation == EPCGMetadataStringOperation::RemoveFromEnd
		|| Operation == EPCGMetadataStringOperation::Find
		|| Operation == EPCGMetadataStringOperation::FindLast;
}

bool UPCGMetadataStringOpSettings::HasThreeOperands() const
{
	return Operation == EPCGMetadataStringOperation::Replace
		|| Operation == EPCGMetadataStringOperation::Mid;
}

bool UPCGMetadataStringOpSettings::OutputsBool() const
{
	return Operation == EPCGMetadataStringOperation::Matches || Operation == EPCGMetadataStringOperation::Substring;
}

bool FPCGMetadataStringOpElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataStringOpElement::Execute);

	const UPCGMetadataStringOpSettings* Settings = CastChecked<UPCGMetadataStringOpSettings>(OperationData.Settings);

	if (Settings->Operation == EPCGMetadataStringOperation::ToUpper)
	{
		return DoUnaryOp<FString>(OperationData, [](const FString& Value) -> FString { return Value.ToUpper(); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::ToLower)
	{
		return DoUnaryOp<FString>(OperationData, [](const FString& Value) -> FString { return Value.ToLower(); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::TrimStart)
	{
		return DoUnaryOp<FString>(OperationData, [](const FString& Value) -> FString { return Value.TrimStart(); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::TrimEnd)
	{
		return DoUnaryOp<FString>(OperationData, [](const FString& Value) -> FString { return Value.TrimEnd(); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::TrimStartAndEnd)
	{
		return DoUnaryOp<FString>(OperationData, [](const FString& Value) -> FString { return Value.TrimStartAndEnd(); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::Append)
	{
		return DoBinaryOp<FString, FString>(OperationData, [](const FString& Value1, const FString& Value2) -> FString { return Value1 + Value2; });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::Substring)
	{
		return DoBinaryOp<FString, FString>(OperationData, [Settings](const FString& A, const FString& B) -> bool { return PCG::Private::MetadataTraits<FString>::Substring(A, B, Settings->SearchCase); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::Matches)
	{
		return DoBinaryOp<FString, FString>(OperationData, [Settings](const FString& A, const FString& B) -> bool { return PCG::Private::MetadataTraits<FString>::Matches(A, B, Settings->SearchCase); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::RemoveFromStart)
	{
		// Intentionally pass A by value, since `RemoveFromStart` is in place, but since we have just a single output, A will be moved (cf. PCG::Private::NAryOperation::Apply)
		return DoBinaryOp<FString, FString>(OperationData, [Settings](FString A, const FString& B) -> FString { A.RemoveFromStart(B, Settings->SearchCase); return A; });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::RemoveFromEnd)
	{
		// Intentionally pass A by value, since `RemoveFromEnd` is in place, but since we have just a single output, A will be moved (cf. PCG::Private::NAryOperation::Apply)
		return DoBinaryOp<FString, FString>(OperationData, [Settings](FString A, const FString& B) -> FString { A.RemoveFromEnd(B, Settings->SearchCase); return A; });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::Left)
	{
		return DoBinaryOp<FString, int32>(OperationData, [](const FString& A, const int32 Count) -> FString { return A.Left(Count); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::LeftChop)
	{
		return DoBinaryOp<FString, int32>(OperationData, [](const FString& A, const int32 Count) -> FString { return A.LeftChop(Count); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::Right)
	{
		return DoBinaryOp<FString, int32>(OperationData, [](const FString& A, const int32 Count) -> FString { return A.Right(Count); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::RightChop)
	{
		return DoBinaryOp<FString, int32>(OperationData, [](const FString& A, const int32 Count) -> FString { return A.RightChop(Count); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::Find)
	{
		return DoBinaryOp<FString, FString>(OperationData, [Settings](const FString& A, const FString& B) -> int32 { return A.Find(B, Settings->SearchCase, ESearchDir::FromStart); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::FindLast)
	{
		return DoBinaryOp<FString, FString>(OperationData, [Settings](const FString& A, const FString& B) -> int32 { return A.Find(B, Settings->SearchCase, ESearchDir::FromEnd); });
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::Mid)
	{
		return DoTernaryOp<FString, int32, int32>(OperationData, [](const FString& InValue, const int32 Start, const int32 Count) -> FString
		{
			return InValue.Mid(Start, Count);
		});
	}
	else if (Settings->Operation == EPCGMetadataStringOperation::Replace)
	{
		return DoTernaryOp<FString, FString, FString>(OperationData, [Settings](const FString& InValue, const FString& InSearch, const FString& InReplace) -> FString
		{
			return InValue.Replace(*InSearch, *InReplace, Settings->SearchCase);
		});
	}
	else
	{
		ensure(false);
		return true;
	}
}
