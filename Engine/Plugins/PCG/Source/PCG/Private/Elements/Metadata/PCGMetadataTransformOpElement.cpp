// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataTransformOpElement.h"

#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Elements/Metadata/PCGMetadataRotatorOpElement.h"

#include "Math/DualQuat.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataTransformOpElement)

namespace PCGMetadataTransfromSettings
{
	inline bool IsUnaryOp(EPCGMetadataTransformOperation Operation)
	{
		return Operation == EPCGMetadataTransformOperation::Invert;
	}

	inline bool IsTernaryOp(EPCGMetadataTransformOperation Operation)
	{
		return Operation == EPCGMetadataTransformOperation::Lerp;
	}

	// Taken from Kismet Math Library
	inline FTransform LerpTransform(const FTransform& A, const FTransform& B, double Alpha, EPCGTransformLerpMode Mode)
	{
		FTransform Result;

		FTransform NA = A;
		FTransform NB = B;
		NA.NormalizeRotation();
		NB.NormalizeRotation();

		// Quaternion interpolation
		if (Mode == EPCGTransformLerpMode::QuatInterp)
		{
			Result.Blend(NA, NB, Alpha);
			return Result;
		}
		// Euler Angle interpolation
		else if (Mode == EPCGTransformLerpMode::EulerInterp)
		{
			Result.SetTranslation(FMath::Lerp(NA.GetTranslation(), NB.GetTranslation(), Alpha));
			Result.SetScale3D(FMath::Lerp(NA.GetScale3D(), NB.GetScale3D(), Alpha));
			Result.SetRotation(FQuat(PCGMetadataRotatorHelpers::RLerp(NA.Rotator(), NB.Rotator(), Alpha, false)));
			return Result;
		}
		// Dual quaternion interpolation
		else
		{
			if ((NB.GetRotation() | NA.GetRotation()) < 0.0f)
			{
				NB.SetRotation(NB.GetRotation() * -1.0f);
			}
			return (FDualQuat(NA) * (1 - Alpha) + FDualQuat(NB) * Alpha).Normalized().AsFTransform(FMath::Lerp(NA.GetScale3D(), NB.GetScale3D(), Alpha));
		}
	}

	inline FTransform ApplyTransformOperation(const FTransform& Value1, const FTransform& Value2, double Ratio, EPCGMetadataTransformOperation Operation, EPCGTransformLerpMode Mode)
	{
		switch (Operation)
		{
		case EPCGMetadataTransformOperation::Invert:
			return Value1.Inverse();
		case EPCGMetadataTransformOperation::Compose:
			return Value1 * Value2;
		case EPCGMetadataTransformOperation::Lerp:
			return LerpTransform(Value1, Value2, Ratio, Mode);
		default:
			return FTransform{};
		}
	}
}

void UPCGMetadataTransformSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (Input1AttributeName_DEPRECATED != NAME_None)
	{
		InputSource1.SetAttributeName(Input1AttributeName_DEPRECATED);
		Input1AttributeName_DEPRECATED = NAME_None;
	}

	if (Input2AttributeName_DEPRECATED != NAME_None)
	{
		InputSource2.SetAttributeName(Input2AttributeName_DEPRECATED);
		Input2AttributeName_DEPRECATED = NAME_None;
	}

	if (Input3AttributeName_DEPRECATED != NAME_None)
	{
		InputSource3.SetAttributeName(Input3AttributeName_DEPRECATED);
		Input3AttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

FName UPCGMetadataTransformSettings::GetInputPinLabel(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return (Operation == EPCGMetadataTransformOperation::Invert) ? PCGPinConstants::DefaultInputLabel : PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
	case 1:
		return PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
	case 2:
		return PCGMetadataSettingsBaseConstants::LerpRatioLabel;
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataTransformSettings::GetOperandNum() const
{
	if (Operation == EPCGMetadataTransformOperation::Invert)
	{
		return 1;
	}
	else if (Operation == EPCGMetadataTransformOperation::Lerp)
	{
		return 3;
	}
	else
	{
		return 2;
	}
}

bool UPCGMetadataTransformSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	if (InputIndex == 2)
	{
		bHasSpecialRequirement = true;
		return PCG::Private::IsOfTypes<float, double>(TypeId);
	}
	else
	{
		bHasSpecialRequirement = false;
		return PCG::Private::IsOfTypes<FTransform>(TypeId);
	}
}

FPCGAttributePropertyInputSelector UPCGMetadataTransformSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	case 2:
		return InputSource3;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

FString UPCGMetadataTransformSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataTransformOperation>())
	{
		return FText::Format(NSLOCTEXT("PCGMetadataTransformSettings", "TransformOperation", "Transform: {0}"), EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation))).ToString();
	}
	else
	{
		return FString();
	}
}

#if WITH_EDITOR
FName UPCGMetadataTransformSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeTransformOp");
}

FText UPCGMetadataTransformSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataTransformSettings", "NodeTitle", "Attribute Transform Op");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataTransformSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataTransformOperation>();
}
#endif // WITH_EDITOR

void UPCGMetadataTransformSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataTransformOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataTransformOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

bool UPCGMetadataTransformSettings::IsPinDefaultValueEnabled(const FName PinLabel) const
{
	// @todo_pcg: Add transforms when they are supported.
	return PinLabel == PCGMetadataSettingsBaseConstants::LerpRatioLabel && Super::IsPinDefaultValueEnabled(PinLabel);
}

EPCGMetadataTypes UPCGMetadataTransformSettings::GetPinDefaultValueType(const FName PinLabel) const
{
	return GetPinInitialDefaultValueType(PinLabel);
}

EPCGMetadataTypes UPCGMetadataTransformSettings::GetPinInitialDefaultValueType(const FName PinLabel) const
{
	if (PinLabel == PCGMetadataSettingsBaseConstants::LerpRatioLabel)
	{
		return EPCGMetadataTypes::Double;
	}
	else
	{
		return EPCGMetadataTypes::Transform;
	}
}

FPCGElementPtr UPCGMetadataTransformSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataTransformElement>();
}

bool FPCGMetadataTransformElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataTransformElement::Execute);

	const UPCGMetadataTransformSettings* Settings = CastChecked<UPCGMetadataTransformSettings>(OperationData.Settings);

	if (PCGMetadataTransfromSettings::IsUnaryOp(Settings->Operation))
	{
		return DoUnaryOp<FTransform>(OperationData, [Operation = Settings->Operation, Mode = Settings->TransformLerpMode](const FTransform& Value)->FTransform {
			double DummyDouble = 0.0;
			return PCGMetadataTransfromSettings::ApplyTransformOperation(Value, FTransform{}, DummyDouble, Operation, Mode);
			});
	}
	else if (PCGMetadataTransfromSettings::IsTernaryOp(Settings->Operation))
	{
		return DoTernaryOp<FTransform, FTransform, double>(OperationData, [Operation = Settings->Operation, Mode = Settings->TransformLerpMode](const FTransform& Value1, const FTransform& Value2, const double& Ratio)->FTransform {
			return PCGMetadataTransfromSettings::ApplyTransformOperation(Value1, Value2, Ratio, Operation, Mode);
			});
	}
	else
	{
		return DoBinaryOp<FTransform, FTransform>(OperationData, [Operation = Settings->Operation, Mode = Settings->TransformLerpMode](const FTransform& Value1, const FTransform& Value2)->FTransform {
			double DummyDouble = 0.0;
			return PCGMetadataTransfromSettings::ApplyTransformOperation(Value1, Value2, DummyDouble, Operation, Mode);
			});
	}
}
