// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataTrigOpElement.h"

#include "Elements/Metadata/PCGMetadataElementCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataTrigOpElement)

namespace PCGMetadataTrigSettings
{
	template <typename OutType>
	OutType UnaryOp(const OutType& Input1, EPCGMetadataTrigOperation Operation)
	{
		switch (Operation)
		{
		case EPCGMetadataTrigOperation::Acos:
			return FMath::Acos(Input1);
		case EPCGMetadataTrigOperation::Asin:
			return FMath::Asin(Input1);
		case EPCGMetadataTrigOperation::Atan:
			return FMath::Atan(Input1);
		case EPCGMetadataTrigOperation::Cos:
			return FMath::Cos(Input1);
		case EPCGMetadataTrigOperation::Sin:
			return FMath::Sin(Input1);
		case EPCGMetadataTrigOperation::Tan:
			return FMath::Tan(Input1);
		case EPCGMetadataTrigOperation::DegToRad:
			return FMath::DegreesToRadians(Input1);
		case EPCGMetadataTrigOperation::RadToDeg:
			return FMath::RadiansToDegrees(Input1);
		default:
			return OutType{};
		}
	}

	template <typename OutType>
	OutType BinaryOp(const OutType& Input1, const OutType& Input2)
	{
		// EPCGMetadataTrigOperation::Atan2:
		return FMath::Atan2(Input1, Input2);
	}
}

void UPCGMetadataTrigSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (Input1AttributeName_DEPRECATED != NAME_None)
	{
		InputSource1.SetAttributeName(Input1AttributeName_DEPRECATED);
		Input1AttributeName_DEPRECATED = NAME_None;
	}

	if (Input2AttributeName_DEPRECATED != NAME_None)
	{
		InputSource2.SetAttributeName(Input2AttributeName_DEPRECATED);
		Input2AttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

FName UPCGMetadataTrigSettings::GetInputPinLabel(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return (Operation != EPCGMetadataTrigOperation::Atan2) ? PCGPinConstants::DefaultInputLabel : PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
	case 1:
		return PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataTrigSettings::GetOperandNum() const
{
	return (Operation != EPCGMetadataTrigOperation::Atan2) ? 1 : 2;
}

bool UPCGMetadataTrigSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;
	return PCG::Private::IsOfTypes<int32, int64, float, double>(TypeId);
}

FPCGAttributePropertyInputSelector UPCGMetadataTrigSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

uint16 UPCGMetadataTrigSettings::GetOutputType(uint16 InputTypeId) const
{
	return (uint16)EPCGMetadataTypes::Double;
}

FString UPCGMetadataTrigSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataTrigOperation>())
	{
		return FText::Format(NSLOCTEXT("PCGMetadataTrigSettings", "TrigOperation", "Trig: {0}"), EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation))).ToString();
	}
	else
	{
		return FString();
	}
}

#if WITH_EDITOR
FName UPCGMetadataTrigSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeTrigOp");
}

FText UPCGMetadataTrigSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataTrigSettings", "NodeTitle", "Attribute Trig Op");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataTrigSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataTrigOperation>();
}
#endif // WITH_EDITOR

void UPCGMetadataTrigSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataTrigOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataTrigOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

FPCGElementPtr UPCGMetadataTrigSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataTrigElement>();
}

bool FPCGMetadataTrigElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataTrigElement::Execute);

	const UPCGMetadataTrigSettings* Settings = CastChecked<UPCGMetadataTrigSettings>(OperationData.Settings);

	if (Settings->Operation == EPCGMetadataTrigOperation::Atan2)
	{
		return DoBinaryOp<double, double>(OperationData, [](const double& Value1, const double& Value2) -> double { return PCGMetadataTrigSettings::BinaryOp(Value1, Value2); });
	}
	else
	{
		return DoUnaryOp<double>(OperationData, [Operation = Settings->Operation](const double& Value) -> double { return PCGMetadataTrigSettings::UnaryOp(Value, Operation); });
	}
}
