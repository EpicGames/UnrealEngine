// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGMetadataVectorOpElement.h"

#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataVectorOpElement)

namespace PCGMetadataVectorConstants
{
	const FName AxisLabel = TEXT("Axis");
	const FName AngleLabel = TEXT("Angle(Deg)");
}

namespace PCGMetadataVectorSettings
{
	inline constexpr bool IsUnaryOp(EPCGMetadataVectorOperation Operation)
	{
		return Operation == EPCGMetadataVectorOperation::Normalize || 
			Operation == EPCGMetadataVectorOperation::Length;
	}

	inline constexpr bool IsTernaryOp(EPCGMetadataVectorOperation Operation)
	{
		return Operation == EPCGMetadataVectorOperation::RotateAroundAxis;
	}

	inline constexpr bool IsTransformOp(EPCGMetadataVectorOperation Operation)
	{
		return (uint16)Operation >= (uint16)EPCGMetadataVectorOperation::TransformOp;
	}

	// Mimic KismetMathLibrary
	template <typename InType>
	inline InType ApplyTransformOperation(const InType& Value, const FTransform& Transform, EPCGMetadataVectorOperation Operation)
	{
		switch (Operation)
		{
		case EPCGMetadataVectorOperation::TransformDirection:
		{
			if constexpr (std::is_same_v<InType, FVector2D>)
			{
				// Operation doesn't exist in KismetMathLibrary, do the operation as Vec3, Z as 0 and discard Z in result
				FVector TempValue{ Value, 0.0 };
				return FVector2D{ Transform.TransformVectorNoScale(TempValue) };
			}
			else if constexpr (std::is_same_v<InType, FVector4>)
			{
				return Transform.TransformFVector4NoScale(Value);
			}
			else
			{
				return Transform.TransformVectorNoScale(Value);
			}
		}
		case EPCGMetadataVectorOperation::TransformLocation:
		{
			if constexpr (std::is_same_v<InType, FVector2D>)
			{
				// Operation doesn't exist in KismetMathLibrary, do the operation as Vec3, Z as 0 and discard Z in result
				FVector TempValue{ Value, 0.0 };
				return FVector2D{ Transform.TransformPosition(TempValue) };
			}
			else if constexpr (std::is_same_v<InType, FVector4>)
			{
				// Operation doesn't exist in KismetMathLibrary, do the operation as Vec3 and set W to 0
				FVector TempValue{ Value };
				return FVector4{ Transform.TransformPosition(TempValue), 0.0 };
			}
			else
			{
				return Transform.TransformPosition(Value);
			}
			break;
		}
		case EPCGMetadataVectorOperation::InverseTransformDirection:
		{
			if constexpr (std::is_same_v<InType, FVector2D>)
			{
				// Operation doesn't exist in KismetMathLibrary, do the operation as Vec3, Z as 0 and discard Z in result
				FVector TempValue{ Value, 0.0 };
				return FVector2D{ Transform.InverseTransformVectorNoScale(TempValue) };
			}
			else if constexpr (std::is_same_v<InType, FVector4>)
			{
				// Operation doesn't exist in KismetMathLibrary, do the operation as Vec3 and set W to 0
				FVector TempValue{ Value };
				return FVector4{ Transform.InverseTransformVectorNoScale(TempValue), 0.0 };
			}
			else
			{
				return Transform.InverseTransformVectorNoScale(Value);
			}
			break;
		}
		case EPCGMetadataVectorOperation::InverseTransformLocation:
		{
			if constexpr (std::is_same_v<InType, FVector2D>)
			{
				// Operation doesn't exist in KismetMathLibrary, do the operation as Vec3, Z as 0 and discard Z in result
				FVector TempValue{ Value, 0.0 };
				return FVector2D{ Transform.InverseTransformPosition(TempValue) };
			}
			else if constexpr (std::is_same_v<InType, FVector4>)
			{
				// Operation doesn't exist in KismetMathLibrary, do the operation as Vec3 and set W to 0
				FVector TempValue{ Value };
				return FVector4{ Transform.InverseTransformPosition(TempValue), 0.0 };
			}
			else
			{
				return Transform.InverseTransformPosition(Value);
			}
			break;
		}
		default:
			break;
		}

		return InType{};
	}

	template <typename InType>
	inline InType ApplyVectorOperation(const InType& Value1, const InType& Value2, double& DoubleValue, EPCGMetadataVectorOperation Operation)
	{
		switch (Operation)
		{
		case EPCGMetadataVectorOperation::Cross:
		{
			if constexpr (std::is_same_v<InType, FVector2D>)
			{
				DoubleValue = Value1 ^ Value2;
			}
			else
			{
				return Value1 ^ Value2;
			}
			break;
		}
		case EPCGMetadataVectorOperation::Dot:
		{
			if constexpr (std::is_same_v<FVector4, InType>)
			{
				DoubleValue = Dot4(Value1, Value2);
			}
			else
			{
				DoubleValue = Value1.Dot(Value2);
			}
			break;
		}
		case EPCGMetadataVectorOperation::Distance:
			DoubleValue = (Value1 - Value2).Size();
			break;
		case EPCGMetadataVectorOperation::RotateAroundAxis:
		{
			if constexpr (std::is_same_v<FVector4, InType>)
			{
				FVector TempValue1{ Value1 };
				FVector TempValue2{ Value2 };

				return FVector4{ TempValue1.RotateAngleAxis(DoubleValue, TempValue2), 0.0 };
			}
			else if constexpr (std::is_same_v<FVector2D, InType>)
			{
				// Ignore the axis if it is Vec2
				return Value1.GetRotated(DoubleValue);
			}
			else
			{
				return Value1.RotateAngleAxis(DoubleValue, Value2);
			}
			break;
		}
		case EPCGMetadataVectorOperation::Normalize:
		{
			InType Res = Value1;
			PCG::Private::MetadataTraits<InType>::Normalize(Res);
			return Res;
		}
		case EPCGMetadataVectorOperation::Length:
			DoubleValue = Value1.Size();
			break;
		default:
			break;
		}

		return InType{};
	};
}

void UPCGMetadataVectorSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (Input1AttributeName_DEPRECATED != NAME_None)
	{
		InputSource1.SetAttributeName(Input1AttributeName_DEPRECATED);
		Input1AttributeName_DEPRECATED = NAME_None;
	}

	if (Input2AttributeName_DEPRECATED != NAME_None)
	{
		InputSource2.SetAttributeName(Input2AttributeName_DEPRECATED);
		Input2AttributeName_DEPRECATED = NAME_None;
	}

	if (Input3AttributeName_DEPRECATED != NAME_None)
	{
		InputSource3.SetAttributeName(Input3AttributeName_DEPRECATED);
		Input3AttributeName_DEPRECATED = NAME_None;
	}
#endif // WITH_EDITOR
}

FName UPCGMetadataVectorSettings::GetInputPinLabel(uint32 Index) const
{
	switch (Index)
	{
	case 0:
	{
		if (PCGMetadataVectorSettings::IsUnaryOp(Operation) || 
			PCGMetadataVectorSettings::IsTransformOp(Operation) ||
			Operation == EPCGMetadataVectorOperation::RotateAroundAxis)
		{
			return PCGPinConstants::DefaultInputLabel;
		}
		else
		{
			return PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;
		}
	}
	case 1:
	{
		if (Operation == EPCGMetadataVectorOperation::RotateAroundAxis)
		{
			return PCGMetadataVectorConstants::AxisLabel;
		}
		else if (PCGMetadataVectorSettings::IsTransformOp(Operation))
		{
			return PCGMetadataSettingsBaseConstants::TransformLabel;
		}
		else
		{
			return PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;
		}
	}
	case 2:
		return PCGMetadataVectorConstants::AngleLabel;
	default:
		return NAME_None;
	}
}

uint32 UPCGMetadataVectorSettings::GetOperandNum() const
{
	if (PCGMetadataVectorSettings::IsUnaryOp(Operation))
	{
		return 1;
	}
	else if (PCGMetadataVectorSettings::IsTernaryOp(Operation))
	{
		return 3;
	}
	else
	{
		return 2;
	}
}

bool UPCGMetadataVectorSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;

	if (Operation == EPCGMetadataVectorOperation::EulerBetweenVectors || Operation == EPCGMetadataVectorOperation::RotatorBetweenVectors)
	{
		return PCG::Private::IsOfTypes<FVector, FVector2D>(TypeId);
	}
	else if (Operation == EPCGMetadataVectorOperation::AngleBetweenVector2D)
	{
		return PCG::Private::IsOfTypes<FVector2D>(TypeId);
	}
	else if (Operation == EPCGMetadataVectorOperation::RotateAroundAxis && InputIndex == 2)
	{
		bHasSpecialRequirement = true;
		return PCG::Private::IsOfTypes<double, float>(TypeId);
	}
	else if (PCGMetadataVectorSettings::IsTransformOp(Operation) && InputIndex == 1)
	{
		bHasSpecialRequirement = true;
		return PCG::Private::IsOfTypes<FTransform>(TypeId);
	}
	else
	{
		return PCG::Private::IsOfTypes<FVector2D, FVector, FVector4>(TypeId);
	}
}

FPCGAttributePropertyInputSelector UPCGMetadataVectorSettings::GetInputSource(uint32 Index) const
{
	switch (Index)
	{
	case 0:
		return InputSource1;
	case 1:
		return InputSource2;
	case 2:
		return InputSource3;
	default:
		return FPCGAttributePropertyInputSelector();
	}
}

FString UPCGMetadataVectorSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataVectorOperation>())
	{
		return FText::Format(NSLOCTEXT("PCGMetadataVectorSettings", "VectorOperation", "Vector: {0}"), EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation))).ToString();
	}
	else
	{
		return FString();
	}
}

#if WITH_EDITOR
FName UPCGMetadataVectorSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeVectorOp");
}

FText UPCGMetadataVectorSettings::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("PCGMetadataVectorSettings", "NodeTitle", "Attribute Vector Op");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGMetadataVectorSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataVectorOperation>({ EPCGMetadataVectorOperation::VectorOp, EPCGMetadataVectorOperation::TransformOp });
}
#endif // WITH_EDITOR

void UPCGMetadataVectorSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataVectorOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGMetadataVectorOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

bool UPCGMetadataVectorSettings::IsPinDefaultValueEnabled(const FName PinLabel) const
{
	// @todo_pcg: Remove this when transforms are enabled.
	if (PCGMetadataVectorSettings::IsTransformOp(Operation) && PinLabel == GetInputPinLabel(1))
	{
		return false;
	}

	return Super::IsPinDefaultValueEnabled(PinLabel);
}

EPCGMetadataTypes UPCGMetadataVectorSettings::GetPinInitialDefaultValueType(const FName PinLabel) const
{
	// Only the angle pin on Rotate Around Axis should be double, otherwise vector.
	return Operation == EPCGMetadataVectorOperation::RotateAroundAxis && PinLabel == PCGMetadataVectorConstants::AngleLabel
		? EPCGMetadataTypes::Double
		: EPCGMetadataTypes::Vector;
}

FPCGElementPtr UPCGMetadataVectorSettings::CreateElement() const
{
	return MakeShared<FPCGMetadataVectorElement>();
}

uint16 UPCGMetadataVectorSettings::GetOutputType(uint16 InputTypeId) const
{
	// Dot, Length and Cross product with Vec2 output Double values
	if (Operation == EPCGMetadataVectorOperation::Dot ||
		Operation == EPCGMetadataVectorOperation::Length ||
		Operation == EPCGMetadataVectorOperation::Distance ||
		Operation == EPCGMetadataVectorOperation::AngleBetweenVector2D ||
		(Operation == EPCGMetadataVectorOperation::Cross && InputTypeId == (uint16)EPCGMetadataTypes::Vector2))
	{
		return (uint16)EPCGMetadataTypes::Double;
	}
	else if (Operation == EPCGMetadataVectorOperation::EulerBetweenVectors)
	{
		return (uint16)EPCGMetadataTypes::Vector;
	}
	else if (Operation == EPCGMetadataVectorOperation::RotatorBetweenVectors)
	{
		return (uint16)EPCGMetadataTypes::Rotator;
	}
	else
	{
		// Otherwise, output the input type
		return InputTypeId;
	}
}

bool FPCGMetadataVectorElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataVectorElement::Execute);

	const UPCGMetadataVectorSettings* Settings = CastChecked<UPCGMetadataVectorSettings>(OperationData.Settings);

	auto VectorFunc = [this, Operation = Settings->Operation, &OperationData]<typename AttributeType>(AttributeType) -> bool
	{
		if constexpr (!PCG::Private::IsOfTypes<AttributeType, FVector2D, FVector, FVector4>())
		{
			ensure(false);
			return true;
		}
		else
		{
			if (PCGMetadataVectorSettings::IsTransformOp(Operation))
			{
				return DoBinaryOp<AttributeType, FTransform>(OperationData, [Operation](const AttributeType& Value, const FTransform& Transform) -> AttributeType { return PCGMetadataVectorSettings::ApplyTransformOperation(Value, Transform, Operation);});
			}
			else if (PCG::Private::IsOfTypes<double>(OperationData.OutputType))
			{
				if (PCGMetadataVectorSettings::IsUnaryOp(Operation))
				{
					return DoUnaryOp<AttributeType>(OperationData, [Operation](const AttributeType& Value) -> double {
						double Res = 0.0;
						PCGMetadataVectorSettings::ApplyVectorOperation(Value, AttributeType{}, Res, Operation);
						return Res;
						});
				}
				else if (Operation == EPCGMetadataVectorOperation::AngleBetweenVector2D)
				{
					return DoBinaryOp<FVector2D, FVector2D>(OperationData, [Operation](const FVector2D& Value1, const FVector2D& Value2) -> double {
							return FMath::RadiansToDegrees(FGenericPlatformMath::Atan2(Value1 ^ Value2, Value1 | Value2 ));
						});
				}
				else
				{
					return DoBinaryOp<AttributeType, AttributeType>(OperationData, [Operation](const AttributeType& Value1, const AttributeType& Value2) -> double {
						double Res = 0.0;
						PCGMetadataVectorSettings::ApplyVectorOperation(Value1, Value2, Res, Operation);
						return Res;
						});
				}
			}
			else if (Operation == EPCGMetadataVectorOperation::EulerBetweenVectors)
			{
				return DoBinaryOp<FVector, FVector>(OperationData, [Operation](const FVector& Value1, const FVector& Value2) -> FVector {
						return FQuat::FindBetweenVectors(Value1, Value2).Euler();
					});
			}
			else if (Operation == EPCGMetadataVectorOperation::RotatorBetweenVectors)
			{
				return DoBinaryOp<FVector, FVector>(OperationData, [Operation](const FVector& Value1, const FVector& Value2) -> FRotator {
						return FQuat::FindBetweenVectors(Value1, Value2).Rotator();
					});
			}
			else
			{
				if (PCGMetadataVectorSettings::IsUnaryOp(Operation))
				{
					return DoUnaryOp<AttributeType>(OperationData, [Operation](const AttributeType& Value) -> AttributeType {
						double DummyDouble = 0.0;
						return PCGMetadataVectorSettings::ApplyVectorOperation(Value, AttributeType{}, DummyDouble, Operation);
						});
				}
				else if (PCGMetadataVectorSettings::IsTernaryOp(Operation))
				{
					return DoTernaryOp<AttributeType, AttributeType, double>(OperationData, [Operation](const AttributeType& Value1, const AttributeType& Value2, const double& DoubleValue) -> AttributeType {
						double DoubleCopy = DoubleValue;
						return PCGMetadataVectorSettings::ApplyVectorOperation(Value1, Value2, DoubleCopy, Operation);
						});
				}
				else
				{
					return DoBinaryOp<AttributeType, AttributeType>(OperationData, [Operation](const AttributeType& Value1, const AttributeType& Value2) -> AttributeType {
						double DummyDouble = 0.0;
						return PCGMetadataVectorSettings::ApplyVectorOperation(Value1, Value2, DummyDouble, Operation);
						});
				}
			}
		}
	};

	return PCGMetadataAttribute::CallbackWithRightType(OperationData.MostComplexInputType, VectorFunc);
}
