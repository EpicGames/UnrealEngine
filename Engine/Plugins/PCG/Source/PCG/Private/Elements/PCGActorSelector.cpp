// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGActorSelector.h"

#include "PCGComponent.h"
#include "PCGIsolatedActor.h"
#include "PCGModule.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGActorHelpers.h"

#include "Algo/AnyOf.h"
#include "Algo/Transform.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGActorSelector)

namespace PCGActorSelector
{
	template<typename NameRangeType>
	bool TagListMatchesTag(const NameRangeType& Range, const FName& TargetTag, bool bUseWildcardSearch, const FString& WildcardTag)
	{
		if (bUseWildcardSearch)
		{
			return Algo::AnyOf(Range, [&WildcardTag](const FName& TagName) { return TagName.ToString().MatchesWildcard(WildcardTag); });
		}
		else
		{
			return Range.Contains(TargetTag);
		}
	}

	void PrepareTag(bool bWillUseTag, FName InTag, bool& OutTagContainsWildcard, FString& OutTagString)
	{
		if (bWillUseTag)
		{
			int32 DummyIndex;
			FString TagString = InTag.ToString();
			if (TagString.FindChar(TCHAR('*'), DummyIndex) || TagString.FindChar(TCHAR('?'), DummyIndex))
			{
				OutTagString = std::move(TagString);
				OutTagContainsWildcard = true;
			}
			else
			{
				OutTagString = FString();
				OutTagContainsWildcard = false;
			}
		}
		else
		{
			OutTagString = FString();
			OutTagContainsWildcard = false;
		}
	}

	// Filter is required if it is not disabled and if we are gathering all world actors or gathering all children.
	bool FilterRequired(const FPCGActorSelectorSettings& InSettings)
	{
		return (InSettings.ActorFilter == EPCGActorFilter::AllWorldActors || InSettings.bIncludeChildren) && !InSettings.bDisableFilter;
	}

	// Need to pass a pointer of pointer to the found actor. The lambda will capture this pointer and modify its value when an actor is found.
	// Note: depending on the filtering type, the settings must stay alive longer than the returned filtering function here.
	TFunction<bool(AActor*)> GetFilteringFunction(const FPCGActorSelectorSettings& InSettings, const TFunction<bool(const AActor*)>& BoundsCheck, const TFunction<bool(const AActor*)>& SelfIgnoreCheck, TArray<AActor*>& InFoundActors)
	{
		InSettings.PrepareForFiltering(/*bForce=*/true);

		if (!FilterRequired(InSettings))
		{
			return [&InFoundActors, &BoundsCheck, &SelfIgnoreCheck](AActor* Actor) -> bool
			{
				if (BoundsCheck(Actor) && SelfIgnoreCheck(Actor))
				{
					InFoundActors.Add(Actor);
				}
				return true;
			};
		}

		const bool bMultiSelect = InSettings.bSelectMultiple;

		switch (InSettings.ActorSelection)
		{
		case EPCGActorSelection::ByTag:
			return[&InSettings, &InFoundActors, bMultiSelect, &BoundsCheck, &SelfIgnoreCheck](AActor* Actor) -> bool
			{
				if(Actor && InSettings.MatchesTag(Actor) && BoundsCheck(Actor) && SelfIgnoreCheck(Actor))
				{
					InFoundActors.Add(Actor);
					return bMultiSelect;
				}

				return true;
			};

		case EPCGActorSelection::ByClass:
			return[ActorSelectionClass = InSettings.ActorSelectionClass, &InFoundActors, bMultiSelect, &BoundsCheck, &SelfIgnoreCheck](AActor* Actor) -> bool
			{
				if (Actor && Actor->IsA(ActorSelectionClass) && BoundsCheck(Actor) && SelfIgnoreCheck(Actor))
				{
					InFoundActors.Add(Actor);
					return bMultiSelect;
				}

				return true;
			};

		case EPCGActorSelection::ByName:
			UE_LOGF(LogPCG, Error, "PCGActorSelector::GetFilteringFunction: Unsupported value for EPCGActorSelection - selection by name is no longer supported.");
			break;

		default:
			break;
		}

		return {};
	}

	TArray<AActor*> FindActors(const FPCGActorSelectorSettings& Settings, const IPCGGraphExecutionSource* InExecutionSource, const TFunction<bool(const AActor*)>& BoundsCheck, const TFunction<bool(const AActor*)>& SelfIgnoreCheck, TArrayView<AActor*> InputActors)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGActorSelector::FindActor);

		UWorld* World = InExecutionSource ? InExecutionSource->GetExecutionState().GetWorld() : nullptr;
		AActor* Self = InExecutionSource ? InExecutionSource->GetExecutionState().GetTypedTarget<AActor>() : nullptr;

		TArray<AActor*> FoundActors;

		if (!World)
		{
			return FoundActors;
		}

		// Early out if we have not the information necessary.
		const bool bNoTagInfo = Settings.ActorSelection == EPCGActorSelection::ByTag && Settings.ActorSelectionTag == NAME_None;
		const bool bNoClassInfo = Settings.ActorSelection == EPCGActorSelection::ByClass && !Settings.ActorSelectionClass;

		if (FilterRequired(Settings) && (bNoTagInfo || bNoClassInfo))
		{
			return FoundActors;
		}

		// We pass FoundActor ref, that will be captured by the FilteringFunction
		// It will modify the FoundActor pointer to the found actor, if found.
		TFunction<bool(AActor*)> FilteringFunction = PCGActorSelector::GetFilteringFunction(Settings, BoundsCheck, SelfIgnoreCheck, FoundActors);

		if (!FilteringFunction)
		{
			return FoundActors;
		}

		// And after filtering, check for adding isolated actors
		auto UpdateFoundActorsWithIsolatedIfNeeded = [IncludeIsolatedActors = Settings.IncludeIsolatedActors](TArray<AActor*>& InOutFoundActors)
		{
			if (IncludeIsolatedActors != EPCGIsolatedActorIncludeFlag::Ignore)
			{
				int ActorIndex = 0;
				while (ActorIndex < InOutFoundActors.Num())
				{
					TArray<APCGIsolatedActor*> IsolatedActors = UPCGActorHelpers::GetIsolatedActorsOriginatingFrom(InOutFoundActors[ActorIndex]);

					if (IncludeIsolatedActors == EPCGIsolatedActorIncludeFlag::Include)
					{
						++ActorIndex;

						if (IsolatedActors.Num() > 0)
						{
							AActor** IsolatedActorsPtr = reinterpret_cast<AActor**>(IsolatedActors.GetData());
							InOutFoundActors.Insert(IsolatedActorsPtr, IsolatedActors.Num(), ActorIndex);
							ActorIndex += IsolatedActors.Num();
						}
					}
					else if (IncludeIsolatedActors == EPCGIsolatedActorIncludeFlag::Only)
					{
						if (IsolatedActors.IsEmpty())
						{
							InOutFoundActors.RemoveAt(ActorIndex);
						}
						else
						{
							InOutFoundActors[ActorIndex++] = IsolatedActors[0];
							if (IsolatedActors.Num() > 1)
							{
								AActor** IsolatedActorsPtr = reinterpret_cast<AActor**>(IsolatedActors.GetData());
								InOutFoundActors.Insert(IsolatedActorsPtr + 1, IsolatedActors.Num() - 1, ActorIndex);
								ActorIndex += IsolatedActors.Num() - 1;
							}
						}
					}
				}
			}				
		};
		
		// In case of iterating over all actors in the world, call our filtering function and get out.
		if (Settings.ActorFilter == EPCGActorFilter::AllWorldActors)
		{
			// If we are going to filter by class in the FilteringFunction, might as well avoid the cost of iterating over all actor types (which can get really expensive)
			const TSubclassOf<AActor> ActorSelectionClass = Settings.ActorSelection == EPCGActorSelection::ByClass && Settings.ActorSelectionClass ? Settings.ActorSelectionClass.Get() : AActor::StaticClass();
			// A potential optimization if we know the sought actors are collide-able could be to obtain overlaps via a collision query.
			UPCGActorHelpers::ForEachActorInWorld(World, ActorSelectionClass, FilteringFunction);
			UpdateFoundActorsWithIsolatedIfNeeded(FoundActors);
			
			// FoundActor is set by the FilteringFunction (captured)
			return FoundActors;
		}

		// Otherwise, gather all the actors we need to check
		TArray<AActor*> ActorsToCheck;
		switch (Settings.ActorFilter)
		{
		case EPCGActorFilter::Self:
			if (Self)
			{
				ActorsToCheck.Add(Self);
			}
			break;

		case EPCGActorFilter::Parent:
			if (Self)
			{
				if (AActor* Parent = Self->GetParentActor())
				{
					ActorsToCheck.Add(Parent);
				}
				else
				{
					// If there is no parent, set the owner as the parent.
					ActorsToCheck.Add(Self);
				}
			}
			break;

		case EPCGActorFilter::Root:
		{
			AActor* Current = Self;
			while (Current != nullptr)
			{
				AActor* Parent = Current->GetParentActor();
				if (Parent == nullptr)
				{
					ActorsToCheck.Add(Current);
					break;
				}
				Current = Parent;
			}

			break;
		}

		case EPCGActorFilter::Original:
		{
			IPCGGraphExecutionSource* OriginalSource = InExecutionSource ? InExecutionSource->GetExecutionState().GetOriginalSource() : nullptr;
			AActor* OriginalActor = OriginalSource ? OriginalSource->GetExecutionState().GetTypedTarget<AActor>() : nullptr;
			if (OriginalActor)
			{
				ActorsToCheck.Add(OriginalActor);
			}
			else if (Self)
			{
				ActorsToCheck.Add(Self);
			}

			break;
		}

		case EPCGActorFilter::FromInput:
		{
			ActorsToCheck.Append(InputActors);
		}

		default:
			break;
		}

		if (Settings.bIncludeChildren)
		{
			const int32 InitialCount = ActorsToCheck.Num();
			for (int32 i = 0; i < InitialCount; ++i)
			{
				ActorsToCheck[i]->GetAttachedActors(ActorsToCheck, /*bResetArray=*/ false, /*bRecursivelyIncludeAttachedActors=*/ true);
			}
		}

		for (AActor* Actor : ActorsToCheck)
		{
			// FoundActor is set by the FilteringFunction (captured)
			if (!FilteringFunction(Actor))
			{
				break;
			}
		}

		UpdateFoundActorsWithIsolatedIfNeeded(FoundActors);

		return FoundActors;
	}

	TArray<AActor*> FilterActors(const FPCGComponentSelectorSettings& Settings, TArrayView<AActor*> InputActors)
	{
		TArray<AActor*> FilteredActors;
		FilteredActors.Reserve(InputActors.Num());

		for (AActor* InputActor : InputActors)
		{
			if (InputActor && Settings.FilterActor(InputActor))
			{
				FilteredActors.Add(InputActor);
			}
		}

		return FilteredActors;
	}

	TArray<AActor*> FindActors(const FPCGActorSelectorSettings* ActorSettings, const FPCGComponentSelectorSettings* ComponentSettings, const IPCGGraphExecutionSource* InExecutionSource, const TFunction<bool(const AActor*)>& BoundsCheck, const TFunction<bool(const AActor*)>& SelfIgnoreCheck, TArrayView<AActor*> InputActors)
	{
		TArray<AActor*> FoundActors;
		if (ActorSettings)
		{
			FoundActors = FindActors(*ActorSettings, InExecutionSource, BoundsCheck, SelfIgnoreCheck, InputActors);
		}
		else
		{
			FoundActors = InputActors;
		}

		if (!FoundActors.IsEmpty() && ComponentSettings)
		{
			FoundActors = FilterActors(*ComponentSettings, FoundActors);
		}

		return FoundActors;
	}

	AActor* FindActor(const FPCGActorSelectorSettings& InSettings, const IPCGGraphExecutionSource* InExecutionSource, const TFunction<bool(const AActor*)>& BoundsCheck, const TFunction<bool(const AActor*)>& SelfIgnoreCheck, TArrayView<AActor*> InputActors)
	{
		// In order to make sure we don't try to select multiple, we'll do a copy of the settings here.
		FPCGActorSelectorSettings Settings = InSettings;
		Settings.bSelectMultiple = false;

		TArray<AActor*> Actors = FindActors(Settings, InExecutionSource, BoundsCheck, SelfIgnoreCheck, InputActors);
		return Actors.IsEmpty() ? nullptr : Actors[0];
	}
}

FPCGSelectionKey::FPCGSelectionKey(const FPCGActorSelectorSettings& InActorSelector)
{
	check(InActorSelector.ActorFilter != EPCGActorFilter::FromInput);
	ActorFilter = InActorSelector.ActorFilter;
	if (ActorFilter == EPCGActorFilter::AllWorldActors)
	{
		check(InActorSelector.ActorSelection == EPCGActorSelection::ByTag || InActorSelector.ActorSelection == EPCGActorSelection::ByClass);
		Selection = InActorSelector.ActorSelection;

		if (Selection == EPCGActorSelection::ByTag)
		{
			Tag = InActorSelector.ActorSelectionTag;
		}
		else
		{
			SelectionClass = InActorSelector.ActorSelectionClass;
		}
	}
}

FPCGSelectionKey::FPCGSelectionKey(EPCGActorFilter InFilter)
{
	check(InFilter != EPCGActorFilter::AllWorldActors && InFilter != EPCGActorFilter::FromInput);
	ActorFilter = InFilter;
}

FPCGSelectionKey::FPCGSelectionKey(FName InTag)
{
	Selection = EPCGActorSelection::ByTag;
	Tag = InTag;
	ActorFilter = EPCGActorFilter::AllWorldActors;
	UpdateAfterTagChange();
}

FPCGSelectionKey::FPCGSelectionKey(TSubclassOf<UObject> InSelectionClass)
{
	Selection = EPCGActorSelection::ByClass;
	SelectionClass = InSelectionClass;
	ActorFilter = EPCGActorFilter::AllWorldActors;
}

FPCGSelectionKey FPCGSelectionKey::CreateFromPath(const FSoftObjectPath& InObjectPath)
{
	FPCGSelectionKey Key{};
	Key.Selection = EPCGActorSelection::ByPath;
	Key.ObjectPath = InObjectPath;
	Key.ActorFilter = EPCGActorFilter::AllWorldActors;

	return Key;
}

FPCGSelectionKey FPCGSelectionKey::CreateFromPath(FSoftObjectPath&& InObjectPath)
{
	FPCGSelectionKey Key{};
	Key.Selection = EPCGActorSelection::ByPath;
	Key.ObjectPath = std::forward<FSoftObjectPath>(InObjectPath);
	Key.ActorFilter = EPCGActorFilter::AllWorldActors;

	return Key;
}

FPCGSelectionKey FPCGSelectionKey::CreateFromObjectPtr(const UObject* InObject)
{
	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(FSoftObjectPath(InObject));
	Key.CachedObject = InObject;

	return Key;
}

void FPCGSelectionKey::SetExtraDependency(const UClass* InExtraDependency)
{
	OptionalExtraDependency = InExtraDependency;
}

// DEPRECATED 5.7
bool FPCGSelectionKey::IsMatching(const UObject* InObject, const FSoftObjectPath& InObjectPath, const UPCGComponent* InComponent) const
{
	// This signature of IsMatching does not support custom keys
	if (CustomKey.IsValid())
	{
		return false;
	}

	const FPCGSelectionKey SelectionKey = FPCGSelectionKey::CreateFromPath(InObjectPath);
	SelectionKey.CachedObject = InObject;
	return IsMatching(SelectionKey, InComponent);
}

bool FPCGSelectionKey::IsMatching(const FPCGSelectionKey& InSelectionKey, const IPCGGraphExecutionSource* InExecutionSource) const
{
	const UObject* SelectionObject = InSelectionKey.GetObjectFromPath();
	if (!SelectionObject)
	{
		return false;
	}

	// If we filter something else than all world actors, matching depends on the component.
	// Re-use the same mechanism than Get Actor Data, which should be cheap since we don't look for all actors in the world.
	if (ActorFilter != EPCGActorFilter::AllWorldActors)
	{
		const AActor* InActor = Cast<const AActor>(SelectionObject);

		if (!InActor)
		{
			return false;
		}

		// InKey provide the info for selecting a given actor.
		// We reconstruct the selector settings from this key, and we also force it to SelectMultiple, since
		// we want to gather all the actors that matches this given key, to find if ours matches.
		FPCGActorSelectorSettings SelectorSettings = FPCGActorSelectorSettings::ReconstructFromKey(*this);
		SelectorSettings.bSelectMultiple = true;
		TArray<AActor*> AllActors = PCGActorSelector::FindActors(SelectorSettings, InExecutionSource, [](const AActor*) { return true; }, [](const AActor*) { return true; });
		return AllActors.Contains(InActor);
	}

	switch (Selection)
	{
	case EPCGActorSelection::ByTag:
	{
		const AActor* InActor = Cast<const AActor>(SelectionObject);
		return InActor && PCGActorSelector::TagListMatchesTag(InActor->Tags, Tag, bTagContainsWildcard, CachedTagString);
	}
	case EPCGActorSelection::ByClass:
		return SelectionObject && SelectionObject->GetClass()->IsChildOf(SelectionClass);
	case EPCGActorSelection::ByPath:
		return ObjectPath == InSelectionKey.ObjectPath;
	default:
		return false;
	}
}

// deprecated
bool FPCGSelectionKey::IsMatching(const UObject* InObject, const FSoftObjectPath& InObjectPath, const TSet<FName>& InRemovedTags, const TSet<UPCGComponent*>& InComponents, TSet<UPCGComponent*>* OptionalMatchedComponents) const
{
	// This signature of IsMatching does not support custom keys
	if (CustomKey.IsValid())
	{
		return false;
	}

	FPCGSelectionKey SelectionKey = FPCGSelectionKey::CreateFromPath(InObjectPath);
	SelectionKey.CachedObject = InObject;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return IsMatching(SelectionKey, InRemovedTags, InComponents, OptionalMatchedComponents);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

// deprecated
bool FPCGSelectionKey::IsMatching(const FPCGSelectionKey& InSelectionKey, const TSet<FName>& InRemovedTags, const TSet<UPCGComponent*>& InComponents, TSet<UPCGComponent*>* OptionalMatchedComponents) const
{
	TSet<IPCGGraphExecutionSource*> OptionalMatchedExecutionSources;
	TSet<IPCGGraphExecutionSource*> ExecutionSources;
	ExecutionSources.Reserve(InComponents.Num());
	Algo::Transform(InComponents, ExecutionSources, [](UPCGComponent* InComponent) { return InComponent; });

	bool bReturn = IsMatching(InSelectionKey, InRemovedTags, ExecutionSources, OptionalMatchedComponents ? &OptionalMatchedExecutionSources : nullptr);
	if (OptionalMatchedComponents)
	{
		OptionalMatchedComponents->Reserve(OptionalMatchedExecutionSources.Num());
		Algo::TransformIf(OptionalMatchedExecutionSources, *OptionalMatchedComponents,
			[](IPCGGraphExecutionSource* InExecutionSource) { return Cast<UPCGComponent>(InExecutionSource) != nullptr; },
			[](IPCGGraphExecutionSource* InExecutionSource) { return Cast<UPCGComponent>(InExecutionSource); });
	}

	return bReturn;
}

bool FPCGSelectionKey::IsMatching(const FPCGSelectionKey& InSelectionKey, const TSet<FName>& InRemovedTags, const TSet<IPCGGraphExecutionSource*>& InExecutionSources, TSet<IPCGGraphExecutionSource*>* OptionalMatchedExecutionSources) const
{
	const UObject* SelectionObject = InSelectionKey.GetObjectFromPath();

	if (!SelectionObject)
	{
		return false;
	}

	// Check custom key first
	if (CustomKey.IsValid() != InSelectionKey.CustomKey.IsValid())
	{
		return false;
	}
	else if (CustomKey.IsValid())
	{
		if (!CustomKey.Get().IsMatching(InSelectionKey.CustomKey.Get()))
		{
			return false;
		}
	}

	// If both keys (this & InSelectionKey) have optional bounds, then check that there is an overlap between any pair.
	if (!OptionalBounds.IsEmpty() && !InSelectionKey.OptionalBounds.IsEmpty())
	{
		bool bHasOverlap = false;
		for (const FBox& Bounds : OptionalBounds)
		{
			if (!Bounds.IsValid)
			{
				continue;
			}

			for (const FBox& OtherBounds : InSelectionKey.OptionalBounds)
			{
				if (!OtherBounds.IsValid)
				{
					continue;
				}

				if (Bounds.Intersect(OtherBounds))
				{
					bHasOverlap = true;
					break;
				}
			}

			if (bHasOverlap)
			{
				break;
			}
		}

		if (!bHasOverlap)
		{
			return false;
		}
	}

	// If we filter something else than all world actors, matching depends on the component.
	// Since we can have a lot of components in InComponents, we go the other way around (Actor to component)
	if (ActorFilter != EPCGActorFilter::AllWorldActors)
	{
		FPCGGetExecutionSourcesFromSelectionKeyParams Params(InSelectionKey, ActorFilter);
		TArray<IPCGGraphExecutionSource*> FoundExecutionSources = FPCGModule::GetPCGModuleChecked().GetConstChangeTrackingRegistry().GetExecutionSourcesFromSelectionKey(Params);
		
		bool bFoundMatch = false;

		for (IPCGGraphExecutionSource* FoundExecutionSource : FoundExecutionSources)
		{
			if (InExecutionSources.Contains(FoundExecutionSource))
			{
				bFoundMatch = true;
				if (OptionalMatchedExecutionSources)
				{
					OptionalMatchedExecutionSources->Add(FoundExecutionSource);
				}
				else
				{
					break;
				}
			}
		}

		return bFoundMatch;
	}

	bool bIsMatched = false;
	switch (Selection)
	{
	case EPCGActorSelection::ByTag:
	{
		const AActor* InActor = Cast<const AActor>(SelectionObject);
		bIsMatched = PCGActorSelector::TagListMatchesTag(InRemovedTags, Tag, bTagContainsWildcard, CachedTagString) ||
			(InActor && PCGActorSelector::TagListMatchesTag(InActor->Tags, Tag, bTagContainsWildcard, CachedTagString));
		break;
	}
	case EPCGActorSelection::ByClass:
		bIsMatched = SelectionObject->IsA(SelectionClass);
		break;
	case EPCGActorSelection::ByPath:
		bIsMatched = InSelectionKey.ObjectPath == ObjectPath;
		break;
	default:
		bIsMatched = false;
		break;
	}

	if (bIsMatched && OptionalMatchedExecutionSources)
	{
		OptionalMatchedExecutionSources->Append(InExecutionSources);
	}

	return bIsMatched;
}

bool FPCGSelectionKey::operator==(const FPCGSelectionKey& InOther) const
{
	if (ActorFilter != InOther.ActorFilter || Selection != InOther.Selection || OptionalExtraDependency != InOther.OptionalExtraDependency)
	{
		return false;
	}

	if (CustomKey.IsValid() != InOther.CustomKey.IsValid())
	{
		return false;
	}
	else if (CustomKey.IsValid())
	{
		if (!CustomKey.Get().Equals(InOther.CustomKey.Get()))
		{
			return false;
		}
	}

	if (OptionalBounds != InOther.OptionalBounds)
	{
		return false;
	}

	switch (Selection)
	{
	case EPCGActorSelection::ByTag:
		return Tag == InOther.Tag;
	case EPCGActorSelection::ByClass:
		return SelectionClass == InOther.SelectionClass;
	case EPCGActorSelection::ByPath:
		return ObjectPath == InOther.ObjectPath;
	case EPCGActorSelection::Unknown: // Fall-through
	case EPCGActorSelection::ByName:
		return true;
	default:
	{
		checkNoEntry();
		return true;
	}
	}
}

void FPCGSelectionKey::UpdateAfterTagChange()
{
	PCGActorSelector::PrepareTag(Selection == EPCGActorSelection::ByTag, Tag, bTagContainsWildcard, CachedTagString);
}

const UObject* FPCGSelectionKey::GetObjectFromPath() const
{
	if (CachedObject)
	{
		return CachedObject;
	}

	if (ObjectPath.IsValid())
	{
		CachedObject = ObjectPath.ResolveObject();
	}

	return CachedObject;
}

FArchive& operator<<(FArchive& Ar, FPCGSelectionKey& Key)
{
	// Serialize the normal UPROPERTY data
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		if (UScriptStruct* ThisStruct = FPCGSelectionKey::StaticStruct())
		{
			ThisStruct->SerializeTaggedProperties(Ar, reinterpret_cast<uint8*>(&Key), ThisStruct, nullptr);
		}

		if (Ar.IsLoading())
		{
			Key.UpdateAfterTagChange();
		}
	}

	return Ar;
}

uint32 GetTypeHash(const FPCGCustomSelectionKey& In)
{
	return In.GetTypeHashInternal();
}

uint32 GetTypeHash(const FPCGSelectionKey& In)
{
	uint32 HashResult = HashCombine(GetTypeHash(In.ActorFilter), GetTypeHash(In.Selection));
	HashResult = HashCombine(HashResult, GetTypeHash(In.Tag));
	HashResult = HashCombine(HashResult, GetTypeHash(In.SelectionClass));
	HashResult = HashCombine(HashResult, GetTypeHash(In.OptionalExtraDependency));
	HashResult = HashCombine(HashResult, GetTypeHash(In.ObjectPath));

	if (In.CustomKey.IsValid())
	{
		HashResult = HashCombine(HashResult, GetTypeHash(In.CustomKey.Get()));
	}

	HashResult = HashCombine(HashResult, GetTypeHash(In.OptionalBounds));

	return HashResult;
}

#if WITH_EDITOR
FText FPCGActorSelectorSettings::GetTaskName() const
{
	if (ActorFilter == EPCGActorFilter::AllWorldActors)
	{
		if (ActorSelection == EPCGActorSelection::ByClass)
		{
			return NSLOCTEXT("PCGActorSelectorSettings", "ClassLabel", "Class");
		}
		else if (ActorSelection == EPCGActorSelection::ByTag)
		{
			return NSLOCTEXT("PCGActorSelectorSettings", "TagLabel", "Tag");
		}
	}
	else if(const UEnum* EnumPtr = StaticEnum<EPCGActorFilter>())
	{
		return EnumPtr->GetDisplayNameTextByValue(static_cast<__underlying_type(EPCGActorFilter)>(ActorFilter));
	}

	return FText();
}

FText FPCGActorSelectorSettings::GetTaskNameSuffix() const
{
	if (ActorFilter == EPCGActorFilter::AllWorldActors)
	{
		if (ActorSelection == EPCGActorSelection::ByClass)
		{
			return ActorSelectionClass.Get() ? ActorSelectionClass->GetDisplayNameText() : FText::FromName(NAME_None);
		}
		else if (ActorSelection == EPCGActorSelection::ByTag)
		{
			return FText::FromName(ActorSelectionTag);
		}
	}

	return FText();
}
#endif // WITH_EDITOR

FPCGSelectionKey FPCGActorSelectorSettings::GetAssociatedKey() const
{
	if (ActorFilter == EPCGActorFilter::AllWorldActors)
	{
		switch (ActorSelection)
		{
		case EPCGActorSelection::ByTag:
			return FPCGSelectionKey(ActorSelectionTag);
		case EPCGActorSelection::ByClass:
			return FPCGSelectionKey(ActorSelectionClass);
		default:
			return FPCGSelectionKey();
		}
	}
	else if (ActorFilter == EPCGActorFilter::FromInput)
	{
		return FPCGSelectionKey();
	}
	else
	{
		return FPCGSelectionKey(ActorFilter);
	}
}

FPCGActorSelectorSettings FPCGActorSelectorSettings::ReconstructFromKey(const FPCGSelectionKey& InKey)
{
	if (InKey.SelectionClass && !InKey.SelectionClass->IsChildOf<AActor>())
	{
		return FPCGActorSelectorSettings{};
	}

	FPCGActorSelectorSettings Result{};
	Result.ActorFilter = InKey.ActorFilter;
	Result.ActorSelection = InKey.Selection;
	Result.ActorSelectionTag = InKey.Tag;
	Result.ActorSelectionClass = InKey.SelectionClass;
	Result.PrepareForFiltering();

	return Result;
}

void FPCGActorSelectorSettings::PrepareForFiltering(bool bForce) const
{
	if (bForce || !bHasPreparedTag)
	{
		PCGActorSelector::PrepareTag(ActorSelection == EPCGActorSelection::ByTag, ActorSelectionTag, bTagContainsWildcards, ActorSelectionTagString);
		bHasPreparedTag = true;
	}
}

bool FPCGActorSelectorSettings::MatchesTag(AActor* Actor) const
{
	PrepareForFiltering();
	return PCGActorSelector::TagListMatchesTag(Actor->Tags, ActorSelectionTag, bTagContainsWildcards, ActorSelectionTagString);
}

void FPCGComponentSelectorSettings::PrepareForFiltering(bool bForce) const
{
	if (bForce || !bHasPreparedTag)
	{
		PCGActorSelector::PrepareTag(ComponentSelection == EPCGComponentSelection::ByTag, ComponentSelectionTag, bTagContainsWildcards, ComponentSelectionTagString);
		bHasPreparedTag = true;
	}
}

bool FPCGComponentSelectorSettings::FilterComponent(UActorComponent* InComponent) const
{
	if (!InComponent)
	{
		return true;
	}

	if (!ComponentList.IsEmpty() && !ComponentList.Contains(InComponent))
	{
		return false;
	}
	else
	{
		if (ComponentSelection == EPCGComponentSelection::ByTag)
		{
			PrepareForFiltering();

			return ComponentSelectionTag == NAME_None ||
				PCGActorSelector::TagListMatchesTag(InComponent->ComponentTags, ComponentSelectionTag, bTagContainsWildcards, ComponentSelectionTagString);
		}
		else
		{
			return ComponentSelectionClass == nullptr || 
				ComponentSelectionClass == UActorComponent::StaticClass() || 
				InComponent->GetClass()->IsChildOf(ComponentSelectionClass);
		}
	}
}

bool FPCGComponentSelectorSettings::FilterActor(AActor* InActor) const
{
	check(InActor);
	TInlineComponentArray<UActorComponent*> ActorComponents;
	InActor->GetComponents(ActorComponents);

	for (UActorComponent* ActorComponent : ActorComponents)
	{
		if (FilterComponent(ActorComponent))
		{
			return true;
		}
	}

	return false;
}

TArray<UActorComponent*> FPCGComponentSelectorSettings::FilterComponents(TArrayView<UActorComponent*> InComponents) const
{
	TArrayView<UActorComponent*> Components;
	TArray<UActorComponent*> InPreFilteredComponents;
	if (!ComponentList.IsEmpty())
	{
		Algo::CopyIf(InComponents, InPreFilteredComponents, [this](UActorComponent* Component) { return ComponentList.Contains(Component); });
		Components = TArrayView<UActorComponent*>(InPreFilteredComponents);
	}
	else
	{
		Components = InComponents;
	}

	TArray<UActorComponent*> FilteredComponents;
	FilteredComponents.Reserve(Components.Num());

	if ((ComponentSelection == EPCGComponentSelection::ByTag && ComponentSelectionTag == NAME_None) ||
		(ComponentSelection == EPCGComponentSelection::ByClass && (ComponentSelectionClass == nullptr || ComponentSelectionClass == UActorComponent::StaticClass())))
	{
		FilteredComponents = Components;
	}
	else if (ComponentSelection == EPCGComponentSelection::ByTag)
	{
		PrepareForFiltering();

		for (UActorComponent* Component : Components)
		{
			if (Component && PCGActorSelector::TagListMatchesTag(Component->ComponentTags, ComponentSelectionTag, bTagContainsWildcards, ComponentSelectionTagString))
			{
				FilteredComponents.Add(Component);
			}
		}
	}
	else if (ComponentSelection == EPCGComponentSelection::ByClass)
	{
		for (UActorComponent* Component : Components)
		{
			if (Component && Component->GetClass()->IsChildOf(ComponentSelectionClass))
			{
				FilteredComponents.Add(Component);
			}
		}
	}

	return FilteredComponents;
}