// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAddComponent.h"

#include "PCGManagedResource.h"
#include "PCGManagedResourceContainer.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include "Components/SceneComponent.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAddComponent)

#define LOCTEXT_NAMESPACE "PCGAddComponentElement"

UPCGAddComponentSettings::UPCGAddComponentSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ComponentReferenceAttribute.SetAttributeName(PCGAddComponentConstants::ComponentReferenceAttribute);
}

void UPCGAddComponentSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Note: Before we were not allowing template editing if we were spawning as attribute, so force it at false in that case.
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGAllowTemplateWhenSpawningFromAttribute && bUseClassAttribute)
	{
		bAllowTemplateComponentEditing = false;
	}

	SetupOnBlueprintChangedEvent();

	if (TemplateComponentClass && TemplateComponent)
	{
		TemplateComponent->ConditionalPostLoad();
	}

	RefreshTemplateComponent();
#endif
}

void UPCGAddComponentSettings::BeginDestroy()
{
#if WITH_EDITOR
	TeardownOnBlueprintChangedEvent();
#endif // WITH_EDITOR

	Super::BeginDestroy();
}

#if WITH_EDITOR
void UPCGAddComponentSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGAddComponentSettings, TemplateComponentClass))
	{
		TeardownOnBlueprintChangedEvent();
	}
}

void UPCGAddComponentSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		const FName& PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGAddComponentSettings, TemplateComponentClass))
		{
			SetupOnBlueprintChangedEvent();
			RefreshTemplateComponent();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGAddComponentSettings, bAllowTemplateComponentEditing))
		{
			RefreshTemplateComponent();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPCGAddComponentSettings::PreEditUndo()
{
	TeardownOnBlueprintChangedEvent();
	Super::PreEditUndo();
}

void UPCGAddComponentSettings::PostEditUndo()
{
	Super::PostEditUndo();
	SetupOnBlueprintChangedEvent();
	RefreshTemplateComponent();
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGAddComponentSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	InputPin.SetRequiredPin();

	FPCGPinProperties& TargetActorPin = PinProperties.Emplace_GetRef(PCGAddComponentConstants::TargetActorPinLabel, EPCGDataType::Any);
	TargetActorPin.SetAdvancedPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGAddComponentSettings::OutputPinProperties() const
{
	return Super::DefaultPointOutputPinProperties();
}

FPCGElementPtr UPCGAddComponentSettings::CreateElement() const
{
	return MakeShared<FPCGAddComponentElement>();
}

#if WITH_EDITOR
void UPCGAddComponentSettings::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances)
{
	if (!TemplateComponent)
	{
		return;
	}

	if (UObject* NewObject = InOldToNewInstances.FindRef(TemplateComponent))
	{
		TemplateComponent = Cast<UActorComponent>(NewObject);
		DirtyCache();
		OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Settings);
	}
}

void UPCGAddComponentSettings::SetupOnBlueprintChangedEvent()
{
	if (UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(TemplateComponentClass))
	{
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UPCGAddComponentSettings::OnObjectsReplaced);
	}
}

void UPCGAddComponentSettings::TeardownOnBlueprintChangedEvent()
{
	if (UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(TemplateComponentClass))
	{
		FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	}
}

void UPCGAddComponentSettings::RefreshTemplateComponent()
{
	// Implementation note: this is similar to the child actor component implementation
	if (TemplateComponentClass && bAllowTemplateComponentEditing)
	{
		const bool bCreateNewTemplateComponent = (!TemplateComponent || TemplateComponent->GetClass() != TemplateComponentClass);

		if (bCreateNewTemplateComponent)
		{
			UActorComponent* NewTemplateComponent = NewObject<UActorComponent>(GetTransientPackage(), TemplateComponentClass, NAME_None, RF_ArchetypeObject | RF_Transactional | RF_Public);

			if (TemplateComponent)
			{
				UEngine::FCopyPropertiesForUnrelatedObjectsParams Options;
				Options.bNotifyObjectReplacement = true;
				UEngine::CopyPropertiesForUnrelatedObjects(TemplateComponent, NewTemplateComponent, Options);

				TemplateComponent->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

				TMap<UObject*, UObject*> OldToNew;
				OldToNew.Emplace(TemplateComponent, NewTemplateComponent);
				GEngine->NotifyToolsOfObjectReplacement(OldToNew);

				TemplateComponent->MarkAsGarbage();
			}

			TemplateComponent = NewTemplateComponent;

			// Record initial object state in case we're in a transaction context.
			TemplateComponent->Modify();

			// Outer to this object
			TemplateComponent->Rename(nullptr, this, REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
	}
	else
	{
		if (TemplateComponent)
		{
			TemplateComponent->MarkAsGarbage();
		}

		TemplateComponent = nullptr;
	}
}

#endif // WITH_EDITOR

void UPCGAddComponentSettings::SetTemplateComponentClass(const TSubclassOf<UActorComponent>& InTemplateComponentClass)
{
#if WITH_EDITOR
	TeardownOnBlueprintChangedEvent();
#endif

	TemplateComponentClass = InTemplateComponentClass;

#if WITH_EDITOR
	SetupOnBlueprintChangedEvent();
	RefreshTemplateComponent();
#endif
}

void UPCGAddComponentSettings::SetAllowTemplateComponentEditing(bool bInAllowTemplateComponentEditing)
{
	bAllowTemplateComponentEditing = bInAllowTemplateComponentEditing;

#if WITH_EDITOR
	RefreshTemplateComponent();
#endif
}

bool FPCGAddComponentElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAddComponentElement::Execute);

	check(InContext);
	FPCGAddComponentContext* Context = static_cast<FPCGAddComponentContext*>(InContext);

	const UPCGAddComponentSettings* Settings = Context->GetInputSettings<UPCGAddComponentSettings>();
	check(Settings);

	// For all entries on the input pin, create one actor component.
	// Add that component either to a default value (this), or a value from the 'Target Actor' pin, which should be
	// either the same cardinality or only one entry.
	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData> TargetActors = Context->InputData.GetInputsByPin(PCGAddComponentConstants::TargetActorPinLabel);

	// Early outs & validation
	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	if (Inputs.Num() == 0 || !ExecutionSource)
	{
		return true;
	}

	if (TargetActors.Num() > 1 && TargetActors.Num() != Inputs.Num())
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("MismatchInputsAndTargetActors", "Expected either a single input for target actor or the same number as the inputs ('{0}') - got '{1}'."), FText::AsNumber(Inputs.Num()), FText::AsNumber(TargetActors.Num())));
		return true;
	}

	while (Context->CurrentInputIndex < Inputs.Num())
	{
		FPCGTaggedData& OutputTaggedData = Context->OutputData.TaggedData.Add_GetRef(Inputs[Context->CurrentInputIndex]);
		const UPCGData* TargetActorData = (!TargetActors.IsEmpty() ? (TargetActors.Num() == Inputs.Num() ? TargetActors[Context->CurrentInputIndex].Data : TargetActors[0].Data) : nullptr);
		const UPCGData* OriginalData = Inputs[Context->CurrentInputIndex++].Data;

		if (!OriginalData)
		{
			continue;
		}

		// Create accessors
		FPCGAttributePropertyInputSelector TransformSelector; // initialize from settings?
		TransformSelector.Update(FString(TEXT("$Transform")));

		TUniquePtr<const IPCGAttributeAccessor> TransformAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OriginalData, TransformSelector);
		TUniquePtr<const IPCGAttributeAccessorKeys> TransformKeys = PCGAttributeAccessorHelpers::CreateConstKeys(OriginalData, TransformSelector);

		if (!TransformAccessor.IsValid() || !TransformKeys.IsValid())
		{
			PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("MissingTransformAttribute", "Input data does not have the target attribute/property '{0}'"), FText::FromName(TransformSelector.GetName())));
			continue;
		}

		FTransform DummyTransform;
		if (TransformKeys->GetNum() > 0 && !TransformAccessor->Get<FTransform>(DummyTransform, *TransformKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
		{
			PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidTransformAttribute", "Input transform attribute/property '{0}' is not compatible with a transform."), FText::FromName(TransformSelector.GetName())));
			continue;
		}

		TUniquePtr<const IPCGAttributeAccessor> ClassOrTemplateAccessor = nullptr;
		TUniquePtr<const IPCGAttributeAccessorKeys> ClassOrTemplateKeys = nullptr;

		if (Settings->bUseClassAttribute)
		{
			FPCGAttributePropertyInputSelector ClassSelector = Settings->ClassAttribute.CopyAndFixLast(OriginalData);
			ClassOrTemplateAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OriginalData, ClassSelector);
			ClassOrTemplateKeys = PCGAttributeAccessorHelpers::CreateConstKeys(OriginalData, ClassSelector);

			if (!ClassOrTemplateAccessor.IsValid() || !ClassOrTemplateKeys.IsValid())
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("MissingClassAttribute", "Input data does not have the target class attribute/property '{0}'"), FText::FromName(ClassSelector.GetName())));
				continue;
			}

			FSoftObjectPath DummyPath;
			if (ClassOrTemplateKeys->GetNum() > 0 && !ClassOrTemplateAccessor->Get<FSoftObjectPath>(DummyPath, *ClassOrTemplateKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidClassOrTemplateAttribute", "Input class attribute/property '{0}' is not an object or class path."), FText::FromName(ClassSelector.GetName())));
				continue;
			}
		}
		else if (Settings->bAllowTemplateComponentEditing && Settings->TemplateComponent)
		{
			ClassOrTemplateAccessor = MakeUnique<FPCGConstantValueAccessor<FSoftObjectPath>>(FSoftObjectPath(Settings->TemplateComponent));
			ClassOrTemplateKeys = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();
		}
		else
		{
			ClassOrTemplateAccessor = MakeUnique<FPCGConstantValueAccessor<FSoftClassPath>>(FSoftClassPath(Settings->TemplateComponentClass));
			ClassOrTemplateKeys = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();
		}

		TUniquePtr<const IPCGAttributeAccessor> TargetActorAccessor = nullptr;
		TUniquePtr<const IPCGAttributeAccessorKeys> TargetActorKeys = nullptr;

		if (TargetActorData)
		{
			FPCGAttributePropertyInputSelector TargetActorSelector = Settings->ActorReferenceAttribute.CopyAndFixLast(TargetActorData);
			TargetActorAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(TargetActorData, TargetActorSelector);
			TargetActorKeys = PCGAttributeAccessorHelpers::CreateConstKeys(TargetActorData, TargetActorSelector);

			if (!TargetActorAccessor.IsValid() || !TargetActorKeys.IsValid())
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("MissingActorReferenceAttribute", "Target actor data does not have the target attribute '{0}'"), FText::FromName(TargetActorSelector.GetName())));
				continue;
			}

			FSoftObjectPath DummyPath;
			if (TargetActorKeys->GetNum() > 0 && !TargetActorAccessor->Get<FSoftObjectPath>(DummyPath, *TargetActorKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidActorReferenceAttribute", "Target actor attribute '{0}' is not an object path."), FText::FromName(TargetActorSelector.GetName())));
				continue;
			}

			// Additional validation: number of target actor keys should be either 1 or the same amount as the input data.
			if (TargetActorKeys->GetNum() > 1 && TargetActorKeys->GetNum() != TransformKeys->GetNum())
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("TargetActorAndTransformsMismatch", "Target actor data entry count '{0}' does not match the input entry count '{1}'."), FText::AsNumber(TargetActorKeys->GetNum()), FText::AsNumber(TransformKeys->GetNum())));
				continue;
			}
		}
		else
		{
			if (!ExecutionSource->GetExecutionState().GetTarget())
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("DefaultTargetSourceInvalid", "Unable to retrieve the owner of the execution source."));
				continue;
			}

			// Wrap on the execution source target (in most cases, this will be the owner of the PCG component)
			TargetActorAccessor = MakeUnique<FPCGConstantValueAccessor<FSoftObjectPath>>(FSoftObjectPath(ExecutionSource->GetExecutionState().GetTarget()));
			TargetActorKeys = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();
		}

		// Create output
		UPCGData* OutputData = OriginalData->DuplicateData(Context);

		// Add component reference attribute
		FPCGAttributePropertyOutputNoSourceSelector ComponentReferenceSelector = Settings->ComponentReferenceAttribute; // No copy source here because there's no 1:1 mapping

		// Create attribute if needed
		if (ComponentReferenceSelector.IsBasicAttribute())
		{
			OutputData->MutableMetadata()->FindOrCreateAttribute(ComponentReferenceSelector.GetName(), FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/false);
		}

		TUniquePtr<IPCGAttributeAccessor> OutputComponentReferenceAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputData, ComponentReferenceSelector);
		TUniquePtr<IPCGAttributeAccessorKeys> OutputComponentReferenceKeys = PCGAttributeAccessorHelpers::CreateKeys(OutputData, ComponentReferenceSelector);

		if (!OutputComponentReferenceAccessor.IsValid() || !OutputComponentReferenceKeys.IsValid())
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("UnableToCreateOutputReference", "Unable to write component references to '{0}'."), FText::FromName(ComponentReferenceSelector.GetName())));
			continue;
		}

		OutputTaggedData.Data = OutputData;

		// Now create and add the components
		AddComponents(Context,
			OriginalData,
			Settings,
			TransformAccessor.Get(),
			TransformKeys.Get(),
			ClassOrTemplateAccessor.Get(),
			ClassOrTemplateKeys.Get(),
			TargetActorAccessor.Get(),
			TargetActorKeys.Get(),
			OutputComponentReferenceAccessor.Get(),
			OutputComponentReferenceKeys.Get());

		// Finally, break out if we're out of time
		if (Context->CurrentInputIndex != Inputs.Num() && Context->ShouldStop())
		{
			break;
		}
	}

	return Context->CurrentInputIndex == Inputs.Num();
}

void FPCGAddComponentElement::AddComponents(FPCGAddComponentContext* Context,
	const UPCGData* OriginalData,
	const UPCGAddComponentSettings* Settings,
	const IPCGAttributeAccessor* TransformAccessor,
	const IPCGAttributeAccessorKeys* TransformKeys,
	const IPCGAttributeAccessor* ClassOrTemplateAccessor,
	const IPCGAttributeAccessorKeys* ClassOrTemplateKeys,
	const IPCGAttributeAccessor* TargetActorAccessor,
	const IPCGAttributeAccessorKeys* TargetActorKeys,
	IPCGAttributeAccessor* OutputComponentAccessor,
	IPCGAttributeAccessorKeys* OutputComponentKeys) const
{
	check(TransformAccessor && TransformKeys);
	check(ClassOrTemplateAccessor && ClassOrTemplateKeys);
	check(TargetActorAccessor && TargetActorKeys);
	check(OutputComponentAccessor && OutputComponentKeys);

	// TODO: implement component reuse
	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	check(ExecutionSource);
	const bool bIsInPreviewMode = ExecutionSource->GetExecutionState().IsInPreviewMode();
	const EObjectFlags ObjectFlags = bIsInPreviewMode ? RF_Transient : RF_NoFlags;

	// Similar approach to PCGMetadataCommon::ApplyOnAccessorRange
	const int32 NumberOfEntries = TransformKeys->GetNum();
	const int32 ChunkSize = PCGMetadataElementCommon::DefaultChunkSize;
	const int32 NumberOfIterations = (NumberOfEntries + ChunkSize - 1) / ChunkSize;

	if (NumberOfEntries == 0)
	{
		return;
	}

	FPCGManagedResourceContainerHelper ManagedResourcesContainerHelper(ExecutionSource);
	UObject* ExecutionSourceAsObject = Cast<UObject>(ExecutionSource);
	if (!ManagedResourcesContainerHelper.IsValid() || !ExecutionSourceAsObject)
	{
		// Need resource container for tracking purposes
		return;
	}

	// TODO: root location? (needed for reuse) + settings UID
	UPCGManagedComponentList* Resource = NewObject<UPCGManagedComponentList>(ExecutionSourceAsObject);
	check(Resource);
	Resource->GeneratedComponents.Reserve(NumberOfEntries);

	TArray<FTransform, TInlineAllocator<ChunkSize>> Transforms;
	TArray<FSoftObjectPath, TInlineAllocator<ChunkSize>> ComponentsClassOrTemplate;
	TArray<FSoftObjectPath, TInlineAllocator<ChunkSize>> TargetActors;
	TArray<FSoftObjectPath, TInlineAllocator<ChunkSize>> ComponentReferences;

	Transforms.SetNumUninitialized(ChunkSize);
	ComponentsClassOrTemplate.SetNum(ChunkSize);
	TargetActors.SetNum(ChunkSize);
	ComponentReferences.SetNum(ChunkSize);

	const EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible;

	for (int32 i = 0; i < NumberOfIterations; ++i)
	{
		const int32 StartIndex = i * ChunkSize;
		const int32 Range = FMath::Min(NumberOfEntries - StartIndex, ChunkSize);
		TArrayView<FTransform> TransformsView(Transforms.GetData(), Range);
		TArrayView<FSoftObjectPath> ComponentsClassOrTemplateView(ComponentsClassOrTemplate.GetData(), Range);
		TArrayView<FSoftObjectPath> TargetActorsView(TargetActors.GetData(), Range);
		TArrayView<FSoftObjectPath> ComponentReferencesView(ComponentReferences.GetData(), Range);

		if(!TransformAccessor->GetRange(TransformsView, StartIndex, *TransformKeys, Flags) ||
			!ClassOrTemplateAccessor->GetRange(ComponentsClassOrTemplateView, StartIndex, *ClassOrTemplateKeys, Flags) ||
			!TargetActorAccessor->GetRange(TargetActorsView, StartIndex, *TargetActorKeys, Flags))
		{
			// Failed getting data...
			continue;
		}

#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("AddComponents", "Adding components to actors from PCG"), Context->ExecutionSource->GetExecutionState().UseTransactions());
#endif

		for (int32 j = 0; j < TransformsView.Num(); ++j)
		{
			ComponentReferencesView[j] = FSoftObjectPath();

			const FTransform& Transform = TransformsView[j];
			AActor* TargetActor = TSoftObjectPtr<AActor>(TargetActorsView[j]).Get();
			const FSoftObjectPath& ClassOrTemplatePath = ComponentsClassOrTemplateView[j];

			TSubclassOf<UActorComponent> ComponentClass;
			UActorComponent* ComponentTemplate = nullptr;

			TSoftClassPtr<UActorComponent> ComponentClassPtr(ClassOrTemplatePath);
			ComponentClass = ComponentClassPtr.Get();

			bool bCopyTemplateComponent = false;

			if (!ComponentClass)
			{
				TSoftObjectPtr<UActorComponent> ComponentObjectPtr(ClassOrTemplatePath);
				ComponentTemplate = ComponentObjectPtr.Get();
				ComponentClass = ComponentTemplate ? ComponentTemplate->GetClass() : nullptr;
			}
			else if (Settings->bAllowTemplateComponentEditing && Settings->TemplateComponent)
			{
				// Use the Template from the settings if it exists. Only use it if it is a parent class of the class we want to spawn
				if (!ComponentClass->IsChildOf(Settings->TemplateComponent->GetClass()))
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidTemplateClass", "For input {0}, point index {1}, tried to spawn a component of class {2}, while providing a template component of type {3}. This is invalid. The template component must be of a parent class of the spawned component class."),
						Context->CurrentInputIndex,
						j,
						FText::FromName(ComponentClass->GetFName()),
						FText::FromName(Settings->TemplateComponent->GetClass()->GetFName())));
					continue;
				}

				bCopyTemplateComponent = true;
			}
			
			if (TargetActor && ComponentClass)
			{
				UActorComponent* Component = NewObject<UActorComponent>(TargetActor, ComponentClass, NAME_None, ObjectFlags, ComponentTemplate);

				if (!Component)
				{
					continue;
				}

				if (bCopyTemplateComponent)
				{
					UEngine::CopyPropertiesForUnrelatedObjects(Settings->TemplateComponent, Component);
				}

				if (!Settings->PropertyOverrideDescriptions.IsEmpty() && OriginalData)
				{
					FPCGObjectOverrides DescriptorOverride(ComponentTemplate);
					DescriptorOverride.Initialize(Settings->PropertyOverrideDescriptions, ComponentTemplate, OriginalData, Context);
					const int32 Index = StartIndex + j;
					if (DescriptorOverride.IsValid() && !DescriptorOverride.Apply(Index))
					{
						PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("FailOverride", "Failed to override descriptor for input {0}"), Index));
					}
				}

				Component->RegisterComponent();
				TargetActor->AddInstanceComponent(Component);

				if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
				{
					if (!SceneComponent->AttachToComponent(TargetActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, false)))
					{
						PCGLog::Component::LogComponentAttachmentFailedWarning(Context);
					}

					SceneComponent->SetWorldTransform(Transform);
				}

				if (UObject* SourceObject = Cast<UObject>(ExecutionSource))
				{
					Component->ComponentTags.Add(SourceObject->GetFName());
				}
				
				Component->ComponentTags.Add(PCGHelpers::DefaultPCGTag);

				// Add component reference on output data
				ComponentReferences[j] = FSoftObjectPath(Component);

				// Add component to generated components
				Resource->GeneratedComponents.Emplace(Component);
			}
		}

		// Write component reference to accessor
		OutputComponentAccessor->SetRange<FSoftObjectPath>(ComponentReferencesView, StartIndex, *OutputComponentKeys, Flags);
	}

	// Finally add the resource to the component
	if (!Resource->GeneratedComponents.IsEmpty())
	{
		ManagedResourcesContainerHelper.AddManagedResource(Resource);
	}	
}

#if WITH_EDITOR
bool UPCGAddComponentSettings::CanEditActorReference() const
{
	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	return !Node || Node->IsInputPinConnected(PCGAddComponentConstants::TargetActorPinLabel);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
