// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAddTag.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGPin.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAddTag)

#define LOCTEXT_NAMESPACE "PCGAddTagElement"

#if WITH_EDITOR
void UPCGAddTagSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DataVersion < FPCGCustomVersion::AttributesAndTagsCanContainSpaces)
	{
		bTokenizeOnWhiteSpace = true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Super::ApplyDeprecation(InOutNode);
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGAddTagSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGAddTagSettings::CreateElement() const
{
	return MakeShared<FPCGAddTagElement>();
}

bool FPCGAddTagElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAddTagElement::Execute);
	
	check(Context);

	const UPCGAddTagSettings* Settings = Context->GetInputSettings<UPCGAddTagSettings>();
	check(Settings);
	
	Context->OutputData.TaggedData = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

    PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FString> TagsArray = Settings->bTokenizeOnWhiteSpace
		? PCGHelpers::GetStringArrayFromCommaSeparatedString(Settings->TagsToAdd, Context)
		: PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->TagsToAdd);
    PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (const FString& Tag : TagsArray)
	{
		FString TagToAdd;

		int32 DividerPosition = INDEX_NONE;
		if (!Settings->bIgnoreTagValueParsing && Tag.FindChar(':', DividerPosition))
		{
			FString LeftSide = Tag.Left(DividerPosition);
			FString RightSide = Tag.RightChop(DividerPosition); // not +1 because we want to keep the ':'

			TagToAdd = Settings->Prefix + LeftSide + Settings->Suffix + RightSide;
		}
		else
		{
			TagToAdd = Settings->Prefix + Tag + Settings->Suffix;
		}

		for (FPCGTaggedData& OutputTaggedData : Context->OutputData.TaggedData)
		{
			OutputTaggedData.Tags.Add(TagToAdd);
		}
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
