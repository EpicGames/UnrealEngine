// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAlignPoints.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Helpers/PCGPointHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAlignPoints)

#define LOCTEXT_NAMESPACE "PCGAlignPointsElement"

namespace PCGAlignPoints
{
	/** Returns the scalar value at AxisIndex (0=X, 1=Y, 2=Z) for the given referential enum. */
	double GetBoundsValue(const FBox& Bounds, EPCGAlignPointsAxisReferential Referential, int32 AxisIndex)
	{
		switch (Referential)
		{
			case EPCGAlignPointsAxisReferential::Min: return Bounds.Min[AxisIndex];
			case EPCGAlignPointsAxisReferential::Max: return Bounds.Max[AxisIndex];
			case EPCGAlignPointsAxisReferential::Center: return Bounds.GetCenter()[AxisIndex];
			default: 
			{
				checkNoEntry(); 
				return Bounds.GetCenter()[AxisIndex];
			}
		}
	}
}

#if WITH_EDITOR

FText UPCGAlignPointsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Align Points");
}

FText UPCGAlignPointsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "For each source point, aligns (changes location and optionally rotation) it so that the chosen edge of its bounding box aligns with the chosen edge of the corresponding target point's bounding box. Bounds are computed in the chosen spatial referential (Target, Source, or World axes).");
}

#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGAlignPointsSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	FPCGPinProperties& SourcePin = PinProperties.Emplace_GetRef(PCGAlignPointsConstants::SourcePointsLabel, EPCGDataType::Point);
#if WITH_EDITOR
	SourcePin.Tooltip = LOCTEXT("SourcePinTooltip", "The points to align. Each source point data entry is paired with a target point data entry.");
#endif
	SourcePin.SetRequiredPin();

	FPCGPinProperties& TargetPin = PinProperties.Emplace_GetRef(PCGAlignPointsConstants::TargetPointsLabel, EPCGDataType::Point);
#if WITH_EDITOR
	TargetPin.Tooltip = LOCTEXT("TargetPinTooltip", "The reference points to align to. Must have the same number of data entries as Source, or a single entry applied to all sources. Within each pair, target must have the same number of points as source, or a single point.");
#endif
	TargetPin.SetRequiredPin();

	return PinProperties;
}

FPCGElementPtr UPCGAlignPointsSettings::CreateElement() const
{
	return MakeShared<FPCGAlignPointsElement>();
}

bool FPCGAlignPointsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAlignPointsElement::Execute);

	const UPCGAlignPointsSettings* Settings = Context->GetInputSettings<UPCGAlignPointsSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Sources = Context->InputData.GetInputsByPin(PCGAlignPointsConstants::SourcePointsLabel);
	const TArray<FPCGTaggedData> Targets = Context->InputData.GetInputsByPin(PCGAlignPointsConstants::TargetPointsLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const int32 NumSources = Sources.Num();
	const int32 NumTargets = Targets.Num();

	if (NumSources == 0 || NumTargets == 0)
	{
		Outputs = Sources;
		return true;
	}

	// Validate data-level pairing: must be N:N, or N:1.
	if (NumSources != NumTargets && NumTargets != 1)
	{
		PCGLog::InputOutput::LogInvalidCardinalityError(PCGAlignPointsConstants::SourcePointsLabel, PCGAlignPointsConstants::TargetPointsLabel, Context);
		Outputs = Sources;
		return true;
	}

	const EPCGAlignPointsSpatialReferential SpatialReferential = Settings->SpatialReferential;
	const FPCGAlignPointsAxisSettings& XAxisSettings = Settings->XAxis;
	const FPCGAlignPointsAxisSettings& YAxisSettings = Settings->YAxis;
	const FPCGAlignPointsAxisSettings& ZAxisSettings = Settings->ZAxis;

	// Early-out: no axis enabled means no work to do — pass sources through unchanged.
	if (!XAxisSettings.bApplyToAxis && !YAxisSettings.bApplyToAxis && !ZAxisSettings.bApplyToAxis)
	{
		Outputs = Sources;
		return true;
	}

	for (int32 SourceIndex = 0; SourceIndex < NumSources; ++SourceIndex)
	{
		const int32 TargetIndex = (SourceIndex % NumTargets);
		const FPCGTaggedData& SourceTaggedData = Sources[SourceIndex];
		const FPCGTaggedData& TargetTaggedData = Targets[TargetIndex];

		// Start output as a copy of the source tagged data (preserves tags, pin label, etc.)
		FPCGTaggedData& Output = Outputs.Add_GetRef(SourceTaggedData);

		if (!SourceTaggedData.Data || !TargetTaggedData.Data)
		{
			PCGLog::InputOutput::LogInvalidInputDataError(Context);
			continue;
		}

		const UPCGBasePointData* SourcePointData = Cast<UPCGBasePointData>(SourceTaggedData.Data);
		const UPCGBasePointData* TargetPointData = Cast<UPCGBasePointData>(TargetTaggedData.Data);

		if (!SourcePointData || !TargetPointData)
		{
			continue;
		}

		const int32 NumSourcePoints = SourcePointData->GetNumPoints();
		const int32 NumTargetPoints = TargetPointData->GetNumPoints();

		// Validate point-level pairing within this data pair.
		if (NumTargetPoints != NumSourcePoints && NumTargetPoints != 1)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NumPointsMismatch", "Source point count ({0}) does not match Target point count ({1}) for data pair {2}. Target must have the same number of points as Source, or exactly one point."), NumSourcePoints, NumTargetPoints, SourceIndex));
			continue;
		}

		// Build the output point data as a copy of the source, with modified transforms.
		UPCGBasePointData* OutPointData = CastChecked<UPCGBasePointData>(SourcePointData->DuplicateData(Context));
		Output.Data = OutPointData;

		OutPointData->AllocateProperties(SourcePointData->GetAllocatedProperties() | EPCGPointNativeProperties::Transform);

		// Read source value ranges.
		FConstPCGPointValueRanges SourceRanges(SourcePointData);
		FConstPCGPointValueRanges TargetRanges(TargetPointData);
		TPCGValueRange<FTransform> OutTransformRange = OutPointData->GetTransformValueRange();

		for (int32 PointIndex = 0; PointIndex < NumSourcePoints; ++PointIndex)
		{
			// Then retrieve both the source & target points so we can compute the appropriate delta to apply.
			FPCGPoint SourcePoint = SourceRanges.GetPoint(PointIndex);

			FTransform SourcePointTransform = SourcePoint.Transform;

			const int32 TargetPointIndex = (NumTargetPoints == 1) ? 0 : PointIndex;
			FPCGPoint TargetPoint = TargetRanges.GetPoint(TargetPointIndex);

			FBox SourceBounds = SourcePoint.GetLocalBounds();
			FBox TargetBounds = TargetPoint.GetLocalBounds();
			FTransform RelativeTransform = FTransform::Identity;
			FTransform ReferentialTransform = FTransform::Identity;

			// Local to target mode enforces the source points to align to the target, prior to alignment.
			if (SpatialReferential == EPCGAlignPointsSpatialReferential::LocalToTarget)
			{
				SourcePointTransform.SetRotation(TargetPoint.Transform.GetRotation());
			}

			if (SpatialReferential == EPCGAlignPointsSpatialReferential::Target || SpatialReferential == EPCGAlignPointsSpatialReferential::LocalToTarget)
			{
				ReferentialTransform = TargetPoint.Transform;
				RelativeTransform = SourcePointTransform.GetRelativeTransform(TargetPoint.Transform);
				SourceBounds = SourceBounds.TransformBy(RelativeTransform);
			}
			else if (SpatialReferential == EPCGAlignPointsSpatialReferential::Source)
			{
				ReferentialTransform = SourcePointTransform;
				RelativeTransform = TargetPoint.Transform.GetRelativeTransform(SourcePoint.Transform);
				TargetBounds = TargetBounds.TransformBy(RelativeTransform);
			}
			else // World
			{
				SourceBounds = SourceBounds.TransformBy(SourcePoint.Transform);
				TargetBounds = TargetBounds.TransformBy(TargetPoint.Transform);
			}

			// Compute per-axis alignment delta in the referential frame.
			FVector DeltaInRef = FVector::ZeroVector;
			if (XAxisSettings.bApplyToAxis)
			{
				DeltaInRef.X = PCGAlignPoints::GetBoundsValue(TargetBounds, XAxisSettings.TargetReferential, 0) - PCGAlignPoints::GetBoundsValue(SourceBounds, XAxisSettings.SourceReferential, 0);
			}

			if (YAxisSettings.bApplyToAxis)
			{
				DeltaInRef.Y = PCGAlignPoints::GetBoundsValue(TargetBounds, YAxisSettings.TargetReferential, 1) - PCGAlignPoints::GetBoundsValue(SourceBounds, YAxisSettings.SourceReferential, 1);
			}

			if (ZAxisSettings.bApplyToAxis)
			{
				DeltaInRef.Z = PCGAlignPoints::GetBoundsValue(TargetBounds, ZAxisSettings.TargetReferential, 2) - PCGAlignPoints::GetBoundsValue(SourceBounds, ZAxisSettings.SourceReferential, 2);
			}

			// Convert the delta from the referential frame back to world space
			const FVector WorldDelta = ReferentialTransform.TransformVector(DeltaInRef);

			// Write the aligned transform: same rotation and scale as source, translated position.
			SourcePointTransform.AddToTranslation(WorldDelta);
			OutTransformRange[PointIndex] = SourcePointTransform;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE