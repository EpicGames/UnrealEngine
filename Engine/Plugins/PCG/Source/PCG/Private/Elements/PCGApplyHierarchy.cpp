// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGApplyHierarchy.h"

#include "PCGDataAsset.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"
#include "Metadata/PCGMetadataPartitionCommon.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGApplyHierarchy)

#define LOCTEXT_NAMESPACE "PCGApplyHierarchyElement"

UPCGApplyHierarchySettings::UPCGApplyHierarchySettings()
{
	PointKeyAttributes.Emplace_GetRef().SetAttributeName(PCGLevelToAssetConstants::ActorIndexAttributeName);
	ParentKeyAttributes.Emplace_GetRef().SetAttributeName(PCGLevelToAssetConstants::ParentIndexAttributeName);
	HierarchyDepthAttribute.SetAttributeName(PCGLevelToAssetConstants::HierarchyDepthAttributeName);
	RelativeTransformAttribute.SetAttributeName(PCGLevelToAssetConstants::RelativeTransformAttributeName);
	ApplyParentRotation = EPCGApplyHierarchyOption::OptOutByAttribute;
	ApplyParentRotationAttribute.SetAttributeName(PCGLevelToAssetConstants::IgnoreParentRotationAttributeName);
	ApplyParentScale = EPCGApplyHierarchyOption::OptOutByAttribute;
	ApplyParentScaleAttribute.SetAttributeName(PCGLevelToAssetConstants::IgnoreParentScaleAttributeName);
	ApplyHierarchy = EPCGApplyHierarchyOption::Always;
}

FPCGElementPtr UPCGApplyHierarchySettings::CreateElement() const
{
	return MakeShared<FPCGApplyHierarchyElement>();
}

bool FPCGApplyHierarchyElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGApplyHierarchyElement::PrepareData);

	const UPCGApplyHierarchySettings* Settings = InContext->GetInputSettings<UPCGApplyHierarchySettings>();
	check(Settings);

	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	Context->InitializePerExecutionState();

	Context->InitializePerIterationStates(Inputs.Num(), [&Inputs, Settings, Context](IterStateType& OutState, const ExecStateType&, const uint32 IterationIndex)
	{
		const FPCGTaggedData& Input = Inputs[IterationIndex];
		const UPCGBasePointData* InputData = Cast<const UPCGBasePointData>(Input.Data);

		if (!InputData || InputData->GetNumPoints() == 0)
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		UPCGBasePointData* OutputData = FPCGContext::NewPointData_AnyThread(Context);
		OutputData->InitializeFromData(InputData);
		OutputData->SetNumPoints(InputData->GetNumPoints());

		// Allocate property we are going to modify
		EPCGPointNativeProperties PropertiesToAllocate = EPCGPointNativeProperties::Transform;

		// If the data doesn't support parent, also allocate any currently used properties
		if (!OutputData->HasSpatialDataParent())
		{
			PropertiesToAllocate |= InputData->GetAllocatedProperties();
		}

		OutputData->AllocateProperties(PropertiesToAllocate);

		OutState.InputData = InputData;
		OutState.OutputData = OutputData;
		OutState.OutputDataIndex = Context->OutputData.TaggedData.Num();
		Context->OutputData.TaggedData.Add(Input);
		
		// Create point index & parent index accessors
		// Implementation note: we could have used PCGMetadataElementCommon::ApplyOnAccessorRange here, as we're doing a transformation
		// but it would require to do the accessor & keys here.
		auto PrepareAccessorAndKeys = [Context, InputData] (const FPCGAttributePropertyInputSelector& InputSelector, TUniquePtr<const IPCGAttributeAccessor>& OutAccessor, TUniquePtr<const IPCGAttributeAccessorKeys>& OutAccessorKeys) -> bool
		{
			OutAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, InputSelector);
			OutAccessorKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, InputSelector);

			if (!OutAccessor || !OutAccessorKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(InputSelector, Context);
				return false;
			}

			if (!PCG::Private::IsConstructible(OutAccessor->GetUnderlyingType(), PCG::Private::MetadataTypes<int32>::Id))
			{
				PCGLog::Metadata::LogFailToGetAttributeError<int32>(InputSelector, OutAccessor.Get(), Context);
				return false;
			}

			return true;
		};

		auto PrepareAccessorAndKeysArray = [PrepareAccessorAndKeys, InputData](const TArray<FPCGAttributePropertyInputSelector>& InputSelectors, TArray<TUniquePtr<const IPCGAttributeAccessor>>& OutAccessors, TArray<TUniquePtr<const IPCGAttributeAccessorKeys>>& OutAccessorKeys) -> bool
		{
			for (const FPCGAttributePropertyInputSelector& InputSelector : InputSelectors)
			{
				const FPCGAttributePropertyInputSelector Selector = InputSelector.CopyAndFixLast(InputData);
				TUniquePtr<const IPCGAttributeAccessor> Accessor;
				TUniquePtr<const IPCGAttributeAccessorKeys> Keys;

				if (PrepareAccessorAndKeys(Selector, Accessor, Keys))
				{
					OutAccessors.Add(std::move(Accessor));
					OutAccessorKeys.Add(std::move(Keys));
				}
				else
				{
					return false;
				}
			}

			return true;
		};

		if (Settings->PointKeyAttributes.IsEmpty() || Settings->ParentKeyAttributes.IsEmpty())
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("RequiresAtLeastOneAttribute", "Both the point key attribute and parent key attribute require valid entries to perform the Apply Hierarchy operation."), Context);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (Settings->PointKeyAttributes.Num() != Settings->ParentKeyAttributes.Num())
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("PointAndParentKeyAttributesMismatch", "There needs to be the same number of point key attributes as parent key attributes."), Context);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (!PrepareAccessorAndKeysArray(Settings->PointKeyAttributes, OutState.PointIndexAccessors, OutState.PointIndexKeys))
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (!PrepareAccessorAndKeysArray(Settings->ParentKeyAttributes, OutState.ParentIndexAccessors, OutState.ParentIndexKeys))
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		const int NumKeys = OutState.PointIndexKeys[0] ? OutState.PointIndexKeys[0]->GetNum() : 0;
		bool bHasCardinalityError = (NumKeys == 0);

		for (const TUniquePtr<const IPCGAttributeAccessorKeys>& PointIndexKey : OutState.PointIndexKeys)
		{
			bHasCardinalityError |= (!PointIndexKey || PointIndexKey->GetNum() != NumKeys);
		}
		
		for (const TUniquePtr<const IPCGAttributeAccessorKeys>& ParentIndexKey : OutState.ParentIndexKeys)
		{
			bHasCardinalityError |= (!ParentIndexKey || ParentIndexKey->GetNum() != NumKeys);
		}

		if (bHasCardinalityError)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("CardinalityMismatch", "Point Key and Parent Key properties do not have the same cardinality."), Context);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Create hierarchy depth keys (TODO: if required)
		// Which we'll use to sort the partition by depth (from 0 to N)
		OutState.HierarchyDepthSelector = Settings->HierarchyDepthAttribute.CopyAndFixLast(InputData);
		if (!PrepareAccessorAndKeys(OutState.HierarchyDepthSelector, OutState.HierarchyDepthAccessor, OutState.HierarchyDepthKeys))
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		auto PrepareApplyOptionsAccessor = [Context, InputData](const FPCGAttributePropertyInputSelector& InputSelector, EPCGApplyHierarchyOption Option, TUniquePtr<const IPCGAttributeAccessor>& OutAccessor, bool& bOutInvert) -> bool
		{
			if (Option == EPCGApplyHierarchyOption::Always || Option == EPCGApplyHierarchyOption::Never)
			{
				OutAccessor = MakeUnique<FPCGConstantValueAccessor<bool>>(true);
				bOutInvert = Option == EPCGApplyHierarchyOption::Never;
			}
			else
			{
				FPCGAttributePropertyInputSelector Selector = InputSelector.CopyAndFixLast(InputData);
				OutAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, Selector);
				bOutInvert = Option == EPCGApplyHierarchyOption::OptOutByAttribute;

				if (!OutAccessor)
				{
					PCGLog::Metadata::LogFailToCreateAccessorError(InputSelector, Context);
					return false;
				}
			}

			return true;
		};

		if (!PrepareApplyOptionsAccessor(Settings->ApplyParentRotationAttribute, Settings->ApplyParentRotation, OutState.ApplyRotationAccessor, OutState.bInvertApplyRotation))
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (!PrepareApplyOptionsAccessor(Settings->ApplyParentScaleAttribute, Settings->ApplyParentScale, OutState.ApplyScaleAccessor, OutState.bInvertApplyScale))
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (!PrepareApplyOptionsAccessor(Settings->ApplyHierarchyAttribute, Settings->ApplyHierarchy, OutState.ApplyHierarchyAccessor, OutState.bInvertApplyHierarchy))
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Finally, get the relative transforms
		FPCGAttributePropertyInputSelector RelativeTransformSelector = Settings->RelativeTransformAttribute.CopyAndFixLast(InputData);
		OutState.RelativeTransformAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, RelativeTransformSelector);

		if (!OutState.RelativeTransformAccessor)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(RelativeTransformSelector, Context);
			return EPCGTimeSliceInitResult::NoOperation;
		}
		
		if (!PCG::Private::IsBroadcastableOrConstructible(OutState.RelativeTransformAccessor->GetUnderlyingType(), PCG::Private::MetadataTypes<FTransform>::Id))
		{
			PCGLog::Metadata::LogFailToGetAttributeError<FTransform>(RelativeTransformSelector, OutState.RelativeTransformAccessor.Get(), Context);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// If everything was validated correctly, move the "new" output result to the output data.
		Context->OutputData.TaggedData.Last().Data = OutputData;
		
		return EPCGTimeSliceInitResult::Success;
	});

	return true;
}

bool FPCGApplyHierarchyElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGApplyHierarchyElement::Execute);

	ContextType* TimeSlicedContext = static_cast<ContextType*>(InContext);
	check(TimeSlicedContext);

	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		return true;
	}

	if (TimeSlicedContext->GetExecutionStateResult() == EPCGTimeSliceInitResult::NoOperation)
	{
		TimeSlicedContext->OutputData = TimeSlicedContext->InputData;
		return true;
	}

	const UPCGApplyHierarchySettings* Settings = TimeSlicedContext->GetInputSettings<UPCGApplyHierarchySettings>();
	check(Settings);

	return ExecuteSlice(TimeSlicedContext, [Settings](ContextType* Context, const ExecStateType& ExecState, IterStateType& IterState, const uint32 IterIndex)
	{
		if (Context->GetIterationStateResult(IterIndex) == EPCGTimeSliceInitResult::NoOperation)
		{
			return true;
		}

		constexpr int32 InvalidParentIndex = -2;

		// 1. Get point indices & parent indices
		// 2. Build index to parent index map
		if (!IterState.bParentMappingDone)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGApplyHierarchyElement::Execute::ParentMapping);
			constexpr int32 KeySize = 4;
			TArray<TArray<int32, TInlineAllocator<KeySize>>> PointKeys;
			TArray<TArray<int32, TInlineAllocator<KeySize>>> ParentKeys;

			constexpr int32 ChunkSize = 256;

			auto InitializePointKeysAndParentKeys = [&IterState, &PointKeys, &ParentKeys]()
			{
				const int32 NumAttributes = IterState.PointIndexKeys.Num();
				const int32 NumKeys = IterState.PointIndexKeys[0]->GetNum();
				// Initialize and set array sizes
				PointKeys.SetNum(NumKeys);
				ParentKeys.SetNum(NumKeys);

				for (auto& PointKeyArray : PointKeys)
				{
					PointKeyArray.SetNumUninitialized(NumAttributes);
				}

				for (auto& ParentKeyArray : ParentKeys)
				{
					ParentKeyArray.SetNumUninitialized(NumAttributes);
				}
			};

			auto GetPointKeysAndParentKeys = [&IterState, &PointKeys, &ParentKeys](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
			{
				TArray<int32, TInlineAllocator<ChunkSize>> LocalArray;
				LocalArray.SetNumUninitialized(Count);

				for(int PointKeyIndex = 0; PointKeyIndex < IterState.PointIndexAccessors.Num(); ++PointKeyIndex)
				{
					IterState.PointIndexAccessors[PointKeyIndex]->GetRange<int32>(LocalArray, StartReadIndex, *IterState.PointIndexKeys[PointKeyIndex], EPCGAttributeAccessorFlags::AllowConstructible);

					for (int32 Index = 0; Index < Count; ++Index)
					{
						PointKeys[StartWriteIndex + Index][PointKeyIndex] = LocalArray[Index];
					}
				}

				for (int ParentKeyIndex = 0; ParentKeyIndex < IterState.ParentIndexAccessors.Num(); ++ParentKeyIndex)
				{
					IterState.ParentIndexAccessors[ParentKeyIndex]->GetRange<int32>(LocalArray, StartReadIndex, *IterState.ParentIndexKeys[ParentKeyIndex], EPCGAttributeAccessorFlags::AllowConstructible);

					for (int32 Index = 0; Index < Count; ++Index)
					{
						ParentKeys[StartWriteIndex + Index][ParentKeyIndex] = LocalArray[Index];
					}
				}

				return Count;
			};

			FPCGAsync::AsyncProcessingOneToOneRangeEx(
				&Context->AsyncState,
				IterState.PointIndexKeys[0]->GetNum(),
				InitializePointKeysAndParentKeys,
				GetPointKeysAndParentKeys,
				/*bEnableTimeSlicing=*/false,
				ChunkSize);

			TMap<TArray<int32, TInlineAllocator<KeySize>>, int32> PointKeysToIndexMap;
			for (int Index = 0; Index < PointKeys.Num(); ++Index)
			{
				PointKeysToIndexMap.Add(PointKeys[Index], Index);
			}

			bool bHasPointsWithInvalidParent = false;

			IterState.ParentIndices.SetNumUninitialized(ParentKeys.Num());
			for(int Index = 0; Index < ParentKeys.Num(); ++Index)
			{
				const TArray<int32, TInlineAllocator<KeySize>>& OriginalParentKey = ParentKeys[Index];
				if(!OriginalParentKey.Contains(INDEX_NONE))
				{
					const int32* ParentIndex = PointKeysToIndexMap.Find(ParentKeys[Index]);
					if (ParentIndex)
					{
						IterState.ParentIndices[Index] = *ParentIndex;
					}
					else
					{
						IterState.ParentIndices[Index] = InvalidParentIndex;
						bHasPointsWithInvalidParent = true;
					}
				}
				else
				{
					IterState.ParentIndices[Index] = INDEX_NONE;
				}
			}

			IterState.bHasPointsWithInvalidParent = bHasPointsWithInvalidParent;
			IterState.bParentMappingDone = true;

			if (Context->ShouldStop())
			{
				return false;
			}
		}

		// 3. * TODO * - could build hierarchy depth at this stage instead of relying on an attribute -
		// 4. Partition on hierarchy depth
		if (!IterState.bHierarchyDepthPartitionDone)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGApplyHierarchyElement::Execute::HierarchyDepthPartition);
			IterState.HierarchyPartition = PCGMetadataPartitionCommon::AttributeGenericPartition(IterState.InputData, IterState.HierarchyDepthSelector, Context);
			if (IterState.HierarchyPartition.IsEmpty())
			{
				// Error logged in the partitioning code
				return true;
			}

			// Build an indirection and perform some slight validation
			TArray<int32, TInlineAllocator<64>> FirstIndexPerPartition;
			Algo::Transform(IterState.HierarchyPartition, FirstIndexPerPartition, [](const TArray<int32>& InPartition) { return InPartition.IsEmpty() ? INDEX_NONE : InPartition[0]; });

			TUniquePtr<IPCGAttributeAccessorKeys> FirstIndexPerPartitionKeys = MakeUnique<FPCGAttributeAccessorKeysPointsSubset>(IterState.InputData, FirstIndexPerPartition);

			TArray<int32> HierarchyDepthPerPartition;
			HierarchyDepthPerPartition.SetNumUninitialized(IterState.HierarchyPartition.Num());
			IterState.HierarchyDepthAccessor->GetRange<int32>(HierarchyDepthPerPartition, 0, *FirstIndexPerPartitionKeys, EPCGAttributeAccessorFlags::AllowConstructible);

			TArray<TPair<int32, int32>> PartitionRepresentatives;
			PartitionRepresentatives.Reserve(HierarchyDepthPerPartition.Num());

			for (int PartitionIndex = 0; PartitionIndex < HierarchyDepthPerPartition.Num(); ++PartitionIndex)
			{
				PartitionRepresentatives.Emplace(PartitionIndex, HierarchyDepthPerPartition[PartitionIndex]);
			}

			PartitionRepresentatives.Sort([](const TPair<int32, int32>& A, const TPair<int32, int32>& B) { return (A.Value < B.Value) || (A.Value == B.Value && A.Key < B.Key); });

			IterState.HierarchyPartitionOrder.Reserve(PartitionRepresentatives.Num());

			for(const auto& [PartitionIndex, PartitionDepth] : PartitionRepresentatives)
			{
				if (IterState.HierarchyPartitionOrder.Num() == PartitionDepth)
				{
					IterState.HierarchyPartitionOrder.Add(PartitionIndex);
				}
				else
				{
					// At this point, anything downstream is going to be broken. We'll go ahead and mark the parents of these as invalid
					for (const int32& InvalidPointIndex : IterState.HierarchyPartition[PartitionIndex])
					{
						IterState.ParentIndices[InvalidPointIndex] = InvalidParentIndex;
					}

					IterState.bHasPointsWithInvalidParent = true;
				}
			}

			IterState.bHierarchyDepthPartitionDone = true;

			if (Context->ShouldStop())
			{
				return false;
			}
		}

		// 5. For all depths partitions, compute & write transform to output data.
		while (IterState.CurrentDepth < IterState.HierarchyPartitionOrder.Num())
		{
			// Perform current depth iteration
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGApplyHierarchyElement::Execute::ComputeTransforms);
				constexpr int32 ChunkSize = 64;
				const TArray<int32>& CurrentDepthIndices = IterState.HierarchyPartition[IterState.HierarchyPartitionOrder[IterState.CurrentDepth]];
				const bool bIsRoot = IterState.CurrentDepth == 0;

				TUniquePtr<IPCGAttributeAccessorKeys> SubsetKeys = MakeUnique<FPCGAttributeAccessorKeysPointsSubset>(IterState.InputData, CurrentDepthIndices);

				if (bIsRoot)
				{
					// Set transform to relative transform x current point transform
					auto AsyncProcessFunc = [&CurrentDepthIndices, &IterState, &SubsetKeys](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
					{
						if(!IterState.OutputData->HasSpatialDataParent())
						{
							const TArrayView<const int32> IndicesView = MakeArrayView(&CurrentDepthIndices[StartReadIndex], Count);
							IterState.InputData->CopyPointsTo(IterState.OutputData, IndicesView, IndicesView);
						}

						TPCGValueRange<FTransform> ReadWriteTransformRange = IterState.OutputData->GetTransformValueRange();

						TArray<FTransform, TInlineAllocator<ChunkSize>> RelativeTransforms;
						RelativeTransforms.SetNumUninitialized(Count);
						IterState.RelativeTransformAccessor->GetRange<FTransform>(RelativeTransforms, StartReadIndex, *SubsetKeys, EPCGAttributeAccessorFlags::AllowConstructible);

						for(int32 Index = 0; Index < Count; ++Index)
						{
							const int32 CurrentPointIndex = CurrentDepthIndices[Index + StartReadIndex];
							FTransform& Transform = ReadWriteTransformRange[CurrentPointIndex];
							const FTransform& RelativeTransform = RelativeTransforms[Index];

							Transform = RelativeTransform * Transform;
						}

						return Count;
					};

					FPCGAsync::AsyncProcessingOneToOneRangeEx(
						&Context->AsyncState,
						CurrentDepthIndices.Num(),
						[]() {},
						AsyncProcessFunc,
						/*bEnableTimeSlicing=*/false);
				}
				else
				{
					bool bHasPointsWithInvalidParent = false;
					// Set transform to relative transform x parent transform (& apply options)
					auto AsyncProcessFunc = [&CurrentDepthIndices, &IterState, &SubsetKeys, &bHasPointsWithInvalidParent](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
					{
						if (!IterState.OutputData->HasSpatialDataParent())
						{
							const TArrayView<const int32> IndicesView = MakeArrayView(&CurrentDepthIndices[StartReadIndex], Count);
							IterState.InputData->CopyPointsTo(IterState.OutputData, IndicesView, IndicesView);
						}

						check(StartReadIndex == StartWriteIndex);
						TPCGValueRange<FTransform> ReadWriteTransformRange = IterState.OutputData->GetTransformValueRange();

						TArray<FTransform, TInlineAllocator<ChunkSize>> RelativeTransforms;
						RelativeTransforms.SetNumUninitialized(Count);
						IterState.RelativeTransformAccessor->GetRange<FTransform>(RelativeTransforms, StartReadIndex, *SubsetKeys, EPCGAttributeAccessorFlags::AllowConstructible);

						TArray<bool, TInlineAllocator<ChunkSize>> ApplyParentRotation;
						ApplyParentRotation.SetNumUninitialized(Count);
						IterState.ApplyRotationAccessor->GetRange<bool>(ApplyParentRotation, StartReadIndex, *SubsetKeys, EPCGAttributeAccessorFlags::AllowConstructible);

						TArray<bool, TInlineAllocator<ChunkSize>> ApplyParentScale;
						ApplyParentScale.SetNumUninitialized(Count);
						IterState.ApplyScaleAccessor->GetRange<bool>(ApplyParentScale, StartReadIndex, *SubsetKeys, EPCGAttributeAccessorFlags::AllowConstructible);

						TArray<bool, TInlineAllocator<ChunkSize>> ApplyHierarchy;
						ApplyHierarchy.SetNumUninitialized(Count);
						IterState.ApplyHierarchyAccessor->GetRange<bool>(ApplyHierarchy, StartReadIndex, *SubsetKeys, EPCGAttributeAccessorFlags::AllowConstructible);

						for (int32 Index = 0; Index < Count; ++Index)
						{
							const int32 CurrentPointIndex = CurrentDepthIndices[Index + StartReadIndex];
							const int32 CurrentParentIndex = IterState.ParentIndices[CurrentPointIndex];

							if (ApplyHierarchy[Index] == IterState.bInvertApplyHierarchy)
							{
								continue;
							}

							// At this point, if the current parent is invalid, or it has been marked invalid, we'll mark this point as invalid too.
							if(CurrentParentIndex < 0 || IterState.ParentIndices[CurrentParentIndex] == InvalidParentIndex)
							{
								if (CurrentParentIndex != InvalidParentIndex)
								{
									IterState.ParentIndices[CurrentPointIndex] = InvalidParentIndex;
									bHasPointsWithInvalidParent = true;
								}
								
								continue;
							}

							FTransform& Transform = ReadWriteTransformRange[CurrentPointIndex];
							const FTransform& ParentTransform = ReadWriteTransformRange[CurrentParentIndex];
							const FTransform& RelativeTransform = RelativeTransforms[Index];

							Transform = RelativeTransform * ParentTransform;

							// Finally, apply options (ignore scale/rotation from parent)
							if (ApplyParentRotation[Index] == IterState.bInvertApplyRotation)
							{
								Transform.SetRotation(RelativeTransform.GetRotation());
							}

							if (ApplyParentScale[Index] == IterState.bInvertApplyScale)
							{
								Transform.SetScale3D(RelativeTransform.GetScale3D());
							}
						}

						return Count;
					};

					FPCGAsync::AsyncProcessingOneToOneRangeEx(
						&Context->AsyncState,
						CurrentDepthIndices.Num(),
						[]() {},
						AsyncProcessFunc,
						/*bEnableTimeSlicing=*/false);

					IterState.bHasPointsWithInvalidParent |= bHasPointsWithInvalidParent;
				}

				++IterState.CurrentDepth;
			}

			if ((IterState.CurrentDepth != IterState.HierarchyPartitionOrder.Num() || IterState.bHasPointsWithInvalidParent) && Context->ShouldStop())
			{
				return false;
			}

			// Finally, if we had points that were unparented, we need to cull them out.
			if (IterState.CurrentDepth == IterState.HierarchyPartitionOrder.Num() && IterState.bHasPointsWithInvalidParent)
			{
				if (Settings->bWarnOnPointsWithInvalidParent)
				{
					PCGLog::LogWarningOnGraph(LOCTEXT("SomePointsHaveAnInvalidParent", "Some points have either an invalid parent index, or an invalid depth. They will be culled from the final results."), Context);
				}

				const UPCGBasePointData* OriginalData = IterState.OutputData;
				UPCGBasePointData* FilteredData = FPCGContext::NewPointData_AnyThread(Context);
				FPCGInitializeFromDataParams InitializeFromDataParams(OriginalData);

				// Do not inherit because we will filter out the invalid points.
				InitializeFromDataParams.bInheritSpatialData = false;
				FilteredData->InitializeFromDataWithParams(InitializeFromDataParams);

				Context->OutputData.TaggedData[IterState.OutputDataIndex].Data = FilteredData;
				
				auto InitializeFunc = [FilteredData, OriginalData]()
				{
					FilteredData->SetNumPoints(OriginalData->GetNumPoints());
					FilteredData->AllocateProperties(OriginalData->GetAllocatedProperties());
					FilteredData->CopyUnallocatedPropertiesFrom(OriginalData);
				};

				auto AsyncProcessRangeFunc = [FilteredData, OriginalData, &IterState](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
				{
					const FConstPCGPointValueRanges ReadRanges(OriginalData);
					FPCGPointValueRanges WriteRanges(FilteredData, /*bAllocate=*/false);

					int32 NumWritten = 0;

					for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
					{
						if (IterState.ParentIndices[ReadIndex] != InvalidParentIndex)
						{
							WriteRanges.SetFromValueRanges(StartWriteIndex + NumWritten, ReadRanges, ReadIndex);
							++NumWritten;
						}
					}

					return NumWritten;
				};

				auto MoveDataRangeFunc = [FilteredData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
				{
					FilteredData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
				};

				auto FinishedFunc = [FilteredData](int32 NumWritten)
				{
					FilteredData->SetNumPoints(NumWritten);
				};

				FPCGAsync::AsyncProcessingRangeEx(
					&Context->AsyncState,
					OriginalData->GetNumPoints(),
					InitializeFunc,
					AsyncProcessRangeFunc,
					MoveDataRangeFunc,
					FinishedFunc,
					/*bEnableTimeSlicing=*/false);
			}
		}

		// Finally, if we had nothing to do, we shouldn't return any points.
		if (IterState.HierarchyPartitionOrder.IsEmpty())
		{
			IterState.OutputData->SetNumPoints(0);
		}

		return true;
	});
}

#undef LOCTEXT_NAMESPACE
