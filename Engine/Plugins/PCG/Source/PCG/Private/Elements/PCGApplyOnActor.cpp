// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGApplyOnActor.h"

#include "PCGContext.h"
#include "PCGEdge.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "Helpers/PCGHelpers.h"

#include "GameFramework/Actor.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGApplyOnActor)

#define LOCTEXT_NAMESPACE "PCGApplyOnActorElement"

namespace PCGApplyOnActorConstants
{
	const FName ObjectPropertyOverridesLabel = TEXT("Property Overrides");
	const FText ObjectPropertyOverridesTooltip = LOCTEXT("ObjectOverrideToolTip", "Provide property overrides for the target object. The attribute name must match the InputSource name in the object property override description.");
}

#if WITH_EDITOR
void UPCGApplyOnActorSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	if(GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGApplyOnActorNodeMoveTargetActorEdgeToInput && ensure(InOutNode))
	{
		// Find in the input pins if we have edge(s) on the "TargetActor" pin
		TObjectPtr<UPCGPin>* TargetActorPinPtr = InputPins.FindByPredicate([](const TObjectPtr<UPCGPin>& InputPin) { return InputPin && InputPin->Properties.Label == TEXT("TargetActor"); });

		if (TargetActorPinPtr && (*TargetActorPinPtr)->EdgeCount() > 0)
		{
			TObjectPtr<UPCGPin> TargetActorPin = *TargetActorPinPtr;

			// Create input pin if needed (should be all the time)
			TObjectPtr<UPCGPin> InputPin = nullptr;
			TObjectPtr<UPCGPin>* PreExistingInputPin = InputPins.FindByPredicate([](const TObjectPtr<UPCGPin>& InputPin) { return InputPin && InputPin->Properties.Label == PCGPinConstants::DefaultInputLabel; });

			if (!PreExistingInputPin)
			{
				InputPin = NewObject<UPCGPin>(InOutNode);
				InputPin->Node = InOutNode;
				InputPin->Properties = FPCGPinProperties(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any, /*bAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);
				InputPins.Insert(InputPin, 0);
			}
			else
			{
				InputPin = *PreExistingInputPin;
			}

			TArray<UPCGPin*> UpstreamPins;
			for (const UPCGEdge* Connection : TargetActorPin->Edges)
			{
				UpstreamPins.Add(Connection->InputPin);
			}

			for (UPCGPin* UpstreamPin : UpstreamPins)
			{
				UpstreamPin->BreakEdgeTo(TargetActorPin);
				UpstreamPin->AddEdgeTo(InputPin);
			}
		}
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InOutNode->RenameInputPin(PCGPinConstants::DefaultDependencyOnlyLabel, PCGPinConstants::DefaultExecutionDependencyLabel, /*bInBroadcastUpdate=*/false);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGApplyOnActorSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any, /*bAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true).SetRequiredPin();
	PinProperties.Emplace(PCGApplyOnActorConstants::ObjectPropertyOverridesLabel, EPCGDataType::Any, /*bAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true, PCGApplyOnActorConstants::ObjectPropertyOverridesTooltip);
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGApplyOnActorSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGApplyOnActorSettings::CreateElement() const
{
	return MakeShared<FPCGApplyOnObjectElement>();
}

bool FPCGApplyOnObjectElement::PrepareDataInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGApplyOnActorElement::PrepareData);
	check(Context);

	const UPCGApplyOnActorSettings* Settings = Context->GetInputSettings<UPCGApplyOnActorSettings>();
	check(Settings);

	FPCGLoadObjectsFromPathContext* ThisContext = static_cast<FPCGLoadObjectsFromPathContext*>(Context);
	return ThisContext->InitializeAndRequestLoad(PCGPinConstants::DefaultInputLabel,
		Settings->ObjectReferenceAttribute,
		{},
		/*bPersistAllData=*/false,
		Settings->bSilenceErrorOnEmptyObjectPath,
		Settings->bSynchronousLoad);
}

bool FPCGApplyOnObjectElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGApplyOnActorElement::Execute);

	check(Context);
	FPCGLoadObjectsFromPathContext* ThisContext = static_cast<FPCGLoadObjectsFromPathContext*>(Context);

	const UPCGApplyOnActorSettings* Settings = Context->GetInputSettings<UPCGApplyOnActorSettings>();
	check(Settings);

	TArray<TPair<UObject*, int32>> TargetObjectsAndIndices;

#if WITH_EDITOR
	const bool bPropagateObjectChangeEvent = Settings->bPropagateObjectChangeEvent;
#else 
	const bool bPropagateObjectChangeEvent = false;
#endif // WITH_EDITOR

	const TArray<FPCGTaggedData> InputTaggedData = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	const TArray<FPCGTaggedData> OverridesTaggedData = Context->InputData.GetInputsByPin(PCGApplyOnActorConstants::ObjectPropertyOverridesLabel);

	// Inputs are always forwarded.
	Context->OutputData.TaggedData = InputTaggedData;

	if (Settings->bUseOneOverrideDataPerElement)
	{
		if (OverridesTaggedData.Num() > 1 && OverridesTaggedData.Num() != ThisContext->PathsToObjectsAndDataIndex.Num())
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InconsistentDataCount", "When having one data set per element, it is expected to have the same number of override data than elements, we expected {0} and got {1} data."), ThisContext->PathsToObjectsAndDataIndex.Num(), InputTaggedData.Num()), Context);
			return true;
		}
	}
	else
	{
		if (OverridesTaggedData.Num() > 1 && OverridesTaggedData.Num() != InputTaggedData.Num())
		{
			PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, PCGApplyOnActorConstants::ObjectPropertyOverridesLabel, Context);
			return true;
		}
	}

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("ApplyToObjects", "Applying PCG changes to Objects"), Context->ExecutionSource.Get() && Context->ExecutionSource->GetExecutionState().UseTransactions());
#endif

	int CurrentPathIndex = 0;
	while (CurrentPathIndex < ThisContext->PathsToObjectsAndDataIndex.Num())
	{
		TargetObjectsAndIndices.Reset();

		int32 InputIndex = ThisContext->PathsToObjectsAndDataIndex[CurrentPathIndex].Get<1>();
		const int32 StartingIndex = CurrentPathIndex;
		while (CurrentPathIndex < ThisContext->PathsToObjectsAndDataIndex.Num() && InputIndex == ThisContext->PathsToObjectsAndDataIndex[CurrentPathIndex].Get<1>())
		{
			TargetObjectsAndIndices.Emplace(ThisContext->PathsToObjectsAndDataIndex[CurrentPathIndex].Get<0>().ResolveObject(), ThisContext->PathsToObjectsAndDataIndex[CurrentPathIndex].Get<2>());
			++CurrentPathIndex;
		}

		if (!OverridesTaggedData.IsEmpty())
		{
			TArray<const UPCGData*, TInlineAllocator<16>> OverridesData;
			if (Settings->bUseOneOverrideDataPerElement)
			{
				const int32 NumElements = CurrentPathIndex - StartingIndex;
				OverridesData.Reserve(NumElements);
				for (int32 i = 0; i < NumElements; ++i)
				{
					OverridesData.Add(OverridesTaggedData[(StartingIndex + i) % OverridesTaggedData.Num()].Data);
				}
			}
			else
			{
				OverridesData.Add(OverridesTaggedData[InputIndex % OverridesTaggedData.Num()].Data);
			}
		
			PCGObjectPropertyOverrideHelpers::FApplyOverrideParams Params =
			{
				.ObjectPropertyOverrideDescriptions = MakeConstArrayView(Settings->PropertyOverrideDescriptions),
				.TargetObjectToOverrideIndices = MakeConstArrayView(TargetObjectsAndIndices),
				.OverridesData = MakeConstArrayView(OverridesData),
				.OptionalContext = Context,
				.bPropagateEditChangeEvent = bPropagateObjectChangeEvent,
				.bOneDataSetPerElement = Settings->bUseOneOverrideDataPerElement,
			};

			PCGObjectPropertyOverrideHelpers::ApplyOverrides(Params);
		}

		if (!TargetObjectsAndIndices.IsEmpty() && !Settings->TagsToAddOnActors.IsEmpty())
		{
			if (UActorComponent* ActorComponent = Cast<UActorComponent>(TargetObjectsAndIndices[0].Key))
			{
				for (const FName& Tag : Settings->TagsToAddOnActors)
				{
					ActorComponent->ComponentTags.AddUnique(Tag);
				}
			}
			else if (AActor* Actor = Cast<AActor>(TargetObjectsAndIndices[0].Key))
			{
				for (const FName& Tag : Settings->TagsToAddOnActors)
				{
					Actor->Tags.AddUnique(Tag);
				}
			}
		}

		for (const TPair<UObject*, int32>& TargetObjectAndIndex : TargetObjectsAndIndices)
		{
			UObject* TargetObject = TargetObjectAndIndex.Key;
			if (!TargetObject)
			{
				continue;
			}

			for (UFunction* Function : PCGHelpers::FindUserFunctions(TargetObject->GetClass(), Settings->PostProcessFunctionNames, { UPCGFunctionPrototypes::GetPrototypeWithNoParams() }, Context))
			{
				TargetObject->ProcessEvent(Function, nullptr);
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
