// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGApplyScaleToBounds.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGApplyScaleToBounds)

#define LOCTEXT_NAMESPACE "PCGApplyScaleToBoundsElement"

FPCGElementPtr UPCGApplyScaleToBoundsSettings::CreateElement() const
{
	return MakeShared<FPCGApplyScaleToBoundsElement>();
}

EPCGPointNativeProperties FPCGApplyScaleToBoundsElement::GetPropertiesToAllocate(FPCGContext* Context) const
{
	return EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax;
}

bool FPCGApplyScaleToBoundsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGApplyScaleToBoundsElement::Execute);
	check(Context);

	FPCGApplyScaleToBoundsElement::ContextType* ApplyScaleContext = static_cast<FPCGApplyScaleToBoundsElement::ContextType*>(Context);

	return ExecutePointOperation(ApplyScaleContext, [](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
	{
		TPCGValueRange<FVector> BoundsMinRange = OutputData->GetBoundsMinValueRange();
		TPCGValueRange<FVector> BoundsMaxRange = OutputData->GetBoundsMaxValueRange();
		TPCGValueRange<FTransform> TransformRange = OutputData->GetTransformValueRange();

		for (int32 Index = StartIndex; Index < (StartIndex + Count); ++Index)
		{
			FVector& BoundsMin = BoundsMinRange[Index];
			FVector& BoundsMax = BoundsMaxRange[Index];
			FTransform& Transform = TransformRange[Index];
			PCGPointHelpers::ApplyScaleToBounds(Transform, BoundsMin, BoundsMax);
		}
		
		return true;
	});
}

#undef LOCTEXT_NAMESPACE
