// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAttractElement.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGSortAttributes.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "SpatialAlgo/PCGOctreeQueries.h"

#include "Algo/Count.h"
#include "Algo/MaxElement.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttractElement)

#define LOCTEXT_NAMESPACE "PCGAttractElement"

namespace PCGAttractElement
{
	namespace Constants
	{
		const FName InputSourceLabel = TEXT("Source");
		const FName InputTargetLabel = TEXT("Target");
		const FName AttractIndexName = TEXT("AttractIndex");
	}

	namespace Helpers
	{
		static void InterpolatePoints(FAttractState& AttractData, const int32 InterpolatorIndex, const UPCGAttractSettings* Settings)
		{
			FAttractState::FAttributeInterpolationData& Interpolator = AttractData.Interpolators[InterpolatorIndex];

			// Process
			int32 OutIndex = 0;
			const int32 SourcePointCount = AttractData.SourceData->GetNumPoints();

			for (int32 SourceIndex = 0; SourceIndex < SourcePointCount; ++SourceIndex)
			{
				const int32 TargetIndex = AttractData.Mapping[SourceIndex];

				if (TargetIndex == INDEX_NONE)
				{
					// Point was not matched, we're either not touching it or dropping it.
					if (!Settings->bRemoveUnattractedPoints)
					{
						++OutIndex;
					}
				}
				else if (AttractData.bTargetIsSource && TargetIndex == SourceIndex)
				{
					// Nothing to do regardless of weights
					++OutIndex; 
				}
				else
				{
					double Alpha = 0.0;
					if (AttractData.SourceWeights.IsEmpty() && AttractData.TargetWeights.IsEmpty())
					{
						Alpha = Settings->Weight;
					}
					else if (AttractData.SourceWeights.IsEmpty())
					{
						Alpha = AttractData.TargetWeights[TargetIndex];
					}
					else if (AttractData.TargetWeights.IsEmpty())
					{
						Alpha = AttractData.SourceWeights[SourceIndex];
					}
					else
					{
						const double SourceWeight = AttractData.SourceWeights[SourceIndex];
						const double TargetWeight = AttractData.TargetWeights[TargetIndex];
						Alpha = (SourceWeight + TargetWeight > UE_SMALL_NUMBER) ? (TargetWeight / (SourceWeight + TargetWeight)) : 0.5;
					}

					auto Callback = [OutIndex, SourceIndex, TargetIndex, Alpha, &Interpolator, &AttractData]<typename T>(T)
					{
						T SourceValue{}, TargetValue{};
						Interpolator.SourceAccessor->Get(SourceValue, SourceIndex, *Interpolator.SourceKeys);
						Interpolator.TargetAccessor->Get(TargetValue, TargetIndex, *Interpolator.TargetKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);

						T OutputValue{};
						if constexpr (PCG::Private::MetadataTraits<T>::CanInterpolate)
						{
							OutputValue = PCG::Private::MetadataTraits<T>::WeightedSum(PCG::Private::MetadataTraits<T>::ZeroValueForWeightedSum(), SourceValue, 1.0 - Alpha);
							OutputValue = PCG::Private::MetadataTraits<T>::WeightedSum(OutputValue, TargetValue, Alpha);

							if constexpr (PCG::Private::MetadataTraits<T>::InterpolationNeedsNormalization)
							{
								static_assert(PCG::Private::MetadataTraits<T>::CanNormalize);

								// We need to normalize the resulting value
								PCG::Private::MetadataTraits<T>::Normalize(OutputValue);
							}
						}
						else // Take value based on maximal weight
						{
							// TODO - Optimization: we don't need to query both values when we're in this case, as we'll use only one of them.
							OutputValue = (Alpha <= 0.5 ? SourceValue : TargetValue);
						}

						Interpolator.OutputAccessor->Set(OutputValue, OutIndex, *AttractData.OutputKeys);
					};

					PCGMetadataAttribute::CallbackWithRightType(Interpolator.SourceAccessor->GetUnderlyingType(), Callback);
					++OutIndex;
				}
			}
		}

		// Runs a single iteration of the attract algorithm.
		static void RunAttractIteration(const int32 Index, FAttractState& OutAttractData, const UPCGAttractSettings* AttractSettings)
		{
			const TConstPCGValueRange<FTransform> TransformRange = OutAttractData.SourceData->GetConstTransformValueRange();

			if (AttractSettings->Mode == EPCGAttractMode::Closest)
			{
				// Note: there is a slight behavior difference here in the target==source case, we don't want a point to be attracted to itself
				if (OutAttractData.bTargetIsSource)
				{
					int32 ClosestTargetIndex = INDEX_NONE;
					double MinDistanceSquared = std::numeric_limits<double>::max();
					bool bFoundColocatedPoint = false;

					UPCGOctreeQueries::ForEachPointInsideSphere(
						OutAttractData.TargetData,
						TransformRange[Index].GetLocation(),
						AttractSettings->Distance,
						[Index, &OutAttractData, &MinDistanceSquared, &ClosestTargetIndex, &bFoundColocatedPoint](const UPCGBasePointData* PointData, int32 TargetIndex, const double DistanceSquared)
						{
							// Ignore self-selection
							if (TargetIndex == Index)
							{
								return;
							}

							if (FMath::IsNearlyZero(DistanceSquared))
							{
								if (bFoundColocatedPoint)
								{
									// Implementation note: this is an arbitrary choice. We could have randomized the targets in a consistent way though.
									ClosestTargetIndex = FMath::Min(TargetIndex, ClosestTargetIndex);
								}
								else
								{
									bFoundColocatedPoint = true;
									ClosestTargetIndex = TargetIndex;
								}

								return;
							}

							// Non-colocated points will just be on a distance basis
							if (DistanceSquared < MinDistanceSquared)
							{
								MinDistanceSquared = DistanceSquared;
								ClosestTargetIndex = TargetIndex;
							}
						});

					// Update assignment based on closest target index.
					OutAttractData.Mapping[Index] = ClosestTargetIndex;
				}
				else
				{
					// Simple closest point query
					const int32 ClosestPointIndex = UPCGOctreeQueries::GetClosestPointIndex(
						OutAttractData.TargetData,
						TransformRange[Index].GetLocation(),
						/*bInDiscardCenter=*/false,
						AttractSettings->Distance);
					
					if(ClosestPointIndex != INDEX_NONE)
					{
						OutAttractData.Mapping[Index] = ClosestPointIndex;
					}
				}
			}
			else // Min/Max attribute test inside of search radius
			{
				UPCGOctreeQueries::ForEachPointInsideSphere(
					OutAttractData.TargetData,
					TransformRange[Index].GetLocation(),
					AttractSettings->Distance,
					[Index, &OutAttractData](const UPCGBasePointData* PointData, int32 TargetIndex, const double DistanceSquared)
					{
						if (OutAttractData.Mapping[Index] == INDEX_NONE)
						{
							OutAttractData.Mapping[Index] = TargetIndex;
						}
						else
						{
							// Implementation note: if we stored 'sorted target indices indices' in the mapping instead
							// we could remove the secondary find here, but it would require another pass to go back to real indices
							int32 SortedTargetIndex = OutAttractData.SortedTargetIndices.IndexOfByKey(TargetIndex);
							check(SortedTargetIndex != INDEX_NONE);

							int32 SortedPreviousIndex = OutAttractData.SortedTargetIndices.IndexOfByKey(OutAttractData.Mapping[Index]);
							check(SortedPreviousIndex != INDEX_NONE);

							if (SortedTargetIndex < SortedPreviousIndex)
							{
								OutAttractData.Mapping[Index] = TargetIndex;
							}
						}
					});
			}
		}
	} // namespace Helpers

	namespace Algorithm
	{
		static bool Sequential(const FPCGContext* InContext, FAttractState& OutAttractData, const UPCGAttractSettings* AttractSettings)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttractElement::Sequential);

			check(InContext);

			while (OutAttractData.IterationIndex < OutAttractData.SourceData->GetNumPoints())
			{
				Helpers::RunAttractIteration(OutAttractData.IterationIndex++, OutAttractData, AttractSettings);

				if (OutAttractData.IterationIndex != OutAttractData.SourceData->GetNumPoints() && InContext->ShouldStop())
				{
					// Not done
					return false;
				}
			}

			// Fully done
			return true;
		}
	} // namespace Algorithm
} // namespace PCGAttractElement

UPCGAttractSettings::UPCGAttractSettings()
{
	AttractorIndexAttribute.Update(PCGAttractElement::Constants::AttractIndexName.ToString());

	FPCGAttributePropertyInputSelector DefaultSourceInput;
	DefaultSourceInput.SetPointProperty(EPCGPointProperties::Position);
	FPCGAttributePropertyInputSelector DefaultTargetInput;
	DefaultTargetInput.SetPointProperty(EPCGPointProperties::Position);
	SourceAndTargetAttributeMapping.Add(DefaultSourceInput, DefaultTargetInput);

	OutputAttractIndexAttribute.Update(PCGAttractElement::Constants::AttractIndexName.ToString());
}

#if WITH_EDITOR
FText UPCGAttractSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Attracts source points to target points based on a max distance and a criteria.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGAttractSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	FPCGPinProperties& SourcePinProperty = Properties.Emplace_GetRef(PCGAttractElement::Constants::InputSourceLabel, EPCGDataType::Point);
	SourcePinProperty.SetRequiredPin();

	// TODO: For now, only allow a single target input. Only N:0 and N:1 are currently supported.
	FPCGPinProperties& TargetPinProperty = Properties.Emplace_GetRef(PCGAttractElement::Constants::InputTargetLabel, EPCGDataType::Point);
	TargetPinProperty.SetRequiredPin();
#if WITH_EDITOR
	SourcePinProperty.Tooltip = LOCTEXT("SourcePinTooltip", "The source points to be attracted by the target points.");
	TargetPinProperty.Tooltip = LOCTEXT("TargetPinTooltip", "The target points that will attract the source points.");
#endif // WITH_EDITOR

	return Properties;
}

FPCGElementPtr UPCGAttractSettings::CreateElement() const
{
	return MakeShared<FPCGAttractElement>();
}

bool FPCGAttractElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttractElement::PrepareData);

	const UPCGAttractSettings* Settings = InContext->GetInputSettings<UPCGAttractSettings>();
	check(Settings);

	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context);

	const TArray<FPCGTaggedData> SourceInputs = Context->InputData.GetInputsByPin(PCGAttractElement::Constants::InputSourceLabel);
	const TArray<FPCGTaggedData> TargetInputs = Context->InputData.GetInputsByPin(PCGAttractElement::Constants::InputTargetLabel);

	// Early out if no source to modify or there is no target to be attracted to.
	if (SourceInputs.IsEmpty() || TargetInputs.IsEmpty())
	{
		Context->OutputData.TaggedData = SourceInputs;
		return true;
	}

	// Only N:1 and N:N are currently supported.
	if (TargetInputs.Num() > 1 && TargetInputs.Num() != SourceInputs.Num())
	{
		PCGLog::InputOutput::LogInvalidCardinalityError(PCGAttractElement::Constants::InputSourceLabel, PCGAttractElement::Constants::InputTargetLabel, InContext);
		return true;
	}

	// Additional validation: if the attract operation would do nothing, log a warning
	if (!Settings->bRemoveUnattractedPoints && !Settings->bOutputAttractIndex && Settings->SourceAndTargetAttributeMapping.IsEmpty())
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("NoOperation", "Attract node settings will do nothing. Sources will be forwarded."), Context);
		Context->OutputData.TaggedData = SourceInputs;
		return true;
	}

	Context->InitializePerExecutionState([Settings](const ContextType*, ExecStateType& OutState)
	{
		OutState.AttractFunction = PCGAttractElement::Algorithm::Sequential;
		check(OutState.AttractFunction);

		return EPCGTimeSliceInitResult::Success;
	});

	Context->InitializePerIterationStates(SourceInputs.Num(), [&SourceInputs, &TargetInputs, Settings, Context](IterStateType& OutState, const ExecStateType&, const uint32 IterationIndex)
	{
		const UPCGBasePointData* SourcePointData = Cast<UPCGBasePointData>(SourceInputs[IterationIndex].Data);

		if (!SourcePointData || SourcePointData->IsEmpty())
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		const UPCGBasePointData* TargetPointData = Cast<UPCGBasePointData>(TargetInputs[IterationIndex % TargetInputs.Num()].Data);

		if (!TargetPointData)
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(Context);
		OutPointData->InitializeFromData(SourcePointData);
		Context->OutputData.TaggedData.Add_GetRef(SourceInputs[IterationIndex]).Data = OutPointData;

		const bool bTargetIsSource = (TargetPointData == SourcePointData);

		OutState.TargetData = TargetPointData;
		OutState.SourceData = SourcePointData;
		OutState.OutputData = OutPointData;
		OutState.IterationIndex = 0;
		OutState.bTargetIsSource = bTargetIsSource;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OutState.TargetPointData = Cast<UPCGPointData>(TargetPointData);
		OutState.SourcePointData = Cast<UPCGPointData>(SourcePointData);
		OutState.OutPointData = Cast<UPCGPointData>(OutPointData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		OutState.Mapping.SetNumUninitialized(SourcePointData->GetNumPoints());

		if (Settings->Mode == EPCGAttractMode::FromIndex)
		{
			const FPCGAttributePropertyInputSelector AttractorIndexSelector = Settings->AttractorIndexAttribute.CopyAndFixLast(SourcePointData);
			TUniquePtr<const IPCGAttributeAccessor> AttractorIndexAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(SourcePointData, AttractorIndexSelector);
			TUniquePtr<const IPCGAttributeAccessorKeys> AttractorIndexKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SourcePointData, AttractorIndexSelector);
			if (!AttractorIndexAccessor || !AttractorIndexKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(AttractorIndexSelector, Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			if (!AttractorIndexAccessor->GetRange(MakeArrayView(OutState.Mapping), 0, *AttractorIndexKeys))
			{
				PCGLog::Metadata::LogFailToGetAttributeError(AttractorIndexSelector, Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			// Fix indices that would be invalid vs target
			const int32 TargetPointCount = TargetPointData->GetNumPoints();
			for (int32& MapIndex : OutState.Mapping)
			{
				if (MapIndex < 0 || MapIndex >= TargetPointCount)
				{
					MapIndex = INDEX_NONE;
				}
			}
		}
		else
		{
			if (bTargetIsSource)
			{
				// Self-assignment
				for (int i = 0; i < OutState.Mapping.Num(); ++i)
				{
					OutState.Mapping[i] = i;
				}
			}
			else
			{
				// Point to no target
				for (int i = 0; i < OutState.Mapping.Num(); ++i)
				{
					OutState.Mapping[i] = INDEX_NONE;
				}
			}
		}

		// Prioritize selected attract target by comparing attributes for the min or max value.
		if (Settings->Mode == EPCGAttractMode::MinAttribute || Settings->Mode == EPCGAttractMode::MaxAttribute)
		{
			const FPCGAttributePropertyInputSelector TargetSelector = Settings->TargetAttribute.CopyAndFixLast(TargetPointData);
			TUniquePtr<const IPCGAttributeAccessor> TargetAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(TargetPointData, TargetSelector);
			TUniquePtr<const IPCGAttributeAccessorKeys> TargetKeys = PCGAttributeAccessorHelpers::CreateConstKeys(TargetPointData, TargetSelector);
			if (!TargetAccessor || !TargetKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(TargetSelector, Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			// Build order of target values - we don't need to keep the values, just the relative ordering is going to be fine.
			TArray<int> TargetIndices;
			TargetIndices.SetNumUninitialized(TargetPointData->GetNumPoints());
			for (int i = 0; i < TargetIndices.Num(); ++i)
			{
				TargetIndices[i] = i;
			}

			PCGAttributeAccessorHelpers::SortByAttribute(*TargetAccessor, *TargetKeys, TargetIndices, Settings->Mode == EPCGAttractMode::MinAttribute);
			OutState.SortedTargetIndices = std::move(TargetIndices);
		}

		if (Settings->bUseSourceWeight)
		{
			const FPCGAttributePropertyInputSelector SourceWeightSelector = Settings->SourceWeightAttribute.CopyAndFixLast(SourcePointData);
			const TUniquePtr<const IPCGAttributeAccessor> SourceWeightAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(SourcePointData, SourceWeightSelector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> SourceWeightKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SourcePointData, SourceWeightSelector);
			if (!SourceWeightAccessor)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(SourceWeightSelector, Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			OutState.SourceWeights.SetNumUninitialized(SourcePointData->GetNumPoints());

			if (!SourceWeightAccessor->GetRange(MakeArrayView(OutState.SourceWeights), 0, *SourceWeightKeys))
			{
				PCGLog::Metadata::LogFailToGetAttributeError(SourceWeightSelector, Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}
		}

		if (Settings->bUseTargetWeight)
		{
			const FPCGAttributePropertyInputSelector TargetWeightSelector = Settings->TargetWeightAttribute.CopyAndFixLast(TargetPointData);
			const TUniquePtr<const IPCGAttributeAccessor> TargetWeightAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(TargetPointData, TargetWeightSelector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> TargetWeightKeys = PCGAttributeAccessorHelpers::CreateConstKeys(TargetPointData, TargetWeightSelector);
			if (!TargetWeightAccessor)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(TargetWeightSelector, Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			OutState.TargetWeights.SetNumUninitialized(TargetPointData->GetNumPoints());

			if (!TargetWeightAccessor->GetRange(MakeArrayView(OutState.TargetWeights), 0, *TargetWeightKeys))
			{
				PCGLog::Metadata::LogFailToGetAttributeError(TargetWeightSelector, Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}
		}

		for (const auto& SourceAndTargetMapping : Settings->SourceAndTargetAttributeMapping)
		{
			const FPCGAttributePropertyInputSelector SourceSelector = SourceAndTargetMapping.Key.CopyAndFixLast(SourcePointData);

			TUniquePtr<const IPCGAttributeAccessor> SourceAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(SourcePointData, SourceSelector);
			TUniquePtr<const IPCGAttributeAccessorKeys> SourceKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SourcePointData, SourceSelector);

			if (!SourceAccessor || !SourceKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(SourceSelector, Context);
				continue;
			}

			const FPCGAttributePropertyInputSelector TargetSelector = SourceAndTargetMapping.Value.CopyAndFixLast(TargetPointData);

			TUniquePtr<const IPCGAttributeAccessor> TargetAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(TargetPointData, TargetSelector);
			TUniquePtr<const IPCGAttributeAccessorKeys> TargetKeys = PCGAttributeAccessorHelpers::CreateConstKeys(TargetPointData, TargetSelector);

			if (!TargetAccessor || !TargetKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(SourceSelector, Context);
				continue;
			}

			// Finally, validate that the target attribute type can be broadcasted to the source type.
			if (!PCG::Private::IsBroadcastableOrConstructible(TargetAccessor->GetUnderlyingType(), SourceAccessor->GetUnderlyingType()))
			{
				PCGLog::Metadata::LogIncomparableAttributesError(TargetSelector, SourceSelector, Context);
				continue;
			}
			
			FPCGAttributePropertyOutputSelector OutputSelector;
			OutputSelector.ImportFromOtherSelector(SourceSelector);
			OutputSelector = OutputSelector.CopyAndFixSource(&SourceSelector, OutPointData);

			PCGAttractElement::FAttractState::FAttributeInterpolationData& Interpolator = OutState.Interpolators.Emplace_GetRef();
			Interpolator.SourceAccessor = std::move(SourceAccessor);
			Interpolator.SourceKeys = std::move(SourceKeys);
			Interpolator.TargetAccessor = std::move(TargetAccessor);
			Interpolator.TargetKeys = std::move(TargetKeys);
			Interpolator.OutputSelector = std::move(OutputSelector);
		}

		return EPCGTimeSliceInitResult::Success;
	});

	return true;
}

bool FPCGAttractElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttractElement::Execute);

	ContextType* TimeSlicedContext = static_cast<ContextType*>(InContext);
	check(TimeSlicedContext);

	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		return true;
	}

	if (TimeSlicedContext->GetExecutionStateResult() == EPCGTimeSliceInitResult::NoOperation)
	{
		TimeSlicedContext->OutputData = TimeSlicedContext->InputData;
		return true;
	}

	const UPCGAttractSettings* Settings = TimeSlicedContext->GetInputSettings<UPCGAttractSettings>();
	check(Settings);

	return ExecuteSlice(TimeSlicedContext, [Settings](const ContextType* Context, const ExecStateType& ExecState, IterStateType& AttractData, const uint32 IterIndex)
	{
		if (Context->GetIterationStateResult(IterIndex) == EPCGTimeSliceInitResult::NoOperation)
		{
			return true;
		}

		if (!AttractData.bAttractPhaseDone)
		{
			if (Settings->Mode == EPCGAttractMode::FromIndex)
			{
				// Nothing to do, we've already gotten the data
			}
			else if (!ExecState.AttractFunction(Context, AttractData, Settings))
			{
				return false;
			}

			AttractData.bAttractPhaseDone = true;
		}

		// Copy points from source that are kept across
		TArray<int32> Mapping;

		if (Settings->bRemoveUnattractedPoints)
		{
			const int32 NumPoints = Algo::CountIf(AttractData.Mapping, [](int32 Index)
			{
				return Index != INDEX_NONE;
			});

			AttractData.OutputData->SetNumPoints(NumPoints, /*bInitializeValues=*/false);
			AttractData.OutputData->AllocateProperties(AttractData.SourceData->GetAllocatedProperties());
			AttractData.OutputData->CopyUnallocatedPropertiesFrom(AttractData.SourceData);

			Mapping.Reserve(NumPoints);

			const FConstPCGPointValueRanges SourceRanges(AttractData.SourceData);
			FPCGPointValueRanges OutRanges(AttractData.OutputData);

			int32 WriteIndex = 0;
			for (int PointIndex = 0; PointIndex < AttractData.SourceData->GetNumPoints(); ++PointIndex)
			{
				if (AttractData.Mapping[PointIndex] != INDEX_NONE)
				{
					OutRanges.SetFromValueRanges(WriteIndex, SourceRanges, PointIndex);
					Mapping.Add(AttractData.Mapping[PointIndex]);
					++WriteIndex;
				}
			}
		}
		else
		{
			if (!AttractData.OutputData->HasSpatialDataParent())
			{
				UPCGBasePointData::SetPoints(AttractData.SourceData, AttractData.OutputData, {}, /*bCopyAll=*/true);
			}

			Mapping = AttractData.Mapping;
		}

		// After we've created the points, we can now create the keys
		AttractData.OutputKeys = PCGAttributeAccessorHelpers::CreateKeys(AttractData.OutputData, FPCGAttributePropertyOutputSelector());

		// After we've created the points we can now create the accessor
		if (Settings->bOutputAttractIndex)
		{
			const FPCGAttributePropertyOutputNoSourceSelector AttractIndexSelector = Settings->OutputAttractIndexAttribute;

			// Make sure we create the attribute if needed.
			if (AttractIndexSelector.IsBasicAttribute())
			{
				AttractData.OutputData->Metadata->FindOrCreateAttribute<int32>(AttractIndexSelector.GetName(), 0, /*bAllowInterpolation=*/false);
			}

			TUniquePtr<IPCGAttributeAccessor> AttractIndexAccessor = PCGAttributeAccessorHelpers::CreateAccessor(AttractData.OutputData, AttractIndexSelector);
			if (!AttractIndexAccessor)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(AttractIndexSelector, Context);
				return true;
			}

			AttractData.OutputAttractIndexAccessor = std::move(AttractIndexAccessor);
			AttractData.OutputAttractIndexAccessor->SetRange<int32>(Mapping, 0, *AttractData.OutputKeys);
		}

		// Apply weighting from source to target
		while (AttractData.InterpolationIndex < AttractData.Interpolators.Num())
		{
			TUniquePtr<IPCGAttributeAccessor> OutputAccessor = PCGAttributeAccessorHelpers::CreateAccessor(AttractData.OutputData, AttractData.Interpolators[AttractData.InterpolationIndex].OutputSelector);

			if (!OutputAccessor)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(AttractData.Interpolators[AttractData.InterpolationIndex].OutputSelector, Context);
				return false;
			}

			AttractData.Interpolators[AttractData.InterpolationIndex].OutputAccessor = std::move(OutputAccessor);
			PCGAttractElement::Helpers::InterpolatePoints(AttractData, AttractData.InterpolationIndex++, Settings);

			// Check for time-slice
			if (AttractData.InterpolationIndex != AttractData.Interpolators.Num() && Context->ShouldStop())
			{
				return false;
			}
		}

		return true;
	});
}

#undef LOCTEXT_NAMESPACE
