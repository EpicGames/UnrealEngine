// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAttributeFilter.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include "Algo/AnyOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeFilter)

#define LOCTEXT_NAMESPACE "PCGPointFilterElement"

namespace PCGAttributeFilterConstants
{
	constexpr int32 ChunkSize = 256;

#if WITH_EDITOR
	const FText InputPinTooltip = LOCTEXT("InputPinTooltip", "This pin accepts Point data and Attribute Sets. Spatial data will be collapsed to point data.");

	const FText FilterPinTooltip = LOCTEXT("FilterPinTooltip", "This pin accepts Statial data and Attribute Sets. If the data is Spatial, it will automatically sample input points in it. "
		"If it is points, it will sample if \"Spatial Query\" is enabled, otherwise points number need to match with input.");
#endif // WITH_EDITOR

	constexpr int32 DefaultAliasIndex = 0;
	constexpr int32 PointFilterAliasIndex = 1;
	constexpr int32 AttributeFilterAliasIndex = 2;
}

namespace PCGAttributeFilterHelpers
{
	struct ThresholdInfo
	{
		TUniquePtr<const IPCGAttributeAccessor> ThresholdAccessor;
		TUniquePtr<const IPCGAttributeAccessorKeys> ThresholdKeys;
		bool bUseInputDataForThreshold = false;
		bool bNeedDifferentAccessorsPerData = true;
		TArray<FPCGTaggedData> FilterData;
		bool bUseSpatialQuery = false;

		UPCGBasePointData* ThresholdPointData = nullptr;
		const UPCGSpatialData* ThresholdSpatialData = nullptr;
	};

	bool InitialPrepareThresholdInfo(FPCGContext* InContext, const TArray<FPCGTaggedData>& DataToFilter, TArray<FPCGTaggedData> FilterData, const FPCGAttributeFilterThresholdSettings& ThresholdSettings, const FName ThresholdPinLabel, ThresholdInfo& OutThresholdInfo)
	{
		if (ThresholdSettings.bUseConstantThreshold)
		{
			auto ConstantThreshold = [&OutThresholdInfo]<typename T>(const T& Value)
			{
				OutThresholdInfo.ThresholdAccessor = MakeUnique<FPCGConstantValueAccessor<T>>(Value);
				// Dummy keys
				OutThresholdInfo.ThresholdKeys = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();
			};

			ThresholdSettings.AttributeTypes.Dispatcher(ConstantThreshold);
			OutThresholdInfo.bNeedDifferentAccessorsPerData = false;
		}
		else
		{
			// If we have more than 1 filter data, it is a N:N operation, we need to have as many data to filter than filter data.
			// We won't create thresholds accessors as we'll need one for each data.
			if (FilterData.Num() > 1 && ThresholdSettings.bFilterEachInputWithEachFilterDataRespectively)
			{
				if (DataToFilter.Num() != FilterData.Num())
				{
					PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, ThresholdPinLabel, InContext);
					return false;
				}
				
				OutThresholdInfo.FilterData = MoveTemp(FilterData);
				OutThresholdInfo.bUseSpatialQuery = ThresholdSettings.bUseSpatialQuery;
			}
			else if (FilterData.IsEmpty())
			{
				// We'll use the input data as filter
				OutThresholdInfo.bUseInputDataForThreshold = true;
			}
			else
			{
				const UPCGData* ThresholdData = FilterData[0].Data;

				if (const UPCGSpatialData* ThresholdSpatialData = Cast<const UPCGSpatialData>(ThresholdData))
				{
					// If the threshold is spatial or points (and spatial query is enabled), we'll use spatial query (meaning we'll have to sample points).
					// Don't create an accessor yet (ThresholdData = nullptr), it will be created further down.
					OutThresholdInfo.ThresholdSpatialData = ThresholdSpatialData;
					if (!ThresholdSpatialData->IsA<UPCGBasePointData>() || ThresholdSettings.bUseSpatialQuery)
					{
						OutThresholdInfo.bUseSpatialQuery = true;
						ThresholdData = nullptr;
					}
				}

				if (ThresholdData)
				{
					FPCGAttributePropertyInputSelector ThresholdSelector = ThresholdSettings.ThresholdAttribute.CopyAndFixLast(ThresholdData);
					OutThresholdInfo.ThresholdAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(ThresholdData, ThresholdSelector);
					OutThresholdInfo.ThresholdKeys = PCGAttributeAccessorHelpers::CreateConstKeys(ThresholdData, ThresholdSelector);
					OutThresholdInfo.bNeedDifferentAccessorsPerData = false;
				}
			}
		}

		return true;
	}

	bool PrepareThresholdInfoFromInput(FPCGContext* InContext, const int32 DataIndex, const UPCGData* InputData, const int32 NumInput, const FPCGAttributeFilterThresholdSettings& ThresholdSettings, ThresholdInfo& InOutThresholdInfo, int16 TargetType, bool bCheckCompare, bool bCheckStringSearch, bool bWarnOnDataMissingAttribute, const ThresholdInfo* OtherInfo = nullptr)
	{
		check(InContext && InputData);

		auto CreateAccessor = [&ThresholdSettings, &InOutThresholdInfo](const UPCGData* InData)
		{
			if (InData)
			{
				const FPCGAttributePropertyInputSelector ThresholdSelector = ThresholdSettings.ThresholdAttribute.CopyAndFixLast(InData);
				InOutThresholdInfo.ThresholdAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InData, ThresholdSelector);
				InOutThresholdInfo.ThresholdKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InData, ThresholdSelector);
			}
		};

		if (InOutThresholdInfo.bUseInputDataForThreshold)
		{
			// If we have no threshold accessor, we use the same data as input
			CreateAccessor(InputData);
		}
		else if (InOutThresholdInfo.bNeedDifferentAccessorsPerData)
		{
			check(InOutThresholdInfo.ThresholdSpatialData != nullptr || !InOutThresholdInfo.FilterData.IsEmpty());
			const UPCGData* FilterData = InOutThresholdInfo.FilterData.IsEmpty() ? InOutThresholdInfo.ThresholdSpatialData : InOutThresholdInfo.FilterData[DataIndex].Data;
			
			if (const UPCGSpatialData* SpatialData = Cast<const UPCGSpatialData>(FilterData); SpatialData && (!SpatialData->IsA<UPCGBasePointData>() || InOutThresholdInfo.bUseSpatialQuery))
			{
				InOutThresholdInfo.ThresholdSpatialData = SpatialData;

				// Don't do 2 spatial query if we are iterating on the same input
				if (OtherInfo != nullptr && OtherInfo->ThresholdSpatialData == InOutThresholdInfo.ThresholdSpatialData)
				{
					InOutThresholdInfo.ThresholdPointData = OtherInfo->ThresholdPointData;
				}
				else
				{
					// Reset the point data and reserving some points
					// No need to reserve the full number of points, since we'll go by chunk
					// Only allocate the chunk size
					InOutThresholdInfo.ThresholdPointData = FPCGContext::NewPointData_AnyThread(InContext);

					FPCGInitializeFromDataParams InitializeFromDataParams(InOutThresholdInfo.ThresholdSpatialData);
					InitializeFromDataParams.bInheritSpatialData = false;

					InOutThresholdInfo.ThresholdPointData->InitializeFromDataWithParams(InitializeFromDataParams);
					InOutThresholdInfo.ThresholdPointData->SetNumPoints(PCGAttributeFilterConstants::ChunkSize);
					InOutThresholdInfo.ThresholdPointData->AllocateProperties(EPCGPointNativeProperties::All);
				}
				
				// Accessor will be valid, but keys will point to default points. But since it is a view, it will be updated when we sample the points.
				CreateAccessor(InOutThresholdInfo.ThresholdPointData);
			}
			else
			{
				// At that point, filter data can't be empty, as it would have been caught by earlier ifs
				check(InOutThresholdInfo.FilterData.IsValidIndex(DataIndex));
				CreateAccessor(InOutThresholdInfo.FilterData[DataIndex].Data);
			}
		}

		if (!InOutThresholdInfo.ThresholdAccessor.IsValid() || !InOutThresholdInfo.ThresholdKeys.IsValid())
		{
			if (bWarnOnDataMissingAttribute)
			{
				PCGE_LOG_C(Warning, GraphAndLog, InContext, FText::Format(LOCTEXT("AttributeMissingForFilter", "Filter data does not have '{0}' threshold attribute/property"), FText::FromName(ThresholdSettings.ThresholdAttribute.GetName())));
			}

			return false;
		}

		// And also validate that types are comparable/constructible. Do it all at once for a single dispatch.
		bool bCanCompare = true;
		bool bCanSearchString = true;
		PCGMetadataAttribute::CallbackWithRightType(InOutThresholdInfo.ThresholdAccessor->GetUnderlyingType(), [&bCanCompare, &bCanSearchString, TargetType](auto ThresholdValue)
		{
			bCanCompare = PCG::Private::MetadataTraits<decltype(ThresholdValue)>::CanCompare;
			bCanSearchString = PCG::Private::MetadataTraits<decltype(ThresholdValue)>::CanSearchString;
		});

		// Comparison between threshold and target data needs to be of the same type. So we have to make sure that we can
		// request target type from threshold type. ie. We need to make sure we can broadcast threshold type to target type, or construct a target type from a threshold type.
		// For example: if target is double but threshold is int32, we can broadcast int32 to double, to compare a double with a double.
		if (!PCG::Private::IsBroadcastableOrConstructible(InOutThresholdInfo.ThresholdAccessor->GetUnderlyingType(), TargetType))
		{
			const FText InputTypeName = PCG::Private::GetTypeNameText(TargetType);
			const FText ThresholdTypeName = PCG::Private::GetTypeNameText(InOutThresholdInfo.ThresholdAccessor->GetUnderlyingType());
			PCGE_LOG_C(Warning, GraphAndLog, InContext, FText::Format(LOCTEXT("TypeConversionFailed", "Cannot convert threshold type '{0}' to input target type '{1}'"),
				ThresholdTypeName,
				InputTypeName));
			return false;
		}

		if (bCheckCompare && !bCanCompare)
		{
			const FText InputTypeName = PCG::Private::GetTypeNameText(TargetType);
			PCGE_LOG_C(Warning, GraphAndLog, InContext, FText::Format(LOCTEXT("TypeComparisonFailed", "Cannot compare target type '{0}'"), InputTypeName));
			return false;
		}

		if (bCheckStringSearch && !bCanSearchString)
		{
			const FText InputTypeName = PCG::Private::GetTypeNameText(TargetType);
			PCGE_LOG_C(Warning, GraphAndLog, InContext, FText::Format(LOCTEXT("TypeStringSearchFailed", "Cannot perform string operations on target type '{0}'"), InputTypeName));
			return false;
		}

		// Check that if we have points as threshold, that the point data has the same number of point that the input data, or there is just a single point
		if (InOutThresholdInfo.ThresholdSpatialData != nullptr && !InOutThresholdInfo.bUseSpatialQuery)
		{
			if (InOutThresholdInfo.ThresholdKeys->GetNum() != NumInput && InOutThresholdInfo.ThresholdKeys->GetNum() != 1)
			{
				PCGE_LOG_C(Warning, GraphAndLog, InContext, FText::Format(LOCTEXT("InvalidNumberOfThresholdPoints", "Threshold point data doesn't have the same number of elements ({0}) than the input data ({1})."), InOutThresholdInfo.ThresholdKeys->GetNum(), NumInput));
				return false;
			}
		}

		return true;
	}
}

#if WITH_EDITOR
void FPCGAttributeFilterThresholdSettings::OnPostLoad()
{
	AttributeTypes.OnPostLoad();
}
#endif

////////////////////////////////////////
// UPCGAttributeFilteringSettings
////////////////////////////////////////

FPCGDataTypeIdentifier UPCGAttributeFilteringSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	check(InPin);
	if (!InPin->IsOutputPin())
	{
		return Super::GetCurrentPinTypesID(InPin);
	}

	// Output pin narrows to union of inputs on first pin
	const FPCGDataTypeIdentifier InputTypeUnion = GetTypeUnionIDOfIncidentEdges(PCGPinConstants::DefaultInputLabel);

	// Spatial is collapsed into points
	if (InputTypeUnion.IsValid() && (InputTypeUnion & EPCGDataType::Spatial) == InputTypeUnion)
	{
		return FPCGDataTypeIdentifier{EPCGDataType::Point};
	}
	else
	{
		return InputTypeUnion.IsValid() ? InputTypeUnion : FPCGDataTypeIdentifier{EPCGDataType::Any};
	}
}

TArray<FPCGPinProperties> UPCGAttributeFilteringSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DataToFilterPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::PointOrParam);
	DataToFilterPinProperty.SetRequiredPin();

#if WITH_EDITOR
	PinProperties.Last().Tooltip = PCGAttributeFilterConstants::InputPinTooltip;
#endif //WITH_EDITOR

	if (!bUseConstantThreshold)
	{
		PinProperties.Emplace(PCGAttributeFilterConstants::FilterLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/ false);

#if WITH_EDITOR
		PinProperties.Last().Tooltip = PCGAttributeFilterConstants::FilterPinTooltip;
#endif // WITH_EDITOR
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGAttributeFilteringSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInFilterLabel, EPCGDataType::PointOrParam);
	PinProperties.Emplace(PCGPinConstants::DefaultOutFilterLabel, EPCGDataType::PointOrParam);

	return PinProperties;
}

FString UPCGAttributeFilteringSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGAttributeFilteringSettings, TargetAttribute)))
	{
		return FString();
	}
	else
#endif
	{
		return TargetAttribute.GetDisplayText().ToString();
	}
}

FPCGElementPtr UPCGAttributeFilteringSettings::CreateElement() const
{
	return MakeShared<FPCGAttributeFilterElement>();
}

UPCGAttributeFilteringSettings::UPCGAttributeFilteringSettings()
	: UPCGSettings()
{
	// Previous default object was: density for both selectors
	// Recreate the same default
	TargetAttribute.SetPointProperty(EPCGPointProperties::Density);
	ThresholdAttribute.SetPointProperty(EPCGPointProperties::Density);

	// Change the default for spatial query to be false
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		bUseSpatialQuery = false;
	}
}

#if WITH_EDITOR
TArray<FPCGPreConfiguredSettingsInfo> UPCGAttributeFilteringSettings::GetPreconfiguredInfo() const
{
	TArray<FPCGPreConfiguredSettingsInfo> PreconfiguredInfo;
	PreconfiguredInfo.Emplace(PCGAttributeFilterConstants::DefaultAliasIndex, GetDefaultNodeTitle());
	PreconfiguredInfo.Emplace(PCGAttributeFilterConstants::PointFilterAliasIndex, LOCTEXT("PointNodeTitle", "Point Filter"));
	PreconfiguredInfo.Emplace(PCGAttributeFilterConstants::AttributeFilterAliasIndex, LOCTEXT("AttributeNodeTitle", "Attribute Filter"));

	return PreconfiguredInfo;
}
#endif

void UPCGAttributeFilteringSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo)
{
	// If index is 1, it is the default ($Density)
	if (PreconfigureInfo.PreconfiguredIndex != PCGAttributeFilterConstants::PointFilterAliasIndex)
	{
		TargetAttribute.SetAttributeName(PCGMetadataAttributeConstants::LastAttributeName);
		ThresholdAttribute.SetAttributeName(PCGMetadataAttributeConstants::LastAttributeName);
	}
}

void UPCGAttributeFilteringSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	AttributeTypes.OnPostLoad();

	// Check for the data spatial to point gate version
	if (DataVersion < FPCGCustomVersion::NoMoreSpatialDataConversionToPointDataByDefaultOnNonPointPins)
	{
		bHasSpatialToPointDeprecation = true;
	}
#endif // WITH_EDITOR
}

////////////////////////////////////////
// UPCGAttributeFilteringRangeSettings
////////////////////////////////////////

void UPCGAttributeFilteringRangeSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	MinThreshold.OnPostLoad();
	MaxThreshold.OnPostLoad();

	// Check for the data spatial to point gate version
	if (DataVersion < FPCGCustomVersion::NoMoreSpatialDataConversionToPointDataByDefaultOnNonPointPins)
	{
		bHasSpatialToPointDeprecation = true;
	}
#endif
}

#if WITH_EDITOR
TArray<FPCGPreConfiguredSettingsInfo> UPCGAttributeFilteringRangeSettings::GetPreconfiguredInfo() const
{
	TArray<FPCGPreConfiguredSettingsInfo> PreconfiguredInfo;
	PreconfiguredInfo.Emplace(PCGAttributeFilterConstants::DefaultAliasIndex, GetDefaultNodeTitle());
	PreconfiguredInfo.Emplace(PCGAttributeFilterConstants::PointFilterAliasIndex, LOCTEXT("PointRangeNodeTitle", "Point Filter Range"));
	PreconfiguredInfo.Emplace(PCGAttributeFilterConstants::AttributeFilterAliasIndex, LOCTEXT("AttributeRangeNodeTitle", "Attribute Filter Range"));

	return PreconfiguredInfo;
}
#endif

void UPCGAttributeFilteringRangeSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo)
{
	// If index is 1, it is the default ($Density)
	if (PreconfigureInfo.PreconfiguredIndex != PCGAttributeFilterConstants::PointFilterAliasIndex)
	{
		TargetAttribute.SetAttributeName(PCGMetadataAttributeConstants::LastAttributeName);
		MinThreshold.ThresholdAttribute.SetAttributeName(PCGMetadataAttributeConstants::LastAttributeName);
		MaxThreshold.ThresholdAttribute.SetAttributeName(PCGMetadataAttributeConstants::LastAttributeName);
	}
}

FString UPCGAttributeFilteringRangeSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGAttributeFilteringRangeSettings, TargetAttribute)))
	{
		return FString();
	}
	else
#endif
	{
		return TargetAttribute.GetDisplayText().ToString();
	}
}

FPCGElementPtr UPCGAttributeFilteringRangeSettings::CreateElement() const
{
	return MakeShared<FPCGAttributeFilterRangeElement>();
}

UPCGAttributeFilteringRangeSettings::UPCGAttributeFilteringRangeSettings()
	: UPCGSettings()
{
	// Previous default object was: density for both selectors
	// Recreate the same default
	TargetAttribute.SetPointProperty(EPCGPointProperties::Density);
	MinThreshold.ThresholdAttribute.SetPointProperty(EPCGPointProperties::Density);
	MaxThreshold.ThresholdAttribute.SetPointProperty(EPCGPointProperties::Density);

	// Change the default for spatial query to be false
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		MinThreshold.bUseSpatialQuery = false;
		MaxThreshold.bUseSpatialQuery = false;
	}
}


FPCGDataTypeIdentifier UPCGAttributeFilteringRangeSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	check(InPin);
	if (!InPin->IsOutputPin())
	{
		return Super::GetCurrentPinTypesID(InPin);
	}

	// Output pin narrows to union of inputs on first pin
	const FPCGDataTypeIdentifier InputTypeUnion = GetTypeUnionIDOfIncidentEdges(PCGPinConstants::DefaultInputLabel);

	// Spatial is collapsed into points
	if (InputTypeUnion.IsValid() && (InputTypeUnion & EPCGDataType::Spatial) == InputTypeUnion)
	{
		return FPCGDataTypeIdentifier{EPCGDataType::Point};
	}
	else
	{
		return InputTypeUnion.IsValid() ? InputTypeUnion : FPCGDataTypeIdentifier{EPCGDataType::Any};
	}
}

TArray<FPCGPinProperties> UPCGAttributeFilteringRangeSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DataToFilterPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::PointOrParam);
	DataToFilterPinProperty.SetRequiredPin();

#if WITH_EDITOR
	PinProperties.Last().Tooltip = PCGAttributeFilterConstants::InputPinTooltip;
#endif //WITH_EDITOR

	if (!MinThreshold.bUseConstantThreshold)
	{
		PinProperties.Emplace(PCGAttributeFilterConstants::FilterMinLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/ false);

#if WITH_EDITOR
		PinProperties.Last().Tooltip = PCGAttributeFilterConstants::FilterPinTooltip;
#endif // WITH_EDITOR
	}

	if (!MaxThreshold.bUseConstantThreshold)
	{
		PinProperties.Emplace(PCGAttributeFilterConstants::FilterMaxLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/ false);

#if WITH_EDITOR
		PinProperties.Last().Tooltip = PCGAttributeFilterConstants::FilterPinTooltip;
#endif // WITH_EDITOR
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGAttributeFilteringRangeSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInFilterLabel, EPCGDataType::PointOrParam);
	PinProperties.Emplace(PCGPinConstants::DefaultOutFilterLabel, EPCGDataType::PointOrParam);

	return PinProperties;
}

////////////////////////////////////////
// FPCGAttributeFilterElementBase
////////////////////////////////////////

bool FPCGAttributeFilterElementBase::DoFiltering(FPCGContext* Context, EPCGAttributeFilterOperator InOperation, const FPCGAttributePropertyInputSelector& InTargetAttribute, bool bHasSpatialToPointDeprecation, bool bWarnOnDataMissingAttribute, bool bAlwaysGenerateOutputData, const FPCGAttributeFilterThresholdSettings& FirstThreshold, const FPCGAttributeFilterThresholdSettings* SecondThreshold) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeFilterElementBase::DoFiltering);
	check(Context);

	struct FOperationData
	{
		const UPCGBasePointData* OriginalPointData = nullptr;
		UPCGBasePointData* InFilterPointData = nullptr;
		UPCGBasePointData* OutFilterPointData = nullptr;
		TBitArray<TInlineAllocator<2048>> FilterBitArray;

		const UPCGData* OriginalData = nullptr;
		UPCGMetadata* InFilterMetadata = nullptr;
		UPCGMetadata* OutFilterMetadata = nullptr;

		bool bIsInputPointData = false;
	} OperationData;

	TArray<FPCGTaggedData> DataToFilter = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const FName FirstThresholdPinLabel = SecondThreshold ? PCGAttributeFilterConstants::FilterMinLabel : PCGAttributeFilterConstants::FilterLabel;
	const FName SecondThresholdPinLabel = PCGAttributeFilterConstants::FilterMaxLabel;

	TArray<FPCGTaggedData> FirstFilterData = Context->InputData.GetInputsByPin(FirstThresholdPinLabel);
	TArray<FPCGTaggedData> SecondFilterData = SecondThreshold ? Context->InputData.GetInputsByPin(SecondThresholdPinLabel) : TArray<FPCGTaggedData>{};

	const EPCGAttributeFilterOperator Operator = InOperation;

	// If there is no input, do nothing
	if (DataToFilter.IsEmpty())
	{
		return true;
	}

	// Only support second threshold with the InRange Operation.
	// We can't have a second threshold if it is not InRange
	if (!ensure((SecondThreshold != nullptr) == (InOperation == EPCGAttributeFilterOperator::InRange)))
	{
		return true;
	}

	PCGAttributeFilterHelpers::ThresholdInfo FirstThresholdInfo;
	PCGAttributeFilterHelpers::ThresholdInfo SecondThresholdInfo;

	if (!PCGAttributeFilterHelpers::InitialPrepareThresholdInfo(Context, DataToFilter, MoveTemp(FirstFilterData), FirstThreshold, FirstThresholdPinLabel, FirstThresholdInfo))
	{
		return true;
	}

	if (SecondThreshold && !PCGAttributeFilterHelpers::InitialPrepareThresholdInfo(Context, DataToFilter, MoveTemp(SecondFilterData), *SecondThreshold, SecondThresholdPinLabel, SecondThresholdInfo))
	{
		return true;
	}

	for (int32 i = 0; i < DataToFilter.Num(); ++i)
	{
		const FPCGTaggedData& Input = DataToFilter[i];
		const UPCGData* OriginalData = Input.Data;
		if (!OriginalData)
		{
			continue;
		}

		if (const UPCGSpatialData* SpatialInput = Cast<const UPCGSpatialData>(OriginalData))
		{
			if (!SpatialInput->IsA<UPCGBasePointData>())
			{
				const UPCGBasePointData* OriginalPointData = nullptr;

				if (bHasSpatialToPointDeprecation)
				{
					OriginalPointData = SpatialInput->ToBasePointData(Context);
				}

				if (!OriginalPointData)
				{
					PCGE_LOG(Error, GraphAndLog, LOCTEXT("NoPointDataInInput", "Unable to get point data from input. Use a conversion node before this node to transform it to points."));
					continue;
				}

				OperationData.bIsInputPointData = true;
				OriginalData = OriginalPointData;
			}
			else
			{
				OperationData.bIsInputPointData = true;
				OriginalData = SpatialInput;
			}
		}
		else if (OriginalData->IsA<UPCGParamData>())
		{
			// Disable Spatial Queries
			FirstThresholdInfo.bUseSpatialQuery = false;
			SecondThresholdInfo.bUseSpatialQuery = false;
			OperationData.bIsInputPointData = false;
		}
		else
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInput", "Input is not a point data nor an attribute set. Unsupported."));
			continue;
		}

		// Helper lambdas to fail nicely and forward input to in/out filter pin
		// If there is a problem with threshold -> forward to InFilter
		auto ForwardInputToInFilterPin = [&Outputs, Input, OriginalData]()
		{
			FPCGTaggedData& InFilterOutput = Outputs.Add_GetRef(Input);
			InFilterOutput.Pin = PCGPinConstants::DefaultInFilterLabel;
			// Use original data because it could have been collapsed
			InFilterOutput.Data = OriginalData;
		};

		// If there is a problem with target -> forward to OutFilter
		auto ForwardInputToOutFilterPin = [&Outputs, Input, OriginalData]()
		{
			FPCGTaggedData& OutFilterOutput = Outputs.Add_GetRef(Input);
			OutFilterOutput.Pin = PCGPinConstants::DefaultOutFilterLabel;
			// Use original data because it could have been collapsed
			OutFilterOutput.Data = OriginalData;
		};

		FPCGAttributePropertyInputSelector TargetAttribute = InTargetAttribute.CopyAndFixLast(OriginalData);
		TUniquePtr<const IPCGAttributeAccessor> TargetAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OriginalData, TargetAttribute);
		TUniquePtr<const IPCGAttributeAccessorKeys> TargetKeys = PCGAttributeAccessorHelpers::CreateConstKeys(OriginalData, TargetAttribute);

		if (!TargetAccessor.IsValid() || !TargetKeys.IsValid())
		{
			if (bWarnOnDataMissingAttribute)
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("TargetMissingAttribute", "Input data doesn't have target attribute/property '{0}'"), FText::FromName(TargetAttribute.GetName())));
			}

			ForwardInputToOutFilterPin();
			continue;
		}

		const int16 TargetType = TargetAccessor->GetUnderlyingType();
		const bool bCheckStringSearch = (Operator == EPCGAttributeFilterOperator::Substring || Operator == EPCGAttributeFilterOperator::Matches);
		const bool bCheckCompare = (Operator != EPCGAttributeFilterOperator::Equal) && (Operator != EPCGAttributeFilterOperator::NotEqual) && !bCheckStringSearch;
		const int32 NumInput = TargetKeys->GetNum();

		if (NumInput == 0)
		{
			ForwardInputToInFilterPin();
			continue;
		}

		if (!PCGAttributeFilterHelpers::PrepareThresholdInfoFromInput(Context, i, OriginalData, NumInput, FirstThreshold, FirstThresholdInfo, TargetType, bCheckCompare, bCheckStringSearch, bWarnOnDataMissingAttribute))
		{
			ForwardInputToInFilterPin();
			continue;
		}

		if (SecondThreshold && !PCGAttributeFilterHelpers::PrepareThresholdInfoFromInput(Context, i, OriginalData, NumInput , *SecondThreshold, SecondThresholdInfo, TargetType, bCheckCompare, bCheckStringSearch, bWarnOnDataMissingAttribute, &FirstThresholdInfo))
		{
			ForwardInputToInFilterPin();
			continue;
		}

		OperationData.OriginalData = OriginalData;
		int32 NumberOfEntries = 0;

		if (OperationData.bIsInputPointData)
		{
			const UPCGBasePointData* OriginalPointData = CastChecked<UPCGBasePointData>(OriginalData);

			OperationData.OriginalPointData = OriginalPointData;
			NumberOfEntries = OriginalPointData->GetNumPoints();
		}
		else
		{
			// Param data
			const UPCGParamData* OriginalParamData = CastChecked<UPCGParamData>(OriginalData);
			NumberOfEntries = OriginalParamData->Metadata ? OriginalParamData->Metadata->GetItemCountForChild() : 0;
		}

		if (NumberOfEntries == 0)
		{
			ForwardInputToInFilterPin();
			continue;
		}

		// Will be set individually in batches
		OperationData.FilterBitArray.SetNumUninitialized(NumberOfEntries);

		auto Operation = [&Operator, &FirstThresholdInfo, &SecondThresholdInfo, &TargetAccessor, &TargetKeys, &NumberOfEntries, &FirstThreshold, &SecondThreshold, &OperationData](auto Dummy) -> bool
		{
			using Type = decltype(Dummy);

			if (NumberOfEntries <= 0)
			{
				return false;
			}

			TArray<Type, TFixedAllocator<PCGAttributeFilterConstants::ChunkSize>> TargetValues;
			TArray<Type, TFixedAllocator<PCGAttributeFilterConstants::ChunkSize>> FirstThresholdValues;
			TArray<Type, TFixedAllocator<PCGAttributeFilterConstants::ChunkSize>> SecondThresholdValues;
			TArray<bool, TFixedAllocator<PCGAttributeFilterConstants::ChunkSize>> SkipTests;
			TargetValues.SetNum(PCGAttributeFilterConstants::ChunkSize);
			FirstThresholdValues.SetNum(PCGAttributeFilterConstants::ChunkSize);
			SecondThresholdValues.SetNum(PCGAttributeFilterConstants::ChunkSize);

			const bool bShouldSample = FirstThresholdInfo.ThresholdPointData || SecondThresholdInfo.ThresholdPointData;
			if (bShouldSample)
			{
				SkipTests.SetNumUninitialized(PCGAttributeFilterConstants::ChunkSize);
			}

			const int32 NumberOfIterations = (NumberOfEntries + PCGAttributeFilterConstants::ChunkSize - 1) / PCGAttributeFilterConstants::ChunkSize;

			for (int32 i = 0; i < NumberOfIterations; ++i)
			{
				const int32 StartIndex = i * PCGAttributeFilterConstants::ChunkSize;
				const int32 Range = FMath::Min(NumberOfEntries - StartIndex, PCGAttributeFilterConstants::ChunkSize);
				TArrayView<Type> TargetView(TargetValues.GetData(), Range);
				TArrayView<Type> FirstThresholdView(FirstThresholdValues.GetData(), Range);
				TArrayView<Type> SecondThresholdView(SecondThresholdValues.GetData(), Range);

				// Need to reset the skip tests to False
				if (bShouldSample)
				{
					FMemory::Memzero(SkipTests.GetData(), SkipTests.Num() * sizeof(bool));
				}

				// Sampling the points if needed
				auto SamplePointData = [Range, StartIndex, &OperationData, &SkipTests](UPCGBasePointData* InPointData, const UPCGSpatialData* InSpatialData)
				{
					if (InPointData != nullptr)
					{
						const FConstPCGPointValueRanges OriginalRanges(OperationData.OriginalPointData);
						FPCGPointValueRanges ThresholdRanges(InPointData, /*bAllocate=*/false);

						// Threshold points only have "ChunkSize" points.
						for (int32 j = 0; j < Range; ++j)
						{
							FPCGPoint ThresholdPoint;
							const FPCGPoint SourcePoint = OriginalRanges.GetPoint(StartIndex + j);

							// If we already mark this point to skip test, don't even try to sample.
							// If the sample fails, mark the point to skip test.
							if (!SkipTests[j] && InSpatialData->SamplePoint(SourcePoint.Transform, SourcePoint.GetLocalBounds(), ThresholdPoint, InPointData->Metadata))
							{
								ThresholdRanges.SetFromPoint(j, ThresholdPoint);
							}
							else
							{
								SkipTests[j] = true;
							}
						}
					}
				};

				if (bShouldSample)
				{
					SamplePointData(FirstThresholdInfo.ThresholdPointData, FirstThresholdInfo.ThresholdSpatialData);

					if (FirstThresholdInfo.ThresholdPointData != SecondThresholdInfo.ThresholdPointData)
					{
						SamplePointData(SecondThresholdInfo.ThresholdPointData, SecondThresholdInfo.ThresholdSpatialData);
					}
				}

				// If ThresholdView point on ThresholdPointData points, there are only "ChunkSize" points in it.
				// But it wraps around, and since StartIndex is a multiple of "ChunkSize", we'll always start at point 0, as wanted. 
				if (!TargetAccessor->GetRange(TargetView, StartIndex, *TargetKeys) ||
					!FirstThresholdInfo.ThresholdAccessor->GetRange(FirstThresholdView, StartIndex, *FirstThresholdInfo.ThresholdKeys, EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible) ||
					(SecondThresholdInfo.ThresholdAccessor.IsValid() && !SecondThresholdInfo.ThresholdAccessor->GetRange(SecondThresholdView, StartIndex, *SecondThresholdInfo.ThresholdKeys, EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible)))
				{
					return false;
				}

				check(Range == 0 || OperationData.FilterBitArray.IsValidIndex(StartIndex + Range - 1));
				for (int32 j = 0; j < Range; ++j)
				{
					if (bShouldSample && SkipTests[j])
					{
						OperationData.FilterBitArray[StartIndex + j] = true;
						continue;
					}

					const bool bShouldKeep = (Operator == EPCGAttributeFilterOperator::InRange ? 
						PCGAttributeFilterHelpers::ApplyRange(TargetValues[j], FirstThresholdValues[j], SecondThresholdValues[j], FirstThreshold.bInclusive, SecondThreshold->bInclusive) : 
						PCGAttributeFilterHelpers::ApplyCompare(TargetValues[j], FirstThresholdValues[j], Operator));

					OperationData.FilterBitArray[StartIndex + j] = bShouldKeep;
				}
			}

			return true;
		};

		if (PCGMetadataAttribute::CallbackWithRightType(TargetAccessor->GetUnderlyingType(), Operation))
		{
			UPCGData* InFilterData = nullptr;
			UPCGData* OutFilterData = nullptr;

			if (OperationData.bIsInputPointData)
			{
				const int32 NumInFilterPoints = OperationData.FilterBitArray.CountSetBits();
				const int32 NumOutFilterPoints = OperationData.OriginalPointData->GetNumPoints() - NumInFilterPoints;

				FPCGInitializeFromDataParams InitializeFromDataParams(OperationData.OriginalPointData);
				InitializeFromDataParams.bInheritSpatialData = false;
				const FConstPCGPointValueRanges OriginalRanges(OperationData.OriginalPointData);

				// Note: Splitting the set vs unset values is cleaner and likely more optimal, due to branch prediction.

				if (NumInFilterPoints > 0 || bAlwaysGenerateOutputData)
				{
					UPCGBasePointData* InFilterPointData = FPCGContext::NewPointData_AnyThread(Context);
					InFilterPointData->InitializeFromDataWithParams(InitializeFromDataParams);
					InFilterData = InFilterPointData;

					InFilterPointData->SetNumPoints(NumInFilterPoints);
					InFilterPointData->AllocateProperties(OperationData.OriginalPointData->GetAllocatedProperties());
					InFilterPointData->CopyUnallocatedPropertiesFrom(OperationData.OriginalPointData);

					FPCGPointValueRanges InFilterRanges(InFilterPointData, /*bAllocate=*/false);

					int32 InFilterWriteIndex = 0;
					for (TConstSetBitIterator It(OperationData.FilterBitArray); It; ++It)
					{
						InFilterRanges.SetFromValueRanges(InFilterWriteIndex++, OriginalRanges, It.GetIndex());
					}
				}

				if (NumOutFilterPoints > 0 || bAlwaysGenerateOutputData)
				{
					UPCGBasePointData* OutFilterPointData = FPCGContext::NewPointData_AnyThread(Context);
					OutFilterPointData->InitializeFromDataWithParams(InitializeFromDataParams);
					OutFilterData = OutFilterPointData;

					OutFilterPointData->SetNumPoints(NumOutFilterPoints);
					OutFilterPointData->AllocateProperties(OperationData.OriginalPointData->GetAllocatedProperties());
					OutFilterPointData->CopyUnallocatedPropertiesFrom(OperationData.OriginalPointData);

					FPCGPointValueRanges OutFilterRanges(OutFilterPointData, /*bAllocate=*/false);

					// @todo_pcg: Could write a TConstUnsetBitIterator in BitArray.h.
					int32 OutFilterWriteIndex = 0;
					for (int32 Index = 0; Index < OperationData.FilterBitArray.Num(); ++Index)
					{
						if (!OperationData.FilterBitArray[Index])
						{
							OutFilterRanges.SetFromValueRanges(OutFilterWriteIndex++, OriginalRanges, Index);
						}
					}
				}
			}
			else
			{
				check(OperationData.OriginalData);

				if (!OperationData.FilterBitArray.IsEmpty())
				{
					const UPCGMetadata* OriginalMetadata = OperationData.OriginalData->ConstMetadata();
					check(OperationData.FilterBitArray.Num() == OriginalMetadata->GetItemCountForChild())

					TArray<PCGMetadataEntryKey, TInlineAllocator<256>> InEntryKeys;
					TArray<PCGMetadataEntryKey, TInlineAllocator<256>> OutEntryKeys;

					const int32 NumInFilterMetadata = OperationData.FilterBitArray.CountSetBits();
					const int32 NumOutFilterMetadata = OriginalMetadata->GetItemCountForChild() - NumInFilterMetadata;

					InEntryKeys.Reserve(NumInFilterMetadata);
					OutEntryKeys.Reserve(NumOutFilterMetadata);

					for (int32 Index = 0; Index < OperationData.FilterBitArray.Num(); ++Index)
					{
						if (OperationData.FilterBitArray[Index])
						{
							InEntryKeys.Add(Index);
						}
						else
						{
							OutEntryKeys.Add(Index);
						}
					}

					// Add all attributes from the original param data, but without any entry/value. It will be added later.
					if (NumInFilterMetadata > 0 || bAlwaysGenerateOutputData)
					{
						UPCGParamData* InFilterParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
						UPCGMetadata* InFilterMetadata = InFilterParamData->Metadata;
						PCGMetadataHelpers::InitializeMetadataWithDataDomainCopyAndElementsNoCopy(OperationData.OriginalData, InFilterParamData);
						InFilterMetadata->AddAttributesFiltered(OriginalMetadata, TSet<FName>(), EPCGMetadataFilterMode::ExcludeAttributes);
						InFilterMetadata->SetAttributes(InEntryKeys, OriginalMetadata, /*InOutOptionalKeys=*/nullptr, Context);
						InFilterData = InFilterParamData;
					}

					if (NumOutFilterMetadata > 0 || bAlwaysGenerateOutputData)
					{
						UPCGParamData* OutFilterParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
						UPCGMetadata* OutFilterMetadata = OutFilterParamData->Metadata;
						PCGMetadataHelpers::InitializeMetadataWithDataDomainCopyAndElementsNoCopy(OperationData.OriginalData, OutFilterParamData);
						OutFilterMetadata->AddAttributesFiltered(OriginalMetadata, TSet<FName>(), EPCGMetadataFilterMode::ExcludeAttributes);
						OutFilterMetadata->SetAttributes(OutEntryKeys, OriginalMetadata, /*InOutOptionalKeys=*/nullptr, Context);
						OutFilterData = OutFilterParamData;
					}
				}
			}

			if (InFilterData)
			{
				FPCGTaggedData& InFilterOutput = Outputs.Add_GetRef(Input);
				InFilterOutput.Pin = PCGPinConstants::DefaultInFilterLabel;
				InFilterOutput.Data = InFilterData;
				InFilterOutput.Tags = Input.Tags;
			}

			if (OutFilterData)
			{
				FPCGTaggedData& OutFilterOutput = Outputs.Add_GetRef(Input);
				OutFilterOutput.Pin = PCGPinConstants::DefaultOutFilterLabel;
				OutFilterOutput.Data = OutFilterData;
				OutFilterOutput.Tags = Input.Tags;
			}
		}
		else
		{
			// Should be caught before when computing threshold info.
			PCGE_LOG(Warning, GraphAndLog, LOCTEXT("TypeCannotBeConverted", "Cannot convert threshold type to target type"));
			ForwardInputToInFilterPin();
		}
	}

	return true;
}

////////////////////////////////////////
// FPCGAttributeFilterElement
////////////////////////////////////////

bool FPCGAttributeFilterElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeFilterElement::Execute);

#if !WITH_EDITOR
	const bool bHasInFilterOutputPin = Context->Node && Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultInFilterLabel);
	const bool bHasOutsideFilterOutputPin = Context->Node && Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultOutFilterLabel);

	// Early out - only in non-editor builds, otherwise we will potentially poison the cache, since it is input-driven
	if (!bHasInFilterOutputPin && !bHasOutsideFilterOutputPin)
	{
		return true;
	}
#endif

	const UPCGAttributeFilteringSettings* Settings = Context->GetInputSettings<UPCGAttributeFilteringSettings>();
	check(Settings);

	FPCGAttributeFilterThresholdSettings ThresholdSettings{};
	ThresholdSettings.bUseConstantThreshold = Settings->bUseConstantThreshold;
	ThresholdSettings.bUseSpatialQuery = Settings->bUseSpatialQuery;
	ThresholdSettings.ThresholdAttribute = Settings->ThresholdAttribute;
	ThresholdSettings.AttributeTypes = Settings->AttributeTypes;
	ThresholdSettings.bFilterEachInputWithEachFilterDataRespectively = Settings->bFilterEachInputWithEachFilterDataRespectively;

	return DoFiltering(Context, Settings->Operator, Settings->TargetAttribute, Settings->bHasSpatialToPointDeprecation, Settings->bWarnOnDataMissingAttribute, Settings->bGenerateOutputDataEvenIfEmpty, ThresholdSettings);
}

////////////////////////////////////////
// FPCGAttributeFilterRangeElement
////////////////////////////////////////

bool FPCGAttributeFilterRangeElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeFilterRangeElement::Execute);

#if !WITH_EDITOR
	const bool bHasInFilterOutputPin = Context->Node && Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultInFilterLabel);
	const bool bHasOutsideFilterOutputPin = Context->Node && Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultOutFilterLabel);

	// Early out - only in non-editor builds, otherwise we will potentially poison the cache, since it is input-driven
	if (!bHasInFilterOutputPin && !bHasOutsideFilterOutputPin)
	{
		return true;
	}
#endif

	const UPCGAttributeFilteringRangeSettings* Settings = Context->GetInputSettings<UPCGAttributeFilteringRangeSettings>();
	check(Settings);

	return DoFiltering(Context, EPCGAttributeFilterOperator::InRange, Settings->TargetAttribute, Settings->bHasSpatialToPointDeprecation, Settings->bWarnOnDataMissingAttribute, Settings->bGenerateOutputDataEvenIfEmpty, Settings->MinThreshold, &Settings->MaxThreshold);
}

#undef LOCTEXT_NAMESPACE
