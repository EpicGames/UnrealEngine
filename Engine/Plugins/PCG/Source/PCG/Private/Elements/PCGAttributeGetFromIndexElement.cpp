// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAttributeGetFromIndexElement.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeGetFromIndexElement)

#define LOCTEXT_NAMESPACE "PCGAttributeGetFromIndexElement"

#if WITH_EDITOR
FName UPCGAttributeGetFromIndexSettings::GetDefaultNodeName() const
{
	return TEXT("GetAttributeFromIndex");
}

FText UPCGAttributeGetFromIndexSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Attribute Set from Index");
}

FText UPCGAttributeGetFromIndexSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Retrieves a single entry from an Attribute Set.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGAttributeGetFromIndexSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Param);
	InputPinProperty.SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGAttributeGetFromIndexSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FPCGElementPtr UPCGAttributeGetFromIndexSettings::CreateElement() const
{
	return MakeShared<FPCGAttributeGetFromIndexElement>();
}

bool FPCGAttributeGetFromIndexElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeGetFromIndexElement::Execute);

	check(Context);

	const UPCGAttributeGetFromIndexSettings* Settings = Context->GetInputSettings<UPCGAttributeGetFromIndexSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	const int32 Index = Settings->Index;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGParamData* Param = Cast<const UPCGParamData>(Input.Data);

		if (!Param || !Param->Metadata)
		{
			continue;
		}

		const UPCGMetadata* ParamMetadata = Param->Metadata;
		const int64 ParamItemCount = ParamMetadata->GetLocalItemCount();

		if (Index >= ParamItemCount)
		{
			PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidParamIndex", "Unable to retrieve entry {0} because there are {1} entries in the AttributeSet"), Index, ParamItemCount));
			continue;
		}

		UPCGParamData* SubParam = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		check(SubParam->Metadata);
		PCGMetadataHelpers::InitializeMetadataWithDataDomainCopyAndElementsNoCopy(Param, SubParam);

		const PCGMetadataEntryKey OriginalEntryKey = Index;
		PCGMetadataEntryKey SingleEntryKey = SubParam->Metadata->AddEntry();
		SubParam->Metadata->SetAttributes(OriginalEntryKey, ParamMetadata, SingleEntryKey);

		TArray<FName> AttributeNames;
		TArray<EPCGMetadataTypes> AttributeTypes;
		SubParam->Metadata->GetAttributes(AttributeNames, AttributeTypes);

		FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef(Input);
		TaggedData.Data = SubParam;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE