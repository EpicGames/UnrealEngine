// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAttributeGetFromPointIndexElement.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGData.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGExtractAttribute.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeGetFromPointIndexElement)

#define LOCTEXT_NAMESPACE "PCGAttributeGetFromPointIndexElement"

#if WITH_EDITOR
FName UPCGAttributeGetFromPointIndexSettings::GetDefaultNodeName() const
{
	return TEXT("GetAttributeFromPointIndex");
}

FText UPCGAttributeGetFromPointIndexSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Attribute From Point Index");
}

void UPCGAttributeGetFromPointIndexSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	if (DataVersion < FPCGCustomVersion::UpdateAttributePropertyInputSelector
		&& (OutputAttributeName.GetName() == NAME_None))
	{
		// Previous behavior of the output attribute for this node was:
		// None => SameName
		OutputAttributeName.SetAttributeName(PCGMetadataAttributeConstants::SourceNameAttributeName);
	}

	Super::ApplyDeprecation(InOutNode);
}
#endif

UPCGAttributeGetFromPointIndexSettings::UPCGAttributeGetFromPointIndexSettings()
{
	// Default was None, but for new objects it will be @Source
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		OutputAttributeName.SetAttributeName(PCGMetadataAttributeConstants::SourceAttributeName);
		bForceOutputAttributeToBeInElementsDomain = false;
	}
	else
	{
		OutputAttributeName.SetAttributeName(NAME_None);
	}
}

void UPCGAttributeGetFromPointIndexSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (InputAttributeName_DEPRECATED != NAME_None)
	{
		InputSource.SetAttributeName(InputAttributeName_DEPRECATED);
		InputAttributeName_DEPRECATED = NAME_None;
	}
#endif
}

TArray<FPCGPinProperties> UPCGAttributeGetFromPointIndexSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	InputPinProperty.SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGAttributeGetFromPointIndexSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGAttributeGetFromPointIndexConstants::OutputAttributeLabel, EPCGDataType::Param);
	PinProperties.Emplace(PCGAttributeGetFromPointIndexConstants::OutputPointLabel, EPCGDataType::Point);

	return PinProperties;
}

FPCGElementPtr UPCGAttributeGetFromPointIndexSettings::CreateElement() const
{
	return MakeShared<FPCGAttributeGetFromPointIndexElement>();
}

bool FPCGAttributeGetFromPointIndexElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeGetFromPointIndexElement::Execute);

	check(Context);

	const UPCGAttributeGetFromPointIndexSettings* Settings = Context->GetInputSettings<UPCGAttributeGetFromPointIndexSettings>();
	check(Settings);

	auto CreatePointData = [Context, Index = Settings->Index](const FPCGTaggedData& Input)
	{
#if !WITH_EDITOR
		// Add the point
		// Eschew output creation only in non-editor builds
		if (Context->Node && Context->Node->IsOutputPinConnected(PCGAttributeGetFromPointIndexConstants::OutputPointLabel))
#endif
		{
			const UPCGBasePointData* PointData = CastChecked<UPCGBasePointData>(Input.Data);
			UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(Context);

			FPCGInitializeFromDataParams InitializeFromDataParams(PointData);
			InitializeFromDataParams.bInheritSpatialData = false;
			OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);

			OutputPointData->SetNumPoints(1);
			OutputPointData->CopyUnallocatedPropertiesFrom(PointData);

			const FConstPCGPointValueRanges InRanges(PointData);
			FPCGPointValueRanges OutRanges(OutputPointData, /*bAllocate=*/false);

			OutRanges.SetFromValueRanges(0, InRanges, Index);

			FPCGTaggedData& OutputPoint = Context->OutputData.TaggedData.Add_GetRef(Input);
			OutputPoint.Data = OutputPointData;
			OutputPoint.Pin = PCGAttributeGetFromPointIndexConstants::OutputPointLabel;
		}
	};

	PCGExtractAttribute::FExtractAttributeParams Params =
	{
		.Context = Context,
		.InputSource = &Settings->InputSource,
		.Index = Settings->Index,
		.OutputAttributeName = &Settings->OutputAttributeName,
		.OptionalClassRequirement = TSubclassOf<UPCGData>(UPCGBasePointData::StaticClass()),
		.OutputLabel =  PCGAttributeGetFromPointIndexConstants::OutputAttributeLabel,
		.OnSuccessExtractionCallback = CreatePointData,
		.bForceOutputAttributeToBeInElementsDomain = Settings->bForceOutputAttributeToBeInElementsDomain
	};

	PCGExtractAttribute::ExtractAttribute(Params);

	return true;
}

#undef LOCTEXT_NAMESPACE
