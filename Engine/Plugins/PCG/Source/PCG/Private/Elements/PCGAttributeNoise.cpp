// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAttributeNoise.h"

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Math/RandomStream.h"
#include "PCGPoint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeNoise)

#define LOCTEXT_NAMESPACE "PCGAttributeNoiseSettings"

namespace PCGAttributeNoiseSettings
{
	template <typename T>
	void ProcessNoise(T& InOutValue, FRandomStream& InRandomSource, const UPCGAttributeNoiseSettings* InSettings, const bool bClampResult)
	{
		if constexpr (std::is_arithmetic_v<T> && !std::is_same_v<T, bool>)
		{
			const EPCGAttributeNoiseMode Mode = InSettings->Mode;
			const float NoiseMin = InSettings->NoiseMin;
			const float NoiseMax = InSettings->NoiseMax;
			const bool bInvertSource = InSettings->bInvertSource;

			const double Noise = InRandomSource.FRandRange(NoiseMin, NoiseMax);

			if (bInvertSource)
			{
				InOutValue = static_cast<T>(1.0 - InOutValue);
			}

			if (Mode == EPCGAttributeNoiseMode::Minimum)
			{
				InOutValue = FMath::Min<T>(InOutValue, Noise);
			}
			else if (Mode == EPCGAttributeNoiseMode::Maximum)
			{
				InOutValue = FMath::Max<T>(InOutValue, Noise);
			}
			else if (Mode == EPCGAttributeNoiseMode::Add)
			{
				InOutValue = static_cast<T>(InOutValue + Noise);
			}
			else if (Mode == EPCGAttributeNoiseMode::Multiply)
			{
				InOutValue = static_cast<T>(InOutValue * Noise);
			}
			else //if (Mode == EPCGAttributeNoiseMode::Set)
			{
				InOutValue = static_cast<T>(Noise);
			}

			if (bClampResult)
			{
				InOutValue = FMath::Clamp<T>(InOutValue, 0, 1);
			}
		}
		else if constexpr (std::is_same_v<FVector2D, T>)
		{
			ProcessNoise(InOutValue.X, InRandomSource, InSettings, bClampResult);
			ProcessNoise(InOutValue.Y, InRandomSource, InSettings, bClampResult);
		}
		else if constexpr (std::is_same_v<FVector, T>)
		{
			ProcessNoise(InOutValue.X, InRandomSource, InSettings, bClampResult);
			ProcessNoise(InOutValue.Y, InRandomSource, InSettings, bClampResult);
			ProcessNoise(InOutValue.Z, InRandomSource, InSettings, bClampResult);
		}
		else if constexpr (std::is_same_v<FVector4, T> || std::is_same_v<FQuat, T>)
		{
			ProcessNoise(InOutValue.X, InRandomSource, InSettings, bClampResult);
			ProcessNoise(InOutValue.Y, InRandomSource, InSettings, bClampResult);
			ProcessNoise(InOutValue.Z, InRandomSource, InSettings, bClampResult);
			ProcessNoise(InOutValue.W, InRandomSource, InSettings, bClampResult);
		}
		else if constexpr (std::is_same_v<FRotator, T>)
		{
			ProcessNoise(InOutValue.Roll, InRandomSource, InSettings, bClampResult);
			ProcessNoise(InOutValue.Pitch, InRandomSource, InSettings, bClampResult);
			ProcessNoise(InOutValue.Yaw, InRandomSource, InSettings, bClampResult);
		}
	}
}

UPCGAttributeNoiseSettings::UPCGAttributeNoiseSettings()
{
	InputSource.SetPointProperty(EPCGPointProperties::Density);
}

#if WITH_EDITOR
TArray<FPCGPreConfiguredSettingsInfo> UPCGAttributeNoiseSettings::GetPreconfiguredInfo() const
{
	TArray<FPCGPreConfiguredSettingsInfo> PreconfiguredInfo;
	PreconfiguredInfo.Emplace(0, GetDefaultNodeTitle());
	PreconfiguredInfo.Emplace(1, LOCTEXT("DensityNoiseNodeTitle", "Density Noise"));

	return PreconfiguredInfo;
}
#endif

void UPCGAttributeNoiseSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo)
{
	// If index is 1, it is the default ($Density)
	if (PreconfigureInfo.PreconfiguredIndex == 0)
	{
		InputSource.SetAttributeName(PCGMetadataAttributeConstants::LastAttributeName);
	}
}

void UPCGAttributeNoiseSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (DensityMode_DEPRECATED != EPCGAttributeNoiseMode::Set)
	{
		Mode = DensityMode_DEPRECATED;
		DensityMode_DEPRECATED = EPCGAttributeNoiseMode::Set;
	}

	if (DensityNoiseMin_DEPRECATED != 0.f)
	{
		NoiseMin = DensityNoiseMin_DEPRECATED;
		DensityNoiseMin_DEPRECATED = 0.f;
	}

	if (DensityNoiseMax_DEPRECATED != 1.f)
	{
		NoiseMax = DensityNoiseMax_DEPRECATED;
		DensityNoiseMax_DEPRECATED = 1.f;
	}

	if (bInvertSourceDensity_DEPRECATED)
	{
		bInvertSource = bInvertSourceDensity_DEPRECATED;
		bInvertSourceDensity_DEPRECATED = false;
	}

	// Check for the data spatial to point gate version
	if (DataVersion < FPCGCustomVersion::NoMoreSpatialDataConversionToPointDataByDefaultOnNonPointPins)
	{
		bHasSpatialToPointDeprecation = true;
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UPCGAttributeNoiseSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	// Overridable properties have been renamed, rename all pins by their counterpart, to avoid breaking existing graphs.
	const TArray<TPair<FName, FName>> OldToNewPinNames =
	{
		{TEXT("Density Mode"), TEXT("Mode")},
		{TEXT("Density Noise Min"), TEXT("Noise Min")},
		{TEXT("Density Noise Max"),TEXT("Noise Max")},
		{TEXT("Invert Source Density"), TEXT("Invert Source")}
	};

	for (const TPair<FName, FName>& OldToNew : OldToNewPinNames)
	{
		InOutNode->RenameInputPin(OldToNew.Key, OldToNew.Value);
	}

	// Param | Point type was not explicitly defined in the data types, and therefore was not serialized correctly, resulting in an Input/Output pin serialized to None.
	auto FixInvalidAllowedTypes = [](UPCGPin* InPin)
	{
		if (InPin && InPin->Properties.AllowedTypes == EPCGDataType::None)
		{
			InPin->Properties.AllowedTypes = EPCGDataType::PointOrParam;
		}
	};

	FixInvalidAllowedTypes(InOutNode->GetInputPin(PCGPinConstants::DefaultInputLabel));
	FixInvalidAllowedTypes(InOutNode->GetOutputPin(PCGPinConstants::DefaultOutputLabel));
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGAttributeNoiseSettings::CreateElement() const
{
	return MakeShared<FPCGAttributeNoiseElement>();
}

TArray<FPCGPinProperties> UPCGAttributeNoiseSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any);
	InputPinProperty.SetRequiredPin();
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGAttributeNoiseSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

bool FPCGAttributeNoiseElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeNoiseElement::Execute);

	FPCGAttributeNoiseContext* Context = static_cast<FPCGAttributeNoiseContext*>(InContext);
	check(Context);

	const UPCGAttributeNoiseSettings* Settings = Context->GetInputSettings<UPCGAttributeNoiseSettings>();
	check(Settings);

	const TArray<FPCGTaggedData>& Inputs = Context->CachedInputs;
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	// Precompute a seed based on the settings one and the component one
	const int Seed = Context->GetSeed();

	if (!Context->bInputsGathered)
	{
		Context->CachedInputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
		Context->bInputsGathered = true;
	}

	while (Context->CurrentInput < Inputs.Num())
	{
		int32 CurrentInput = Context->CurrentInput;
		const FPCGTaggedData& Input = Inputs[CurrentInput];

		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeNoiseElement::InputLoop);

		if (!Context->bDataPreparedForCurrentInput)
		{
			const UPCGData* InputData = Input.Data;
			if (!InputData || !InputData->ConstMetadata())
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InputUnsuportedData", "Data {0} is neither spatial nor an attribute set, unsupported."), Context->CurrentInput));
				Context->CurrentInput++;
				continue;
			}

			// For deprecation
			if (const UPCGSpatialData* InputSpatialData = Cast<const UPCGSpatialData>(InputData))
			{
				if (Settings->bHasSpatialToPointDeprecation)
				{
					InputData = InputSpatialData->ToBasePointData(Context);
				}
			}

			Context->InputSource = Settings->InputSource.CopyAndFixLast(InputData);

			Context->InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, Context->InputSource);
			if (!Context->InputAccessor)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Context->InputSource, Context);
				Context->CurrentInput++;
				continue;
			}

			// Also need to make sure the accessor is a "noisable" type
			if (!PCG::Private::IsOfTypes<int32, int64, float, double, FVector, FVector2D, FVector4, FRotator, FQuat>(Context->InputAccessor->GetUnderlyingType()))
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("AttributeIsNotANumericalType", "Attribute/Property '{0}' is not a numerical type, we can't apply noise to it."), Context->InputSource.GetDisplayText()));
				Context->CurrentInput++;
				continue;
			}

			// Same verification for the seed. If we have no custom seed but points, use the $Seed accessor.
			// Otherwise, it will just use the index of the element as the seed (combined with component + settings seed of course)
			const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(Input.Data);
			if (Settings->bHasCustomSeedSource || InputPointData)
			{
				if (Settings->bHasCustomSeedSource)
				{
					Context->CustomSeedSource = Settings->CustomSeedSource.CopyAndFixLast(InputData);
				}
				else
				{
					Context->CustomSeedSource.SetPointProperty(EPCGPointProperties::Seed);
				}
				
				Context->CustomSeedAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, Context->CustomSeedSource);
				if (!Context->CustomSeedAccessor)
				{
					PCGLog::Metadata::LogFailToCreateAccessorError(Context->CustomSeedSource, Context);
					Context->CurrentInput++;
					continue;
				}
				
				if (!PCG::Private::IsBroadcastableOrConstructible(Context->CustomSeedAccessor->GetUnderlyingType(), PCG::Private::MetadataTypes<int32>::Id))
				{
					PCGLog::Metadata::LogFailToGetAttributeError<int32>(Context->CustomSeedSource, Context->CustomSeedAccessor.Get(), Context);
					Context->CurrentInput++;
					continue;
				}

				Context->CustomSeedKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, Context->CustomSeedSource);
				check(Context->CustomSeedKeys);
			}

			FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
			UPCGData* OutputData = InputData->DuplicateData(Context);
			Output.Data = OutputData;

			// Then create the keys
			Context->InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, Context->InputSource);

			// It won't fail because we validated on the input data and output is initialized from input.
			check(Context->InputAccessor && Context->InputKeys);

			Context->OutputTarget = Settings->OutputTarget.CopyAndFixSource(&Context->InputSource, Input.Data);

			Context->OutputAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputData, Context->OutputTarget);
			if (!Context->OutputAccessor && Context->OutputTarget.IsBasicAttribute())
			{
				PCGMetadataAttribute::CallbackWithRightType(Context->InputAccessor->GetUnderlyingType(), [Context, OutputData](auto&& Dummy)
				{
					using AttributeType = std::decay_t<decltype(Dummy)>;

					OutputData->MutableMetadata()->CreateAttribute<AttributeType>(Context->OutputTarget.GetName(), AttributeType{}, /*bAllowsInterpolation=*/ true, /*bOverrideParent=*/false);
					Context->OutputAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputData, Context->OutputTarget);
				});
			}

			Context->OutputKeys = PCGAttributeAccessorHelpers::CreateKeys(OutputData, Context->OutputTarget);

			if (!Context->OutputAccessor || !Context->OutputKeys)
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("OutputTargetInvalid", "Failed to find/create Attribute/Property '{0}'."), Context->OutputTarget.GetDisplayText()));
				Outputs.RemoveAt(Outputs.Num() - 1);
				Context->CurrentInput++;
				continue;
			}

			if (!PCG::Private::IsBroadcastableOrConstructible(Context->InputAccessor->GetUnderlyingType(), Context->OutputAccessor->GetUnderlyingType()))
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("CantBroadcast", "Cannot convert Attribute '{0}' ({1}) into Attribute '{2}' ({3})."),
					Context->InputSource.GetDisplayText(),
					PCG::Private::GetTypeNameText(Context->InputAccessor->GetUnderlyingType()),
					Context->OutputTarget.GetDisplayText(),
					PCG::Private::GetTypeNameText(Context->OutputAccessor->GetUnderlyingType())));
				
				Outputs.RemoveAt(Outputs.Num() - 1);
				Context->CurrentInput++;
				continue;
			}

			Context->bDataPreparedForCurrentInput = true;
		}

		// Force clamp on Density
		const bool bClampResult = Settings->bClampResult || (Context->OutputTarget.GetSelection() == EPCGAttributePropertySelection::Property && Context->OutputTarget.GetPointProperty() == EPCGPointProperties::Density);

		const bool bDone = PCGMetadataAttribute::CallbackWithRightType(Context->InputAccessor->GetUnderlyingType(), [Settings, Context, Seed, bClampResult](auto&& Dummy) -> bool
		{
			using AttributeType = std::decay_t<decltype(Dummy)>;
			constexpr int32 ChunkSize = 64;
			const int32 NumIterations = Context->InputKeys->GetNum();

			// No init
			auto Initialize = []() {};
			// It's a 1 for 1 operation, should never move
			auto MoveDataRange = [](int32, int32, int32) { ensure(false); };
			// It's finished if we processed all elements.
			auto Finished = [NumIterations](int32 Count) { ensure(NumIterations == Count); };

			return FPCGAsync::AsyncProcessingRangeEx(&Context->AsyncState, NumIterations, Initialize, [Settings, Seed, bClampResult, Context, ChunkSize, NumIterations](int32 StartReadIndex, int32 StartWriteIndex, int32 Count) -> int32
			{
				TArray<AttributeType, TInlineAllocator<ChunkSize>> Values;
				Values.SetNumUninitialized(Count);

				TArray<int32, TInlineAllocator<ChunkSize>> Seeds;
				bool bHasCustomSeed = false;
				if (Context->CustomSeedAccessor.IsValid())
				{
					bHasCustomSeed = true;
					Seeds.SetNumUninitialized(Count);
					if (!ensure(Context->CustomSeedAccessor->GetRange<int32>(Seeds, StartReadIndex, *Context->CustomSeedKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible)))
					{
						return 0;
					}
				}

				if (Context->InputAccessor->GetRange<AttributeType>(Values, StartReadIndex, *Context->InputKeys))
				{
					for (int32 i = 0; i < Count; ++i)
					{
						// Use the point seed if we have points, otherwise the index. Don't start at 0 (that's why there is a +1)
						// Warning: It makes it order independant for points, but order dependant for the rest.
						const int32 ElementSeed = bHasCustomSeed ? Seeds[i] : StartReadIndex + i + 1;
						FRandomStream RandomSource(PCGHelpers::ComputeSeed(Seed, ElementSeed));
						PCGAttributeNoiseSettings::ProcessNoise(Values[i], RandomSource, Settings, bClampResult);
					}

					Context->OutputAccessor->SetRange<AttributeType>(Values, StartWriteIndex, *Context->OutputKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
				}

				return Count;
			}, MoveDataRange, Finished, /*bEnableTimeSlicing=*/true, ChunkSize);
		});

		if (bDone)
		{
			Context->CurrentInput++;
			Context->bDataPreparedForCurrentInput = false;
		}
		
		if (!bDone || Context->ShouldStop())
		{
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
