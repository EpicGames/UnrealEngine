// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAttributeReduceElement.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGSpatialData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeReduceElement)

#define LOCTEXT_NAMESPACE "PCGAttributeReduceElement"

namespace PCGAttributeReduceElement
{
	template <typename T>
	bool Average(const IPCGAttributeAccessorKeys& Keys, const IPCGAttributeAccessor& Accessor, T& OutValue)
	{
		if constexpr (!PCG::Private::MetadataTraits<T>::CanInterpolate)
		{
			return false;
		}
		else
		{
			const double Weight = 1.0f / Keys.GetNum();

			// If we need normalization or can't sub/add, we need to do the weighted sum sequentially.
			// Otherwise, we can just add all the values together and multiply the result by the weight.
			if constexpr (PCG::Private::MetadataTraits<T>::InterpolationNeedsNormalization || !PCG::Private::MetadataTraits<T>::CanSubAdd)
			{
				OutValue = PCG::Private::MetadataTraits<T>::ZeroValueForWeightedSum();

				bool bSuccess = PCGMetadataElementCommon::ApplyOnAccessor<T>(Keys, Accessor, [Weight, &OutValue](const T& InValue, int32 Index)
				{
					OutValue = PCG::Private::MetadataTraits<T>::WeightedSum(OutValue, InValue, Weight);
				});

				if (bSuccess)
				{
					if constexpr (PCG::Private::MetadataTraits<T>::InterpolationNeedsNormalization)
					{
						static_assert(PCG::Private::MetadataTraits<T>::CanNormalize);

						// We need to normalize the resulting value
						PCG::Private::MetadataTraits<T>::Normalize(OutValue);
					}
				}

				return bSuccess;
			}
			else
			{
				OutValue = PCG::Private::MetadataTraits<T>::ZeroValue();

				bool bSuccess = PCGMetadataElementCommon::ApplyOnAccessor<T>(Keys, Accessor, [&OutValue](const T& InValue, int32)
				{
					OutValue = PCG::Private::MetadataTraits<T>::Add(OutValue, InValue);
				});

				if (bSuccess)
				{
					OutValue = PCG::Private::MetadataTraits<T>::WeightedSum(PCG::Private::MetadataTraits<T>::ZeroValue(), OutValue, Weight);
				}

				return bSuccess;
			}
		}
	}

	template <typename T, bool bIsMin>
	bool MinMax(const IPCGAttributeAccessorKeys& Keys, const IPCGAttributeAccessor& Accessor, T& OutValue)
	{
		if constexpr (!PCG::Private::MetadataTraits<T>::CanMinMax)
		{
			return false;
		}
		else
		{
			OutValue = PCG::Private::MetadataTraits<T>::ZeroValue();
			bool bFirstValue = true;

			return PCGMetadataElementCommon::ApplyOnAccessor<T>(Keys, Accessor, [&OutValue, &bFirstValue](const T& InValue, int32)
				{
					if (bFirstValue)
					{
						OutValue = InValue;
						bFirstValue = false;
					}
					else
					{
						if constexpr (bIsMin)
						{
							OutValue = PCG::Private::MetadataTraits<T>::Min(OutValue, InValue);
						}
						else
						{
							OutValue = PCG::Private::MetadataTraits<T>::Max(OutValue, InValue);
						}
					}
				});
		}
	}

	template<typename T>
	bool Sum(const IPCGAttributeAccessorKeys& Keys, const IPCGAttributeAccessor& Accessor, T& OutValue)
	{
		if constexpr (!PCG::Private::MetadataTraits<T>::CanSubAdd)
		{
			return false;
		}
		else
		{
			OutValue = PCG::Private::MetadataTraits<T>::ZeroValue();

			return PCGMetadataElementCommon::ApplyOnAccessor<T>(Keys, Accessor, [&OutValue](const T& InValue, int32)
			{
				OutValue = PCG::Private::MetadataTraits<T>::Add(OutValue, InValue);
			});
		}
	}

	template<typename T, typename OT>
	bool Join(const IPCGAttributeAccessorKeys& Keys, const IPCGAttributeAccessor& Accessor, const FString& Delimiter, OT& OutValue)
	{
		if constexpr (std::is_constructible_v<OT, FString>)
		{
			TArray<FString> StringsToJoin;
			StringsToJoin.Reserve(Keys.GetNum());

			bool bAppliedAccessor = PCGMetadataElementCommon::ApplyOnAccessor<T>(Keys, Accessor, [&StringsToJoin](const T& InValue, int32)
			{
				StringsToJoin.Add(PCG::Private::MetadataTraits<T>::ToString(InValue));
			});

			if (bAppliedAccessor)
			{
				OutValue = OT(FString::Join(StringsToJoin, *Delimiter));
				return true;
			}
			else
			{
				return false;
			}
		}
		else
		{
			return false;
		}
	}
}

#if WITH_EDITOR
FName UPCGAttributeReduceSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeReduce");
}

FText UPCGAttributeReduceSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Attribute Reduce");
}

void UPCGAttributeReduceSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	if (DataVersion < FPCGCustomVersion::UpdateAttributePropertyInputSelector
		&& (OutputAttributeName == NAME_None))
	{
		// Previous behavior of the output attribute for this node was:
		// None => SameName
		OutputAttributeName = PCGMetadataAttributeConstants::SourceNameAttributeName;
	}

	Super::ApplyDeprecation(InOutNode);
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGAttributeReduceSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGAttributeReduceOperation>();
}
#endif

bool UPCGAttributeReduceSettings::HasDynamicPins() const
{
	return bWriteToDataDomain;
}

void UPCGAttributeReduceSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGAttributeReduceOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGAttributeReduceOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

void UPCGAttributeReduceSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (InputAttributeName_DEPRECATED != NAME_None)
	{
		InputSource.SetAttributeName(InputAttributeName_DEPRECATED);
		InputAttributeName_DEPRECATED = NAME_None;
	}
#endif
}

FString UPCGAttributeReduceSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGAttributeReduceOperation>())
	{
		const FText OperationName = EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation));

		FName InputAttributeName = InputSource.GetName();
		if (InputAttributeName == NAME_None)
		{
			InputAttributeName = FName(TEXT("LastAttribute"));
		}

		if (InputAttributeName != OutputAttributeName && OutputAttributeName != NAME_None)
		{
			return FText::Format(LOCTEXT("ReduceInputToOutputWithOperation", "Reduce {0} to {1}: {2}"), FText::FromName(InputAttributeName), FText::FromName(OutputAttributeName), OperationName).ToString();
		}
		else
		{
			return FText::Format(LOCTEXT("ReduceInplaceWithOperation", "Reduce {0}: {1}"), FText::FromName(InputAttributeName), OperationName).ToString();
		}
	}
	else
	{
		return FString();
	}
}

TArray<FPCGPinProperties> UPCGAttributeReduceSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any);
	InputPin.SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGAttributeReduceSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, bWriteToDataDomain ? EPCGDataType::Any : EPCGDataType::Param);

	return PinProperties;
}

FPCGElementPtr UPCGAttributeReduceSettings::CreateElement() const
{
	return MakeShared<FPCGAttributeReduceElement>();
}

bool FPCGAttributeReduceElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeReduceElement::Execute);

	check(Context);

	const UPCGAttributeReduceSettings* Settings = Context->GetInputSettings<UPCGAttributeReduceSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	UPCGData* OutputData = nullptr;
	FPCGMetadataAttributeBase* NewAttribute = nullptr;

	for (int32 i = 0; i < Inputs.Num(); ++i)
	{
		const UPCGData* InputData = Inputs[i].Data;

		if (!InputData || !InputData->ConstMetadata())
		{
			PCGLog::InputOutput::LogInvalidInputDataError(Context);
			continue;
		}

		if (Settings->bWriteToDataDomain && !InputData->IsSupportedMetadataDomainID(PCGMetadataDomainID::Data))
		{
			static const FPCGAttributePropertySelector DataSelector = FPCGAttributePropertySelector::CreateAttributeSelector(NAME_None, PCGDataConstants::DataDomainName);
			PCGLog::Metadata::LogInvalidMetadataDomain(DataSelector, Context);
			continue;
		}

		const FPCGAttributePropertyInputSelector InputSource = Settings->InputSource.CopyAndFixLast(InputData);
		const FName OutputAttributeName = (Settings->OutputAttributeName == PCGMetadataAttributeConstants::SourceNameAttributeName) ? InputSource.GetName() : Settings->OutputAttributeName;

		TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, InputSource);
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, InputSource);

		if (!Accessor.IsValid() || !Keys.IsValid())
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(InputSource, Context);
			continue;
		}

		const bool bMergeOutputAttributes = Settings->ShouldMergeOutputAttributes();

		auto DoOperation = [&Accessor, &Keys, Settings, &OutputData, &NewAttribute, OutputAttributeName, &Context, InputData, bMergeOutputAttributes](auto DummyValue) -> bool
		{
			using AttributeType = decltype(DummyValue);

			const EPCGAttributeReduceOperation Operation = Settings->Operation;

			bool bCreatedNewData = false;
			if (!OutputData || !bMergeOutputAttributes)
			{
				if (Settings->bWriteToDataDomain)
				{
					OutputData = InputData->DuplicateData(Context);
				}
				else
				{
					OutputData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
				}

				bCreatedNewData = true;
			}

			auto DoOperationInternal = [&OutputData, &Accessor, &Keys, Settings, bCreatedNewData, &NewAttribute, OutputAttributeName, bMergeOutputAttributes](auto DummyOutputValue) -> bool
			{
				using OutAttributeType = decltype(DummyOutputValue);
				const EPCGAttributeReduceOperation Operation = Settings->Operation;
				const FString& JoinDelimiter = Settings->JoinDelimiter;

				OutAttributeType OutputValue = PCG::Private::MetadataTraits<OutAttributeType>::ZeroValue();
				FPCGMetadataDomain* OutputDomain = Settings->bWriteToDataDomain ? OutputData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data) : OutputData->MutableMetadata()->GetDefaultMetadataDomain();
				check(OutputDomain);

				if (bCreatedNewData)
				{
					if (OutputDomain->HasAttribute(OutputAttributeName))
					{
						OutputDomain->DeleteAttribute(OutputAttributeName);
					}

					NewAttribute = OutputDomain->CreateAttribute<OutAttributeType>(OutputAttributeName, OutputValue, /*bAllowInterpolation=*/ true, /*bOverrideParent=*/false);

					if (!NewAttribute)
					{
						OutputData = nullptr;
						return false;
					}
				}

				bool bSuccess = false;

				if constexpr (std::is_same_v<AttributeType, OutAttributeType>)
				{
					switch (Operation)
					{
					case EPCGAttributeReduceOperation::Average:
						bSuccess = PCGAttributeReduceElement::Average<AttributeType>(*Keys, *Accessor, OutputValue);
						break;
					case EPCGAttributeReduceOperation::Max:
						bSuccess = PCGAttributeReduceElement::MinMax<AttributeType, /*bIsMin*/false>(*Keys, *Accessor, OutputValue);
						break;
					case EPCGAttributeReduceOperation::Min:
						bSuccess = PCGAttributeReduceElement::MinMax<AttributeType, /*bIsMin*/true>(*Keys, *Accessor, OutputValue);
						break;
					case EPCGAttributeReduceOperation::Sum:
						bSuccess = PCGAttributeReduceElement::Sum<AttributeType>(*Keys, *Accessor, OutputValue);
						break;
					case EPCGAttributeReduceOperation::Join:
						bSuccess = PCGAttributeReduceElement::Join<AttributeType>(*Keys, *Accessor, JoinDelimiter, OutputValue);
					default:
						break;
					}
				}
				else
				{
					switch (Operation)
					{
					case EPCGAttributeReduceOperation::Join:
						bSuccess = PCGAttributeReduceElement::Join<AttributeType>(*Keys, *Accessor, JoinDelimiter, OutputValue);
					default:
						break;
					}
				}

				if (bSuccess)
				{
					FPCGMetadataAttribute<OutAttributeType>* TypedNewAttribute = static_cast<FPCGMetadataAttribute<OutAttributeType>*>(NewAttribute);
					check(TypedNewAttribute);

					PCGMetadataEntryKey EntryKey = PCGFirstEntryKey;
					if (!Settings->bWriteToDataDomain || OutputDomain->GetItemCountForChild() == 0)
					{
						EntryKey = OutputDomain->AddEntry();
					}

					// Implementation note: since the default value does not match the value computed here
					// and because we might have multiple entries, we need to set it in the attribute
					TypedNewAttribute->SetValue(EntryKey, OutputValue);
				}

				return bSuccess;
			};

			if (Operation == EPCGAttributeReduceOperation::Join)
			{
				return DoOperationInternal(FString{});
			}
			else
			{
				return DoOperationInternal(AttributeType{});
			}
		};

		if (!PCGMetadataAttribute::CallbackWithRightType(Accessor->GetUnderlyingType(), DoOperation))
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("AttributeOperationFailed", "Operation was not compatible with the attribute type {0} or could not create attribute '{1}' for input {2}"), PCG::Private::GetTypeNameText(Accessor->GetUnderlyingType()), FText::FromName(OutputAttributeName), FText::AsNumber(i)));
			continue;
		}

		if (ensure(OutputData) && (Outputs.IsEmpty() || !bMergeOutputAttributes))
		{
			FPCGTaggedData& Output = Outputs.Add_GetRef(Inputs[i]);
			Output.Data = OutputData;
		}
	}

	return true;
}

EPCGElementExecutionLoopMode FPCGAttributeReduceElement::ExecutionLoopMode(const UPCGSettings* InSettings) const
{
	const UPCGAttributeReduceSettings* Settings = Cast<UPCGAttributeReduceSettings>(InSettings);
	const bool bMergeOutputAttributes = Settings && Settings->ShouldMergeOutputAttributes();
	return !bMergeOutputAttributes ? EPCGElementExecutionLoopMode::SinglePrimaryPin : EPCGElementExecutionLoopMode::NotALoop;
}

#undef LOCTEXT_NAMESPACE
