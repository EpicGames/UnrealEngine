// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGAttributeRemap.h"

#include "PCGContext.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeRemap)

#define LOCTEXT_NAMESPACE "PCGAttributeRemapElement"

namespace PCGAttributeRemapElement
{
	constexpr int32 DensityRemapPreconfiguredIndex = 0;
	constexpr int32 CurveRemapPreconfiguredIndex = 1;
	
	struct FPCGAttributeRemapParams
	{
		EPCGAttributeRemapMode Mode = EPCGAttributeRemapMode::Ranges;

		double InRangeStart; // Note: Start and End can be a positive or negative range (e.g. [0, 1] or [1, 0])
		double InRangeEnd;
		double InRangeMin; // Note: Min and Max are a true min/max used to reject samples that fall outside the range
		double InRangeMax;
		double Slope;
		double Intercept;
		bool bClampToUnitRange;
		bool bIgnoreValuesOutsideInputRange;
		
		const FRichCurve* Curve = nullptr;
	};

	// Fallback for all unsupported types, does nothing.
	template <typename T>
	void Remap(T& InOutValue, const FPCGAttributeRemapParams& InParams) {}

	// Int/Float version, utilizing the concepts.
	template <typename T> requires (std::is_arithmetic_v<T> && !std::is_same_v<T, bool>)
	void Remap(T& InOutValue, const FPCGAttributeRemapParams& InParams)
	{
		if (InParams.Mode == EPCGAttributeRemapMode::Ranges)
		{
			if (!InParams.bIgnoreValuesOutsideInputRange || (InOutValue >= InParams.InRangeMin && InOutValue <= InParams.InRangeMax))
			{
				InOutValue = InParams.Slope * (InOutValue - InParams.InRangeStart) + InParams.Intercept;
				if (InParams.bClampToUnitRange)
				{
					InOutValue = FMath::Clamp<T>(InOutValue, T(0.0), T(1.0));
				}
			}
		}
		else if (InParams.Mode == EPCGAttributeRemapMode::Curve && ensure(InParams.Curve))
		{
			InOutValue = static_cast<T>(InParams.Curve->Eval(static_cast<float>(InOutValue)));
		}
	}

	template <>
	void Remap<FVector2D>(FVector2D& InOutValue, const FPCGAttributeRemapParams& InParams)
	{
		Remap(InOutValue.X, InParams);
		Remap(InOutValue.Y, InParams);
	}

	template <>
	void Remap<FVector>(FVector& InOutValue, const FPCGAttributeRemapParams& InParams)
	{
		Remap(InOutValue.X, InParams);
		Remap(InOutValue.Y, InParams);
		Remap(InOutValue.Z, InParams);
	}

	template <>
	void Remap<FVector4>(FVector4& InOutValue, const FPCGAttributeRemapParams& InParams)
	{
		Remap(InOutValue.X, InParams);
		Remap(InOutValue.Y, InParams);
		Remap(InOutValue.Z, InParams);
		Remap(InOutValue.W, InParams);
	}

	template <>
	void Remap<FRotator>(FRotator& InOutValue, const FPCGAttributeRemapParams& InParams)
	{
		Remap(InOutValue.Roll, InParams);
		Remap(InOutValue.Pitch, InParams);
		Remap(InOutValue.Yaw, InParams);
	}
}

UPCGAttributeRemapSettings::UPCGAttributeRemapSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		bAllowInverseRange = true;
	}
}

#if WITH_EDITOR
FName UPCGAttributeRemapSettings::GetDefaultNodeName() const
{
	return FName(TEXT("AttributeRemap"));
}

FText UPCGAttributeRemapSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Attribute Remap");
}

FText UPCGAttributeRemapSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Remap an attribute using either a range or a curve.");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGAttributeRemapSettings::GetPreconfiguredInfo() const
{
	return {
		FPCGPreConfiguredSettingsInfo{PCGAttributeRemapElement::DensityRemapPreconfiguredIndex, LOCTEXT("DensityNodeTitle", "Density Remap")},
		FPCGPreConfiguredSettingsInfo{PCGAttributeRemapElement::CurveRemapPreconfiguredIndex, LOCTEXT("CurveNodeTitle", "Attribute Curve Remap")},
	};
}
#endif // WITH_EDITOR

FString UPCGAttributeRemapSettings::GetAdditionalTitleInformation() const
{
	TStringBuilder<256> Builder;
	Builder += (Mode == EPCGAttributeRemapMode::Curve ? LOCTEXT("CurveSubtitle", "Curve") : LOCTEXT("RangesSubtitle", "Ranges")).ToString();

	if (!IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGAttributeRemapSettings, InputSource)) && !IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGAttributeRemapSettings, OutputTarget)))
	{
		if (OutputTarget.GetAttributeName() != PCGMetadataAttributeConstants::SourceAttributeName)
		{
			Builder += FString::Printf(TEXT(" %s -> %s"), *InputSource.ToString(), *OutputTarget.ToString());
		}
		else
		{
			Builder += TEXT(" ");
			Builder += InputSource.ToString();
		}
	}
	
	return Builder.ToString();
}

void UPCGAttributeRemapSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo)
{
	switch (PreconfigureInfo.PreconfiguredIndex)
	{
	case PCGAttributeRemapElement::DensityRemapPreconfiguredIndex:
		InputSource.SetPointProperty(EPCGPointProperties::Density);
		bClampToUnitRange = true;
		break;
	case PCGAttributeRemapElement::CurveRemapPreconfiguredIndex:
		Mode = EPCGAttributeRemapMode::Curve;
		break;
	default:
		break;
	}
}

FPCGAttributePropertyInputSelector UPCGAttributeRemapSettings::GetInputSource(uint32 Index) const
{
	return Index == 0 ? InputSource : FPCGAttributePropertyInputSelector{};
}

bool UPCGAttributeRemapSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	bHasSpecialRequirement = false;
	return PCG::Private::IsOfTypes<int32, int64, float, double, FVector2D, FVector, FVector4, FRotator, FQuat>(TypeId);
}

FPCGElementPtr UPCGAttributeRemapSettings::CreateElement() const
{
	return MakeShared<FPCGAttributeRemapElement>();
}

bool FPCGAttributeRemapElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeRemapElement::Execute);

	const UPCGAttributeRemapSettings* Settings = CastChecked<UPCGAttributeRemapSettings>(OperationData.Settings);

	PCGAttributeRemapElement::FPCGAttributeRemapParams Params{};
	Params.Mode = Settings->Mode;

	if (Params.Mode == EPCGAttributeRemapMode::Ranges)
	{
		Params.bClampToUnitRange = Settings->bClampToUnitRange;
		Params.bIgnoreValuesOutsideInputRange = Settings->bIgnoreValuesOutsideInputRange;

		Params.InRangeMin = FMath::Min(Settings->InRangeMin, Settings->InRangeMax);
		Params.InRangeMax = FMath::Max(Settings->InRangeMin, Settings->InRangeMax);

		double OutRangeMin = 0.0;
		double OutRangeMax = 0.0;

		if (Settings->bAllowInverseRange)
		{
			Params.InRangeStart = Settings->InRangeMin;
			Params.InRangeEnd = Settings->InRangeMax;
			OutRangeMin = Settings->OutRangeMin;
			OutRangeMax = Settings->OutRangeMax;
		}
		else
		{
			Params.InRangeStart = Params.InRangeMin;
			Params.InRangeEnd = Params.InRangeMax;
			OutRangeMin = FMath::Min(Settings->OutRangeMin, Settings->OutRangeMax);
			OutRangeMax = FMath::Max(Settings->OutRangeMin, Settings->OutRangeMax);
		}

		const double InRangeDifference = Params.InRangeEnd - Params.InRangeStart;
		const double OutRangeDifference = OutRangeMax - OutRangeMin;

		// When InRange is a point leave the Slope at 0 so that Density = Intercept
		if (InRangeDifference == 0)
		{
			Params.Slope = 0;
			Params.Intercept = (OutRangeMin + OutRangeMax) / 2.f;
		}
		else
		{
			Params.Slope = OutRangeDifference / InRangeDifference;
			Params.Intercept = OutRangeMin;
		}
	}
	else if (Params.Mode == EPCGAttributeRemapMode::Curve)
	{
		Params.Curve = Settings->Curve.GetRichCurveConst();
		if (!Params.Curve)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidCurve", "Curve is invalid"), OperationData.Context);
			return true;
		}
	}
	else
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("InvalidMode", "Invalid mode"), OperationData.Context);
		return true;
	}

	auto RemapFunc = [this, &Params, &OperationData]<typename AttributeType>(AttributeType) -> bool
	{
		using FinalType = std::conditional_t<std::is_same_v<AttributeType, FQuat>, FRotator, AttributeType>;
		return DoUnaryOp<FinalType>(OperationData, [&Params](const FinalType& Value) -> FinalType 
		{ 
			FinalType Out = Value;
			PCGAttributeRemapElement::Remap<FinalType>(Out, Params); 
			return Out; 
		});
	};

	return PCGMetadataAttribute::CallbackWithRightType(OperationData.MostComplexInputType, RemapFunc);
}

#undef LOCTEXT_NAMESPACE
