// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAttributeRemoveDuplicates.h"

#include "PCGContext.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadataPartitionCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeRemoveDuplicates)

#define LOCTEXT_NAMESPACE "PCGAttributeRemoveDuplicatesElement"

TArray<FPCGPinProperties> UPCGAttributeRemoveDuplicatesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGAttributeRemoveDuplicatesSettings::CreateElement() const
{
	return MakeShared<FPCGAttributeRemoveDuplicatesElement>();
}

FString UPCGAttributeRemoveDuplicatesSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGAttributeRemoveDuplicatesSettings, AttributeNamesToRemoveDuplicates)))
	{
		return FString();
	}
	else
#endif
	{
		if (AttributeSelectors.IsEmpty())
		{
			return FString();
		}

		FString OutString = AttributeSelectors[0].GetDisplayText().ToString();
		for (int i = 1; i < AttributeSelectors.Num(); ++i)
		{
			OutString += ", " + AttributeSelectors[i].GetDisplayText().ToString();
		}

		return OutString;
	}
}

bool FPCGAttributeRemoveDuplicatesElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataPartitionElement::Execute);
	check(Context);

	const UPCGAttributeRemoveDuplicatesSettings* Settings = Context->GetInputSettings<UPCGAttributeRemoveDuplicatesSettings>();
	check(Settings);

	// TODO: This is a temporary solution for overrides until arrays are supported
	TArray<FPCGAttributePropertyInputSelector> OverriddenSelectors;

	const TArray<FString> AttributeNames = PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->AttributeNamesToRemoveDuplicates);

	// If the names are overridden by the user, generate the selectors with them
	OverriddenSelectors.Reserve(AttributeNames.Num());
	for (const FString& AttributeName : AttributeNames)
	{
		OverriddenSelectors.Emplace_GetRef().Update(AttributeName);
	}

	const TArray<FPCGAttributePropertyInputSelector>& ActiveSelectors = OverriddenSelectors.IsEmpty() ? Settings->AttributeSelectors : OverriddenSelectors;

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	TArray<FPCGAttributePropertySelector> PartitionAttributeSources;
	PartitionAttributeSources.SetNum(ActiveSelectors.Num());

	if (PartitionAttributeSources.IsEmpty())
	{
		Outputs = Inputs;
		return true;
	}

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGData* InData = Input.Data;

		for (int32 i = 0; i < PartitionAttributeSources.Num(); ++i)
		{
			PartitionAttributeSources[i] = static_cast<FPCGAttributePropertySelector>(ActiveSelectors[i].CopyAndFixLast(InData));
		}

		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
		if (UPCGData* DataWithoutDuplicates = PCGMetadataPartitionCommon::RemoveDuplicates(InData, PartitionAttributeSources, Context))
		{
			Output.Data = DataWithoutDuplicates;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
