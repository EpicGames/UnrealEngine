// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAttributeSelectElement.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeSelectElement)

#define LOCTEXT_NAMESPACE "PCGAttributeSelectElement"

namespace PCGAttributeSelectElement
{
	// Of course, Dot doesn't exist for FVector4...
	template <typename T>
	inline double Dot(const T& A, const T& B)
	{
		if constexpr (std::is_same_v<FVector4, T>)
		{
			return Dot4(A, B);
		}
		else
		{
			return A.Dot(B);
		}
	}

	// No need to do a dot product if the axis is either X, Y, Z or W.
	// We still need to check the type for Z and W, since it needs to compile even if we
	// will never call this function with the wrong type for a given axis.
	// If T is a scalar, just return InValue. Use decltype(auto) to return an int if T is an int.
	// It will allow to do comparison between int, instead of losing precision by converting it to double.
	template <typename T, int Axis>
	inline decltype(auto) Projection(const T& InValue, const T& InAxis)
	{
		if constexpr (PCG::Private::IsOfTypes<T, FVector2D, FVector, FVector4>())
		{
			if constexpr (Axis == 0)
			{
				return InValue.X;
			}
			else if constexpr (Axis == 1)
			{
				return InValue.Y;
			}
			else if constexpr (Axis == 2 && PCG::Private::IsOfTypes<T, FVector, FVector4>())
			{
				return InValue.Z;
			}
			else if constexpr (Axis == 3 && PCG::Private::IsOfTypes<T, FVector4>())
			{
				return InValue.W;
			}
			else
			{
				return Dot(InValue, InAxis);
			}
		}
		else
		{
			// Use T constructor to force the compiler to match decltype(auto) to T.
			// Otherwise, it would match to const T, which leads to issues.
			return T(InValue);
		}
	}

	template <typename T, bool bIsMin, int Axis>
	bool MinMaxSelect(const IPCGAttributeAccessorKeys& Keys, const IPCGAttributeAccessor& Accessor, const T& InAxis, T& OutValue, int32& OutIndex)
	{
		// Will be int or double
		using CompareType = decltype(Projection<T, Axis>(T{}, InAxis));

		CompareType MinMaxValue{};
		bool bFirstValue = true;

		return PCGMetadataElementCommon::ApplyOnAccessor<T>(Keys, Accessor, [&OutValue, &OutIndex, &bFirstValue, &MinMaxValue, &InAxis](const T& InValue, int32 InIndex)
			{
				CompareType CurrentValue = Projection<T, Axis>(InValue, InAxis);

				if constexpr (bIsMin)
				{
					if (bFirstValue || CurrentValue < MinMaxValue)
					{
						MinMaxValue = CurrentValue;
						OutValue = InValue;
						OutIndex = InIndex;
					}
				}
				else
				{
					if (bFirstValue || CurrentValue > MinMaxValue)
					{
						MinMaxValue = CurrentValue;
						OutValue = InValue;
						OutIndex = InIndex;
					}
				}

				bFirstValue = false;
			});
	}

	template <typename T, int Axis>
	bool MedianSelect(const IPCGAttributeAccessorKeys& Keys, const IPCGAttributeAccessor& Accessor, const T& InAxis, T& OutValue, int32& OutIndex)
	{
		// Will be int or double
		using CompareType = decltype(Projection<T, Axis>(T{}, InAxis));

		struct Item
		{
			Item(int32 InIndex, CompareType InCompareValue, const T& InAttributeValue)
				: Index(InIndex)
				, CompareValue(InCompareValue)
				, AttributeValue(InAttributeValue)
			{}

			int32 Index;
			CompareType CompareValue;
			T AttributeValue;
		};

		const int32 NumberOfEntries = Keys.GetNum();

		if (NumberOfEntries == 0)
		{
			return false;
		}

		TArray<Item> CompareItems;
		CompareItems.Reserve(NumberOfEntries);

		PCGMetadataElementCommon::ApplyOnAccessor<T>(Keys, Accessor, [&CompareItems, &InAxis](const T& InValue, int32 InIndex)
			{
				CompareItems.Emplace(InIndex, Projection<T, Axis>(InValue, InAxis), InValue);
			});

		// TODO: Use a better algo than just sorting.
		Algo::Sort(CompareItems, [](const Item& A, const Item& B) -> bool { return A.CompareValue < B.CompareValue; });

		// Since we need to return an index, we can't do the mean on 2 values if the number of entries is even, since it will yield a value that might not exist in the original dataset.
		// In this case we arbitrarily chose one entry.
		const int32 Index = NumberOfEntries / 2;
		OutIndex = CompareItems[Index].Index;
		OutValue = CompareItems[Index].AttributeValue;

		return true;
	}

	template <typename T, int Axis>
	inline bool DispatchOperation(const IPCGAttributeAccessorKeys& Keys, const IPCGAttributeAccessor& Accessor, const T& InAxis, EPCGAttributeSelectOperation Operation, T& OutValue, int32& OutIndex)
	{
		switch (Operation)
		{
		case EPCGAttributeSelectOperation::Min:
			return PCGAttributeSelectElement::MinMaxSelect<T, /*bIsMin=*/true, Axis>(Keys, Accessor, InAxis, OutValue, OutIndex);
		case EPCGAttributeSelectOperation::Max:
			return PCGAttributeSelectElement::MinMaxSelect<T, /*bIsMin=*/false, Axis>(Keys, Accessor, InAxis, OutValue, OutIndex);
		case EPCGAttributeSelectOperation::Median:
			return PCGAttributeSelectElement::MedianSelect<T, Axis>(Keys, Accessor, InAxis, OutValue, OutIndex);
		default:
			return false;
		}
	}
}

void UPCGAttributeSelectSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (InputAttributeName_DEPRECATED != NAME_None)
	{
		InputSource.SetAttributeName(InputAttributeName_DEPRECATED);
		InputAttributeName_DEPRECATED = NAME_None;
	}
#endif
}

#if WITH_EDITOR
FName UPCGAttributeSelectSettings::GetDefaultNodeName() const
{
	return TEXT("AttributeSelect");
}

FText UPCGAttributeSelectSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Attribute Select");
}

void UPCGAttributeSelectSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	if (DataVersion < FPCGCustomVersion::UpdateAttributePropertyInputSelector
		&& (OutputAttributeName == NAME_None))
	{
		// Previous behavior of the output attribute for this node was:
		// None => SameName
		OutputAttributeName = PCGMetadataAttributeConstants::SourceNameAttributeName;
	}

	Super::ApplyDeprecation(InOutNode);
}
#endif

FString UPCGAttributeSelectSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumOpPtr = StaticEnum<EPCGAttributeSelectOperation>())
	{
		if (const UEnum* EnumAxisPtr = StaticEnum<EPCGAttributeSelectAxis>())
		{
			const FText OperationName = EnumOpPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation));
			FString AxisName;
			if (Axis == EPCGAttributeSelectAxis::CustomAxis)
			{
				AxisName = FString::Printf(TEXT("(%.2f, %.2f, %.2f, %.2f)"), CustomAxis.X, CustomAxis.Y, CustomAxis.Z, CustomAxis.W);
			}
			else
			{
				AxisName = EnumAxisPtr->GetDisplayNameTextByValue(static_cast<int64>(Axis)).ToString();
			}

			FName InputAttributeName = InputSource.GetName();
			if (InputAttributeName == NAME_None)
			{
				InputAttributeName = FName(TEXT("LastAttribute"));
			}

			if (InputAttributeName != OutputAttributeName && OutputAttributeName != NAME_None)
			{
				return FText::Format(LOCTEXT("SelectInputToOutputWithOperationAndAxis", "Select {0} to {1}: {2} on {3}"), FText::FromName(InputAttributeName), FText::FromName(OutputAttributeName), OperationName, FText::FromString(AxisName)).ToString();
			}
			else
			{
				return FText::Format(LOCTEXT("SelectWithOperationAndAxis", "Select {0}: {1} on {2}"), FText::FromName(InputAttributeName), OperationName, FText::FromString(AxisName)).ToString();
			}
		}
	}

	return FString();
}

TArray<FPCGPinProperties> UPCGAttributeSelectSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spatial, /*bInAllowMultipleConnections=*/ false);
	InputPinProperty.SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGAttributeSelectSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGAttributeSelectConstants::OutputAttributeLabel, EPCGDataType::Param);
	PinProperties.Emplace(PCGAttributeSelectConstants::OutputPointLabel, EPCGDataType::Point);

	return PinProperties;
}

FPCGElementPtr UPCGAttributeSelectSettings::CreateElement() const
{
	return MakeShared<FPCGAttributeSelectElement>();
}

bool FPCGAttributeSelectElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeSelectElement::Execute);

	check(Context);

	const UPCGAttributeSelectSettings* Settings = Context->GetInputSettings<UPCGAttributeSelectSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	if (Inputs.Num() != 1)
	{
		PCGE_LOG(Error, LogOnly, FText::Format(LOCTEXT("WrongNumberOfInputs", "Input pin expected to have one input data element, encountered {0}"), Inputs.Num()));
		return true;
	}

	const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Inputs[0].Data);

	if (!SpatialData)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("InputNotSpatialData", "Input is not a spatial data"));
		return true;
	}

	const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(SpatialData);
	if (!PointData && Context->Node && Context->Node->IsOutputPinConnected(PCGAttributeSelectConstants::OutputPointLabel))
	{
		PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InputMissingPointData", "No point data in input, will output nothing in the '{0}' output pin"), FText::FromName(PCGAttributeSelectConstants::OutputPointLabel)));
	}

	if (!SpatialData->Metadata)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("InputMissingMetadata", "Input data doesn't have metadata"));
		return true;
	}

	FPCGAttributePropertyInputSelector InputSource = Settings->InputSource.CopyAndFixLast(SpatialData);

	const FName OutputAttributeName = (Settings->OutputAttributeName == PCGMetadataAttributeConstants::SourceNameAttributeName) ? InputSource.GetName() : Settings->OutputAttributeName;
	UPCGParamData* OutputParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);

	TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, InputSource);
	TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, InputSource);

	if (!Accessor.IsValid() || !Keys.IsValid())
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("AttributeMissing", "Input attribute/property does not exist"));
		return true;
	}

	auto DoOperation = [this, &Accessor, &Keys, &Settings, OutputAttributeName, OutputParamData, Context](auto DummyValue) -> int32
	{
		using AttributeType = decltype(DummyValue);

		if constexpr (!PCG::Private::IsOfTypes<AttributeType, int32, int64, float, double, FVector2D, FVector, FVector4>())
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("AttributeNotVectorScalar", "Attribute type is not a Vector nor a scalar"));
			return -1;
		}
		else
		{
			bool bSuccess = false;
			AttributeType OutputValue{};
			AttributeType Axis{};
			int32 OutputIndex = -1;

			FPCGMetadataAttribute<AttributeType>* NewAttribute = static_cast<FPCGMetadataAttribute<AttributeType>*>(
				OutputParamData->Metadata->CreateAttribute<AttributeType>(OutputAttributeName, OutputValue, /*bAllowInterpolation=*/ true, /*bOverrideParent=*/false));

			if (!NewAttribute)
			{
				return -1;
			}

			const FText InvalidErrorAxisMessage = LOCTEXT("InvalidAxis", "Invalid axis for attribute type");

			// First we need to verify if the axis we want to project on is valid for dimension of our vector type.
			// If it is a scalar, we won't project anything.
			if constexpr (PCG::Private::IsOfTypes<AttributeType, FVector2D, FVector, FVector4>())
			{
				bool bIsValid = false;

				switch (Settings->Axis)
				{
				case EPCGAttributeSelectAxis::X:
				case EPCGAttributeSelectAxis::Y:
					bIsValid = true;
					break;
				case EPCGAttributeSelectAxis::Z:
					bIsValid = PCG::Private::IsOfTypes<AttributeType, FVector, FVector4>();
					break;
				case EPCGAttributeSelectAxis::W:
					bIsValid = PCG::Private::IsOfTypes<AttributeType, FVector4>();
					break;
				case EPCGAttributeSelectAxis::CustomAxis:
					Axis = AttributeType(Settings->CustomAxis);
					bIsValid = !Axis.Equals(AttributeType::Zero());
					break;
				default:
					break;
				}

				if (!bIsValid)
				{
					PCGE_LOG(Error, GraphAndLog, InvalidErrorAxisMessage);
					return -1;
				}
			}

			// Finally dispatch the operation depending on the axis.
			// If the axis is X, Y, Z or W, the projection is overkill (Dot product), so use templates to 
			// indicate which coordinate we should take. If the axis value is -1, it will do the projection with the custom axis, passed as parameter.
			switch (Settings->Axis)
			{
			case EPCGAttributeSelectAxis::X:
				bSuccess = PCGAttributeSelectElement::DispatchOperation<AttributeType, 0>(*Keys, *Accessor, Axis, Settings->Operation, OutputValue, OutputIndex);
				break;
			case EPCGAttributeSelectAxis::Y:
				bSuccess = PCGAttributeSelectElement::DispatchOperation<AttributeType, 1>(*Keys, *Accessor, Axis, Settings->Operation, OutputValue, OutputIndex);
				break;
			case EPCGAttributeSelectAxis::Z:
				bSuccess = PCGAttributeSelectElement::DispatchOperation<AttributeType, 2>(*Keys, *Accessor, Axis, Settings->Operation, OutputValue, OutputIndex);
				break;
			case EPCGAttributeSelectAxis::W:
				bSuccess = PCGAttributeSelectElement::DispatchOperation<AttributeType, 3>(*Keys, *Accessor, Axis, Settings->Operation, OutputValue, OutputIndex);
				break;
			case EPCGAttributeSelectAxis::CustomAxis:
				bSuccess = PCGAttributeSelectElement::DispatchOperation<AttributeType, -1>(*Keys, *Accessor, Axis, Settings->Operation, OutputValue, OutputIndex);
				break;
			default:
				break;
			}

			if (bSuccess)
			{
				OutputParamData->Metadata->AddEntry();
				NewAttribute->SetDefaultValue(OutputValue);
			}
			else
			{
				PCGE_LOG(Error, GraphAndLog, InvalidErrorAxisMessage);
				OutputIndex = -1;
			}

			return OutputIndex;
		}
	};

	int32 OutputIndex = PCGMetadataAttribute::CallbackWithRightType(Accessor->GetUnderlyingType(), DoOperation);
	if (OutputIndex < 0)
	{
		return true;
	}

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	FPCGTaggedData& Output = Outputs.Add_GetRef(Inputs[0]);
	Output.Data = OutputParamData;
	Output.Pin = PCGAttributeSelectConstants::OutputAttributeLabel;

#if WITH_EDITOR
	if(PointData)
#else
	if(PointData && Context->Node && Context->Node->IsOutputPinConnected(PCGAttributeSelectConstants::OutputPointLabel))
#endif
	{
		UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(Context);
		
		FPCGInitializeFromDataParams InitializeFromDataParams(PointData);
		InitializeFromDataParams.bInheritSpatialData = false;
		OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);

		OutputPointData->SetNumPoints(1);
		OutputPointData->CopyUnallocatedPropertiesFrom(PointData);

		const FConstPCGPointValueRanges InRanges(PointData);
		FPCGPointValueRanges OutRanges(OutputPointData, /*bAllocate=*/false);

		OutRanges.SetFromValueRanges(0, InRanges, OutputIndex);

		FPCGTaggedData& PointOutput = Outputs.Add_GetRef(Inputs[0]);
		PointOutput.Data = OutputPointData;
		PointOutput.Pin = PCGAttributeSelectConstants::OutputPointLabel;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
