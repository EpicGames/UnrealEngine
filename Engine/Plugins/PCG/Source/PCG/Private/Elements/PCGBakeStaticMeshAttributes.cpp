// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGBakeStaticMeshAttributes.h"

#include "PCGCommon.h"
#include "PCGDilate.h"
#include "PCGModule.h"
#include "PCGUnwrapMesh.h"
#include "Data/PCGRenderTargetData.h"
#include "Utils/PCGLogErrors.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "StaticMeshResources.h"
#include "TextureResource.h"
#include "Engine/StaticMesh.h"
#include "Engine/TextureRenderTarget2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBakeStaticMeshAttributes)

#define LOCTEXT_NAMESPACE "PCGBakeStaticMeshAttributes"

static_assert(static_cast<uint8>(EPCGMeshAttribute::LocalPosition) == static_cast<uint8>(PCGUnwrapMesh::EMeshAttribute::LocalPosition));
static_assert(static_cast<uint8>(EPCGMeshAttribute::Mask)          == static_cast<uint8>(PCGUnwrapMesh::EMeshAttribute::Mask));

#if WITH_EDITOR
FText UPCGBakeStaticMeshAttributesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Bake Static Mesh Attributes");
}

FText UPCGBakeStaticMeshAttributesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Renders a static mesh into a texture by UV unwrap. Each output texel receives the selected mesh attribute at the surface point its UV addresses,"
		" producing a Render Target data downstream PCG nodes can sample. Nanite-only meshes use their LOD0 coarse fallback.");
}
#endif // WITH_EDITOR

FString UPCGBakeStaticMeshAttributesSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGMeshAttribute>())
	{
		return EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(BakeParams.Attribute)).ToString();
	}

	return FString();
}

TArray<FPCGPinProperties> UPCGBakeStaticMeshAttributesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& Pin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::RenderTarget);
	Pin.bAllowMultipleData = false;
	return PinProperties;
}

FPCGElementPtr UPCGBakeStaticMeshAttributesSettings::CreateElement() const
{
	return MakeShared<FPCGBakeStaticMeshAttributesElement>();
}

FPCGBakeStaticMeshAttributesContext::~FPCGBakeStaticMeshAttributesContext()
{
	if (bPinnedLOD0)
	{
		if (UStaticMesh* Mesh = LoadedMesh.Get())
		{
			// Only restore if the flag is still set. If something else explicitly unpinned the mesh while we were using it, preserve that intent rather than re-pinning.
			if (Mesh->bForceMiplevelsToBeResident)
			{
				Mesh->bForceMiplevelsToBeResident = bPrevForceMipsResident;
			}
		}
	}
}

void FPCGBakeStaticMeshAttributesContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(LoadedMesh);
	Collector.AddReferencedObject(RenderTarget);
	Collector.AddReferencedObject(OutputRenderTargetData);
}

bool FPCGBakeStaticMeshAttributesElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBakeStaticMeshAttributesElement::PrepareDataInternal);
	check(InContext);

	FPCGBakeStaticMeshAttributesContext* Context = static_cast<FPCGBakeStaticMeshAttributesContext*>(InContext);
	const UPCGBakeStaticMeshAttributesSettings* Settings = Context->GetInputSettings<UPCGBakeStaticMeshAttributesSettings>();
	check(Settings);

	if (Context->WasLoadRequested())
	{
		return true;
	}

	if (Settings->StaticMesh.IsNull())
	{
		return true;
	}

	TArray<FSoftObjectPath> ToLoad;
	ToLoad.Add(Settings->StaticMesh.ToSoftObjectPath());
	return Context->RequestResourceLoad(Context, MoveTemp(ToLoad), !Settings->bSynchronousLoad);
}

bool FPCGBakeStaticMeshAttributesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBakeStaticMeshAttributesElement::Execute);
	check(InContext);

	FPCGBakeStaticMeshAttributesContext* Context = static_cast<FPCGBakeStaticMeshAttributesContext*>(InContext);
	const UPCGBakeStaticMeshAttributesSettings* Settings = Context->GetInputSettings<UPCGBakeStaticMeshAttributesSettings>();
	check(Settings);

	if (Settings->StaticMesh.IsNull())
	{
		return true;
	}

	auto SleepUntilNextFrame = [Context]()
	{
		Context->bIsPaused = true;
		FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
		{
			if (TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin())
			{
				if (FPCGContext* ContextPtr = SharedHandle->GetContext())
				{
					ContextPtr->bIsPaused = false;
				}
			}
		});
	};

	// Resolve the mesh and build the render target.
	if (!Context->LoadedMesh)
	{
		UStaticMesh* Mesh = Settings->StaticMesh.Get();
		if (!Mesh)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FailedLoad", "Failed to load static mesh '{0}'."), FText::FromString(Settings->StaticMesh.ToString())), Context);
			return true;
		}

		// Pin LOD0 resident on the streamer. Saved value is restored on the success/output-emit path below.
		if (!Context->bPinnedLOD0)
		{
			Context->bPrevForceMipsResident = Mesh->bForceMiplevelsToBeResident;
			Mesh->bForceMiplevelsToBeResident = true;
			Context->bPinnedLOD0 = true;
		}

		if (Mesh->HasPendingInitOrStreaming())
		{
			UE_LOGF(LogPCG, Verbose, "PCGBakeStaticMeshAttributes: mesh '%ls' has pending init or streaming; deferring", *Mesh->GetName());
			SleepUntilNextFrame();
			return false;
		}

		// HasPendingInitOrStreaming returning false does not guarantee LOD0 is resident. With LOD0 pinned above, GetCurrentFirstLODIdx will converge to 0 once the streamer satisfies the request. Defer until then.
		if (const FStaticMeshRenderData* RD = Mesh->GetRenderData())
		{
			const int32 CurrentFirstLOD = RD->GetCurrentFirstLODIdx(/*MinLODIdx=*/0);
			if (CurrentFirstLOD != 0)
			{
				UE_LOGF(LogPCG, Verbose, "PCGBakeStaticMeshAttributes: LOD0 not resident yet for mesh '%ls' (CurrentFirstLOD=%d); deferring", *Mesh->GetName(), CurrentFirstLOD);
				SleepUntilNextFrame();
				return false;
			}
		}

		Context->LoadedMesh = Mesh;

		const FStaticMeshRenderData* RenderData = Context->LoadedMesh->GetRenderData();
		if (!RenderData || RenderData->LODResources.IsEmpty())
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NoRenderData", "Static mesh '{0}' has no render data."), FText::FromName(Context->LoadedMesh->GetFName())), Context);
			return true;
		}

		const FStaticMeshLODResources& LOD = RenderData->LODResources[0];
		if (LOD.GetNumTriangles() == 0)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ZeroTris", "Static mesh '{0}' LOD0 has zero triangles (Nanite coarse fallback may be missing)."), FText::FromName(Context->LoadedMesh->GetFName())), Context);
			return true;
		}

		const uint32 NumTexCoords = LOD.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
		if (NumTexCoords == 0)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NoUVs", "Static mesh '{0}' has no UV channels."), FText::FromName(Context->LoadedMesh->GetFName())), Context);
			return true;
		}

		if (static_cast<uint32>(Settings->BakeParams.UVChannelIndex) >= NumTexCoords)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("UVChannelOutOfRange", "Static mesh '{0}' has {1} UV channel(s); requested UVChannelIndex {2} is out of range."), FText::FromName(Context->LoadedMesh->GetFName()), NumTexCoords, Settings->BakeParams.UVChannelIndex), Context);
			return true;
		}

		// Defensive check: a non-resident or stripped LOD can have NumTriangles > 0 in metadata while VertexBuffer GPU storage is 0 bytes. BuffersSize is the
		// CPU-known total of vertex+index buffer sizes; zero means buffers aren't actually loaded for this LOD.
		if (LOD.GetNumVertices() == 0 || LOD.BuffersSize == 0)
		{
			UE_LOG(LogPCG, Error, TEXT("Static mesh '%s' LOD0 buffers are not resident (NumVerts=%d, BuffersSize=%d). Likely streamed-out or stripped."), *Context->LoadedMesh->GetName(), LOD.GetNumVertices(), LOD.BuffersSize);
			return true;
		}
	}

	if (!Context->RenderTarget)
	{
		UTextureRenderTarget2D* RenderTarget = FPCGContext::NewObject_AnyThread<UTextureRenderTarget2D>(Context);
		RenderTarget->RenderTargetFormat = Settings->BakeParams.Format.GetValue();
		RenderTarget->ClearColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
		RenderTarget->bAutoGenerateMips = false;
		RenderTarget->bCanCreateUAV = false;
		// We write raw numeric attribute data (e.g. local positions), never a color. Force linear to avoid surprise if the user picks an 8-bit format.
		RenderTarget->TargetGamma = 1.0f;
		RenderTarget->InitAutoFormat(Settings->BakeParams.Resolution.X, Settings->BakeParams.Resolution.Y);
		// The raster pass clears on bind via ERenderTargetLoadAction::EClear, so no init-time clear needed.
		RenderTarget->UpdateResourceImmediate(false);

		Context->RenderTarget = RenderTarget;

		Context->OutputRenderTargetData = FPCGContext::NewObject_AnyThread<UPCGRenderTargetData>(Context);
		Context->OutputRenderTargetData->Initialize(RenderTarget, FTransform::Identity, /*bInTakeOwnershipOfRenderTarget=*/true);
	}

	if (!Context->LoadedMesh || !Context->RenderTarget || !Context->OutputRenderTargetData)
	{
		return true;
	}

	if (!Context->bEnqueuedRenderCommand)
	{
		// Hold the LOD by ref-count across the GT->RT hop. FStaticMeshLODResources inherits FQueryableRefCountedObject, so the LOD survives even if the streamer evicts/replaces it before the render command runs.
		const FStaticMeshRenderData* RenderData = Context->LoadedMesh->GetRenderData();
		if (!RenderData || RenderData->LODResources.IsEmpty())
		{
			return true;
		}
		TRefCountPtr<const FStaticMeshLODResources> LODRef(&RenderData->LODResources[0]);

		PCGUnwrapMesh::FUnwrapParams UnwrapParams;
		UnwrapParams.UVChannelIndex = Settings->BakeParams.UVChannelIndex;
		UnwrapParams.Attribute = static_cast<PCGUnwrapMesh::EMeshAttribute>(Settings->BakeParams.Attribute);
		UnwrapParams.Resolution = Settings->BakeParams.Resolution;

		// The coverage mask must remain crisp so it can be used as a sample-acceptance signal, other attributes can apply padding.
		const int32 PaddingIterations = (Settings->BakeParams.Attribute == EPCGMeshAttribute::Mask) ? 0 : Settings->BakeParams.Padding;

		Context->bIsPaused = true;

		ENQUEUE_RENDER_COMMAND(PCGBakeStaticMeshAttributes)(
			[ContextHandle = Context->GetOrCreateHandle(), UnwrapParams, LODRef = MoveTemp(LODRef), PaddingIterations](FRHICommandListImmediate& RHICmdList) mutable
			{
				FPCGContext::FSharedContext<FPCGBakeStaticMeshAttributesContext> SharedContext(ContextHandle);
				FPCGBakeStaticMeshAttributesContext* LocalContext = SharedContext.Get();
				if (!LocalContext)
				{
					return;
				}

				ON_SCOPE_EXIT
				{
					LocalContext->bIsPaused = false;
				};

				check(LODRef);
				UnwrapParams.InitFromLOD(*LODRef);

				if (!PCGUnwrapMesh::ValidateParams(UnwrapParams))
				{
					return;
				}

				UTextureRenderTarget2D* RenderTarget = LocalContext->RenderTarget;
				FTextureRenderTargetResource* RTResource = RenderTarget ? RenderTarget->GetRenderTargetResource() : nullptr;
				FTextureRHIRef TargetRHI = RTResource ? RTResource->GetRenderTargetTexture() : FTextureRHIRef();
				if (!TargetRHI)
				{
					return;
				}

				FRDGBuilder GraphBuilder(RHICmdList);
				FRDGTextureRef RDGOutput = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(TargetRHI, TEXT("PCGBakeMeshAttr.Output")));
				if (PCGUnwrapMesh::AddUnwrapMeshPass(GraphBuilder, RDGOutput, UnwrapParams))
				{
					PCGDilate::AddDilatePass(GraphBuilder, RDGOutput, PaddingIterations);
				}
				else
				{
					PCGLog::LogErrorOnGraph(LOCTEXT("UnwrapPassFailed", "Failed to add unwrap mesh pass."), LocalContext);
				}
				GraphBuilder.Execute();
			});

		Context->bEnqueuedRenderCommand = true;

		return false;
	}

	FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
	Output.Data = Context->OutputRenderTargetData;
	Output.Pin = PCGPinConstants::DefaultOutputLabel;

	return true;
}

#undef LOCTEXT_NAMESPACE
