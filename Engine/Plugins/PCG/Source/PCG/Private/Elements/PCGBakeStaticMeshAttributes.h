// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Elements/PCGMeshAttribute.h"

#include "Engine/TextureRenderTarget2D.h"

#include "PCGBakeStaticMeshAttributes.generated.h"

class UPCGRenderTargetData;
class UStaticMesh;
class UTextureRenderTarget2D;

/**
 * Renders a static mesh into a texture by UV unwrap. Each output texel receives the selected mesh attribute at the surface point its UV addresses,
 * producing a Render Target data downstream PCG nodes can sample. Nanite-only meshes use their LOD0 coarse fallback.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGBakeStaticMeshAttributesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("BakeStaticMeshAttributes")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Static mesh to bake. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TSoftObjectPtr<UStaticMesh> StaticMesh;

	/** Shared render-target/unwrap parameters. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGBakeMeshAttributesParams BakeParams;

	/** By default, mesh loading is asynchronous, can force it synchronous if needed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

struct FPCGBakeStaticMeshAttributesContext : public FPCGContext, public IPCGAsyncLoadingContext
{
public:
	virtual ~FPCGBakeStaticMeshAttributesContext();

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;

public:
	bool bEnqueuedRenderCommand = false;

	/** True once we have asked the streamer to force LOD0 resident for this static mesh. */
	bool bPinnedLOD0 = false;

	// Snapshot of bForceMiplevelsToBeResident captured before we set it. Restored when execution exits.
	bool bPrevForceMipsResident = false;

	/** Loaded mesh held alive for the duration of the render command. */
	TObjectPtr<UStaticMesh> LoadedMesh;

	/** Render target the pass writes into. */
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

	/** PCG-typed view of the render target produced as the node's output. */
	TObjectPtr<UPCGRenderTargetData> OutputRenderTargetData;
};

class FPCGBakeStaticMeshAttributesElement : public IPCGElementWithCustomContext<FPCGBakeStaticMeshAttributesContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
