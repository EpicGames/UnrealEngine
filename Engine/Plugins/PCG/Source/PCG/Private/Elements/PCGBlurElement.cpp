// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGBlurElement.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "SpatialAlgo/PCGOctreeQueries.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBlurElement)

#define LOCTEXT_NAMESPACE "PCGBlurElement"

#if WITH_EDITOR
FName UPCGBlurSettings::GetDefaultNodeName() const
{
	return FName(TEXT("BlurElement"));
}

FText UPCGBlurSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Blur");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGBlurSettings::CreateElement() const
{
	return MakeShared<FPCGBlurElement>();
}

bool FPCGBlurElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBlurElement::PrepareData);

	const UPCGBlurSettings* Settings = InContext->GetInputSettings<UPCGBlurSettings>();
	check(Settings);

	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context);

	TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	// Making sure the parameters are not negative or zero. Extra carefulness with search distance.
	if (Settings->NumIterations <= 0 || Settings->SearchDistance <= 0 || FMath::IsNearlyZero(Settings->SearchDistance))
	{
		// Nothing to do, so forward the input
		Outputs = Inputs;
		return true;
	}

	// Verify that standard deviation is valid
	if (Settings->BlurMode == EPCGBlurElementMode::Gaussian && Settings->bUseCustomStandardDeviation && (Settings->CustomStandardDeviation <= 0 || FMath::IsNearlyZero(Settings->CustomStandardDeviation)))
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("InvalidStdDev", "Blur mode was set to Gaussian but Standard Deviation was less or equal than zero. Abort."));
		return true;
	}

	// No exec init
	Context->InitializePerExecutionState();

	Context->InitializePerIterationStates(Inputs.Num(), [&Inputs, &Outputs, InContext, Settings](FPCGBlurIterState& OutState, const ExecStateType&, const uint32 IterationIndex) -> EPCGTimeSliceInitResult
	{
		FPCGTaggedData& OutputData = Outputs.Emplace_GetRef(Inputs[IterationIndex]);
		OutputData.Pin = PCGPinConstants::DefaultOutputLabel;


		OutState.InputData = Cast<UPCGBasePointData>(Inputs[IterationIndex].Data);
		if (!OutState.InputData)
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		FPCGAttributePropertyInputSelector InputSelector = Settings->InputSource.CopyAndFixLast(OutState.InputData);

		OutState.InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OutState.InputData, InputSelector);
		OutState.InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(OutState.InputData, InputSelector);

		if (!OutState.InputAccessor || !OutState.InputKeys)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InputAttributeNotFound", "Attribute {0} was not found in input {1}"), InputSelector.GetDisplayText(), IterationIndex), InContext);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Nothing to do
		if (OutState.InputKeys->GetNum() == 0)
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (!PCG::Private::IsOfTypes<int32, int64, float, double, FVector2D, FVector, FVector4, FRotator, FQuat>(OutState.InputAccessor->GetUnderlyingType()))
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InputNotNumerical", "Attribute {0} is not numerical for input {1}"), InputSelector.GetDisplayText(), IterationIndex), InContext);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		UPCGBasePointData* OutputPointData = CastChecked<UPCGBasePointData>(OutState.InputData->DuplicateData(InContext));

		FPCGAttributePropertyOutputSelector OutputSelector = Settings->OutputTarget.CopyAndFixSource(&InputSelector, OutState.InputData);

		// Create attribute if needed
		if (OutputSelector.IsBasicAttribute())
		{
			check(OutputPointData->Metadata);
			const FName AttributeName = OutputSelector.GetName();
			if (OutputPointData->Metadata->HasAttribute(AttributeName))
			{
				OutputPointData->Metadata->DeleteAttribute(AttributeName);
			}

			auto CreateAttribute = [&InAccessor = OutState.InputAccessor, AttributeName, OutputPointData](auto&& Dummy) -> FPCGMetadataAttributeBase*
			{
				check(InAccessor.IsValid());
				using AttributeType = std::decay_t<decltype(Dummy)>;
				AttributeType DefaultValue = PCG::Private::MetadataTraits<AttributeType>::ZeroValue();
				InAccessor->Get<AttributeType>(DefaultValue, FPCGAttributeAccessorKeysEntries(PCGInvalidEntryKey));
				return OutputPointData->Metadata->CreateAttribute<AttributeType>(AttributeName, DefaultValue, /*bAllowInterpolation=*/true, /*bOverrideParent=*/false);
			};

			if (!PCGMetadataAttribute::CallbackWithRightType(OutState.InputAccessor->GetUnderlyingType(), std::move(CreateAttribute)))
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FailedToCreateAttr", "Could not create output attribute '{0}' for input {1}"), OutputSelector.GetDisplayText(), IterationIndex), InContext);
				return EPCGTimeSliceInitResult::NoOperation;
			}
		}

		OutState.OutputAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputPointData, OutputSelector);
		OutState.OutputKeys = PCGAttributeAccessorHelpers::CreateKeys(OutputPointData, OutputSelector);

		if (!OutState.OutputAccessor || !OutState.OutputKeys)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("OutputAttributeFailiure", "Attribute {0} was not found, not compatible or failed to be created for input {1}"), OutputSelector.GetDisplayText(), IterationIndex), InContext);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		OutputData.Data = OutputPointData;

		return EPCGTimeSliceInitResult::Success;
	});

	return true;
}

bool FPCGBlurElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBlurElement::Execute);

	check(InContext);

	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context);

	if (!Context->DataIsPreparedForExecution())
	{
		return true;
	}

	const UPCGBlurSettings* Settings = InContext->GetInputSettings<UPCGBlurSettings>();
	check(Settings);

	check(Settings->BlurMode != EPCGBlurElementMode::Gaussian || !Settings->bUseCustomStandardDeviation || Settings->CustomStandardDeviation > 0);

	const double StdDeviation = Settings->BlurMode == EPCGBlurElementMode::Gaussian && !Settings->bUseCustomStandardDeviation 
		? Settings->SearchDistance / 9.0
		: Settings->CustomStandardDeviation;
	const double GaussianExpRatio = Settings->BlurMode == EPCGBlurElementMode::Gaussian ? 1.0 / (2.0 * StdDeviation * StdDeviation) : 0.0;
	const double GaussianRatio = Settings->BlurMode == EPCGBlurElementMode::Gaussian ? GaussianExpRatio / UE_DOUBLE_PI : 0.0;

	const bool bDone = ExecuteSlice(Context, [this, Settings, GaussianExpRatio, GaussianRatio](ContextType* Context, const ExecStateType& ExecState, IterStateType& IterState, const uint32 IterIndex)
	{
		if (Context->GetIterationStateResult(IterIndex) != EPCGTimeSliceInitResult::Success)
		{
			return true;
		}

		auto Process = [&IterState, Settings, Context, GaussianExpRatio, GaussianRatio]<typename AttributeType>(AttributeType) -> bool
		{
			if (IterState.WorkingBuffer1.IsEmpty())
			{
				ensure(IterState.WorkingBuffer2.IsEmpty());
				IterState.WorkingBuffer1.SetNumUninitialized(sizeof(AttributeType) * IterState.InputKeys->GetNum());
				IterState.WorkingBuffer2.SetNumUninitialized(sizeof(AttributeType) * IterState.InputKeys->GetNum());
				IterState.InputAccessor->GetRange<AttributeType>(MakeArrayView(reinterpret_cast<AttributeType*>(IterState.WorkingBuffer1.GetData()), IterState.InputKeys->GetNum()), 0, *IterState.InputKeys);
			}

			while (IterState.CurrentIteration < Settings->NumIterations)
			{
				const uint8* ReadBuffer = (IterState.CurrentIteration % 2 == 0) ? IterState.WorkingBuffer1.GetData() : IterState.WorkingBuffer2.GetData();
				uint8* WriteBuffer = (IterState.CurrentIteration % 2 == 1) ? IterState.WorkingBuffer1.GetData() : IterState.WorkingBuffer2.GetData();

				TArrayView<const AttributeType> TypedReadBuffer = MakeArrayView(reinterpret_cast<const AttributeType*>(ReadBuffer), IterState.InputKeys->GetNum());
				TArrayView<AttributeType> TypedWriteBuffer = MakeArrayView(reinterpret_cast<AttributeType*>(WriteBuffer), IterState.InputKeys->GetNum());

				const bool bIsDone = FPCGAsync::AsyncProcessingOneToOneRangeEx(&Context->AsyncState, IterState.InputKeys->GetNum(), []() {}, [&IterState, Settings, &TypedReadBuffer, TypedWriteBuffer, GaussianExpRatio, GaussianRatio](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
				{
					const TConstPCGValueRange<FTransform> ReadTransformRange = IterState.InputData->GetConstTransformValueRange();

					int32 NumWritten = 0;

					for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
					{
						const int32 WriteIndex = StartWriteIndex + NumWritten;
						const FVector PointLocation = ReadTransformRange[ReadIndex].GetLocation();
						TArray<TTuple<int32, double>> NeighborsAndDistance;

						UPCGOctreeQueries::ForEachPointInsideSphere(IterState.InputData, PointLocation, Settings->SearchDistance, [&NeighborsAndDistance](const UPCGBasePointData* PointData, int32 Neighbor, double SquaredDistance)
						{
							NeighborsAndDistance.Emplace(Neighbor, SquaredDistance);
						});

						if (!ensure(!NeighborsAndDistance.IsEmpty()))
						{
							++NumWritten;
							continue;
						}

						auto GetNeighborValue = [TypedReadBuffer, &IterState](int32 NeighborIndex) -> AttributeType
						{
							// Pointer arithmetic to know the neighbor index.
							return TypedReadBuffer[NeighborIndex];
						};

						// If it is weighted by distance, we compute all the weights first, to have the sum of all of them (for normalization)
						bool bValidWeights = false;
						TArray<double> Weights;
						if (Settings->BlurMode != EPCGBlurElementMode::Constant)
						{
							double WeightSum = 0.0;
							bValidWeights = true;
							Weights.Reserve(NeighborsAndDistance.Num());
							for (const auto& [Neighbor, SquaredDistance] : NeighborsAndDistance)
							{
								double Weight = 0.0;
								switch (Settings->BlurMode)
								{
								case EPCGBlurElementMode::Linear:
									Weight = 1.0 - FVector::Distance(PointLocation, ReadTransformRange[Neighbor].GetLocation()) / Settings->SearchDistance;
									break;
								case EPCGBlurElementMode::Gaussian:
									Weight = GaussianRatio * FMath::Exp(-SquaredDistance * GaussianExpRatio);
									break;
								default:
									checkNoEntry();
									break;
								}

								Weights.Add(Weight);
								WeightSum += Weight;
							}

							if (!FMath::IsNearlyZero(WeightSum))
							{
								for (double& Weight : Weights)
								{
									Weight /= WeightSum;
								}
							}
							else
							{
								bValidWeights = false;
							}
						}

						const double DefaultWeight = 1.0 / NeighborsAndDistance.Num();
						const bool bUseDefaultWeight = !bValidWeights;

						TypedWriteBuffer[WriteIndex] = PCG::Private::MetadataTraits<AttributeType>::ZeroValueForWeightedSum();

						for (int32 i = 0; i < NeighborsAndDistance.Num(); ++i)
						{
							AttributeType Value = GetNeighborValue(NeighborsAndDistance[i].Get<0>());
							const double Weight = bUseDefaultWeight ? DefaultWeight : Weights[i];
							TypedWriteBuffer[WriteIndex] = PCG::Private::MetadataTraits<AttributeType>::WeightedSum(TypedWriteBuffer[WriteIndex], Value, Weight);
						}

						if constexpr (PCG::Private::MetadataTraits<AttributeType>::InterpolationNeedsNormalization)
						{
							static_assert(PCG::Private::MetadataTraits<AttributeType>::CanNormalize);
								
							// We need to normalize the resulting value
							PCG::Private::MetadataTraits<AttributeType>::Normalize(TypedWriteBuffer[WriteIndex]);
						}

						++NumWritten;
					}

					check(Count == NumWritten);
					return Count;
				},/*bEnableTimeSlicing=*/true);

				if (!bIsDone)
				{
					return false;
				}

				++IterState.CurrentIteration;
			}

			// Copy back
			const uint8* ReadBuffer = (IterState.CurrentIteration % 2 == 0) ? IterState.WorkingBuffer1.GetData() : IterState.WorkingBuffer2.GetData();
			IterState.OutputAccessor->SetRange<AttributeType>(MakeArrayView(reinterpret_cast<const AttributeType*>(ReadBuffer), IterState.InputKeys->GetNum()), 0, *IterState.OutputKeys, EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible);

			IterState.WorkingBuffer1.Reset();
			IterState.WorkingBuffer2.Reset();
			return true;
		};

		auto Dispatch = [&IterState, Settings, Context, Process]<typename T>(T Dummy) -> bool
		{
			// Discard any non-numerical types
			if constexpr (!PCG::Private::IsOfTypes<T, int32, int64, float, double, FVector2D, FVector, FVector4, FRotator, FQuat>())
			{
				// We should not enter there.
				ensure(false);
				return true;
			}
			else
			{
				// Work on double for ints and floats to avoid approximation/rounding errors
				// Also Rotators are treated as Quaternions
				using AttributeType = std::conditional_t<std::is_arithmetic_v<T>, double, std::conditional_t<std::is_same_v<FRotator, T>, FQuat, T>>; 
				return Process(AttributeType{});
			}
		};

		return PCGMetadataAttribute::CallbackWithRightType(IterState.InputAccessor->GetUnderlyingType(), Dispatch);
	});

	return bDone;
}

#undef LOCTEXT_NAMESPACE
