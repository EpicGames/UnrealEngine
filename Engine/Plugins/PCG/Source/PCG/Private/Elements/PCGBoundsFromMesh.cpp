// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGBoundsFromMesh.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGAsync.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#if WITH_EDITOR
#include "Helpers/PCGDynamicTrackingHelpers.h"
#endif // WITH_EDITOR

#include "Algo/Transform.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBoundsFromMesh)

#define LOCTEXT_NAMESPACE "PCGBoundsFromMeshElement"

namespace PCGBoundsFromMeshSettings
{
	const FName AttributeLabel = TEXT("Attribute");
	const FText AttributeTooltip = LOCTEXT("AttributeTooltip", "Optional Attribute Set to select the mesh from. Not used if not connected.");
}

#if WITH_EDITOR
FName UPCGBoundsFromMeshSettings::GetDefaultNodeName() const
{
	return FName(TEXT("BoundsFromMesh"));
}

FText UPCGBoundsFromMeshSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Bounds From Mesh");
}

FText UPCGBoundsFromMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Sets the bounds according to the static or skeletal mesh(es) provided in the mesh pin.");
}

bool UPCGBoundsFromMeshSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	return !InPin || (InPin->Properties.Label != PCGBoundsFromMeshSettings::AttributeLabel) || InPin->IsConnected();
}
#endif //WITH_EDITOR

TArray<FPCGPinProperties> UPCGBoundsFromMeshSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	InputPinProperty.SetRequiredPin();

	PinProperties.Emplace(PCGBoundsFromMeshSettings::AttributeLabel, EPCGDataType::Param, /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false, PCGBoundsFromMeshSettings::AttributeTooltip);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGBoundsFromMeshSettings::OutputPinProperties() const
{
	return Super::DefaultPointOutputPinProperties();
}

// Creates the Element to be used for ExecuteInternal.
FPCGElementPtr UPCGBoundsFromMeshSettings::CreateElement() const
{
	return MakeShared<FPCGBoundsFromMeshElement>();
}

bool FPCGBoundsFromMeshElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBoundsFromMeshElement::PrepareData);

	check(InContext);
	FPCGBoundsFromMeshContext* Context = static_cast<FPCGBoundsFromMeshContext*>(InContext);

	const UPCGBoundsFromMeshSettings* Settings = Context->GetInputSettings<UPCGBoundsFromMeshSettings>();
	check(Settings);

	if (!Context->bInputsGathered)
	{
		Context->CachedInputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
		Context->bInputsGathered = true;
	}

	if (!Context->bPrepareDone)
	{
		// Some background info:
		// Two common use cases:
		// 1. Input is a point data, and an attribute set is used to provide the mesh to take bounds from.
		// 2. Input is a point data, with a mesh attribute.
		TArray<FPCGTaggedData> SourceAttributeInputs = Context->InputData.GetInputsByPin(PCGBoundsFromMeshSettings::AttributeLabel);
		const UPCGParamData* SourceAttributeSet = (!SourceAttributeInputs.IsEmpty() ? Cast<UPCGParamData>(SourceAttributeInputs[0].Data) : nullptr);

		if (SourceAttributeSet)
		{
			// In this case, we don't need to partition anything, we'll have a single mesh entry.
			FPCGAttributePropertyInputSelector MeshAttributeSelector = Settings->MeshAttribute.CopyAndFixLast(SourceAttributeSet);
			TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(SourceAttributeSet, MeshAttributeSelector);
			TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SourceAttributeSet, MeshAttributeSelector);

			// Warn if the attribute set has multiple entries, because we'll use only the first value.
			if (SourceAttributeSet->ConstMetadata()->GetLocalItemCount() > 1)
			{
				PCGE_LOG(Warning, GraphAndLog, LOCTEXT("AttributeSetHasMultipleEntries", "Input attribute set has multiple entries - only the first one will be used."));
			}

			if (!InputAccessor.IsValid() || !InputKeys.IsValid())
			{
				if (!Settings->bSilenceAttributeNotFoundErrors)
				{
					PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("AttributeDoesNotExistOnAttributeSet", "Input attribute/property '{0}' does not exist on attribute set, skipped."), MeshAttributeSelector.GetDisplayText()));
				}
			}
			else
			{
				if (InputAccessor->Get<FSoftObjectPath>(Context->SingleMesh, 0, *InputKeys, EPCGAttributeAccessorFlags::AllowConstructible))
				{
					if (!Context->SingleMesh.IsNull())
					{
						Context->MeshesToLoad.Add(Context->SingleMesh);
					}
				}
				else if (!Settings->bSilenceAttributeNotFoundErrors)
				{
					PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("AttributeOfTheWrongTypeOnAttributeSet", "Input attribute/property '{0}' on attribute set does not match the expected type, skipped"), MeshAttributeSelector.GetDisplayText()));
				}
			}

			Context->bPrepareDone = true;
		}
		else
		{
			// In this case, we'll visit all points, gather the unique meshes and assign to each point a given mesh index
			const TArray<FPCGTaggedData>& Inputs = Context->CachedInputs;

			while (Context->CurrentPrepareIndex < Inputs.Num())
			{
				const FPCGTaggedData& Input = Inputs[Context->CurrentPrepareIndex];
				const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Input.Data);

				if (!PointData)
				{
					++Context->CurrentPrepareIndex;
					continue;
				}

				FPCGAttributePropertyInputSelector MeshAttributeSelector = Settings->MeshAttribute.CopyAndFixLast(PointData);

				FPCGBoundsFromMeshContext::InputMeshesData InputMeshData;
				InputMeshData.InputIndex = Context->CurrentPrepareIndex++;

				// Retrieve meshes
				TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, MeshAttributeSelector);
				TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, MeshAttributeSelector);

				if (!InputAccessor.IsValid() || !InputKeys.IsValid())
				{
					if (!Settings->bSilenceAttributeNotFoundErrors)
					{
						PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("AttributeDoesNotExistOnInput", "Input attribute/property '{0}' does not exist on input {1}, skipped"), MeshAttributeSelector.GetDisplayText(), FText::AsNumber(Context->CurrentPrepareIndex - 1)));
					}

					continue;
				}

				InputMeshData.MeshValueIndex.Reserve(InputKeys->GetNum());
				auto GatherMeshPathsAndCreateMap = [this, Context, &InputMeshData](const TArrayView<FSoftObjectPath>& Meshes, int Start, int Range)
				{
					for (const FSoftObjectPath& Mesh : Meshes)
					{
						InputMeshData.MeshValueIndex.Add(Mesh.IsNull() ? INDEX_NONE : Context->MeshesToLoad.AddUnique(Mesh));
					}
				};

				if (!PCGMetadataElementCommon::ApplyOnAccessorRange<FSoftObjectPath>(*InputKeys, *InputAccessor, GatherMeshPathsAndCreateMap, EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible))
				{
					if (!Settings->bSilenceAttributeNotFoundErrors)
					{
						PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("AttributeOfTheWrongTypeOnInput", "Input attribute/property '{0}' on input {1} does not match the expected type, skipped"), MeshAttributeSelector.GetDisplayText(), FText::AsNumber(Context->CurrentPrepareIndex - 1)));
					}
				}
				else
				{
					Context->PerInputData.Add(MoveTemp(InputMeshData));
				}

				if (Context->ShouldStop())
				{
					break;
				}
			}

			if (Context->CurrentPrepareIndex == Inputs.Num())
			{
				Context->bPrepareDone = true;
			}

			// Given partitioning can be expensive, check if we're out of time for this frame
			if (Context->ShouldStop())
			{
				return false;
			}
		}
	}

	if (Context->bPrepareDone && !Context->WasLoadRequested())
	{
		TArray<FSoftObjectPath> MeshesToLoad = Context->MeshesToLoad; // because of the move
		return Context->RequestResourceLoad(Context, MoveTemp(MeshesToLoad), !Settings->bSynchronousLoad);
	}

	return true;
}

bool FPCGBoundsFromMeshElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBoundsFromMeshElement::Execute);

	check(InContext);
	FPCGBoundsFromMeshContext* Context = static_cast<FPCGBoundsFromMeshContext*>(InContext);

	const UPCGBoundsFromMeshSettings* Settings = Context->GetInputSettings<UPCGBoundsFromMeshSettings>();
	check(Settings);

	const TArray<FPCGTaggedData>& Inputs = Context->CachedInputs;
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	auto BoundsFromMesh = [](const UObject* MeshObject, FBox& OutBounds) -> bool
	{
		if (const UStaticMesh* StaticMesh = Cast<UStaticMesh>(MeshObject))
		{
			OutBounds = StaticMesh->GetBoundingBox();
			return true;
		}
		else if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(MeshObject))
		{
			OutBounds = SkeletalMesh->GetBounds().GetBox();
			return true;
		}
		else
		{
			return false;
		}
	};

	if (!Context->bBoundsQueried)
	{
#if WITH_EDITOR
		// Tracking is always dynamic in this case, since we are always pulling the mesh from an attribute.
		FPCGDynamicTrackingHelper DynamicTracking;
		DynamicTracking.EnableAndInitialize(Context, Context->MeshesToLoad.Num());
#endif // WITH_EDITOR
		
		for (int MeshIndex = 0; MeshIndex < Context->MeshesToLoad.Num(); ++MeshIndex)
		{
			const FSoftObjectPath& MeshPath = Context->MeshesToLoad[MeshIndex];
			const UObject* MeshObject = MeshPath.ResolveObject();

			FBox Bounds{};

			if (BoundsFromMesh(MeshObject, Bounds))
			{
				Context->MeshToBoundsMap.Add(MeshIndex, MoveTemp(Bounds));
#if WITH_EDITOR
				DynamicTracking.AddToTracking(FPCGSelectionKey::CreateFromPath(MeshPath), /*bIsCulled=*/ false);
#endif // WITH_EDITOR
			}

#if WITH_EDITOR
			DynamicTracking.Finalize(Context);
#endif // WITH_EDITOR
		}

		Context->bBoundsQueried = true;
	}

	while (Context->CurrentExecuteIndex < Inputs.Num())
	{
		const FPCGTaggedData& Input = Inputs[Context->CurrentExecuteIndex++];
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		// If input is not a point data -> passthrough
		const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Input.Data);

		if (!PointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data"));
			continue;
		}

		// Two cases here: either we have a single mesh and need to set its bounds to all points in the input
		// or we need to compare against our list of meshes
		if (Context->SingleMesh.IsValid())
		{
			const UObject* MeshObject = Context->SingleMesh.ResolveObject();
			FBox MeshBounds{};

			if (!BoundsFromMesh(MeshObject, MeshBounds))
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("LoadStaticMeshFailed", "Failed to load StaticMesh or SkeletalMesh"));
				continue;
			}
			
			UPCGBasePointData* OutputData = FPCGContext::NewPointData_AnyThread(Context);

			// Inherit from Input
			OutputData->InitializeFromData(PointData);
			OutputData->SetNumPoints(PointData->GetNumPoints(), /*bInitializeValues=*/false);

			if (OutputData->HasSpatialDataParent())
			{
				OutputData->SetBoundsMin(MeshBounds.Min);
				OutputData->SetBoundsMax(MeshBounds.Max);
			}
			else
			{
				OutputData->AllocateProperties(PointData->GetAllocatedProperties() | EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax);
				OutputData->CopyUnallocatedPropertiesFrom(PointData);

				auto CopyAndSetBounds = [OutputData, PointData, &MeshBounds](const int32 StartReadIndex, const int32 StartWriteIndex, const int32 Count)
				{
					int32 NumWritten = 0;

					const FConstPCGPointValueRanges InRanges(PointData);
					FPCGPointValueRanges OutRanges(OutputData, /*bAllocate=*/false);

					for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
					{
						const int32 WriteIndex = StartWriteIndex + NumWritten;
						OutRanges.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);

						OutRanges.BoundsMinRange[WriteIndex] = MeshBounds.Min;
						OutRanges.BoundsMaxRange[WriteIndex] = MeshBounds.Max;
						++NumWritten;
					}

					check(Count == NumWritten);
					return Count;
				};

				FPCGAsync::AsyncProcessingOneToOneRangeEx(&Context->AsyncState, PointData->GetNumPoints(), /*Initialize=*/[]() {}, CopyAndSetBounds, /*bEnableTimeSlicing=*/false);
			}

			Output.Data = OutputData;
		}
		else
		{
			FPCGBoundsFromMeshContext::InputMeshesData* InputData = Context->PerInputData.FindByPredicate([Index = Context->CurrentExecuteIndex - 1](const FPCGBoundsFromMeshContext::InputMeshesData& InputData) { return InputData.InputIndex == Index; });
			if (InputData)
			{
				UPCGBasePointData* OutputData = FPCGContext::NewPointData_AnyThread(Context);
				OutputData->InitializeFromData(PointData);
				OutputData->SetNumPoints(PointData->GetNumPoints(), /*bInitializeValues=*/false);

				if (!OutputData->HasSpatialDataParent())
				{
					OutputData->AllocateProperties(PointData->GetAllocatedProperties());
				}

				OutputData->AllocateProperties(EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax);
				OutputData->CopyUnallocatedPropertiesFrom(PointData);

				auto CopyAndSetBounds = [OutputData, PointData, Context, InputData](const int32 StartReadIndex, const int32 StartWriteIndex, const int32 Count)
				{
					int32 NumWritten = 0;

					const FConstPCGPointValueRanges InRanges(PointData);
					FPCGPointValueRanges OutRanges(OutputData, /*bAllocate=*/false);

					for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
					{
						const int32 WriteIndex = StartWriteIndex + NumWritten;

						if (!OutputData->HasSpatialDataParent())
						{
							OutRanges.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);
						}

						if (FBox* MatchingBounds = Context->MeshToBoundsMap.Find(InputData->MeshValueIndex[ReadIndex]))
						{
							OutRanges.BoundsMinRange[WriteIndex] = MatchingBounds->Min;
							OutRanges.BoundsMaxRange[WriteIndex] = MatchingBounds->Max;
						}

						++NumWritten;
					}
					check(NumWritten == Count);
					return Count;
				};

				FPCGAsync::AsyncProcessingOneToOneRangeEx(&Context->AsyncState, PointData->GetNumPoints(), /*Initialize=*/[]() {}, CopyAndSetBounds, /*bEnableTimeSlicing=*/false);

				Output.Data = OutputData;
			}
			else
			{
				// The data didn't have the attribute or all meshes were unloadable -> passthrough
				continue;
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
