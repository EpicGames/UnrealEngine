// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGBoundsModifier.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBoundsModifier)

#define LOCTEXT_NAMESPACE "PCGBoundsModifier"

namespace PCGBoundsModifier
{
	// TODO: Evaluate this value for optimization
	// An evolving best guess for the most optimized number of points to operate per thread per slice
	static constexpr int32 PointsPerChunk = 65536;
}

FPCGElementPtr UPCGBoundsModifierSettings::CreateElement() const
{
	return MakeShared<FPCGBoundsModifier>();
}

#if WITH_EDITOR
FText UPCGBoundsModifierSettings::GetNodeTooltipText() const
{
	return LOCTEXT("BoundsModifierNodeTooltip", "Applies a transformation on the point bounds & optionally its steepness.");
}
#endif // WITH_EDITOR

EPCGPointNativeProperties FPCGBoundsModifier::GetPropertiesToAllocate(FPCGContext* InContext) const
{
	return EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax | EPCGPointNativeProperties::Steepness;
}

bool FPCGBoundsModifier::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBoundsModifier::Execute);

	check(Context);
	FPCGBoundsModifier::ContextType* BoundsModifierContext = static_cast<FPCGBoundsModifier::ContextType*>(Context);

	const UPCGBoundsModifierSettings* Settings = Context->GetInputSettings<UPCGBoundsModifierSettings>();
	const FBox SettingsBounds(Settings->BoundsMin, Settings->BoundsMax);
	check(Settings);

	auto SetBoundsLoop = [Settings, &SettingsBounds](UPCGBasePointData* OutputData, int32 StartIndex, int32 Count, TFunctionRef<void(const FBox&, const FBox&, FVector&, FVector&, float&)> BoundsModifierLoop)
	{
		TPCGValueRange<FVector> BoundsMinRange = OutputData->GetBoundsMinValueRange();
		TPCGValueRange<FVector> BoundsMaxRange = OutputData->GetBoundsMaxValueRange();
		TPCGValueRange<float> SteepnessRange = OutputData->GetSteepnessValueRange();
				
		for (int32 Index = StartIndex; Index < (StartIndex + Count); ++Index)
		{
			FVector& BoundsMin = BoundsMinRange[Index];
			FVector& BoundsMax = BoundsMaxRange[Index];
			float& Steepness = SteepnessRange[Index];

			const FBox LocalBounds = PCGPointHelpers::GetLocalBounds(BoundsMin, BoundsMax);

			BoundsModifierLoop(SettingsBounds, LocalBounds, BoundsMin, BoundsMax, Steepness);
		}

		return true;
	};

	switch (Settings->Mode)
	{
		case EPCGBoundsModifierMode::Intersect:
			return ExecutePointOperation(BoundsModifierContext, [SetBoundsLoop, Settings](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetBoundsLoop(OutputData, StartIndex, Count, [Settings](const FBox& SettingsBounds, const FBox& LocalBounds, FVector& OutBoundsMin, FVector& OutBoundsMax, float& OutSteepness)
				{
					PCGPointHelpers::SetLocalBounds(LocalBounds.Overlap(SettingsBounds), OutBoundsMin, OutBoundsMax);

					if (Settings->bAffectSteepness)
					{
						OutSteepness = FMath::Min(OutSteepness, Settings->Steepness);
					}
				});
			}, PCGBoundsModifier::PointsPerChunk);

		case EPCGBoundsModifierMode::Include:
			return ExecutePointOperation(BoundsModifierContext, [SetBoundsLoop, Settings](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetBoundsLoop(OutputData, StartIndex, Count, [Settings](const FBox& SettingsBounds, const FBox& LocalBounds, FVector& OutBoundsMin, FVector& OutBoundsMax, float& OutSteepness)
				{
					PCGPointHelpers::SetLocalBounds(LocalBounds + SettingsBounds, OutBoundsMin, OutBoundsMax);

					if (Settings->bAffectSteepness)
					{
						OutSteepness = FMath::Max(OutSteepness, Settings->Steepness);
					}
				});
			}, PCGBoundsModifier::PointsPerChunk);

		case EPCGBoundsModifierMode::Translate:
			return ExecutePointOperation(BoundsModifierContext, [SetBoundsLoop, Settings](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetBoundsLoop(OutputData, StartIndex, Count, [Settings](const FBox& SettingsBounds, const FBox& LocalBounds, FVector& OutBoundsMin, FVector& OutBoundsMax, float& OutSteepness)
				{
					OutBoundsMin += Settings->BoundsMin;
					OutBoundsMax += Settings->BoundsMax;

					if (Settings->bAffectSteepness)
					{
						OutSteepness = FMath::Clamp(OutSteepness + Settings->Steepness, 0.0f, 1.0f);
					}
				});
			}, PCGBoundsModifier::PointsPerChunk);

		case EPCGBoundsModifierMode::Scale:
			return ExecutePointOperation(BoundsModifierContext, [SetBoundsLoop, Settings](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetBoundsLoop(OutputData, StartIndex, Count, [Settings](const FBox& SettingsBounds, const FBox& LocalBounds, FVector& OutBoundsMin, FVector& OutBoundsMax, float& OutSteepness)
				{
					OutBoundsMin *= Settings->BoundsMin;
					OutBoundsMax *= Settings->BoundsMax;

					if (Settings->bAffectSteepness)
					{
						OutSteepness = FMath::Clamp(OutSteepness * Settings->Steepness, 0.0f, 1.0f);
					}
				});
			}, PCGBoundsModifier::PointsPerChunk);

		case EPCGBoundsModifierMode::Set:
			return ExecutePointOperation(BoundsModifierContext, [SetBoundsLoop, Settings](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetBoundsLoop(OutputData, StartIndex, Count, [Settings](const FBox& SettingsBounds, const FBox& LocalBounds, FVector& OutBoundsMin, FVector& OutBoundsMax, float& OutSteepness)
				{
					PCGPointHelpers::SetLocalBounds(SettingsBounds, OutBoundsMin, OutBoundsMax);

					if (Settings->bAffectSteepness)
					{
						OutSteepness = Settings->Steepness;
					}
				});
			}, PCGBoundsModifier::PointsPerChunk);

		default:
			checkNoEntry();
			return true;
	}
}

#undef LOCTEXT_NAMESPACE
