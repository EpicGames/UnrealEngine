// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCleanSpline.h"

#include "PCGContext.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGPolygon2DData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCleanSpline)

#define LOCTEXT_NAMESPACE "PCGCleanSplineElement"

namespace PCGCleanSplineHelpers
{
	bool VectorsAreCollinear(const FVector& FirstVector, const FVector& SecondVector, const double Threshold)
	{
		return FMath::Abs(FirstVector.GetSafeNormal() | SecondVector.GetSafeNormal()) >= Threshold;
	}

	bool ControlPointsAreColocated(
		const FVector& Point1, 
		const FVector& Point2, 
		const FTransform& SplineTransform, 
		const double Threshold, 
		const bool bUseLocalSpace = false)
	{
		const FVector Location1 = bUseLocalSpace ? Point1: SplineTransform.TransformPosition(Point1);
		const FVector Location2 = bUseLocalSpace ? Point2: SplineTransform.TransformPosition(Point2);
		return FVector::DistSquared(Location1, Location2) < Threshold * Threshold;
	}

	bool ControlPointsAreCollinear(const FVector& Point1, const FVector& Point2, const FVector& Point3, const double Threshold)
	{
		return VectorsAreCollinear(Point2 - Point1, Point3 - Point2, Threshold);
	}

	// Implementation note: Co-located points will have a vector dot product of zero, regardless of tangents, and will be thus collinear.
	bool ControlPointsAreCollinear(const FSplinePoint& Point1, const FSplinePoint& Point2, const FSplinePoint& Point3, const double Threshold)
	{
		const FVector Segment = Point3.Position - Point1.Position;

		/* Need to check all four tangents against the segment to guarantee collinearity. Note: This works for linear
		 * segments only. It is possible to find more control points on curves that would have no effect on the final
		 * result, but it would be extremely rare for a user to wind up in that situation. It would also provide little
		 * to no benefit to do so as well.
		 */
		return (
			VectorsAreCollinear(Point1.LeaveTangent, Segment, Threshold) &&
			VectorsAreCollinear(Point2.ArriveTangent, Segment, Threshold) &&
			VectorsAreCollinear(Point2.LeaveTangent, Segment, Threshold) &&
			VectorsAreCollinear(Point3.ArriveTangent, Segment, Threshold)
		);
	}
}

FPCGElementPtr UPCGCleanSplineSettings::CreateElement() const
{
	return MakeShared<FPCGCleanSplineElement>();
}

#if WITH_EDITOR
void UPCGCleanSplineSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGCleanSplineSettings, bUseRadians))
		{
			CollinearAngleThreshold = bUseRadians ? FMath::DegreesToRadians(CollinearAngleThreshold) : FMath::RadiansToDegrees(CollinearAngleThreshold);
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGCleanSplineSettings, CollinearAngleThreshold))
		{
			static constexpr double MaxCollinearAngleToleranceDegrees = 89;
			CollinearAngleThreshold = FMath::Clamp(CollinearAngleThreshold, 0, bUseRadians ? FMath::DegreesToRadians(MaxCollinearAngleToleranceDegrees) : MaxCollinearAngleToleranceDegrees);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGCleanSplineSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spline | EPCGDataType::Polygon2D).SetRequiredPin();
	return Properties;
}

TArray<FPCGPinProperties> UPCGCleanSplineSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spline | EPCGDataType::Polygon2D);
	return Properties;
}

bool FPCGCleanSplineElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCleanSplineElement::Execute);

	check(InContext);

	const UPCGCleanSplineSettings* Settings = InContext->GetInputSettings<UPCGCleanSplineSettings>();
	check(Settings);

	// Nothing to do. Forward the output.
	if (!Settings->bFuseColocatedControlPoints && !Settings->bRemoveCollinearControlPoints)
	{
		InContext->OutputData = InContext->InputData;
		PCGLog::LogWarningOnGraph(LOCTEXT("NoOperation", "No Clean Spline operations selected. Input will be forwarded"), InContext);
		return true;
	}

	for (const FPCGTaggedData& InputData : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if (const UPCGSplineData* InputSplineData = Cast<UPCGSplineData>(InputData.Data))
		{
			ProcessSpline(InContext, Settings, InputData, InputSplineData);
		}
		else if (const UPCGPolygon2DData* InputPolygonData = Cast<UPCGPolygon2DData>(InputData.Data))
		{
			ProcessPolygon(InContext, Settings, InputData, InputPolygonData);
		}
	}

	return true;
}

void FPCGCleanSplineElement::ProcessSpline(FPCGContext* InContext, const UPCGCleanSplineSettings* Settings, const FPCGTaggedData& InputData, const UPCGSplineData* InputSplineData) const
{
	check(InContext && Settings && InputSplineData);

	// Early out on splines that are already invalid
	if (InputSplineData->GetNumSegments() < 1)
	{
		return;
	}

	// Pre-calculate the tolerance value from the user defined tolerance in radians/degrees to cross-product comparable value.
	const double DotProductToleranceRad = Settings->bUseRadians ? Settings->CollinearAngleThreshold : FMath::DegreesToRadians(Settings->CollinearAngleThreshold);
	// Max the tolerance with an epsilon to accomodate rounding errors.
	const double DotProductTolerance = FMath::Max(FMath::Abs(FMath::Cos(DotProductToleranceRad)), UE_DOUBLE_SMALL_NUMBER);

	const FTransform& SplineTransform = InputSplineData->SplineStruct.Transform;
	const FInterpCurveVector& ControlPointsPosition = InputSplineData->SplineStruct.GetSplinePointsPosition();
	const FInterpCurveQuat& ControlPointsRotation = InputSplineData->SplineStruct.GetSplinePointsRotation();
	const FInterpCurveVector& ControlPointsScale = InputSplineData->SplineStruct.GetSplinePointsScale();
	const TConstArrayView<PCGMetadataEntryKey> ControlPointsMetadataEntry = InputSplineData->SplineStruct.GetConstControlPointsEntryKeys();
	const int32 NumControlPoints = ControlPointsPosition.Points.Num();
	bool bControlPointWasRemoved = false;
	const bool bIsClosed = InputSplineData->IsClosed();

	TArray<FSplinePoint> ControlPoints;
	ControlPoints.Reserve(NumControlPoints);

	// Making sure that the number of metadata entry keys matches, if not, we are in an invalid state and will reset the metadata entry keys.
	TArray<PCGMetadataEntryKey> ControlPointsKeys;
	if (ControlPointsMetadataEntry.Num() == NumControlPoints)
	{
		ControlPointsKeys.Append(ControlPointsMetadataEntry);
	}

	// For code clarity, generate the points first and remove them as needed.
	for (int32 i = 0; i < NumControlPoints; ++i)
	{
		/* Implementation note: Decay to custom tangents. They assist the user in building the spline, but the
		 * interpolation modes will affect the recalculations unpredictably when control points are removed.
		 */
		ESplinePointType::Type ControlPointType = ConvertInterpCurveModeToSplinePointType(ControlPointsPosition.Points[i].InterpMode);

		if ((ControlPointType == ESplinePointType::Curve || ControlPointType == ESplinePointType::CurveClamped || ControlPointType == ESplinePointType::CurveCustomTangent))
		{
			ControlPointType = ESplinePointType::CurveCustomTangent;
		}

		ControlPoints.Emplace(/*InputKey=*/i,
			ControlPointsPosition.Points[i].OutVal,
			ControlPointsPosition.Points[i].ArriveTangent,
			ControlPointsPosition.Points[i].LeaveTangent,
			ControlPointsRotation.Points[i].OutVal.Rotator(),
			ControlPointsScale.Points[i].OutVal,
			ControlPointType);
	}

	auto RemovePoint = [&ControlPoints, &ControlPointsKeys, &bControlPointWasRemoved](int Index)
		{
			if (!ControlPointsKeys.IsEmpty())
			{
				check(ControlPointsKeys.IsValidIndex(Index));
				ControlPointsKeys.RemoveAt(Index, EAllowShrinking::No);
			}

			check(ControlPoints.IsValidIndex(Index));
			ControlPoints.RemoveAt(Index, EAllowShrinking::No);
			bControlPointWasRemoved = true;
		};

	auto GetPreviousIndexFromOffset = [&ControlPoints](int32 Index, int32 Offset)
		{
			if (!ensure(!ControlPoints.IsEmpty()))
			{
				return 0;
			}

			int32 Result = Index - Offset;
			while (Result < 0)
			{
				Result += ControlPoints.Num();
			}

			return Result;
		};

	if (Settings->bFuseColocatedControlPoints)
	{
		const int32 MinIndex = bIsClosed ? 0 : 1;

		// Will evaluate by pairs. Reverse order for optimizing RemoveAt.
		for (int Index = ControlPoints.Num() - 1; Index >= MinIndex; --Index)
		{
			const int32 PreviousPointIndex = GetPreviousIndexFromOffset(Index, 1);
			const int32 CurrentPointIndex = Index;

			// Evaluate by pairs for colocated.
			if (PCGCleanSplineHelpers::ControlPointsAreColocated(
				ControlPoints[PreviousPointIndex].Position,
				ControlPoints[CurrentPointIndex].Position,
				SplineTransform,
				Settings->ColocationDistanceThreshold,
				Settings->bUseSplineLocalSpace))
			{
				EPCGControlPointFuseMode FuseMode = Settings->FuseMode;
				// Generally, keep the previous control point (first), but preserve the final control point to maintain the spline's length (second) in non-closed splines.
				if (FuseMode == EPCGControlPointFuseMode::Auto)
				{
					FuseMode = (!bIsClosed && CurrentPointIndex == ControlPoints.Num() - 1) ? EPCGControlPointFuseMode::KeepSecond : EPCGControlPointFuseMode::KeepFirst;
				}

				switch (FuseMode)
				{
					// Keep the first control point
				case EPCGControlPointFuseMode::KeepFirst:
					ControlPoints[PreviousPointIndex].LeaveTangent = ControlPoints[CurrentPointIndex].LeaveTangent;
					RemovePoint(CurrentPointIndex);
					break;
					// Keep the second control point
				case EPCGControlPointFuseMode::KeepSecond:
					ControlPoints[CurrentPointIndex].ArriveTangent = ControlPoints[PreviousPointIndex].ArriveTangent;
					RemovePoint(PreviousPointIndex);
					break;
					// Average the two control points' transforms and update the leave tangent
				case EPCGControlPointFuseMode::Merge:
					ControlPoints[PreviousPointIndex].Position = FMath::Lerp(ControlPoints[PreviousPointIndex].Position, ControlPoints[CurrentPointIndex].Position, 0.5);
					ControlPoints[PreviousPointIndex].Rotation = FMath::Lerp(ControlPoints[PreviousPointIndex].Rotation, ControlPoints[CurrentPointIndex].Rotation, 0.5);
					ControlPoints[PreviousPointIndex].Scale = FMath::Lerp(ControlPoints[PreviousPointIndex].Scale, ControlPoints[CurrentPointIndex].Scale, 0.5);
					ControlPoints[PreviousPointIndex].LeaveTangent = ControlPoints[CurrentPointIndex].LeaveTangent;
					RemovePoint(CurrentPointIndex);
					break;
				case EPCGControlPointFuseMode::Auto: // Should've been picked up by now. Fallthrough...
					checkNoEntry();
				default:
					break;
				}
			}
		}
	}

	if (Settings->bRemoveCollinearControlPoints)
	{
		const int32 MinIndex = bIsClosed ? 0 : 2;

		// Will evaluate by triplets. Reverse order for optimizing RemoveAt.
		for (int Index = ControlPoints.Num() - 1; Index >= MinIndex; --Index)
		{
			const int32 SecondPreviousPointIndex = GetPreviousIndexFromOffset(Index, 2);
			const int32 PreviousPointIndex = GetPreviousIndexFromOffset(Index, 1);
			const int32 CurrentPointIndex = Index;

			if (PCGCleanSplineHelpers::ControlPointsAreCollinear(
				ControlPoints[SecondPreviousPointIndex],
				ControlPoints[PreviousPointIndex],
				ControlPoints[CurrentPointIndex],
				DotProductTolerance))
			{
				ControlPoints[SecondPreviousPointIndex].LeaveTangent = ControlPoints[CurrentPointIndex].Position - ControlPoints[SecondPreviousPointIndex].Position;
				ControlPoints[CurrentPointIndex].ArriveTangent = ControlPoints[SecondPreviousPointIndex].LeaveTangent;
				RemovePoint(PreviousPointIndex);
			}
		}
	}

	FPCGTaggedData& Output = InContext->OutputData.TaggedData.Emplace_GetRef(InputData);

	// We only need to create a new data if a point was removed.
	if (bControlPointWasRemoved)
	{
		// Update input keys if a point was removed, to keep them monotonically incremental
		for (int i = 0; i < ControlPoints.Num(); ++i)
		{
			ControlPoints[i].InputKey = i;
		}

		UPCGSplineData* NewSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(InContext);
		NewSplineData->InitializeFromData(InputSplineData);
		NewSplineData->Initialize(ControlPoints, InputSplineData->IsClosed(), InputSplineData->GetTransform(), std::move(ControlPointsKeys));

		Output.Data = NewSplineData;
	}
}

void FPCGCleanSplineElement::ProcessPolygon(FPCGContext* Context, const UPCGCleanSplineSettings* Settings, const FPCGTaggedData& Input, const UPCGPolygon2DData* PolygonData) const
{
	check(Context && Settings && PolygonData);

	// Pre-calculate the tolerance value from the user defined tolerance in radians/degrees to cross-product comparable value.
	const double DotProductToleranceRad = Settings->bUseRadians ? Settings->CollinearAngleThreshold : FMath::DegreesToRadians(Settings->CollinearAngleThreshold);
	// Max the tolerance with an epsilon to accomodate rounding errors.
	const double DotProductTolerance = FMath::Max(FMath::Abs(FMath::Cos(DotProductToleranceRad)), UE_DOUBLE_SMALL_NUMBER);

	const EPCGControlPointFuseMode FuseMode = Settings->FuseMode;

	// Get data from the polygon
	const UE::Geometry::FGeneralPolygon2d& OriginalPolygon = PolygonData->GetPolygon();
	const FTransform Transform = PolygonData->GetTransform();

	// Early out if polygon is invalid
	if (OriginalPolygon.GetOuter().VertexCount() < 3)
	{
		return;
	}

	// Copy the polygon (outer + holes)
	TArray<UE::Geometry::FPolygon2d> WorkingPolygons;
	WorkingPolygons.Add(OriginalPolygon.GetOuter());
	WorkingPolygons.Append(OriginalPolygon.GetHoles());
	
	// Get the metadata entry keys
	TArray<PCGMetadataEntryKey> MetadataEntryKeys;
	MetadataEntryKeys.Append(PolygonData->GetConstVerticesEntryKeys());

	bool bHasChangedData = false;
	int VertexOffset = 0;

	// For all polygons...
	for (int PolygonIndex = 0; PolygonIndex < WorkingPolygons.Num(); ++PolygonIndex)
	{
		UE::Geometry::FPolygon2d& Polygon = WorkingPolygons[PolygonIndex];

		auto RemoveVertex = [&bHasChangedData, &Polygon, &MetadataEntryKeys, &VertexOffset](int Index)
		{
			bHasChangedData = true;
			Polygon.RemoveVertex(Index);
			if (!MetadataEntryKeys.IsEmpty())
			{
				MetadataEntryKeys.RemoveAt(Index + VertexOffset);
			}
		};

		// Start by removing colocated points
		int VertexIndex = 0;
		int CurrentMergeWeight = 2;

		while (VertexIndex < Polygon.VertexCount() && Polygon.VertexCount() > 3)
		{
			const int NextVertexIndex = (VertexIndex + 1) % Polygon.VertexCount();

			if (PCGCleanSplineHelpers::ControlPointsAreColocated(
				FVector(Polygon[VertexIndex], 0.0),
				FVector(Polygon[NextVertexIndex], 0.0),
				Transform,
				Settings->ColocationDistanceThreshold,
				Settings->bUseSplineLocalSpace))
			{
				switch (FuseMode)
				{
				case EPCGControlPointFuseMode::Auto: // auto is always keep first in polygons
				case EPCGControlPointFuseMode::KeepFirst:
					RemoveVertex(NextVertexIndex);
					break;
				case EPCGControlPointFuseMode::KeepSecond:
					RemoveVertex(VertexIndex);
					break;
				case EPCGControlPointFuseMode::Merge:
					Polygon[VertexIndex] = FMath::Lerp(Polygon[VertexIndex], Polygon[NextVertexIndex], 1.0 / static_cast<double>(CurrentMergeWeight++));
					// @todo_pcg: lerp metadata
					RemoveVertex(NextVertexIndex);
					break;
				}
			}
			else
			{
				++VertexIndex;
				CurrentMergeWeight = 2;
			}
		}

		// Then remove collinear points
		VertexIndex = 0;
		while (VertexIndex < Polygon.VertexCount() && Polygon.VertexCount() > 3)
		{
			const int PreviousVertexIndex = (VertexIndex + Polygon.VertexCount() - 1) % Polygon.VertexCount();
			const int NextVertexIndex = (VertexIndex + 1) % Polygon.VertexCount();

			if (PCGCleanSplineHelpers::ControlPointsAreCollinear(
				FVector(Polygon[PreviousVertexIndex], 0.0),
				FVector(Polygon[VertexIndex], 0.0),
				FVector(Polygon[NextVertexIndex], 0.0),
				DotProductTolerance))
			{
				// @todo_pcg: Perform additional validation for metadata similarity, e.g. lerping between properties of previous vs next gives out similar properties to current vertex
				RemoveVertex(VertexIndex);
			}
			else
			{
				++VertexIndex;
			}
		}

		// @todo_pcg: final step for holes that could be removed if under a certain area/if all 3 remaining points are colocated.
		// ...

		VertexOffset += Polygon.VertexCount();
	}

	// Add to the output (even if trivial)
	FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef(Input);

	// If we've made changes, create the data and push it on the output
	if (bHasChangedData)
	{
		UPCGPolygon2DData* NewPolygonData = FPCGContext::NewObject_AnyThread<UPCGPolygon2DData>(Context);
		NewPolygonData->InitializeFromData(PolygonData);
		Output.Data = NewPolygonData;

		UE::Geometry::FGeneralPolygon2d OutputPolygon(MoveTemp(WorkingPolygons[0]));
		for (int HoleIndex = 1; HoleIndex < WorkingPolygons.Num(); ++HoleIndex)
		{
			OutputPolygon.AddHole(MoveTemp(WorkingPolygons[HoleIndex]), /*bCheckContainment=*/false, /*bCheckOrientation=*/false);
		}

		TConstArrayView<PCGMetadataEntryKey> EntryKeysView(MetadataEntryKeys);
		NewPolygonData->SetPolygon(MoveTemp(OutputPolygon), EntryKeysView.IsEmpty() ? nullptr : &EntryKeysView);
		NewPolygonData->SetTransform(Transform);
	}
}

#undef LOCTEXT_NAMESPACE
