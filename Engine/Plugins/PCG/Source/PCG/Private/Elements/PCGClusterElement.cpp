// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGClusterElement.h"

#include "PCGContext.h"
#include "Algo/Accumulate.h"
#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGClusterElement)

#define LOCTEXT_NAMESPACE "PCGClusterElement"

namespace PCGClusterElement
{
	namespace Constants
	{
		const FName InputCentroidsLabel = TEXT("Initial Centroids");
		const FName FinalCentroidsLabel = TEXT("Final Centroids");
		const FName ClusterIDAttributeDefaultName = TEXT("ClusterID");
		const FName FinalCentroidElementCountAttributeDefaultName = TEXT("ClusterElementCount");
	}

	namespace Helpers
	{
		FMatrix OuterProduct(const FVector& V)
		{
			return FMatrix
			{
				{V.X * V.X, V.X * V.Y, V.X * V.Z, 0},
				{V.Y * V.X, V.Y * V.Y, V.Y * V.Z, 0},
				{V.Z * V.X, V.Z * V.Y, V.Z * V.Z, 0},
				{0, 0, 0, 1}
			};
		}
	} // namespace Helpers

	namespace Algorithm
	{
		/* Method of vector quantization to try and partition a number of points into a selected number of clusters.
		 * Reference: https://en.wikipedia.org/wiki/K-means_clustering
		 */
		bool KMeans(FPCGContext* Context, FClusteringData& ClusteringData, const int MaxIterations)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::KMeans);

			// Set up a few references to state data.
			uint8& ProcessingStage = ClusteringData.ProcessingStage;
			const TConstPCGValueRange<FTransform> TransformRange = ClusteringData.OutputData->GetConstTransformValueRange();
			const TArray<FVector, TInlineAllocator<16>>& Centroids = ClusteringData.Centroids;
			TArray<int32>& ClusterAssignments = ClusteringData.PointToClusterAssignments;

			while (ClusteringData.Iteration < MaxIterations)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::KMeans::IterationLoop);

				// Find the centroid that best represents each point.
				auto ProcessPoints = [&TransformRange, &Centroids, &bConverged = ClusteringData.bConverged](const int32& PointIndex, int32& InOutCentroidIndex)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::KMeans::ProcessPoints);

					/** Start with the previously selected centroid. The distance will need to be recalculated in case the
					 * centroid's location changed on the last iteration.
					 * Note: The centroids should have all been originally initialized to the first cluster (0) for the
					 * first iteration.
					 */
					int CentroidIndex = InOutCentroidIndex;
					double MinDistanceSquared = FVector::DistSquared(TransformRange[PointIndex].GetLocation(), Centroids[InOutCentroidIndex]);
					for (CentroidIndex = 0; CentroidIndex < Centroids.Num(); ++CentroidIndex)
					{
						// Stabilizes against flipping centroids in the event of equidistant points
						if (CentroidIndex == InOutCentroidIndex)
						{
							continue;
						}

						const double DistanceSquared = FVector::DistSquared(TransformRange[PointIndex].GetLocation(), Centroids[CentroidIndex]);
						if (DistanceSquared < MinDistanceSquared)
						{
							MinDistanceSquared = DistanceSquared;
							InOutCentroidIndex = CentroidIndex;
							// Only flag non-convergence if the cluster assignment has changed.
							bConverged = false;
						}
					}

					return true;
				};

				// Will alternate stages until convergence is found or the max iterations are reached.
				if (ProcessingStage == 1) // Stage 1 - Process Points
				{
					if (!FPCGAsync::AsyncProcessing(&Context->AsyncState, ClusterAssignments.Num(), ClusterAssignments, ProcessPoints, /*bEnableTimeSlicing=*/true, /*ChunkSize=*/64))
					{
						return false;
					}

					if (ClusteringData.bConverged)
					{
						break; // We're done.
					}
					else // Go to Stage 2 - Process Centroids
					{
						ClusteringData.bConverged = true; // Reset until it's recomputed in Stage 1.
						ProcessingStage = 2;
					}
				}

				// Update the centroids with newly added or removed points
				auto ProcessCentroids = [TransformRange, ClusterAssignments](const int32 CentroidIndex, FVector& OutResult)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::KMeans::ProcessCentroids);

					FVector CentroidSum = FVector::ZeroVector;
					int Count = 0;

					for (int PointIndex = 0; PointIndex < TransformRange.Num(); ++PointIndex)
					{
						if (ClusterAssignments[PointIndex] == CentroidIndex)
						{
							CentroidSum += TransformRange[PointIndex].GetLocation();
							++Count;
						}
					}

					if (Count > 0)
					{
						// Average location of the points
						OutResult = CentroidSum / Count;
					}

					return true;
				};

				// TODO: This can be better optimized for async, by having workers build partial sums for all/one cluster at a time and synchronizing at the end.
				// Stage 2 - Process Centroids
				if (!FPCGAsync::AsyncProcessing(&Context->AsyncState, ClusteringData.Centroids.Num(), ClusteringData.Centroids, ProcessCentroids, /*bEnableTimeSlicing=*/true, /*ChunkSize=*/1))
				{
					return false;
				}

				++ClusteringData.Iteration;
				ProcessingStage = 1; // Stage 1 - Process Points
			}

			return true;
		}

		/* Expectation-Maximization: An iterative method to find the local maximum likelihood estimates for clustering.
		 * Reference: https://en.wikipedia.org/wiki/Expectation-maximization_algorithm
		 */
		bool ExpectationMaximization(FPCGContext* Context, FClusteringData& ClusteringData, const int MaxIterations)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::ExpectationMaximization);

			// Set up a few references to state data
			uint8& ProcessingStage = ClusteringData.ProcessingStage;
			const TArray<FVector>& Points = ClusteringData.EMData.NormalizedPointLocations;
			TArray<int32>& ClusterAssignments = ClusteringData.PointToClusterAssignments;

			using EM = FClusteringData::FExpectationMaximizationData;
			TArray<EM::FGaussian, TInlineAllocator<16>>& Gaussians = ClusteringData.EMData.Gaussians;
			EM::FProbabilityArray& Probabilities = ClusteringData.EMData.Probabilities;
			std::atomic<double>& LogLikelihoodSum = ClusteringData.EMData.LogLikelihoodSum;
			double& PreviousLogLikelihood = ClusteringData.EMData.PreviousLogLikelihood;

			// Process until convergence via Log Likelihood or max iterations is reached.
			while (!ClusteringData.bConverged && ClusteringData.Iteration < MaxIterations)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::ExpectationMaximization::IterationLoop);
				if (ProcessingStage == 1) // Stage 1 - Expectation Step
				{
					// Gaussian Probability Density Function - The likelihood of a point taking a particular value in a continuous distribution.
					auto GPDFunction = [](const FVector& Point, const FVector& Mean, const FMatrix& InvCovariance, const double NormalizationFactor)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::ExpectationMaximization::GPDFunction);

						const FVector Diff = Point - Mean;
						// TODO: Could likely be optimized to handle the dot product and transform in a single step.
						const double ExpTerm = -0.5 * (Diff | InvCovariance.TransformVector(Diff));

						return NormalizationFactor * FMath::Exp(ExpTerm);
					};

					auto ProcessExpectationStep = [&Points, &Probabilities, &Gaussians, &LogLikelihoodSum, GPDFunction](const int32 StartIndex, int32, const int32 Count)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::ExpectationMaximization::ProcessExpectationStep);

						const int32 EndIndex = StartIndex + Count;
						double LocalLikelihoodSum = 0;

						for (int PointIndex = StartIndex; PointIndex < EndIndex; ++PointIndex)
						{
							double Sum = 0.0;
							for (int i = 0; i < Gaussians.Num(); ++i)
							{
								if (Gaussians[i].bDead)
								{
									continue;
								}

								double Probability = GPDFunction(Points[PointIndex], Gaussians[i].Mean, Gaussians[i].InvCovariance, Gaussians[i].NormalizationFactor);
								Probability *= Gaussians[i].Weight;
								Probabilities[PointIndex][i] = Probability;
								Sum += Probability;
							}

							/* The probabilities may just be extremely small for very far away points. We can
							 * artificially bump them up a little to avoid precision errors without any problems.
							 */
							Sum = FMath::Max(Sum, UE_DOUBLE_SMALL_NUMBER);

							for (int i = 0; i < Gaussians.Num(); ++i)
							{
								Probabilities[PointIndex][i] /= Sum;
							}

							LocalLikelihoodSum += FMath::Loge(Sum);
						}

						// Atomically += the overall sum. Could be LogLikelihoodSum += LocalLikelihoodSum; once C++20 is enabled.
						for (double l = LogLikelihoodSum.load(); !LogLikelihoodSum.compare_exchange_strong(l, l + LocalLikelihoodSum);) {}

						return Count;
					};

					// Implementation Note: This lambda executes independently at conclusion of the expectation stage.
					auto UpdateLogLikelihood = [&PreviousLogLikelihood, &LogLikelihoodSum, &bConverged = ClusteringData.bConverged, Tolerance = ClusteringData.EMData.Tolerance](int32)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::ExpectationMaximization::UpdateLogLikelihood);

						if (FMath::Abs(PreviousLogLikelihood - LogLikelihoodSum) < Tolerance)
						{
							bConverged = true;
						}

						PreviousLogLikelihood = LogLikelihoodSum;
						LogLikelihoodSum = 0.0;
					};

					// Run Expectation Stage
					if (!FPCGAsync::AsyncProcessingRangeEx(
						&Context->AsyncState,
						Points.Num(),
						/*Initialize=*/[] {},
						ProcessExpectationStep,
						/*MoveDataRange=*/[](int32, int32, int32) {},
						/*Finished=*/UpdateLogLikelihood,
						/*bEnableTimeSlicing=*/true,
						/*ChunkSize=*/256))
					{
						return false;
					}

					// If we converged in the E-Step, we can stop iterating.
					if (ClusteringData.bConverged)
					{
						break;
					}
					else // Go to Stage 2 - Maximization Step
					{
						ProcessingStage = 2;
					}
				}

				if (ProcessingStage == 2) // Stage 2 - Maximization Step
				{
					auto ProcessMaximizationStep = [&Points, &Probabilities](const int32 GaussianIndex, EM::FGaussian& OutGaussian)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::ExpectationMaximization::ProcessMaximizationStep);

						if (OutGaussian.bDead)
						{
							return true;
						}

						double WeightSum = 0.0;
						FVector MeanSum = FVector::ZeroVector;
						FMatrix CovarianceSum = FMatrix::Identity;

						for (int i = 0; i < Points.Num(); ++i)
						{
							MeanSum += Probabilities[i][GaussianIndex] * Points[i];
							WeightSum += Probabilities[i][GaussianIndex];
						}

						// The gaussian is far enough to have nearly no impact on any points. Remove it from further calculation.
						if (FMath::IsNearlyZero(WeightSum))
						{
							OutGaussian.bDead = true;
							return true;
						}

						const double WeightFactor = 1.0 / WeightSum;
						OutGaussian.Mean = MeanSum * WeightFactor;
						OutGaussian.Weight = WeightSum / Points.Num();

						for (int i = 0; i < Points.Num(); ++i)
						{
							const FVector Difference = Points[i] - OutGaussian.Mean;
							CovarianceSum += Helpers::OuterProduct(Difference) * Probabilities[i][GaussianIndex];
						}

						OutGaussian.Covariance = CovarianceSum * WeightFactor;
						if (!ensure(!OutGaussian.Covariance.ContainsNaN()))
						{
							// If we have an instability, clamp it back to something insignificant and then remove it.
							OutGaussian.Covariance = FMatrix::Identity * UE_DOUBLE_SMALL_NUMBER;
							OutGaussian.bDead = true;
						}

						// To account for 4D homogenous FMatrix with non-homogenous 3D points.
						OutGaussian.Covariance.M[3][3] = 1;
						OutGaussian.InvCovariance = OutGaussian.Covariance.Inverse();

						// Normalization factor tailored for 3D: 1 / ((2 * pi) ^ (3 / 2)) * |sigma| ^ (1 / 2)), where sigma = M_covariance
						static double constexpr NormalizationFactor3DConstant = 15.74960994572241974429064599;
						OutGaussian.NormalizationFactor = 1.0 / (NormalizationFactor3DConstant * FMath::Sqrt(OutGaussian.Covariance.Determinant()));

						return true;
					};

					// Run Maximization Stage
					if (!FPCGAsync::AsyncProcessing(&Context->AsyncState, Gaussians.Num(), Gaussians, ProcessMaximizationStep, /*bEnableTimeSlicing=*/true, /*ChunkSize=*/1))
					{
						return false;
					}
					else // Go back to Stage 1 - Expectation Step
					{
						checkSlow(FMath::IsNearlyEqual(1.0, Algo::Accumulate(Gaussians, 0.0, [](double PreviousSum, const EM::FGaussian& Gaussian) { return PreviousSum + Gaussian.Weight; })));
						ProcessingStage = 1;
					}
				}

				++ClusteringData.Iteration;
			}

			// Assign clusters to the points.
			auto ProcessClusterAssignment = [&Probabilities](const int32 PointIndex, int32& OutClusterAssignment)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::ExpectationMaximization::ProcessClusterAssignment);

				double MaxProb = Probabilities[PointIndex][0];
				int32 ClusterAssignment = 0;

				for (int CentroidIndex = 1; CentroidIndex < Probabilities[PointIndex].Num(); ++CentroidIndex)
				{
					if (Probabilities[PointIndex][CentroidIndex] > MaxProb)
					{
						ClusterAssignment = CentroidIndex;
						MaxProb = Probabilities[PointIndex][CentroidIndex];
					}
				}

				OutClusterAssignment = ClusterAssignment;

				return true;
			};

			if (!FPCGAsync::AsyncProcessing(&Context->AsyncState, ClusterAssignments.Num(), ClusterAssignments, ProcessClusterAssignment, /*bEnableTimeSlicing=*/true, /*ChunkSize=*/2048))
			{
				return false;
			}

			return true;
		}
	} // namespace Algorithm
} // namespace PCGClusterElement

UPCGClusterSettings::UPCGClusterSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ClusterAttribute.SetAttributeName(PCGClusterElement::Constants::ClusterIDAttributeDefaultName);
	FinalCentroidElementCountAttribute.SetAttributeName(PCGClusterElement::Constants::FinalCentroidElementCountAttributeDefaultName);
}

#if WITH_EDITOR
FText UPCGClusterSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Given a desired number of clusters (categories), find the best fit cluster for each point by distance, using one of various clustering algorithms.");
}
#endif // WITH_EDITOR

bool UPCGClusterSettings::UseSeed() const
{
	return true;
}

TArray<FPCGPinProperties> UPCGClusterSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point).SetRequiredPin();

	// TODO: For now, currently only support one input for N:1, but N:N would be useful too.
	FPCGPinProperties& InputCentroidsPin = Properties.Emplace_GetRef(PCGClusterElement::Constants::InputCentroidsLabel, EPCGDataType::Point, /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false);
#if WITH_EDITOR
	InputCentroidsPin.Tooltip = LOCTEXT("InputCentroidsTooltip", "(Optional) Points whose locations will be used as both the quantity and location of the initial centroids for the clustering algorithm. If none provided, random points will be used as centroids.");
#endif // WITH_EDITOR

	return Properties;
}

TArray<FPCGPinProperties> UPCGClusterSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);

	if (bOutputFinalCentroids)
	{
		FPCGPinProperties& FinalCentroids = Properties.Emplace_GetRef(PCGClusterElement::Constants::FinalCentroidsLabel, EPCGDataType::Point);
#if WITH_EDITOR
		FinalCentroids.Tooltip = LOCTEXT("FinalCentroidsTooltip", "The final locations of the centroids/gaussians.");
#endif // WITH_EDITOR
	}

	return Properties;
}

FPCGElementPtr UPCGClusterSettings::CreateElement() const
{
	return MakeShared<FPCGClusterElement>();
}

bool FPCGClusterElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::PrepareData);

	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context);
	const UPCGClusterSettings* Settings = Context->GetInputSettings<UPCGClusterSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> PointInputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	// Early out if no source to modify
	if (PointInputs.IsEmpty())
	{
		return true;
	}

	const TArray<FPCGTaggedData> CentroidInputs = Context->InputData.GetInputsByPin(PCGClusterElement::Constants::InputCentroidsLabel);

	Context->InitializePerExecutionState([&CentroidInputs, Settings](const ContextType* Context, ExecStateType& OutState)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::PrepareData::InitializePerExecutionState);

		switch (Settings->Algorithm)
		{
			case EPCGClusterAlgorithm::KMeans:
				OutState.ClusteringFunction = PCGClusterElement::Algorithm::KMeans;
				break;
			case EPCGClusterAlgorithm::EM:
				OutState.ClusteringFunction = PCGClusterElement::Algorithm::ExpectationMaximization;
				break;
			default:
				checkNoEntry();
				return EPCGTimeSliceInitResult::AbortExecution;
		}
		check(OutState.ClusteringFunction);

		if (CentroidInputs.IsEmpty())
		{
			OutState.bSelectRandomCentroids = true;
			// Centroids will be initialized on a per-iteration input basis, but we can allocate them here.
			OutState.InitialCentroids.SetNum(Settings->NumClusters);
		}
		else
		{
			OutState.bSelectRandomCentroids = false;
			if (CentroidInputs.Num() > 1)
			{
				PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGClusterElement::Constants::InputCentroidsLabel, Context);
			}

			const UPCGBasePointData* CentroidPoints = Cast<UPCGBasePointData>(CentroidInputs[0].Data);
			if (!CentroidPoints || CentroidPoints->IsEmpty())
			{
				return EPCGTimeSliceInitResult::NoOperation;
			}

			OutState.InitialCentroids.Reserve(CentroidPoints->GetNumPoints());
			const TConstPCGValueRange<FTransform> TransformRange = CentroidPoints->GetConstTransformValueRange();
			for (const FTransform& Transform : TransformRange)
			{
				OutState.InitialCentroids.Emplace(Transform.GetLocation());
			}
		}

		return EPCGTimeSliceInitResult::Success;
	});

	auto InitializeIteration = [&PointInputs, Settings, Context](IterStateType& OutState, const ExecStateType& ExecState, const uint32 IterationIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::PrepareData::InitializeIteration);

		const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(PointInputs[IterationIndex].Data);
		if (!InputPointData || InputPointData->IsEmpty())
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(Context);
		OutPointData->InitializeFromData(InputPointData);

		if (!OutPointData->HasSpatialDataParent())
		{
			UPCGBasePointData::SetPoints(InputPointData, OutPointData, {}, /*bCopyAll=*/true);
		}

		UPCGMetadata* OutPointMetadata = OutPointData->MutableMetadata();
		check(OutPointMetadata);

		const FPCGAttributePropertySelector& Selector = Settings->ClusterAttribute;
		if (Selector.IsBasicAttribute())
		{
			PCGMetadataElementCommon::ClearOrCreateAttribute(OutPointMetadata, Selector, 0, Context);
		}

		OutState.OutputData = OutPointData;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OutState.OutPointData = Cast<UPCGPointData>(OutPointData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		OutState.Accessor = PCGAttributeAccessorHelpers::CreateAccessor(OutPointData, Selector);
		OutState.Keys = PCGAttributeAccessorHelpers::CreateKeys(OutPointData, Selector);
		if (!OutState.Accessor || !OutState.Keys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(Selector, Context);
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		const int32 NumPoints = OutPointData->GetNumPoints();
		const int32 NumClusters = ExecState.InitialCentroids.Num();

		// Early out if centroid count makes this trivial. The default value for the attribute is already set to 0.
		if (NumClusters < 2)
		{
			Context->OutputData.TaggedData.Emplace_GetRef().Data = OutPointData;
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// With more clusters than points, each point should be assigned a unique cluster--so this is an acceptable shortcut.
		if (NumClusters >= NumPoints)
		{
			TArray<int32> Clusters;
			Clusters.Reserve(NumPoints);
			for (int i = 0; i < NumPoints; ++i)
			{
				Clusters.Emplace(i);
			}

			OutState.Accessor->SetRange(MakeConstArrayView(Clusters), 0, *OutState.Keys);
			Context->OutputData.TaggedData.Emplace_GetRef().Data = OutPointData;
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Initialize all the cluster assignments to the first cluster (0)
		OutState.PointToClusterAssignments.SetNumZeroed(NumPoints);

		auto InitializeCentroids = [Settings, NumClusters, NumPoints, &OutState, &ExecState]
		{
			if (ExecState.bSelectRandomCentroids)
			{
				const TConstPCGValueRange<FTransform> TransformRange = OutState.OutputData->GetConstTransformValueRange();

				FRandomStream RandomStream(Settings->GetSeed());
				TArray<int32> StartingIndices = PCGHelpers::GetRandomIndices(RandomStream, NumPoints, NumClusters);

				OutState.Centroids.Reserve(NumClusters);
				for (int i = 0; i < NumClusters; ++i)
				{
					OutState.Centroids.Emplace(TransformRange[StartingIndices[i]].GetLocation());
				}
			}
			else
			{
				OutState.Centroids.Append(ExecState.InitialCentroids);
			}
		};

		switch (Settings->Algorithm)
		{
			case EPCGClusterAlgorithm::KMeans:
			{
				/* TODO: There are some pre-warming algorithms that could be employed to reduce the work required to
				 * reach convergence. They would need investigation and profiling. Some possibilities may include:
				 * - KMeans++ - https://en.wikipedia.org/wiki/K-means++
				 * - Find furthest points using the Octree
				 * - For large point clouds, Principal Component Analysis might be useful.
				 */

				// Randomly select indices to start as cluster centroids
				InitializeCentroids();

				break;
			}

			case EPCGClusterAlgorithm::EM:
			{
				InitializeCentroids();

				const TConstPCGValueRange<FTransform> TransformRange = OutState.OutputData->GetConstTransformValueRange();

				// Normalization of point locations may be required to bring the GPD Function's Exponential Term into the -709.8 -> 709.8 range to prevent numerical overflow.
				double MaxSquaredLength = std::numeric_limits<double>::min();
				for (const FTransform& Transform : TransformRange)
				{
					MaxSquaredLength = FMath::Max(MaxSquaredLength, Transform.GetLocation().SquaredLength());
				}

				// Only normalize the system if needed.
				OutState.EMData.ScalingFactor = 1.0;
				if (MaxSquaredLength > 1.0)
				{
					const double Length = FMath::Sqrt(MaxSquaredLength);
					OutState.EMData.ScalingFactor = UE_DOUBLE_EULERS_NUMBER / Length;
				}

				OutState.EMData.Gaussians.Reserve(NumClusters);
				for (int i = 0; i < NumClusters; ++i)
				{
					OutState.EMData.Gaussians.Emplace(OutState.Centroids[i] * OutState.EMData.ScalingFactor, FMatrix::Identity, 1.0 / NumClusters);
				}

				OutState.EMData.Probabilities.Initialize(OutState.OutputData->GetNumPoints(), NumClusters);

				OutState.EMData.NormalizedPointLocations.Reserve(OutState.OutputData->GetNumPoints());
				Algo::Transform(TransformRange, OutState.EMData.NormalizedPointLocations, [ScalingFactor = OutState.EMData.ScalingFactor](const FTransform& Transform)
				{
					return Transform.GetLocation() * ScalingFactor;
				});

				break;
			}
			default:
				checkNoEntry();
				return EPCGTimeSliceInitResult::AbortExecution;
		}

		return EPCGTimeSliceInitResult::Success;
	};

	Context->InitializePerIterationStates(PointInputs.Num(), InitializeIteration);

	return true;
}

bool FPCGClusterElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::Execute);

	ContextType* TimeSlicedContext = static_cast<ContextType*>(InContext);
	check(TimeSlicedContext);

	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		return true;
	}

	if (TimeSlicedContext->GetExecutionStateResult() == EPCGTimeSliceInitResult::NoOperation)
	{
		TimeSlicedContext->OutputData = TimeSlicedContext->InputData;
		return true;
	}

	const UPCGClusterSettings* Settings = TimeSlicedContext->GetInputSettings<UPCGClusterSettings>();
	check(Settings);

	return ExecuteSlice(TimeSlicedContext, [Settings](ContextType* Context, const ExecStateType& ExecState, IterStateType& ClusteringData, const uint32 IterIndex)
	{
		if (Context->GetIterationStateResult(IterIndex) == EPCGTimeSliceInitResult::NoOperation)
		{
			return true;
		}

		if (!ExecState.ClusteringFunction(Context, ClusteringData, Settings->MaxIterations))
		{
			return false;
		}

		// Once finished, set the attribute values.
		check(ClusteringData.OutputData);
		FPCGTaggedData& OutputData = Context->OutputData.TaggedData.Emplace_GetRef();
		OutputData.Pin = PCGPinConstants::DefaultOutputLabel;
		OutputData.Data = ClusteringData.OutputData;

		// Update the cluster assignment attribute.
		ClusteringData.Accessor->SetRange(MakeConstArrayView(ClusteringData.PointToClusterAssignments), 0, *ClusteringData.Keys);

		// If optionally outputting the final centroids, unscale the final gaussians for EM or use the centroids directly in K-Means.
		if (Settings->bOutputFinalCentroids)
		{
			check(!ClusteringData.Centroids.IsEmpty());

			TArray<FVector> OutCentroids;
			if (Settings->Algorithm == EPCGClusterAlgorithm::EM)
			{
				OutCentroids.Reserve(ClusteringData.EMData.Gaussians.Num());
				using EM = PCGClusterElement::FClusteringData::FExpectationMaximizationData;
				for (const EM::FGaussian& Gaussian : ClusteringData.EMData.Gaussians)
				{
					if (!Gaussian.bDead)
					{
						OutCentroids.Emplace(Gaussian.Mean * 1.0 / ClusteringData.EMData.ScalingFactor);
					}
				}
			}
			else
			{
				OutCentroids = std::move(ClusteringData.Centroids);
			}

			UPCGBasePointData* OutputCentroidPointData = FPCGContext::NewPointData_AnyThread(Context);
			OutputCentroidPointData->SetNumPoints(OutCentroids.Num());
			OutputCentroidPointData->SetDensity(1.0f);
			OutputCentroidPointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed);

			FPCGPointValueRanges OutRanges(OutputCentroidPointData, /*bAllocate=*/false);
			for (int32 i = 0; i < OutCentroids.Num(); ++i)
			{
				OutRanges.TransformRange[i] = FTransform(OutCentroids[i]);
				OutRanges.SeedRange[i] = PCGHelpers::ComputeSeedFromPosition(OutCentroids[i]);
			}

			FPCGTaggedData& OutputCentroidData = Context->OutputData.TaggedData.Emplace_GetRef();
			OutputCentroidData.Pin = PCGClusterElement::Constants::FinalCentroidsLabel;
			OutputCentroidData.Data = OutputCentroidPointData;

			if (Settings->bOutputFinalCentroidElementCount)
			{
				check(OutputCentroidPointData->MutableMetadata());

				const FPCGAttributePropertyOutputNoSourceSelector& Selector = Settings->FinalCentroidElementCountAttribute;
				if (Selector.IsBasicAttribute())
				{
					PCGMetadataElementCommon::ClearOrCreateAttribute(OutputCentroidPointData->MutableMetadata(), Selector, 0, Context);
				}

				const TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputCentroidPointData, Selector);
				const TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(OutputCentroidPointData, Selector);
				if (!Accessor || !Keys)
				{
					PCGLog::Metadata::LogFailToCreateAccessorError(Selector, Context);
					return true;
				}

				TArray<int32, TInlineAllocator<16>> CentroidElementCount;
				CentroidElementCount.SetNumZeroed(OutCentroids.Num());
				// Tabulate the count of each ClusterID
				for (int32 PointIndex = 0; PointIndex < ClusteringData.PointToClusterAssignments.Num(); ++PointIndex)
				{
					++CentroidElementCount[ClusteringData.PointToClusterAssignments[PointIndex]];
				}

				Accessor->SetRange(MakeConstArrayView(CentroidElementCount), 0, *Keys);
			}
		}

		return true;
	});
}

#undef LOCTEXT_NAMESPACE
