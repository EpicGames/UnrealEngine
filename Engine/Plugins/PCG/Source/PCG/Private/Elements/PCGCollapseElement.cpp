// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCollapseElement.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCollapseElement)

void UPCGCollapseSettings::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGAttributeSetToPointAlwaysConverts)
	{
		bPassThroughEmptyAttributeSets = true;
	}
}

#if WITH_EDITOR
bool UPCGCollapseSettings::GetCompactNodeIcon(FName& OutCompactNodeIcon) const
{
	OutCompactNodeIcon = PCGNodeConstants::Icons::CompactNodeConvert;
	return true;
}
#endif

TArray<FPCGPinProperties> UPCGCollapseSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spatial);
	InputPinProperty.SetRequiredPin();

	return PinProperties;
}

FPCGElementPtr UPCGCollapseSettings::CreateElement() const
{
	return MakeShared<FPCGCollapseElement>();
}

TArray<FPCGPinProperties> UPCGConvertToPointDataSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Param);
	InputPinProperty.SetRequiredPin();

	return PinProperties;
}

bool FPCGCollapseElement::ShouldComputeFullOutputDataCrc(FPCGContext* InContext) const
{
	FPCGCollapseContext* Context = static_cast<FPCGCollapseContext*>(InContext);
	return Context && Context->bShouldComputeFullOutputDataCrc;
}

bool FPCGCollapseElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCollapseElement::Execute);
	check(InContext);
	FPCGCollapseContext* Context = static_cast<FPCGCollapseContext*>(InContext);

	const UPCGCollapseSettings* Settings = InContext->GetInputSettings<UPCGCollapseSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		if (!Input.Data)
		{
			continue;
		}

		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		if (const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Input.Data))
		{
			// Currently we support collapsing to point data only, but at some point in the future that might be different
			Output.Data = Cast<const UPCGSpatialData>(Input.Data)->ToBasePointData(Context);

			if (Output.Data != Input.Data)
			{
				Context->bShouldComputeFullOutputDataCrc = true;
			}
		}
		else if (const UPCGParamData* ParamData = Cast<UPCGParamData>(Input.Data))
		{
			const UPCGMetadata* ParamMetadata = ParamData->Metadata;
			const int64 ParamItemCount = ParamMetadata->GetLocalItemCount();

			if (ParamItemCount == 0 && Settings->bPassThroughEmptyAttributeSets)
			{
				continue;
			}

			UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(Context);
			check(PointData->Metadata);
			PointData->Metadata->Initialize(ParamMetadata);
			PointData->SetNumPoints(ParamItemCount);
			PointData->AllocateProperties(EPCGPointNativeProperties::MetadataEntry);
			TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange(/*bAllocate=*/false);

			for (int PointIndex = 0; PointIndex < ParamItemCount; ++PointIndex)
			{
				MetadataEntryRange[PointIndex] = PointIndex;
			}

			const UPCGConvertToPointDataSettings* ConvertToPointDataSettings = InContext->GetInputSettings<UPCGConvertToPointDataSettings>();
			if (ConvertToPointDataSettings && ConvertToPointDataSettings->bMatchAttributeNamesWithPropertyNames)
			{
				TArray<FName> AttributeNames;
				TArray<EPCGMetadataTypes> AttributeTypes;

				PointData->ConstMetadata()->GetAttributes(AttributeNames, AttributeTypes);
				for (const FName AttributeName : AttributeNames)
				{
					FPCGAttributePropertyOutputSelector OutputSelector = FPCGAttributePropertySelector::CreatePropertySelector<FPCGAttributePropertyOutputSelector>(AttributeName);

					// Need to create the accessor to know if it exists.
					if (!PCGAttributeAccessorHelpers::CreateAccessor(PointData, OutputSelector, /*bQuiet=*/true))
					{
						continue;
					}

					PCGMetadataHelpers::FPCGCopyAttributeParams CopyParams
					{
						.SourceData = ParamData,
						.TargetData = PointData,
						.InputSource = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(AttributeName),
						.OutputTarget = MoveTemp(OutputSelector),
						.bSameOrigin = true
					};

					if (PCGMetadataHelpers::CopyAttribute(CopyParams) && ConvertToPointDataSettings->bDeleteOriginalRemappedAttribute)
					{
						PointData->MutableMetadata()->DeleteAttribute(AttributeName);
					}
				}
			}

			if (ConvertToPointDataSettings && !ConvertToPointDataSettings->AttributeToConvert.IsEmpty())
			{
				for (const auto& [InputSource, OutputTarget] : ConvertToPointDataSettings->AttributeToConvert)
				{
					PCGMetadataHelpers::FPCGCopyAttributeParams CopyParams
					{
						.SourceData = ParamData,
						.TargetData = PointData,
						.InputSource = InputSource,
						.OutputTarget = OutputTarget,
						.bSameOrigin = true
					};

					if (PCGMetadataHelpers::CopyAttribute(CopyParams) && ConvertToPointDataSettings->bDeleteOriginalRemappedAttribute)
					{
						PointData->MutableMetadata()->DeleteAttribute(InputSource.CopyAndFixLast(ParamData).GetName());
					}
				}
			}

			Output.Data = PointData;
			Context->bShouldComputeFullOutputDataCrc = true;
		}
	}

	return true;
}
