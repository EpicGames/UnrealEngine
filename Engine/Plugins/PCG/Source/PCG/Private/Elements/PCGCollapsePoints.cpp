// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCollapsePoints.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "SpatialAlgo/PCGOctreeQueries.h"

#include "Algo/MaxElement.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCollapsePoints)

#define LOCTEXT_NAMESPACE "PCGCollapsePointsElement"

namespace PCGCollapsePoints
{
	namespace Algo
	{
		void MergePairs(FPCGContext* InContext, const FCollapsePointsSettings& Settings, FCollapsePointsState& OutState)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGCollapsePointsElement::Algo::MergePairs);

			check(!OutState.Selections.IsEmpty());

			for (const TPair<int32, int32>& MergePair : OutState.Selections)
			{
				int PrimaryPointIndex = MergePair.Key;
				int SecondaryPointIndex = MergePair.Value;

				check(OutState.Merged[PrimaryPointIndex] == INDEX_NONE && OutState.Merged[SecondaryPointIndex] == INDEX_NONE);

				const double& PrimaryWeight = OutState.Weights[PrimaryPointIndex];
				const double& SecondaryWeight = OutState.Weights[SecondaryPointIndex];
				const double WeightSum = PrimaryWeight + SecondaryWeight;

				double Alpha = FMath::IsNearlyZero(WeightSum) ? 0.5 : (SecondaryWeight / WeightSum);
				const FVector DeltaPosition = Settings.GetPointPositionFunc(OutState.PointTransforms, OutState.SourcePointData, SecondaryPointIndex) - Settings.GetPointPositionFunc(OutState.PointTransforms, OutState.SourcePointData, PrimaryPointIndex);

				OutState.PointTransforms[PrimaryPointIndex].AddToTranslation(Alpha * DeltaPosition);
				OutState.Weights[PrimaryPointIndex] = WeightSum;
				OutState.Merged[SecondaryPointIndex] = PrimaryPointIndex;
			}
		}

		void RebuildOctree(FPCGContext* InContext, const FCollapsePointsSettings& Settings, FCollapsePointsState& OutState)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGCollapsePointsElement::Algo::RebuildOctree);
			check(OutState.SourcePointData && OutState.Merged.Num() == OutState.SourcePointData->GetNumPoints());

			PCGPointOctree::FPointOctree NewOctree(OutState.SourcePointData->GetBounds().GetCenter(), OutState.SourcePointData->GetBounds().GetExtent().Length());
			for (int PointIndex = 0; PointIndex < OutState.SourcePointData->GetNumPoints(); ++PointIndex)
			{
				if (OutState.Merged[PointIndex] == INDEX_NONE)
				{
					NewOctree.AddElement(Settings.GetPointReferenceFunc(OutState.PointTransforms, OutState.SourcePointData, PointIndex));
				}
			}

			OutState.PointOctree = MoveTemp(NewOctree);
		}
	}

	namespace Modes
	{
		// Creates exclusive pairs of points to merge, in the visit order.
		bool PairwiseSelection(FPCGContext* InContext, const FCollapsePointsSettings& Settings, FCollapsePointsState& OutState)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGCollapsePointsElement::Modes::PairwiseSelection);
			OutState.Selections.Reset();

			// Note: can't do async trivially here as we will block off things downstream
			// 1. Reset visited state
			OutState.Visited.SetNumUninitialized(OutState.VisitOrder.Num(), EAllowShrinking::No);
			for (bool& Visited : OutState.Visited)
			{
				Visited = false;
			}

			for (int VisitIndex = 0; VisitIndex < OutState.VisitOrder.Num(); ++VisitIndex)
			{
				int PointIndex = OutState.VisitOrder[VisitIndex];

				if (OutState.Merged[PointIndex] != INDEX_NONE || OutState.Visited[PointIndex])
				{
					continue;
				}

				// this isn't strictly needed as we will never read from this value
				OutState.Visited[PointIndex] = true;

				// Find closest unvisited point after current point that's within the distance threshold.
				double MinSqrDistance = Settings.DistanceThreshold * Settings.DistanceThreshold;
				int ClosestUnvisitedIndex = INDEX_NONE;
				bool bHasColocatedPoint = false;

				const double Extents = UE_DOUBLE_SQRT_2 * Settings.DistanceThreshold;
				FBoxCenterAndExtent SearchBounds = Settings.GetPointSearchBoundsFunc(OutState.PointTransforms, OutState.SourcePointData, PointIndex, Extents);
				OutState.PointOctree.FindElementsWithBoundsTest(SearchBounds, [&Settings, PointIndex, &MinSqrDistance, &ClosestUnvisitedIndex, &bHasColocatedPoint, &OutState](const PCGPointOctree::FPointRef& PointRef)
				{
					int NeighborIndex = PointRef.Index;

					if (OutState.Merged[NeighborIndex] != INDEX_NONE || OutState.Visited[NeighborIndex])
					{
						return;
					}

					double SqrDistance = (Settings.GetPointPositionFunc(OutState.PointTransforms, OutState.SourcePointData, PointIndex) - Settings.GetPointPositionFunc(OutState.PointTransforms, OutState.SourcePointData, NeighborIndex)).SquaredLength();

					if (FMath::IsNearlyZero(SqrDistance))
					{
						if (bHasColocatedPoint)
						{
							check(ClosestUnvisitedIndex != INDEX_NONE);
							// Prioritize visit order then
							const int32 NeighborVisitOrder = OutState.VisitOrder.IndexOfByKey(NeighborIndex);
							const int32 ClosestVisitOrder = OutState.VisitOrder.IndexOfByKey(ClosestUnvisitedIndex);

							if (NeighborVisitOrder < ClosestVisitOrder)
							{
								ClosestUnvisitedIndex = NeighborIndex;
							}
						}
						else
						{
							bHasColocatedPoint = true;
							ClosestUnvisitedIndex = NeighborIndex;
							MinSqrDistance = SqrDistance;
						}
					}
					else if (SqrDistance < MinSqrDistance)
					{
						MinSqrDistance = SqrDistance;
						ClosestUnvisitedIndex = NeighborIndex;
					}
				});

				if (ClosestUnvisitedIndex != INDEX_NONE)
				{
					OutState.Visited[ClosestUnvisitedIndex] = true;
					OutState.Selections.Emplace(PointIndex, ClosestUnvisitedIndex);
				}
			}

			return !OutState.Selections.IsEmpty();
		}

		bool ClosestPairSelection(FPCGContext* InContext, const FCollapsePointsSettings& Settings, FCollapsePointsState& OutState)
		{
			// TODO - not implemented. Needs to find the absolute closest pair and insert it in the selections array.
			return true;
		}
	} // namespace Modes

	namespace ComparisonModes
	{
		FVector GetPosition(const TArray<FTransform>& InTransforms, const UPCGBasePointData* InPointData, int32 InIndex)
		{
			return InTransforms[InIndex].GetLocation();
		}

		FVector GetCenter(const TArray<FTransform>& InTransforms, const UPCGBasePointData* InPointData, int32 InIndex)
		{
			return InTransforms[InIndex].TransformPosition(InPointData->GetLocalCenter(InIndex));
		}

		PCGPointOctree::FPointRef GetPositionPointRef(const TArray<FTransform>& InTransforms, const UPCGBasePointData* InPointData, int32 InIndex)
		{
			check(InPointData);
			return PCGPointOctree::FPointRef(InIndex, FBox(FVector::ZeroVector, FVector::ZeroVector).TransformBy(InTransforms[InIndex]));
		}

		PCGPointOctree::FPointRef GetCenterPointRef(const TArray<FTransform>& InTransforms, const UPCGBasePointData* InPointData, int32 InIndex)
		{
			check(InPointData);
			const FVector LocalCenter = InPointData->GetLocalCenter(InIndex);
			return PCGPointOctree::FPointRef(InIndex, FBox(LocalCenter, LocalCenter).TransformBy(InTransforms[InIndex]));
		}

		FBoxCenterAndExtent GetPositionSearchBounds(const TArray<FTransform>& InTransforms, const UPCGBasePointData* InPointData, int32 InIndex, const double& Extents)
		{
			check(InPointData);
			return FBoxCenterAndExtent(InTransforms[InIndex].GetLocation(), FVector(Extents, Extents, Extents));
		}

		FBoxCenterAndExtent GetCenterSearchBounds(const TArray<FTransform>& InTransforms, const UPCGBasePointData* InPointData, int32 InIndex, const double& Extents)
		{
			check(InPointData);
			return FBoxCenterAndExtent(InTransforms[InIndex].TransformPosition(InPointData->GetLocalCenter(InIndex)), FVector(Extents, Extents, Extents));
		}
	}
} // namespace PCGCollapsePoints

UPCGCollapsePointsSettings::UPCGCollapsePointsSettings()
{
	FPCGAttributePropertyOutputNoSourceSelector& DefaultAttribute = AttributesToMerge.Emplace_GetRef();
	DefaultAttribute.SetPointProperty(EPCGPointProperties::Position);
}

#if WITH_EDITOR
FText UPCGCollapsePointsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Collapses points with their closest neighbors until all points are farther than the search distance.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGCollapsePointsSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point).SetRequiredPin();
	return PinProperties;
}

bool UPCGCollapsePointsSettings::UseSeed() const
{
	return VisitOrder == EPCGCollapseVisitOrder::Random;
}

FPCGElementPtr UPCGCollapsePointsSettings::CreateElement() const
{
	return MakeShared<FPCGCollapsePointsElement>();
}

bool FPCGCollapsePointsElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCollapsePointsElement::PrepareData);

	const UPCGCollapsePointsSettings* Settings = InContext->GetInputSettings<UPCGCollapsePointsSettings>();
	check(Settings);

	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	if (Inputs.IsEmpty())
	{
		return true;
	}

	Context->InitializePerExecutionState([Settings](const ContextType*, ExecStateType& OutState)
	{
		PCGCollapsePoints::FCollapsePointsSettings::PairSelectionFuncType PairSelectionFunc = nullptr;
		PCGCollapsePoints::FCollapsePointsSettings::GetPointPositionFuncType GetPointPositionFunc = nullptr;
		PCGCollapsePoints::FCollapsePointsSettings::GetPointReferenceFuncType GetPointReferenceFunc = nullptr;
		PCGCollapsePoints::FCollapsePointsSettings::GetPointSearchBoundsFuncType GetPointSearchBoundsFunc = nullptr;

		if (Settings->Mode == EPCGCollapseMode::PairwiseClosest)
		{
			PairSelectionFunc = PCGCollapsePoints::Modes::PairwiseSelection;
		}
		else if(Settings->Mode == EPCGCollapseMode::AbsoluteClosest)
		{
			PairSelectionFunc = PCGCollapsePoints::Modes::ClosestPairSelection;
		}
		else
		{
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		if (Settings->ComparisonMode == EPCGCollapseComparisonMode::Position)
		{
			GetPointPositionFunc = PCGCollapsePoints::ComparisonModes::GetPosition;
			GetPointReferenceFunc = PCGCollapsePoints::ComparisonModes::GetPositionPointRef;
			GetPointSearchBoundsFunc = PCGCollapsePoints::ComparisonModes::GetPositionSearchBounds;
		}
		else if (Settings->ComparisonMode == EPCGCollapseComparisonMode::Center)
		{
			GetPointPositionFunc = PCGCollapsePoints::ComparisonModes::GetCenter;
			GetPointReferenceFunc = PCGCollapsePoints::ComparisonModes::GetCenterPointRef;
			GetPointSearchBoundsFunc = PCGCollapsePoints::ComparisonModes::GetCenterSearchBounds;
		}
		else
		{
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		check(PairSelectionFunc);

		OutState = PCGCollapsePoints::FCollapsePointsSettings
		{
			.Settings = Settings,
			.PairSelectionFunc = PairSelectionFunc,
			.MergeSelectionFunc = PCGCollapsePoints::Algo::MergePairs,
			.GetPointPositionFunc = GetPointPositionFunc,
			.GetPointReferenceFunc = GetPointReferenceFunc,
			.GetPointSearchBoundsFunc = GetPointSearchBoundsFunc,
			.DistanceThreshold = Settings->DistanceThreshold
		};

		return EPCGTimeSliceInitResult::Success;
	});

	Context->InitializePerIterationStates(Inputs.Num(), [&Inputs, Settings, Context](IterStateType& OutState, const ExecStateType& ExecState, const uint32 IterationIndex)
	{
		const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Inputs[IterationIndex].Data);

		if (!PointData || PointData->IsEmpty())
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (PointData->GetBounds().ContainsNaN())
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidBounds", "Input data at index {0} has invalid bounds and will be skipped."), IterationIndex), Context);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		OutState.SourcePointData = PointData;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OutState.SourceData = Cast<UPCGPointData>(PointData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Get points, octree will be rebuilt later
		OutState.PointTransforms = PointData->GetTransformsCopy();

		// Get weights
		const int32 NumPoints = PointData->GetNumPoints();
		OutState.Weights.SetNumUninitialized(NumPoints);

		ensure(OutState.PointTransforms.Num() == OutState.Weights.Num());

		if (Settings->bUseMergeWeightAttribute)
		{
			const FPCGAttributePropertyInputSelector Selector = Settings->MergeWeightAttribute.CopyAndFixLast(PointData);
			const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, Selector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, Selector);
			if (!Accessor || !Keys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}
			
			if (!Accessor->GetRange(TArrayView<double>(OutState.Weights), 0, *Keys))
			{
				PCGLog::Metadata::LogFailToGetAttributeError(Selector, Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			// Fixup weights so that they are not negative
			for (double& Weight : OutState.Weights)
			{
				Weight = FMath::Max(Weight, 0.0);
			}
		}
		else
		{
			// Default uniform weights
			for (double& Weight : OutState.Weights)
			{
				Weight = 1.0;
			}
		}

		// We need a copy of the weights because we'll change it over the process of merging, but need the original weights at the end of the process
		OutState.OriginalWeights = OutState.Weights;

		// Build visit order
		if (Settings->Mode == EPCGCollapseMode::PairwiseClosest)
		{
			OutState.VisitOrder.SetNumUninitialized(NumPoints);

			for (int Index = 0; Index < NumPoints; ++Index)
			{
				OutState.VisitOrder[Index] = Index;
			}

			if (Settings->VisitOrder == EPCGCollapseVisitOrder::Ordered)
			{
				// Nothing to do, initial assignment is fine
			} 
			else if(Settings->VisitOrder == EPCGCollapseVisitOrder::Random)
			{
				FRandomStream RandomStream(Context->GetSeed());
				PCGHelpers::ShuffleArray(RandomStream, OutState.VisitOrder);
			}
			else if (Settings->VisitOrder == EPCGCollapseVisitOrder::MinAttribute || Settings->VisitOrder == EPCGCollapseVisitOrder::MaxAttribute)
			{
				const FPCGAttributePropertyInputSelector VisitOrderSelector = Settings->VisitOrderAttribute.CopyAndFixLast(PointData);
				TUniquePtr<const IPCGAttributeAccessor> VisitOrderAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, VisitOrderSelector);
				TUniquePtr<const IPCGAttributeAccessorKeys> VisitOrderKeys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, VisitOrderSelector);
				if (!VisitOrderAccessor || !VisitOrderKeys)
				{
					PCGLog::Metadata::LogFailToCreateAccessorError(VisitOrderSelector, Context);
					return EPCGTimeSliceInitResult::NoOperation;
				}

				PCGAttributeAccessorHelpers::SortByAttribute(*VisitOrderAccessor, *VisitOrderKeys, OutState.VisitOrder, /*bAscending=*/Settings->VisitOrder == EPCGCollapseVisitOrder::MinAttribute);
			}
			else
			{
				checkNoEntry();
				return EPCGTimeSliceInitResult::NoOperation;
			}
		}

		// Initialize merged state to "not merged"
		OutState.Merged.SetNumUninitialized(NumPoints);
		for (int& MergeIndex : OutState.Merged)
		{
			MergeIndex = INDEX_NONE;
		}

		// Create the output point data now. Operate on a copy of the input point data.
		UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(Context);

		FPCGInitializeFromDataParams InitializeFromDataParams(PointData);
		InitializeFromDataParams.bInheritSpatialData = false;
		OutPointData->InitializeFromDataWithParams(InitializeFromDataParams);

		OutState.OutputPointData = OutPointData;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OutState.OutData = Cast<UPCGPointData>(OutPointData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Initialize merge accessors
		for(const FPCGAttributePropertyOutputNoSourceSelector& InAttributeToMerge : Settings->AttributesToMerge)
		{
			FPCGAttributePropertyInputSelector SourceAttributeToMerge;
			SourceAttributeToMerge.ImportFromOtherSelector(InAttributeToMerge);
			
			const FPCGAttributePropertyInputSelector AttributeToMerge = SourceAttributeToMerge.CopyAndFixLast(PointData);
			TUniquePtr<const IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, AttributeToMerge);

			if (!AttributeAccessor)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(AttributeToMerge, Context);
				continue;
			}

			FPCGAttributePropertyOutputSelector OutputAttributeSelector;
			OutputAttributeSelector.ImportFromOtherSelector(AttributeToMerge); 

			OutState.SourceMergeAccessors.Add(std::move(AttributeAccessor));
			OutState.OutputMergeSelectors.Add(std::move(OutputAttributeSelector));
		}

		OutState.SourceMergeKeys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, FPCGAttributePropertyInputSelector());
		if (!OutState.SourceMergeKeys)
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Rebuild octree
		PCGCollapsePoints::Algo::RebuildOctree(Context, ExecState, OutState);

		Context->OutputData.TaggedData.Emplace_GetRef(Inputs[IterationIndex]).Data = OutPointData;
		return EPCGTimeSliceInitResult::Success;
	});

	return true;
}

bool FPCGCollapsePointsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCollapsePointsElement::Execute);

	ContextType* TimeSlicedContext = static_cast<ContextType*>(InContext);
	check(TimeSlicedContext);

	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		return true;
	}

	if (TimeSlicedContext->GetExecutionStateResult() == EPCGTimeSliceInitResult::NoOperation)
	{
		TimeSlicedContext->OutputData = TimeSlicedContext->InputData;
		return true;
	}

	const UPCGCollapsePointsSettings* Settings = TimeSlicedContext->GetInputSettings<UPCGCollapsePointsSettings>();
	check(Settings);

	return ExecuteSlice(TimeSlicedContext, [InContext](const ContextType* Context, const ExecStateType& CollapseSettings, IterStateType& CollapseState, const uint32 IterIndex)
	{
		if (Context->GetIterationStateResult(IterIndex) == EPCGTimeSliceInitResult::NoOperation)
		{
			return true;
		}

		// A single iteration is:
		// - build list of pairs (all OR single closest pair)
		// - merge them, update weights
		// continue until nothing is merged
		if (CollapseSettings.PairSelectionFunc(InContext, CollapseSettings, CollapseState))
		{
			// Merge based on selection
			CollapseSettings.MergeSelectionFunc(InContext, CollapseSettings, CollapseState);
			// Rebuild octree for next iteration
			PCGCollapsePoints::Algo::RebuildOctree(InContext, CollapseSettings, CollapseState);

			return false;
		}
		else
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCollapsePointsElement::Execute::ComputeFinalResults);
			TArray<int>& Merged = CollapseState.Merged;

			// We're done - compute final results
			// Go up the merge hierarchy to point to the root
			for (int PointIndex = 0; PointIndex < Merged.Num(); ++PointIndex)
			{
				while (Merged[PointIndex] != INDEX_NONE && Merged[Merged[PointIndex]] != INDEX_NONE)
				{
					Merged[PointIndex] = Merged[Merged[PointIndex]];
				}
			}

			// Finally, make unmerged points/roots point to themselves
			for (int PointIndex = 0; PointIndex < Merged.Num(); ++PointIndex)
			{
				if (Merged[PointIndex] == INDEX_NONE)
				{
					Merged[PointIndex] = PointIndex;
				}
			}

			// Second, partition by merge target
			TMap<int, TArray<int, TInlineAllocator<8>>> Partition;

			for (int PointIndex = 0; PointIndex < Merged.Num(); ++PointIndex)
			{
				Partition.FindOrAdd(Merged[PointIndex]).Add(PointIndex);
			}

			// Create one point per partition
			check(CollapseState.SourcePointData);
			check(CollapseState.OutputPointData);
			
			// Need to allocate prior to key creation
			CollapseState.OutputPointData->SetNumPoints(Partition.Num(), /*bInitializeValues=*/false);
			CollapseState.OutputPointData->AllocateProperties(CollapseState.SourcePointData->GetAllocatedProperties());
			CollapseState.OutputPointData->CopyUnallocatedPropertiesFrom(CollapseState.SourcePointData);

			const bool bHasLastSelector = CollapseState.OutputPointData->HasCachedLastSelector();
			FPCGAttributePropertyInputSelector LastSelector;
			if (bHasLastSelector)
			{
				LastSelector = CollapseState.OutputPointData->GetCachedLastSelector();
			}

			// Build accessor list(s) + keys, based on settings
			TArray<double> PartitionWeights;
			TUniquePtr<IPCGAttributeAccessorKeys> OutputKeys = PCGAttributeAccessorHelpers::CreateKeys(CollapseState.OutputPointData, FPCGAttributePropertyOutputSelector());
			check(OutputKeys);
			TUniquePtr<const IPCGAttributeAccessorKeys>& SourceKeys = CollapseState.SourceMergeKeys;
			check(SourceKeys);
			
			FPCGPointValueRanges OutRanges(CollapseState.OutputPointData, /*bAllocate=*/false);
			const FConstPCGPointValueRanges InRanges(CollapseState.SourcePointData);

			// Create Accessors now that data is allocated
			check(CollapseState.OutputMergeAccessors.IsEmpty());
			for (int AccessorIndex = 0; AccessorIndex < CollapseState.SourceMergeAccessors.Num(); ++AccessorIndex)
			{
				TUniquePtr<IPCGAttributeAccessor> OutputAccessor = PCGAttributeAccessorHelpers::CreateAccessor(CollapseState.OutputPointData, CollapseState.OutputMergeSelectors[AccessorIndex]);
				if (!OutputAccessor)
				{
					PCGLog::Metadata::LogFailToCreateAccessorError(CollapseState.OutputMergeSelectors[AccessorIndex], Context);
					return true;
				}
				CollapseState.OutputMergeAccessors.Add(std::move(OutputAccessor));
			}

			// TODO: do this in parallel
			int PartitionIndex = 0;
			for(const TPair<int, TArray<int, TInlineAllocator<8>>>& PartitionEntry : Partition)
			{
				int PrimaryPointIndex = PartitionEntry.Key;
				const TArray<int, TInlineAllocator<8>>& SecondaryPointIndices = PartitionEntry.Value;

				// First, copy the primary point as-is.
				OutRanges.SetFromValueRanges(PartitionIndex, InRanges, PrimaryPointIndex);

				if (SecondaryPointIndices.Num() == 1)
				{
					++PartitionIndex;
					continue;
				}

				// Prepare respective weights
				double TotalWeight = 0;
				PartitionWeights.SetNumUninitialized(SecondaryPointIndices.Num());
				for(int SecondaryIndex = 0; SecondaryIndex < SecondaryPointIndices.Num(); ++SecondaryIndex)
				{
					const double& OriginalWeight = CollapseState.OriginalWeights[SecondaryPointIndices[SecondaryIndex]];
					TotalWeight += OriginalWeight;
					PartitionWeights[SecondaryIndex] = OriginalWeight;
				}

				for (double& PartitionWeight : PartitionWeights)
				{
					PartitionWeight = (FMath::IsNearlyZero(TotalWeight) ? 1.0 / PartitionWeights.Num() : PartitionWeight / TotalWeight);
				}

				for (int AccessorIndex = 0; AccessorIndex < CollapseState.SourceMergeAccessors.Num(); ++AccessorIndex)
				{
					TUniquePtr<const IPCGAttributeAccessor>& SourceAccessor = CollapseState.SourceMergeAccessors[AccessorIndex];
					TUniquePtr<IPCGAttributeAccessor>& OutputAccessor = CollapseState.OutputMergeAccessors[AccessorIndex];

					auto Callback = [PartitionIndex, &SecondaryPointIndices, &PartitionWeights, &SourceAccessor, &SourceKeys, &OutputAccessor, &OutputKeys]<typename T>(T)
					{
						if constexpr (PCG::Private::MetadataTraits<T>::CanInterpolate)
						{
							T Value = PCG::Private::MetadataTraits<T>::ZeroValueForWeightedSum();

							for (int Index = 0; Index < SecondaryPointIndices.Num(); ++Index)
							{
								int PointIndex = SecondaryPointIndices[Index];

								T SecondaryValue{};
								SourceAccessor->Get(SecondaryValue, PointIndex, *SourceKeys);

								Value = PCG::Private::MetadataTraits<T>::WeightedSum(Value, SecondaryValue, PartitionWeights[Index]);
							}

							if constexpr (PCG::Private::MetadataTraits<T>::InterpolationNeedsNormalization)
							{
								static_assert(PCG::Private::MetadataTraits<T>::CanNormalize);

								// We need to normalize the resulting value
								PCG::Private::MetadataTraits<T>::Normalize(Value);
							}
							
							OutputAccessor->Set(Value, PartitionIndex, *OutputKeys);
						}
						else
						{
							// Nothing to do because the default point will already have the value it would end up with.
						}
					};

					PCGMetadataAttribute::CallbackWithRightType(OutputAccessor->GetUnderlyingType(), Callback);
				}

				++PartitionIndex;
			}

			// Set Last Selector if needed
			if (bHasLastSelector)
			{
				CollapseState.OutputPointData->SetLastSelector(LastSelector);
			}

			return true;
		}
	});
}

#undef LOCTEXT_NAMESPACE
