// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCombinePoints.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCombinePoints)

#define LOCTEXT_NAMESPACE "PCGCombinePointsElement"

FPCGElementPtr UPCGCombinePointsSettings::CreateElement() const
{
	return MakeShared<FPCGCombinePointsElement>();
}

bool FPCGCombinePointsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCombinePointsElement::Execute);
	check(Context);

	const UPCGCombinePointsSettings* Settings = Context->GetInputSettings<UPCGCombinePointsSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (int i = 0; i < Inputs.Num(); ++i)
	{
		const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(Inputs[i].Data);
		if (!InputPointData)
		{
			PCGE_LOG(Warning, GraphAndLog, LOCTEXT("InvalidInputPointData", "The input is not point data, skipped."));
			continue;
		}

		if (InputPointData->IsEmpty())
		{
			PCGE_LOG(Warning, GraphAndLog, LOCTEXT("NoPointsFound", "No points were found in the input data, skipped."));
			continue;
		}

		const FConstPCGPointValueRanges InRanges(InputPointData);
		FPCGPoint InputPoint = InRanges.GetPoint(0);
		InputPoint.ApplyScaleToBounds();

		FTransform PointTransform = Settings->bUseFirstPointTransform ? InputPoint.Transform : Settings->PointTransform;

		if (!PointTransform.IsRotationNormalized())
		{
			PointTransform.SetRotation(FQuat::Identity);
			PCGLog::Validation::LogInvalidRotationsReset();
		}

		const FMatrix InversePointTransform = PointTransform.ToInverseMatrixWithScale();

		FPCGTaggedData& Output = Outputs.Add_GetRef(Inputs[i]);
		UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(Context);
		
		FPCGInitializeFromDataParams InitializeFromDataParams(InputPointData);
		InitializeFromDataParams.bInheritSpatialData = false;
		OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);
		
		OutputPointData->SetNumPoints(1);
		OutputPointData->AllocateProperties(EPCGPointNativeProperties::All);
		Output.Data = OutputPointData;

		FPCGPoint OutPoint = FPCGPoint();
		FBox OutBox(EForceInit::ForceInit);

		bool bHasInvalidRotation = false;
		FTransform TempTransform;

		for (int j = 0; j < InputPointData->GetNumPoints(); ++j)
		{
			const FTransform& PointToCombineTransform = InRanges.TransformRange[j];

			const bool bIsRotationNormalized = PointToCombineTransform.IsRotationNormalized();
			if (!bIsRotationNormalized)
			{
				TempTransform = PointToCombineTransform;
				TempTransform.SetRotation(FQuat::Identity);
				bHasInvalidRotation = true;
			}

			const FTransform& BoxTransform = (bIsRotationNormalized ? PointToCombineTransform : TempTransform);
			OutBox += PCGPointHelpers::GetLocalBounds(InRanges.BoundsMinRange[j], InRanges.BoundsMaxRange[j]).TransformBy(BoxTransform.ToMatrixWithScale() * InversePointTransform);
		};

		if (bHasInvalidRotation)
		{
			PCGLog::Validation::LogInvalidRotationsReset();
		}

		OutPoint.SetLocalBounds(OutBox);
		OutPoint.Transform = PointTransform;
		OutPoint.Seed = PCGHelpers::ComputeSeedFromPosition(OutPoint.Transform.GetLocation());

		if (Settings->bCenterPivot)
		{
			OutPoint.ResetPointCenter(FVector(0.5));
		}

		FPCGPointValueRanges OutRanges(OutputPointData, /*bAllocate=*/false);
		OutRanges.SetFromPoint(0, OutPoint);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
