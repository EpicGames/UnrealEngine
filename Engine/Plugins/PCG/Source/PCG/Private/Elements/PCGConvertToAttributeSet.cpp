// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGConvertToAttributeSet.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGTagHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGConvertToAttributeSet)

#if WITH_EDITOR
bool UPCGConvertToAttributeSetSettings::GetCompactNodeIcon(FName& OutCompactNodeIcon) const
{
	OutCompactNodeIcon = PCGNodeConstants::Icons::CompactNodeConvert;
	return true;
}
#endif

TArray<FPCGPinProperties> UPCGConvertToAttributeSetSettings::InputPinProperties() const
{
	return Super::DefaultPointInputPinProperties();
}

TArray<FPCGPinProperties> UPCGConvertToAttributeSetSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FPCGElementPtr UPCGConvertToAttributeSetSettings::CreateElement() const
{
	return MakeShared<FPCGConvertToAttributeSetElement>();
}

bool FPCGConvertToAttributeSetElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGConvertToAttributeSetElement::Execute);
	check(Context);

	const UPCGConvertToAttributeSetSettings* Settings = Context->GetInputSettings<UPCGConvertToAttributeSetSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		if (Cast<UPCGBasePointData>(Input.Data) == nullptr)
		{
			continue;
		}

		const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Input.Data);
		const UPCGMetadata* SourceMetadata = PointData->Metadata;
		check(SourceMetadata);

		// Note: this is very similar in idea to UPCGPointData::Flatten
		if ((Settings->PropertiesToConvert.IsEmpty() && SourceMetadata->GetAttributeCount() == 0) || PointData->GetNumPoints() == 0)
		{
			continue;
		}

		UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		UPCGMetadata* Metadata = ParamData->MutableMetadata();
		check(Metadata);

		FPCGMetadataInitializeParams Params(SourceMetadata);
		FPCGMetadataDomainInitializeParams ElementsDomainParams(SourceMetadata->GetConstMetadataDomain(PCGMetadataDomainID::Elements));

		const TConstPCGValueRange<int64> MetadataEntryRange = PointData->GetConstMetadataEntryValueRange();
		check(PointData->GetNumPoints() == MetadataEntryRange.Num());
		ElementsDomainParams.OptionalEntriesToCopy = MetadataEntryRange;

		Params.DomainInitializeParams.Emplace(PCGMetadataDomainID::Elements, std::move(ElementsDomainParams));
		Metadata->InitializeAsCopy(Params);

		if (!Settings->PropertiesToConvert.IsEmpty())
		{
			for (const auto& [Source, Target] : Settings->PropertiesToConvert)
			{
				// @Source is treated as @SourceName
				FPCGAttributePropertyOutputSelector OutputSelector =
					Target.GetAttributeName() == PCGMetadataAttributeConstants::SourceAttributeName
					? FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyOutputSelector>(PCGMetadataAttributeConstants::SourceNameAttributeName)
					: Target;
				
				// Verify if the target attribute exists
				if (!Settings->bOverrideExistingAttributes)
				{
					const FPCGAttributePropertyInputSelector InputSelector = Source.CopyAndFixLast(Input.Data);
					OutputSelector = OutputSelector.CopyAndFixSource(&InputSelector, Input.Data);

					if (ParamData->ConstMetadata()->HasAttribute(OutputSelector.GetName()))
					{
						continue;
					}
				}
				
				PCGMetadataHelpers::FPCGCopyAttributeParams CopyParams
				{
					.SourceData = Input.Data,
					.TargetData = ParamData,
					.InputSource = Source,
					.OutputTarget = MoveTemp(OutputSelector)
				};

				PCGMetadataHelpers::CopyAttribute(CopyParams);
			}
		}
		
		FPCGTaggedData& Output = Outputs.Emplace_GetRef(Input);
		Output.Data = ParamData;
	}

	return true;
}

TArray<FPCGPinProperties> UPCGTagsToAttributeSetSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FPCGElementPtr UPCGTagsToAttributeSetSettings::CreateElement() const
{
	return MakeShared<FPCGTagsToAttributeSetElement>();
}

bool FPCGTagsToAttributeSetElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTagsToAttributeSetElement::Execute);
	check(Context);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		if (!Input.Data)
		{
			continue;
		}

		UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		check(ParamData && ParamData->Metadata);

		FPCGTaggedData& Output = Outputs.Emplace_GetRef(Input);
		Output.Data = ParamData;

		PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();

		for (const FString& DataTag : Input.Tags)
		{
			PCG::Private::SetAttributeFromTag(DataTag, ParamData->Metadata, EntryKey, PCG::Private::ESetAttributeFromTagFlags::CreateAttribute);
		}
	}

	return true;
}