// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGConvertToDataView.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGSplineData.h"
#include "Data/DataView/PCGDataViewData.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGConvertToDataView)

#define LOCTEXT_NAMESPACE "PCGConvertToDataViewElement"

#if WITH_EDITOR
EPCGChangeType UPCGConvertToDataViewSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGConvertToDataViewSettings, bSelectionAsInput))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}

bool UPCGConvertToDataViewSettings::ShouldDrawNodeCompact() const
{
	return !bSelectionAsInput;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGConvertToDataViewSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(
		PCGPinConstants::DefaultInputLabel,
		FPCGDataTypeIdentifier::Construct(
			FPCGDataTypeInfoSpatial::AsId(),
			FPCGDataTypeInfoParam::AsId()
			)).SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGConvertToDataViewSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeInfoDataView::AsId());

	return PinProperties;
}

FPCGElementPtr UPCGConvertToDataViewSettings::CreateElement() const
{
	return MakeShared<FPCGConvertToDataViewElement>();
}

bool FPCGConvertToDataViewElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGConvertToDataViewElement::Execute);
	check(InContext);

	const UPCGConvertToDataViewSettings* Settings = InContext->GetInputSettings<UPCGConvertToDataViewSettings>();

	// Collect attribute selection from input.
	TArray<FString> AttributeStrings;
	if (Settings->bSelectionAsInput)
	{
		Settings->SelectionSources.ParseIntoArrayWS(AttributeStrings, TEXT(","));
	}

	TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGMetadata* SourceMetadata = Input.Data->Metadata;
		check(SourceMetadata);

		FPCGDataView DataView{};
		DataView.ViewedData = Input.Data;
		if (Settings->bSelectionAsInput)
		{
			DataView.Selection.bAllAttributes = false;
			for (const FString& AttributeString : AttributeStrings)
			{
				DataView.Selection.Attributes.AddUnique(FPCGAttributePropertySelector::CreateSelectorFromString(AttributeString));
			}
		}
		else
		{
			DataView.Selection = Settings->Selection;
		}

		UPCGDataViewData* DataViewData = InContext->NewObject_AnyThread<UPCGDataViewData>(InContext);
		DataViewData->Initialize(MoveTemp(DataView));

		Outputs.Emplace_GetRef(Input).Data = DataViewData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
