// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGConvexHull2D.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Data/PCGBasePointData.h"

#include "Math/ConvexHull2d.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGConvexHull2D)

#define LOCTEXT_NAMESPACE "PCGConvexHull2DElement"

FPCGElementPtr UPCGConvexHull2DSettings::CreateElement() const
{
	return MakeShared<FPCGConvexHull2DElement>();
}

bool FPCGConvexHull2DElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGConvexHullElement::Execute);

	check(Context);

	const UPCGConvexHull2DSettings* Settings = Context->GetInputSettings<UPCGConvexHull2DSettings>();
	check(Settings);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Input.Data);

		if (!PointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InputNotPointData", "Input is not a point data"));
			continue;
		}

		TArray<FVector> PointsPositions;
		TArray<int32> ConvexHullIndices;

		const FConstPCGPointValueRanges InRanges(PointData);
		Algo::Transform(InRanges.TransformRange, PointsPositions, [](const FTransform& Transform) { return Transform.GetLocation(); });

		ConvexHull2D::ComputeConvexHull(PointsPositions, ConvexHullIndices);

		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
		UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(Context);
		
		FPCGInitializeFromDataParams InitializeFromDataParams(PointData);
		InitializeFromDataParams.bInheritSpatialData = false;
		OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);
		
		OutputPointData->SetNumPoints(ConvexHullIndices.Num());
		OutputPointData->AllocateProperties(PointData->GetAllocatedProperties());
		OutputPointData->CopyUnallocatedPropertiesFrom(PointData);

		FPCGPointValueRanges OutRanges(OutputPointData, /*bAllocate=*/false);
		int32 WriteIndex = 0;
		for (int32 Index : ConvexHullIndices)
		{
			OutRanges.SetFromValueRanges(WriteIndex, InRanges, Index);
			++WriteIndex;
		}

		Output.Data = OutputPointData;
		Output.Pin = PCGPinConstants::DefaultOutputLabel;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
