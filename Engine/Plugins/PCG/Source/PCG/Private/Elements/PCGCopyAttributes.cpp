// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCopyAttributes.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGEdge.h"
#include "PCGParamData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGCopyPoints.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGMetadataHelpers.h"

// Keep loctext namespace to its previous value to avoid to redo loc for this.

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCopyAttributes)
#define LOCTEXT_NAMESPACE "PCGMetadataElement"


UPCGCopyAttributesSettings::UPCGCopyAttributesSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		InputSource.SetAttributeName(PCGMetadataAttributeConstants::LastAttributeName);
		OutputTarget.SetAttributeName(PCGMetadataAttributeConstants::SourceAttributeName);
	}
	else
	{
		// Previous default object was: None for source and target attribute, density for point property
		// and Property to attribute
		// Recreate the same default
		InputSource.SetPointProperty(EPCGPointProperties::Density);
		OutputTarget.SetAttributeName(NAME_None);
	}
}

FString UPCGCopyAttributesSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	if (bCopyAllAttributes)
	{
		return LOCTEXT("NoteTitleAllAttributes", "All Attributes").ToString();
	}

	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGCopyAttributesSettings, InputSource)) || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGCopyAttributesSettings, OutputTarget)))
	{
		return FString();
	}
#endif

	return FString::Printf(TEXT("%s -> %s"), *InputSource.GetDisplayText().ToString(), *OutputTarget.GetDisplayText().ToString());
}

#if WITH_EDITOR
EPCGChangeType UPCGCopyAttributesSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGCopyAttributesSettings::CreateElement() const
{
	return MakeShared<FPCGCopyAttributesElement>();
}

#if WITH_EDITOR
FText UPCGCopyAttributesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Copy from the Input Source to Output Target attribute.");
}

void UPCGCopyAttributesSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	if (DataVersion < FPCGCustomVersion::UpdateAttributePropertyInputSelector
		&& OutputTarget.GetSelection() == EPCGAttributePropertySelection::Attribute
		&& OutputTarget.GetAttributeName() == PCGMetadataAttributeConstants::SourceAttributeName)
	{
		// Previous behavior of the output target for this node was:
		// None => LastCreated
		OutputTarget.SetAttributeName(PCGMetadataAttributeConstants::LastCreatedAttributeName);
	}

	if (bWasDeprecated)
	{
		UPCGPin* TargetPin = InOutNode->GetInputPin(PCGCopyPointsConstants::TargetPointsLabel);
		UPCGPin* SourcePin = InOutNode->GetInputPin(PCGCopyPointsConstants::SourcePointsLabel);

		check(TargetPin && SourcePin);

		if (!SourcePin->IsConnected())
		{
			for (UPCGEdge* Edge : TargetPin->Edges)
			{
				if (!Edge || !Edge->InputPin)
				{
					continue;
				}

				Edge->InputPin->AddEdgeTo(SourcePin);
			}
		}
	}

	bWasDeprecated = false;

	Super::ApplyDeprecation(InOutNode);
}

void UPCGCopyAttributesSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	if (!InOutNode)
	{
		return;
	}

	bWasDeprecated = InputPins.ContainsByPredicate([](const TObjectPtr<UPCGPin>& Pin) -> bool { return Pin && Pin->Properties.Label == PCGPinConstants::DefaultInputLabel; });

	InOutNode->RenameInputPin(PCGPinConstants::DefaultInputLabel, PCGCopyPointsConstants::TargetPointsLabel, /*bInBroadcastUpdate=*/false);
	InOutNode->RenameInputPin(TEXT("Attribute"), PCGCopyPointsConstants::SourcePointsLabel, /*bInBroadcastUpdate=*/false);
}

TArray<FText> UPCGCopyAttributesSettings::GetNodeTitleAliases() const
{
	return { LOCTEXT("CopyAttributesAliasTransfer", "Transfer Attributes") };
}

#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGCopyAttributesSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGCopyPointsConstants::TargetPointsLabel, EPCGDataType::Any).SetRequiredPin();
	Properties.Emplace_GetRef(PCGCopyPointsConstants::SourcePointsLabel, EPCGDataType::Any).SetRequiredPin();

	return Properties;
}

TArray<FPCGPinProperties> UPCGCopyAttributesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return Properties;
}

bool FPCGCopyAttributesElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataOperationElement::Execute);

	const UPCGCopyAttributesSettings* Settings = Context->GetInputSettings<UPCGCopyAttributesSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Sources = Context->InputData.GetInputsByPin(PCGCopyPointsConstants::SourcePointsLabel);
	TArray<FPCGTaggedData> Targets = Context->InputData.GetInputsByPin(PCGCopyPointsConstants::TargetPointsLabel);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	
	const bool bCopyAllAttributes = Settings->bCopyAllAttributes;
	const bool bCopyAllDomains = bCopyAllAttributes && Settings->bCopyAllDomains;

	const int32 NumSources = Sources.Num();
	const int32 NumTargets = Targets.Num();

	if (NumSources == 0 || NumTargets == 0)
	{
		// Nothing to do
		return true;
	}

	int32 NumIterations = 0;
	switch (Settings->Operation)
	{
	case EPCGCopyAttributesOperation::CopyEachSourceOnEveryTarget:
		NumIterations = NumSources * NumTargets;
		break;
	case EPCGCopyAttributesOperation::MergeSourcesAndCopyToAllTargets:
		NumIterations = NumTargets;
		break;
	case EPCGCopyAttributesOperation::CopyEachSourceToEachTargetRespectively:
	{
		if (NumSources != NumTargets && NumSources != 1 && NumTargets != 1)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MismatchNum", "Num Sources ({0}) mismatches with Num Targets ({1}). Only supports N:N, 1:N and N:1 operation."), NumSources, NumTargets), Context);
			Outputs = Sources;
			return true;
		}

		NumIterations = NumTargets;
		break;
	}
	}

	for (int32 i = 0; i < NumIterations; ++i)
	{
		const int32 TargetIndex = i % NumTargets;
		const FPCGTaggedData& Target = Targets[TargetIndex];

		FPCGTaggedData& Output = Outputs.Add_GetRef(Target);
		const UPCGData* TargetData = Target.Data;

		bool bSuccess = false;
		UPCGData* OutputData = nullptr;

		auto DoCopy = [this, TargetData, &Sources, Context, Settings, &bSuccess, &OutputData, bCopyAllAttributes, bCopyAllDomains](int32 SourceIndex)
		{
			const FPCGTaggedData& Source = Sources[SourceIndex];
			const UPCGData* SourceData = Source.Data;

			const bool bIsSameData = TargetData == SourceData;

			if (!TargetData || !SourceData)
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data"));
				return;
			}

			const UPCGMetadata* SourceMetadata = SourceData->ConstMetadata();

			if (!SourceMetadata)
			{
				PCGE_LOG(Warning, GraphAndLog, LOCTEXT("MissingMetadata", "Source has no metadata"));
				return;
			}

			// Early out when trying to copy all attributes from self, since nothing will happen (Only when we do not have a mapping or copying all domains)
			if (bIsSameData && ((bCopyAllAttributes && Settings->MetadataDomainsMapping.IsEmpty()) || bCopyAllDomains))
			{
				PCGE_LOG(Verbose, LogOnly, LOCTEXT("TrivialAllCopy", "Copying all attributes on itself is a trivial operation."));
				return;
			}

			// Early out when trying to copy same value to self
			if (bIsSameData && Settings->InputSource == Settings->OutputTarget && !bCopyAllAttributes)
			{
				PCGE_LOG(Verbose, LogOnly, LOCTEXT("TrivialCopy", "Copying attribute to itself is a trivial operation."));
				return;
			}

			if (!OutputData)
			{
				OutputData = TargetData->DuplicateData(Context);
			}

			if (bCopyAllAttributes)
			{
				PCGMetadataHelpers::FPCGCopyAllAttributesParams Params
				{
					.SourceData = SourceData,
					.TargetData = OutputData,
					.OptionalContext = Context,
				};

				if (bCopyAllDomains)
				{
					Params.InitializeMappingForAllDomains();
				}
				else
				{
					Params.InitializeMappingFromDomainNames(Settings->MetadataDomainsMapping);
				}

				// Nothing to do
				if (Params.DomainMapping.IsEmpty())
				{
					return;
				}
				
				bSuccess |= PCGMetadataHelpers::CopyAllAttributes(Params);
			}
			else
			{
				PCGMetadataHelpers::FPCGCopyAttributeParams Params{};
				Params.SourceData = SourceData;
				Params.TargetData = OutputData;
				Params.InputSource = Settings->InputSource;
				Params.OutputTarget = Settings->OutputTarget;
				Params.OptionalContext = Context;
				Params.bSameOrigin = bIsSameData;

				bSuccess |= PCGMetadataHelpers::CopyAttribute(Params);
			}
		};

		switch (Settings->Operation)
		{
		case EPCGCopyAttributesOperation::CopyEachSourceOnEveryTarget:
		{
			DoCopy(i / NumTargets);
			break;
		}
		case EPCGCopyAttributesOperation::MergeSourcesAndCopyToAllTargets:
		{
			for (int32 j = 0; j < NumSources; ++j)
			{
				DoCopy(j);
			}
			break;
		}
		case EPCGCopyAttributesOperation::CopyEachSourceToEachTargetRespectively:
		default:
		{
			DoCopy(FMath::Min(i, NumSources - 1));
			break;
		}
		}

		if (bSuccess && OutputData)
		{
			Output.Data = OutputData;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
