// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCopyPoints.h"

#include "PCGContext.h"
#include "Compute/PCGKernelHelpers.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGCopyPointsAnalysisKernel.h"
#include "Elements/PCGCopyPointsKernel.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"

#include "Async/ParallelFor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCopyPoints)

#define LOCTEXT_NAMESPACE "PCGCopyPointsElement"

#if WITH_EDITOR
FText UPCGCopyPointsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "For each point pair from the source and the target, create a copy, inheriting properties & attributes depending on the node settings.");
}

void UPCGCopyPointsSettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);

	CreateParams.NodeInputPinsToWire = { PCGCopyPointsConstants::SourcePointsLabel, PCGCopyPointsConstants::TargetPointsLabel };

	UPCGCopyPointsKernel* CopyKernel = PCGKernelHelpers::CreateKernel<UPCGCopyPointsKernel>(InOutContext, CreateParams, OutKernels, OutEdges);
	CopyKernel->SetMatchBasedOnAttribute(bMatchBasedOnAttribute);

	if (bMatchBasedOnAttribute)
	{
		// Don't wire analysis kernel to node output pin, wire manually to the copy kernel below.
		CreateParams.NodeOutputPinsToWire.Empty();

		UPCGComputeKernel* AnalysisKernel = PCGKernelHelpers::CreateKernel<UPCGCopyPointsAnalysisKernel>(InOutContext, CreateParams, OutKernels, OutEdges);

		// Wire analysis kernel to copy kernel.
		OutEdges.Emplace(FPCGPinReference(AnalysisKernel, PCGPinConstants::DefaultOutputLabel), FPCGPinReference(CopyKernel, PCGCopyPointsConstants::SelectedFlagsPinLabel));
	}
}
#endif

TArray<FPCGPinProperties> UPCGCopyPointsSettings::InputPinProperties() const
{
	// Note: If executing on the GPU, we need to prevent multiple connections on inputs, since it is not supported at this time.
	// Also note: Since the ShouldExecuteOnGPU() is already tied to structural changes, we don't need to implement any logic for this in GetChangeTypeForProperty()
	const bool bAllowMultipleConnections = !ShouldExecuteOnGPU();

	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& SourcePinProperty = PinProperties.Emplace_GetRef(PCGCopyPointsConstants::SourcePointsLabel, EPCGDataType::Point, bAllowMultipleConnections);
	SourcePinProperty.SetRequiredPin();

	FPCGPinProperties& TargetPinProperty = PinProperties.Emplace_GetRef(PCGCopyPointsConstants::TargetPointsLabel, EPCGDataType::Point, bAllowMultipleConnections);
	TargetPinProperty.SetRequiredPin();

	return PinProperties;
}

FPCGElementPtr UPCGCopyPointsSettings::CreateElement() const
{
	return MakeShared<FPCGCopyPointsElement>();
}

bool FPCGCopyPointsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCopyPointsElement::Execute);

	const UPCGCopyPointsSettings* Settings = Context->GetInputSettings<UPCGCopyPointsSettings>();
	check(Settings && !Settings->ShouldExecuteOnGPU());

	const EPCGCopyPointsInheritanceMode RotationInheritance = Settings->RotationInheritance;
	const EPCGCopyPointsInheritanceMode ScaleInheritance = Settings->ScaleInheritance;
	const EPCGCopyPointsInheritanceMode ColorInheritance = Settings->ColorInheritance;
	const EPCGCopyPointsInheritanceMode SeedInheritance = Settings->SeedInheritance;
	const EPCGCopyPointsMetadataInheritanceMode AttributeInheritance = Settings->AttributeInheritance;
	const bool bApplyTargetRotationToPositions = Settings->bApplyTargetRotationToPositions;
	const bool bApplyTargetScaleToPositions = Settings->bApplyTargetScaleToPositions;

	const TArray<FPCGTaggedData> Sources = Context->InputData.GetInputsByPin(PCGCopyPointsConstants::SourcePointsLabel);
	const TArray<FPCGTaggedData> Targets = Context->InputData.GetInputsByPin(PCGCopyPointsConstants::TargetPointsLabel);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const int32 NumSources = Sources.Num();
	const int32 NumTargets = Targets.Num();

	if (NumSources == 0 || NumTargets == 0)
	{
		// Nothing to do
		return true;
	}

	int32 NumIterations = 0;
	if (Settings->bCopyEachSourceOnEveryTarget)
	{
		NumIterations = NumSources * NumTargets;
	}
	else
	{
		if (NumSources != NumTargets && NumSources != 1 && NumTargets != 1)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NumDataMismatch", "Num Sources ({0}) mismatches with Num Targets ({1}). Only supports N:N, 1:N and N:1 operation."), NumSources, NumTargets), Context);
			Outputs = Sources;
			return true;
		}

		NumIterations = FMath::Max(NumSources, NumTargets);
	}

	for (int32 i = 0; i < NumIterations; ++i)
	{
		const int32 SourceIndex = Settings->bCopyEachSourceOnEveryTarget ? (i / NumTargets) : FMath::Min(i, Sources.Num() - 1);
		const int32 TargetIndex = Settings->bCopyEachSourceOnEveryTarget ? (i % NumTargets) : FMath::Min(i, Targets.Num() - 1);

		const FPCGTaggedData& Source = Sources[SourceIndex];
		const FPCGTaggedData& Target = Targets[TargetIndex];

		FPCGTaggedData& Output = Outputs.Add_GetRef(Source);
		if (!Source.Data || !Target.Data) 
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data"));
			return true;
		}

		const UPCGSpatialData* SourceSpatialData = Cast<UPCGSpatialData>(Source.Data);
		const UPCGSpatialData* TargetSpatialData = Cast<UPCGSpatialData>(Target.Data);

		if (!SourceSpatialData || !TargetSpatialData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("CouldNotObtainSpatialData", "Unable to get Spatial Data from input"));
			return true;
		}

		const UPCGBasePointData* SourcePointData = SourceSpatialData->ToBasePointData(Context);
		const UPCGBasePointData* TargetPointData = TargetSpatialData->ToBasePointData(Context);

		if (!SourcePointData || !TargetPointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("CouldNotGetPointData", "Unable to get Point Data from input"));
			return true;
		}

		const UPCGMetadata* SourcePointMetadata = SourcePointData->Metadata;
		const UPCGMetadata* TargetPointMetadata = TargetPointData->Metadata;

		UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(Context);
		Output.Data = OutPointData;

		// Make sure that output contains both collection of tags from source and target
		if (Settings->TagInheritance == EPCGCopyPointsTagInheritanceMode::Target)
		{
			Output.Tags = Target.Tags;
		}
		else if (Settings->TagInheritance == EPCGCopyPointsTagInheritanceMode::Both)
		{
			Output.Tags.Append(Target.Tags);
		}

		// RootMetadata will be parent to the ouptut metadata, while NonRootMetadata will carry attributes from the input not selected for inheritance
		// Note that this is a preference, as we can and should pick more efficiently in the trivial cases
		const UPCGMetadata* RootMetadata = nullptr;
		const UPCGMetadata* NonRootMetadata = nullptr;

		const bool bSourceHasMetadata = (SourcePointMetadata->GetAttributeCount() > 0 && SourcePointMetadata->GetItemCountForChild() > 0);
		const bool bTargetHasMetadata = (TargetPointMetadata->GetAttributeCount() > 0 && TargetPointMetadata->GetItemCountForChild() > 0);

		bool bInheritMetadataFromSource = true;
		bool bProcessMetadata = (bSourceHasMetadata || bTargetHasMetadata);

		if (AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::SourceOnly)
		{
			bInheritMetadataFromSource = true;
			bProcessMetadata = bSourceHasMetadata;
			RootMetadata = SourcePointMetadata;
			NonRootMetadata = nullptr;

			FPCGInitializeFromDataParams InitializeFromDataParams(SourcePointData);
			InitializeFromDataParams.bInheritSpatialData = false;
			OutPointData->InitializeFromDataWithParams(InitializeFromDataParams);
		}
		else if (AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::TargetOnly)
		{
			bInheritMetadataFromSource = false;
			bProcessMetadata = bTargetHasMetadata;
			RootMetadata = TargetPointMetadata;
			NonRootMetadata = nullptr;

			FPCGInitializeFromDataParams InitializeFromDataParams(TargetPointData);
			InitializeFromDataParams.bInheritSpatialData = false;
			OutPointData->InitializeFromDataWithParams(InitializeFromDataParams);
		}
		else if (AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::SourceFirst || AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::TargetFirst)
		{
			if (AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::SourceFirst)
			{
				bInheritMetadataFromSource = bSourceHasMetadata || !bTargetHasMetadata;
			}
			else // TargetFirst
			{
				bInheritMetadataFromSource = !bTargetHasMetadata && bSourceHasMetadata;
			}

			RootMetadata = bInheritMetadataFromSource ? SourcePointMetadata : TargetPointMetadata;
			NonRootMetadata = bInheritMetadataFromSource ? TargetPointMetadata : SourcePointMetadata;

			FPCGInitializeFromDataParams InitializeFromDataParams(bInheritMetadataFromSource ? SourcePointData : TargetPointData);
			InitializeFromDataParams.bInheritSpatialData = false;			
			OutPointData->InitializeFromDataWithParams(InitializeFromDataParams);
		}
		else // None
		{
			FPCGInitializeFromDataParams InitializeFromDataParams(SourcePointData);
			InitializeFromDataParams.bInheritSpatialData = false;
			InitializeFromDataParams.bInheritMetadata = false;
			InitializeFromDataParams.bInheritAttributes = false;
			OutPointData->InitializeFromDataWithParams(InitializeFromDataParams);

			bProcessMetadata = false;
			RootMetadata = NonRootMetadata = nullptr;
		}

		// Always use the target actor from the target, irrespective of the source
		OutPointData->TargetActor = TargetPointData->TargetActor;

		UPCGMetadata* OutMetadata = OutPointData->Metadata;
		check(OutMetadata);

		TArray<FPCGMetadataAttributeBase*> AttributesToSet;
		TArray<const FPCGMetadataAttributeBase*> NonRootAttributes;
		TArray<TTuple<int64, int64>> AllMetadataEntries;
		TArray<TArray<TTuple<PCGMetadataEntryKey, PCGMetadataValueKey>>> AttributeValuesToSet;

		const int32 NumPoints = SourcePointData->GetNumPoints() * TargetPointData->GetNumPoints();

		if (bProcessMetadata)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCopyPointsElement::Execute::SetupMetadata);
			if (NonRootMetadata)
			{
				// Prepare the attributes from the non-root that we'll need to use to copy values over
				TArray<FName> AttributeNames;
				TArray<EPCGMetadataTypes> AttributeTypes;
				NonRootMetadata->GetAttributes(AttributeNames, AttributeTypes);

				for (const FName& AttributeName : AttributeNames)
				{
					if (!OutMetadata->HasAttribute(AttributeName))
					{
						const FPCGMetadataAttributeBase* Attribute = NonRootMetadata->GetConstAttribute(AttributeName);
						if (FPCGMetadataAttributeBase* NewAttribute = OutMetadata->CopyAttribute(Attribute, AttributeName, /*bKeepRoot=*/false, /*bCopyEntries=*/false, /*bCopyValues=*/true))
						{
							AttributesToSet.Add(NewAttribute);
							NonRootAttributes.Add(Attribute);
						}
					}
				}

				// Considering writing to the attribute value requires a lock, we'll gather the value keys to write
				// and do it on a 1-thread-per-attribute basis at the end
				AttributeValuesToSet.SetNum(AttributesToSet.Num());

				for (TArray<TTuple<PCGMetadataEntryKey, PCGMetadataValueKey>>& AttributeValues : AttributeValuesToSet)
				{
					AttributeValues.SetNumUninitialized(NumPoints);
				}
			}

			// Preallocate the metadata entries array if we're going to use it
			AllMetadataEntries.SetNumUninitialized(NumPoints);
		}

		// Properties that might need to be allocated
		EPCGPointNativeProperties OptionalProperties = EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Color;
		if (bProcessMetadata)
		{
			OptionalProperties |= EPCGPointNativeProperties::MetadataEntry;
		}

		auto InitializeFunc = [OutPointData, SourcePointData, TargetPointData, NumPoints, bProcessMetadata, ColorInheritance, SeedInheritance, OptionalProperties]()
		{
			OutPointData->SetNumPoints(NumPoints, /*bInitializeValues=*/false);
			EPCGPointNativeProperties SourceAllocatedProperties = SourcePointData->GetAllocatedProperties();
			EPCGPointNativeProperties TargetAllocatedProperties = TargetPointData->GetAllocatedProperties();
						
			// Allocate all properties from source except properties that might not need to be
			EPCGPointNativeProperties PropertiesToAllocate = SourcePointData->GetAllocatedProperties() & ~OptionalProperties;

			// Allocate MetadataEntry
			if (bProcessMetadata)
			{
				PropertiesToAllocate |= EPCGPointNativeProperties::MetadataEntry;
			}
									
			auto GetPropertyToAllocate = [&PropertiesToAllocate, SourceAllocatedProperties, TargetAllocatedProperties](EPCGCopyPointsInheritanceMode Inheritance, EPCGPointNativeProperties Property)
			{
				if (Inheritance == EPCGCopyPointsInheritanceMode::Relative)
				{
					return ((SourceAllocatedProperties | TargetAllocatedProperties) & Property);
				}
				else if (Inheritance == EPCGCopyPointsInheritanceMode::Source)
				{
					return (SourceAllocatedProperties & Property);
				}
				else // if (Inheritance == EPCGCopyPointsInheritanceMode::Target)
				{
					return (TargetAllocatedProperties & Property);
				}
			};

			PropertiesToAllocate |= EPCGPointNativeProperties::Transform;
			PropertiesToAllocate |= GetPropertyToAllocate(SeedInheritance, EPCGPointNativeProperties::Seed);
			PropertiesToAllocate |= GetPropertyToAllocate(ColorInheritance, EPCGPointNativeProperties::Color);

			OutPointData->AllocateProperties(PropertiesToAllocate);
			OutPointData->CopyUnallocatedPropertiesFrom(SourcePointData);
		};

		const int32 TargetNumPoints = TargetPointData->GetNumPoints();
		
		// Use implicit capture, since we capture a lot
		auto ProcessRangeFunc = [&](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			int32 NumWritten = 0;
						
			const FConstPCGPointValueRanges SourceRanges(SourcePointData);
			const FConstPCGPointValueRanges TargetRanges(TargetPointData);
			FPCGPointValueRanges OutRanges(OutPointData, /*bAllocate=*/false);
												
			for (int32 Index = StartReadIndex; Index < (StartReadIndex+Count); ++Index)
			{
				const int32 SourceIndex = Index / TargetNumPoints;
				const int32 TargetIndex = Index % TargetNumPoints;
								
				const int32 WriteIndex = StartWriteIndex + NumWritten;

				// Copy properties from source
				FPCGPoint OutPoint = SourceRanges.GetPoint(SourceIndex);

				const FTransform& SourcePointTransform = SourceRanges.TransformRange[SourceIndex];
				const FTransform& TargetPointTransform = TargetRanges.TransformRange[TargetIndex];

				FTransform SourceTransform(SourcePointTransform.GetLocation());
				FTransform TargetTransform(TargetPointTransform.GetLocation());

				// Set Rotation, Scale, and Color based on inheritance mode
				if (RotationInheritance != EPCGCopyPointsInheritanceMode::Target)
				{
					SourceTransform.SetRotation(SourcePointTransform.GetRotation());
				}

				if (RotationInheritance != EPCGCopyPointsInheritanceMode::Source || bApplyTargetRotationToPositions)
				{
					TargetTransform.SetRotation(TargetPointTransform.GetRotation());
				}

				if (ScaleInheritance != EPCGCopyPointsInheritanceMode::Target)
				{
					SourceTransform.SetScale3D(SourcePointTransform.GetScale3D());
				}

				if (ScaleInheritance != EPCGCopyPointsInheritanceMode::Source || bApplyTargetScaleToPositions)
				{
					TargetTransform.SetScale3D(TargetPointTransform.GetScale3D());
				}

				OutPoint.Transform = SourceTransform * TargetTransform;

				if (RotationInheritance == EPCGCopyPointsInheritanceMode::Source && bApplyTargetRotationToPositions)
				{
					OutPoint.Transform.SetRotation(SourcePointTransform.GetRotation());
				}

				if (ScaleInheritance == EPCGCopyPointsInheritanceMode::Source && bApplyTargetScaleToPositions)
				{
					OutPoint.Transform.SetScale3D(SourcePointTransform.GetScale3D());
				}

				if (ColorInheritance == EPCGCopyPointsInheritanceMode::Relative)
				{
					OutPoint.Color = SourceRanges.ColorRange[SourceIndex] * TargetRanges.ColorRange[TargetIndex];
				}
				else if (ColorInheritance == EPCGCopyPointsInheritanceMode::Target)
				{
					OutPoint.Color = TargetRanges.ColorRange[TargetIndex];
				}
				else // if (ColorInheritance == EPCGCopyPointsInheritanceMode::Source)
				{
					OutPoint.Color = SourceRanges.ColorRange[SourceIndex];
				}

				// Set seed based on inheritance mode
				if (SeedInheritance == EPCGCopyPointsInheritanceMode::Relative)
				{
					OutPoint.Seed = PCGHelpers::ComputeSeed(SourceRanges.SeedRange[SourceIndex], TargetRanges.SeedRange[TargetIndex]);
				}
				else if (SeedInheritance == EPCGCopyPointsInheritanceMode::Target)
				{
					OutPoint.Seed = TargetRanges.SeedRange[TargetIndex];
				}
				else // if (SeedInheritance == EPCGCopyPointsInheritanceMode::Source)
				{
					OutPoint.Seed = SourceRanges.SeedRange[SourceIndex];
				}

				if (bProcessMetadata)
				{
					const int64 RootMetadataEntry = bInheritMetadataFromSource ? SourceRanges.MetadataEntryRange[SourceIndex] : TargetRanges.MetadataEntryRange[TargetIndex];
					const int64 NonRootMetadataEntry = bInheritMetadataFromSource ? TargetRanges.MetadataEntryRange[TargetIndex] : SourceRanges.MetadataEntryRange[SourceIndex];

					OutPoint.MetadataEntry = OutMetadata->AddEntryPlaceholder();
					AllMetadataEntries[Index] = TTuple<int64, int64>(OutPoint.MetadataEntry, RootMetadataEntry);

					if (NonRootMetadata)
					{
						// Copy EntryToValue key mappings from NonRootAttributes - no need to do it if the non-root uses the default values
						if (NonRootMetadataEntry != PCGInvalidEntryKey)
						{
							for (int32 AttributeIndex = 0; AttributeIndex < NonRootAttributes.Num(); ++AttributeIndex)
							{
								AttributeValuesToSet[AttributeIndex][Index] = TTuple<PCGMetadataEntryKey, PCGMetadataValueKey>(OutPoint.MetadataEntry, NonRootAttributes[AttributeIndex]->GetValueKey(NonRootMetadataEntry));
							}
						}
						else
						{
							for (int32 AttributeIndex = 0; AttributeIndex < NonRootAttributes.Num(); ++AttributeIndex)
							{
								AttributeValuesToSet[AttributeIndex][Index] = TTuple<PCGMetadataEntryKey, PCGMetadataValueKey>(OutPoint.MetadataEntry, PCGDefaultValueKey);
							}
						}
					}
				}
				else
				{
					// Reset the metadata entry if we have no metadata.
					OutPoint.MetadataEntry = PCGInvalidEntryKey;
				}

				OutRanges.SetFromPoint(WriteIndex, OutPoint);
				++NumWritten;
			}

			check(NumWritten == Count);
			return NumWritten;
		};

		FPCGAsync::AsyncProcessingOneToOneRangeEx(
			&Context->AsyncState,
			NumPoints,
			InitializeFunc,
			ProcessRangeFunc,
			/*bTimeSliceEnabled=*/false);
				
		if (bProcessMetadata)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCopyPointsElement::Execute::SetMetadata);
			check(AttributesToSet.Num() == AttributeValuesToSet.Num());
			if (AttributesToSet.Num() > 0)
			{
				int32 AttributeOffset = 0;
				const int32 DefaultAttributePerDispatch = 128;
				int32 AttributePerDispatch = DefaultAttributePerDispatch;
				if (Context->AsyncState.NumAvailableTasks > 0)
				{
					AttributePerDispatch = FMath::Min(Context->AsyncState.NumAvailableTasks, AttributePerDispatch);
				}

				while (AttributeOffset < AttributesToSet.Num())
				{
					const int32 AttributeCountInCurrentDispatch = FMath::Min(AttributePerDispatch, AttributesToSet.Num() - AttributeOffset);
					ParallelFor(AttributeCountInCurrentDispatch, [AttributeOffset, &AttributesToSet, &AttributeValuesToSet](int32 WorkerIndex)
					{
						LLM_SCOPE_BYTAG(PCG);
						FPCGMetadataAttributeBase* Attribute = AttributesToSet[AttributeOffset + WorkerIndex];
						const TArray<TTuple<PCGMetadataEntryKey, PCGMetadataValueKey>>& Values = AttributeValuesToSet[AttributeOffset + WorkerIndex];
						check(Attribute);
						Attribute->SetValuesFromValueKeys(Values, /*bResetValueOnDefaultValueKey*/false); // no need for the reset here, our points will not have any prior value for these attributes
					});

					AttributeOffset += AttributeCountInCurrentDispatch;
				}
			}

			OutMetadata->AddDelayedEntries(AllMetadataEntries);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
