// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCopyPointsAnalysisKernel.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/DataInterfaces/Elements/PCGCopyPointsAnalysisDataInterface.h"
#include "Elements/PCGCopyPoints.h"
#include "Elements/PCGCopyPointsKernelShared.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include "ShaderCompilerCore.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCopyPointsAnalysisKernel)

#define LOCTEXT_NAMESPACE "PCGCopyPointsAnalysisKernel"

bool UPCGCopyPointsAnalysisKernel::IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCopyPointsAnalysisKernel::IsKernelDataValid);

	if (!Super::IsKernelDataValid(InDataBinding, InContext))
	{
		return false;
	}

	if (!InDataBinding)
	{
		return true;
	}

	return PCGCopyPointsKernel::IsKernelDataValid(this, InDataBinding, InContext);
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGCopyPointsAnalysisKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	const FPCGKernelParams* KernelParams = InBinding->GetCachedKernelParams(this);

	if (!ensure(KernelParams))
	{
		return nullptr;
	}

	// Code assumes single output pin.
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	const FPCGKernelPin SourceKernelPin(GetKernelIndex(), PCGCopyPointsConstants::SourcePointsLabel, /*bIsInput=*/true);
	const FPCGKernelPin TargetKernelPin(GetKernelIndex(), PCGCopyPointsConstants::TargetPointsLabel, /*bIsInput=*/true);

	const TSharedPtr<const FPCGDataCollectionDesc> SourcesDesc = InBinding->ComputeKernelPinDataDesc(SourceKernelPin);
	const TSharedPtr<const FPCGDataCollectionDesc> TargetsDesc = InBinding->ComputeKernelPinDataDesc(TargetKernelPin);

	if (!ensure(SourcesDesc && TargetsDesc))
	{
		return nullptr;
	}

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeShared();

	const uint32 NumSourceDatas = SourcesDesc->GetDataDescriptions().Num();
	const uint32 NumTargetDatas = TargetsDesc->GetDataDescriptions().Num();

	// Output: a single attribute set with a single attribute true/false value per NumSource*NumTarget possible outputs.
	const bool bCopyEachSourceOnEveryTarget = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bCopyEachSourceOnEveryTarget));

	// Validation should have ensured already that these are valid: (S, T) = (N, N), (N, 1) or (1, N).
	const int32 ElementCount = bCopyEachSourceOnEveryTarget ? (NumSourceDatas * NumTargetDatas) : FMath::Max(NumSourceDatas, NumTargetDatas);
	FPCGDataDesc& DataDesc = OutDataDesc->GetDataDescriptionsMutable().Emplace_GetRef(EPCGDataType::Param, ElementCount);

	FPCGKernelAttributeKey SelectedFlagAttributeKey = FPCGKernelAttributeKey(PCGCopyPointsConstants::SelectedFlagAttributeName, EPCGKernelAttributeType::Bool);
	DataDesc.AddAttribute(SelectedFlagAttributeKey, InBinding);

	return OutDataDesc;
}

int UPCGCopyPointsAnalysisKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const TSharedPtr<const FPCGDataCollectionDesc> OutputPinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultOutputLabel, /*bIsInput=*/false);
	return ensure(OutputPinDesc) ? OutputPinDesc->ComputeTotalElementCount() : 0;
}

#if WITH_EDITOR
void UPCGCopyPointsAnalysisKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGCopyPointsAnalysisDataInterface> NodeDI = InOutContext.NewObject_AnyThread<UPCGCopyPointsAnalysisDataInterface>(InObjectOuter);
	NodeDI->SetProducerKernel(this);

	OutDataInterfaces.Add(NodeDI);
}
#endif

void UPCGCopyPointsAnalysisKernel::GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const
{
	// Register the attribute this node creates.
	OutKeys.AddUnique(FPCGKernelAttributeKey(PCGCopyPointsConstants::SelectedFlagAttributeName, EPCGKernelAttributeType::Bool));
}

void UPCGCopyPointsAnalysisKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGCopyPointsConstants::SourcePointsLabel, EPCGDataType::Point);
	OutPins.Emplace(PCGCopyPointsConstants::TargetPointsLabel, EPCGDataType::Point);
}

void UPCGCopyPointsAnalysisKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);
}

#undef LOCTEXT_NAMESPACE
