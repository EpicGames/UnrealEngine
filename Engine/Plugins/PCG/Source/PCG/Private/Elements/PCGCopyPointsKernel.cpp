// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCopyPointsKernel.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/DataInterfaces/Elements/PCGCopyPointsDataInterface.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Elements/PCGCopyPoints.h"
#include "Elements/PCGCopyPointsKernelShared.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include "ShaderCompilerCore.h"
#include "Containers/StaticArray.h"
#include "Internationalization/Regex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCopyPointsKernel)

#define LOCTEXT_NAMESPACE "PCGCopyPointsKernel"

bool UPCGCopyPointsKernel::IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGCopyPointsKernel::IsKernelDataValid);

	if (!Super::IsKernelDataValid(InDataBinding, InContext))
	{
		return false;
	}

	if (!InDataBinding)
	{
		return true;
	}

	return PCGCopyPointsKernel::IsKernelDataValid(this, InDataBinding, InContext);
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGCopyPointsKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	const FPCGKernelParams* KernelParams = InBinding->GetCachedKernelParams(this);

	if (!ensure(KernelParams))
	{
		return nullptr;
	}

	// Code assumes single output pin.
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	// A graph split was injected before this kernel. We expect to find selected flags
	int NumSelected = INDEX_NONE;
	TArray<bool> Selected;

	if (bMatchBasedOnAttribute)
	{
		const int32 AnalysisDataIndex = InBinding->GetFirstInputDataIndex(this, PCGCopyPointsConstants::SelectedFlagsPinLabel);

		if (AnalysisDataIndex != INDEX_NONE)
		{
			const UPCGParamData* AnalysisResultsData = Cast<UPCGParamData>(InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data);
			const UPCGMetadata* AnalysisMetadata = AnalysisResultsData ? AnalysisResultsData->ConstMetadata() : nullptr;
			const FPCGMetadataAttributeBase* AnalysisAttributeBase = AnalysisMetadata ? AnalysisMetadata->GetConstAttribute(PCGCopyPointsConstants::SelectedFlagAttributeName) : nullptr;

			if (AnalysisAttributeBase && AnalysisAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<bool>::Id)
			{
				const FPCGMetadataAttribute<bool>* AnalysisAttribute = static_cast<const FPCGMetadataAttribute<bool>*>(AnalysisAttributeBase);
				const int32 NumElements = AnalysisMetadata->GetItemCountForChild();

				Selected.SetNumUninitialized(NumElements);

				NumSelected = 0;

				for (int64 MetadataKey = 0; MetadataKey < NumElements; ++MetadataKey)
				{
					const bool bSelected = AnalysisAttribute->GetValue(MetadataKey);
					Selected[MetadataKey] = bSelected;
					NumSelected += bSelected ? 1 : 0;
				}
			}
			else
			{
				UE_LOGF(LogPCG, Warning, "No analysis data received by copy points kernel, no points will be copied.");
				if (InBinding->GetInputDataCollection().TaggedData[AnalysisDataIndex].Data->IsA<UPCGProxyForGPUData>())
				{
					UE_LOGF(LogPCG, Error, "Data was not read back.");
				}

				return nullptr;
			}
		}
	}

	const FPCGKernelPin SourceKernelPin(GetKernelIndex(), PCGCopyPointsConstants::SourcePointsLabel, /*bIsInput=*/true);
	const FPCGKernelPin TargetKernelPin(GetKernelIndex(), PCGCopyPointsConstants::TargetPointsLabel, /*bIsInput=*/true);

	const TSharedPtr<const FPCGDataCollectionDesc> SourcePinDesc = InBinding->ComputeKernelPinDataDesc(SourceKernelPin);
	const TSharedPtr<const FPCGDataCollectionDesc> TargetPinDesc = InBinding->ComputeKernelPinDataDesc(TargetKernelPin);

	if (!ensure(SourcePinDesc && TargetPinDesc))
	{
		return nullptr;
	}

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeShared();

	const bool bCopyEachSourceOnEveryTarget = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bCopyEachSourceOnEveryTarget));
	const EPCGCopyPointsInheritanceMode ColorInheritance = KernelParams->GetValueEnum<EPCGCopyPointsInheritanceMode>(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, ColorInheritance));
	const EPCGCopyPointsInheritanceMode SeedInheritance = KernelParams->GetValueEnum<EPCGCopyPointsInheritanceMode>(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, SeedInheritance));
	const EPCGCopyPointsMetadataInheritanceMode AttributeInheritance = KernelParams->GetValueEnum<EPCGCopyPointsMetadataInheritanceMode>(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, AttributeInheritance));
	const EPCGCopyPointsTagInheritanceMode TagInheritance = KernelParams->GetValueEnum<EPCGCopyPointsTagInheritanceMode>(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, TagInheritance));
	const int32 NumSources = SourcePinDesc->GetDataDescriptions().Num();
	const int32 NumTargets = TargetPinDesc->GetDataDescriptions().Num();
	const int32 NumIterations = bCopyEachSourceOnEveryTarget ? NumSources * NumTargets : FMath::Max(NumSources, NumTargets);

	// Properties that might need to be allocated
	const EPCGPointNativeProperties OptionalProperties = EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Color;

	if (NumSources > 0 && NumTargets > 0 && (bCopyEachSourceOnEveryTarget || NumSources == NumTargets || NumSources == 1 || NumTargets == 1))
	{
		for (int32 I = 0; I < NumIterations; ++I)
		{
			if (!Selected.IsEmpty() && !Selected[I])
			{
				continue;
			}

			const int32 SourceIndex = bCopyEachSourceOnEveryTarget ? (I / NumTargets) : FMath::Min(I, NumSources - 1);
			const int32 TargetIndex = bCopyEachSourceOnEveryTarget ? (I % NumTargets) : FMath::Min(I, NumTargets - 1);

			const FPCGDataDesc& SourceDesc = SourcePinDesc->GetDataDescriptions()[SourceIndex];
			const FPCGDataDesc& TargetDesc = TargetPinDesc->GetDataDescriptions()[TargetIndex];

			FPCGDataDesc& ResultDataDesc = OutDataDesc->GetDataDescriptionsMutable().Emplace_GetRef(EPCGDataType::Point, SourceDesc.GetElementCount() * TargetDesc.GetElementCount());

			const TConstArrayView<FPCGKernelAttributeDesc> SourceAttributeDescs = SourceDesc.GetAttributeDescriptions();
			const TConstArrayView<FPCGKernelAttributeDesc> TargetAttributeDescs = TargetDesc.GetAttributeDescriptions();

			TArray<FPCGKernelAttributeDesc>& ResultAttributeDescs = ResultDataDesc.GetAttributeDescriptionsMutable();

			if (AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::SourceFirst)
			{
				ResultAttributeDescs = SourceAttributeDescs;

				for (const FPCGKernelAttributeDesc& AttrDesc : TargetAttributeDescs)
				{
					ResultAttributeDescs.AddUnique(AttrDesc);
				}
			}
			else if (AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::TargetFirst)
			{
				ResultAttributeDescs = TargetAttributeDescs;

				for (const FPCGKernelAttributeDesc& AttrDesc : SourceAttributeDescs)
				{
					ResultAttributeDescs.AddUnique(AttrDesc);
				}
			}
			else if (AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::SourceOnly)
			{
				ResultAttributeDescs = SourceAttributeDescs;
			}
			else if (AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::TargetOnly)
			{
				ResultAttributeDescs = TargetAttributeDescs;
			}

			if (TagInheritance == EPCGCopyPointsTagInheritanceMode::Source || TagInheritance == EPCGCopyPointsTagInheritanceMode::Both)
			{
				for (const int32 SourceTagStringKey : SourceDesc.GetTagStringKeys())
				{
					ResultDataDesc.AddUniqueTagStringKey(SourceTagStringKey);
				}
			}
			
			if (TagInheritance == EPCGCopyPointsTagInheritanceMode::Target || TagInheritance == EPCGCopyPointsTagInheritanceMode::Both)
			{
				for (const int32 TargetTagStringKey : TargetDesc.GetTagStringKeys())
				{
					ResultDataDesc.AddUniqueTagStringKey(TargetTagStringKey);
				}
			}

			const EPCGPointNativeProperties SourceAllocatedProperties = SourceDesc.GetAllocatedProperties();
			const EPCGPointNativeProperties TargetAllocatedProperties = TargetDesc.GetAllocatedProperties();

			// Allocate all properties from source except properties that might not need to be.
			EPCGPointNativeProperties PropertiesToAllocate = SourceAllocatedProperties & ~OptionalProperties;

			auto GetPropertyToAllocate = [&PropertiesToAllocate, SourceAllocatedProperties, TargetAllocatedProperties](EPCGCopyPointsInheritanceMode Inheritance, EPCGPointNativeProperties Property)
			{
				if (Inheritance == EPCGCopyPointsInheritanceMode::Relative)
				{
					return ((SourceAllocatedProperties | TargetAllocatedProperties) & Property);
				}
				else if (Inheritance == EPCGCopyPointsInheritanceMode::Source)
				{
					return (SourceAllocatedProperties & Property);
				}
				else // if (Inheritance == EPCGCopyPointsInheritanceMode::Target)
				{
					return (TargetAllocatedProperties & Property);
				}
			};

			PropertiesToAllocate |= EPCGPointNativeProperties::Transform;
			PropertiesToAllocate |= GetPropertyToAllocate(SeedInheritance, EPCGPointNativeProperties::Seed);
			PropertiesToAllocate |= GetPropertyToAllocate(ColorInheritance, EPCGPointNativeProperties::Color);

			ResultDataDesc.AllocateProperties(PropertiesToAllocate);
		}
	}

	return OutDataDesc;
}

int UPCGCopyPointsKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const TSharedPtr<const FPCGDataCollectionDesc> OutputPinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultOutputLabel, /*bIsInput=*/false);
	return ensure(OutputPinDesc) ? OutputPinDesc->ComputeTotalElementCount() : 0;
}

#if WITH_EDITOR
void UPCGCopyPointsKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGCopyPointsDataInterface> NodeDI = InOutContext.NewObject_AnyThread<UPCGCopyPointsDataInterface>(InObjectOuter);
	NodeDI->SetProducerKernel(this);

	OutDataInterfaces.Add(NodeDI);
}
#endif

void UPCGCopyPointsKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGCopyPointsConstants::SourcePointsLabel, EPCGDataType::Point);
	OutPins.Emplace(PCGCopyPointsConstants::TargetPointsLabel, EPCGDataType::Point);

	if (bMatchBasedOnAttribute)
	{
		// One bool flag per output data that signals if the output data should be computed or not.
		OutPins.Emplace(PCGCopyPointsConstants::SelectedFlagsPinLabel, EPCGDataType::Param);
	}
}

void UPCGCopyPointsKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
}

#undef LOCTEXT_NAMESPACE
