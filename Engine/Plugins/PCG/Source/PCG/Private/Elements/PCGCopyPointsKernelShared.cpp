// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCopyPointsKernelShared.h"

#include "PCGContext.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/PCGDataBinding.h"
#include "Elements/PCGCopyPoints.h"

#define LOCTEXT_NAMESPACE "PCGCopyPointsKernel"

bool PCGCopyPointsKernel::IsKernelDataValid(const UPCGComputeKernel* InKernel, const UPCGDataBinding* InDataBinding, FPCGContext* InContext)
{
	check(InKernel);

	if (ensure(InDataBinding))
	{
		const TSharedPtr<const FPCGDataCollectionDesc> SourcePinDesc = InDataBinding->GetCachedKernelPinDataDesc(InKernel, PCGCopyPointsConstants::SourcePointsLabel, /*bIsInputPin=*/true);

		if (!ensure(SourcePinDesc))
		{
			return false;
		}

		const TSharedPtr<const FPCGDataCollectionDesc> TargetPinDesc = InDataBinding->GetCachedKernelPinDataDesc(InKernel, PCGCopyPointsConstants::TargetPointsLabel, /*bIsInputPin=*/true);

		if (!ensure(TargetPinDesc))
		{
			return false;
		}

		const UPCGSettings* Settings = InKernel->GetSettings();
		const FPCGKernelParams* KernelParams = InDataBinding->GetCachedKernelParams(InKernel);

		if (!ensure(KernelParams))
		{
			return false;
		}

		const bool bCopyEachSourceOnEveryTarget = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bCopyEachSourceOnEveryTarget));
		const bool bMatchBasedOnAttribute = KernelParams->GetValueBool(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bMatchBasedOnAttribute));
		const FName MatchAttribute = KernelParams->GetValueName(GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, MatchAttribute));

		const int32 NumSources = SourcePinDesc->GetDataDescriptions().Num();
		const int32 NumTargets = TargetPinDesc->GetDataDescriptions().Num();

		if (!bCopyEachSourceOnEveryTarget && (NumSources != NumTargets && NumSources != 1 && NumTargets != 1 && NumSources != 0 && NumTargets != 0))
		{
			PCG_KERNEL_VALIDATION_ERR(InContext, Settings, FText::Format(
				LOCTEXT("NumDataMismatch", "Num Sources ({0}), mismatches with Num Targets ({1}). Only supports N:N, 1:N and N:1 operation."),
				NumSources,
				NumTargets));
		}

		if (bMatchBasedOnAttribute)
		{
			auto ValidateAttributeExists = [InContext, Settings, MatchAttribute](const TSharedPtr<const FPCGDataCollectionDesc> PinDataDesc) -> bool
			{
				const FName MatchAttributeName = MatchAttribute;

				// todo_pcg: Can generalize this to any type?
				constexpr EPCGKernelAttributeType MatchAttributeType = EPCGKernelAttributeType::Int;

				for (const FPCGDataDesc& DataDesc : PinDataDesc->GetDataDescriptions())
				{
					if (!DataDesc.ContainsAttribute(MatchAttributeName, MatchAttributeType))
					{
						PCG_KERNEL_VALIDATION_ERR(InContext, Settings, FText::Format(
							LOCTEXT("MatchAttributeMissing", "Match attribute '{0}' not found, this attribute must be present on all input data, and be of type Integer."),
							FText::FromName(MatchAttributeName)));

						return false;
					}
				}

				// Valid for execution if we have some data to process.
				return !PinDataDesc->GetDataDescriptions().IsEmpty();
			};

			if (!ValidateAttributeExists(SourcePinDesc) || !ValidateAttributeExists(TargetPinDesc))
			{
				return false;
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
