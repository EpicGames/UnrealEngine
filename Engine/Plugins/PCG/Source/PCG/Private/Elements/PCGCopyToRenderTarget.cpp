// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCopyToRenderTarget.h"

#include "Data/PCGTextureData.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "TextureResource.h"
#include "Engine/TextureRenderTarget2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCopyToRenderTarget)

#define LOCTEXT_NAMESPACE "PCGCopyToRenderTarget"

namespace PCGCopyToRenderTarget
{
	const FName RenderTargetOverridesLabel = TEXT("RenderTargets");
}

TArray<FPCGPinProperties> UPCGCopyToRenderTargetSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::BaseTexture).SetRequiredPin();

	if (bOverrideFromInput)
	{
		Properties.Emplace_GetRef(PCGCopyToRenderTarget::RenderTargetOverridesLabel, EPCGDataType::Param).SetRequiredPin();
	}

	return Properties;
}

TArray<FPCGPinProperties> UPCGCopyToRenderTargetSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DependencyPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any);
	DependencyPin.Usage = EPCGPinUsage::DependencyOnly;

	return PinProperties;
}

FPCGElementPtr UPCGCopyToRenderTargetSettings::CreateElement() const
{
	return MakeShared<FPCGCopyToRenderTargetElement>();
}

#if WITH_EDITOR
EPCGChangeType UPCGCopyToRenderTargetSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGCopyToRenderTargetSettings, bOverrideFromInput))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

void FPCGCopyToRenderTargetContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& [TextureData, RenderTarget] : TexturesToCopy)
	{
		Collector.AddReferencedObject(RenderTarget);
	}
}

bool FPCGCopyToRenderTargetElement::CanExecuteOnlyOnMainThread(FPCGContext* Context) const
{
	// PrepareData does loading, which requires the game thread.
	return !Context || Context->CurrentPhase == EPCGExecutionPhase::PrepareData;
}

bool FPCGCopyToRenderTargetElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCopyToRenderTargetElement::PrepareDataInternal);
	check(InContext);

	FPCGCopyToRenderTargetContext* Context = static_cast<FPCGCopyToRenderTargetContext*>(InContext);

	const UPCGCopyToRenderTargetSettings* Settings = Context->GetInputSettings<UPCGCopyToRenderTargetSettings>();
	check(Settings);

	if (!Context->WasLoadRequested())
	{
		if (!Settings->bOverrideFromInput)
		{
			Algo::Transform(Settings->RenderTargets, Context->RenderTargetsToLoad, [](const TSoftObjectPtr<UTextureRenderTarget2D> RenderTarget)
			{
				return RenderTarget.ToSoftObjectPath();
			});
		}
		else
		{
			const TArray<FPCGTaggedData> OverrideTaggedDatas = Context->InputData.GetInputsByPin(PCGCopyToRenderTarget::RenderTargetOverridesLabel);
			TArray<FSoftObjectPath> RenderTargetOverrides;

			for (const FPCGTaggedData& OverrideTaggedData : OverrideTaggedDatas)
			{
				if (const UPCGData* OverrideData = OverrideTaggedData.Data)
				{
					const FPCGAttributePropertyInputSelector RenderTargetSelector = Settings->RenderTargetAttribute.CopyAndFixLast(OverrideData);

					if (PCGAttributeAccessorHelpers::ExtractAllValues(OverrideData, RenderTargetSelector, RenderTargetOverrides, Context))
					{
						Context->RenderTargetsToLoad.Append(RenderTargetOverrides);
					}
					else
					{
						PCGLog::LogWarningOnGraph(LOCTEXT("FailExtractRenderTargetOverrides", "Failed to extract render target overrides."), Context);
					}
				}
			}
		}

		TArray<FSoftObjectPath> RenderTargetsToLoad = Context->RenderTargetsToLoad; // because of the move
		return Context->RequestResourceLoad(Context, MoveTemp(RenderTargetsToLoad), !Settings->bSynchronousLoad);
	}

	return true;
}

bool FPCGCopyToRenderTargetElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCopyToRenderTargetElement::Execute);
	check(InContext);

	FPCGCopyToRenderTargetContext* Context = static_cast<FPCGCopyToRenderTargetContext*>(InContext);

	if (!Context->bEnqueuedCopyCommand)
	{
		const UPCGCopyToRenderTargetSettings* Settings = Context->GetInputSettings<UPCGCopyToRenderTargetSettings>();
		check(Settings);

		const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

		if (Inputs.Num() != Context->RenderTargetsToLoad.Num())
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("NumInputMismatch", "Expected {0} data but receieved {1}. Number of input textures to copy must match the number of render targets to copy to."), Context->RenderTargetsToLoad.Num(), Inputs.Num()));
			return true;
		}

		for (int Index = 0; Index < Inputs.Num(); ++Index)
		{
			const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(Inputs[Index].Data);

			if (!TextureData)
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidTextureData", "Invalid texture data at index {0}, skipping."), Index));
				continue;
			}

			UTextureRenderTarget2D* RenderTarget = Cast<UTextureRenderTarget2D>(Context->RenderTargetsToLoad[Index].ResolveObject());

			if (!RenderTarget)
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidRenderTarget", "Invalid render target at index {0}, skipping."), Index));
				continue;
			}

			Context->TexturesToCopy.Emplace(TextureData, RenderTarget);
		}

		if (!Context->TexturesToCopy.IsEmpty())
		{
			// Pause execution until the render command has been scheduled.
			Context->bIsPaused = true;

			ENQUEUE_RENDER_COMMAND(CopyTextureDataToRenderTargets)([ContextHandle = Context->GetOrCreateHandle()](FRHICommandListImmediate& RHICmdList)
			{
				FPCGContext::FSharedContext<FPCGCopyToRenderTargetContext> SharedContext(ContextHandle);
				FPCGCopyToRenderTargetContext* Context = SharedContext.Get();

				if (!Context)
				{
					return;
				}

				ON_SCOPE_EXIT
				{
					// When this render command has been scheduled, we can continue execution.
					Context->bIsPaused = false;
				};

				const uint32 NumTexturesToCopy = Context->TexturesToCopy.Num();

				TArray<FRHITransitionInfo> SourceTransitionInfos;
				TArray<FRHITransitionInfo> TargetTransitionInfos;
				SourceTransitionInfos.Reserve(NumTexturesToCopy);
				TargetTransitionInfos.Reserve(NumTexturesToCopy);

				TArray<TPair<FRHITexture*, FRHITexture*>> TexturesToCopy;
				TexturesToCopy.Reserve(NumTexturesToCopy);

				// Collect TransitionInfos and TextureRHIs
				for (const auto& [TextureData, RenderTarget] : Context->TexturesToCopy)
				{
					FRHITexture* SourceTexture = TextureData ? TextureData->GetTextureRHI() : nullptr;

					if (!SourceTexture)
					{
						PCGLog::LogWarningOnGraph(LOCTEXT("InvalidSourceTexture", "Invalid source texture, skipping."), Context);
						continue;
					}

					FTextureRenderTargetResource* RenderTargetResource = RenderTarget ? RenderTarget->GetRenderTargetResource() : nullptr;
					FRHITexture* TargetTexture = RenderTargetResource ? RenderTargetResource->GetTextureRHI().GetReference() : nullptr;

					if (!TargetTexture)
					{
						PCGLog::LogWarningOnGraph(LOCTEXT("InvalidTargetTexture", "Invalid target texture, skipping."), Context);
						continue;
					}

					const FRHITextureDesc& SourceTextureDesc = SourceTexture->GetDesc();
					const FRHITextureDesc& TargetTextureDesc = TargetTexture->GetDesc();

					if (SourceTextureDesc.Extent != TargetTextureDesc.Extent)
					{
						PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("ExtentsMismatch", "Source texture extents ({0}) do not match target texture extents ({1}), skipping."),
							FText::FromString(SourceTextureDesc.Extent.ToString()),
							FText::FromString(TargetTextureDesc.Extent.ToString())),
							Context);
						continue;
					}

					if (SourceTextureDesc.Format != TargetTextureDesc.Format)
					{
						PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("FormatMismatch", "Source texture format '{0}' does not match target texture format '{1}', skipping."),
							FText::FromString(GetPixelFormatString(SourceTextureDesc.Format)),
							FText::FromString(GetPixelFormatString(TargetTextureDesc.Format))),
							Context);
						continue;
					}

					SourceTransitionInfos.Emplace(SourceTexture, ERHIAccess::Unknown, ERHIAccess::CopySrc);
					TargetTransitionInfos.Emplace(TargetTexture, ERHIAccess::Unknown, ERHIAccess::CopyDest);
					TexturesToCopy.Emplace(SourceTexture, TargetTexture);
				}

				if (SourceTransitionInfos.IsEmpty() || SourceTransitionInfos.Num() != TargetTransitionInfos.Num())
				{
					return;
				}

				// Batch transition textures to CopySrc and CopyDst
				RHICmdList.Transition(MakeArrayView(SourceTransitionInfos));
				RHICmdList.Transition(MakeArrayView(TargetTransitionInfos));

				// Copy source textures into target textures.
				for (const auto& [SourceTexture, TargetTexture] : TexturesToCopy)
				{
					FRHICopyTextureInfo CopyInfo;
					RHICmdList.CopyTexture(SourceTexture, TargetTexture, CopyInfo);
				}
			});

			Context->bEnqueuedCopyCommand = true;

			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
