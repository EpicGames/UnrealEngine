// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCreateAttribute.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGGraph.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGUserParameterGet.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGConversion.h"
#include "Helpers/PCGGraphParametersHelpers.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Utils/PCGLogErrors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCreateAttribute)

#define LOCTEXT_NAMESPACE "PCGCreateAttributeElement"

namespace PCGCreateAttributeConstants
{
	const FName NodeNameAddAttribute = TEXT("AddAttribute");
	const FText NodeTitleAddAttribute = LOCTEXT("NodeTitleAddAttribute", "Add Attribute");
	const FName NodeNameCreateConstant = TEXT("CreateConstant");
	const FText NodeTitleCreateConstant = LOCTEXT("NodeTitleCreateConstant", "Create Constant");
	const FText NodeAliasCreateConstant = LOCTEXT("NodeAliasCreateConstant", "Create Attribute");
	const FText NodeTooltipFormatCreateConstant = LOCTEXT("NodeTooltipFormatCreateConstant", "Outputs an attribute set containing a constant '{0}' value:  {1}");
	const FName AttributesLabel = TEXT("Attributes");
	const FText AttributesTooltip = LOCTEXT("AttributesTooltip", "Optional Attribute Set to create from. Not used if not connected.");
	const FText ErrorCreatingAttributeMessage = LOCTEXT("ErrorCreatingAttribute", "Error while creating attribute '{0}'");

	namespace Conversion
	{
		constexpr int32 ToGetGraphParameterIndex = 0;

		const FText ToGetGraphParameterTooltip = LOCTEXT("ToGetGraphParameterTooltip", "Convert constant to graph parameter.");
	}
}

namespace PCGCreateAttribute
{
	FPCGMetadataAttributeBase* ClearOrCreateAttribute(const FPCGMetadataTypesConstantStruct& AttributeTypes, FPCGMetadataDomain* Metadata, const FName OutputAttributeName)
	{
		check(Metadata);

		auto CreateAttribute = [Metadata, OutputAttributeName](auto&& Value) -> FPCGMetadataAttributeBase*
		{
			return PCGMetadataElementCommon::ClearOrCreateAttribute(Metadata, OutputAttributeName, std::forward<decltype(Value)>(Value));
		};

		return AttributeTypes.Dispatcher(CreateAttribute);
	}
}

#if WITH_EDITOR
bool UPCGAddAttributeSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	return !InPin || (InPin->Properties.Label != PCGCreateAttributeConstants::AttributesLabel) || InPin->IsConnected();
}

bool UPCGAddAttributeSettings::CanEditChange(const FProperty* InProperty) const
{
	if (!InProperty || !Super::CanEditChange(InProperty))
	{
		return false;
	}

	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	const bool AttributesPinIsConnected = Node ? Node->IsInputPinConnected(PCGCreateAttributeConstants::AttributesLabel) : false;

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGAddAttributeSettings, InputSource))
	{
		return AttributesPinIsConnected;
	}
	else if (InProperty->GetOwnerStruct() == FPCGMetadataTypesConstantStruct::StaticStruct())
	{
		return !AttributesPinIsConnected;
	}

	return true;
}

void UPCGAddAttributeSettings::ApplyStructuralDeprecation(UPCGNode* InOutNode)
{
	check(InOutNode);
	// Arbitrary version that approximately matches the time when Add/Create attributes changed.
	// It will convert any add attributes that have nothing connected to it to a Create Constant.
	if (DataVersion < FPCGCustomVersion::SupportPartitionedComponentsInNonPartitionedLevels)
	{
		if (!InOutNode->IsInputPinConnected(PCGPinConstants::DefaultInputLabel))
		{
			UPCGCreateAttributeSetSettings* NewSettings = NewObject<UPCGCreateAttributeSetSettings>(InOutNode);
			NewSettings->OutputTarget.ImportFromOtherSelector(OutputTarget);
			NewSettings->AttributeTypes = AttributeTypes;
			InOutNode->SetSettingsInterface(NewSettings);
		}
	}

	Super::ApplyStructuralDeprecation(InOutNode);
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGAddAttributeSettings::CreateElement() const
{
	return MakeShared<FPCGAddAttributeElement>();
}

FString UPCGAddAttributeSettings::GetAdditionalTitleInformation() const
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return PCGCreateAttributeConstants::NodeNameAddAttribute.ToString();
	}

	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	const bool bAttributesPinIsConnected = Node ? Node->IsInputPinConnected(PCGCreateAttributeConstants::AttributesLabel) : false;

	if (bAttributesPinIsConnected)
	{
		if (bCopyAllAttributes)
		{
			return LOCTEXT("AllAttributes", "All Attributes").ToString();
		}
		else
		{
			const FString SourceParamAttributeName = InputSource.ToString();
			const FString OutputAttributeName = OutputTarget.CopyAndFixSource(&InputSource, nullptr).ToString();
			if (OutputAttributeName.IsEmpty() && SourceParamAttributeName.IsEmpty())
			{
				return PCGCreateAttributeConstants::NodeNameAddAttribute.ToString();
			}
			else
			{
				return OutputAttributeName.IsEmpty() ? SourceParamAttributeName : OutputAttributeName;
			}
		}
	}
	else
	{
		return FString::Printf(TEXT("%s: %s"), *OutputTarget.ToString(), *AttributeTypes.ToString());
	}
}

UPCGAddAttributeSettings::UPCGAddAttributeSettings()
{
	OutputTarget.SetAttributeName(NAME_None);
}

void UPCGAddAttributeSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (SourceParamAttributeName_DEPRECATED != NAME_None)
	{
		InputSource.SetAttributeName(SourceParamAttributeName_DEPRECATED);
		SourceParamAttributeName_DEPRECATED = NAME_None;
	}

	if (OutputAttributeName_DEPRECATED != NAME_None)
	{
		OutputTarget.SetAttributeName(OutputAttributeName_DEPRECATED);
		OutputAttributeName_DEPRECATED = NAME_None;
	}

	AttributeTypes.OnPostLoad();
#endif // WITH_EDITOR
}

#if WITH_EDITOR
FName UPCGAddAttributeSettings::GetDefaultNodeName() const
{
	return PCGCreateAttributeConstants::NodeNameAddAttribute;
}

FText UPCGAddAttributeSettings::GetDefaultNodeTitle() const
{
	return PCGCreateAttributeConstants::NodeTitleAddAttribute;
}

void UPCGAddAttributeSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	const bool AttributesPinIsConnected = InOutNode ? InOutNode->IsInputPinConnected(PCGCreateAttributeConstants::AttributesLabel) : false;

	if (DataVersion < FPCGCustomVersion::UpdateAddAttributeWithSelectors
		&& OutputTarget.GetSelection() == EPCGAttributePropertySelection::Attribute
		&& OutputTarget.GetAttributeName() == NAME_None
		&& AttributesPinIsConnected)
	{
		// Previous behavior of the output target for this node was:
		// None => Source if Attributes pin is connected
		OutputTarget.SetAttributeName(PCGMetadataAttributeConstants::SourceAttributeName);
	}

	Super::ApplyDeprecation(InOutNode);
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGAddAttributeSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any);
	InputPinProperty.SetRequiredPin();

	PinProperties.Emplace(PCGCreateAttributeConstants::AttributesLabel, EPCGDataType::Param, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true, PCGCreateAttributeConstants::AttributesTooltip);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGAddAttributeSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

void UPCGCreateAttributeSetSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (OutputAttributeName_DEPRECATED != NAME_None)
	{
		OutputTarget.SetAttributeName(OutputAttributeName_DEPRECATED);
		OutputAttributeName_DEPRECATED = NAME_None;
	}

	AttributeTypes.OnPostLoad();
#endif // WITH_EDITOR
}

#if WITH_EDITOR
FName UPCGCreateAttributeSetSettings::GetDefaultNodeName() const
{
	return PCGCreateAttributeConstants::NodeNameCreateConstant;
}

FText UPCGCreateAttributeSetSettings::GetDefaultNodeTitle() const
{
	return PCGCreateAttributeConstants::NodeTitleCreateConstant;
}

TArray<FText> UPCGCreateAttributeSetSettings::GetNodeTitleAliases() const
{
	return {PCGCreateAttributeConstants::NodeAliasCreateConstant};
}

FText UPCGCreateAttributeSetSettings::GetNodeTooltipText() const
{
	return FText::Format(PCGCreateAttributeConstants::NodeTooltipFormatCreateConstant, FText::FromString(AttributeTypes.TypeToString()), FText::FromString(AttributeTypes.ToString()));
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGCreateAttributeSetSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataTypes>(
		/*InValuesToSkip=*/{EPCGMetadataTypes::Count, EPCGMetadataTypes::Unknown},
		/*InOptionalFormat=*/FTextFormat(LOCTEXT("PreconfigureFormat", "New {0} Constant")));
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGCreateAttributeSetSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

TArray<FPCGPinProperties> UPCGCreateAttributeSetSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& Pin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	Pin.AllowedTypes.CustomSubtype = static_cast<int32>(AttributeTypes.Type);

	return PinProperties;
}

FString UPCGCreateAttributeSetSettings::GetAdditionalTitleInformation() const
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return PCGCreateAttributeConstants::NodeNameAddAttribute.ToString();
	}

	return FString::Printf(TEXT("%s: %s"), *OutputTarget.ToString(), *AttributeTypes.ToString());
}

void UPCGCreateAttributeSetSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo)
{
	if (PreconfigureInfo.PreconfiguredIndex < 0 || PreconfigureInfo.PreconfiguredIndex >= static_cast<uint8>(EPCGMetadataTypes::Count))
	{
		return;
	}

	AttributeTypes.Type = static_cast<EPCGMetadataTypes>(PreconfigureInfo.PreconfiguredIndex);
}

#if WITH_EDITOR
TArray<FPCGPreconfiguredInfo> UPCGCreateAttributeSetSettings::GetConversionInfo() const
{
	using namespace PCGCreateAttributeConstants::Conversion;
	TArray<FPCGPreconfiguredInfo> ConversionInfo;

	ConversionInfo.Emplace(
		ToGetGraphParameterIndex,
		PCGConversion::Helpers::GetDefaultNodeTitle(UPCGUserParameterGetSettings::StaticClass()),
		ToGetGraphParameterTooltip);

	return ConversionInfo;
}
#endif // WITH_EDITOR

bool UPCGCreateAttributeSetSettings::ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo)
{
	UPCGNode* Node = CastChecked<UPCGNode>(GetOuter());

	switch (ConversionInfo.PreconfiguredIndex)
	{
		case PCGCreateAttributeConstants::Conversion::ToGetGraphParameterIndex:
		{
			FPCGSingleNodeConverter NodeConverter(Node, UPCGUserParameterGetSettings::StaticClass());
			if (!NodeConverter.IsGraphInitialized() || !NodeConverter.IsSourceInitialized())
			{
				PCGLog::Settings::LogInvalidPreconfigurationWarning(ConversionInfo.PreconfiguredIndex, PCGCreateAttributeConstants::NodeTitleCreateConstant);
				return false;
			}

			NodeConverter.PrepareData();

			// The property will determine the structure of the Graph Parameter node.
			if (UPCGGraph* Graph = NodeConverter.GetGraph())
			{
				const FInstancedPropertyBag* UserParametersStruct = Graph->GetUserParametersStruct();
				FName PropertyName = OutputTarget.GetName();
				if (!PCGGraphParameter::Helpers::GenerateUniqueName(Graph, PropertyName))
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("CouldNotCreateUniqueGraphParameterName", "Could not create a unique graph parameter with name '{0}'. Check if the graph is valid and the name has not reached its maximum."), FText::FromName(PropertyName)));
					return false;
				}
				check(UserParametersStruct->IsPropertyNameValid(PropertyName));

				FPropertyBagPropertyDesc PropertyDesc = PCGPropertyHelpers::CreatePropertyBagDescWithMetadataType(PropertyName, AttributeTypes.Type);

				const EPropertyBagAlterationResult Result = Graph->AddUserParameters({PropertyDesc});
				if (Result != EPropertyBagAlterationResult::Success)
				{
					PCGGraphParameter::Helpers::LogGraphParamNamingErrors(PropertyName, Result);
					return false;
				}

				if (const FPropertyBagPropertyDesc* Desc = UserParametersStruct->FindPropertyDescByName(PropertyName))
				{
					// Update the generated node with the new user parameter
					UPCGUserParameterGetSettings* Settings = CastChecked<UPCGUserParameterGetSettings>(NodeConverter.GetGeneratedSettings());
					Settings->PropertyName = Desc->Name;
					Settings->PropertyGuid = Desc->ID;
					AttributeTypes.Dispatcher([PropertyName, Graph]<typename T>(T&& Value) { Graph->SetGraphParameter(PropertyName, std::forward<T>(Value)); });
				}
				else
				{
					// This should never trigger, but keep it a soft assert just in case.
					UE_LOGF(LogPCG, Error, "Could not create a unique graph parameter with name '%ls'.", *PropertyName.ToString());
					return ensure(false);
				}
			}

			NodeConverter.ApplyStructural();
			NodeConverter.Finalize();

			return NodeConverter.IsComplete();
		}

		default:
			break;
	}

	return false;
}

FPCGElementPtr UPCGCreateAttributeSetSettings::CreateElement() const
{
	return MakeShared<FPCGCreateAttributeElement>();
}

bool FPCGAddAttributeElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAddAttributeElement::Execute);

	check(Context);

	const UPCGAddAttributeSettings* Settings = Context->GetInputSettings<UPCGAddAttributeSettings>();
	check(Settings);

	const bool bAttributesPinIsConnected = Context->Node ? Context->Node->IsInputPinConnected(PCGCreateAttributeConstants::AttributesLabel) : false;
	TArray<FPCGTaggedData> SourceParams = Context->InputData.GetInputsByPin(PCGCreateAttributeConstants::AttributesLabel);
	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	const FName OutputAttributeName = Settings->OutputTarget.GetName();

	// If we add from a constant
	if (SourceParams.IsEmpty() && !bAttributesPinIsConnected)
	{
		for (int32 i = 0; i < Inputs.Num(); ++i)
		{
			const UPCGData* InData = Inputs[i].Data;
			if (!InData || !InData->ConstMetadata())
			{
				continue;
			}

			UPCGData* OutputData = InData->DuplicateData(Context);
			check(OutputData);
			FPCGMetadataDomain* OutputMetadata = OutputData->MutableMetadata()->GetMetadataDomainFromSelector(Settings->OutputTarget);

			if (!OutputMetadata)
			{
				PCGLog::Metadata::LogInvalidMetadataDomain(Settings->OutputTarget, Context);
				continue;
			}

			if (!PCGCreateAttribute::ClearOrCreateAttribute(Settings->AttributeTypes, OutputMetadata, OutputAttributeName))
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(PCGCreateAttributeConstants::ErrorCreatingAttributeMessage, FText::FromName(OutputAttributeName)));
				return true;
			}

			// Making sure we have at least one entry.
			if (OutputMetadata && OutputMetadata->GetItemCountForChild() == 0)
			{
				OutputMetadata->AddEntry();
			}

			FPCGTaggedData& NewData = Context->OutputData.TaggedData.Add_GetRef(Inputs[i]);
			NewData.Data = OutputData;
		}

		return true;
	}

	// Otherwise, is is like a copy
	const UPCGParamData* FirstSourceParamData = !SourceParams.IsEmpty() ? Cast<UPCGParamData>(SourceParams[0].Data) : nullptr;
	if (!FirstSourceParamData)
	{
		// Nothing to do
		Context->OutputData.TaggedData = Inputs;
		return true;
	}

	// If we copy all attributes, support to have multiple source param. Otherwise, add a warning
	if (SourceParams.Num() > 1 && !Settings->bCopyAllAttributes)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("MultiAttributeWhenNoCopyAll", "Multiple source param detected in the Attributes pin, but we do not copy all attributes. We will only look into the first source param."), Context);
	}

	for (int32 i = 0; i < Inputs.Num(); ++i)
	{
		const UPCGData* InputData = Inputs[i].Data;
		if (!InputData)
		{
			continue;
		}

		UPCGData* TargetData = InputData->DuplicateData(Context);
		FPCGTaggedData& Output = Context->OutputData.TaggedData.Add_GetRef(Inputs[i]);

		bool bSuccess = true;
		if (Settings->bCopyAllAttributes)
		{
			for (const FPCGTaggedData& SourceParamData : SourceParams)
			{
				if (const UPCGParamData* ParamData = Cast<UPCGParamData>(SourceParamData.Data))
				{
					PCGMetadataHelpers::FPCGCopyAllAttributesParams Params
					{
						.SourceData = ParamData,
						.TargetData = TargetData,
						.OptionalContext = Context
					};

					if (Settings->bCopyAllDomains)
					{
						Params.InitializeMappingForAllDomains();
					}
					else
					{
						Params.InitializeMappingFromDomainNames(Settings->MetadataDomainsMapping);
					}

					// Nothing to do
					if (Params.DomainMapping.IsEmpty())
					{
						continue;
					}

					bSuccess &= PCGMetadataHelpers::CopyAllAttributes(Params);
				}
			}
		}
		else
		{
			PCGMetadataHelpers::FPCGCopyAttributeParams Params{};
			Params.SourceData = FirstSourceParamData;
			Params.TargetData = TargetData;
			Params.InputSource = Settings->InputSource;
			Params.OutputTarget = Settings->OutputTarget;
			Params.OptionalContext = Context;
			Params.bSameOrigin = false;

			bSuccess = PCGMetadataHelpers::CopyAttribute(Params);
		}

		if (bSuccess)
		{
			Output.Data = TargetData;
		}
	}

	return true;
}

bool FPCGCreateAttributeElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreateAttributeElement::Execute);

	check(Context);

	const UPCGCreateAttributeSetSettings* Settings = Context->GetInputSettings<UPCGCreateAttributeSetSettings>();
	check(Settings);

	FName OutputAttributeName = Settings->OutputTarget.GetName();

	const UPCGParamData* DefaultParam = GetDefault<UPCGParamData>();
	const FPCGMetadataDomainID OutputMetadataDomainID = DefaultParam->GetMetadataDomainIDFromSelector(Settings->OutputTarget);
	if (!DefaultParam->IsSupportedMetadataDomainID(OutputMetadataDomainID))
	{
		PCGLog::Metadata::LogInvalidMetadataDomain(Settings->OutputTarget, Context);
		return true;
	}

	UPCGParamData* OutputData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	check(OutputData);

	FPCGMetadataDomain* OutputMetadata = OutputData->MutableMetadata()->GetMetadataDomainFromSelector(Settings->OutputTarget);

	if (!OutputMetadata)
	{
		PCGLog::Metadata::LogInvalidMetadataDomain(Settings->OutputTarget, Context);
		return true;
	}

	OutputMetadata->AddEntry();

	if (!PCGCreateAttribute::ClearOrCreateAttribute(Settings->AttributeTypes, OutputMetadata, OutputAttributeName))
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(PCGCreateAttributeConstants::ErrorCreatingAttributeMessage, FText::FromName(OutputAttributeName)));
		return true;
	}

	FPCGTaggedData& NewData = Context->OutputData.TaggedData.Emplace_GetRef();
	NewData.Data = OutputData;

	return true;
}

#undef LOCTEXT_NAMESPACE
