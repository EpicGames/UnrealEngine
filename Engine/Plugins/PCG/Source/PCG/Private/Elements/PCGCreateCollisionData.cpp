// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCreateCollisionData.h"

#include "PCGContext.h"
#include "Data/PCGCollisionWrapperData.h"
#include "Data/PCGBasePointData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCreateCollisionData)

#define LOCTEXT_NAMESPACE "PCGCreateCollisionDataElement"

void UPCGCreateCollisionDataSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (bUseComplexCollision_DEPRECATED)
	{
		CollisionQueryFlag = EPCGCollisionQueryFlag::Complex;
		bUseComplexCollision_DEPRECATED = false;
	}
#endif
}

#if WITH_EDITOR
FText UPCGCreateCollisionDataSettings::GetNodeTooltipText() const
{
	return LOCTEXT("CreateCollisionTooltip", "Creates a volumetric representation of the points as if they had their selected mesh collision.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGCreateCollisionDataSettings::InputPinProperties() const
{
	return Super::DefaultPointInputPinProperties();
}

TArray<FPCGPinProperties> UPCGCreateCollisionDataSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Primitive);

	return PinProperties;
}

FPCGElementPtr UPCGCreateCollisionDataSettings::CreateElement() const
{
	return MakeShared<FPCGCreateCollisionDataElement>();
}

void FPCGCreateCollisionContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	for (InputMeshData& ThisInputData : PerInputData)
	{
		Collector.AddReferencedObject(ThisInputData.Data);
	}
}

bool FPCGCreateCollisionDataElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreateCollisionDataElement::PrepareData);
	check(InContext);
	FPCGCreateCollisionContext* Context = static_cast<FPCGCreateCollisionContext*>(InContext);

	if (!Context->WasLoadRequested())
	{
		TArray<FSoftObjectPath> MeshesToLoad;

		const UPCGCreateCollisionDataSettings* Settings = Context->GetInputSettings<UPCGCreateCollisionDataSettings>();
		check(Settings);

		const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
		for(int InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
		{
			const FPCGTaggedData& Input = Inputs[InputIndex];

			if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Input.Data))
			{
				FPCGCreateCollisionContext::InputMeshData InputMeshData;
				InputMeshData.InputIndex = InputIndex;
				InputMeshData.Data = FPCGContext::NewObject_AnyThread<UPCGCollisionWrapperData>(Context);

				if (!InputMeshData.Data->PreInitializeAndGatherMeshesEx(PointData, Settings->CollisionAttribute, Settings->CollisionQueryFlag, InputMeshData.MeshPaths))
				{
					if (Settings->bWarnIfAttributeCouldNotBeUsed)
					{
						PCGLog::LogWarningOnGraph(LOCTEXT("FailedToAccessMeshes", "Failed to access the mesh attribute provided - will behave like a PointData."), Context);
					}
				}
				else
				{
					for (const FSoftObjectPath& MeshPath : InputMeshData.MeshPaths)
					{
						if (!MeshPath.IsNull())
						{
							MeshesToLoad.AddUnique(MeshPath);
						}
					}
					
					Context->PerInputData.Add(MoveTemp(InputMeshData));
				}
			}
		}

		// Finally, request loading for all meshes we gathered
		return Context->RequestResourceLoad(Context, MoveTemp(MeshesToLoad), !Settings->bSynchronousLoad);
	}

	return true;
}

bool FPCGCreateCollisionDataElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreateCollisionDataElement::Execute);

	check(InContext);
	FPCGCreateCollisionContext* Context = static_cast<FPCGCreateCollisionContext*>(InContext);

	const UPCGCreateCollisionDataSettings* Settings = Context->GetInputSettings<UPCGCreateCollisionDataSettings>();
	check(Settings);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	for (int InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		const FPCGTaggedData& Input = Inputs[InputIndex];
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		if (FPCGCreateCollisionContext::InputMeshData* MatchingData = Context->PerInputData.FindByPredicate([InputIndex](const FPCGCreateCollisionContext::InputMeshData& Data) { return Data.InputIndex == InputIndex; }))
		{
			check(MatchingData->Data);
			MatchingData->Data->FinalizeInitializationEx(MatchingData->MeshPaths);

			Output.Data = MatchingData->Data;
			
			// Null out the data so the context destructor doesn't interact with the rooting
			MatchingData->Data = nullptr;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE