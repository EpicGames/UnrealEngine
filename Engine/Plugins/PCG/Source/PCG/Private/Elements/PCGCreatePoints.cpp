// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCreatePoints.h"

#include "PCGContext.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCreatePoints)

#define LOCTEXT_NAMESPACE "PCGCreatePointsElement"

UPCGCreatePointsSettings::UPCGCreatePointsSettings()
{
	// Add one default point in the array, with a steepness of 1 for new nodes.
	FPCGPoint DefaultPoint{};
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		DefaultPoint.Steepness = 1.0f;
	}
	
	PointsToCreate.Add(DefaultPoint);
}

void UPCGCreatePointsSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (GridPivot_DEPRECATED != EPCGLocalGridPivot::Global)
	{
		CoordinateSpace = static_cast<EPCGCoordinateSpace>(static_cast<uint8>(GridPivot_DEPRECATED));
		GridPivot_DEPRECATED = EPCGLocalGridPivot::Global;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

#if WITH_EDITOR
void UPCGCreatePointsSettings::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGCreatePointsSettings, PointsToCreate)
		&& PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd && !PointsToCreate.IsEmpty())
	{
		// Force the steepness to 1 to any new added points to the array.
		PointsToCreate.Last().Steepness = 1.0f;	
	}
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGCreatePointsSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

FPCGElementPtr UPCGCreatePointsSettings::CreateElement() const
{
	return MakeShared<FPCGCreatePointsElement>();
}

bool FPCGCreatePointsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreatePointsElement::Execute);
	check(Context);

	if (!Context->ExecutionSource.IsValid())
	{
		return true;
	}

	const UPCGCreatePointsSettings* Settings = Context->GetInputSettings<UPCGCreatePointsSettings>();
	check(Settings);

	// Used for culling, regardless of generation coordinate space
	const UPCGSpatialData* CullingShape = Settings->bCullPointsOutsideVolume ? Cast<UPCGSpatialData>(Context->ExecutionSource->GetExecutionState().GetSelfData()) : nullptr;

	// Early out if the culling shape isn't valid
	if (!CullingShape && Settings->bCullPointsOutsideVolume)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("CannotCullWithoutAShape", "Unable to cull since the supporting actor has no data."));
		return true;
	}

	FTransform LocalTransform = FTransform::Identity;
	if (Settings->CoordinateSpace == EPCGCoordinateSpace::LocalComponent)
	{
		LocalTransform = Context->ExecutionSource->GetExecutionState().GetTransform();
	}
	else if (Settings->CoordinateSpace == EPCGCoordinateSpace::OriginalComponent)
	{
		LocalTransform = Context->ExecutionSource->GetExecutionState().GetOriginalTransform();
	}

	// Reset scale as we are not going to derive the points size from it
	LocalTransform.SetScale3D(FVector::One());

	const TArray<FPCGPoint>& PointsToLoopOn = Settings->PointsToCreate;
	
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	FPCGTaggedData& Output = Outputs.Emplace_GetRef();

	UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(Context);
	check(PointData); 
	PointData->SetNumPoints(PointsToLoopOn.Num(), /*bInitializeValues=*/false);

	// @todo_pcg: Could probably be optimzed by comparing the source points to find the properties that are default but this could also be expensive
	PointData->AllocateProperties(EPCGPointNativeProperties::All);

	Output.Data = PointData;
	int32 NumWritten = 0;

	if (Settings->CoordinateSpace == EPCGCoordinateSpace::World)
	{
		FPCGPointValueRanges OutValueRanges(PointData, /*bAllocate=*/false);

		if (CullingShape)
		{
			for (const FPCGPoint& Point : PointsToLoopOn)
			{
				if (CullingShape->GetDensityAtPosition(Point.Transform.GetLocation()) > 0)
				{
					OutValueRanges.SetFromPoint(NumWritten, Point);
					++NumWritten;
				}
			}
		}
		else
		{
			for (const FPCGPoint& Point : PointsToLoopOn)
			{
				OutValueRanges.SetFromPoint(NumWritten, Point);
				++NumWritten;
			}
		}

		for (int32 Index = 0; Index < NumWritten; ++Index)
		{
			if (OutValueRanges.SeedRange[Index] == 0)
			{
				// If the seed is the default value, generate a new seed based on the its transform
				OutValueRanges.SeedRange[Index] = UPCGBlueprintHelpers::ComputeSeedFromPosition(OutValueRanges.TransformRange[Index].GetLocation());
			}
		}

		PointData->SetNumPoints(NumWritten);
	}
	else
	{
		check(Settings->CoordinateSpace == EPCGCoordinateSpace::LocalComponent || Settings->CoordinateSpace == EPCGCoordinateSpace::OriginalComponent);

		auto MoveDataRangeFunc = [PointData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
		{
			PointData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		};

		auto FinishedFunc = [PointData](int32 NumWritten)
		{
			PointData->SetNumPoints(NumWritten);
		};

		auto ProcessRangeFunc = [PointData, &PointsToLoopOn, &LocalTransform, CullingShape](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			int32 NumWritten = 0;

			FPCGPointValueRanges OutValueRanges(PointData, /*bAllocate=*/false);

			for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
			{
				const int32 WriteIndex = StartWriteIndex + NumWritten;

				const FPCGPoint& InPoint = PointsToLoopOn[ReadIndex];
				const FTransform OutTransform = InPoint.Transform * LocalTransform;
				
				if (!CullingShape || (CullingShape->GetDensityAtPosition(OutTransform.GetLocation()) > 0.0f))
				{
					OutValueRanges.SetFromPoint(WriteIndex, InPoint);
					OutValueRanges.TransformRange[WriteIndex] = OutTransform;

					const int SeedFromPosition = UPCGBlueprintHelpers::ComputeSeedFromPosition(OutTransform.GetLocation());
					OutValueRanges.SeedRange[WriteIndex] = (InPoint.Seed == 0 ? SeedFromPosition : PCGHelpers::ComputeSeed(InPoint.Seed, SeedFromPosition));

					++NumWritten;
				}
			}

			return NumWritten;
		};

		FPCGAsync::AsyncProcessingRangeEx(
			&Context->AsyncState,
			PointsToLoopOn.Num(),
			[](){},
			ProcessRangeFunc,
			MoveDataRangeFunc,
			FinishedFunc,
			/*bEnableTimeSlicing=*/false
		);
	}

	return true;
}

bool FPCGCreatePointsElement::IsCacheable(const UPCGSettings* InSettings) const
{
	const UPCGCreatePointsSettings* Settings = Cast<const UPCGCreatePointsSettings>(InSettings);

	return Settings && Settings->CoordinateSpace == EPCGCoordinateSpace::World;
}

namespace PCGCreatePointsHelper
{
	UPCGData* GetDependenciesData(IPCGGraphExecutionSource* InExecutionSource, EPCGCoordinateSpace InCoordinateSpace, bool bInCullPointsOutsideVolume)
	{
		if (InExecutionSource)
		{
			if (bInCullPointsOutsideVolume || InCoordinateSpace == EPCGCoordinateSpace::LocalComponent)
			{
				return InExecutionSource->GetExecutionState().GetSelfData();
			}
			else if (InCoordinateSpace == EPCGCoordinateSpace::OriginalComponent)
			{
				if (IPCGGraphExecutionSource* OriginalExecutionSource = InExecutionSource ? InExecutionSource->GetExecutionState().GetOriginalSource() : nullptr)
				{
					return OriginalExecutionSource->GetExecutionState().GetSelfData();
				}
				else
				{
					return InExecutionSource->GetExecutionState().GetSelfData();
				}
			}
		}

		return nullptr;
	}
}

void FPCGCreatePointsElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	if (const UPCGCreatePointsSettings* Settings = Cast<UPCGCreatePointsSettings>(InParams.Settings))
	{
		int CoordinateSpace = static_cast<int>(EPCGCoordinateSpace::World);
		bool bCullPointsOutsideVolume = false;
		PCGSettingsHelpers::GetOverrideValue(*InParams.InputData, Settings, GET_MEMBER_NAME_CHECKED(UPCGCreatePointsSettings, CoordinateSpace), static_cast<int>(Settings->CoordinateSpace), CoordinateSpace);
		PCGSettingsHelpers::GetOverrideValue(*InParams.InputData, Settings, GET_MEMBER_NAME_CHECKED(UPCGCreatePointsSettings, bCullPointsOutsideVolume), Settings->bCullPointsOutsideVolume, bCullPointsOutsideVolume);

		// We're using the bounds of the pcg volume, so we extract the actor data here
		EPCGCoordinateSpace EnumCoordinateSpace = static_cast<EPCGCoordinateSpace>(CoordinateSpace);

		if (UPCGData* Data = PCGCreatePointsHelper::GetDependenciesData(InParams.ExecutionSource, EnumCoordinateSpace, bCullPointsOutsideVolume))
		{
			Crc.Combine(Data->GetOrComputeCrc(/*bFullDataCrc=*/false));
		}
	}

	OutCrc = Crc;
}

#undef LOCTEXT_NAMESPACE
