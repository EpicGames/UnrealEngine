// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCreatePointsGrid.h"

#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Elements/PCGCreatePoints.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCreatePointsGrid)

#define LOCTEXT_NAMESPACE "PCGCreatePointsGridElement"

UPCGCreatePointsGridSettings::UPCGCreatePointsGridSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		PointSteepness = 1.0f;
	}
}


void UPCGCreatePointsGridSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (GridPivot_DEPRECATED != EPCGGridPivot::Global)
	{
		CoordinateSpace = static_cast<EPCGCoordinateSpace>(static_cast<int8>(GridPivot_DEPRECATED));
		GridPivot_DEPRECATED = EPCGGridPivot::Global;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

TArray<FPCGPinProperties> UPCGCreatePointsGridSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

FPCGElementPtr UPCGCreatePointsGridSettings::CreateElement() const
{
	return MakeShared<FPCGCreatePointsGridElement>();
}

bool FPCGCreatePointsGridElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreatePointsGridElement::Execute);
	check(Context);

	if (!Context->ExecutionSource.IsValid())
	{
		return true;
	}

	const UPCGCreatePointsGridSettings* Settings = Context->GetInputSettings<UPCGCreatePointsGridSettings>();
	check(Settings);

	// Used for culling, regardless of generation coordinate space
	const UPCGSpatialData* CullingShape = Settings->bCullPointsOutsideVolume ? Cast<UPCGSpatialData>(Context->ExecutionSource->GetExecutionState().GetSelfData()) : nullptr;

	// Early out if the culling shape isn't valid
	if (!CullingShape && Settings->bCullPointsOutsideVolume)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("CannotCullWithoutAShape", "Unable to cull since the supporting actor has no data."));
		return true;
	}

	FTransform LocalTransform = FTransform::Identity;
	if (Settings->CoordinateSpace == EPCGCoordinateSpace::LocalComponent)
	{
		LocalTransform = Context->ExecutionSource->GetExecutionState().GetTransform();
	}
	else if (Settings->CoordinateSpace == EPCGCoordinateSpace::OriginalComponent)
	{
		LocalTransform = Context->ExecutionSource->GetExecutionState().GetOriginalTransform();
	}

	// Reset the scale on the local transform since we don't want to derive the points scale from the referential
	LocalTransform.SetScale3D(FVector::One());

	if (Settings->CellSize.X <= 0.0 || Settings->CellSize.Y <= 0.0 || Settings->CellSize.Z <= 0.0)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("InvalidCellDataInput", "CellSize must not be less than 0"));
		return true;
	}

	if (Settings->GridExtents.X < 0.0 || Settings->GridExtents.Y < 0.0 || Settings->GridExtents.Z < 0.0)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("InvalidGridDataInput", "GridExtents must not be less than 0"));
		return true;
	}

	FVector GridExtents = Settings->GridExtents;
	const FVector& CellSize = Settings->CellSize;

	int32 PointCountX = FMath::TruncToInt((2 * GridExtents.X) / CellSize.X);
	int32 PointCountY = FMath::TruncToInt((2 * GridExtents.Y) / CellSize.Y);
	int32 PointCountZ = FMath::TruncToInt((2 * GridExtents.Z) / CellSize.Z); 

	if (Settings->PointPosition == EPCGPointPosition::CellCorners)
	{
		PointCountX++;
		PointCountY++;
		PointCountZ++;

		if (GridExtents.X < (CellSize.X / 2))
		{
			GridExtents.X = 0.0;
		}

		if (GridExtents.Y < (CellSize.Y / 2))
		{
			GridExtents.Y = 0.0;
		}

		if (GridExtents.Z < (CellSize.Z / 2))
		{
			GridExtents.Z = 0.0;
		}
	}

	// If the GridExtent would produce an off center result, snap it to the center of the grid
	if (Settings->PointPosition == EPCGPointPosition::CellCenter)
	{
		if (GridExtents.X < (CellSize.X / 2))
		{
			PointCountX++;
		}

		if (GridExtents.Y < (CellSize.Y / 2))
		{
			PointCountY++;
		}

		if (GridExtents.Z < (CellSize.Z / 2))
		{
			PointCountZ++;
		}

		GridExtents.X = GridExtents.X - FMath::Fmod(GridExtents.X, CellSize.X / 2);
		GridExtents.Y = GridExtents.Y - FMath::Fmod(GridExtents.Y, CellSize.Y / 2);
		GridExtents.Z = GridExtents.Z - FMath::Fmod(GridExtents.Z, CellSize.Z / 2);
	}

	int64 NumIterations64 = static_cast<int64>(PointCountX) * static_cast<int64>(PointCountY) * static_cast<int64>(PointCountZ);

	if (NumIterations64 <= 0)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidNumberOfIterations", "The number of iterations produced cannot be less than or equal to 0."));
		return true;
	}

	// Adjust point count/offset with respect to the Culling Shape bounding box if any.
	int32 PointOffsetX = 0;
	int32 PointOffsetY = 0;
	int32 PointOffsetZ = 0;

	if (CullingShape)
	{
		// Get target bounds in world space
		FBox TargetBounds = CullingShape->GetBounds();

		// Move bounds to coordinate space
		TargetBounds = TargetBounds.InverseTransformBy(LocalTransform);

		// Discretize target bounds to the cell size
		TargetBounds.Min /= CellSize;
		TargetBounds.Max /= CellSize;
		
		FIntVector DiscreteMin(FMath::FloorToInt(TargetBounds.Min.X), FMath::FloorToInt(TargetBounds.Min.Y), FMath::FloorToInt(TargetBounds.Min.Z));
		FIntVector DiscreteMax(FMath::CeilToInt(TargetBounds.Max.X), FMath::CeilToInt(TargetBounds.Max.Y), FMath::CeilToInt(TargetBounds.Max.Z));

		// Grid is built from -GridExtents to +GridExtents, so intuitively we'll do this:
		//PointOffset = (PointCount / 2) + FMath::Max(-PointCount / 2, DiscreteMin);
		//PointCount = (PointCount / 2) + FMath::Min(+PointCount / 2, DiscreteMax) - PointOffset;
		PointOffsetX = FMath::Max(0, PointCountX / 2 + DiscreteMin.X);
		PointOffsetY = FMath::Max(0, PointCountY / 2 + DiscreteMin.Y);
		PointOffsetZ = FMath::Max(0, PointCountZ / 2 + DiscreteMin.Z);

		// Implementation note: the +1 here is to make sure we properly manage the case where PointCount isn't even
		PointCountX = FMath::Min(PointCountX, (PointCountX+1) / 2 + DiscreteMax.X) - PointOffsetX;
		PointCountY = FMath::Min(PointCountY, (PointCountY+1) / 2 + DiscreteMax.Y) - PointOffsetY;
		PointCountZ = FMath::Min(PointCountZ, (PointCountZ+1) / 2 + DiscreteMax.Z) - PointOffsetZ;

		// Update iteration count
		NumIterations64 = static_cast<int64>(PointCountX) * static_cast<int64>(PointCountY) * static_cast<int64>(PointCountZ);

		// If it is now equal to zero, return quietly as we might have culled completely
		if (NumIterations64 <= 0)
		{
			return true;
		}
	}

	if (NumIterations64 >= MAX_int32)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("Overflow_int32", "The number of iterations produced is larger than what a 32-bit integer can hold."));
		return true;
	}

	if (PCGFeatureSwitches::CVarCheckSamplerMemory.GetValueOnAnyThread() && FPlatformMemory::GetStats().AvailablePhysical < sizeof(FPCGPoint) * NumIterations64)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("MemoryOverflow", "The number of iterations produced is larger than available memory."));
		return true;
	}

	const int32 NumIterations = static_cast<int32>(NumIterations64);
		
	UPCGBasePointData* PtData = nullptr;

	auto InitFunc = [Context, Settings, CellSize, NumIterations, &PtData]()
	{ 
		PtData = FPCGContext::NewPointData_AnyThread(Context);
		
		check(PtData);
				
		PtData->SetNumPoints(NumIterations);

		// Set Steepness
		PtData->SetSteepness(Settings->PointSteepness);
		if (Settings->bSetPointsBounds)
		{
			PtData->SetExtents(CellSize * 0.5);
		}
						
		PtData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed);
	};

	auto ProcessFunc = [Settings, CullingShape, &PointCountX, &PointCountY, &PointOffsetX, &PointOffsetY, &PointOffsetZ, &CellSize, &GridExtents, &LocalTransform, &PtData](int32 StartIndex, int32 StartWriteIndex, int32 Count)
	{
		TPCGValueRange<FTransform> TransformRange = PtData->GetTransformValueRange(/*bAllocate=*/false);
		TPCGValueRange<int32> SeedRange = PtData->GetSeedValueRange(/*bAllocate=*/false);

		ensure(TransformRange.Num() == SeedRange.Num() && (StartWriteIndex + Count) <= TransformRange.Num());

		int32 NumWritten = 0;
		for(int32 Index = StartIndex; Index < StartIndex+Count; ++Index)
		{
			double XCoordinate = (Index % PointCountX) + PointOffsetX;
			double YCoordinate = ((Index / PointCountX) % PointCountY) + PointOffsetY;
			double ZCoordinate = (Index / (PointCountX * PointCountY)) + PointOffsetZ;

			if (Settings->PointPosition == EPCGPointPosition::CellCenter)
			{
				// If extents are smaller than a point in that dimension, place in center, otherwise offset 
				XCoordinate += 0.5;
				YCoordinate += 0.5;
				ZCoordinate += 0.5;
			}

			// If the extends are smaller than the point, set point to origin
			if (GridExtents.X < CellSize.X / 2.0)
			{
				XCoordinate = 0.0;
			}
			if (GridExtents.Y < CellSize.Y / 2.0)
			{
				YCoordinate = 0.0;
			}
			if (GridExtents.Z < CellSize.Z / 2.0)
			{
				ZCoordinate = 0.0;
			}

			FTransform PointTransform = FTransform(FVector((CellSize.X * XCoordinate) - GridExtents.X, (CellSize.Y * YCoordinate) - GridExtents.Y, (CellSize.Z * ZCoordinate) - GridExtents.Z));

			if (Settings->CoordinateSpace == EPCGCoordinateSpace::LocalComponent || Settings->CoordinateSpace == EPCGCoordinateSpace::OriginalComponent)
			{
				PointTransform *= LocalTransform;
			}

			// Discards points outside of the volume
			if (!CullingShape || (CullingShape->GetDensityAtPosition(PointTransform.GetLocation()) > 0.0f))
			{
				const int32 WriteIndex = StartWriteIndex + NumWritten;
				TransformRange[WriteIndex] = PointTransform;
				SeedRange[WriteIndex] = PCGHelpers::ComputeSeedFromPosition(PointTransform.GetLocation());
				++NumWritten;
			}
		}

		return NumWritten;
	};

	auto MoveRangeFunc = [&PtData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
	{
		PtData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
	};

	auto FinishFunc = [&PtData](int32 NumWritten)
	{
		PtData->SetNumPoints(NumWritten);
	};

	FPCGAsync::AsyncProcessingRangeEx(&Context->AsyncState, NumIterations, InitFunc, ProcessFunc, MoveRangeFunc, FinishFunc, false);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	FPCGTaggedData& Output = Outputs.Emplace_GetRef();
	Output.Data = PtData;

	return true;
}

void FPCGCreatePointsGridElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);
	
	if (const UPCGCreatePointsGridSettings* Settings = Cast<UPCGCreatePointsGridSettings>(InParams.Settings))
	{
		int CoordinateSpace = static_cast<int>(EPCGCoordinateSpace::World);
		bool bCullPointsOutsideVolume = false;
		PCGSettingsHelpers::GetOverrideValue(*InParams.InputData, Settings, GET_MEMBER_NAME_CHECKED(UPCGCreatePointsGridSettings, CoordinateSpace), static_cast<int>(Settings->CoordinateSpace), CoordinateSpace);
		PCGSettingsHelpers::GetOverrideValue(*InParams.InputData, Settings, GET_MEMBER_NAME_CHECKED(UPCGCreatePointsGridSettings, bCullPointsOutsideVolume), Settings->bCullPointsOutsideVolume, bCullPointsOutsideVolume);
		
		// We're using the bounds of the pcg volume, so we extract the actor data here
		EPCGCoordinateSpace EnumCoordinateSpace = static_cast<EPCGCoordinateSpace>(CoordinateSpace);

		if (UPCGData* Data = PCGCreatePointsHelper::GetDependenciesData(InParams.ExecutionSource, EnumCoordinateSpace, bCullPointsOutsideVolume))
		{
			Crc.Combine(Data->GetOrComputeCrc(/*bFullDataCrc=*/false));
		}
	}

	OutCrc = Crc;
}

#undef LOCTEXT_NAMESPACE
