// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCreatePointsSphere.h"

#include "Data/PCGBasePointData.h"
#include "Elements/PCGCreatePoints.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCreatePointsSphere)

#define LOCTEXT_NAMESPACE "PCGCreatePointsSphereElement"

namespace PCGCreatePointsSphere
{
	constexpr double Epsilon = UE_DOUBLE_SMALL_NUMBER;

	struct FCommonParams
	{
		// Handle clamping in the constructor for consistency in every helper function
		explicit FCommonParams(
			double InLatitudinalStartAngle,
			double InLatitudinalEndAngle,
			double InLongitudinalStartAngle,
			double InLongitudinalEndAngle,
			double InRadius,
			const FVector& InOrigin,
			int32 InSeed = 0,
			int32 InPointLimit = INT_MAX,
			FPCGContext* InContext = nullptr)
		: LatitudinalStartAngle(FMath::Abs(InLatitudinalStartAngle) > 90 ? FMath::Fmod(InLatitudinalStartAngle, 90) : InLatitudinalStartAngle)
		, LatitudinalEndAngle(FMath::Abs(InLatitudinalEndAngle) > 90 ? FMath::Fmod(InLatitudinalEndAngle, 90) : InLatitudinalEndAngle)
		, LongitudinalStartAngle(FMath::Abs(InLongitudinalStartAngle) > 180 ? FMath::Fmod(InLongitudinalStartAngle, 180) : InLongitudinalStartAngle)
		, LongitudinalEndAngle(FMath::Abs(InLongitudinalEndAngle) > 180 ? FMath::Fmod(InLongitudinalEndAngle, 180) : InLongitudinalEndAngle)
		, Radius(InRadius)
		, Origin(InOrigin)
		, Seed(InSeed)
		, PointLimit(InPointLimit)
		, Context(InContext) {}

		double LatitudinalStartAngle = 0;
		double LatitudinalEndAngle = 0;
		double LongitudinalStartAngle = 0;
		double LongitudinalEndAngle = 0;
		double Radius = 0;
		FVector Origin = FVector::ZeroVector;
		int32 Seed = 0;
		int32 PointLimit = INT_MAX;
		FPCGContext* Context = nullptr;
	};

	bool GenerationWouldOverflow(const int32 PointCount, const int32 PointLimit, const FPCGContext* InContext = nullptr)
	{
		if (PointCount > PointLimit)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("PointOverflow", "The number of points to be produced exceeds the point limit. See 'Point Limit' under Advanced Settings."), InContext);
			return true;
		}
		else if (PCGFeatureSwitches::CVarCheckSamplerMemory.GetValueOnAnyThread() && PCGFeatureSwitches::Helpers::GetAvailableMemoryForSamplers() < sizeof(FPCGPoint) * PointCount)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("MemoryOverflow", "The number of iterations produced is larger than available memory."), InContext);
			return true;
		}
		else
		{
			return false;
		}
	}

	FVector GeneratePointInWindow(const FCommonParams& Params, FRandomStream& RandomSource)
	{
		// Use the inverse cosine's range of [-1,1] to drive the angle for uniform sampling
		const double NormalizedValue = FMath::GetMappedRangeValueUnclamped<double>({-90, 90}, {-1, 1}, RandomSource.FRandRange(Params.LatitudinalStartAngle, Params.LatitudinalEndAngle));
		const double Theta = FMath::Acos(NormalizedValue);
		const double Phi = FMath::DegreesToRadians(Params.LongitudinalEndAngle - Params.LongitudinalStartAngle) * RandomSource.FRand();
		const double SinTheta = FMath::Sin(Theta);

		return Params.Origin + FVector(SinTheta * FMath::Cos(Phi), SinTheta * FMath::Sin(Phi), FMath::Cos(Theta)) * Params.Radius;
	}

	void GenerateGeodesicSpherePointLocations(const FCommonParams& Params, int32 SubdivisionCount, TArray<FVector>& OutLocations)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreatePointsSphereElement::Geodesic);
		OutLocations.Empty();

		// TODO: Factor out the icosahedron and add options like tetrahedron, octahedron, etc

		// Golden ratio = (1 + sqrt(5))/2
		constexpr double Tau = 1.618033988749894;
		// Norm = sqrt(Tau^2 + 1^2 + 0^2)
		constexpr double Norm = 1.902113032590306;
		// To bring the icosahedron to unit sphere, normalize
		constexpr double OneNorm = 1 / Norm;
		constexpr double TauNorm = Tau / Norm;

		// Icosahedron hardcoded unit sphere vertex locations.
		const TStaticArray<FVector, 12> InitialPointLocations(
			FVector(0, -OneNorm, TauNorm),  // 0
			FVector(0, OneNorm, TauNorm),   // 1
			FVector(-TauNorm, 0, OneNorm),  // 2
			FVector(TauNorm, 0, OneNorm),   // 3
			FVector(-OneNorm, TauNorm, 0),  // 4
			FVector(OneNorm, TauNorm, 0),   // 5
			FVector(-OneNorm, -TauNorm, 0), // 6
			FVector(OneNorm, -TauNorm, 0),  // 7
			FVector(-TauNorm, 0, -OneNorm), // 8
			FVector(TauNorm, 0, -OneNorm),  // 9
			FVector(0, -OneNorm, -TauNorm), // 10
			FVector(0, OneNorm, -TauNorm)); // 11

		// Faster to calculate the sin here and compare it directly later
		const double LatStartSin = FMath::Sin(FMath::DegreesToRadians(Params.LatitudinalStartAngle));
		const double LatEndSin = FMath::Sin(FMath::DegreesToRadians(Params.LatitudinalEndAngle));
		const double LonStartRad = FMath::DegreesToRadians(Params.LongitudinalStartAngle);
		const double LonEndRad = FMath::DegreesToRadians(Params.LongitudinalEndAngle);

		// If all the new indices are outside the window (on the same side), no need to recurse. Use -1 for <, 0 for inside, and 1 for >.
		auto CheckLatWindow = [LatStartSin, LatEndSin](const FVector& PointLocation)
		{
			if (PointLocation.Z < LatStartSin) { return -1; }
			else if (PointLocation.Z > LatEndSin) { return 1; }
			else return 0;
		};

		auto CheckLonWindow = [LonStartRad, LonEndRad](const FVector& PointLocation)
		{
			const double LonAngle = FMath::Atan2(PointLocation.Y, PointLocation.X);
			if (LonAngle < LonStartRad) { return -1; }
			else if (LonAngle > LonEndRad) { return 1; }
			else return 0;
		};

		// First subdivision starts with hardcoded locations. So, prime output with culled initial locations.
		TArray<FVector> PointLocations;
		PointLocations.Reserve(InitialPointLocations.Num());
		for (const FVector& PointLocation : InitialPointLocations)
		{
			if (CheckLatWindow(PointLocation) == 0 && CheckLonWindow(PointLocation) == 0)
			{
				PointLocations.Emplace(PointLocation);
			}
		}

		auto SubdivideTriangleRecursive = [&Out = PointLocations, CheckLatWindow, CheckLonWindow](const FVector& V0, const FVector& V1, const FVector& V2, int32 RemainingSubdivisionCount, auto&& RecursiveCall)
		{
			if (RemainingSubdivisionCount < 1)
			{
				return;
			}

			// First calculate and add the midpoints
			const FVector M0 = (V0 + V1) * 0.5;
			const FVector M1 = (V1 + V2) * 0.5;
			const FVector M2 = (V2 + V0) * 0.5;
			// Normalize
			const FVector N0 = (1.0 / M0.Length()) * M0;
			const FVector N1 = (1.0 / M1.Length()) * M1;
			const FVector N2 = (1.0 / M2.Length()) * M2;

			// Check if these midpoints are in the window (0), inside (-1), or outside (1)
			const int Lat0 = CheckLatWindow(N0);
			const int Lat1 = CheckLatWindow(N1);
			const int Lat2 = CheckLatWindow(N2);
			const int Lon0 = CheckLonWindow(N0);
			const int Lon1 = CheckLonWindow(N1);
			const int Lon2 = CheckLonWindow(N2);

			// TODO: This could likely be better optimized to not rely on unique
			// Add the points as long as they're in both windows
			if (Lat0 == 0 && Lon0 == 0) { Out.AddUnique(N0); }
			if (Lat1 == 0 && Lon1 == 0) { Out.AddUnique(N1); }
			if (Lat2 == 0 && Lon2 == 0) { Out.AddUnique(N2); }

			// Cull subdivisions if the points are outside the windows, but only if both sets are on the same side, since they could straddle.
			if (Lat0 && Lat0 == Lat1 && Lat0 == Lat2)
			{
				if (Lon0 && Lon0 == Lon1 && Lon0 == Lon2)
				{
					return;
				}
			}

			// Then recurse on the four new triangles
			RecursiveCall(V0, M0, M2, RemainingSubdivisionCount - 1, RecursiveCall);
			RecursiveCall(V1, M0, M1, RemainingSubdivisionCount - 1, RecursiveCall);
			RecursiveCall(V2, M1, M2, RemainingSubdivisionCount - 1, RecursiveCall);
			RecursiveCall(M0, M1, M2, RemainingSubdivisionCount - 1, RecursiveCall);
		};

		// Hardcoded Icosahedron Triangles
		constexpr int32 InitialTriangles[] = {0, 1, 2, 0, 1, 3, 0, 2, 6, 0, 6, 7, 0, 3, 7, 1, 2, 4, 1, 3, 5, 1, 4, 5, 2, 4, 8, 2, 6, 8, 3, 5, 9, 3, 7, 9, 4, 5, 11, 4, 8, 11, 5, 9, 11, 6, 7, 10, 6, 8, 10, 7, 9, 10, 8, 10, 11, 9, 10, 11};

		// Call the recursive algorithm on the original triangles
		if (SubdivisionCount > 0)
		{
			for (int I = 2; I < std::size(InitialTriangles); I += 3)
			{
				SubdivideTriangleRecursive(InitialPointLocations[InitialTriangles[I - 2]], InitialPointLocations[InitialTriangles[I - 1]], InitialPointLocations[InitialTriangles[I]], SubdivisionCount, SubdivideTriangleRecursive);
			}
		}

		for (FVector& PointLocation : PointLocations)
		{
			// Finally, apply the user defined transform parameters
			PointLocation = (PointLocation * Params.Radius) + Params.Origin;
		}

		OutLocations.Append(PointLocations);
	}

	void GenerateStandardSpherePointLocations(const FCommonParams& Params, const double InTheta, const double InPhi, const double InJitter, TArray<FVector>& OutLocations)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreatePointsSphereElement::StandardSphere);
		OutLocations.Empty();

		const double& LatStartAngle = Params.LatitudinalStartAngle;
		const double& LatEndAngle = Params.LatitudinalEndAngle;
		const double& LonStartAngle = Params.LongitudinalStartAngle;
		const double& LonEndAngle = Params.LongitudinalEndAngle;

		// Clamp to match the UI input and guard against divide-by-zero.
		const double ThetaPerSegment = FMath::Clamp(InTheta, 0.001, 90.0);
		const double PhiPerSegment = FMath::Clamp(InPhi, 0.001, 180.0);

		const int32 LatSegCount = 1 + FMath::FloorToInt32(FMath::Abs(LatEndAngle - LatStartAngle) / ThetaPerSegment);
		const int32 LonSegCount = FMath::FloorToInt32(FMath::Abs(LonEndAngle - LonStartAngle) / PhiPerSegment);

		int64 PointCount = 0;
		// To produce a single pole point, instead of a whole segment
		int32 ReducedLatSegCount = LatSegCount;

		// Account for the poles
		bool bSouthPole = false;
		if (FMath::IsNearlyEqual(LatEndAngle, -90.0))
		{
			--ReducedLatSegCount;
			++PointCount;
			bSouthPole = true;
		}

		bool bNorthPole = false;
		if (FMath::IsNearlyEqual(LatStartAngle, 90.0))
		{
			--ReducedLatSegCount;
			++PointCount;
			bNorthPole = true;
		}

		PointCount += ReducedLatSegCount * LonSegCount;

		if (PointCount < 1)
		{
			return;
		}

		if (GenerationWouldOverflow(PointCount, Params.PointLimit, Params.Context))
		{
			return;
		}

		OutLocations.SetNumUninitialized(PointCount);

		FRandomStream RandomSource(Params.Seed);
		// Jitter is [0, 0.5) which is the window to not have any overlap possibility
		const double Jitter = FMath::Clamp(InJitter, 0.0, 0.5 - Epsilon);
		bool bApplyJitter = !FMath::IsNearlyZero(Jitter);

		int32 CurrentIndex = 0;
		for (int LatIndex = 0; LatIndex < LatSegCount; ++LatIndex)
		{
			// Add the poles under certain conditions
			if (LatIndex == 0 && bSouthPole)
			{
				OutLocations[CurrentIndex++] = Params.Origin - FVector::ZAxisVector * Params.Radius;
				continue;
			}

			if (LatIndex == LatSegCount - 1 && bNorthPole)
			{
				OutLocations[CurrentIndex++] = Params.Origin + FVector::ZAxisVector * Params.Radius;
				continue;
			}

			// -90 to map to a -180 to 0 range for cosine
			const double Theta = FMath::DegreesToRadians(LatStartAngle - 90 + ThetaPerSegment * LatIndex);
			// Pre-calculation of these for the internal loop is possible unless jittered
			double SinTheta = bApplyJitter ? 0.0 : FMath::Sin(Theta);
			double CosTheta = bApplyJitter ? 0.0 : FMath::Cos(Theta);

			for (int LonIndex = 0; LonIndex < LonSegCount; ++LonIndex)
			{
				double Phi = FMath::DegreesToRadians(LonStartAngle + PhiPerSegment * LonIndex);

				if (bApplyJitter)
				{
					Phi += FMath::DegreesToRadians(PhiPerSegment) * RandomSource.FRandRange(-Jitter, Jitter);
					const double JitteredTheta = Theta + FMath::DegreesToRadians(ThetaPerSegment) * RandomSource.FRandRange(-Jitter, Jitter);
					SinTheta = FMath::Sin(JitteredTheta);
					CosTheta = FMath::Cos(JitteredTheta);
				}

				OutLocations[CurrentIndex++] = Params.Origin + FVector(SinTheta * FMath::Cos(Phi), SinTheta * FMath::Sin(Phi), CosTheta) * Params.Radius;
			}
		}

		check(CurrentIndex == PointCount);
	}

	void GenerateRandomSamplingPointLocations(const FCommonParams& Params, const int32 SampleCount, TArray<FVector>& OutLocations)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreatePointsSphereElement::RandomSampling);
		OutLocations.Empty();

		if (GenerationWouldOverflow(SampleCount, Params.PointLimit, Params.Context))
		{
			return;
		}

		OutLocations.SetNumUninitialized(SampleCount);

		FRandomStream RandomSource(Params.Seed);

		for (int32 Index = 0; Index < SampleCount; ++Index)
		{
			OutLocations[Index] = GeneratePointInWindow(Params, RandomSource);
		}
	}

	void GeneratePoissonSamplingPointLocations(const FCommonParams& Params, const double Distance, const int32 SampleCount, const int32 MaxAttempts, TArray<FVector>& OutLocations)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreatePointsSphereElement::Poisson);
		OutLocations.Empty();

		if (GenerationWouldOverflow(SampleCount, Params.PointLimit, Params.Context))
		{
			return;
		}

		OutLocations.Reserve(SampleCount);

		const double DistanceSquared = Distance * Distance;

		FRandomStream RandomSource(Params.Seed);

		for (int I = 0; I < SampleCount; ++I)
		{
			for (int32 Attempt = 0; Attempt < MaxAttempts; ++Attempt)
			{
				FVector Point = GeneratePointInWindow(Params, RandomSource);

				// TODO: Might be beneficial to check against an octree before the distance check
				bool bValidPoint = true;
				for (const FVector& PreviousPoint : OutLocations)
				{
					if ((PreviousPoint - Point).SizeSquared() < DistanceSquared)
					{
						bValidPoint = false;
						break;
					}
				}

				if (bValidPoint)
				{
					OutLocations.Emplace(Point);
					break;
				}
			}
		}
	}
}

UPCGCreatePointsSphereSettings::UPCGCreatePointsSphereSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		PointSteepness = 1.0f;
	}
}

TArray<FPCGPinProperties> UPCGCreatePointsSphereSettings::OutputPinProperties() const
{
	return DefaultPointOutputPinProperties();
}

FPCGElementPtr UPCGCreatePointsSphereSettings::CreateElement() const
{
	return MakeShared<FPCGCreatePointsSphereElement>();
}

bool FPCGCreatePointsSphereElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreatePointsSphereElement::Execute);
	check(InContext);

	if (!InContext->ExecutionSource.IsValid())
	{
		return true;
	}

	const UPCGCreatePointsSphereSettings* Settings = InContext->GetInputSettings<UPCGCreatePointsSphereSettings>();

	// Used for culling, regardless of generation coordinate space
	const UPCGSpatialData* CullingShape = Settings->bCullPointsOutsideVolume ? Cast<UPCGSpatialData>(InContext->ExecutionSource->GetExecutionState().GetSelfData()) : nullptr;

	// Early out if the culling shape isn't valid
	if (!CullingShape && Settings->bCullPointsOutsideVolume)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("CannotCullWithoutAShape", "Unable to cull since the supporting actor has no data."));
		return true;
	}

	const PCGCreatePointsSphere::FCommonParams Params(
		Settings->LatitudinalStartAngle,
		Settings->LatitudinalEndAngle,
		Settings->LongitudinalStartAngle,
		Settings->LongitudinalEndAngle,
		Settings->Radius,
		Settings->Origin,
		InContext->GetSeed(),
		Settings->PointLimit,
		InContext);

	TArray<FVector> OutLocalPointLocations;
	switch (Settings->SphereGeneration)
	{
		case EPCGSphereGeneration::Geodesic:
			{
				PCGCreatePointsSphere::GenerateGeodesicSpherePointLocations(Params, Settings->GeodesicSubdivisions, OutLocalPointLocations);
			}
			break;
		case EPCGSphereGeneration::Angle:
			{
				PCGCreatePointsSphere::GenerateStandardSpherePointLocations(Params, Settings->Theta, Settings->Phi, Settings->Jitter, OutLocalPointLocations);
			}
			break;
		case EPCGSphereGeneration::Segments:
			{
				if (Settings->LatitudinalSegments < 1 || Settings->LongitudinalSegments < 1)
				{
					return true;
				}

				const double Theta = FMath::Abs(Settings->LatitudinalStartAngle - Settings->LatitudinalEndAngle) / Settings->LatitudinalSegments;
				const double Phi = FMath::Abs(Settings->LongitudinalStartAngle - Settings->LongitudinalEndAngle) / Settings->LongitudinalSegments;
				PCGCreatePointsSphere::GenerateStandardSpherePointLocations(Params, Theta, Phi, Settings->Jitter, OutLocalPointLocations);
			}
			break;
		case EPCGSphereGeneration::Random:
			{
				PCGCreatePointsSphere::GenerateRandomSamplingPointLocations(Params, Settings->SampleCount, OutLocalPointLocations);
			}
			break;
		case EPCGSphereGeneration::Poisson:
			{
				PCGCreatePointsSphere::GeneratePoissonSamplingPointLocations(Params, Settings->PoissonDistance, Settings->SampleCount, Settings->PoissonMaxAttempts, OutLocalPointLocations);
			}
			break;
		default:
			checkNoEntry();
			break;
	}

	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;
	FPCGTaggedData& Output = Outputs.Emplace_GetRef();

	UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(InContext);
	Output.Data = PointData;

	FTransform LocalTransform = FTransform::Identity;
	if (Settings->CoordinateSpace == EPCGCoordinateSpace::LocalComponent)
	{
		LocalTransform = InContext->ExecutionSource->GetExecutionState().GetTransform();
	}
	else if (Settings->CoordinateSpace == EPCGCoordinateSpace::OriginalComponent)
	{
		LocalTransform = InContext->ExecutionSource->GetExecutionState().GetOriginalTransform();
	}

	// Point scale should not be relative to the coordinate space
	LocalTransform.SetScale3D(FVector::One());

	TArray<TTuple<FTransform, int32>> Points;
	Points.Reserve(OutLocalPointLocations.Num());

	for (int Index = 0; Index < OutLocalPointLocations.Num(); ++Index)
	{
		FRotator LocalPointOrientation = FRotator::ZeroRotator;

		// TODO: A second pass would be useful, as we could likely make this more efficient.
		switch (Settings->PointOrientation)
		{
			case EPCGSpherePointOrientation::Radial:
			{
				LocalPointOrientation = FRotationMatrix::MakeFromZ(OutLocalPointLocations[Index] - Settings->Origin).Rotator();
			}
			break;
			case EPCGSpherePointOrientation::Centric:
			{
				LocalPointOrientation = FRotationMatrix::MakeFromZ(Settings->Origin - OutLocalPointLocations[Index]).Rotator();
			}
			break;
			case EPCGSpherePointOrientation::None:
				break;
			default:
				checkNoEntry();
				break;
		}

		FTransform PointTransform(LocalPointOrientation, OutLocalPointLocations[Index]);

		if (Settings->CoordinateSpace == EPCGCoordinateSpace::LocalComponent || Settings->CoordinateSpace == EPCGCoordinateSpace::OriginalComponent)
		{
			PointTransform *= LocalTransform;
		}

		if (!CullingShape || (CullingShape->GetDensityAtPosition(PointTransform.GetLocation()) > 0.f))
		{
			Points.Add({ PointTransform, PCGHelpers::ComputeSeedFromPosition(OutLocalPointLocations[Index]) });
		}
	}

	PointData->SetNumPoints(Points.Num());
	PointData->SetDensity(1.0f);
	PointData->SetSteepness(Settings->PointSteepness);
	PointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed);

	TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange(/*bAllocate=*/false);
	TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange(/*bAllocate=*/false);

	for (int32 Index = 0; Index < PointData->GetNumPoints(); ++Index)
	{
		TransformRange[Index] = Points[Index].Get<0>();
		SeedRange[Index] = Points[Index].Get<1>();
	}

	return true;
}

void FPCGCreatePointsSphereElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	if (const UPCGCreatePointsSphereSettings* Settings = Cast<UPCGCreatePointsSphereSettings>(InParams.Settings))
	{
		int CoordinateSpace = static_cast<int>(EPCGCoordinateSpace::World);
		bool bCullPointsOutsideVolume = false;
		PCGSettingsHelpers::GetOverrideValue(*InParams.InputData, Settings, GET_MEMBER_NAME_CHECKED(UPCGCreatePointsSphereSettings, CoordinateSpace), static_cast<int>(Settings->CoordinateSpace), CoordinateSpace);
		PCGSettingsHelpers::GetOverrideValue(*InParams.InputData, Settings, GET_MEMBER_NAME_CHECKED(UPCGCreatePointsSphereSettings, bCullPointsOutsideVolume), Settings->bCullPointsOutsideVolume, bCullPointsOutsideVolume);

		EPCGCoordinateSpace EnumCoordinateSpace = static_cast<EPCGCoordinateSpace>(CoordinateSpace);

		if (UPCGData* Data = PCGCreatePointsHelper::GetDependenciesData(InParams.ExecutionSource, EnumCoordinateSpace, bCullPointsOutsideVolume))
		{
			Crc.Combine(Data->GetOrComputeCrc(/*bFullDataCrc=*/false));
		}
	}

	OutCrc = Crc;
}

#undef LOCTEXT_NAMESPACE
