// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCreateSpline.h"

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGManagedResource.h"
#include "PCGManagedResourceContainer.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Elements/PCGSplineSampler.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include "Components/SplineComponent.h"
#include "Engine/World.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCreateSpline)

#define LOCTEXT_NAMESPACE "PCGCreateSpline"

UPCGCreateSplineSettings::UPCGCreateSplineSettings()
{
	// Change the default for Arrive and Leave tangent
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		ArriveTangentAttribute = PCGSplineSamplerConstants::ArriveTangentAttributeName;
		LeaveTangentAttribute = PCGSplineSamplerConstants::LeaveTangentAttributeName;
		InterpTypeAttribute = PCGSplineSamplerConstants::InterpTypeAttributeName;
	}
}

#if WITH_EDITOR
FText UPCGCreateSplineSettings::GetNodeTooltipText() const
{
	return LOCTEXT("CreateSplineTooltip", "Creates PCG spline data from the input PCG point data, in a sequential order.");
}
#endif

TArray<FPCGPinProperties> UPCGCreateSplineSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spline);

	return PinProperties;
}

FPCGElementPtr UPCGCreateSplineSettings::CreateElement() const
{
	return MakeShared<FPCGCreateSplineElement>();
}

bool FPCGCreateSplineElement::CanExecuteOnlyOnMainThread(FPCGContext* Context) const
{
	// Creating the spline component requires to run on the main thread, but if the settings/context aren't available we'll err on the side of caution.
	const UPCGCreateSplineSettings* Settings = Context ? Context->GetInputSettings<UPCGCreateSplineSettings>() : nullptr;
	return !Settings || Settings->Mode == EPCGCreateSplineMode::CreateComponent;
}

bool FPCGCreateSplineElement::IsCacheable(const UPCGSettings* InSettings) const
{
	const UPCGCreateSplineSettings* Settings = Cast<const UPCGCreateSplineSettings>(InSettings);
	return !Settings || Settings->Mode == EPCGCreateSplineMode::CreateDataOnly;
}

bool FPCGCreateSplineElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreateSplineElement::Execute);

	const UPCGCreateSplineSettings* Settings = Context->GetInputSettings<UPCGCreateSplineSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	EPCGCreateSplineMode Mode = Settings->Mode;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Input.Data);

		if (!SpatialData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data"));
			continue;
		}

		AActor* TargetActor = Settings->TargetActor.Get() ? Settings->TargetActor.Get() : Context->GetTypedExecutionTarget<AActor>();
		if (!TargetActor && Settings->Mode != EPCGCreateSplineMode::CreateDataOnly)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidTargetActor", "Invalid target actor. Ensure TargetActor member is initialized when creating SpatialData."));
			continue;
		}

		const UPCGBasePointData* PointData = SpatialData->ToBasePointData(Context);

		if (!PointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("UnableToGetPointData", "Unable to get point data from input"));
			continue;
		}

		const FPCGMetadataAttribute<FVector>* ArriveTangentAttribute = nullptr;
		const FPCGMetadataAttribute<FVector>* LeaveTangentAttribute = nullptr;
		const FPCGMetadataAttribute<int32>* InterpTypeAttribute = nullptr;

		if (Settings->bApplyCustomTangents && !Settings->bLinear)
		{
			const UPCGMetadata* PointMetadata = PointData->ConstMetadata();
			check(PointMetadata);

			FName LocalArriveTangentName = ((Settings->ArriveTangentAttribute == NAME_None) ? PointMetadata->GetLatestAttributeNameOrNone() : Settings->ArriveTangentAttribute);
			FName LocalLeaveTangentName = ((Settings->LeaveTangentAttribute == NAME_None) ? PointMetadata->GetLatestAttributeNameOrNone() : Settings->LeaveTangentAttribute);

			const FText AttributeMissingOrNotVector = LOCTEXT("AttributeMissingOrNotVector", "Attribute '{0}' does not exist or is not a vector");

			const FPCGMetadataAttributeBase* ArriveTangentBaseAttribute = PointMetadata->GetConstAttribute(LocalArriveTangentName);
			if (!ArriveTangentBaseAttribute || ArriveTangentBaseAttribute->GetTypeId() != PCG::Private::MetadataTypes<FVector>::Id)
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(AttributeMissingOrNotVector, FText::FromString(LocalArriveTangentName.ToString())));
				continue;
			}

			const FPCGMetadataAttributeBase* LeaveTangentBaseAttribute = PointMetadata->GetConstAttribute(LocalLeaveTangentName);
			if (!LeaveTangentBaseAttribute || LeaveTangentBaseAttribute->GetTypeId() != PCG::Private::MetadataTypes<FVector>::Id)
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(AttributeMissingOrNotVector, FText::FromString(LocalLeaveTangentName.ToString())));
				continue;
			}

			ArriveTangentAttribute = static_cast<const FPCGMetadataAttribute<FVector>*>(ArriveTangentBaseAttribute);
			LeaveTangentAttribute = static_cast<const FPCGMetadataAttribute<FVector>*>(LeaveTangentBaseAttribute);
		}

		if (Settings->bUseInterpTypeAttribute)
		{
			const UPCGMetadata* PointMetadata = PointData->ConstMetadata();
			check(PointMetadata);

			FName LocalPointAttributeName = ((Settings->InterpTypeAttribute == NAME_None) ? PointMetadata->GetLatestAttributeNameOrNone() : Settings->InterpTypeAttribute);
			const FPCGMetadataAttributeBase* InterpTypeBaseAttribute = PointMetadata->GetConstAttribute(LocalPointAttributeName);
			if (!InterpTypeBaseAttribute || InterpTypeBaseAttribute->GetTypeId() != PCG::Private::MetadataTypes<int32>::Id)
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("PointTypeAttributeMissing", "Point Type Attribute '{0}' does not exist or is not an int32."), FText::FromString(LocalPointAttributeName.ToString())));
				continue;
			}

			InterpTypeAttribute = static_cast<const FPCGMetadataAttribute<int32>*>(InterpTypeBaseAttribute);
		}

		UPCGSplineData* SplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
		SplineData->InitializeFromData(PointData);
		AActor* SplineActor = TargetActor;

		const int32 NumPoints = PointData->GetNumPoints();
		TArray<FSplinePoint> SplinePoints;
		SplinePoints.Reserve(NumPoints);

		TArray<PCGMetadataEntryKey> SplineEntryKeys;
		SplineEntryKeys.Reserve(NumPoints);
		bool bHasAValidEntry = false;

		const FTransform SplineActorTransform = SplineActor ? SplineActor->GetTransform() : FTransform::Identity;
		
		ESplinePointType::Type FixedPointType = ESplinePointType::Curve;
		if (Settings->bLinear)
		{
			FixedPointType = ESplinePointType::Linear;
		}
		else if (Settings->bApplyCustomTangents)
		{
			FixedPointType = ESplinePointType::CurveCustomTangent;
		}

		FConstPCGPointValueRanges InRanges(PointData);

		for(int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			const FTransform& PointTransform = InRanges.TransformRange[PointIndex];
			const FVector LocalPosition = PointTransform.GetLocation() - SplineActorTransform.GetLocation();

			int64 PointMetadataEntry = InRanges.MetadataEntryRange[PointIndex];
			SplinePoints.Emplace(static_cast<float>(PointIndex),
				LocalPosition,
				ArriveTangentAttribute ? ArriveTangentAttribute->GetValueFromItemKey(PointMetadataEntry) : FVector::ZeroVector,
				LeaveTangentAttribute ? LeaveTangentAttribute->GetValueFromItemKey(PointMetadataEntry) : FVector::ZeroVector,
				PointTransform.GetRotation().Rotator(),
				PointTransform.GetScale3D(),
				InterpTypeAttribute ? static_cast<ESplinePointType::Type>(InterpTypeAttribute->GetValueFromItemKey(PointMetadataEntry)) : FixedPointType);

			SplineEntryKeys.Emplace(PointMetadataEntry);
			bHasAValidEntry |= (PointMetadataEntry != PCGInvalidEntryKey);
		}

		if (!bHasAValidEntry)
		{
			SplineEntryKeys.Empty();
		}

		SplineData->Initialize(SplinePoints, Settings->bClosedLoop, FTransform(SplineActorTransform.GetLocation()), std::move(SplineEntryKeys));

		USplineComponent* SplineComponent = nullptr;

		if (Settings->Mode != EPCGCreateSplineMode::CreateDataOnly)
		{
			check(IsInGameThread());
			check(SplineActor);

			IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
			FPCGManagedResourceContainerHelper ManagedResourcesContainerHelper(ExecutionSource);
			UObject* ExecutionSourceAsObject = Cast<UObject>(ExecutionSource);

#if WITH_EDITOR
			FScopedTransaction Transaction(LOCTEXT("AddSplineComponents", "Adding spline component to actor from PCG"), ExecutionSource && ExecutionSource->GetExecutionState().UseTransactions());
#endif

			if(ExecutionSourceAsObject && ManagedResourcesContainerHelper.IsValid())
			{
				SplineComponent = NewObject<USplineComponent>(SplineActor);
				SplineComponent->ComponentTags.Add(ExecutionSourceAsObject->GetFName());
				SplineComponent->ComponentTags.Add(PCGHelpers::DefaultPCGTag);
				for (const FName& Tag : Settings->TagsToAddOnComponents)
				{
					SplineComponent->ComponentTags.AddUnique(Tag);
				}

				SplineComponent->RegisterComponent();
				SplineActor->AddInstanceComponent(SplineComponent);

				if (!SplineComponent->AttachToComponent(SplineActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, false)))
				{
					PCGLog::Component::LogComponentAttachmentFailedWarning(Context);
				}

				SplineData->ApplyTo(SplineComponent);

				UPCGManagedComponent* ManagedComponent = NewObject<UPCGManagedComponent>(ExecutionSourceAsObject);
				ManagedComponent->GeneratedComponent = SplineComponent;

				ManagedResourcesContainerHelper.AddManagedResource(ManagedComponent);
			}

			UFunction* FunctionPrototype = UPCGFunctionPrototypes::GetPrototypeWithNoParams();

			// Execute PostProcess Functions
			for (UFunction* Function : PCGHelpers::FindUserFunctions(SplineActor->GetClass(), Settings->PostProcessFunctionNames, { FunctionPrototype }, Context))
			{
				SplineActor->ProcessEvent(Function, nullptr);
			}
		}

		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
		Output.Data = SplineData;
	}

	// Pass-through settings & exclusions
	Context->OutputData.TaggedData.Append(Context->InputData.GetAllSettings());

	return true;
}

#undef LOCTEXT_NAMESPACE