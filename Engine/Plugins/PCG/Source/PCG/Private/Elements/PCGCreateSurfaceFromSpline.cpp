// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCreateSurfaceFromSpline.h"

#include "PCGContext.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGSplineInteriorSurfaceData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCreateSurfaceFromSpline)

#define LOCTEXT_NAMESPACE "PCGCreateSurfaceFromSplineElement"

#if WITH_EDITOR
FText UPCGCreateSurfaceFromSplineSettings::GetNodeTooltipText() const
{
	// TODO: We should refactor interior sampling to allow for local-space projection to yield more correct sampling for rotated splines.
	return LOCTEXT("NodeTooltip", "Create an implicit surface for each given spline. The surface is given by the top-down 2D projection of the spline. Each spline must be closed.");
}
#endif

TArray<FPCGPinProperties> UPCGCreateSurfaceFromSplineSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spline, /*bAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);
	PinProperties[0].SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGCreateSurfaceFromSplineSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Surface, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);
	return PinProperties;
}

FPCGElementPtr UPCGCreateSurfaceFromSplineSettings::CreateElement() const
{
	return MakeShared<FPCGCreateSurfaceFromSplineElement>();
}

bool FPCGCreateSurfaceFromSplineElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreateSurfaceFromSplineElement::Execute);

	const UPCGCreateSurfaceFromSplineSettings* Settings = Context->GetInputSettings<UPCGCreateSurfaceFromSplineSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGSplineData* SplineData = Cast<UPCGSplineData>(Input.Data);

		if (!SplineData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input. Must be a spline data."));
			continue;
		}

		if (!SplineData->IsClosed())
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidOpenSpline", "Unable to create an implicit surface from an open spline. Make sure your splines are closed."));
			continue;
		}

		UPCGSplineInteriorSurfaceData* SurfaceData = FPCGContext::NewObject_AnyThread<UPCGSplineInteriorSurfaceData>(Context);
		SurfaceData->Initialize(Context, SplineData);

		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
		Output.Data = SurfaceData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
