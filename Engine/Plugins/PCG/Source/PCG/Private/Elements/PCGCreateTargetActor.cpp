// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCreateTargetActor.h"

#include "PCGContext.h"
#include "PCGCrc.h"
#include "PCGElement.h"
#include "PCGManagedResource.h"
#include "PCGManagedResourceContainer.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Data/PCGPointData.h"
#include "Graph/PCGStackContext.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGDataLayerHelpers.h"
#include "Helpers/PCGHelpers.h"

#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Serialization/ArchiveCrc32.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCreateTargetActor)

#define LOCTEXT_NAMESPACE "PCGCreateTargetActor"

#if WITH_EDITOR

FText UPCGCreateTargetActor::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Create Target Actor");
}

#endif // WITH_EDITOR

namespace PCGCreateTargetActorConstants
{
	const FName ActorPropertyOverridesLabel = TEXT("Property Overrides");
	const FText ActorPropertyOverridesTooltip = LOCTEXT("ActorOverrideToolTip", "Provide property overrides for the created target actor. The attribute name must match the InputSource name in the actor property override description.");
}

namespace PCGCreateTargetActor
{
	static TAutoConsoleVariable<bool> CVarCreateTargetActorAllowReuse(
		TEXT("pcg.CreateTargetActor.AllowReuse"),
		true,
		TEXT("Controls whether PCG Create Target Actor node can reuse actors when re-executing (requires Create Target Actor node resave so they have a Stable Reuse Guid)"));

	static TAutoConsoleVariable<bool> CVarEnableWarningOnDeprecatedCSVActorTags(
		TEXT("pcg.Graph.EnableWarningOnDeprecatedCSVActorTags"),
		true,
		TEXT("Output warnings when using dreprecated CSV actor tags inputs."));

	FPCGCrc GetAdditionalDependenciesCrc(const UPCGActorHelpers::FSpawnDefaultActorParams& InParams, AActor* TargetActor)
	{
		FArchiveCrc32 Ar;

		Ar << TargetActor;

		// Do not CRC Everything as some of those params come from the Settings which is already CRCed
		UObject* Level = InParams.SpawnParams.OverrideLevel;
		Ar << Level;

		int32 ObjectFlags = InParams.SpawnParams.ObjectFlags;
		Ar << ObjectFlags;
				
		// Include transform - round sufficiently to ensure stability
		FVector TransformLocation = InParams.Transform.GetLocation();
		FIntVector Location(FMath::RoundToInt(TransformLocation.X), FMath::RoundToInt(TransformLocation.Y), FMath::RoundToInt(TransformLocation.Z));
		Ar << Location;

		FRotator Rotator(InParams.Transform.Rotator().GetDenormalized());
		const int32 MAX_DEGREES = 360;
		FIntVector Rotation(FMath::RoundToInt(Rotator.Pitch) % MAX_DEGREES, FMath::RoundToInt(Rotator.Yaw) % MAX_DEGREES, FMath::RoundToInt(Rotator.Roll) % MAX_DEGREES);
		Ar << Rotation;

		FVector TransformScale = InParams.Transform.GetScale3D();
		const float SCALE_FACTOR = 100;
		FIntVector Scale(FMath::RoundToInt(TransformScale.X * SCALE_FACTOR), FMath::RoundToInt(TransformScale.Y * SCALE_FACTOR), FMath::RoundToInt(TransformScale.Z * SCALE_FACTOR));
		Ar << Scale;
				
#if WITH_EDITOR
		TArray<const UDataLayerInstance*> DataLayerInstances = InParams.DataLayerInstances;
		DataLayerInstances.Sort([](const UDataLayerInstance& A, const UDataLayerInstance& B)
		{
			return A.GetDataLayerFullName() < B.GetDataLayerFullName();
		});

		for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
		{
			UDataLayerInstance* NonConstDataLayerInstance = const_cast<UDataLayerInstance*>(DataLayerInstance);
			Ar << NonConstDataLayerInstance;
		}

		if (InParams.HLODLayer)
		{
			UHLODLayer* HLODLayer = InParams.HLODLayer;
			Ar << HLODLayer;
		}
#endif
		
		return FPCGCrc(Ar.GetCrc());
	}
}

UPCGCreateTargetActor::UPCGCreateTargetActor(const FObjectInitializer& ObjectInitializer)
	: UPCGSettings(ObjectInitializer)
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		AttachOptions = EPCGAttachOptions::InFolder;
	}
}

FPCGElementPtr UPCGCreateTargetActor::CreateElement() const
{
	return MakeShared<FPCGCreateTargetActorElement>();
}

#if WITH_EDITOR
EPCGChangeType UPCGCreateTargetActor::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	ChangeType |= DataLayerSettings.GetChangeTypeForProperty(InPropertyName);

	return ChangeType;
}
#endif

TArray<FPCGPinProperties> UPCGCreateTargetActor::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Add(PCGObjectPropertyOverrideHelpers::CreateObjectPropertiesOverridePin(PCGCreateTargetActorConstants::ActorPropertyOverridesLabel, PCGCreateTargetActorConstants::ActorPropertyOverridesTooltip));
	PinProperties.Append(DataLayerSettings.InputPinProperties());

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGCreateTargetActor::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param, false);

	return PinProperties;
}

void UPCGCreateTargetActor::BeginDestroy()
{
#if WITH_EDITOR
	TeardownBlueprintEvent();
#endif

	Super::BeginDestroy();
}

#if WITH_EDITOR
void UPCGCreateTargetActor::SetupBlueprintEvent()
{
	if (UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(TemplateActorClass))
	{
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UPCGCreateTargetActor::OnObjectsReplaced);
	}
}

void UPCGCreateTargetActor::TeardownBlueprintEvent()
{
	if (UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(TemplateActorClass))
	{
		FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	}
}

void UPCGCreateTargetActor::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGCreateTargetActor, TemplateActorClass))
	{
		TeardownBlueprintEvent();
	}
}

void UPCGCreateTargetActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		const FName& PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGCreateTargetActor, TemplateActorClass))
		{
			SetupBlueprintEvent();
			RefreshTemplateActor();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGCreateTargetActor, bAllowTemplateActorEditing))
		{
			RefreshTemplateActor();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPCGCreateTargetActor::PreEditUndo()
{
	TeardownBlueprintEvent();

	Super::PreEditUndo();
}

void UPCGCreateTargetActor::PostEditUndo()
{
	Super::PostEditUndo();

	SetupBlueprintEvent();
	RefreshTemplateActor();
}

void UPCGCreateTargetActor::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances)
{
	if (!TemplateActor)
	{
		return;
	}
	
	if (UObject* NewObject = InOldToNewInstances.FindRef(TemplateActor))
	{
		TemplateActor = Cast<AActor>(NewObject);
		OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Settings);
	}
}

void UPCGCreateTargetActor::RefreshTemplateActor()
{
	// Implementation note: this is similar to the child actor component implementation
	if (TemplateActorClass && bAllowTemplateActorEditing)
	{
		const bool bCreateNewTemplateActor = (!TemplateActor || TemplateActor->GetClass() != TemplateActorClass);

		if (bCreateNewTemplateActor)
		{
			AActor* NewTemplateActor = NewObject<AActor>(GetTransientPackage(), TemplateActorClass, NAME_None, RF_ArchetypeObject | RF_Transactional | RF_Public);

			if (TemplateActor)
			{
				UEngine::FCopyPropertiesForUnrelatedObjectsParams Options;
				Options.bNotifyObjectReplacement = true;
				UEngine::CopyPropertiesForUnrelatedObjects(TemplateActor, NewTemplateActor, Options);

				TemplateActor->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

				TMap<UObject*, UObject*> OldToNew;
				OldToNew.Emplace(TemplateActor, NewTemplateActor);
				GEngine->NotifyToolsOfObjectReplacement(OldToNew);

				TemplateActor->MarkAsGarbage();
			}

			TemplateActor = NewTemplateActor;

			// Record initial object state in case we're in a transaction context.
			TemplateActor->Modify();

			// Outer to this object
			TemplateActor->Rename(nullptr, this, REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
	}
	else
	{
		if (TemplateActor)
		{
			TemplateActor->MarkAsGarbage();
		}

		TemplateActor = nullptr;
	}
}

#endif // WITH_EDITOR

void UPCGCreateTargetActor::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Since the template actor editing is set to false by default, this needs to be corrected on post-load for proper deprecation
	if (TemplateActor)
	{
		bAllowTemplateActorEditing = true;
	}

	SetupBlueprintEvent();

	if (TemplateActorClass)
	{
		if (TemplateActor)
		{
			TemplateActor->ConditionalPostLoad();
		}

		RefreshTemplateActor();
	}
#endif // WITH_EDITOR
}

void UPCGCreateTargetActor::SetTemplateActorClass(const TSubclassOf<AActor>& InTemplateActorClass)
{
#if WITH_EDITOR
	TeardownBlueprintEvent();
#endif // WITH_EDITOR

	TemplateActorClass = InTemplateActorClass;

#if WITH_EDITOR
	SetupBlueprintEvent();
	RefreshTemplateActor();
#endif // WITH_EDITOR
}

void UPCGCreateTargetActor::SetAllowTemplateActorEditing(bool bInAllowTemplateActorEditing)
{
	bAllowTemplateActorEditing = bInAllowTemplateActorEditing;

#if WITH_EDITOR
	RefreshTemplateActor();
#endif // WITH_EDITOR
}

bool FPCGCreateTargetActorElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreateTargetActorElement::Execute);

	check(IsInGameThread());

	// Early out if the actor isn't going to be consumed by something else
	if (Context->Node && !Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultOutputLabel))
	{
		return true;
	}

	const UPCGCreateTargetActor* Settings = Context->GetInputSettings<UPCGCreateTargetActor>();
	check(Settings);

	// Early out if the template actor isn't valid
	const TSubclassOf<AActor> TemplateActorClass = Settings->OverrideTemplateActorClass ? Settings->OverrideTemplateActorClass : Settings->TemplateActorClass;

	if (!TemplateActorClass || TemplateActorClass->HasAnyClassFlags(CLASS_Abstract) || !TemplateActorClass->GetDefaultObject()->IsA<AActor>())
	{
		const FText ClassName = TemplateActorClass ? FText::FromString(TemplateActorClass->GetFName().ToString()) : FText::FromName(NAME_None);
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InvalidTemplateActorClass", "Invalid template actor class '{0}'"), ClassName));
		return true;
	}

	if (Settings->TemplateActor && !TemplateActorClass->IsChildOf(Settings->TemplateActor->GetClass()))
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidTemplateActor", "Template actor class ({0}) is not a child class of template actor class ({1})"), 
			FText::FromString(TemplateActorClass->GetName()), FText::FromString(Settings->TemplateActor->GetClass()->GetName())));
		return true;
	}

	AActor* TargetActor = Settings->RootActor.Get() ? Settings->RootActor.Get() : Context->GetTypedExecutionTarget<AActor>();
	UObject* TargetWorldObject = TargetActor ? TargetActor : Context->GetExecutionTarget();

	if (!TargetWorldObject || !TargetWorldObject->GetWorld())
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidTargetObject", "Invalid target object. The action will fail because it is required to obtain the world."));
		return true;
	}

	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	const bool bHasAuthority = !ExecutionSource || ExecutionSource->GetExecutionState().HasAuthority();
	const bool bSpawnedActorRequiresAuthority = CastChecked<AActor>(TemplateActorClass->GetDefaultObject())->GetIsReplicated();

	if (!bHasAuthority && bSpawnedActorRequiresAuthority)
	{
		return true;
	}

	// Mimic Spawn Actor node.
	AActor* TemplateActor = nullptr;
	// We can use the template actor as-is if the classes matches perfectly.
	const bool bClassMismatch = Settings->TemplateActor && TemplateActorClass != Settings->TemplateActor->GetClass();

	if (Settings->TemplateActor && !bClassMismatch)
	{
		if (Settings->PropertyOverrideDescriptions.IsEmpty())
		{
			TemplateActor = Settings->TemplateActor;
		}
		else
		{
			TemplateActor = DuplicateObject(Settings->TemplateActor, GetTransientPackage());
		}
	}
	else
	{
		if (Settings->PropertyOverrideDescriptions.IsEmpty() && !bClassMismatch)
		{
			TemplateActor = Cast<AActor>(TemplateActorClass->GetDefaultObject());
		}
		else
		{
			TemplateActor = NewObject<AActor>(GetTransientPackage(), TemplateActorClass, NAME_None, RF_ArchetypeObject);

			// If we have a class mismatch, copy the properties from the template actor.
			if (bClassMismatch)
			{
				UEngine::CopyPropertiesForUnrelatedObjects(Settings->TemplateActor, TemplateActor);
			}
		}
	}

	check(TemplateActor);
	
	if (!Settings->PropertyOverrideDescriptions.IsEmpty())
	{
		// Apply property overrides to the Template actor
		PCGObjectPropertyOverrideHelpers::ApplyOverridesFromParams(Settings->PropertyOverrideDescriptions, TemplateActor, PCGCreateTargetActorConstants::ActorPropertyOverridesLabel, Context);
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.Template = TemplateActor;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnParams.OverrideLevel = TargetActor ? TargetActor->GetLevel() : Cast<ULevel>(TargetWorldObject);

	FTransform Transform = TargetActor ? TargetActor->GetTransform() : FTransform::Identity;
	if(Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGCreateTargetActor, ActorPivot)))
	{
		Transform = Settings->ActorPivot;
	}

	UPCGActorHelpers::FSpawnDefaultActorParams SpawnDefaultActorParams(TargetWorldObject->GetWorld(), TemplateActorClass, Transform, SpawnParams);
	SpawnDefaultActorParams.bIsRuntime = (ExecutionSource && ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem());
#if WITH_EDITOR
	SpawnDefaultActorParams.bIsPreviewActor = (ExecutionSource && ExecutionSource->GetExecutionState().IsInPreviewMode());

	int32 DataLayerCrc = 0;
	if (TargetActor)
	{
		SpawnDefaultActorParams.DataLayerInstances = PCGDataLayerHelpers::GetDataLayerInstancesAndCrc(Context, Settings->DataLayerSettings, TargetActor, DataLayerCrc);
	}

	int32 HLODLayerCrc = 0;
	if (TargetActor)
	{
		AActor* TemplateActorOrDefault = TemplateActor ? TemplateActor : Cast<AActor>(TemplateActorClass->GetDefaultObject());
		SpawnDefaultActorParams.HLODLayer = PCGHLODHelpers::GetHLODLayerAndCrc(Context, Settings->HLODSettings, TargetActor, TemplateActorOrDefault, HLODLayerCrc);
	}
#endif

	FPCGManagedResourceContainerHelper ManagedResourcesContainerHelper(ExecutionSource);
	UPCGManagedActors* ReusableResource = nullptr;
	FPCGCrc ResourceCrc;

	if (PCGCreateTargetActor::CVarCreateTargetActorAllowReuse.GetValueOnAnyThread())
	{
		if (!Context->DependenciesCrc.IsValid())
		{
			// TODO: we should be able to use the InputData to compute Crcs but this Crc contains a non stable UID
			// since CreateTargetActor settings are already overriden by the inputdata, it doesn't matter here and we can just ignore it
			// but for future reference if we want to have better reusing and stable generation across, we need to fix this.
			FPCGDataCollection EmptyCollection;
			GetDependenciesCrc(FPCGGetDependenciesCrcParams(&EmptyCollection, Settings, Context->ExecutionSource.Get()), Context->DependenciesCrc);

#if WITH_EDITOR
			if (DataLayerCrc != 0)
			{
				Context->DependenciesCrc.Combine(DataLayerCrc);
			}

			if (HLODLayerCrc != 0)
			{
				Context->DependenciesCrc.Combine(HLODLayerCrc);
			}
#endif
		}

		if (Context->DependenciesCrc.IsValid())
		{
			ResourceCrc = Context->DependenciesCrc;

			if (const FPCGStack* Stack = Context->GetStack())
			{
				const FPCGCrc StackCRC = Stack->GetCrc();
				ResourceCrc.Combine(StackCRC);
			}

			const FPCGCrc AdditionalCRC = PCGCreateTargetActor::GetAdditionalDependenciesCrc(SpawnDefaultActorParams, TargetActor);
			ResourceCrc.Combine(AdditionalCRC);

			if (ManagedResourcesContainerHelper.IsValid())
			{
				ManagedResourcesContainerHelper.ForEachManagedResource([&ReusableResource, ResourceCrc, &Context, bIsPreview = SpawnDefaultActorParams.bIsPreviewActor](UPCGManagedResource* InResource)
				{
					if (ReusableResource)
					{
						return;
					}

#if WITH_EDITOR
					if (InResource->IsPreview() != bIsPreview)
					{
						return;
					}
#endif

					if (UPCGManagedActors* Resource = Cast<UPCGManagedActors>(InResource))
					{
						if (Resource->GetCrc().IsValid() && Resource->GetCrc() == ResourceCrc && Resource->GetConstGeneratedActors().Num() == 1 && Resource->GetConstGeneratedActors()[0].IsValid())
						{
							ReusableResource = Resource;
						}
					}
				});
			}
		}
	}
		
	AActor* GeneratedActor = nullptr;
	if (ReusableResource)
	{
		ReusableResource->MarkAsReused();
		GeneratedActor = ReusableResource->GetConstGeneratedActors()[0].Get();
		ensure(GeneratedActor->Tags.Contains(PCGHelpers::DefaultPCGActorTag));
	}
	else
	{
#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("SpawnTargetActor", "Spawning target actor from PCG"), ExecutionSource && ExecutionSource->GetExecutionState().UseTransactions());
#endif

		GeneratedActor = UPCGActorHelpers::SpawnDefaultActor(SpawnDefaultActorParams);
		if (!GeneratedActor)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("ActorSpawnFailed", "Failed to spawn actor"));
			return true;
		}

#if WITH_EDITOR
		if (!Settings->ActorLabel.IsEmpty())
		{
			GeneratedActor->SetActorLabel(Settings->ActorLabel);
		}
#endif

		// Always attach if root actor is provided
		PCGHelpers::AttachToParent(GeneratedActor, TargetActor, Settings->RootActor.Get() ? EPCGAttachOptions::Attached : Settings->AttachOptions, Context);

		// Finally add tags
		GeneratedActor->Tags.Add(PCGHelpers::DefaultPCGActorTag);

		if (!Settings->CommaSeparatedActorTags.IsEmpty())
		{
			if (PCGCreateTargetActor::CVarEnableWarningOnDeprecatedCSVActorTags.GetValueOnAnyThread())
			{
				PCGE_LOG(Warning, GraphAndLog, LOCTEXT("Deprecated Property", "\"CommaSeparatedActorTags\" is now deprecated, please use \"TagsToAddOnActor\"."));
			}
			const TArray<FString> AdditionalTags = PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->CommaSeparatedActorTags);
			for (const FString& Tag : AdditionalTags)
			{
				GeneratedActor->Tags.AddUnique(FName(Tag));
			}
		}

		for (const FName& Tag : Settings->TagsToAddOnActor)
		{
			GeneratedActor->Tags.AddUnique(Tag);
		}
	}
	
	for (UFunction* Function : PCGHelpers::FindUserFunctions(GeneratedActor->GetClass(), Settings->PostProcessFunctionNames, { UPCGFunctionPrototypes::GetPrototypeWithNoParams() }, Context))
	{
		GeneratedActor->ProcessEvent(Function, nullptr);
	}

	// Create Resource if it isn't reused
	if (!ReusableResource)
	{
		if(UObject* ExecutionSourceObject = Cast<UObject>(ExecutionSource); ExecutionSourceObject && ManagedResourcesContainerHelper.IsValid())
		{
			UPCGManagedActors* ManagedActors = NewObject<UPCGManagedActors>(ExecutionSourceObject);
			if (ResourceCrc.IsValid())
			{
				ManagedActors->SetCrc(ResourceCrc);
			}
#if WITH_EDITOR
			ManagedActors->SetIsPreview(SpawnDefaultActorParams.bIsPreviewActor);
#endif
			ManagedActors->GetMutableGeneratedActors().AddUnique(GeneratedActor);
			ManagedActors->bSupportsReset = !Settings->bDeleteActorsBeforeGeneration;

			ManagedResourcesContainerHelper.AddManagedResource(ManagedActors);
		}
	}

	// Create param data output with reference to actor
	FSoftObjectPath GeneratedActorPath(GeneratedActor);

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	check(ParamData && ParamData->Metadata);
	FPCGMetadataAttribute<FSoftObjectPath>* ActorPathAttribute = ParamData->Metadata->CreateAttribute<FSoftObjectPath>(PCGPointDataConstants::ActorReferenceAttribute, GeneratedActorPath, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
	check(ActorPathAttribute);
	ParamData->Metadata->AddEntry();

	// Add param data to output and we're done
	Context->OutputData.TaggedData.Emplace_GetRef().Data = ParamData;
	return true;
}

#undef LOCTEXT_NAMESPACE
