// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCullPointsOutsideActorBounds.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGPin.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/PCGKernelHelpers.h"
#include "Compute/Elements/PCGCullPointsOutsideActorBoundsKernel.h"
#include "Data/PCGBasePointData.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCullPointsOutsideActorBounds)

#define LOCTEXT_NAMESPACE "PCGCullPointsOutsideActorBoundsElement"

#if WITH_EDITOR
FText UPCGCullPointsOutsideActorBoundsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Cull Points Outside Actor Bounds");
}

FText UPCGCullPointsOutsideActorBoundsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Culls points that lie outside the current actor bounds.");
}

void UPCGCullPointsOutsideActorBoundsSettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);
	PCGKernelHelpers::CreateKernel<UPCGCullPointsOutsideActorBoundsKernel>(InOutContext, CreateParams, OutKernels, OutEdges);

	// @todo_pcg: Inject a compaction kernel to automatically compact after culling. Perhaps with a threshold to only compact if 10%+ of points were culled?
}
#endif

TArray<FPCGPinProperties> UPCGCullPointsOutsideActorBoundsSettings::InputPinProperties() const
{
	return Super::DefaultPointInputPinProperties();
}

TArray<FPCGPinProperties> UPCGCullPointsOutsideActorBoundsSettings::OutputPinProperties() const
{
	return Super::DefaultPointOutputPinProperties();
}

FPCGElementPtr UPCGCullPointsOutsideActorBoundsSettings::CreateElement() const
{
	return MakeShared<FPCGCullPointsOutsideActorBoundsElement>();
}

bool FPCGCullPointsOutsideActorBoundsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCullPointsOutsideActorBoundsElement::Execute);

	if (!Context->ExecutionSource.IsValid())
	{
		return true;
	}

	const UPCGCullPointsOutsideActorBoundsSettings* Settings = Context->GetInputSettings<UPCGCullPointsOutsideActorBoundsSettings>();
	check(Settings);

	// Initialize directly. Could also have gone through PCGComponent::CreateActorPCGDataCollection.
	const FBox BoundsBox = Context->ExecutionSource->GetExecutionState().GetBounds().ExpandBy(Settings->BoundsExpansion);

	const TArray<FPCGTaggedData>& Inputs = Context->InputData.TaggedData;
	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(Input.Data);

		// Skip non-points or empty point data, or cases where the point data bounds do not intersect at all with the bounds.
		if(!InputPointData || InputPointData->GetNumPoints() == 0 || !InputPointData->GetBounds().Intersect(BoundsBox))
		{
			continue;
		}

		const UPCGBasePointData* OutputData = nullptr;

		TArray<int32, TInlineAllocator<4096>> KeptPointIndices;
		bool bNeedsSort = false;
		bool bTrivialOperation = false;

		// First, test if the point bounds are fully inside the bounds box, if so it's a trivial operation
		if (BoundsBox.IsInsideOrOn(InputPointData->GetBounds()))
		{
			bTrivialOperation = true;
		}
		// If an octree is available, we'll perform a query on it instead of going through all points, as in most cases it will be significantly less work.
		// Otherwise, fall-back to testing every point (on multiple threads), which will always yield sorted results.
		else if (InputPointData->IsPointOctreeDirty())
		{
			// Implementation note: while this is a slower code path in general, building the octree normally dwarfs filtering (on MT, especially)
			// @todo_pcg: review this if we improve on the octree creation
			auto InitializeFunc = [InputPointData, &KeptPointIndices]()
			{
				KeptPointIndices.SetNumUninitialized(InputPointData->GetNumPoints());
			};

			auto AsyncProcessRangeFunc = [InputPointData, &BoundsBox, &KeptPointIndices](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
			{
				const TConstPCGValueRange<FTransform> ReadTransformRange = InputPointData->GetConstTransformValueRange();
				const TConstPCGValueRange<FVector> ReadBoundsMinRange = InputPointData->GetConstBoundsMinValueRange();
				const TConstPCGValueRange<FVector> ReadBoundsMaxRange = InputPointData->GetConstBoundsMaxValueRange();

				int32 NumWritten = 0;

				for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
				{
					const FVector PointCenter = ReadTransformRange[ReadIndex].TransformPosition(PCGPointHelpers::GetLocalCenter(ReadBoundsMinRange[ReadIndex], ReadBoundsMaxRange[ReadIndex]));
					if (PCGHelpers::IsInsideBounds(BoundsBox, PointCenter))
					{
						const int32 WriteIndex = StartWriteIndex + NumWritten;
						KeptPointIndices[WriteIndex] = ReadIndex;
						++NumWritten;
					}
				}

				return NumWritten;
			};

			auto MoveDataRangeFunc = [&KeptPointIndices](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
			{
				std::memmove(&KeptPointIndices[MoveToIndex], &KeptPointIndices[RangeStartIndex], sizeof(int32) * NumElements);
			};

			auto FinishedFunc = [&KeptPointIndices](int32 NumWritten)
			{
				KeptPointIndices.SetNum(NumWritten, EAllowShrinking::No);
			};

			FPCGAsync::AsyncProcessingRangeEx(
				Context ? &Context->AsyncState : nullptr,
				InputPointData->GetNumPoints(),
				InitializeFunc,
				AsyncProcessRangeFunc,
				MoveDataRangeFunc,
				FinishedFunc,
				/*bEnableTimeSlicing=*/false
			);
		}
		else
		{
			// Octree-based fast path; this assumes that not all points are in the bounds, otherwise it won't be faster than the other solution.
			// @todo_pcg: Split up this query by subdividing into smaller boxes and perform the query on multiple threads
			const PCGPointOctree::FPointOctree& InputOctree = InputPointData->GetPointOctree();
			FBoxCenterAndExtent BoxCenterAndExtents(BoundsBox);
		
			InputOctree.FindElementsWithBoundsTest(BoxCenterAndExtents, [&KeptPointIndices, &BoundsBox](const PCGPointOctree::FPointRef& PointRef)
			{
				// Do a final validation that the center is inside the box
				if(PCGHelpers::IsInsideBounds(BoundsBox, PointRef.Bounds.Origin))
				{
					KeptPointIndices.Add(PointRef.Index);
				}
			});

			bNeedsSort = Settings->Mode == EPCGCullPointsMode::Ordered;
		}

		if (bTrivialOperation || KeptPointIndices.Num() == InputPointData->GetNumPoints())
		{
			// Trivial operation, we've kept all the points
			OutputData = InputPointData;
		}
		else if (!KeptPointIndices.IsEmpty())
		{
			UPCGBasePointData* CulledPointsData = FPCGContext::NewPointData_AnyThread(Context);

			// Since we'll be selecting only a subset of points, we can't inherit the data directly
			FPCGInitializeFromDataParams InitializeParams(InputPointData);
			InitializeParams.bInheritSpatialData = false;
			CulledPointsData->InitializeFromDataWithParams(InitializeParams);

			if (bNeedsSort)
			{
				KeptPointIndices.Sort();
			}

			CulledPointsData->SetPointsFrom(InputPointData, KeptPointIndices);
			OutputData = CulledPointsData;
		}
		
		if (OutputData)
		{
			FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef(Input);
			Output.Data = OutputData;
		}
	}

	return true;
}

void FPCGCullPointsOutsideActorBoundsElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	// The culling volume depends on the component transform.
	const UPCGData* SelfData = InParams.ExecutionSource ? InParams.ExecutionSource->GetExecutionState().GetSelfData() : nullptr;
	if (SelfData)
	{
		Crc.Combine(SelfData->GetOrComputeCrc(/*bFullDataCrc=*/false));
	}

	OutCrc = Crc;
}

#undef LOCTEXT_NAMESPACE
