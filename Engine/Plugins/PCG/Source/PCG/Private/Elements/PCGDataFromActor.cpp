// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDataFromActor.h"

#include "PCGComponent.h"
#include "PCGCustomVersion.h"
#include "PCGIsolatedActor.h"
#include "PCGParamData.h"
#include "PCGSubsystem.h"
#include "PCGTrackingManager.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGVolumeData.h"
#include "Elements/PCGAddComponent.h"
#include "Elements/PCGMergeElement.h"
#include "Elements/PCGMergeAttributes.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGTagHelpers.h"
#include "Utils/PCGGraphExecutionLogging.h"

#include "Algo/AnyOf.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataFromActor)

#define LOCTEXT_NAMESPACE "PCGDataFromActorElement"

namespace PCGDataFromActorConstants
{
	static const FName SinglePointPinLabel = TEXT("Single Point");
	static const FName BoundingShapeLabel = TEXT("BoundingShape");
	static const FString PCGComponentDataGridSizeTagPrefix = TEXT("PCG_GridSize_");
	static const FText TagNamesSanitizedWarning = LOCTEXT("TagAttributeNamesSanitized", "One or more tag names contained invalid characters and were sanitized when creating the corresponding attributes.");
	static const FText TagNamesReservedWarning = LOCTEXT("TagAttributeNamesReserved", "One or more tag names match to reserved tags (ActorReference, ComponentReference) and were ignored when parsing the data.");
}

namespace PCGDataFromActorHelpers
{
	/**
	 * Get the PCG Components associated with an actor. Optionally, also search for any local components associated with components
	 * on the actor using the 'bGetLocalComponents' flag. By default, gets data on all grids, but alternatively you can provide a
	 * set of 'AllowedGrids' to match against.
	 *
	 * If 'bMustOverlap' is true, it will only collect components which overlap with the given 'OverlappingBounds'. Note that this
	 * overlap does not include bounds which are only touching, with no overlapping volume.
	 */
	static TInlineComponentArray<UPCGComponent*, 1> GetPCGComponentsFromActor(
		AActor* Actor,
		UPCGSubsystem* Subsystem,
		bool bGetLocalComponents = false,
		bool bGetAllGrids = true,
		int32 AllowedGrids = (int32)EPCGHiGenGrid::Uninitialized,
		bool bMustOverlap = false,
		const FBox& OverlappingBounds = FBox())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataFromActorElement::GetPCGComponentsFromActor);

		TInlineComponentArray<UPCGComponent*, 1> PCGComponents;

		if (!Actor || !Subsystem)
		{
			return PCGComponents;
		}

		Actor->GetComponents(PCGComponents);

		if (bMustOverlap)
		{
			// Remove actor components that do not overlap the source bounds.
			// Note: This assumes that a local component always lies inside the bounds of its original component,
			// which is true at the time of writing, but may not always be the case (e.g. "truly" unbounded execution).
			for (int I = PCGComponents.Num() - 1; I >= 0; I--)
			{
				const FBox ComponentBounds = PCGComponents[I]->GetGridBounds();

				// We reject overlaps with zero volume instead of simply checking Intersect(...) to avoid bounds which touch but do not overlap.
				if (OverlappingBounds.Overlap(ComponentBounds).GetVolume() <= 0)
				{
					PCGComponents.RemoveAtSwap(I);
				}
			}
		}

		TArray<UPCGComponent*> LocalComponents;

		if (bGetLocalComponents)
		{
			auto AddComponent = [&LocalComponents, bGetAllGrids, AllowedGrids](UPCGComponent* LocalComponent)
			{
				const EPCGHiGenGrid UnscaledGrid = LocalComponent->GetUnscaledGenerationGrid();
				if (bGetAllGrids || (AllowedGrids & (int32)UnscaledGrid))
				{
					LocalComponents.Add(LocalComponent);
				}
			};

			// Collect the local components for each actor PCG component.
			for (UPCGComponent* Component : PCGComponents)
			{
				if (Component && Component->IsPartitioned())
				{
					if (bMustOverlap)
					{
						Subsystem->ForAllRegisteredIntersectingLocalComponents(Component, OverlappingBounds, AddComponent);
					}
					else
					{
						Subsystem->ForAllRegisteredLocalComponents(Component, AddComponent);
					}
				}
			}
		}

		// Remove the actor's PCG components if they aren't on an allowed grid size.
		// Implementation note: We delay removing these components until now because they may have had local components on an allowed grid size.
		if (!bGetAllGrids)
		{
			for (int I = PCGComponents.Num() - 1; I >= 0; I--)
			{
				if (!(AllowedGrids & (int32)PCGComponents[I]->GetGenerationGridSize()))
				{
					PCGComponents.RemoveAtSwap(I);
				}
			}
		}

		if (bGetLocalComponents)
		{
			PCGComponents.Append(LocalComponents);
		}

		return PCGComponents;
	}
}

#if WITH_EDITOR
void UPCGDataFromActorSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	// If the input pin is connected, tracking is dynamic.
	const UPCGNode* Node = Cast<const UPCGNode>(GetOuter());
	if (Node && Node->IsInputPinConnected(PCGPinConstants::DefaultInputLabel))
	{
		return;
	}

	// When bounds come from the pin we can't statically restrict tracking to actor bounds, so we'll consider it fully dynamic
	const bool bBoundingShapePinConnected = Node && Node->IsInputPinConnected(PCGDataFromActorConstants::BoundingShapeLabel);
	if (bTrackActorsOnlyWithinBounds && bBoundingShapePinConnected)
	{
		return;
	}

	FPCGSelectionKey Key = ActorSelector.GetAssociatedKey();
	if (Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents)
	{
		Key.SetExtraDependency(UPCGComponent::StaticClass());
	}

	OutKeysToSettings.FindOrAdd(Key).Emplace(this, bTrackActorsOnlyWithinBounds);
}

void UPCGDataFromActorSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	if (DataVersion < FPCGCustomVersion::GetPCGComponentDataMustOverlapSourceComponentByDefault)
	{
		// Old versions of GetActorData did not require found components to overlap self, but going forward it's a more efficient default.
		bComponentsMustOverlapSelf = false;
	}

	Super::ApplyDeprecation(InOutNode);
}

FText UPCGDataFromActorSettings::GetNodeTooltipText() const
{
	return LOCTEXT("DataFromActorTooltip", "Builds a collection of PCG-compatible data from the selected actors.");
}

void UPCGDataFromActorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGDataFromActorSettings, ActorSelector))
	{
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FPCGActorSelectorSettings, ActorSelection))
		{
			// Make sure that when switching away from the 'by class' selection, we actually break that data dependency
			if (ActorSelector.ActorSelection != EPCGActorSelection::ByClass)
			{
				ActorSelector.ActorSelectionClass = GetDefaultActorSelectorClass();
			}
		}
	}
}

EPCGChangeType UPCGDataFromActorSettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(PropertyChangedEvent) | EPCGChangeType::Cosmetic;

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGDataFromActorSettings, ActorSelector))
	{
		// If we change from/to FromInput, this needs to trigger a graph recompilation especially for culling.
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FPCGActorSelectorSettings, ActorFilter))
		{
			ChangeType |= EPCGChangeType::Structural;
		}
	}

	return ChangeType;
}
#endif // WITH_EDITOR

TSubclassOf<AActor> UPCGDataFromActorSettings::GetDefaultActorSelectorClass() const
{
	return TSubclassOf<AActor>();
}

#if WITH_EDITOR
bool UPCGDataFromActorSettings::DisplayModeSettings() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return bDisplayModeSettings;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif

void UPCGDataFromActorSettings::PostLoad()
{
	Super::PostLoad();

	if (ActorSelector.ActorSelection != EPCGActorSelection::ByClass)
	{
		ActorSelector.ActorSelectionClass = GetDefaultActorSelectorClass();
	}
}

FString UPCGDataFromActorSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	bool bTaskIsOverridden = false;
	if (ActorSelector.ActorFilter == EPCGActorFilter::AllWorldActors)
	{
		if (ActorSelector.ActorSelection == EPCGActorSelection::ByClass
			&& IsPropertyOverriddenByPin({GET_MEMBER_NAME_CHECKED(UPCGDataFromActorSettings, ActorSelector), GET_MEMBER_NAME_CHECKED(FPCGActorSelectorSettings, ActorSelectionClass)}))
		{
			bTaskIsOverridden = true;
		}
		else if (ActorSelector.ActorSelection == EPCGActorSelection::ByTag
			&& IsPropertyOverriddenByPin({GET_MEMBER_NAME_CHECKED(UPCGDataFromActorSettings, ActorSelector), GET_MEMBER_NAME_CHECKED(FPCGActorSelectorSettings, ActorSelectionTag)}))
		{
			bTaskIsOverridden = true;
		}
	}

	const FText Suffix = ActorSelector.GetTaskNameSuffix();
	const FTextFormat Format = FText::FromString(Suffix.IsEmpty() ? "{0}" : "{0}: {1}");
	return FText::Format(Format, ActorSelector.GetTaskName(), bTaskIsOverridden ? LOCTEXT("TaskOverridden", "Overridden") : Suffix).ToString();
#else
	return Super::GetAdditionalTitleInformation();
#endif
}

FPCGElementPtr UPCGDataFromActorSettings::CreateElement() const
{
	return MakeShared<FPCGDataFromActorElement>();
}

FPCGDataTypeIdentifier UPCGDataFromActorSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	check(InPin);

	if (InPin->IsOutputPin())
	{
		if (Mode == EPCGGetDataFromActorMode::GetSinglePoint)
		{
			return FPCGDataTypeIdentifier{EPCGDataType::Point};
		}
		else if (Mode == EPCGGetDataFromActorMode::GetActorReference || Mode == EPCGGetDataFromActorMode::GetComponentsReference)
		{
			return FPCGDataTypeIdentifier{EPCGDataType::Param};
		}
	}

	// Implementation note: since we can have an input pin in some instances, we can't rely on the
	// base class to provide the proper output type, as this will override the values set in the OutputPinProperties.
	return InPin->Properties.AllowedTypes;
}

TArray<FPCGPinProperties> UPCGDataFromActorSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	if (ActorSelector.ActorFilter == EPCGActorFilter::FromInput)
	{
		PinProperties.Emplace(PCGPinConstants::DefaultInputLabel,
			EPCGDataType::Any,
			/*bAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true);
	}

	FPCGPinProperties& BoundingShapePin = PinProperties.Emplace_GetRef(
		PCGDataFromActorConstants::BoundingShapeLabel,
		EPCGDataType::Spatial,
		/*bAllowMultipleConnections=*/false,
		/*bAllowMultipleData=*/false
#if WITH_EDITOR
		, LOCTEXT("BoundingShapePinTooltip", "Optional bounding shape to filter found actors by. When connected, overrides the implicit PCG actor bounds used by 'Must Overlap Self'.")
#endif
	);
	BoundingShapePin.SetAdvancedPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGDataFromActorSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Pins = Super::OutputPinProperties();

	if (Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents)
	{
		for (const FName& Pin : ExpectedPins)
		{
			Pins.Emplace(Pin);
		}

		if (bAlsoOutputSinglePointData)
		{
			Pins.Emplace(PCGDataFromActorConstants::SinglePointPinLabel,
				EPCGDataType::Point,
				/*bAllowMultipleConnections=*/true,
				/*bAllowMultiData=*/true,
				LOCTEXT("SinglePointPinTooltip", "Matching single point associated to the actors from which data has been retrieved"));
		}
	}

	return Pins;
}

bool FPCGDataFromActorElement::PrepareDataInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataFromActorElement::PrepareData);
	check(Context);

	const UPCGDataFromActorSettings* Settings = Context->GetInputSettings<UPCGDataFromActorSettings>();
	check(Settings);

	if (Settings->ActorSelector.ActorFilter == EPCGActorFilter::FromInput)
	{
		FPCGDataFromActorContext* ThisContext = static_cast<FPCGDataFromActorContext*>(Context);
		return ThisContext->InitializeAndRequestLoad(PCGPinConstants::DefaultInputLabel,
			Settings->ActorSelector.ActorReferenceSelector,
			{},
			/*bPersistAllData=*/false,
			/*bSilenceErrorOnEmptyObjectPath=*/true,
			/*bSynchronousLoad=*/false);
	}
	else
	{
		return true;
	}
}

bool FPCGDataFromActorElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataFromActorElement::Execute);

	check(InContext);
	FPCGDataFromActorContext* Context = static_cast<FPCGDataFromActorContext*>(InContext);

	// Done waiting
	if (Context->bWaitingOnProcessActors)
	{
		return true;
	}

	const UPCGDataFromActorSettings* Settings = Context->GetInputSettings<UPCGDataFromActorSettings>();
	check(Settings);

	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	if (!ExecutionSource)
	{
		return true;
	}

	AActor* Self = ExecutionSource->GetExecutionState().GetTypedTarget<AActor>();

	if (!Context->bPerformedQuery)
	{
		Context->ComponentSelector = Settings->ComponentSelector;

		TArray<FPCGTaggedData> BoundingShapeInputData = Context->InputData.GetInputsByPin(PCGDataFromActorConstants::BoundingShapeLabel);
		const UPCGSpatialData* BoundingShape = !BoundingShapeInputData.IsEmpty() ? Cast<UPCGSpatialData>(BoundingShapeInputData[0].Data) : nullptr;
		const FBox Bounds = BoundingShape ? BoundingShape->GetBounds() : (Settings->ActorSelector.bMustOverlapSelf ? ExecutionSource->GetExecutionState().GetBounds() : FBox(EForceInit::ForceInit));

		TFunction<bool(const AActor*)> BoundsCheck = [](const AActor*) -> bool { return true; };
		if (BoundingShape || Settings->ActorSelector.bMustOverlapSelf)
		{
			// Capture Bounds by value because it goes out of scope
			BoundsCheck = [Bounds, ExecutionSource](const AActor* OtherActor) -> bool
			{
				const FBox OtherActorBounds = OtherActor ? PCGHelpers::GetGridBounds(OtherActor, ExecutionSource) : FBox(EForceInit::ForceInit);
				return OtherActorBounds.IsValid && Bounds.IsValid && Bounds.Intersect(OtherActorBounds);
			};
		}

		TFunction<bool(const AActor*)> SelfIgnoreCheck = [](const AActor*) -> bool { return true; };
		if (Settings->ActorSelector.bIgnoreSelfAndChildren)
		{
			// @todo_pcg: RG source should support self ignore check
			SelfIgnoreCheck = [Self](const AActor* OtherActor) -> bool
			{
				// Check if OtherActor is a child of self
				const AActor* CurrentOtherActor = OtherActor;
				while (CurrentOtherActor)
				{
					if (CurrentOtherActor == Self)
					{
						return false;
					}

					CurrentOtherActor = CurrentOtherActor->GetParentActor();
				}

				// Check if Self is a child of OtherActor
				const AActor* CurrentSelfActor = Self;
				while (CurrentSelfActor)
				{
					if (CurrentSelfActor == OtherActor)
					{
						return false;
					}

					CurrentSelfActor = CurrentSelfActor->GetParentActor();
				}

				return true;
			};
		}

		// When gathering PCG data on any world actor, we can leverage the octree kept by the Tracking system, and get all intersecting components if we need to overlap self
		// or just gather all registered components (which is way faster than going through all actors in the world).
		if (Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent && Settings->ActorSelector.ActorFilter == EPCGActorFilter::AllWorldActors)
		{
			if (UPCGSubsystem* Subsystem = Cast<UPCGSubsystem>(ExecutionSource->GetExecutionState().GetSubsystem()))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataFromActorElement::Execute::FindPCGComponents);

				const FPCGSelectionKey Key = Settings->ActorSelector.GetAssociatedKey();

				// TODO: Perhaps move the logic into the selector.
				if (BoundingShape || Settings->ActorSelector.bMustOverlapSelf)
				{
					for (IPCGGraphExecutionSource* CurrentExecutionSource : Subsystem->GetAllIntersectingExecutionSources(Bounds))
					{
						if (UPCGComponent* Component = Cast<UPCGComponent>(CurrentExecutionSource))
						{
							if (AActor* Actor = Component->GetOwner())
							{
								if (Key.IsMatching(FPCGSelectionKey::CreateFromObjectPtr(Actor), Component))
								{
									Context->FoundActors.AddUnique(Actor);
								}
							}
						}
					}
				}
				else
				{
					for (IPCGGraphExecutionSource* CurrentExecutionSource : Subsystem->GetAllRegisteredExecutionSources())
					{
						if (UPCGComponent* Component = Cast<UPCGComponent>(CurrentExecutionSource))
						{
							if (AActor* Actor = Component->GetOwner())
							{
								if (Key.IsMatching(FPCGSelectionKey::CreateFromObjectPtr(Actor), Component))
								{
									Context->FoundActors.AddUnique(Actor);
								}
							}
						}
					}
				}

				Context->bPerformedQuery = true;
			}
		}
		
		if (!Context->bPerformedQuery)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataFromActorElement::Execute::FindActors);
			TArray<AActor*> ActorsFromInput;
			TArray<UActorComponent*> ComponentsFromInput;

			if (Settings->ActorSelector.ActorFilter == EPCGActorFilter::FromInput)
			{
				for (const TTuple<FSoftObjectPath, int32, int32>& InPath : Context->PathsToObjectsAndDataIndex)
				{
					UObject* Object = InPath.Get<0>().ResolveObject();
					if (AActor* Actor = Cast<AActor>(Object))
					{
						ActorsFromInput.AddUnique(Actor);
					}
					else if (UActorComponent* ActorComponent = Cast<UActorComponent>(Object))
					{
						ComponentsFromInput.Add(ActorComponent);
						ActorsFromInput.AddUnique(ActorComponent->GetOwner());
					}
				}
			}

			Context->ComponentSelector.ComponentList = std::move(ComponentsFromInput);

			Context->FoundActors = PCGActorSelector::FindActors(&Settings->ActorSelector, &Context->ComponentSelector, ExecutionSource, BoundsCheck, SelfIgnoreCheck, ActorsFromInput);
			Context->bPerformedQuery = true;
		}

#if WITH_EDITOR
		// Setup dynamic tracking if needed
		if (Settings->ActorSelector.ActorFilter == EPCGActorFilter::FromInput)
		{
			FPCGDynamicTrackingHelper DynamicTracking;
			DynamicTracking.EnableAndInitialize(Context, Context->PathsToObjectsAndDataIndex.Num());
			for (const TTuple<FSoftObjectPath, int32, int32>& Path : Context->PathsToObjectsAndDataIndex)
			{
				// Skip dependency tracking on actors that couldn't be loaded
				if (!Path.Get<0>().ResolveObject())
				{
					continue;
				}

				FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(Path.Get<0>());
				if (Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents)
				{
					Key.SetExtraDependency(UPCGComponent::StaticClass());
				}
				
				DynamicTracking.AddToTracking(MoveTemp(Key), /*bIsCulled=*/Settings->bTrackActorsOnlyWithinBounds);
			}

			DynamicTracking.Finalize(Context);
		}
		else if (BoundingShape || Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGDataFromActorSettings, ActorSelector)))
		{
			FPCGSelectionKey Key(Settings->ActorSelector);
			if (Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents)
			{
				Key.SetExtraDependency(UPCGComponent::StaticClass());
			}

			// If we were using a custom bounding shape for retrieval, pass it to the selection key so it's properly preserved and tested against.
			if (BoundingShape)
			{
				Key.OptionalBounds.Add(Bounds);
			}

			//@todo_pcg: if the bounding shape bounds are strictly smaller than the execution source bounds, then we technically could cull them.
			// however this wouldn't be super robust to the execution source moving either.
			const bool bIsCulled = !BoundingShape && Settings->ActorSelector.bMustOverlapSelf;
			FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, MoveTemp(Key), bIsCulled);
		}

		// Any isolated actors should also be dynamically tracked.
		if (Settings->ActorSelector.IncludeIsolatedActors != EPCGIsolatedActorIncludeFlag::Ignore)
		{
			for (AActor* FoundActor : Context->FoundActors)
			{
				if (APCGIsolatedActor* IsolatedActor = Cast<APCGIsolatedActor>(FoundActor))
				{
					FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, FPCGSelectionKey::CreateFromObjectPtr(IsolatedActor), /*bIsCulled=*/false);
				}
			}
		}
#endif // WITH_EDITOR

		if (Context->FoundActors.IsEmpty())
		{
			PCGE_LOG(Verbose, LogOnly, LOCTEXT("NoActorFound", "No matching actor was found"));
			return true;
		}

		// If we're looking for PCG component data, we might have to wait for it.
		if (Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents)
		{
			TArray<FPCGTaskId> WaitOnTaskIds;
			for (AActor* Actor : Context->FoundActors)
			{
				GatherWaitTasks(Actor, Context, WaitOnTaskIds);
			}

			if (!WaitOnTaskIds.IsEmpty())
			{
				// Add a trivial task after these generations that wakes up this task
				Context->bIsPaused = true;

				Context->ScheduleGeneric(FPCGScheduleGenericParams(
					[ContextHandle = Context->GetOrCreateHandle()](FPCGContext*) // Normal execution: Wake up the current task
					{
						FPCGContext::FSharedContext<FPCGDataFromActorContext> SharedContext(ContextHandle);
						if (FPCGDataFromActorContext* ContextPtr = SharedContext.Get())
						{
							ContextPtr->bIsPaused = false;
						}
						return true;
					}, 
					[ContextHandle = Context->GetOrCreateHandle()](FPCGContext*) // On Abort: Wake up on abort, clear all results and mark as cancelled
					{
						FPCGContext::FSharedContext<FPCGDataFromActorContext> SharedContext(ContextHandle);
						if (FPCGDataFromActorContext* ContextPtr = SharedContext.Get())
						{
							ContextPtr->bIsPaused = false;
							ContextPtr->FoundActors.Reset();
							ContextPtr->OutputData.bCancelExecution = true;
						}
					},
					Context->ExecutionSource.Get(),
					WaitOnTaskIds));

				return false;
			}
		}
	}

	if (Context->bPerformedQuery)
	{
#if WITH_EDITOR
		UPCGComponent* PCGComponent = Cast<UPCGComponent>(ExecutionSource);
		// Remove ignored change origins now that we've completed the wait tasks.
		UPCGComponent* OriginalComponent = PCGComponent ? PCGComponent->GetOriginalComponent() : nullptr;
		if (OriginalComponent)
		{
			for (TObjectKey<UObject>& IgnoredChangeOriginKey : Context->IgnoredChangeOrigins)
			{
				if (UObject* IgnoredChangeOrigin = IgnoredChangeOriginKey.ResolveObjectPtr())
				{
					OriginalComponent->StopIgnoringChangeOriginDuringGeneration(IgnoredChangeOrigin);
				}
			}
		}
#endif

		TArray<FPCGTaskId> OutDynamicDependencies;
		ProcessActors(Context, Settings, Context->FoundActors, OutDynamicDependencies);
		if (!OutDynamicDependencies.IsEmpty())
		{
			Context->bWaitingOnProcessActors = true;
			Context->bIsPaused = true;
			Context->DynamicDependencies.Append(OutDynamicDependencies);
			return false;
		}
	}

	return true;
}

bool ShouldIgnorePCGComponent(FPCGDataFromActorContext* Context, const UPCGDataFromActorSettings* Settings, UPCGComponent* Component, const IPCGGraphExecutionSource* DownstreamOriginalSource)
{
	check(Context && Settings);
	if (!Context->ComponentSelector.FilterComponent(Component))
	{
		return true;
	}

	const UPCGComponent* OriginalComponent = Component ? Component->GetOriginalComponent() : nullptr;
	const UObject* DownstreamOriginalOwner = DownstreamOriginalSource ? DownstreamOriginalSource->GetExecutionState().GetTarget() : nullptr;

	return !OriginalComponent
		|| OriginalComponent == DownstreamOriginalSource
		|| (Settings->ActorSelector.bIgnoreSelfAndChildren && OriginalComponent->GetExecutionState().GetTarget() == DownstreamOriginalOwner);
}

void FPCGDataFromActorElement::GatherWaitTasks(AActor* FoundActor, FPCGContext* InContext, TArray<FPCGTaskId>& OutWaitTasks) const
{
	if (!FoundActor)
	{
		return;
	}

	FPCGDataFromActorContext* Context = static_cast<FPCGDataFromActorContext*>(InContext);
	check(Context);

	const UPCGDataFromActorSettings* Settings = Context->GetInputSettings<UPCGDataFromActorSettings>();
	check(Settings);

	const IPCGGraphExecutionSource* Source = Context->ExecutionSource.Get();
	IPCGGraphExecutionSource* OriginalSource = Source ? Source->GetExecutionState().GetOriginalSource() : nullptr;

	if (!OriginalSource)
	{
		return;
	}

	TInlineComponentArray<UPCGComponent*, 1> UpstreamComponents = PCGDataFromActorHelpers::GetPCGComponentsFromActor(
		FoundActor,
		Cast<UPCGSubsystem>(Source->GetExecutionState().GetSubsystem()),
		/*bGetLocalComponents=*/true,
		Settings->bGetDataOnAllGrids,
		Settings->AllowedGrids,
		Settings->bComponentsMustOverlapSelf,
		Settings->bComponentsMustOverlapSelf ? Source->GetExecutionState().GetBounds() : FBox());

	for (UPCGComponent* UpstreamComponent : UpstreamComponents)
	{
		// Avoid waiting on our own execution (including local components).
		if (ShouldIgnorePCGComponent(Context, Settings, UpstreamComponent, OriginalSource))
		{
			continue;
		}

		if (UpstreamComponent->IsGenerating())
		{
			OutWaitTasks.Add(UpstreamComponent->GetGenerationTaskId());
		}
		else if (!UpstreamComponent->bGenerated && UpstreamComponent->bActivated && (UpstreamComponent->GetSerializedEditingMode() == EPCGEditorDirtyMode::Preview) && UpstreamComponent->GetOwner())
		{
#if WITH_EDITOR
			// Signal that any change notifications from generating upstream component should not trigger re-executions of this component.
			// Such change notifications can cancel the current execution.
			// Note: Uses owner because FPCGTrackingManager::OnPCGGraphGeneratedOrCleaned reports change on owner.
			if (UPCGComponent* OriginalComponent = Cast<UPCGComponent>(OriginalSource))
			{
				OriginalComponent->StartIgnoringChangeOriginDuringGeneration(UpstreamComponent->GetOwner());
			}
			Context->IgnoredChangeOrigins.Add(UpstreamComponent->GetOwner());
#endif

			const FPCGTaskId GenerateTask = UpstreamComponent->GenerateLocalGetTaskId(EPCGComponentGenerationTrigger::GenerateOnDemand, /*bForce=*/false);
			if (GenerateTask != InvalidPCGTaskId)
			{
				PCGGraphExecutionLogging::LogGraphScheduleDependency(UpstreamComponent, Context->GetStack());

				OutWaitTasks.Add(GenerateTask);
			}
			else
			{
				PCGGraphExecutionLogging::LogGraphScheduleDependencyFailed(UpstreamComponent, Context->GetStack());
			}
		}
	}
}

void FPCGDataFromActorElement::ProcessActors(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, const TArray<AActor*>& FoundActors, TArray<FPCGTaskId>& OutDynamicDependencies) const
{
	ProcessActors(Context, Settings, FoundActors);
}

void FPCGDataFromActorElement::ProcessActors(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, const TArray<AActor*>& FoundActors) const
{
	// Special case:
	// If we're asking for single point with the merge single point data, we can do a more efficient process
	if (Settings->Mode == EPCGGetDataFromActorMode::GetSinglePoint && Settings->bMergeSinglePointData && FoundActors.Num() > 1)
	{
		MergeActorsIntoData(Context, Settings, FoundActors);
	}
	else if((Settings->Mode == EPCGGetDataFromActorMode::GetActorReference || Settings->Mode == EPCGGetDataFromActorMode::GetComponentsReference))
	{
		if(Settings->bMergeSinglePointData)
		{
			CreateReferenceData(Context, Settings, FoundActors);
		}
		else
		{
			for(AActor* Actor : FoundActors)
			{
				CreateReferenceData(Context, Settings, {Actor});
			}
		}
	}
	else
	{
		for (AActor* Actor : FoundActors)
		{
			ProcessActor(Context, Settings, Actor);
		}
	}
}

void FPCGDataFromActorElement::CreateReferenceData(FPCGContext* InContext, const UPCGDataFromActorSettings* Settings, const TArray<AActor*>& Actors) const
{
	check(InContext);
	FPCGDataFromActorContext* Context = static_cast<FPCGDataFromActorContext*>(InContext);
	check(Settings && (Settings->Mode == EPCGGetDataFromActorMode::GetActorReference || Settings->Mode == EPCGGetDataFromActorMode::GetComponentsReference));

	const bool bCreateComponentReferences = (Settings->Mode == EPCGGetDataFromActorMode::GetComponentsReference);

	UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	UPCGMetadata* Metadata = ParamData->MutableMetadata();

	TSet<FString> AllGatheredTags;

	TSet<FName> ReservedTags;
	ReservedTags.Add(PCGPointDataConstants::ActorReferenceAttribute);
	ReservedTags.Add(PCGAddComponentConstants::ComponentReferenceAttribute);

	FPCGMetadataAttribute<FSoftObjectPath>* ActorReference = Metadata->FindOrCreateAttribute(PCGPointDataConstants::ActorReferenceAttribute, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true);
	FPCGMetadataAttribute<FSoftObjectPath>* ComponentReference = bCreateComponentReferences ? Metadata->FindOrCreateAttribute(PCGAddComponentConstants::ComponentReferenceAttribute, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true) : nullptr;

	bool bHasReservedTag = false;
	bool bHasSanitizedTag = false;

	auto SetAttributesFromTags = [Metadata, &ReservedTags, &bHasReservedTag, &bHasSanitizedTag](const auto& Tags, PCGMetadataEntryKey Entry)
	{
		for(FName Tag : Tags)
		{
			PCG::Private::FParseTagResult TagData(Tag);
			if (!ReservedTags.Contains(FName(TagData.Attribute)))
			{
				if (PCG::Private::SetAttributeFromTag(TagData, Metadata, Entry, PCG::Private::ESetAttributeFromTagFlags::CreateAttribute))
				{
					bHasSanitizedTag |= TagData.HasBeenSanitized();
				}
			}
			else
			{
				bHasReservedTag = true;
			}
		}
	};

	auto AddTagsToGatheredTags = [&AllGatheredTags](const auto& Tags)
	{
		for (FName Tag : Tags)
		{
			AllGatheredTags.Add(Tag.ToString());
		}
	};

	for (AActor* Actor : Actors)
	{
		if (!Actor)
		{
			continue;
		}

		FSoftObjectPath ActorPath(Actor);

		if (ComponentReference)
		{
			TInlineComponentArray<UActorComponent*, 16> Components;
			Actor->GetComponents(Components);

			bool bHasAddedActorTags = false;

			for (UActorComponent* Component : Components)
			{
				if(!Context->ComponentSelector.FilterComponent(Component))
				{
					continue;
				}

				if (!bHasAddedActorTags)
				{
					bHasAddedActorTags = true;
					AddTagsToGatheredTags(Actor->Tags);
				}

				PCGMetadataEntryKey Entry = ParamData->MutableMetadata()->AddEntry();
				ActorReference->SetValue(Entry, ActorPath);
				ComponentReference->SetValue(Entry, FSoftObjectPath(Component));

				SetAttributesFromTags(Actor->Tags, Entry);
				SetAttributesFromTags(Component->ComponentTags, Entry);
				AddTagsToGatheredTags(Component->ComponentTags);
			}
		}
		else
		{
			PCGMetadataEntryKey Entry = ParamData->MutableMetadata()->AddEntry();
			ActorReference->SetValue(Entry, ActorPath);

			SetAttributesFromTags(Actor->Tags, Entry);
			AddTagsToGatheredTags(Actor->Tags);
		}
	}

	// Mark the actor or component reference as the last attribute in order to behave like before wrt to this.
	FPCGAttributePropertySelector LastSelector;
	LastSelector.SetAttributeName(bCreateComponentReferences ? PCGAddComponentConstants::ComponentReferenceAttribute : PCGPointDataConstants::ActorReferenceAttribute);
	ParamData->SetLastSelector(LastSelector);

	if (ParamData->MutableMetadata()->GetItemCountForChild() > 0)
	{
		FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = ParamData;
		TaggedData.Tags = MoveTemp(AllGatheredTags);
	}

	// Finally, log warnings when/if required
	if (bHasReservedTag && !Settings->bSilenceReservedAttributeNameWarnings)
	{
		PCGE_LOG(Warning, GraphAndLog, PCGDataFromActorConstants::TagNamesReservedWarning);
	}

	if (bHasSanitizedTag && !Settings->bSilenceSanitizedAttributeNameWarnings)
	{
		PCGE_LOG(Warning, GraphAndLog, PCGDataFromActorConstants::TagNamesSanitizedWarning);
	}
}

void FPCGDataFromActorElement::MergeActorsIntoData(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, const TArray<AActor*>& FoundActors) const
{
	check(Context);
	check(Settings && Settings->Mode == EPCGGetDataFromActorMode::GetSinglePoint);

	// At this point in time, the partition actors behave slightly differently, so if we are in the case where
	// we have one or more partition actors, we'll go through the normal process and do post-processing to merge the point data instead.
	UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(Context);

	bool bHasData = false;
	bool bAnyAttributeNameWasSanitized = false;

	for (AActor* Actor : FoundActors)
	{
		if (Actor)
		{
			bool bAttributeNameWasSanitized = false;
			PointData->AddSinglePointFromActor(Actor, &bAttributeNameWasSanitized);
			bAnyAttributeNameWasSanitized |= bAttributeNameWasSanitized;

			bHasData = true;
		}
	}

	if (bAnyAttributeNameWasSanitized && !Settings->bSilenceSanitizedAttributeNameWarnings)
	{
		PCGE_LOG(Warning, GraphAndLog, PCGDataFromActorConstants::TagNamesSanitizedWarning);
	}

	if (bHasData)
	{
		FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = PointData;
	}
}

void FPCGDataFromActorElement::ProcessActor(FPCGContext* InContext, const UPCGDataFromActorSettings* Settings, AActor* FoundActor) const
{
	check(InContext);
	FPCGDataFromActorContext* Context = static_cast<FPCGDataFromActorContext*>(InContext);
	check(Settings);

	const IPCGGraphExecutionSource* Source = Context->ExecutionSource.Get();
	const IPCGGraphExecutionSource* OriginalSource = Source ? Source->GetExecutionState().GetOriginalSource() : nullptr;

	if (!FoundActor || !IsValid(FoundActor) || !OriginalSource)
	{
		return;
	}

	const UObject* SourceOwner = OriginalSource->GetExecutionState().GetTarget();
	TInlineComponentArray<UPCGComponent*, 1> UpstreamComponents;
	bool bHasGeneratedPCGData = false;
	FProperty* FoundProperty = nullptr;

	if (Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents)
	{
		UpstreamComponents = PCGDataFromActorHelpers::GetPCGComponentsFromActor(
			FoundActor,
			Cast<UPCGSubsystem>(Source->GetExecutionState().GetSubsystem()),
			/*bGetLocalComponents=*/true,
			Settings->bGetDataOnAllGrids,
			Settings->AllowedGrids,
			Settings->bComponentsMustOverlapSelf,
			Settings->bComponentsMustOverlapSelf ? Source->GetExecutionState().GetBounds() : FBox());

		// Remove any PCG components that are filtered from the component selector &
		// Remove any PCG components that don't belong to an external execution context (i.e. share the same original component), or that share a common root actor.
		UpstreamComponents.RemoveAllSwap([Context, Settings, OriginalSource](UPCGComponent* UpstreamComponent)
		{
			return ShouldIgnorePCGComponent(Context, Settings, UpstreamComponent, OriginalSource);
		});

		for (UPCGComponent* UpstreamComponent : UpstreamComponents)
		{
			bHasGeneratedPCGData |= !UpstreamComponent->GetGeneratedGraphOutput().TaggedData.IsEmpty();
		}
		if (APCGIsolatedActor* IsolatedActor = Cast<APCGIsolatedActor>(FoundActor))
		{
			bHasGeneratedPCGData |= !IsolatedActor->Data.GetAllInputs().IsEmpty();
		}
	}
	else if (Settings->Mode == EPCGGetDataFromActorMode::GetDataFromProperty)
	{
		if (Settings->PropertyName != NAME_None)
		{
			FoundProperty = FindFProperty<FProperty>(FoundActor->GetClass(), Settings->PropertyName);
		}
	}

	// Some additional validation
	if (Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent && !bHasGeneratedPCGData)
	{
		if (!UpstreamComponents.IsEmpty())
		{
			PCGE_LOG(Log, GraphAndLog, FText::Format(LOCTEXT("ActorHasNoGeneratedData", "Actor '{0}' does not have any previously generated data, or all its components were filtered out."), FText::FromName(FoundActor->GetFName())));
		}

		return;
	}
	else if (Settings->Mode == EPCGGetDataFromActorMode::GetDataFromProperty && !FoundProperty)
	{
		PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("ActorHasNoProperty", "Actor '{0}' does not have a property name '{1}'"), FText::FromName(FoundActor->GetFName()), FText::FromName(Settings->PropertyName)));
		return;
	}

	auto GetActorDataCollection = [Source, Context, Settings](AActor* Actor, EPCGDataType DataType, bool bParseActor) -> FPCGDataCollection
	{
		FPCGGetDataFunctionRegistryParams GDFRParams;
		GDFRParams.Source = Source;
		GDFRParams.ComponentSelector = &Context->ComponentSelector;
		GDFRParams.bParseActor = bParseActor;
		GDFRParams.DataTypeFilter = DataType;
		GDFRParams.bIgnorePCGGeneratedComponents = Settings->bIgnorePCGGeneratedComponents;

		FPCGGetDataFunctionRegistryOutput GDFROutput;
		FPCGModule::ConstGetDataFunctionRegistry().GetDataFromActor(Context, GDFRParams, Actor, GDFROutput);

		if (GDFROutput.bSanitizedTagAttributeNames && !Settings->bSilenceSanitizedAttributeNameWarnings)
		{
			PCGE_LOG_C(Warning, GraphAndLog, Context, PCGDataFromActorConstants::TagNamesSanitizedWarning);
		}

		return GDFROutput.Collection;
	};

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	if (bHasGeneratedPCGData)
	{
		for (UPCGComponent* Component : UpstreamComponents)
		{
			for (const FPCGTaggedData& TaggedData : Component->GetGeneratedGraphOutput().TaggedData)
			{
				if (ensure(TaggedData.Data))
				{
					FPCGTaggedData& DuplicatedTaggedData = Outputs.Add_GetRef(TaggedData);

					// In cases where the owner of the data (component -> owner) isn't in the main world, we MUST duplicate the data, otherwise we'll cause reference leaks in the cache.
#if WITH_EDITOR
					if (!SourceOwner || (Component->GetOwner() && Component->GetOwner()->GetLevel() != SourceOwner->GetWorld()->PersistentLevel))
					{
						DuplicatedTaggedData.Data = Cast<UPCGData>(StaticDuplicateObject(TaggedData.Data, GetTransientPackage()));
					}
#endif

					// Use stable/canonical grid size for tags.
					const EPCGHiGenGrid UnscaledGrid = Component->GetUnscaledGenerationGrid();
					DuplicatedTaggedData.Tags.Add(PCGDataFromActorConstants::PCGComponentDataGridSizeTagPrefix + FString::FromInt(PCGHiGenGrid::GridToGridSize(UnscaledGrid)));

					// Add some logging if we have un-expected output pin
					if (DuplicatedTaggedData.Pin != PCGPinConstants::DefaultOutputLabel && !Settings->ExpectedPins.IsEmpty() && !Settings->ExpectedPins.Contains(TaggedData.Pin))
					{
						PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidComponentOutputPinPCGComponentData", "Component '{0}' on actor '{1}' has data from an unexpected output pin: '{2}'.\nMaybe the data is outdated, try regenerate this component to refresh its output data"),
							FText::FromString(Component->GetName()),
							FText::FromString(PCGLog::GetExecutionSourceName(Component, /*bUseLabel=*/true)),
							FText::FromName(TaggedData.Pin)), Context);
					}
				}
			}
		}
		
		// If the Actor is an Isolated actor, we also want its own data.
		if (APCGIsolatedActor* IsolatedActor = Cast<APCGIsolatedActor>(FoundActor))
		{
			for (const FPCGTaggedData& TaggedData : IsolatedActor->Data.GetAllInputs())
			{
				if (ensure(TaggedData.Data))
				{
					FPCGTaggedData& DuplicatedTaggedData = Outputs.Add_GetRef(TaggedData);

					// In cases where the owner of the data (component -> owner) isn't in the main world, we MUST duplicate the data, otherwise we'll cause reference leaks in the cache.
#if WITH_EDITOR
					if (!SourceOwner || (FoundActor && FoundActor->GetLevel() != SourceOwner->GetWorld()->PersistentLevel))
					{
						DuplicatedTaggedData.Data = Cast<UPCGData>(StaticDuplicateObject(TaggedData.Data, GetTransientPackage()));
					}
#endif
					
					//todo_pcg Should we add the grid size tags from the owner'component?
					//  DuplicatedTaggedData.Tags.Add(PCGDataFromActorConstants::PCGComponentDataGridSizeTagPrefix + FString::FromInt(PCGHiGenGrid::GridToGridSize(Component->GetGenerationGrid())));

					// Add some logging if we have un-expected output pin
					if (DuplicatedTaggedData.Pin != PCGPinConstants::DefaultOutputLabel && !Settings->ExpectedPins.IsEmpty() && !Settings->ExpectedPins.Contains(TaggedData.Pin))
					{
						PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidIsolatedActorOutputPinPCGComponentData", 
							"Isolated actor '{0}' has data from an unexpected output pin: '{1}'.\nMaybe the data is outdated, try regenerate this component to refresh its output data"),
							FText::FromName(FoundActor->GetFName()),
							FText::FromName(TaggedData.Pin)), Context);
					}
				}
			}
			
		}
	}
	else if (FoundProperty)
	{
		bool bAbleToGetProperty = false;
		const void* PropertyAddressData = FoundProperty->ContainerPtrToValuePtr<void>(FoundActor);
		// TODO: support more property types here
		// Pointer to UPCGData
		// Soft object pointer to UPCGData
		// Array of pcg data -> all on the default pin
		// Map of pcg data -> use key for pin? might not be robust
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(FoundProperty))
		{
			if (StructProperty->Struct == FPCGDataCollection::StaticStruct())
			{
				const FPCGDataCollection* CollectionInProperty = reinterpret_cast<const FPCGDataCollection*>(PropertyAddressData);
				Outputs.Append(CollectionInProperty->TaggedData);

				bAbleToGetProperty = true;
			}
		}

		if (!bAbleToGetProperty)
		{
			PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("PropertyTypeUnsupported", "Actor '{0}' property '{1}' does not have a supported type"), FText::FromName(FoundActor->GetFName()), FText::FromName(Settings->PropertyName)));
		}
	}
	else
	{
		const bool bParseActor = (Settings->Mode != EPCGGetDataFromActorMode::GetSinglePoint);
		EPCGDataType TargetType = (Settings->Mode == EPCGGetDataFromActorMode::GetPointData ? EPCGDataType::Point : Settings->GetDataFilter());

		FPCGDataCollection ActorDataCollection = GetActorDataCollection(FoundActor, TargetType, bParseActor);
		Outputs += ActorDataCollection.TaggedData;
	}

	// Finally, if we're in a case where we need to output the single point data too, let's do it now.
	if (Settings->bAlsoOutputSinglePointData && (Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents))
	{
		const bool bParseActor = false;
		FPCGDataCollection SinglePointActorDataCollection = GetActorDataCollection(FoundActor, EPCGDataType::Any, bParseActor);

		for (const FPCGTaggedData& SinglePointData : SinglePointActorDataCollection.TaggedData)
		{
			FPCGTaggedData& OutSinglePoint = Outputs.Add_GetRef(SinglePointData);
			OutSinglePoint.Pin = PCGDataFromActorConstants::SinglePointPinLabel;
		}
	}
}

bool FPCGDataFromActorElement::IsCacheable(const UPCGSettings* InSettings) const
{
	if (const UPCGDataFromActorSettings* Settings = Cast<UPCGDataFromActorSettings>(InSettings))
	{
		return !Settings->bAlwaysRequeryActors;
	}
	else
	{
		return false;
	}
}

void FPCGDataFromActorElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	// If we track self or original, we are dependent on the actor data
	if (const UPCGDataFromActorSettings* Settings = Cast<const UPCGDataFromActorSettings>(InParams.Settings))
	{
		if (Settings->bAlwaysRequeryActors)
		{
			return;
		}

		// When the bounding shape pin is connected its data is already CRC'd by the base class;
		// suppress the self-bounds CRC contribution to avoid stale invalidation when the PCG actor moves.
		const bool bBoundingShapeConnected = InParams.InputData && InParams.InputData->GetInputsByPin(PCGDataFromActorConstants::BoundingShapeLabel).Num() > 0;
		const bool bDependsOnSelfOrHierarchy = (Settings->ActorSelector.ActorFilter == EPCGActorFilter::Self || Settings->ActorSelector.ActorFilter == EPCGActorFilter::Original);
		const bool bDependsOnSelfBounds = Settings->ActorSelector.bMustOverlapSelf && !bBoundingShapeConnected;

		if (InParams.ExecutionSource && (bDependsOnSelfOrHierarchy || bDependsOnSelfBounds))
		{
			IPCGGraphExecutionSource* ExecutionSourceToCheck = (Settings->ActorSelector.ActorFilter == EPCGActorFilter::Original) ? InParams.ExecutionSource->GetExecutionState().GetOriginalSource() : InParams.ExecutionSource;
			const UPCGData* SelfData = ExecutionSourceToCheck ? ExecutionSourceToCheck->GetExecutionState().GetSelfData() : nullptr;

			if (SelfData)
			{
				Crc.Combine(SelfData->GetOrComputeCrc(/*bFullDataCrc=*/false));

				// Also if it is depending on Self, we need to CRC the tags (as they will be passed to the output data)
				// Same with anything that would pull the actor reference.
				if (bDependsOnSelfOrHierarchy && ExecutionSourceToCheck)
				{
					if (const AActor* Owner = ExecutionSourceToCheck->GetExecutionState().GetTypedTarget<AActor>())
					{
						const bool bOutputActorReference = Settings->bAlsoOutputSinglePointData
							|| Settings->Mode == EPCGGetDataFromActorMode::GetActorReference
							|| Settings->Mode == EPCGGetDataFromActorMode::GetComponentsReference
							|| Settings->Mode == EPCGGetDataFromActorMode::GetSinglePoint;
						
						Crc.Combine(UPCGActorHelpers::ComputeHashFromActorTagsAndReference(Owner, /*bIncludeTags=*/true, bOutputActorReference));
					}
				}
			}

			// Include isolated actors when relevant.
			// @todo_pcg : is this a good idea if this isn't on the GT?
			if (Settings->ActorSelector.IncludeIsolatedActors != EPCGIsolatedActorIncludeFlag::Ignore && ExecutionSourceToCheck)
			{
				TArray<APCGIsolatedActor*> IsolatedActors = UPCGActorHelpers::GetIsolatedActorsOriginatingFrom(ExecutionSourceToCheck->GetExecutionState().GetTypedTarget<AActor>());
				for (APCGIsolatedActor* IsolatedActor : IsolatedActors)
				{
					Crc.Combine(GetTypeHash(IsolatedActor->GetPathName()));
				}
			}
		}

		const bool bDependsOnComponentData = Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Settings->Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents;
		const bool bDependsOnLocalComponentBounds = (Settings->bComponentsMustOverlapSelf || !Settings->bGetDataOnAllGrids) && !bBoundingShapeConnected;

		if (InParams.ExecutionSource && bDependsOnComponentData && bDependsOnLocalComponentBounds)
		{
			if (const UPCGData* LocalActorData = InParams.ExecutionSource->GetExecutionState().GetSelfData())
			{
				Crc.Combine(LocalActorData->GetOrComputeCrc(/*bFullDataCrc=*/false));
			}
		}
	}

	OutCrc = Crc;
}

#undef LOCTEXT_NAMESPACE
