// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDataFromTool.h"
#include "PCGComponent.h"
#include "PCGCustomVersion.h"
#include "PCGSubsystem.h"
#include "PCGTrackingManager.h"
#include "Data/PCGBasePointData.h"
#include "Data/Tool/PCGToolBaseData.h"
#include "Elements/PCGAddComponent.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Utils/PCGGraphExecutionLogging.h"

#include "Algo/AnyOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataFromTool)

#define LOCTEXT_NAMESPACE "PCGDataFromTool"

FString UPCGDataFromTool::GetAdditionalTitleInformation() const
{
	TStringBuilder<128> Builder;
	if (ToolTag.IsNone() == false)
	{
		Builder.Append(ToolTag.ToString());
	}

	bool bAddSeparator = false;
	if (ToolTag.IsNone() == false && DataInstance.IsNone() == false)
	{
		bAddSeparator = true;	
	}
	
	if (bAddSeparator)
	{
		Builder.Append(" | ");
	}

	if (DataInstance.IsNone() == false)
	{
		Builder.Append(DataInstance.ToString());
	}
	
	return FString(Builder);
}

#if WITH_EDITOR
FText UPCGDataFromTool::GetNodeTooltipText() const
{
	return LOCTEXT("DataFromToolTooltip", "Builds a collection of PCG-compatible data from the selected tools.");
}

EPCGChangeType UPCGDataFromTool::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGDataFromTool, ToolTag) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGDataFromTool, DataInstance))
	{
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	return ChangeType;
}
#endif

FPCGElementPtr UPCGDataFromTool::CreateElement() const
{
	return MakeShared<FPCGDataFromToolElement>();
}

TArray<FPCGPinProperties> UPCGDataFromTool::InputPinProperties() const
{
	// No input pins
	TArray<FPCGPinProperties> PinProperties;
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGDataFromTool::OutputPinProperties() const
{
	// Default output pin
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
	return PinProperties;
}

bool FPCGDataFromToolElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataFromToolElement::Execute);

	check(Context);
	
	const UPCGDataFromTool* Settings = Context->GetInputSettings<UPCGDataFromTool>();
	check(Settings);
	
	UPCGComponent* PCGComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get());

	if (PCGComponent && Settings->ToolTag.IsNone() == false)
	{
		FNameBuilder Builder(Settings->ToolTag);

		// This mirrors UPCGInteractiveToolSettings::GetWorkingDataIdentifier
		if (Settings->DataInstance.IsNone() == false)
		{
			Builder.AppendChar(TEXT('.'));
			Builder.Append(Settings->DataInstance.ToString());
		}

		FName WorkingDataIdentifier = Builder.ToString();

		const FPCGInteractiveToolWorkingData* Data = PCGComponent->ToolDataContainer.GetTypedWorkingData<FPCGInteractiveToolWorkingData>(WorkingDataIdentifier);
		if (Data && Data->IsValid())
		{
			Data->InitializeRuntimeElementData(Context);
		}
		// @todo_pcg
		// If we return false this causes accumulation of generation tasks, but if we return true it won't recognize the data change until we Flush Cache since the dynamic tracking key wasn't actually created yet
		else
		{
			return true;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
