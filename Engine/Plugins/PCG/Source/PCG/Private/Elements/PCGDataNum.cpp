// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDataNum.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Compute/PCGKernelHelpers.h"
#include "Compute/BuiltInKernels/PCGDataNumKernel.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataNum)

#define LOCTEXT_NAMESPACE "PCGDataNumElement"

#if WITH_EDITOR
FName UPCGDataNumSettings::GetDefaultNodeName() const
{
	return TEXT("DataNum");
}

FText UPCGDataNumSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Data Count");
}

FText UPCGDataNumSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Returns the count of data in the input data collection.");
}

void UPCGDataNumSettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);
	PCGKernelHelpers::CreateKernel<UPCGDataNumKernel>(InOutContext, CreateParams, OutKernels, OutEdges);
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGDataNumSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(
		PCGPinConstants::DefaultOutputLabel,
		EPCGDataType::Param,
		/*bInAllowMultipleConnections=*/true,
		/*bInAllowMultipleData=*/true,
		LOCTEXT("OutParamTooltip", "Attribute set containing the data count from the input collection"));

	return PinProperties;
}

FPCGElementPtr UPCGDataNumSettings::CreateElement() const
{
	return MakeShared<FPCGDataNumElement>();
}

bool FPCGDataNumElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataNumElement::Execute);

	check(Context);

	const UPCGDataNumSettings* Settings = Context->GetInputSettings<UPCGDataNumSettings>();
	check(Settings);

	const int32 InputDataCount = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel).Num();

	UPCGParamData* OutputParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	FPCGMetadataAttribute<int32>* NumAttribute = OutputParamData->Metadata->CreateAttribute<int32>(Settings->OutputAttributeName, InputDataCount, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);

	if (!NumAttribute)
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("AttributeCreationFailed", "Failed to create attribute {0}"), FText::FromName(Settings->OutputAttributeName)));
		return true;
	}

	OutputParamData->Metadata->AddEntry();

	FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
	Output.Data = OutputParamData;

	return true;
}

#undef LOCTEXT_NAMESPACE