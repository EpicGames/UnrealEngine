// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDataTableRowToParamData.h"

#include "PCGParamData.h"
#include "PCGPin.h"
#include "PCGModule.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "Metadata/PCGMetadata.h"

#include "Engine/DataTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataTableRowToParamData)

#define LOCTEXT_NAMESPACE "PCGDataTableRowToParamDataElement"

#if WITH_EDITOR
void UPCGDataTableRowToParamDataSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGDataTableRowToParamDataSettings, DataTable)) || DataTable.IsNull())
	{
		// Dynamic tracking or null settings
		return;
	}

	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(DataTable.ToSoftObjectPath());

	OutKeysToSettings.FindOrAdd(Key).Emplace(this, /*bCulling=*/false);
}

void UPCGDataTableRowToParamDataSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	// Overridable properties have been renamed, rename all pins by their counterpart, to avoid breaking existing graphs.
	InOutNode->RenameInputPin(TEXT("Path Override"), TEXT("Data Table"));
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGDataTableRowToParamDataSettings::CreateElement() const
{
	return MakeShared<FPCGDataTableRowToParamData>();
}

TArray<FPCGPinProperties> UPCGDataTableRowToParamDataSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

TArray<FPCGPinProperties> UPCGDataTableRowToParamDataSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param, false);

	return PinProperties;
}

void UPCGDataTableRowToParamDataSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (!PathOverride_DEPRECATED.IsEmpty())
	{
		DataTable = TSoftObjectPtr<UDataTable>(FSoftObjectPath(PathOverride_DEPRECATED));
		PathOverride_DEPRECATED.Empty();
	}
#endif // WITH_EDITOR
}

FString UPCGDataTableRowToParamDataSettings::GetAdditionalTitleInformation() const
{
	return FString::Printf(TEXT("%s[ %s ]"), DataTable ? *DataTable->GetFName().ToString() : TEXT("None"), *RowName.ToString());
}

bool FPCGDataTableRowToParamData::PrepareDataInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataTableRowToParamData::PrepareDataInternal);

	check(Context);

	const UPCGDataTableRowToParamDataSettings* Settings = Context->GetInputSettings<UPCGDataTableRowToParamDataSettings>();
	check(Settings);

	if (Settings->DataTable.IsNull())
	{
		return true;
	}

	FPCGDataTableRowToParamDataContext* ThisContext = static_cast<FPCGDataTableRowToParamDataContext*>(Context);

	if (!ThisContext->WasLoadRequested())
	{
		return ThisContext->RequestResourceLoad(ThisContext, { Settings->DataTable.ToSoftObjectPath() }, !Settings->bSynchronousLoad);
	}

	return true;
}

bool FPCGDataTableRowToParamData::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDataTableRowToParamData::Execute);

	check(Context);

	const UPCGDataTableRowToParamDataSettings* Settings = Context->GetInputSettings<UPCGDataTableRowToParamDataSettings>();
	check(Settings);

	const FName RowName = Settings->RowName;

	const TSoftObjectPtr<UDataTable> DataTablePtr = Settings->DataTable;

	const UDataTable* DataTable = DataTablePtr.Get();
	if (!DataTable)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("DataTableInvalid", "FPCGDataTableRowToParamData: Data table is invalid"));
		return true;
	}

	const uint8* const* RowDataItr = DataTable->GetRowMap().Find(RowName);
	const uint8* RowDataPtr = RowDataItr ? *RowDataItr : nullptr;
	if (!RowDataPtr)
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("NoRowFound", "FPCGDataTableRowToParamData: Data table '{0}' has no row named '{1}'"), FText::FromString(DataTable->GetPathName()), FText::FromName(RowName)));
		return true;
	}

	// From there, we should be able to create the data.
	UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	UPCGMetadata* Metadata = ParamData->MutableMetadata();
	check(Metadata);
	PCGMetadataEntryKey EntryKey = Metadata->AddEntry();

	for (TFieldIterator<const FProperty> FieldIt(DataTable->GetRowStruct(), EFieldIterationFlags::IncludeSuper); FieldIt; ++FieldIt)
	{
		const FString FieldName = DataTable->GetRowStruct()->GetAuthoredNameForField(*FieldIt);
		const FName AttributeName(FieldName);
		
		if (!Metadata->SetAttributeFromDataProperty(AttributeName, EntryKey, RowDataPtr, *FieldIt, /*bCreate=*/ true))
		{
			PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("AttributeCreationFailed", "Error while creating new attribute '{0}'. Either the property type is not supported by PCG or attribute creation failed."), FText::FromName(AttributeName)));
		}
	}
	
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	FPCGTaggedData& Output = Outputs.Emplace_GetRef();
	Output.Data = ParamData;

#if WITH_EDITOR
	// If we have an override, register for dynamic tracking.
	if (Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGDataTableRowToParamDataSettings, DataTable)))
	{
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, FPCGSelectionKey::CreateFromPath(DataTable), /*bIsCulled=*/false);
	}
#endif // WITH_EDITOR

	return true;
}

#undef LOCTEXT_NAMESPACE
