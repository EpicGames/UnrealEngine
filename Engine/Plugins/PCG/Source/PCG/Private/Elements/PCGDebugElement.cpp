// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDebugElement.h"

#include "PCGContext.h"
#include "PCGModule.h"

#if WITH_EDITOR
#include "PCGDataVisualization.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDebugElement)

#define LOCTEXT_NAMESPACE "PCGDebugElement"

FPCGElementPtr UPCGDebugSettings::CreateElement() const
{
	return MakeShared<FPCGDebugElement>();
}

TArray<FPCGPinProperties> UPCGDebugSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGDebugSettings::OutputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

bool FPCGDebugElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDebugElement::Execute);

#if WITH_EDITOR
	// Early validation: if we don't have a valid PCG component, we're not going to add the debug display info.
	if (!Context->ExecutionSource.IsValid())
	{
		return true;
	}

	const FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetConstPCGDataVisualizationRegistry();
	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	for (const FPCGTaggedData& Input : Inputs)
	{
		if (!Input.Data)
		{
			continue;
		}

		const UPCGDebugSettings* Settings = Context->GetInputSettings<UPCGDebugSettings>();
		check(Settings);

		AActor* TargetActor = Settings->TargetActor.Get();

		if (!TargetActor)
		{
			TargetActor = Context->GetDebugDisplayActor();
		}

		if (const IPCGDataVisualization* DataVis = DataVisRegistry.GetDataVisualization(Input.Data->GetClass()))
		{
			DataVis->ExecuteDebugDisplay(Context, Settings, Input.Data, TargetActor);
		}
	}
#endif
	
	return true;
}

#undef LOCTEXT_NAMESPACE
