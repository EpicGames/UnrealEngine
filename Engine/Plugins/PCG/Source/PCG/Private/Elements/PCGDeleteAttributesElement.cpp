// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDeleteAttributesElement.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGSpatialData.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDeleteAttributesElement)

#define LOCTEXT_NAMESPACE "PCGDeleteAttributesElement"

namespace PCGAttributeFilterConstants
{
	const FName NodeName = TEXT("DeleteAttributes");
	const FText NodeTitle = LOCTEXT("NodeTitle", "Delete Attributes");
	const FText NodeTitleAlias = LOCTEXT("NodeTitleAlias", "Filter Attributes By Name");
}

UPCGDeleteAttributesSettings::UPCGDeleteAttributesSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		Operation = EPCGAttributeFilterOperation::DeleteSelectedAttributes;
	}
}

void UPCGDeleteAttributesSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (!AttributesToKeep_DEPRECATED.IsEmpty())
	{
		SelectedAttributes.Empty();
		Operation = EPCGAttributeFilterOperation::KeepSelectedAttributes;
		// Can't use FString::Join since it is an array of FName
		for (int i = 0; i < AttributesToKeep_DEPRECATED.Num(); ++i)
		{
			if (i != 0)
			{
				SelectedAttributes += TEXT(",");
			}

			SelectedAttributes += AttributesToKeep_DEPRECATED[i].ToString();
		}

		AttributesToKeep_DEPRECATED.Empty();
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UPCGDeleteAttributesSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DataVersion < FPCGCustomVersion::AttributesAndTagsCanContainSpaces)
	{
		bTokenizeOnWhiteSpace = true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Super::ApplyDeprecation(InOutNode);
}

FName UPCGDeleteAttributesSettings::GetDefaultNodeName() const
{
	return PCGAttributeFilterConstants::NodeName;
}

FText UPCGDeleteAttributesSettings::GetDefaultNodeTitle() const
{
	return PCGAttributeFilterConstants::NodeTitle;
}

TArray<FText> UPCGDeleteAttributesSettings::GetNodeTitleAliases() const
{
	return { PCGAttributeFilterConstants::NodeTitleAlias };
}
#endif

FString UPCGDeleteAttributesSettings::GetAdditionalTitleInformation() const
{
	// The display name for the operation is way too long when put in a node title, so abbreviate it here.
	FString OperationString;
	if (Operation == EPCGAttributeFilterOperation::KeepSelectedAttributes)
	{
		OperationString = LOCTEXT("OperationKeep", "Keep").ToString();
	}
	else if (Operation == EPCGAttributeFilterOperation::DeleteSelectedAttributes)
	{
		OperationString = LOCTEXT("OperationDelete", "Delete").ToString();
	}
	else
	{
		ensureMsgf(false, TEXT("Unrecognized operation"));
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FString> AttributesToKeep = bTokenizeOnWhiteSpace
		? PCGHelpers::GetStringArrayFromCommaSeparatedString(SelectedAttributes)
		: PCGHelpers::GetStringArrayFromCommaSeparatedList(SelectedAttributes);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (AttributesToKeep.Num() == 1)
	{
		return FString::Printf(TEXT("%s (%s)"), *OperationString, *AttributesToKeep[0]);
	}
	else if (AttributesToKeep.IsEmpty())
	{
		return FString::Printf(TEXT("%s (%s)"), *OperationString, *LOCTEXT("NoAttributes", "None").ToString());
	}
	else
	{
		return FString::Printf(TEXT("%s (%s)"), *OperationString, *LOCTEXT("KeepMultipleAttributes", "Multiple").ToString());
	}
}

TArray<FPCGPinProperties> UPCGDeleteAttributesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGDeleteAttributesSettings::CreateElement() const
{
	return MakeShared<FPCGDeleteAttributesElement>();
}

bool FPCGDeleteAttributesElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDeleteAttributesElement::Execute);

	check(Context);

	const UPCGDeleteAttributesSettings* Settings = Context->GetInputSettings<UPCGDeleteAttributesSettings>();

	const bool bAddAttributesFromParent = (Settings->Operation == EPCGAttributeFilterOperation::DeleteSelectedAttributes);
	const EPCGMetadataFilterMode FilterMode = bAddAttributesFromParent ? EPCGMetadataFilterMode::ExcludeAttributes : EPCGMetadataFilterMode::IncludeAttributes;

	TSet<FName> AttributesToFilter;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FString> FilterAttributes = Settings->bTokenizeOnWhiteSpace
	   ? PCGHelpers::GetStringArrayFromCommaSeparatedString(Settings->SelectedAttributes, Context)
	   : PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->SelectedAttributes);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (const FString& FilterAttribute : FilterAttributes)
	{
		AttributesToFilter.Add(FName(*FilterAttribute));
	}

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	for (const FPCGTaggedData& InputTaggedData : Inputs)
	{
		const UPCGData* InputData = InputTaggedData.Data;

		const UPCGMetadata* InputMetadata = InputData->ConstMetadata();
		if (!InputMetadata)
		{
			PCGLog::Metadata::LogInvalidMetadata(Context);
			continue;
		}

		const FPCGAttributePropertySelector DomainSelector = FPCGAttributePropertySelector::CreateAttributeSelector(NAME_None, Settings->MetadataDomain);
		const FPCGMetadataDomainID SelectedDomainID = InputData->GetMetadataDomainIDFromSelector(DomainSelector);
		if (!SelectedDomainID.IsValid())
		{
			PCGLog::Metadata::LogInvalidMetadataDomain(DomainSelector, Context);
			continue;
		}

		const FPCGMetadataDomain* InputMetadataDomain = InputMetadata->GetConstMetadataDomain(SelectedDomainID);

		if (!InputMetadataDomain)
		{
			// Domain is supported but not allocated, nothing to do.
			continue;
		}

		UPCGData* OutputData = InputData->DuplicateData(Context, /*bInitializeMetadata=*/ false);
		if (!ensure(OutputData))
		{
			continue;
		}

		UPCGMetadata* OutputMetadata = OutputData->MutableMetadata();

		for (const FPCGMetadataDomainID DomainID : InputData->GetAllSupportedMetadataDomainIDs())
		{
			const FPCGMetadataDomain* InDomain = InputMetadata->GetConstMetadataDomain(DomainID);
			if (!InDomain)
			{
				// Not having an input domain is fine, it just means you don't have anything there.
				continue;
			}
			
			FPCGMetadataDomain* OutDomain = OutputMetadata->GetMetadataDomain(DomainID);
			check(OutDomain);
			
			if (InputMetadataDomain != InDomain)
			{
				OutDomain->Initialize(InDomain);
			}
			else
			{
				FPCGMetadataDomainInitializeParams Params(InputMetadataDomain);
				Params.FilterMode = FilterMode;
				Params.MatchOperator = Settings->Operator;
				Params.FilteredAttributes = AttributesToFilter;

				OutDomain->Initialize(Params);
			}
		}

		TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
		FPCGTaggedData& Output = Outputs.Add_GetRef(InputTaggedData);
		Output.Data = OutputData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
