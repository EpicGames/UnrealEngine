// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDeleteTags.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGData.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGTagHelpers.h"

#include "Algo/AnyOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDeleteTags)

#define LOCTEXT_NAMESPACE "PCGDeleteTagsElement"

#if WITH_EDITOR
void UPCGDeleteTagsSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DataVersion < FPCGCustomVersion::AttributesAndTagsCanContainSpaces)
	{
		bTokenizeOnWhiteSpace = true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Super::ApplyDeprecation(InOutNode);
}

FName UPCGDeleteTagsSettings::GetDefaultNodeName() const
{
	return TEXT("DeleteTags");
}

FText UPCGDeleteTagsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Delete Tags");
}
#endif // WITH_EDITOR

FString UPCGDeleteTagsSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* SelectionEnum = StaticEnum<EPCGTagFilterOperation>())
	{
		FText OperationText = SelectionEnum->GetDisplayNameTextByValue(static_cast<int64>(Operation));

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const TArray<FString> TagsToProcess = bTokenizeOnWhiteSpace
			? PCGHelpers::GetStringArrayFromCommaSeparatedString(SelectedTags)
			: PCGHelpers::GetStringArrayFromCommaSeparatedList(SelectedTags);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (TagsToProcess.Num() == 1)
		{
			return FString::Printf(TEXT("%s (%s)"), *OperationText.ToString(), *TagsToProcess[0]);
		}
		else
		{
			return OperationText.ToString();
		}
	}
	else
	{
		return FString();
	}
}

TArray<FPCGPinProperties> UPCGDeleteTagsSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGDeleteTagsSettings::CreateElement() const
{
	return MakeShared<FPCGDeleteTagsElement>();
}

bool FPCGDeleteTagsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGManageTagsElement::Execute);
	check(Context);

	const UPCGDeleteTagsSettings* Settings = Context->GetInputSettings<UPCGDeleteTagsSettings>();
	check(Settings);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FString> FilterTags = Settings->bTokenizeOnWhiteSpace
		? PCGHelpers::GetStringArrayFromCommaSeparatedString(Settings->SelectedTags, Context)
		: PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->SelectedTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TSet<FString> ValueTagsToFilter;
	TSet<FString> StringTagsToFilter;

	// When the user specifies tags with values inside, we'll do a straight string comparison,
	// otherwise, we'll assume that the input tag can be a tag value.
	for (const FString& FilterTag : FilterTags)
	{
		PCG::Private::FParseTagResult ParsedFilterTag(FilterTag);

		if (ParsedFilterTag.IsValid())
		{
			if (ParsedFilterTag.HasValue())
			{
				StringTagsToFilter.Add(FilterTag);
			}
			else
			{
				ValueTagsToFilter.Add(FilterTag);
			}
		}
	}

	const bool bKeepInFilter = (Settings->Operation == EPCGTagFilterOperation::KeepOnlySelectedTags);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	auto TestTag = [Settings, bKeepInFilter](const TSet<FString>& TagsToFilter, const FString Tag)
	{
		return ((Settings->Operator == EPCGStringMatchingOperator::Equal && TagsToFilter.Contains(Tag) == bKeepInFilter) ||
			(Settings->Operator == EPCGStringMatchingOperator::Substring && Algo::AnyOf(TagsToFilter, [&Tag](const FString& TagToFilter) { return Tag.Contains(TagToFilter); }) == bKeepInFilter) ||
			(Settings->Operator == EPCGStringMatchingOperator::Matches && Algo::AnyOf(TagsToFilter, [&Tag](const FString& TagToFilter) { return Tag.MatchesWildcard(TagToFilter); }) == bKeepInFilter));
	};

	for (const FPCGTaggedData& Input : Inputs)
	{
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		Output.Tags.Reset();
		Output.Tags.Reserve(Input.Tags.Num());

		for (const FString& Tag : Input.Tags)
		{
			const bool TestAgainstStringTags = TestTag(StringTagsToFilter, Tag);
			const bool TestAgainstValueTags = TestTag(ValueTagsToFilter, PCG::Private::FParseTagResult(Tag).GetOriginalAttribute());

			if ((bKeepInFilter && (TestAgainstStringTags || TestAgainstValueTags)) ||
				(!bKeepInFilter && (TestAgainstStringTags && TestAgainstValueTags)))
			{
				Output.Tags.Add(Tag);
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
