// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDensityFilter.h"

#include "Data/PCGSpatialData.h"
#include "PCGCustomVersion.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"
#include "PCGContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDensityFilter)

#define LOCTEXT_NAMESPACE "PCGDensityFilterElement"

FPCGElementPtr UPCGDensityFilterSettings::CreateElement() const
{
	return MakeShared<FPCGDensityFilterElement>();
}

bool FPCGDensityFilterElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDensityFilterElement::Execute);

	const UPCGDensityFilterSettings* Settings = Context->GetInputSettings<UPCGDensityFilterSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const bool bInvertFilter = Settings->bInvertFilter;
	const float LowerBound = Settings->LowerBound;
	const float UpperBound = Settings->UpperBound;
#if WITH_EDITOR
	const bool bKeepZeroDensityPoints = Settings->bKeepZeroDensityPoints;
#else
	const bool bKeepZeroDensityPoints = false;
#endif
	
	const bool bNormalizeOutputDensity = Settings->bNormalizeOutputDensity && !bInvertFilter && !bKeepZeroDensityPoints;

	const float MinBound = FMath::Min(LowerBound, UpperBound);
	const float MaxBound = FMath::Max(LowerBound, UpperBound);

	const bool bNoResults = (MaxBound <= 0.0f && !bInvertFilter) || (MinBound == 0.0f && MaxBound >= 1.0f && bInvertFilter);
	const bool bTrivialFilter = (MinBound <= 0.0f && MaxBound >= 1.0f && !bInvertFilter) || (MinBound == 0.0f && MaxBound == 0.0f && bInvertFilter);
	
	// Coefficients to remap the output density from [min, max] to [0, 1]: out_density = alpha * in_density + beta
	// If valid remap (min != max): alpha = 1 / (max - min) ; beta = -alpha * min 
	// If invalid remap (min == max): alpha = 0 ; beta = 0.5
	// If no remap: alpha = 1 ; beta = 0
	float Alpha = 1.0f;
	float Beta = 0.0f;
	if (bNormalizeOutputDensity)
	{
		const bool bValidRemap = !FMath::IsNearlyEqual(MinBound, MaxBound);
		Alpha = bValidRemap ? 1.0f / (MaxBound - MinBound) : 0.0f;
		Beta = bValidRemap ? -Alpha * MinBound : 0.5f;
	}
	
	auto RemapDensity = [Alpha, Beta](float InDensity) -> float
	{
		return Alpha * InDensity + Beta;
	};

	if (bNoResults && !bKeepZeroDensityPoints)
	{
		PCGE_LOG(Verbose, LogOnly, LOCTEXT("AllInputsRejected", "Skipped - all inputs rejected"));
		return true;
	}

	// TODO: embarassingly parallel loop
	for (const FPCGTaggedData& Input : Inputs)
	{
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		if (!Input.Data || Cast<UPCGSpatialData>(Input.Data) == nullptr)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data"));
			continue;
		}

		// Skip processing if the transformation is trivial
		if (bTrivialFilter)
		{
			PCGE_LOG(Verbose, LogOnly, LOCTEXT("TrivialFilter", "Skipped - trivial filter"));
			continue;
		}

		const UPCGBasePointData* OriginalData = Cast<UPCGSpatialData>(Input.Data)->ToBasePointData(Context);

		if (!OriginalData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("NoPointDataInInput", "Unable to get point data from input"));
			continue;
		}

		UPCGBasePointData* FilteredData = FPCGContext::NewPointData_AnyThread(Context);

		FPCGInitializeFromDataParams InitializeFromDataParams(OriginalData);

		// Do not inherit because we are going to filter out some points
		InitializeFromDataParams.bInheritSpatialData = false;
		FilteredData->InitializeFromDataWithParams(InitializeFromDataParams);
		
		Output.Data = FilteredData;

		auto InitializeFunc = [FilteredData, OriginalData]()
		{
			FilteredData->SetNumPoints(OriginalData->GetNumPoints(), /*bInitializeValues=*/false);
			FilteredData->AllocateProperties(OriginalData->GetAllocatedProperties() | EPCGPointNativeProperties::Density);
			FilteredData->CopyUnallocatedPropertiesFrom(OriginalData);
		};

		auto AsyncProcessRangeFunc = [FilteredData, OriginalData, MinBound, MaxBound, bInvertFilter, bKeepZeroDensityPoints, &RemapDensity](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			const FConstPCGPointValueRanges ReadRanges(OriginalData);
			FPCGPointValueRanges WriteRanges(FilteredData, /*bAllocate=*/false);
						
			int32 NumWritten = 0;

			for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
			{
				const int32 WriteIndex = StartWriteIndex + NumWritten;
				const float ReadDensity = ReadRanges.DensityRange[ReadIndex];
				const bool bInRange = (ReadDensity >= MinBound && ReadDensity <= MaxBound);
			
				if (bInRange != bInvertFilter)
				{
					WriteRanges.SetFromValueRanges(WriteIndex, ReadRanges, ReadIndex);
					WriteRanges.DensityRange[WriteIndex] = RemapDensity(ReadDensity);
					++NumWritten;
				}
#if WITH_EDITOR
				else if (bKeepZeroDensityPoints)
				{
					WriteRanges.SetFromValueRanges(WriteIndex, ReadRanges, ReadIndex);
					WriteRanges.DensityRange[WriteIndex] = 0.f;
					++NumWritten;
				}
#endif
			}

			return NumWritten;
		};

		auto MoveDataRangeFunc = [FilteredData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
		{
			FilteredData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		};

		auto FinishedFunc = [FilteredData](int32 NumWritten)
		{
			FilteredData->SetNumPoints(NumWritten);
		};

		FPCGAsyncState* AsyncState = Context ? &Context->AsyncState : nullptr;
		FPCGAsync::AsyncProcessingRangeEx(
			AsyncState,
			OriginalData->GetNumPoints(),
			InitializeFunc,
			AsyncProcessRangeFunc,
			MoveDataRangeFunc,
			FinishedFunc,
			/*bEnableTimeSlicing=*/false);

		PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("GenerationInfo", "Generated {0} points out of {1} source points"), FilteredData->GetNumPoints(), OriginalData->GetNumPoints()));
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
