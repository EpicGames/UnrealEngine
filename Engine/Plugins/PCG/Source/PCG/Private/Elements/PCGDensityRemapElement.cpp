// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDensityRemapElement.h"

#include "PCGContext.h"
#include "PCGPoint.h"
#include "Data/PCGBasePointData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDensityRemapElement)

UPCGDensityRemapSettings::UPCGDensityRemapSettings()
{
#if WITH_EDITOR
	bExposeToLibrary = false;
#endif // WITH_EDITOR
}

FPCGElementPtr UPCGDensityRemapSettings::CreateElement() const
{
	return MakeShared<FPCGDensityRemapElement>();
}

EPCGPointNativeProperties FPCGDensityRemapElement::GetPropertiesToAllocate(FPCGContext* InContext) const
{
	return EPCGPointNativeProperties::Density;
}

bool FPCGDensityRemapElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDensityRemapElement::Execute);

	ContextType* Context = static_cast<ContextType*>(InContext);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const UPCGDensityRemapSettings* Settings = Context->GetInputSettings<UPCGDensityRemapSettings>();
	check(Settings);

	const float InRangeMin = Settings->InRangeMin;
	const float InRangeMax = Settings->InRangeMax;
	const float OutRangeMin = Settings->OutRangeMin;
	const float OutRangeMax = Settings->OutRangeMax;
	const bool bExcludeValuesOutsideInputRange = Settings->bExcludeValuesOutsideInputRange;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// used to determine if a density value lies between TrueMin and TrueMax
	const float InRangeTrueMin = FMath::Min(InRangeMin, InRangeMax);
	const float InRangeTrueMax = FMath::Max(InRangeMin, InRangeMax);

	const float InRangeDifference = InRangeMax - InRangeMin;
	const float OutRangeDifference = OutRangeMax - OutRangeMin;

	float Slope;
	float Intercept;

	// When InRange is a point leave the Slope at 0 so that Density = Intercept
	if (InRangeDifference == 0)
	{
		Slope = 0;
		Intercept = (OutRangeMin + OutRangeMax) / 2.f;
	}
	else
	{
		Slope = OutRangeDifference / InRangeDifference;
		Intercept = OutRangeMin;
	}

	return ExecutePointOperation(Context, [bExcludeValuesOutsideInputRange, InRangeTrueMin, InRangeTrueMax, Slope, InRangeMin, Intercept](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
	{
		const TConstPCGValueRange<float> SourceDensityRange = InputData->GetConstDensityValueRange();
		TPCGValueRange<float> DensityRange = OutputData->GetDensityValueRange();

		for (int32 Index = StartIndex; Index < (StartIndex + Count); ++Index)
		{
			const float SourceDensity = SourceDensityRange[Index];
			if (!bExcludeValuesOutsideInputRange || (SourceDensity >= InRangeTrueMin && SourceDensity <= InRangeTrueMax))
			{
				const float UnclampedDensity = Slope * (SourceDensity - InRangeMin) + Intercept;
				DensityRange[Index] = FMath::Clamp(UnclampedDensity, 0.f, 1.f);
			}
		}

		return true;
	});
}
