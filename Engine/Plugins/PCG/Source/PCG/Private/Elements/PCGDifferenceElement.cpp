// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDifferenceElement.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGUnionData.h"
#include "Elements/PCGUnionElement.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDifferenceElement)

#define LOCTEXT_NAMESPACE "PCGDifferenceElement"

#if WITH_EDITOR
FText UPCGDifferenceSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Spatially subtracts the target difference data from the source data, outputing the difference of the two.");
}

void UPCGDifferenceSettings::ApplyStructuralDeprecation(UPCGNode* InOutNode)
{
	if (DataVersion < FPCGCustomVersion::DifferenceNodeIterateOnSourceAndUnionDifferences)
	{
		UPCGPin* SourcePin = InOutNode->GetInputPin(PCGDifferenceConstants::SourceLabel);
		check(SourcePin);

		// Check if single connection AND single data, to avoid adding a union if not necessary
		const int32 SourceEdgeCount = SourcePin->EdgeCount();
		if (SourceEdgeCount > 1 || (SourceEdgeCount == 1 && !SourcePin->Edges.IsEmpty() && SourcePin->Edges[0]->InputPin->AllowsMultipleData()))
		{
			// To match previous default behavior, we'll add a union node preceding the source inputs
			UPCGGraph* PCGGraph = InOutNode->GetGraph();
			check(PCGGraph);

			UPCGSettings* UnionSettings = nullptr;
			UPCGNode* UnionNode = PCGGraph->AddNodeOfType(UPCGUnionSettings::StaticClass(), UnionSettings);
			check(UnionNode && !UnionNode->GetInputPins().IsEmpty() && !UnionNode->GetOutputPins().IsEmpty());
			UnionSettings->bEnabled = InOutNode->GetSettings()->bEnabled;

			UPCGPin* UnionSourcePin = UnionNode->GetInputPins()[0];
			UPCGPin* UnionOutputPin = UnionNode->GetOutputPins()[0];

			int32 SumSourceNodesPositionX = 0;
			int32 SumSourceNodesPositionY = 0;
			while(!SourcePin->Edges.IsEmpty())
			{
				UPCGPin* UpstreamPin = SourcePin->Edges[0]->InputPin;
				check(UpstreamPin);

				// Collect the source node's position for later mean
				check(UpstreamPin->Node);
				int32 NodePositionX, NodePositionY;
				UpstreamPin->Node->GetNodePosition(NodePositionX, NodePositionY);
				SumSourceNodesPositionX += NodePositionX;
				SumSourceNodesPositionY += NodePositionY;
		
				UpstreamPin->BreakEdgeTo(SourcePin);
				UpstreamPin->AddEdgeTo(UnionSourcePin);
			}

			// The injected node should be placed halfway between the difference node and the mean of the sources' position
			int32 NodePositionX, NodePositionY;
			InOutNode->GetNodePosition(NodePositionX, NodePositionY);
			const int32 SourceNodeLocalPositionMeanX = SumSourceNodesPositionX / SourceEdgeCount;
			const int32 SourceNodeLocalPositionMeanY = SumSourceNodesPositionY / SourceEdgeCount;
			
			UnionNode->SetNodePosition((NodePositionX + SourceNodeLocalPositionMeanX) / 2, (NodePositionY + SourceNodeLocalPositionMeanY) / 2);
			UnionNode->NodeComment = LOCTEXT("DeprecationUnionNodeCreated", "Node added to replicate deprecated behavior of the Difference node.").ToString();
			UnionNode->bCommentBubbleVisible = 1;

			// Reconnect union back to source
			UnionOutputPin->AddEdgeTo(SourcePin);		

			UE_LOGF(LogPCG, Log, "Difference node of previous version detected. A 'Union' node was created automatically to replicate the previous version's behavior.");
		}
	}
	
	Super::ApplyStructuralDeprecation(InOutNode);
}
#endif // WITH_EDITOR

FString UPCGDifferenceSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGDifferenceSettings, DensityFunction)))
	{
		return FString();
	}
	else
#endif
	{
		if (const UEnum* EnumPtr = StaticEnum<EPCGDifferenceDensityFunction>())
		{
			return EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(DensityFunction)).ToString();
		}
		else
		{
			return FString();
		}
	}
}

TArray<FPCGPinProperties> UPCGDifferenceSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& PinPropertiesSource = PinProperties.Emplace_GetRef(PCGDifferenceConstants::SourceLabel, EPCGDataType::Spatial);
	PinPropertiesSource.SetRequiredPin();
	FPCGPinProperties& PinPropertiesDifferences = PinProperties.Emplace_GetRef(PCGDifferenceConstants::DifferencesLabel, EPCGDataType::Spatial);

#if WITH_EDITOR
	PinPropertiesSource.Tooltip = LOCTEXT("SourcePropertiesTooltip", "The source input of the operation, from which the difference will be subtracted. Multiple pins or data on this pin will be iterated upon, resulting in an output of the difference operation for every input.");
	PinPropertiesDifferences.Tooltip = LOCTEXT("DifferencesPropertiesTooltip", "The difference input of the operation. Inputs on this pin will be implicitly unioned together to be used in the difference operation.");
#endif // WITH_EDITOR

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGDifferenceSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& PinPropertiesOutput = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spatial);
	
#if WITH_EDITOR
	PinPropertiesOutput.Tooltip = LOCTEXT("OutputPropertiesTooltip", "The data after being processed through the difference operation.");
#endif // WITH_EDITOR
	
	return PinProperties;
}

FPCGElementPtr UPCGDifferenceSettings::CreateElement() const
{
	return MakeShared<FPCGDifferenceElement>();
}

bool FPCGDifferenceElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDifferenceElement::Execute);

	// Early-out for previous behavior (without labeled edges)
	if (Context->Node && !Context->Node->IsInputPinConnected(PCGDifferenceConstants::SourceLabel) && !Context->Node->IsInputPinConnected(PCGDifferenceConstants::DifferencesLabel))
	{
		return true;
	}

	const UPCGDifferenceSettings* Settings = Context->GetInputSettings<UPCGDifferenceSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Sources = Context->InputData.GetInputsByPin(PCGDifferenceConstants::SourceLabel);
	TArray<FPCGTaggedData> Differences = Context->InputData.GetInputsByPin(PCGDifferenceConstants::DifferencesLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	bool bHasPointsInSource = false;
	bool bHasPointsInDifferences = false;

	// Validate that there's usable data on the differences pin
	bool bDifferenceDataHasValidData = false;
	for (const FPCGTaggedData& Difference : Differences)
	{
		if (Difference.Data && Difference.Data->IsA<UPCGSpatialData>())
		{
			bDifferenceDataHasValidData = true;
			bHasPointsInDifferences |= Difference.Data->IsA<UPCGBasePointData>();
			break;
		}
	}

	// If no valid difference data, send sources directly to output and return
	if (!bDifferenceDataHasValidData)
	{
		Outputs = Sources;
		return true;
	}

	// Iterate over the sources and subtract the differences
	for (const FPCGTaggedData& Source : Sources)
	{
		const UPCGSpatialData* SourceSpatialData = Cast<UPCGSpatialData>(Source.Data);

		// Propagate non-spatial data
		if (!SourceSpatialData)
		{
			Outputs.Add(Source);
			continue;
		}

		bHasPointsInSource |= SourceSpatialData->IsA<UPCGBasePointData>();

		UPCGDifferenceData* DifferenceData = FPCGContext::NewObject_AnyThread<UPCGDifferenceData>(Context);
		DifferenceData->Initialize(SourceSpatialData);
		
		for (const FPCGTaggedData& Difference : Differences)
		{
			if (const UPCGSpatialData* DifferenceSpatialData = Cast<const UPCGSpatialData>(Difference.Data))
			{
				DifferenceData->AddDifference(Context, DifferenceSpatialData);
			}
		}

		DifferenceData->SetDensityFunction(Settings->DensityFunction);
		DifferenceData->bDiffMetadata = Settings->bDiffMetadata;
		DifferenceData->bKeepZeroDensityPoints = Settings->bKeepZeroDensityPoints;

		FPCGTaggedData& Output = Outputs.Add_GetRef(Source);
		Output.Data = DifferenceData;

		// Finally, apply any discretization based on the mode
		if ((Settings->Mode == EPCGDifferenceMode::Discrete || 
			(Settings->Mode == EPCGDifferenceMode::Inferred && bHasPointsInSource && bHasPointsInDifferences)))
		{
			Output.Data = DifferenceData->ToBasePointData(Context);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE