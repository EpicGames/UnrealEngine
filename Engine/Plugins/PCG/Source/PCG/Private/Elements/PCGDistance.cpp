// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDistance.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGGather.h"
#include "Helpers/PCGAsync.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDistance)

#define LOCTEXT_NAMESPACE "PCGDistanceElement"

namespace PCGDistance
{
	const FName SourceLabel = TEXT("Source");
	const FName TargetLabel = TEXT("Target");

	FVector CalcPosition(PCGDistanceShape Shape, const FTransform& SourceTransform, const FVector& SourceBoundsMin, const FVector& SourceBoundsMax, const FVector SourceCenter, const FVector TargetCenter)
	{
		if (Shape == PCGDistanceShape::SphereBounds)
		{
			FVector Dir = TargetCenter - SourceCenter;
			Dir.Normalize();

			return SourceCenter + Dir * PCGPointHelpers::GetScaledExtents(SourceTransform, SourceBoundsMin, SourceBoundsMax).Length();
		}
		else if (Shape == PCGDistanceShape::BoxBounds)
		{
			const FVector LocalTargetCenter = SourceTransform.InverseTransformPosition(TargetCenter);

			const double DistanceSquared = ComputeSquaredDistanceFromBoxToPoint(SourceBoundsMin, SourceBoundsMax, LocalTargetCenter);

			FVector Dir = -LocalTargetCenter;
			Dir.Normalize();

			const FVector LocalClosestPoint = LocalTargetCenter + Dir * FMath::Sqrt(DistanceSquared);

			return SourceTransform.TransformPosition(LocalClosestPoint);
		}

		// PCGDistanceShape::Center
		return SourceCenter;
	}
}

#if WITH_EDITOR
FText UPCGDistanceSettings::GetNodeTooltipText() const
{
	return LOCTEXT("PCGDistanceTooltip", "Calculates and appends a signed 'Distance' attribute to the source data. For each of the source points, a distance attribute will be calculated between it and the nearest target point.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGDistanceSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& PinPropertySource = PinProperties.Emplace_GetRef(PCGDistance::SourceLabel, EPCGDataType::Point);
	PinPropertySource.SetRequiredPin();
	FPCGPinProperties& PinPropertyTarget = PinProperties.Emplace_GetRef(PCGDistance::TargetLabel, EPCGDataType::Point);
	if (bCheckSourceAgainstRespectiveTarget)
	{
		PinPropertyTarget.SetRequiredPin();
	}

#if WITH_EDITOR
	PinPropertySource.Tooltip = LOCTEXT("PCGSourcePinTooltip", "For each of the source points, a distance attribute will be calculated between it and the nearest target point.");
	PinPropertyTarget.Tooltip = LOCTEXT("PCGTargetPinTooltip", "The target points to conduct a distance check with each source point.");
#endif // WITH_EDITOR

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGDistanceSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& PinPropertyOutput = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);

#if WITH_EDITOR
	PinPropertyOutput.Tooltip = LOCTEXT("PCGOutputPinTooltip", "The source points will be output with the newly added 'Distance' attribute as well as have their density set to [0,1] based on the 'Maximum Distance' if 'Set Density' is enabled.");
#endif // WITH_EDITOR
	
	return PinProperties;
}

FPCGElementPtr UPCGDistanceSettings::CreateElement() const
{
	return MakeShared<FPCGDistanceElement>();
}

#if WITH_EDITOR
EPCGChangeType UPCGDistanceSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGDistanceSettings, bCheckSourceAgainstRespectiveTarget))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif

void UPCGDistanceSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (AttributeName_DEPRECATED != PCGDistanceConstants::DefaultOutputAttributeName)
	{
		// "None" was previously used to indicate that nothing should be written to attribute
		if (AttributeName_DEPRECATED == NAME_None)
		{
			bOutputToAttribute = false;
			OutputAttribute.SetAttributeName(PCGDistanceConstants::DefaultOutputAttributeName);
		}
		else
		{
			bOutputToAttribute = true;
			OutputAttribute.SetAttributeName(AttributeName_DEPRECATED);
		}

		AttributeName_DEPRECATED = PCGDistanceConstants::DefaultOutputAttributeName;
	}
#endif // WITH_EDITOR
}

EPCGElementExecutionLoopMode FPCGDistanceElement::ExecutionLoopMode(const UPCGSettings* InSettings) const
{
	if (const UPCGDistanceSettings* Settings = Cast<UPCGDistanceSettings>(InSettings))
	{
		return Settings->bCheckSourceAgainstRespectiveTarget ? EPCGElementExecutionLoopMode::PrimaryPinAndBroadcastablePins : EPCGElementExecutionLoopMode::SinglePrimaryPin;
	}
	else
	{
		return EPCGElementExecutionLoopMode::NotALoop;
	}
}

bool FPCGDistanceElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDistanceElement::Execute);
	check(Context);

	if (Context->Node && !Context->Node->IsInputPinConnected(PCGDistance::TargetLabel))
	{
		// If Target pin is unconnected then we no-op and pass through all data from Source pin.
		Context->OutputData = PCGGather::GatherDataForPin(Context->InputData, PCGDistance::SourceLabel);
		return true;
	}

	const UPCGDistanceSettings* Settings = Context->GetInputSettings<UPCGDistanceSettings>();
	check(Settings);

	const bool bSetDensity = Settings->bSetDensity;
	const bool bOutputDistanceVector = Settings->bOutputDistanceVector;
	const PCGDistanceShape SourceShape = Settings->SourceShape;
	const PCGDistanceShape TargetShape = Settings->TargetShape;

	const double MaximumDistance = FMath::Max(0.0, Settings->MaximumDistance);
	const double MaximumDistanceRecip = MaximumDistance > UE_DOUBLE_SMALL_NUMBER ? 1.0 / MaximumDistance : 0.0;

	TArray<FPCGTaggedData> Sources = Context->InputData.GetInputsByPin(PCGDistance::SourceLabel);
	TArray<FPCGTaggedData> Targets = Context->InputData.GetInputsByPin(PCGDistance::TargetLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	TArray<const UPCGBasePointData*> TargetPointDatas;
	TargetPointDatas.Reserve(Targets.Num());

	for (const FPCGTaggedData& Target : Targets)
	{
		const UPCGSpatialData* TargetData = Cast<UPCGSpatialData>(Target.Data);

		if (!TargetData)
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("TargetMustBeSpatial", "Target must be Spatial data, found '{0}'"), FText::FromString(Target.Data->GetClass()->GetName())));
			continue;
		}

		const UPCGBasePointData* TargetPointData = TargetData->ToBasePointData(Context);
		if (!TargetPointData)
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("CannotConvertToPoint", "Cannot convert target '{0}' into Point data"), FText::FromString(Target.Data->GetClass()->GetName())));
			continue;
		}

		TargetPointDatas.Add(TargetPointData);
	}

	// @todo_pcg: normally, we should do nothing if there is no target data, but this would break older data.
	// We should introduce an advanced option to "perform" the operation even if there are no targets, make that true for old data, and false for newer data.
	if (TargetPointDatas.IsEmpty() && !Settings->bOutputToAttribute)
	{
		// If Target pin has no valid data then we no-op and pass through all data from Source pin.
		Context->OutputData = PCGGather::GatherDataForPin(Context->InputData, PCGDistance::SourceLabel);
		return true;
	}

	if (Settings->bCheckSourceAgainstRespectiveTarget && Sources.Num() > 1 && TargetPointDatas.Num() > 1 && Sources.Num() != TargetPointDatas.Num())
	{
		PCGLog::InputOutput::LogInvalidCardinalityError(PCGDistance::SourceLabel, PCGDistance::TargetLabel, Context);
		return true;
	}

	// First find the total Input bounds which will determine the size of each cell
	for (int32 SourceIndex = 0; SourceIndex < Sources.Num(); ++SourceIndex) 
	{
		const FPCGTaggedData& Source = Sources[SourceIndex];
		// Add the point bounds to the input cell

		const UPCGSpatialData* SourceData = Cast<UPCGSpatialData>(Source.Data);

		if (!SourceData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data"));
			continue;
		}

		const UPCGBasePointData* SourcePointData = SourceData->ToBasePointData(Context);
		if (!SourcePointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("CannotConvertToPointData", "Cannot convert input Spatial data to Point data"));
			continue;
		}

		UPCGBasePointData* OutputData = FPCGContext::NewPointData_AnyThread(Context);
		OutputData->InitializeFromData(SourcePointData);
		OutputData->SetNumPoints(SourcePointData->GetNumPoints(), /*bInitializeValues=*/false);

		if (!OutputData->HasSpatialDataParent())
		{
			OutputData->AllocateProperties(SourcePointData->GetAllocatedProperties());
		}

		if (bSetDensity)
		{
			OutputData->AllocateProperties(EPCGPointNativeProperties::Density);
		}

		Outputs.Add_GetRef(Source).Data = OutputData;

		if (Settings->bOutputToAttribute && Settings->OutputAttribute.IsBasicAttribute())
		{
			if (bOutputDistanceVector)
			{
				OutputData->Metadata->FindOrCreateAttribute<FVector>(Settings->OutputAttribute.GetAttributeName());
			}
			else
			{
				OutputData->Metadata->FindOrCreateAttribute<double>(Settings->OutputAttribute.GetAttributeName());
			}
		}

		// There is nothing to do as we will search against nothing - leave the default value in the attribute.
		if (TargetPointDatas.IsEmpty())
		{
			continue;
		}

		TUniquePtr<IPCGAttributeAccessor> Accessor;
		TUniquePtr<IPCGAttributeAccessorKeys> Keys;

		if (Settings->bOutputToAttribute)
		{
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputData, Settings->OutputAttribute);
		}

		// If the selected attribute is a property or extra property and not the correct type, invalidate the accessor
		if (Accessor.IsValid())
		{
			using PCG::Private::MetadataTypes;

			if ((bOutputDistanceVector && !IsBroadcastableOrConstructible(Accessor->GetUnderlyingType(), MetadataTypes<FVector>::Id)) ||
				(!bOutputDistanceVector && !IsBroadcastableOrConstructible(Accessor->GetUnderlyingType(), MetadataTypes<double>::Id)))
			{
				PCGE_LOG(Warning, GraphAndLog, LOCTEXT("InvalidAccessorType", "Selected type for Output Attribute is incompatible with distance as output."));
				Accessor = nullptr;
			}
		}

		if (Accessor.IsValid())
		{
			Keys = PCGAttributeAccessorHelpers::CreateKeys(OutputData, Settings->OutputAttribute);

			if (!Keys.IsValid())
			{
				PCGE_LOG(Warning, GraphAndLog, LOCTEXT("CannotCreateAccessorKeys", "Cannot create accessor keys on output points."));
				Accessor = nullptr;
			}
		}

		struct TemporaryResultCache
		{
			TArray<double> Distances;
			TArray<FVector> DistanceVectors;
		} ResultCache;

		// Set up a cache so we can set all the attributes in a single range set
		if (Accessor.IsValid())
		{
			if (bOutputDistanceVector)
			{
				ResultCache.DistanceVectors.SetNumUninitialized(SourcePointData->GetNumPoints());
			}
			else
			{
				ResultCache.Distances.SetNumUninitialized(SourcePointData->GetNumPoints());
			}
		}

		auto ProcessDistanceFunc = [SourceShape, TargetShape, &TargetPointDatas, MaximumDistance, MaximumDistanceRecip, bSetDensity, bOutputDistanceVector, OutputData, SourcePointData, &ResultCache, bWriteToAttribute = Accessor.IsValid(), Settings, SourceIndex](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			check(StartReadIndex == StartWriteIndex);

			if (!OutputData->HasSpatialDataParent())
			{
				SourcePointData->CopyPointsTo(OutputData, StartReadIndex, StartWriteIndex, Count);
			}

			struct FPointDataInfo
			{
				explicit FPointDataInfo(const UPCGBasePointData* InPointData)
					: PointData(InPointData)
					, TransformRange(PointData->GetConstTransformValueRange())
					, BoundsMinRange(PointData->GetConstBoundsMinValueRange())
					, BoundsMaxRange(PointData->GetConstBoundsMaxValueRange())
				{}

				const UPCGBasePointData* PointData;
				const TConstPCGValueRange<FTransform> TransformRange;
				const TConstPCGValueRange<FVector> BoundsMinRange;
				const TConstPCGValueRange<FVector> BoundsMaxRange;
			};

			const FPointDataInfo SourceDataInfo{SourcePointData};

			TPCGValueRange<float> OutputDensityRange = bSetDensity ? OutputData->GetDensityValueRange(/*bAllocate=*/false) : TPCGValueRange<float>();

			for (int32 Index = StartReadIndex; Index < (StartReadIndex + Count); ++Index)
			{
				const FTransform& SourceTransform = SourceDataInfo.TransformRange[Index];
				const FVector& SourceBoundsMin = SourceDataInfo.BoundsMinRange[Index];
				const FVector& SourceBoundsMax = SourceDataInfo.BoundsMaxRange[Index];

				const FBoxSphereBounds SourceQueryBounds = FBoxSphereBounds(FBox(SourceBoundsMin - FVector(MaximumDistance), SourceBoundsMax + FVector(MaximumDistance))).TransformBy(SourceTransform);

				const FVector SourceCenter = SourceTransform.TransformPosition(PCGPointHelpers::GetLocalCenter(SourceBoundsMin, SourceBoundsMax));

				double MinDistanceSquared = MaximumDistance * MaximumDistance;
				FVector MinDistanceVector = FVector::ZeroVector;

				// Signed distance field for calculating the closest point of source and target
				auto CalculateSDF = [Index, &MinDistanceSquared, &MinDistanceVector, SourcePointData, &SourceTransform, &SourceBoundsMin, &SourceBoundsMax, SourceCenter, SourceShape, TargetShape](const FPointDataInfo& TargetDataInfo, const int32 TargetPointIndex, const FBoxSphereBounds& Bounds)
				{
					// If the source pointer and target pointer are the same, ignore distance to the exact same point
					if (Index == TargetPointIndex && SourcePointData == TargetDataInfo.PointData)
					{
						return;
					}

					const FVector& TargetCenter = Bounds.Origin;

					const FVector SourceShapePos = PCGDistance::CalcPosition(SourceShape, SourceTransform, SourceBoundsMin, SourceBoundsMax, SourceCenter, TargetCenter);
					const FVector TargetShapePos = PCGDistance::CalcPosition(TargetShape, TargetDataInfo.TransformRange[TargetPointIndex], TargetDataInfo.BoundsMinRange[TargetPointIndex], TargetDataInfo.BoundsMaxRange[TargetPointIndex], TargetCenter, SourceCenter);

					const FVector ToTargetShapeDir = TargetShapePos - SourceShapePos;
					const FVector ToTargetCenterDir = TargetCenter - SourceCenter;

					const double Sign = FMath::Sign(ToTargetShapeDir.Dot(ToTargetCenterDir));
					const double ThisDistanceSquared = ToTargetShapeDir.SquaredLength() * Sign;

					if (ThisDistanceSquared < MinDistanceSquared)
					{
						MinDistanceSquared = ThisDistanceSquared;
						MinDistanceVector = ToTargetShapeDir;
					}
				};

				auto CheckAgainstTargetPointData = [&SourceQueryBounds, &CalculateSDF](const UPCGBasePointData* TargetPointData)
				{
					check(TargetPointData);

					const PCGPointOctree::FPointOctree& Octree = TargetPointData->GetPointOctree();
					
					Octree.FindElementsWithBoundsTest(FBoxCenterAndExtent(SourceQueryBounds.Origin, SourceQueryBounds.BoxExtent),
						[TargetDataInfo = FPointDataInfo{TargetPointData}, &CalculateSDF](const PCGPointOctree::FPointRef& InPointRef)
						{
							CalculateSDF(TargetDataInfo, InPointRef.Index, InPointRef.Bounds);
						});
				};

				if (Settings->bCheckSourceAgainstRespectiveTarget)
				{
					CheckAgainstTargetPointData(TargetPointDatas[SourceIndex % TargetPointDatas.Num()]);
				}
				else
				{
					for (const UPCGBasePointData* TargetPointData : TargetPointDatas)
					{
						CheckAgainstTargetPointData(TargetPointData);
					}
				}

				const double Distance = FMath::Sign(MinDistanceSquared) * FMath::Sqrt(FMath::Abs(MinDistanceSquared));

				if (bWriteToAttribute)
				{
					if (bOutputDistanceVector)
					{
						ResultCache.DistanceVectors[Index] = MinDistanceVector;
					}
					else
					{
						ResultCache.Distances[Index] = Distance;
					}
				}

				if (bSetDensity)
				{
					OutputDensityRange[Index] = MaximumDistance > UE_DOUBLE_SMALL_NUMBER ? (FMath::Clamp(Distance, -MaximumDistance, MaximumDistance) * MaximumDistanceRecip) : 1.0f;
				}
			}

			return Count;
		};

		if (FPCGAsync::AsyncProcessingOneToOneRangeEx(
			&Context->AsyncState,
			SourcePointData->GetNumPoints(),
			/*InitializeFunc=*/[]{},
			ProcessDistanceFunc,
			/*bTimeSliceEnabled=*/false))
		{
			if (Accessor.IsValid())
			{
				// Set all the attributes at once
				if (bOutputDistanceVector)
				{
					Accessor->SetRange<FVector>(ResultCache.DistanceVectors, 0, *Keys);
				}
				else
				{
					Accessor->SetRange<double>(ResultCache.Distances, 0, *Keys);
				}
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
