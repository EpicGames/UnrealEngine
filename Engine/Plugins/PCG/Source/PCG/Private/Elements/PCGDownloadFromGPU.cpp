// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDownloadFromGPU.h"

#include "PCGContext.h"
#include "PCGPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDownloadFromGPU)

#define LOCTEXT_NAMESPACE "PCGDownloadFromGPUElement"

TArray<FPCGPinProperties> UPCGDownloadFromGPUSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
	return PinProperties;
}

FPCGElementPtr UPCGDownloadFromGPUSettings::CreateElement() const
{
	return MakeShared<FPCGDownloadFromGPUElement>();
}

bool FPCGDownloadFromGPUElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDownloadFromGPUElement::Execute);
	check(Context);

	// By the time we reach ExecuteInternal, ConvertInputsIfNeeded has already read back any GPU resident data. Pure passthrough.
	Context->OutputData.TaggedData = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	return true;
}

#undef LOCTEXT_NAMESPACE
