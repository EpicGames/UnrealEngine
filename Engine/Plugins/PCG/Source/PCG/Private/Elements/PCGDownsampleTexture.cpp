// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDownsampleTexture.h"

#include "PCGContext.h"
#include "PCGData.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Data/PCGTextureData.h"

#include "RenderCaptureInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"
#include "Algo/AnyOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDownsampleTexture)

#define LOCTEXT_NAMESPACE "PCGDownsampleTextureElement"

namespace PCGDownsampleTexture
{
#if !UE_BUILD_SHIPPING
	static int32 TriggerGPUCaptureDispatchIndex = 0;
	static FAutoConsoleVariableRef CVarTriggerGPUCaptureDispatchIndex(
		TEXT("pcg.GPU.TriggerRenderCaptures.DownsampleTexture"),
		TriggerGPUCaptureDispatchIndex,
		TEXT("Index of the next dispatch to capture. I.e. if set to 2, will ignore one dispatch and then trigger a capture on the next one."),
		ECVF_RenderThreadSafe
	);
#endif // !UE_BUILD_SHIPPING
}

#if WITH_EDITOR
FText UPCGDownsampleTextureSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Downsample Texture");
}

FText UPCGDownsampleTextureSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Downsample a texture and populate its mip chain.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGDownsampleTextureSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, FPCGDataTypeInfoTexture2DBase::AsId(), /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGDownsampleTextureSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGDownsampleTextureSettings::CreateElement() const
{
	return MakeShared<FPCGDownsampleTextureElement>();
}

bool FPCGDownsampleTextureElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDownsampleTextureElement::Execute);
	check(InContext);
	FPCGDownsampleTextureContext* Context = static_cast<FPCGDownsampleTextureContext*>(InContext);

	const UPCGDownsampleTextureSettings* Settings = Context->GetInputSettings<UPCGDownsampleTextureSettings>();
	check(Settings);

	if (!Context->bSubmittedRenderCommands)
	{
		for (const FPCGTaggedData& Input : Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
		{
			const UPCGTexture2DBaseData* InputTexture2D = Cast<UPCGTexture2DBaseData>(Input.Data);
			if (!InputTexture2D)
			{
				continue;
			}
			
			const TRefCountPtr<IPooledRenderTarget> InputTextureHandle = InputTexture2D->GetRefCountedTexture();
			if (!InputTextureHandle)
			{
				continue;
			}

			const int32 OutputIndex = Context->TextureOutputInfo.Num();

			FPCGDownsampleTextureContext::FTextureOutputInfo& OutputInfo = Context->TextureOutputInfo.Emplace_GetRef();
			OutputInfo.Transform = InputTexture2D->GetTransform();
			OutputInfo.Filter = InputTexture2D->Filter;
			OutputInfo.ArraySize = InputTexture2D->GetArraySize();
			OutputInfo.bIsArray = InputTexture2D->IsA<UPCGTexture2DArrayData>();

			FIntPoint Resolution = InputTexture2D->GetResolution();

			if (Settings->PadResolutionToNextPowerOfTwo())
			{
				Resolution.X = 1u << FMath::CeilLogTwo(Resolution.X);
				Resolution.Y = 1u << FMath::CeilLogTwo(Resolution.Y);
			}

			if (Settings->PadResolutionToSquare())
			{
				const int32 MaxDim = FMath::Max(Resolution.X, Resolution.Y);
				Resolution.X = Resolution.Y = MaxDim;
			}

			FRDGTextureDesc OutputTextureDesc = FRDGTextureDesc::Create2DArray(
				Resolution,
				InputTexture2D->GetFormat(),
				FClearValueBinding(),
				ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV,
				InputTexture2D->GetArraySize()
			);

			OutputTextureDesc.NumMips = FMath::FloorLog2(FMath::Max(Resolution.X, Resolution.Y)) + 1;

			++Context->NumSubmittedRenderCommands;

			ENQUEUE_RENDER_COMMAND(PCGDownsampleTexture)([ContextHandle=Context->GetOrCreateHandle(), InputTextureHandle, OutputTextureDesc, OutputIndex, Mode=Settings->GetMode()](FRHICommandListImmediate& RHICmdList)
			{
				LLM_SCOPE_BYTAG(PCG);

				FPCGContext::FSharedContext<FPCGDownsampleTextureContext> SharedContext(ContextHandle);
				FPCGDownsampleTextureContext* ContextInner = SharedContext.Get();
				if (!ContextInner)
				{
					return;
				}

				FRDGBuilder GraphBuilder(RHICmdList);
				TRefCountPtr<IPooledRenderTarget> OutputTextureExported;

				{
					RDG_EVENT_SCOPE(GraphBuilder, "PCGDownsampleTexture");

#if !UE_BUILD_SHIPPING
					RenderCaptureInterface::FScopedCapture RenderCapture(PCGDownsampleTexture::TriggerGPUCaptureDispatchIndex > 0, GraphBuilder, TEXT("PCGDownsampleTexture"));
					PCGDownsampleTexture::TriggerGPUCaptureDispatchIndex = FMath::Max(PCGDownsampleTexture::TriggerGPUCaptureDispatchIndex - 1, 0);
#endif // !UE_BUILD_SHIPPING

					FRDGTextureRef InputTexture = GraphBuilder.RegisterExternalTexture(InputTextureHandle);
					FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputTextureDesc, TEXT("PCGDownsampledTexture"));

					// Populate mip 0
					{
						FRHICopyTextureInfo CopyInfo;
						CopyInfo.SourceMipIndex = 0;
						CopyInfo.DestMipIndex = 0;
						CopyInfo.NumMips = 1;
						CopyInfo.SourceSliceIndex = 0;
						CopyInfo.DestSliceIndex = 0;
						CopyInfo.NumSlices = OutputTextureDesc.ArraySize;

						AddCopyTexturePass(GraphBuilder, InputTexture, OutputTexture, CopyInfo);
					}

					// Downsample passes to populate rest of mips
					{
						PCGTextureDownsample::FParams Params;
						Params.Texture = OutputTexture;
						Params.Sampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
						Params.SliceIndex = 0;
						Params.NumSlices = OutputTextureDesc.ArraySize;
						Params.Mode = Mode;

						PCGTextureDownsample::DownsampleTexture(GraphBuilder, Params);
					}

					OutputTextureExported = GraphBuilder.ConvertToExternalTexture(OutputTexture);
					GraphBuilder.SetTextureAccessFinal(OutputTexture, ERHIAccess::SRVCompute);
				}

				GraphBuilder.Execute();

				// Pass exported buffer back to game thread and wake up this element.
				ExecuteOnGameThread(UE_SOURCE_LOCATION, [ContextHandle, OutputTextureExported, OutputIndex]()
				{
					LLM_SCOPE_BYTAG(PCG);

					FPCGContext::FSharedContext<FPCGDownsampleTextureContext> SharedContext(ContextHandle);
					if (FPCGDownsampleTextureContext* Context = SharedContext.Get())
					{
						Context->TextureOutputInfo[OutputIndex].OutputHandle = OutputTextureExported;
						++Context->NumCompletedRenderCommands;

						if (Context->NumCompletedRenderCommands >= Context->NumSubmittedRenderCommands)
						{
							Context->bIsPaused = false;
						}
					}
				});
			});
		}

		if (Context->NumSubmittedRenderCommands == 0)
		{
			// No valid data, nothing to be done, finish execution.
			return true;
		}

		Context->bSubmittedRenderCommands = true;

		// Render command will wake this task up after completing.
		Context->bIsPaused = true;

		return false;
	}

	if (Algo::AnyOf(Context->TextureOutputInfo, [](const FPCGDownsampleTextureContext::FTextureOutputInfo& InInfo) { return !InInfo.OutputHandle; }))
	{
		UE_LOGF(LogPCG, Error, "Downsample operation failed - null output texture handle encountered.");
		return true;
	}

	if (!Context->bCreatedOutputData)
	{
		for (FPCGDownsampleTextureContext::FTextureOutputInfo& OutputInfo : Context->TextureOutputInfo)
		{
			if (!OutputInfo.bIsArray)
			{
				OutputInfo.OutputData = FPCGContext::NewObject_AnyThread<UPCGTextureData>(Context);
				OutputInfo.OutputData->Filter = OutputInfo.Filter;
			}
			else
			{
				OutputInfo.OutputData = FPCGContext::NewObject_AnyThread<UPCGTexture2DArrayData>(Context);
			}

			FPCGTaggedData& OutTaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
			OutTaggedData.Data = OutputInfo.OutputData;
			OutTaggedData.Pin = PCGPinConstants::DefaultOutputLabel;
		}

		Context->bCreatedOutputData = true;
	}

	bool bAllInitialized = true;

	for (FPCGDownsampleTextureContext::FTextureOutputInfo& OutputInfo : Context->TextureOutputInfo)
	{
		if (UPCGTextureData* TextureData = Cast<UPCGTextureData>(OutputInfo.OutputData))
		{
			bAllInitialized &= TextureData->Initialize(OutputInfo.OutputHandle, /*InTextureIndex=*/0, OutputInfo.Transform);
		}
		else if (UPCGTexture2DArrayData* TextureArrayData = Cast<UPCGTexture2DArrayData>(OutputInfo.OutputData))
		{
			FPCGTexture2DArrayDataInitParams Params;
			Params.Transform = OutputInfo.Transform;
			Params.Filter = OutputInfo.Filter;

			bAllInitialized &= TextureArrayData->Initialize(OutputInfo.OutputHandle, Params);
		}
	}

	return bAllInitialized;
}

#undef LOCTEXT_NAMESPACE
