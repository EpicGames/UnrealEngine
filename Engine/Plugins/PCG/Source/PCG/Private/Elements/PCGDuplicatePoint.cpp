// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDuplicatePoint.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadataAccessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDuplicatePoint)

#define LOCTEXT_NAMESPACE "PCGDuplicatePointElement"

FPCGElementPtr UPCGDuplicatePointSettings::CreateElement() const
{
	return MakeShared<FPCGDuplicatePointElement>();
}

bool FPCGDuplicatePointElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDuplicatePointElement::Execute);
	check(Context);

	const UPCGDuplicatePointSettings* Settings = Context->GetInputSettings<UPCGDuplicatePointSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	if (Settings->Iterations <= 0)
	{
		PCGE_LOG(Verbose, GraphAndLog, LOCTEXT("InvalidNumberOfIterations", "The number of interations must be at least 1."));
		return true;
	}

	const int Iterations = Settings->Iterations;

	const FVector Direction(FMath::Clamp(Settings->Direction.X, -1.0, 1.0), FMath::Clamp(Settings->Direction.Y, -1.0, 1.0), FMath::Clamp(Settings->Direction.Z, -1.0, 1.0));

	for (int i = 0; i < Inputs.Num(); ++i)
	{
		const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(Inputs[i].Data);
		if (!InputPointData)
		{
			PCGE_LOG(Warning, GraphAndLog, LOCTEXT("InvalidInputPointData", "The input is not point data, skipped."));
			continue;
		}

		// Determines whether or not to include the source point in data
		const bool bKeepSourcePoint = Settings->bOutputSourcePoint;
		const int DuplicatesPerPoint = Iterations + (bKeepSourcePoint ? 1 : 0);
		const int NumIterations = DuplicatesPerPoint * InputPointData->GetNumPoints();
		const int FirstDuplicateIndex = Settings->bOutputSourcePoint ? 0 : 1;

		FPCGTaggedData& Output = Outputs.Add_GetRef(Inputs[i]);
		if (InputPointData->GetNumPoints() == 0)
		{
			Output.Data = InputPointData;
			continue;
		}

		UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(Context);
		
		FPCGInitializeFromDataParams InitializeFromDataParams(InputPointData);
		InitializeFromDataParams.bInheritSpatialData = false;

		OutPointData->InitializeFromDataWithParams(InitializeFromDataParams);
		Output.Data = OutPointData;

		auto InitializeFunc = [NumIterations, OutPointData, InputPointData]()
		{
			OutPointData->SetNumPoints(NumIterations, /*bInitializeValues=*/false);
			OutPointData->AllocateProperties(InputPointData->GetAllocatedProperties() | EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed);
			OutPointData->CopyUnallocatedPropertiesFrom(InputPointData);
		};

		const FTransform& SourceDuplicateTransform = Settings->PointTransform;

		if (Settings->bDirectionAppliedInRelativeSpace)
		{
			auto ProcessRangeFunc = [SourceDuplicateTransform, DuplicatesPerPoint, bKeepSourcePoint, &Direction, InputPointData, OutPointData](const int32 StartReadIndex, const int32 StartWriteIndex, const int32 Count)
			{
				const FConstPCGPointValueRanges InRanges(InputPointData);
				FPCGPointValueRanges OutRanges(OutPointData, /*bAllocate=*/false);

				int32 NumWritten = 0;

				for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
				{
					const FTransform DuplicateAxisTransform = FTransform((InRanges.BoundsMaxRange[ReadIndex] - InRanges.BoundsMinRange[ReadIndex]) * Direction);
					const FTransform DuplicateTransform = DuplicateAxisTransform * SourceDuplicateTransform;
					FTransform CurrentTransform = InRanges.TransformRange[ReadIndex];

					const int32 BaseWriteIndex = (StartWriteIndex * DuplicatesPerPoint) + NumWritten;
					int32 LocalWriteIndex = 0;

					if (bKeepSourcePoint)
					{
						const int32 WriteIndex = BaseWriteIndex + LocalWriteIndex;
						OutRanges.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);
						++LocalWriteIndex;
					}

					while (LocalWriteIndex < DuplicatesPerPoint)
					{
						const int32 WriteIndex = BaseWriteIndex + LocalWriteIndex;
						OutRanges.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);
						
						CurrentTransform = DuplicateTransform * CurrentTransform;
						OutRanges.TransformRange[WriteIndex] = CurrentTransform;
						OutRanges.SeedRange[WriteIndex] = PCGHelpers::ComputeSeedFromPosition(CurrentTransform.GetLocation());

						++LocalWriteIndex;
					}

					NumWritten += LocalWriteIndex;
				}
				
				check(NumWritten == Count * DuplicatesPerPoint);
				return Count;
			};

			FPCGAsync::AsyncProcessingOneToOneRangeEx(&Context->AsyncState, InputPointData->GetNumPoints(), InitializeFunc, ProcessRangeFunc, /*bEnableTimeSlicing=*/false);
		}
		else
		{
			auto ProcessRangeFunc = [SourceDuplicateTransform, &Direction, InputPointData, OutPointData, FirstDuplicateIndex](const int32 StartReadIndex, const int32 StartWriteIndex, const int32 Count)
			{
				const FConstPCGPointValueRanges InRanges(InputPointData);
				FPCGPointValueRanges OutRanges(OutPointData, /*bAllocate=*/false);
				const FConstPCGPointValueRanges ConstOutRanges(OutPointData);

				int32 NumWritten = 0;
				const int32 NumInputPoints = InputPointData->GetNumPoints();

				for (int32 GlobalReadIndex = StartReadIndex; GlobalReadIndex < StartReadIndex + Count; ++GlobalReadIndex)
				{
					const int32 ReadIndex = GlobalReadIndex % NumInputPoints;
					const int32 WriteIndex = StartWriteIndex + NumWritten;
					const int DuplicateIndex = FirstDuplicateIndex + GlobalReadIndex / NumInputPoints;

					OutRanges.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);

					if (DuplicateIndex != 0)
					{
						const FVector DuplicateLocationOffset = ((ConstOutRanges.BoundsMaxRange[WriteIndex] - ConstOutRanges.BoundsMinRange[WriteIndex]) * Direction + SourceDuplicateTransform.GetLocation()) * DuplicateIndex;
						const FRotator DuplicateRotationOffset = SourceDuplicateTransform.Rotator() * DuplicateIndex;
						const FVector DuplicateScaleMultiplier = FVector(
							FMath::Pow(SourceDuplicateTransform.GetScale3D().X, DuplicateIndex),
							FMath::Pow(SourceDuplicateTransform.GetScale3D().Y, DuplicateIndex),
							FMath::Pow(SourceDuplicateTransform.GetScale3D().Z, DuplicateIndex));

						OutRanges.TransformRange[WriteIndex] = FTransform(DuplicateRotationOffset, DuplicateLocationOffset, DuplicateScaleMultiplier) * InRanges.TransformRange[ReadIndex];
						OutRanges.SeedRange[WriteIndex] = PCGHelpers::ComputeSeedFromPosition(OutRanges.TransformRange[WriteIndex].GetLocation());
					}

					++NumWritten;
				}
				
				check(NumWritten == Count);
				return Count;
			};

			FPCGAsync::AsyncProcessingOneToOneRangeEx(&Context->AsyncState, NumIterations, InitializeFunc, ProcessRangeFunc, /*bEnableTimeSlicing=*/false);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
