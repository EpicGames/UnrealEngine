// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGEditPoints.h"

#include "PCGContext.h"
#include "PCGPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEditPoints)

FPCGElementPtr UPCGEditPointsSettings::CreateElement() const
{
	return MakeShared<FPCGEditPointsElement>();
}

bool FPCGEditPointsElement::ExecuteInternal(FPCGContext* Context) const
{
	check(Context);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	Context->OutputData.TaggedData.Append(Inputs);

	return true;
}
