// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGElevationIsolines.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGIntersectionData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGSurfaceData.h"
#include "Elements/PCGSurfaceSampler.h"
#include "Helpers/PCGAsync.h"
#include "SpatialAlgo/PCGMarchingSquares.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGElevationIsolines)

#define LOCTEXT_NAMESPACE "PCGElevationIsolinesElement"

#if WITH_EDITOR
FName UPCGElevationIsolinesSettings::GetDefaultNodeName() const
{
	return FName(TEXT("ElevationIsolines"));
}

FText UPCGElevationIsolinesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Elevation Isolines");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGElevationIsolinesSettings::CreateElement() const
{
	return MakeShared<FPCGElevationIsolinesElement>();
}

TArray<FPCGPinProperties> UPCGElevationIsolinesSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	FPCGPinProperties& InputPin = Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Surface);
	InputPin.SetRequiredPin();

	Properties.Emplace(TEXT("BoundingShape"), EPCGDataType::Spatial);

	return Properties;
}

TArray<FPCGPinProperties> UPCGElevationIsolinesSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, bOutputAsSpline ? EPCGDataType::Spline : EPCGDataType::Point);
	return Properties;
}

bool FPCGElevationIsolinesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGElevationIsolinesElement::Execute);

	check(InContext);

	const UPCGElevationIsolinesSettings* Settings = InContext->GetInputSettings<UPCGElevationIsolinesSettings>();
	check(Settings);

	if (FMath::IsNearlyZero(Settings->ElevationIncrement) || Settings->ElevationEnd < Settings->ElevationStart)
	{
		return true;
	}

	bool bOutUnionDataCreated = false;
	const UPCGSpatialData* BoundingShape = InContext->InputData.GetSpatialUnionOfInputsByPin(InContext, TEXT("BoundingShape"), bOutUnionDataCreated);

	// Fallback to getting bounds from actor
	if (!BoundingShape && InContext->ExecutionSource.IsValid())
	{
		check(!bOutUnionDataCreated);
		BoundingShape = Cast<UPCGSpatialData>(InContext->ExecutionSource->GetExecutionState().GetSelfData());
	}

	if (!BoundingShape)
	{
		return true;
	}

	const FBox Bounds = BoundingShape->GetBounds();
	const FVector CellSize{ FMath::Abs(Settings->Resolution), FMath::Abs(Settings->Resolution), 1.0 };
	const FVector HalfCellSize = CellSize / 2.0;
	const FBox CellBounds{ -HalfCellSize, HalfCellSize };
	const FVector Origin{ Bounds.Min.X, Bounds.Min.Y, Bounds.Max.Z };
	const int64 CellMinX = FMath::CeilToInt((Bounds.Min.X) / CellSize.X);
	const int64 CellMaxX = FMath::FloorToInt((Bounds.Max.X) / CellSize.X);
	const int64 CellMinY = FMath::CeilToInt((Bounds.Min.Y) / CellSize.Y);
	const int64 CellMaxY = FMath::FloorToInt((Bounds.Max.Y) / CellSize.Y);
	const int64 CellXCount = CellMaxX - CellMinX;
	const int64 CellYCount = CellMaxY - CellMinY;
	const int64 CellCount64 = CellXCount * CellYCount;

	using FHeightRotation = TTuple<double, FQuat>;

	if (PCGFeatureSwitches::CVarCheckSamplerMemory.GetValueOnAnyThread()
		&& PCGFeatureSwitches::Helpers::GetAvailableMemoryForSamplers() < (sizeof(FHeightRotation) * CellCount64))
	{
		PCGLog::LogErrorOnGraph(FText::Format((LOCTEXT("TooManyPoints", "Skipped - tried to sample too many points ({0}).\nAdjust the bounds, resolution or 'pcg.SamplerMemoryThreshold' if needed.")), CellCount64), InContext);
		return true;
	}

	const int32 CellCount = static_cast<int32>(CellCount64);

	for (const FPCGTaggedData& InputData : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		const UPCGSurfaceData* SurfaceData = Cast<UPCGSurfaceData>(InputData.Data);
		if (!SurfaceData)
		{
			continue;
		}

		TArray<FHeightRotation> HeightmapWithRotation;
		HeightmapWithRotation.SetNumUninitialized(CellCount);

		FPCGProjectionParams ProjectionParams{};
		ProjectionParams.bProjectPositions = true;
		ProjectionParams.bProjectRotations = Settings->bProjectSurfaceNormal;

		// TODO: Time-slice
		FPCGAsync::AsyncProcessingOneToOneEx(&InContext->AsyncState, HeightmapWithRotation.Num(), []() {}, [CellYCount, Settings, Origin, CellSize, SurfaceData, &HeightmapWithRotation, &ProjectionParams](int32 ReadIndex, int32 WriteIndex)
		{
			const int32 CellX = ReadIndex / CellYCount;
			const int32 CellY = ReadIndex % CellYCount;

			FVector Cell{ Origin.X + CellSize.X * CellX, Origin.Y + CellSize.Y * CellY, Origin.Z };

			FPCGPoint Projected;
			bool bSuccess = SurfaceData->ProjectPoint(FTransform(Cell), FBox(EForceInit::ForceInit), ProjectionParams, Projected, nullptr);
			HeightmapWithRotation[WriteIndex].Get<0>() = bSuccess ? Projected.Transform.GetLocation().Z : std::numeric_limits<double>::min();

			if (ProjectionParams.bProjectRotations)
			{
				HeightmapWithRotation[WriteIndex].Get<1>() = bSuccess ? Projected.Transform.GetRotation() : FQuat::Identity;
			}

		}, /*bEnableTimeSlicing=*/false);

		auto GetElevation = [CellYCount, &HeightmapWithRotation](int32 X, int32 Y) -> double
		{
			return HeightmapWithRotation[X * CellYCount + Y].Get<0>();
		};

		// TODO: Async + Support non-Z up cases.
		// Run the Marching Squares algorithm on each isoline we want.
		int32 ElevationCount = 0;
		for (double Elevation = Settings->ElevationStart; Elevation < Settings->ElevationEnd; Elevation += Settings->ElevationIncrement)
		{
			for (const PCGSpatialAlgo::FPCGMarchingSquareResult& Result : PCGSpatialAlgo::MarchingSquares(CellXCount, CellYCount, Elevation, GetElevation, /*bUseLinearInterpolation=*/true))
			{
				// We can't do anything with less than 2 points.
				if (Result.LinkedGridCoordinates.Num() < 2)
				{
					continue;
				}

				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGElevationIsolinesElement::Execute::CreateOutputData);

				auto TransformFromGridCoordinate = [&Origin, &CellSize, Elevation](const FVector2D& GridCoordinates) -> FTransform
				{
					return FTransform(FVector(Origin.X + CellSize.X * GridCoordinates.X, Origin.Y + CellSize.Y * GridCoordinates.Y, Elevation));
				};

				// Each point coordinate will have an integer coordinate and a non-integer one. (That's how Marching Square is building it).
				// So to interpolate the normals, we find the two corner of the cell where the edge is.
				// Floor and Ceil will return the same value if X/Y is an integer. So depending on the fractional value of X or Y, it will give us
				// the two wanted corners. Then we do a Slerp with this ratio between the 2 rotations computed before (when we did the projection).
				auto ReconstructPointRotation = [CellYCount, HeightmapWithRotation](const FVector2D& GridCoordinates) -> FQuat
				{
					const int32 Index1 = FMath::FloorToInt(GridCoordinates.X) * CellYCount + FMath::FloorToInt(GridCoordinates.Y);
					const int32 Index2 = FMath::CeilToInt(GridCoordinates.X) * CellYCount + FMath::CeilToInt(GridCoordinates.Y);

					FQuat Rotation = HeightmapWithRotation[Index1].Get<1>();

					if (Index1 != Index2)
					{
						const double RatioX = FMath::Frac(GridCoordinates.X);
						const double RatioY = FMath::Frac(GridCoordinates.Y);
						const double Ratio = FMath::IsNearlyZero(RatioX) ? RatioY : RatioX;

						Rotation = FQuat::Slerp(Rotation, HeightmapWithRotation[Index2].Get<1>(), Ratio);
					}

					return Rotation;
				};

				// Because we can have points that are very close to each other (the edge is in the corner of the cell), we merge them (by doing an average in the middle)
				// Threshold = (Resolution * 0.5)^2
				const double Threshold = (Settings->Resolution * Settings->Resolution * 0.25);
				auto MergePointsIfTooClose = [Threshold](FTransform& PreviousTransform, const FTransform& CurrentTransform) -> bool
				{
					if (FVector::DistSquared(PreviousTransform.GetLocation(), CurrentTransform.GetLocation()) < Threshold)
					{
						FQuat LerpQuat = FQuat::Slerp(PreviousTransform.GetRotation(), CurrentTransform.GetRotation(), 0.5);
						PreviousTransform.SetRotation(LerpQuat);
						PreviousTransform.LerpTranslationScale3D(PreviousTransform, CurrentTransform, ScalarRegister(0.5));
						return true;
					}
					else
					{
						return false;
					}
				};

				// Process all the transforms
				TArray<FTransform> FinalTransforms;
				FinalTransforms.Reserve(Result.LinkedGridCoordinates.Num());

				for (int32 Index = 0; Index < Result.LinkedGridCoordinates.Num(); ++Index)
				{
					const FVector2D GridCoordinates = Result.LinkedGridCoordinates[Index];
					FTransform Transform = TransformFromGridCoordinate(GridCoordinates);

					// Reconstruct the rotation from the normal if needed
					if (Settings->bProjectSurfaceNormal)
					{
						Transform.SetRotation(ReconstructPointRotation(GridCoordinates));
					}

					// If we have more than one transform, try to merge with the previous point if they are too close.
					// The result of the merge will be in the first transform passed as parameter (update in place).
					// INFO: If more than 2 points are close together, it will first average the first two then average the next point (and again).
					// It's unlikely to happen though.
					if (!FinalTransforms.IsEmpty())
					{
						FTransform& PreviousTransform = FinalTransforms.Last();
						const bool bHasMergedWithPrevious = MergePointsIfTooClose(PreviousTransform, Transform);
						bool bSkipPoint = bHasMergedWithPrevious;

						// Also if we are at the end, and we are a closed loop, try to merge with the first transform.
						if (Result.bClosed && Index == Result.LinkedGridCoordinates.Num() - 1)
						{
							const FTransform& CurrentTransform = bHasMergedWithPrevious ? PreviousTransform : Transform;
							if (MergePointsIfTooClose(FinalTransforms[0], CurrentTransform))
							{
								bSkipPoint = true;

								// If we merged the last and first point, check if we merged it with the penultimate one, if so, pop it.
								if (bHasMergedWithPrevious)
								{
									FinalTransforms.Pop();
								}
							}
						}

						if (bSkipPoint)
						{
							continue;
						}
					}

					FinalTransforms.Add(std::move(Transform));
				}

				// Make sure that after the merge, we still have more than 2 points, otherwise, just discard this line.
				if (FinalTransforms.Num() < 2)
				{
					continue;
				}

				// And finally create the data.
				FPCGTaggedData& OutputData = InContext->OutputData.TaggedData.Emplace_GetRef(InputData);
				OutputData.Pin = PCGPinConstants::DefaultOutputLabel;

				if (Settings->bAddTagOnOutputForSameElevation)
				{
					OutputData.Tags.Add(FString::Printf(TEXT("ElevationIdx_%d"), ElevationCount));
				}

				if (Settings->bOutputAsSpline)
				{
					TArray<FSplinePoint> SplinePoints;
					SplinePoints.Reserve(FinalTransforms.Num());

					for (const FTransform& Transform : FinalTransforms)
					{
						// Control points will be locally at 0 in Z.
						// It is to prevent any approximation errors while interpolating on the spline.
						// The spline itself will be a Z = Elevation.
						FVector Location = Transform.GetLocation();
						Location.Z = 0;

						SplinePoints.Emplace(static_cast<float>(SplinePoints.Num()),
							Location,
							FVector::ZeroVector,
							FVector::ZeroVector,
							Transform.GetRotation().Rotator(),
							Transform.GetScale3D(),
							Settings->bLinearSpline ? ESplinePointType::Linear : ESplinePointType::Curve);
					}

					UPCGSplineData* OutSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(InContext);
					OutSplineData->Initialize(SplinePoints, Result.bClosed, FTransform(FVector::UpVector * Elevation));
					OutputData.Data = OutSplineData;
				}
				else
				{
					UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(InContext);
					OutPointData->SetNumPoints(FinalTransforms.Num());
					OutPointData->SetBoundsMin(CellBounds.Min);
					OutPointData->SetBoundsMax(CellBounds.Max);
					OutPointData->AllocateProperties(EPCGPointNativeProperties::Transform);
					
					TPCGValueRange<FTransform> TransformRange = OutPointData->GetTransformValueRange(/*bAllocate=*/false);
					for (int32 Index = 0; Index < FinalTransforms.Num(); ++Index)
					{
						TransformRange[Index] = FinalTransforms[Index];
					}

					OutputData.Data = OutPointData;
				}
			}

			++ElevationCount;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
