// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGExecuteBlueprint.h"

#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Elements/Blueprint/PCGBlueprintDeprecatedElement.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "Algo/Transform.h"
#include "Engine/Blueprint.h"
#include "UObject/Package.h"
#include "UObject/Stack.h"
#include "UObject/UObjectThreadContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGExecuteBlueprint)

#define LOCTEXT_NAMESPACE "PCGExecuteBlueprint"

FPCGContextBlueprintScope::FPCGContextBlueprintScope(FPCGContext* InContext)
	: Context(InContext)
{
	if (Context)
	{
		// Make sure handle is not copied when calling ExecuteWithContext which creates a FPCGContext copy on the stack
		ContextHandle = Context->Handle;
		Context->Handle = nullptr;

		// Similarly we don't want to register to GC-protection on release.
		GraphExecutor = Context->GraphExecutor;
		Context->GraphExecutor = nullptr;
	}
}

FPCGContextBlueprintScope::~FPCGContextBlueprintScope()
{
	if (Context)
	{
		Context->Handle = ContextHandle;
		Context->GraphExecutor = GraphExecutor;
	}
}

UPCGBlueprintSettings::UPCGBlueprintSettings()
{
#if WITH_EDITORONLY_DATA
	bExposeToLibrary = HasAnyFlags(RF_ClassDefaultObject);
#endif
}

void UPCGBlueprintSettings::SetupBlueprintEvent()
{
#if WITH_EDITOR
	if (BlueprintElementType)
	{
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UPCGBlueprintSettings::OnObjectsReplaced);
	}
#endif
}

void UPCGBlueprintSettings::TeardownBlueprintEvent()
{
#if WITH_EDITOR
	if (BlueprintElementType)
	{
		FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	}
#endif
}

void UPCGBlueprintSettings::SetupBlueprintElementEvent()
{
#if WITH_EDITOR
	if (BlueprintElementInstance)
	{
		BlueprintElementInstance->OnBlueprintElementChangedDelegate.AddUObject(this, &UPCGBlueprintSettings::OnBlueprintElementChanged);
	}
#endif
}

void UPCGBlueprintSettings::TeardownBlueprintElementEvent()
{
#if WITH_EDITOR
	if (BlueprintElementInstance)
	{
		BlueprintElementInstance->OnBlueprintElementChangedDelegate.RemoveAll(this);
	}
#endif
}

void UPCGBlueprintSettings::PostLoad()
{
	Super::PostLoad();

	SetupBlueprintEvent();

	if (!BlueprintElementInstance)
	{
		RefreshBlueprintElement();
	}
	else
	{
		SetupBlueprintElementEvent();
	}

	if (BlueprintElementInstance)
	{
		BlueprintElementInstance->ConditionalPostLoad();
		BlueprintElementInstance->SetFlags(RF_Transactional);
	}
}

void UPCGBlueprintSettings::BeginDestroy()
{
	TeardownBlueprintElementEvent();
	TeardownBlueprintEvent();

	Super::BeginDestroy();
}

#if WITH_EDITOR
void UPCGBlueprintSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange)
	{
		if (PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGBlueprintSettings, BlueprintElementType))
		{
			TeardownBlueprintEvent();
		}
	}
}

void UPCGBlueprintSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGBlueprintSettings, BlueprintElementType))
		{
			SetupBlueprintEvent();
		}
	}

	if (!BlueprintElementInstance || BlueprintElementInstance->GetClass() != BlueprintElementType)
	{
		RefreshBlueprintElement();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPCGBlueprintSettings::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances)
{
	if (!BlueprintElementInstance)
	{
		return;
	}

	if (UObject* NewObject = InOldToNewInstances.FindRef(BlueprintElementInstance))
	{
		// When the blueprint changes, the element gets recreated, so we must rewire it here.
		TeardownBlueprintElementEvent();
		BlueprintElementInstance = Cast<UPCGBlueprintBaseElement>(NewObject);
		SetupBlueprintElementEvent();

		DirtyCache();

		if (BlueprintElementInstance)
		{
			BlueprintElementInstance->Initialize();
		}

		// Also, reconstruct overrides
		InitializeCachedOverridableParams(/*bReset=*/true);

		OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Settings);
	}
}

void UPCGBlueprintSettings::OnBlueprintElementChanged(UPCGBlueprintBaseElement* InElement)
{
	if (InElement == BlueprintElementInstance)
	{
		// When a data dependency is changed, this means we have to dirty the cache, otherwise it will not register as a change.
		DirtyCache();

		OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Settings);
	}
}
#endif // WITH_EDITOR

// deprecated
void UPCGBlueprintSettings::SetElementType(TSubclassOf<UPCGBlueprintElement> InElementType, UPCGBlueprintElement*& ElementInstance)
{
	if (!BlueprintElementInstance || InElementType != BlueprintElementType)
	{
		if (InElementType != BlueprintElementType)
		{
			TeardownBlueprintEvent();
			BlueprintElementType = InElementType;
			SetupBlueprintEvent();
		}

		RefreshBlueprintElement();
	}

	ElementInstance = Cast<UPCGBlueprintElement>(BlueprintElementInstance);
}

void UPCGBlueprintSettings::SetBlueprintElementType(TSubclassOf<UPCGBlueprintBaseElement> InElementType, UPCGBlueprintBaseElement*& ElementInstance)
{
	if (!BlueprintElementInstance || InElementType != BlueprintElementType)
	{
		if (InElementType != BlueprintElementType)
		{
			TeardownBlueprintEvent();
			BlueprintElementType = InElementType;
			SetupBlueprintEvent();
		}

		RefreshBlueprintElement();
	}

	ElementInstance = BlueprintElementInstance;
}

void UPCGBlueprintSettings::RefreshBlueprintElement()
{
	TeardownBlueprintElementEvent();

	if (BlueprintElementType)
	{
		BlueprintElementInstance = NewObject<UPCGBlueprintBaseElement>(this, BlueprintElementType, NAME_None, RF_Transactional);
		BlueprintElementInstance->Initialize();
		SetupBlueprintElementEvent();
	}
	else
	{
		BlueprintElementInstance = nullptr;
	}

	// Also, reconstruct overrides
	InitializeCachedOverridableParams(/*bReset=*/true);
}

bool UPCGBlueprintSettings::HasDynamicPins() const
{
	return BlueprintElementInstance ? BlueprintElementInstance->bHasDynamicPins : UPCGSettings::HasDynamicPins();
}

FPCGDataTypeIdentifier UPCGBlueprintSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	check(InPin);

	// We can't call a BP function while we are PostLoading, in that case (or if we don't have an instance or it's not dynamic) just return the pin allowed types.
	if (BlueprintElementInstance && BlueprintElementInstance->bHasDynamicPins && !FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		FPCGDataTypeIdentifier Result{ static_cast<EPCGDataType>(BlueprintElementInstance->DynamicPinTypesOverride(this, InPin)) };
		return Result.IsValid() ? Result : InPin->Properties.AllowedTypes;
	}
	else
	{
		return InPin->Properties.AllowedTypes;
	}
}

#if WITH_EDITOR
FLinearColor UPCGBlueprintSettings::GetNodeTitleColor() const
{
	if (BlueprintElementInstance && BlueprintElementInstance->NodeColorOverride() != FLinearColor::White)
	{
		return BlueprintElementInstance->NodeColorOverride();
	}
	else
	{
		return Super::GetNodeTitleColor();
	}
}

EPCGSettingsType UPCGBlueprintSettings::GetType() const
{
	if (BlueprintElementInstance)
	{
		return BlueprintElementInstance->NodeTypeOverride();
	}
	else
	{
		return EPCGSettingsType::Blueprint;
	}
}

void UPCGBlueprintSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	for (const FName& Tag : TrackedActorTags)
	{
		OutKeysToSettings.FindOrAdd(FPCGSelectionKey(Tag)).Emplace(this, bTrackActorsOnlyWithinBounds);
	}
}

UObject* UPCGBlueprintSettings::GetJumpTargetForDoubleClick() const
{
	if (BlueprintElementType)
	{
		return BlueprintElementType->ClassGeneratedBy;
	}

	return Super::GetJumpTargetForDoubleClick();
}

void UPCGBlueprintSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	// Rename first found 'Param' or 'Params' pin to 'Overrides' which helps to ensure legacy params pins will retain incident edges.
	for (TObjectPtr<UPCGPin>& InputPin : InputPins)
	{
		if (InputPin && InputPin->Properties.AllowedTypes == EPCGDataType::Param &&
			(InputPin->Properties.Label == FName(TEXT("Params")) || InputPin->Properties.Label == FName(TEXT("Param"))))
		{
			InputPin->Properties.Label = PCGPinConstants::DefaultParamsLabel;
			break;
		}
	}
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGBlueprintSettings::GetPreconfiguredInfo() const
{
	return BlueprintElementInstance ? BlueprintElementInstance->PreconfiguredInfo : TArray<FPCGPreConfiguredSettingsInfo>();
}

bool UPCGBlueprintSettings::OnlyExposePreconfiguredSettings() const
{
	return BlueprintElementInstance && BlueprintElementInstance->bOnlyExposePreconfiguredSettings;
}
#endif // WITH_EDITOR

void UPCGBlueprintSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& InPreconfiguredInfo)
{
	if (BlueprintElementInstance)
	{
		BlueprintElementInstance->ApplyPreconfiguredSettings(InPreconfiguredInfo);
	}
}

FString UPCGBlueprintSettings::GetAdditionalTitleInformation() const
{
	// We can't call a BP function while we are PostLoading, in that case (or if we don't have an instance or it's not dynamic) just return the default title.
	if (BlueprintElementInstance && !FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		const FName NodeTitleOverride = BlueprintElementInstance->NodeTitleOverride();
		if (NodeTitleOverride != NAME_None)
		{
			return BlueprintElementInstance->NodeTitleOverride().ToString();
		}
	}

	{
		FString ElementName;

#if WITH_EDITOR
		ElementName = (BlueprintElementType && BlueprintElementType->ClassGeneratedBy) ? BlueprintElementType->ClassGeneratedBy->GetName() : Super::GetAdditionalTitleInformation();
#else
		ElementName = BlueprintElementType ? BlueprintElementType->GetName() : Super::GetAdditionalTitleInformation();
#endif

		// Normalize node name only if not explicitly set in the NodeTitleOverride call
		if (ElementName.IsEmpty())
		{
			return LOCTEXT("MissingBlueprint", "Missing Blueprint").ToString();
		}
		else
		{
			return FName::NameToDisplayString(ElementName, false);
		}
	}
}

TArray<FPCGPinProperties> UPCGBlueprintSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	if (!BlueprintElementInstance || BlueprintElementInstance->bHasDefaultInPin)
	{
		// Here we do not want the base class implementation as it makes the input pin required.
		PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any);
	}

	if (BlueprintElementInstance)
	{
		PinProperties.Append(BlueprintElementInstance->CustomInputPins);
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGBlueprintSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	if (BlueprintElementInstance)
	{
		if (BlueprintElementInstance->bHasDefaultOutPin)
		{
			// Note: we do not use the default base class pin here, as a blueprint node might return anything
			PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
		}

		PinProperties.Append(BlueprintElementInstance->CustomOutputPins);
	}
	else
	{
		PinProperties = Super::OutputPinProperties();
	}

	return PinProperties;
}

FPCGElementPtr UPCGBlueprintSettings::CreateElement() const
{
	return MakeShared<FPCGExecuteBlueprintElement>();
}

#if WITH_EDITOR
TArray<FPCGSettingsOverridableParam> UPCGBlueprintSettings::GatherOverridableParams() const
{
	TArray<FPCGSettingsOverridableParam> OverridableParams = Super::GatherOverridableParams();

	if (UClass* BPClass = *BlueprintElementType)
	{
		PCGSettingsHelpers::FPCGGetAllOverridableParamsConfig Config;
		Config.bExcludeSuperProperties = true;
		Config.bExtractArrays = true;
		Config.MaxContainersNum = 1; // Do not support more than 1 container deep.
		Config.ShouldKeepPropertyFunc = [](const FProperty* InProperty, int32) { return InProperty && !InProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance | CPF_EditConst | CPF_BlueprintReadOnly); };
		OverridableParams.Append(PCGSettingsHelpers::GetAllOverridableParams(BPClass, Config));
	}

	return OverridableParams;
}
#endif // WITH_EDITOR

void UPCGBlueprintSettings::FixingOverridableParamPropertyClass(FPCGSettingsOverridableParam& Param) const
{
	bool bFound = false;

	if (!Param.PropertiesNames.IsEmpty())
	{
		UClass* BPClass = *BlueprintElementType;
		if (BPClass && BPClass->FindPropertyByName(Param.PropertiesNames[0]))
		{
			Param.PropertyClass = BPClass;
			bFound = true;
		}
	}

	if (!bFound)
	{
		Super::FixingOverridableParamPropertyClass(Param);
	}
}

bool FPCGExecuteBlueprintElement::ExecuteInternal(FPCGContext* InContext) const
{
	check(InContext);
	FPCGBlueprintExecutionContext* Context = static_cast<FPCGBlueprintExecutionContext*>(InContext);

	if (Context->BlueprintElementInstance)
	{
		UClass* BPClass = Context->BlueprintElementInstance->GetClass();

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FPCGExecuteBlueprintElement::Execute (%s)"), BPClass ? *BPClass->GetFName().ToString() : TEXT("")));

#if WITH_EDITOR
		/** Check if the blueprint has been successfully compiled */
		if (UBlueprint* Blueprint = Cast<UBlueprint>(BPClass ? BPClass->ClassGeneratedBy : nullptr))
		{
			if (Blueprint->Status == BS_Error)
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("BPNotCompiled", "Blueprint cannot be executed since '{0}' is not properly compiled"),
					FText::FromName(Blueprint->GetFName())));
				return true;
			}
		}
#endif

		// Log info on inputs
		for (int32 InputIndex = 0; InputIndex < Context->InputData.TaggedData.Num(); ++InputIndex)
		{
			const FPCGTaggedData& Input = Context->InputData.TaggedData[InputIndex];
			if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Input.Data))
			{
				PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("InputPointInfo", "Input {0} has {1} points"), InputIndex, PointData->GetNumPoints()));
			}
		}

		// Note that the context is actually sliced so there should never be any members in the BP element context that are visible/accessible from blueprint
		/** Finally, execute the actual blueprint */
		Context->BlueprintElementInstance->SetCurrentContext(Context);

		{
			// Support deprecated ExecuteWithContext
			UPCGBlueprintElement* DeprecatedElement = Cast<UPCGBlueprintElement>(Context->BlueprintElementInstance);

			// When running outside of main thread make sure GC can't run (BP Nodes might create objects which can't happen while GC runs)
			if (!IsInGameThread())
			{
				ensure(!Context->AsyncState.bIsRunningOnMainThread);
				FGCScopeGuard Scope;

				if (DeprecatedElement)
				{
					FPCGContextBlueprintScope BlueprintScope(Context);
					DeprecatedElement->ExecuteWithContext(*Context, Context->InputData, Context->OutputData);
				}
				else
				{
					Context->BlueprintElementInstance->Execute(Context->InputData, Context->OutputData);
				}
			}
			else
			{
				if (DeprecatedElement)
				{
					FPCGContextBlueprintScope BlueprintScope(Context);
					DeprecatedElement->ExecuteWithContext(*Context, Context->InputData, Context->OutputData);
				}
				else
				{
					Context->BlueprintElementInstance->Execute(Context->InputData, Context->OutputData);
				}
			}
		}

		Context->BlueprintElementInstance->SetCurrentContext(nullptr);
	}
	else
	{
		// Nothing to do but forward data
		Context->OutputData = Context->InputData;
	}

	return true;
}

void FPCGExecuteBlueprintElement::PostExecuteInternal(FPCGContext* InContext) const
{
	check(InContext);
	FPCGBlueprintExecutionContext* Context = static_cast<FPCGBlueprintExecutionContext*>(InContext);

	if (Context->BlueprintElementInstance)
	{
		check(IsInGameThread());

		// Build a list of InputObjects so that we don't remove Async flags from Outputs that come from the Input as it isn't our responsability
		TSet<TObjectPtr<const UObject>> InputObjects;
		Algo::TransformIf(Context->InputData.TaggedData, InputObjects,
			[](const FPCGTaggedData& TaggedData)
			{
				return TaggedData.Data != nullptr;
			},
			[](const FPCGTaggedData& TaggedData)
			{
				return TaggedData.Data;
			});

		// Log info on outputs
		for (int32 OutputIndex = 0; OutputIndex < Context->OutputData.TaggedData.Num(); ++OutputIndex)
		{
			FPCGTaggedData& Output = Context->OutputData.TaggedData[OutputIndex];

			if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Output.Data))
			{
				PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("OutputPointInfo", "Output {0} has {1} points"), OutputIndex, PointData->GetNumPoints()));
			}

			// Important implementation note:
			// Any data that was created by the user in the blueprint will have that data parented to this blueprint element instance
			// Which will cause issues wrt to reference leaks. We need to fix this here.
			// Note that we will recurse up the outer tree to make sure we catch every case.
			if (Output.Data)
			{
				// Clear Async flags on objects created outside of the main thread and not part of the Context known async objects
				if (Output.Data->HasAnyInternalFlags(EInternalObjectFlags::Async) && !Context->ContainsAsyncObject(Output.Data) && !InputObjects.Contains(Output.Data))
				{
					Output.Data->ClearInternalFlags(EInternalObjectFlags::Async);
					ForEachObjectWithOuter(Output.Data, [](UObject* SubObject) { SubObject->ClearInternalFlags(EInternalObjectFlags::Async); }, EGetObjectsFlags::IncludeNestedObjects);
				}

				auto ReOuterToTransientPackageIfCreatedFromThis = [Context](UObject* InObject)
					{
						bool bHasInstanceAsOuter = false;
						UObject* CurrentObject = InObject;
						while (CurrentObject && !bHasInstanceAsOuter)
						{
							bHasInstanceAsOuter = (CurrentObject->GetOuter() == Context->BlueprintElementInstance);
							CurrentObject = CurrentObject->GetOuter();
						}

						if (bHasInstanceAsOuter)
						{
							InObject->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
						}
					};

				UObject* ThisData = const_cast<UPCGData*>(Output.Data.Get());
				ReOuterToTransientPackageIfCreatedFromThis(ThisData);

				// Similarly, if the metadata on the data inherits from a non-transient data created by this BP instance, it should be reoutered.
				const UPCGMetadata* Metadata = nullptr;
				if (UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(ThisData))
				{
					Metadata = SpatialData->Metadata;
				}
				else if (UPCGParamData* ParamData = Cast<UPCGParamData>(ThisData))
				{
					Metadata = ParamData->Metadata;
				}

				if (Metadata)
				{
					while (Metadata->GetParent())
					{
						UObject* OuterObject = Metadata->GetParent()->GetOuter();
						ReOuterToTransientPackageIfCreatedFromThis(OuterObject);
						Metadata = Metadata->GetParent();
					}
				}

				// If current element (deprecated elements only) does not support base point data then convert its output back to Point Array Data
				if (CVarPCGEnablePointArrayData.GetValueOnAnyThread() && !SupportsBasePointDataInputs(Context))
				{
					if (const UPCGPointData* PointData = Cast<UPCGPointData>(Output.Data))
					{
						Output.Data = PointData->ToPointArrayData(Context);
					}
				}
			}
		}
	}
}

void FPCGBlueprintExecutionContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	if (BlueprintElementInstance)
	{
		Collector.AddReferencedObject(BlueprintElementInstance);
	}
}

FPCGContext* FPCGExecuteBlueprintElement::Initialize(const FPCGInitializeElementParams& InParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGExecuteBlueprintElement::Initialize);
	FPCGBlueprintExecutionContext* Context = new FPCGBlueprintExecutionContext();
	Context->InitFromParams(InParams);

	const UPCGBlueprintSettings* Settings = Context->GetInputSettings<UPCGBlueprintSettings>();
	if (Settings && Settings->BlueprintElementInstance)
	{
		Context->BlueprintElementInstance = CastChecked<UPCGBlueprintBaseElement>(StaticDuplicateObject(Settings->BlueprintElementInstance, GetTransientPackage(), FName()));

#if !WITH_EDITOR
		if (Context->ExecutionSource.IsValid())
		{
			Context->BlueprintElementInstance->SetInstanceWorld(Context->ExecutionSource->GetExecutionState().GetWorld());
		}
#endif
	}
	else
	{
		Context->BlueprintElementInstance = nullptr;
	}

	return Context;
}

bool FPCGExecuteBlueprintElement::IsCacheable(const UPCGSettings* InSettings) const
{
	const UPCGBlueprintSettings* BPSettings = Cast<const UPCGBlueprintSettings>(InSettings);
	if (BPSettings && BPSettings->BlueprintElementInstance)
	{
		return BPSettings->BlueprintElementInstance->IsCacheableOverride();
	}
	else
	{
		return false;
	}
}

bool FPCGExecuteBlueprintElement::ShouldComputeFullOutputDataCrc(FPCGContext* Context) const
{
	check(Context);
	const UPCGBlueprintSettings* BPSettings = Context->GetInputSettings<UPCGBlueprintSettings>();
	if (BPSettings && BPSettings->BlueprintElementInstance)
	{
		return !IsCacheable(BPSettings) && BPSettings->BlueprintElementInstance->bComputeFullDataCrc;
	}
	else
	{
		return false;
	}
}

bool FPCGExecuteBlueprintElement::CanExecuteOnlyOnMainThread(FPCGContext* Context) const
{
	if (!Context)
	{
		return true;
	}

	FPCGBlueprintExecutionContext* BPContext = static_cast<FPCGBlueprintExecutionContext*>(Context);

	// Always execute PostExecute on main thread
	if (Context->CurrentPhase == EPCGExecutionPhase::PostExecute && BPContext->BlueprintElementInstance)
	{
		return true;
	}

	const UPCGBlueprintSettings* BPSettings = Context->GetInputSettings<UPCGBlueprintSettings>();
	if (BPSettings && BPSettings->BlueprintElementInstance)
	{
		return BPSettings->BlueprintElementInstance->bRequiresGameThread;
	}
	else
	{
		return false;
	}
}

bool FPCGExecuteBlueprintElement::SupportsBasePointDataInputs(FPCGContext* Context) const
{
	if (!Context)
	{
		return true;
	}

	// Consider all BP Elements to support base point data unless they are child classes of the deprecated UPCGBlueprintElement
	FPCGBlueprintExecutionContext* BPContext = static_cast<FPCGBlueprintExecutionContext*>(Context);
	const UPCGBlueprintSettings* BPSettings = Context->GetInputSettings<UPCGBlueprintSettings>();
	return !BPSettings->BlueprintElementType->IsChildOf<UPCGBlueprintElement>();
}

#undef LOCTEXT_NAMESPACE
