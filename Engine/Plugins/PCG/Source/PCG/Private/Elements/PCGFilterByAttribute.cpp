// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGFilterByAttribute.h"

#include "PCGContext.h"
#include "PCGData.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Algo/AnyOf.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGFilterByAttribute)

#define LOCTEXT_NAMESPACE "PCGFilterByAttributeElement"

namespace PCGFilterByAttributeElement
{
	const FText ThresholdText = LOCTEXT("Threshold", "Threshold");
	
	bool FilterByExistence(const FPCGTaggedData& Input, const UPCGFilterByAttributeSettings& Settings, const TArray<FString>& Attributes)
	{
		TArray<FString, TInlineAllocator<64>> DataAttributeStrings;

		const UPCGMetadata* Metadata = Input.Data ? Input.Data->ConstMetadata() : nullptr;
		const FPCGMetadataDomainID MetadataDomainID = Input.Data && Metadata ? Input.Data->GetMetadataDomainIDFromSelector(FPCGAttributePropertySelector::CreateAttributeSelector(NAME_None, Settings.MetadataDomain)) : PCGMetadataDomainID::Invalid;
		const FPCGMetadataDomain* MetadataDomain = MetadataDomainID.IsValid() && Metadata ? Metadata->GetConstMetadataDomain(MetadataDomainID) : nullptr;
		
		if (!MetadataDomain)
		{
			return false;
		}

		// All attributes from the list must have a match in order to put the data in the In Filter pin.
		if (Settings.Operator != EPCGStringMatchingOperator::Equal)
		{
			TArray<FName> DataAttributes;
			TArray<EPCGMetadataTypes> DataAttributeTypes;
			
			// Gather all attributes & properties based on data type
			MetadataDomain->GetAttributes(DataAttributes, DataAttributeTypes);
			Algo::Transform(DataAttributes, DataAttributeStrings, [](const FPCGAttributeIdentifier& InAttribute) { return InAttribute.ToString(); });

			if (!Settings.bIgnoreProperties)
			{
				// @pcg_todo We should have a way of querying all properties for a given data.
				
				if (Cast<UPCGBasePointData>(Input.Data))
				{
					if (const UEnum* PointProperties = StaticEnum<EPCGPointProperties>())
					{
						DataAttributeStrings.Reserve(DataAttributeStrings.Num() + PointProperties->NumEnums());
						for (int32 EnumIndex = 0; EnumIndex < PointProperties->NumEnums(); ++EnumIndex)
						{
							DataAttributeStrings.Add(FString(TEXT("$")) + PointProperties->GetNameStringByIndex(EnumIndex));
						}
					}
				}

				if (const UEnum* ExtraProperties = StaticEnum<EPCGExtraProperties>())
				{
					DataAttributeStrings.Reserve(DataAttributeStrings.Num() + ExtraProperties->NumEnums());
					for (int32 EnumIndex = 0; EnumIndex < ExtraProperties->NumEnums(); ++EnumIndex)
					{
						DataAttributeStrings.Add(FString(TEXT("$")) + ExtraProperties->GetNameStringByIndex(EnumIndex));
					}
				}
			}
		}

		bool bInFilter = true;

		for (const FString& Attribute : Attributes)
		{
			FPCGAttributePropertySelector Selector;
			Selector.Update(Attribute);

			// In the case of the equal test, we can test directly if the selector would yield something valid
			if (Settings.Operator == EPCGStringMatchingOperator::Equal)
			{
				TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, Selector, /*bQuiet=*/true);
				if (!Accessor || !Accessor.IsValid())
				{
					bInFilter = false;
					break;
				}
			}
			else
			{
				// Otherwise, it's going to be a bit more complex -
				// First, reconstruct the main property/attribute name from the selector, because it might have removed the $ character.
				const FString AttributeWithNoAccessor = Selector.GetAttributePropertyString(/*bAddQualifier=*/true);
				const FString AttributeAccessors = Selector.GetAttributePropertyAccessorsString(/*bAddLeadingSeparator=*/true);
				
				// Try to find a valid match of AttributeWithNoAccessor against the DataAttributeStrings.
				bool bFoundValidMatch = false;
				for (const FString& DataAttribute : DataAttributeStrings)
				{
					if ((Settings.Operator == EPCGStringMatchingOperator::Substring && !DataAttribute.Contains(AttributeWithNoAccessor)) ||
						(Settings.Operator == EPCGStringMatchingOperator::Matches && !DataAttribute.MatchesWildcard(AttributeWithNoAccessor)))
					{
						continue;
					}

					// We have a valid name-based match, now check if the full attribute can be used as a valid extractor.
					FPCGAttributePropertySelector DataSelector;
					DataSelector.Update(DataAttribute + AttributeAccessors);

					TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, DataSelector, /*bQuiet=*/true);
					if (Accessor && Accessor.IsValid())
					{
						bFoundValidMatch = true;
						break;
					}
				}

				if (!bFoundValidMatch)
				{
					bInFilter = false;
					break;
				}
			}
		}

		return bInFilter;
	}

	bool CreateAndValidateAccessorThreshold(
		TUniquePtr<const IPCGAttributeAccessor>& ThresholdAccessor,
		TUniquePtr<const IPCGAttributeAccessorKeys>& ThresholdAccessorKeys,
		const FPCGFilterByAttributeThresholdSettings& ThresholdSettings,
		const FPCGTaggedData* ThresholdData,
		const uint16 InputType,
		const FPCGContext* Context)
	{
		if (ThresholdData)
		{
			const FPCGAttributePropertyInputSelector ThresholdAttribute = ThresholdSettings.ThresholdAttribute.CopyAndFixLast(ThresholdData->Data);
			ThresholdAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(ThresholdData->Data, ThresholdAttribute);
			ThresholdAccessorKeys = PCGAttributeAccessorHelpers::CreateConstKeys(ThresholdData->Data, ThresholdAttribute);

			if (!ThresholdAccessor || !ThresholdAccessorKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(ThresholdAttribute, Context);
				return false;
			}
		}
		else
		{
			auto ConstantThreshold = [&ThresholdAccessor, &ThresholdAccessorKeys](auto&& Value) -> bool
			{
				using ConstantType = std::decay_t<decltype(Value)>;

				ThresholdAccessor = MakeUnique<FPCGConstantValueAccessor<ConstantType>>(std::forward<decltype(Value)>(Value));
				// Dummy keys
				ThresholdAccessorKeys = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();

				return true;
			};

			if (!ThresholdSettings.AttributeTypes.Dispatcher(ConstantThreshold))
			{
				return false;
			}
		}

		check(ThresholdAccessor && ThresholdAccessorKeys);

		if (!PCG::Private::IsBroadcastableOrConstructible(ThresholdAccessor->GetUnderlyingType(), InputType))
		{
			const FText InputTypeName = PCG::Private::GetTypeNameText(InputType);
			const FText ThresholdTypeName = PCG::Private::GetTypeNameText(ThresholdAccessor->GetUnderlyingType());
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("TypeConversionFailed", "Cannot convert threshold type '{0}' to input target type '{1}'"),
				ThresholdTypeName,
				InputTypeName), Context);
			return false;
		}

		return true;
	}

	bool FilterByValue(const FPCGTaggedData& Input, const UPCGFilterByAttributeSettings& Settings, const FPCGTaggedData* ThresholdData, const FPCGContext* Context)
	{
		const FPCGAttributePropertyInputSelector TargetAttribute = Settings.TargetAttribute.CopyAndFixLast(Input.Data);
		TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, TargetAttribute);
		TUniquePtr<const IPCGAttributeAccessorKeys> InputAccessorKeys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, TargetAttribute);

		if (!InputAccessor || !InputAccessorKeys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(TargetAttribute, Context);
			return false;
		}
		
		TUniquePtr<const IPCGAttributeAccessor> ThresholdAccessor;
		TUniquePtr<const IPCGAttributeAccessorKeys> ThresholdAccessorKeys;
		if (!CreateAndValidateAccessorThreshold(ThresholdAccessor, ThresholdAccessorKeys, Settings.Threshold, ThresholdData, InputAccessor->GetUnderlyingType(), Context))
		{
			return false;
		}

		check(InputAccessor && InputAccessorKeys && ThresholdAccessor && ThresholdAccessorKeys);

		auto DoFilter = [&Settings, &InputAccessor, &InputAccessorKeys, &ThresholdAccessor, &ThresholdAccessorKeys]<typename T>(T) -> bool
		{
			bool bResult = false;
			// if bResult != bShouldContinue, we can stop the processing
			const bool bShouldContinue = Settings.FilterByValueMode == EPCGFilterByAttributeValueMode::AllOf;

			PCGMetadataElementCommon::ApplyOnMultiAccessors<T, T>(
				{InputAccessorKeys.Get(), ThresholdAccessorKeys.Get()},
				{InputAccessor.Get(), ThresholdAccessor.Get()},
				[&bResult, FilterOperator = Settings.FilterOperator, bShouldContinue](const T& Value, const T& Threshold, int) -> bool
			{
				bResult = PCGAttributeFilterHelpers::ApplyCompare(Value, Threshold, FilterOperator);
				return bResult == bShouldContinue;
			}, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);

			return bResult;
		};

		return PCGMetadataAttribute::CallbackWithRightType(InputAccessor->GetUnderlyingType(), DoFilter);
	}

	bool FilterByValueRange(const FPCGTaggedData& Input, const UPCGFilterByAttributeSettings& Settings, const FPCGTaggedData* MinThresholdData, const FPCGTaggedData* MaxThresholdData, FPCGContext* Context)
	{
		const FPCGAttributePropertyInputSelector TargetAttribute = Settings.TargetAttribute.CopyAndFixLast(Input.Data);
		TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, TargetAttribute);
		TUniquePtr<const IPCGAttributeAccessorKeys> InputAccessorKeys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, TargetAttribute);

		if (!InputAccessor || !InputAccessorKeys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(TargetAttribute, Context);
			return false;
		}
		
		TUniquePtr<const IPCGAttributeAccessor> MinThresholdAccessor, MaxThresholdAccessor;
		TUniquePtr<const IPCGAttributeAccessorKeys> MinThresholdAccessorKeys, MaxThresholdAccessorKeys;
		if (!CreateAndValidateAccessorThreshold(MinThresholdAccessor, MinThresholdAccessorKeys, Settings.MinThreshold, MinThresholdData, InputAccessor->GetUnderlyingType(), Context)
			|| !CreateAndValidateAccessorThreshold(MaxThresholdAccessor, MaxThresholdAccessorKeys, Settings.MaxThreshold, MaxThresholdData, InputAccessor->GetUnderlyingType(), Context))
		{
			return false;
		}

		check(InputAccessor && InputAccessorKeys && MinThresholdAccessor && MinThresholdAccessorKeys && MaxThresholdAccessor && MaxThresholdAccessorKeys);
		
		auto DoFilter = [&Settings, &InputAccessor, &InputAccessorKeys, &MinThresholdAccessor, &MinThresholdAccessorKeys, &MaxThresholdAccessor, &MaxThresholdAccessorKeys]<typename T>(T) -> bool
		{
			bool bResult = false;
			
			// Remove some types to avoid compiling too many version of ApplyOnMultiAccessor
			if constexpr (PCG::Private::MetadataTraits<T>::CanCompare)
			{
				// if bResult != bShouldContinue, we can stop the processing
				const bool bShouldContinue = Settings.FilterByValueMode == EPCGFilterByAttributeValueMode::AllOf;

				PCGMetadataElementCommon::ApplyOnMultiAccessors<T, T, T>(
					{InputAccessorKeys.Get(), MinThresholdAccessorKeys.Get(), MaxThresholdAccessorKeys.Get()},
					{InputAccessor.Get(), MinThresholdAccessor.Get(), MaxThresholdAccessor.Get()},
					[&bResult, MinInclusive = Settings.MinThreshold.bInclusive, MaxInclusive = Settings.MaxThreshold.bInclusive, bShouldContinue](const T& Value, const T& MinThreshold, const T& MaxThreshold, int) -> bool
				{
					bResult = PCGAttributeFilterHelpers::ApplyRange(Value, MinThreshold, MaxThreshold, MinInclusive, MaxInclusive);
					return bResult == bShouldContinue;
				}, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
			}

			return bResult;
		};

		return PCGMetadataAttribute::CallbackWithRightType(InputAccessor->GetUnderlyingType(), DoFilter);
	}
}

#if WITH_EDITOR
FText UPCGFilterByAttributeSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Filter Data By Attribute");
}
#endif // WITH_EDITOR

FString UPCGFilterByAttributeSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGFilterByAttributeSettings, FilterMode)))
	{
		return {};
	}
#endif // WITH_EDITOR
	
	if (FilterMode == EPCGFilterByAttributeMode::FilterByExistence)
	{
		static const FText Subtitle = LOCTEXT("FilterByExistence", "By existence"); 
#if WITH_EDITOR
		if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGFilterByAttributeSettings, Attribute)))
		{
			return Subtitle.ToString();
		}
		else
#endif
		{
			return FText::Format(INVTEXT("{0}: {1}"), Subtitle, FText::FromName(Attribute)).ToString();
		}
	}
	else if (FilterMode == EPCGFilterByAttributeMode::FilterByValue || FilterMode == EPCGFilterByAttributeMode::FilterByValueRange)
	{
		static const FText SubtitleValue = LOCTEXT("FilterByValue", "By value");
		static const FText SubtitleValueRange = LOCTEXT("FilterByValueRange", "By value range");
		const FText& Subtitle = FilterMode == EPCGFilterByAttributeMode::FilterByValue ? SubtitleValue : SubtitleValueRange;
#if WITH_EDITOR
		if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGFilterByAttributeSettings, TargetAttribute)))
		{
			return Subtitle.ToString();
		}
		else
#endif
		{
			return FText::Format(INVTEXT("{0}: {1}"), Subtitle, TargetAttribute.GetDisplayText()).ToString();
		}
	}

	return {};
}

#if WITH_EDITOR
EPCGChangeType UPCGFilterByAttributeSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGFilterByAttributeSettings, FilterMode))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGFilterByAttributeSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel).SetRequiredPin();

	if (FilterMode == EPCGFilterByAttributeMode::FilterByValue && !Threshold.bUseConstantThreshold)
	{
		Properties.Emplace_GetRef(PCGAttributeFilterConstants::FilterLabel).SetRequiredPin();
	}
	else if (FilterMode == EPCGFilterByAttributeMode::FilterByValueRange)
	{
		if (!MinThreshold.bUseConstantThreshold)
		{
			Properties.Emplace_GetRef(PCGAttributeFilterConstants::FilterMinLabel).SetRequiredPin();
		}

		if (!MaxThreshold.bUseConstantThreshold)
		{
			Properties.Emplace_GetRef(PCGAttributeFilterConstants::FilterMaxLabel).SetRequiredPin();
		}
	}

	return Properties;
}

FPCGElementPtr UPCGFilterByAttributeSettings::CreateElement() const
{
	return MakeShared<FPCGFilterByAttributeElement>();
}

bool FPCGFilterByAttributeElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGFilterByAttributeElement::Execute);
	check(Context);

	const UPCGFilterByAttributeSettings* Settings = Context->GetInputSettings<UPCGFilterByAttributeSettings>();
	check(Settings);

	TArray<FString> Attributes = PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->Attribute.ToString());

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData> ThresholdData = Context->InputData.GetInputsByPin(PCGAttributeFilterConstants::FilterLabel);
	TArray<FPCGTaggedData> MinThresholdData = Context->InputData.GetInputsByPin(PCGAttributeFilterConstants::FilterMinLabel);
	TArray<FPCGTaggedData> MaxThresholdData = Context->InputData.GetInputsByPin(PCGAttributeFilterConstants::FilterMaxLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	if (Inputs.IsEmpty())
	{
		return true;
	}

	auto ValidateCardinality = [InputNum = Inputs.Num(), Context](const FPCGFilterByAttributeThresholdSettings& ThresholdSettings, const FName ThresholdLabel, const int ThresholdNum)
	{
		if (!ThresholdSettings.bUseConstantThreshold && InputNum != ThresholdNum && ThresholdNum != 1)
		{
			PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, ThresholdLabel, Context);
			return false;
		}

		return true;
	};

	// Validation
	if (Settings->FilterMode == EPCGFilterByAttributeMode::FilterByValue
		&& !ValidateCardinality(Settings->Threshold, PCGAttributeFilterConstants::FilterLabel, ThresholdData.Num()))
	{
		Context->OutputData = Context->InputData;
		return true;
	}
	else if (Settings->FilterMode == EPCGFilterByAttributeMode::FilterByValueRange
		&& (!ValidateCardinality(Settings->MinThreshold, PCGAttributeFilterConstants::FilterMinLabel, MinThresholdData.Num())
			|| !ValidateCardinality(Settings->MaxThreshold, PCGAttributeFilterConstants::FilterMaxLabel, MaxThresholdData.Num())))
	{
		Context->OutputData = Context->InputData;
		return true;
	}

	for (int32 i = 0; i < Inputs.Num(); ++i)
	{
		const FPCGTaggedData& Input = Inputs[i];
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
		Output.Pin = PCGPinConstants::DefaultOutFilterLabel;

		bool bInFilter = false;
		switch (Settings->FilterMode)
		{
		case EPCGFilterByAttributeMode::FilterByExistence:
			bInFilter = PCGFilterByAttributeElement::FilterByExistence(Input, *Settings, Attributes);
			break;
		case EPCGFilterByAttributeMode::FilterByValue:
		{
			const FPCGTaggedData* ThresholdTaggedData = Settings->Threshold.bUseConstantThreshold ? nullptr : &ThresholdData[i % ThresholdData.Num()];
			bInFilter = PCGFilterByAttributeElement::FilterByValue(Input, *Settings, ThresholdTaggedData, Context);
			break;
		}
		case EPCGFilterByAttributeMode::FilterByValueRange:
		{
			const FPCGTaggedData* MinThresholdTaggedData = Settings->MinThreshold.bUseConstantThreshold ? nullptr : &MinThresholdData[i % MinThresholdData.Num()];
			const FPCGTaggedData* MaxThresholdTaggedData = Settings->MaxThreshold.bUseConstantThreshold ? nullptr : &MaxThresholdData[i % MaxThresholdData.Num()];
			bInFilter = PCGFilterByAttributeElement::FilterByValueRange(Input, *Settings, MinThresholdTaggedData, MaxThresholdTaggedData, Context);
			break;
		}
		default:
			break;
		}

		if (bInFilter)
		{
			Output.Pin = PCGPinConstants::DefaultInFilterLabel;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
