// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGFilterByIndex.h"

#include "PCGContext.h"
#include "Helpers/Parsing/PCGIndexing.h"
#include "Helpers/Parsing/PCGParsing.h"
#include "Utils/PCGLogErrors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGFilterByIndex)

#define LOCTEXT_NAMESPACE "PCGFilterByIndexElement"

namespace PCGFilterByIndexConstants
{
	// After so many characters on the node, it will truncate with an ellipsis ...
	static constexpr int32 IndexExpressionTruncation = 12;
}

#if WITH_EDITOR
FName UPCGFilterByIndexSettings::GetDefaultNodeName() const
{
	return FName(TEXT("FilterDataByIndex"));
}

FText UPCGFilterByIndexSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Filter Data By Index");
}

FText UPCGFilterByIndexSettings::GetNodeTooltipText() const
{
	return LOCTEXT("FilterByIndexNodeTooltip", "Filters data in the collection according to user selected indices");
}
#endif

FString UPCGFilterByIndexSettings::GetAdditionalTitleInformation() const
{
	FString NodeName = LOCTEXT("FilterIndices", "Indices: ").ToString();
	NodeName += SelectedIndices.Len() <= PCGFilterByIndexConstants::IndexExpressionTruncation
		? SelectedIndices
		: SelectedIndices.Left(PCGFilterByIndexConstants::IndexExpressionTruncation - 3) + TEXT("...");

	return NodeName;
}

FPCGElementPtr UPCGFilterByIndexSettings::CreateElement() const
{
	return MakeShared<FPCGFilterByIndexElement>();
}

bool FPCGFilterByIndexElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGFilterByIndexElement::ExecuteInternal);

	const UPCGFilterByIndexSettings* Settings = Context->GetInputSettings<UPCGFilterByIndexSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	if (Inputs.IsEmpty())
	{
		return true;
	}

	PCGIndexing::FPCGIndexCollection FilteredIndices(Inputs.Num());
	// Parse the indices and switch through possible issues.
	switch (PCGParser::ParseIndexRanges(FilteredIndices, Settings->SelectedIndices))
	{
		// Error cases are caught for early out.
		case PCGParser::EPCGParserResult::InvalidCharacter:
			PCGLog::Parsing::LogInvalidCharacterInParsedStringError(Settings->SelectedIndices, Context);
			return true;

		case PCGParser::EPCGParserResult::InvalidExpression:
			PCGLog::Parsing::LogInvalidExpressionInParsedStringError(Settings->SelectedIndices, Context);
			return true;

		// If the expression is empty, treat it as though there are simply no selected indices. But, also log.
		case PCGParser::EPCGParserResult::EmptyExpression:
			PCGLog::Parsing::LogEmptyExpressionWarning(Context);
			break;

		case PCGParser::EPCGParserResult::Success:
			break;

		default:
			checkNoEntry();
			return true;
	}

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	Outputs = Inputs;

	for (int Index = 0; Index < Outputs.Num(); ++Index)
	{
		bool bIndexIsFiltered = FilteredIndices.ContainsIndex(Index);

		bIndexIsFiltered = Settings->bInvertFilter ? !bIndexIsFiltered : bIndexIsFiltered;

		if (bIndexIsFiltered)
		{
			Outputs[Index].Pin = PCGPinConstants::DefaultInFilterLabel;
		}
		else
		{
			Outputs[Index].Pin = PCGPinConstants::DefaultOutFilterLabel;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
