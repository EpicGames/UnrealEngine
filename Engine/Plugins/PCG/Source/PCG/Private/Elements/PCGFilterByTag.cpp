// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGFilterByTag.h"
#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "Helpers/PCGHelpers.h"

#include "Algo/AnyOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGFilterByTag)

#define LOCTEXT_NAMESPACE "PCGFilterByTag"

#if WITH_EDITOR
void UPCGFilterByTagSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DataVersion < FPCGCustomVersion::AttributesAndTagsCanContainSpaces)
	{
		bTokenizeOnWhiteSpace = true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Super::ApplyDeprecation(InOutNode);
}

FName UPCGFilterByTagSettings::GetDefaultNodeName() const
{
	return FName(TEXT("FilterDataByTag"));
}

FText UPCGFilterByTagSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Filter Data By Tag");
}

FText UPCGFilterByTagSettings::GetNodeTooltipText() const
{
	return LOCTEXT("FilterByTagNodeTooltip", "Filters data in the collection according to whether they have, or don't have, some tags");
}
#endif

FString UPCGFilterByTagSettings::GetAdditionalTitleInformation() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FString> Tags = bTokenizeOnWhiteSpace
		? PCGHelpers::GetStringArrayFromCommaSeparatedString(SelectedTags)
		: PCGHelpers::GetStringArrayFromCommaSeparatedList(SelectedTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const FString Prefix = (Operation == EPCGFilterByTagOperation::KeepTagged ? LOCTEXT("FilteredTagKeep", "Tag (Keep):") : LOCTEXT("FilteredTagRemove", "Tag (Remove):")).ToString();

	if (Tags.IsEmpty())
	{
		return Prefix;
	}
	else if (Tags.Num() == 1)
	{
		return FString::Printf(TEXT("%s %s"), *Prefix, *Tags[0]);
	}
	else
	{
		const FText MultipleEntriesIndicator = LOCTEXT("FilterMultipleTags", "(multiple)");
		return FString::Printf(TEXT("%s %s"), *Prefix, *MultipleEntriesIndicator.ToString());
	}
}

FPCGElementPtr UPCGFilterByTagSettings::CreateElement() const
{
	return MakeShared<FPCGFilterByTagElement>();
}

bool FPCGFilterByTagElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGFilterByTagElement::Execute);
	check(Context);

	const UPCGFilterByTagSettings* Settings = Context->GetInputSettings<UPCGFilterByTagSettings>();

	const bool bKeepIfTag = (Settings->Operation == EPCGFilterByTagOperation::KeepTagged);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FString> Tags = Settings->bTokenizeOnWhiteSpace
		? PCGHelpers::GetStringArrayFromCommaSeparatedString(Settings->SelectedTags, Context)
		: PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->SelectedTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
		Output.Pin = PCGPinConstants::DefaultOutFilterLabel;

		bool bHasCommonTags = false;

		for (const FString& Tag : Tags)
		{
			if((Settings->Operator == EPCGStringMatchingOperator::Equal && Input.Tags.Contains(Tag)) ||
				(Settings->Operator == EPCGStringMatchingOperator::Substring && Algo::AnyOf(Input.Tags, [&Tag](const FString& InputTag) { return InputTag.Contains(Tag); })) ||
				(Settings->Operator == EPCGStringMatchingOperator::Matches && Algo::AnyOf(Input.Tags, [&Tag](const FString& InputTag) { return InputTag.MatchesWildcard(Tag); })))
			{
				bHasCommonTags = true;
				break;
			}
		}

		if (bKeepIfTag == bHasCommonTags)
		{
			Output.Pin = PCGPinConstants::DefaultInFilterLabel;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE