// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGFilterByType.h"

#include "PCGContext.h"
#include "PCGPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGFilterByType)

#define LOCTEXT_NAMESPACE "PCGFilterByType"

#if WITH_EDITOR
FText UPCGFilterByTypeSettings::GetDefaultNodeTitle() const
{
	if (this == GetClass()->GetDefaultObject())
	{
		return LOCTEXT("DefaultNodeTitle", "Filter Data By Type");
	}

	return FText::Format(LOCTEXT("NodeTitleFormat", "Filter Data - {0}"), TargetType.ToDisplayText());
}

FText UPCGFilterByTypeSettings::GetNodeTooltipText() const
{
	return LOCTEXT("FilterByTypeNodeTooltip", "Filters data in the collection according to data type");
}

bool UPCGFilterByTypeSettings::GetCompactNodeIcon(FName& OutCompactNodeIcon) const
{
	OutCompactNodeIcon = PCGNodeConstants::Icons::CompactNodeFilter;
	return true;
}
#endif // WITH_EDITOR

FPCGDataTypeIdentifier UPCGFilterByTypeSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	check(InPin);
	if (InPin->Properties.Label == PCGPinConstants::DefaultInFilterLabel)
	{
		return TargetType;
	}
	else
	{
		return Super::GetCurrentPinTypesID(InPin);
	}
}

TArray<FPCGPinProperties> UPCGFilterByTypeSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties = Super::OutputPinProperties();

	if (!bShowOutsideFilter)
	{
		PinProperties.RemoveAll([](const FPCGPinProperties& InProperties) { return InProperties.Label == PCGPinConstants::DefaultOutFilterLabel; });
	}

	return PinProperties;
}

FPCGElementPtr UPCGFilterByTypeSettings::CreateElement() const
{
	return MakeShared<FPCGFilterByTypeElement>();
}

bool FPCGFilterByTypeElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGFilterByTypeElement::Execute);
	check(InContext);

	const UPCGFilterByTypeSettings* Settings = InContext->GetInputSettings<UPCGFilterByTypeSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		bool bDataIsInFilter = false;

		if (Input.Data)
		{
			// No match if data does not overlap in type with the target, or if data is broader than the target
			const FPCGDataTypeIdentifier InType{Input.Data->GetDataTypeId()};
			const bool bDataTypeOverlapsWithTarget = InType.Intersects(Settings->TargetType);
			const bool bDataTypeBroaderThanTarget = InType.IsWider(Settings->TargetType);

			if (bDataTypeOverlapsWithTarget && !bDataTypeBroaderThanTarget)
			{
				bDataIsInFilter = true;
			}
		}

		if (!bDataIsInFilter && !Settings->bShowOutsideFilter)
		{
			continue;
		}

		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
		Output.Pin = bDataIsInFilter ? PCGPinConstants::DefaultInFilterLabel : PCGPinConstants::DefaultOutFilterLabel;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
