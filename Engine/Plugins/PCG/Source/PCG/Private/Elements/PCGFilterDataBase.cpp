// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGFilterDataBase.h"

#include "PCGCustomVersion.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "Helpers/PCGSettingsHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGFilterDataBase)

#if WITH_EDITOR
void UPCGFilterDataBaseSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	if (DataVersion < FPCGCustomVersion::UpdateFilterNodeOutputPins)
	{
		InOutNode->RenameOutputPin(PCGPinConstants::DefaultOutputLabel, PCGPinConstants::DefaultInFilterLabel);
	}
}

#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGFilterDataBaseSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInFilterLabel, EPCGDataType::Any);
	PinProperties.Emplace(PCGPinConstants::DefaultOutFilterLabel, EPCGDataType::Any);

	return PinProperties;
}