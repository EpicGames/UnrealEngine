// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGFilterElementsByIndex.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/Parsing/PCGParsing.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGFilterElementsByIndex)

#define LOCTEXT_NAMESPACE "PCGFilterElementsByIndexElement"

namespace PCGFilterElementsByIndex
{
	namespace Constants
	{
		const FLazyName FilterInputsPinLabel = "Filter";
		const FLazyName InFilterOutputPinLabel = "In Filter";
		const FLazyName OutFilterOutputPinLabel = "Out Filter";

		// After so many characters on the node, it will truncate with an ellipsis ...
		static constexpr int32 IndexExpressionTruncation = 12;
	}

	namespace Helpers
	{
		PCGIndexing::FPCGIndexCollection StringToIndexCollection(const int32 ElementCount, const FStringView& Expression, const FPCGContext* InContext)
		{
			check(InContext);

			if (ElementCount == 0)
			{
				return PCGIndexing::FPCGIndexCollection::Invalid();
			}

			PCGIndexing::FPCGIndexCollection FilteredIndices(ElementCount);

			// Parse the indices and switch through possible issues.
			switch (PCGParser::ParseIndexRanges(FilteredIndices, Expression))
			{
				// Error cases are caught for early out.
				case PCGParser::EPCGParserResult::InvalidCharacter:
					PCGLog::Parsing::LogInvalidCharacterInParsedStringError(Expression, InContext);
					return PCGIndexing::FPCGIndexCollection::Invalid();

				case PCGParser::EPCGParserResult::InvalidExpression:
					PCGLog::Parsing::LogInvalidExpressionInParsedStringError(Expression, InContext);
					return PCGIndexing::FPCGIndexCollection::Invalid();

				// If the expression is empty, treat it as though there are simply no selected indices. But, also log.
				case PCGParser::EPCGParserResult::EmptyExpression:
					PCGLog::Parsing::LogEmptyExpressionWarning(InContext);
					break;

				case PCGParser::EPCGParserResult::Success:
					break;

				default:
					checkNoEntry();
					return PCGIndexing::FPCGIndexCollection::Invalid();
			}

			return FilteredIndices;
		}

		void PassThroughInputData(FPCGContext* InContext)
		{
			const TArray<FPCGTaggedData> PrimaryInputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
			InContext->OutputData.TaggedData.Reserve(PrimaryInputs.Num());

			for (const FPCGTaggedData& Input : PrimaryInputs)
			{
				InContext->OutputData.TaggedData.Emplace_GetRef(Input).Pin = Constants::InFilterOutputPinLabel;
			}
		}
	} // namespace Helpers
}

FPCGElementPtr UPCGFilterElementsByIndexSettings::CreateElement() const
{
	return MakeShared<FPCGFilterElementsByIndexElement>();
}

FString UPCGFilterElementsByIndexSettings::GetAdditionalTitleInformation() const
{
	if (bSelectIndicesByInput)
	{
		return Super::GetAdditionalTitleInformation();
	}

	FString NodeName = LOCTEXT("ListOfIndices", "Indices: ").ToString();
	if (SelectedIndices == FString(TEXT(":")))
	{
		NodeName += LOCTEXT("AllIndicesSelected", "All").ToString();
	}
	else
	{
		NodeName += SelectedIndices.Len() <= PCGFilterElementsByIndex::Constants::IndexExpressionTruncation
					? SelectedIndices
					: SelectedIndices.Left(PCGFilterElementsByIndex::Constants::IndexExpressionTruncation - 3) + TEXT("...");
	}

	return NodeName;
}

TArray<FPCGPinProperties> UPCGFilterElementsByIndexSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& PrimaryPinProperties = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::PointOrParam);
	PrimaryPinProperties.SetRequiredPin();
#if WITH_EDITOR
	PrimaryPinProperties.Tooltip = LOCTEXT("PrimaryInputPinTooltip", "To be filtered by selected indices from a secondary input or a selection string.");
#endif // WITH_EDITOR

	if (bSelectIndicesByInput)
	{
		FPCGPinProperties& FilterPinProperties = PinProperties.Emplace_GetRef(PCGFilterElementsByIndex::Constants::FilterInputsPinLabel, EPCGDataType::PointOrParam);
#if WITH_EDITOR
		FilterPinProperties.Tooltip = LOCTEXT("FilterInputPinTooltip", "An attribute on elements from this input will be used to determine which elements from the first input will be filtered.");
#endif // WITH_EDITOR
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGFilterElementsByIndexSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InFilterPinProperties = PinProperties.Emplace_GetRef(PCGFilterElementsByIndex::Constants::InFilterOutputPinLabel, EPCGDataType::PointOrParam);
#if WITH_EDITOR
	InFilterPinProperties.Tooltip = LOCTEXT("InFilterOutputPinTooltip", "The elements that remain included after the filtering process.");
#endif // WITH_EDITOR

	if (bOutputDiscardedElements)
	{
		FPCGPinProperties& OutFilterPinProperties = PinProperties.Emplace_GetRef(PCGFilterElementsByIndex::Constants::OutFilterOutputPinLabel, EPCGDataType::PointOrParam);
#if WITH_EDITOR
		OutFilterPinProperties.Tooltip = LOCTEXT("OutFilterOutputPinTooltip", "The elements that were excluded during the filtering process.");
#endif // WITH_EDITOR
	}

	return PinProperties;
}

bool FPCGFilterElementsByIndexElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGFilterElementsByIndexElement::Execute);
	using namespace PCGFilterElementsByIndex;

	check(InContext);

	const TArray<FPCGTaggedData> PrimaryInputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	if (PrimaryInputs.IsEmpty())
	{
		return true;
	}

	const UPCGFilterElementsByIndexSettings* Settings = InContext->GetInputSettings<UPCGFilterElementsByIndexSettings>();
	check(Settings);

	PCGIndexing::FPCGIndexCollection AllFilteredIndices = PCGIndexing::FPCGIndexCollection::Invalid();

	// Getting the filtered indices from the secondary input pin.
	if (Settings->bSelectIndicesByInput)
	{
		TArray<FPCGTaggedData> FilterInputs = InContext->InputData.GetInputsByPin(Constants::FilterInputsPinLabel);

		// Forward inputs if no filtering data.
		if (FilterInputs.IsEmpty())
		{
			Helpers::PassThroughInputData(InContext);
			return true;
		}

		// ... or if not 1:N, N:1, or N:N.
		if (PrimaryInputs.Num() != FilterInputs.Num() && (PrimaryInputs.Num() != 1 && FilterInputs.Num() != 1))
		{
			PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, Constants::FilterInputsPinLabel, InContext);
			Helpers::PassThroughInputData(InContext);
			return true;
		}

		// Collect all the filter information first.
		for (const FPCGTaggedData& FilterInput : FilterInputs)
		{
			const UPCGData* FilterInputData = FilterInput.Data;
			const UPCGBasePointData* FilterPointData = Cast<UPCGBasePointData>(FilterInputData);
			const UPCGParamData* FilterParamData = Cast<UPCGParamData>(FilterInputData);
			if (!(FilterPointData || FilterParamData))
			{
				PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::PointOrParam, Constants::FilterInputsPinLabel, InContext);
				continue;
			}

			FPCGAttributePropertyInputSelector IndexSelector = Settings->IndexSelectionAttribute.CopyAndFixLast(FilterInputData);
			const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(FilterInputData, IndexSelector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(FilterInputData, IndexSelector);

			if (!Accessor || !Keys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(IndexSelector, InContext);
				continue;
			}

			if (Keys->GetNum() < 1)
			{
				continue;
			}

			// Get the indices from the secondary input.
			TArray<int32> IndexArray;
			IndexArray.SetNumUninitialized(Keys->GetNum());
			if (!Accessor->GetRange<int32>(IndexArray, 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::Metadata::LogFailToGetAttributeError(IndexSelector, InContext);
				continue;
			}
			check(!IndexArray.IsEmpty());

			IndexArray.Sort();
			// Convert the highest index found to a local array size.
			PCGIndexing::FPCGIndexCollection FilteredIndices(IndexArray.Last() + 1);
			if (!FilteredIndices.IsValid())
			{
				PCGLog::LogWarningOnGraph(LOCTEXT("FailureToGetIndicesFromInput", "Filtered indices were invalid and could not be used to filter the input!"));
				continue;
			}

			// Now sorted, add by range rather than adding every single index as a range.
			int32 RangeStartIndex = IndexArray[0];
			int32 RangeEndIndex = IndexArray[0];
			for (int i = 1; i < IndexArray.Num(); ++i)
			{
				if (IndexArray[i] != RangeEndIndex + 1)
				{
					// Ranges are exclusive on max, so +1
					FilteredIndices.AddRange(RangeStartIndex, RangeEndIndex + 1);
					RangeStartIndex = IndexArray[i];
					RangeEndIndex = RangeStartIndex;
				}
				else
				{
					RangeEndIndex = IndexArray[i];
				}
			}

			// Add the final range.
			FilteredIndices.AddRange(RangeStartIndex, RangeEndIndex + 1);

			// Update the base list.
			AllFilteredIndices += FilteredIndices;
		}

		// No filtered indices. Pass-through.
		if (AllFilteredIndices.GetTotalIndexCount() < 1)
		{
			Helpers::PassThroughInputData(InContext);
			return true;
		}
	}

	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	for (const FPCGTaggedData& PrimaryInput : PrimaryInputs)
	{
		const UPCGData* PrimaryInputData = PrimaryInput.Data;

		// Get the filtered indices directly from the user string.
		if (!Settings->bSelectIndicesByInput)
		{
			const int32 NumElements = PCGHelpers::GetNumberOfElements(PrimaryInputData);
			AllFilteredIndices = Helpers::StringToIndexCollection(NumElements, Settings->SelectedIndices, InContext);
		}

		auto AddNewOutput = [&Outputs, &Tags = PrimaryInput.Tags](UPCGData* Data, FName Pin)
		{
			Outputs.Emplace(FPCGTaggedData
			{
				.Data = Data,
				.Tags = Tags,
				.Pin = Pin
			});
		};

		if (const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(PrimaryInputData))
		{
			UPCGBasePointData* InFilterPointData = FPCGContext::NewPointData_AnyThread(InContext);

			FPCGInitializeFromDataParams InitializeFromDataParams(InputPointData);
			InitializeFromDataParams.bInheritSpatialData = false;

			InFilterPointData->InitializeFromDataWithParams(InitializeFromDataParams);

			const int32 NumInFilterPoints = Settings->bInvertFilter ? InputPointData->GetNumPoints() - AllFilteredIndices.GetTotalIndexCount() : AllFilteredIndices.GetTotalIndexCount();
			InFilterPointData->SetNumPoints(NumInFilterPoints);
			InFilterPointData->AllocateProperties(InputPointData->GetAllocatedProperties());
			InFilterPointData->CopyUnallocatedPropertiesFrom(InputPointData);

			AddNewOutput(InFilterPointData, Constants::InFilterOutputPinLabel);

			UPCGBasePointData* OutFilterPointData = nullptr;
			if (Settings->bOutputDiscardedElements)
			{
				OutFilterPointData = FPCGContext::NewPointData_AnyThread(InContext);
				
				OutFilterPointData->InitializeFromDataWithParams(InitializeFromDataParams);
				
				const int32 NumOutFilterPoints = InputPointData->GetNumPoints() - NumInFilterPoints;
				OutFilterPointData->SetNumPoints(NumOutFilterPoints);
				OutFilterPointData->AllocateProperties(InputPointData->GetAllocatedProperties());
				OutFilterPointData->CopyUnallocatedPropertiesFrom(InputPointData);

				AddNewOutput(OutFilterPointData, Constants::OutFilterOutputPinLabel);
			}

			const FConstPCGPointValueRanges InRanges(InputPointData);
			FPCGPointValueRanges InFilterRanges(InFilterPointData, /*bAllocate=*/false);
			FPCGPointValueRanges OutFilterRanges = OutFilterPointData ? FPCGPointValueRanges(OutFilterPointData, /*bAllocate=*/false) : FPCGPointValueRanges();
			
			int32 InFilterWritten = 0;
			int32 OutFilterWritten = 0;
			for (int PointIndex = 0; PointIndex < InputPointData->GetNumPoints(); ++PointIndex)
			{
				bool bIndexIsFiltered = AllFilteredIndices.ContainsIndex(PointIndex);
				bIndexIsFiltered = Settings->bInvertFilter ? !bIndexIsFiltered : bIndexIsFiltered;
				if (bIndexIsFiltered)
				{
					InFilterRanges.SetFromValueRanges(InFilterWritten, InRanges, PointIndex);
					++InFilterWritten;
				}
				else if (Settings->bOutputDiscardedElements)
				{
					OutFilterRanges.SetFromValueRanges(OutFilterWritten, InRanges, PointIndex);
					++OutFilterWritten;
				}
			}
		}
		else if (const UPCGParamData* InputParamData = Cast<UPCGParamData>(PrimaryInputData))
		{
			const UPCGMetadata* InputMetadata = InputParamData->Metadata;
			check(InputMetadata);

			const int32 NumEntries = InputMetadata->GetItemCountForChild();
			TArray<PCGMetadataEntryKey> InEntryKeys;
			TArray<PCGMetadataEntryKey> OutEntryKeys;
			InEntryKeys.Reserve(NumEntries);
			if (Settings->bOutputDiscardedElements)
			{
				OutEntryKeys.Reserve(NumEntries);
			}

			for (PCGMetadataEntryKey i = 0; i < NumEntries; ++i)
			{
				bool bIndexIsFiltered = AllFilteredIndices.ContainsIndex(i);
				bIndexIsFiltered = Settings->bInvertFilter ? !bIndexIsFiltered : bIndexIsFiltered;
				if (bIndexIsFiltered)
				{
					InEntryKeys.Emplace(i);
				}
				else if (Settings->bOutputDiscardedElements)
				{
					OutEntryKeys.Emplace(i);
				}
			}

			UPCGParamData* InFilterParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InContext);
			AddNewOutput(InFilterParamData, Constants::InFilterOutputPinLabel);

			UPCGMetadata* InFilterMetadata = InFilterParamData->Metadata;
			check(InFilterMetadata);
			InFilterMetadata->InitializeAsCopy(FPCGMetadataInitializeParams(InputMetadata, &InEntryKeys));

			if (Settings->bOutputDiscardedElements)
			{
				UPCGParamData* OutFilterParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InContext);
				UPCGMetadata* OutFilterMetadata = OutFilterParamData->Metadata;
				check(OutFilterMetadata);
				AddNewOutput(OutFilterParamData, Constants::OutFilterOutputPinLabel);
				OutFilterMetadata->InitializeAsCopy(FPCGMetadataInitializeParams(InputMetadata, &OutEntryKeys));
			}
		}
		else
		{
			PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::PointOrParam, PCGPinConstants::DefaultInputLabel, InContext);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
