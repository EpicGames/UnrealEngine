// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGather.h"

#include "PCGContext.h"
#include "PCGPin.h"

#include "Algo/Transform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGather)

#define LOCTEXT_NAMESPACE "PCGGatherElement"

#if WITH_EDITOR
void UPCGGatherSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InOutNode->RenameInputPin(PCGPinConstants::DefaultDependencyOnlyLabel, PCGPinConstants::DefaultExecutionDependencyLabel, /*bInBroadcastUpdate=*/false);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGatherSettings::StaticInputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	// Do not explicitly mark the static input pin as required, as data on any input pin should prevent culling.
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any);
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGatherSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGGatherSettings::CreateElement() const
{
	return MakeShared<FPCGGatherElement>();
}

FName UPCGGatherSettings::GetDynamicInputPinsBaseLabel() const
{
	return PCGPinConstants::DefaultInputLabel;
}

FPCGDataTypeIdentifier UPCGGatherSettings::GetCurrentPinTypesID(const UPCGPin* Pin) const
{
	// Output pin depends on the union of all input pins.
	if (Pin->IsOutputPin())
	{
		TArray<FPCGPinProperties> InputPins = InputPinProperties();
		TArray<FName> InputPinLabels;
		Algo::Transform(InputPins, InputPinLabels, [](const FPCGPinProperties& InPinProperty) { return InPinProperty.Label; });
		
		const FPCGDataTypeIdentifier InputTypeUnion = GetTypeUnionIDOfAllIncidentEdges(InputPinLabels);
		return InputTypeUnion.IsValid() ? InputTypeUnion : FPCGDataTypeIdentifier{ EPCGDataType::Any };
	}
	else
	{
		return Super::GetCurrentPinTypesID(Pin);
	}
}

#if WITH_EDITOR
void UPCGGatherSettings::AddDefaultDynamicInputPin()
{
	FPCGPinProperties SecondaryPinProperties(
		FName(GetDynamicInputPinsBaseLabel().ToString() + FString::FromInt(DynamicInputPinProperties.Num() + 2)),
		EPCGDataType::Any,
		/*bInAllowMultipleConnections=*/false);
	AddDynamicInputPin(std::move(SecondaryPinProperties));
}
#endif // WITH_EDITOR

namespace PCGGather
{
	FPCGDataCollection GatherDataForPin(const FPCGDataCollection& InputData, const FName InputLabel, const FName OutputLabel)
	{
		TArray<FPCGTaggedData> GatheredData = InputData.GetInputsByPin(InputLabel);
		FPCGDataCollection Output;

		if (GatheredData.IsEmpty())
		{
			return Output;
		}
	
		if (GatheredData.Num() == InputData.TaggedData.Num())
		{
			Output = InputData;
		}
		else
		{
			Output.TaggedData = MoveTemp(GatheredData);
		}

		for (FPCGTaggedData& TaggedData : Output.TaggedData)
		{
			TaggedData.Pin = OutputLabel;
		}

		return Output;
	}
}

bool FPCGGatherElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGatherElement::Execute);

	if (const UPCGGatherSettings* Settings = Context->GetInputSettings<UPCGGatherSettings>())
	{
		for (FName PinLabel : Settings->GetNodeDefinedPinLabels())
		{
			Context->OutputData += PCGGather::GatherDataForPin(Context->InputData, PinLabel);
		}
	}
	else
	{
		// If running settings-less, just gather from default primary input pin.
		Context->OutputData = PCGGather::GatherDataForPin(Context->InputData, PCGPinConstants::DefaultInputLabel);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
