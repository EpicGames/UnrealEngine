// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGenerateLandscapeTextures.h"

#include "PCGComponent.h"
#include "PCGGrassMapUnpackerCS.h"
#include "PCGModule.h"
#include "PCGSubsystem.h"
#include "Compute/PCGComputeCommon.h"
#include "Data/PCGLandscapeData.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Data/PCGTextureData.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Algo/Transform.h"
#include "DrawDebugHelpers.h"
#include "EngineDefines.h" // For UE_ENABLE_DEBUG_DRAWING
#include "Landscape.h"
#include "LandscapeComponent.h"
#include "LandscapeGrassType.h"
#include "LandscapeGrassWeightExporter.h"
#include "LandscapeProxy.h"
#include "LandscapeUtils.h"
#include "RenderCaptureInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SystemTextures.h"
#include "Engine/World.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGenerateLandscapeTextures)

#define LOCTEXT_NAMESPACE "PCGGenerateLanscapeTexturesElement"

namespace PCGGenerateLandscapeTextures
{
	const FName InputPinLabel = TEXT("Landscape");
	const FName GrassTypeOverridesPinLabel = TEXT("Grass Types");
	const FName OutputHeightPinLabel = TEXT("Height");
	const FName OutputGrassMapsPinLabel = TEXT("GrassMaps");

	const FText LandscapeComponentLostError = LOCTEXT("LandscapeComponentLost", "Reference to one or more landscape components lost, grass maps will not be generated.");

#if !UE_BUILD_SHIPPING
	static int32 GTriggerGPUCaptureDispatches = 0;
	static FAutoConsoleVariableRef CVarTriggerGPUCapture(
		TEXT("pcg.GPU.TriggerRenderCaptures.GrassMapGeneration"),
		GTriggerGPUCaptureDispatches,
		TEXT("Trigger GPU captures for this many of the subsequent grass generations."));
#endif // !UE_BUILD_SHIPPING

#if WITH_EDITOR
	TAutoConsoleVariable<bool> CVarDebugDrawGeneratedComponents(
		TEXT("pcg.Grass.DebugDrawGeneratedComponents"),
		false,
		TEXT("Draws debug boxes around landscapes for which grass maps are generated, colored by the current task ID."));
#endif

	static bool IsTextureFullyStreamedIn(UTexture* InTexture)
	{
		return InTexture &&
#if WITH_EDITOR
			!InTexture->IsDefaultTexture() &&
#endif // WITH_EDITOR
			!InTexture->HasPendingInitOrStreaming() && InTexture->IsFullyStreamedIn();
	}

	struct FGenerateComponentsData
	{
		FPCGGenerateLandscapeTexturesContext* Context = nullptr;
		TArray<ULandscapeComponent*> LandscapeComponents;
		const UPCGGenerateLandscapeTexturesSettings* Settings = nullptr;
		ALandscape* Landscape = nullptr;
		TArray<FIntVector2> LandscapeTileCoords;
		TArray<FIntPoint> LandscapeComponentKeys;
		FBox LandscapeComponentBounds = FBox(EForceInit::ForceInit);
		bool bLastGenerationPass = false;
	};

	bool GenerateComponents(FGenerateComponentsData&& InData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGGenerateLandscapeTextures::GenerateComponents);
		if (!InData.Context || !InData.Settings || !InData.Landscape || !InData.Context->OutputTextureData)
		{
			return false;
		}

		const bool bGenerateGrassMaps = !InData.Context->GrassNamesToExport.IsEmpty();
		const bool bGenerateHeightMap = InData.Settings->bGenerateHeightMap;

		UE::Landscape::Grass::EGrassWeightExporterFlags Flags = UE::Landscape::Grass::EGrassWeightExporterFlags::None;
		if (bGenerateGrassMaps)
		{
			Flags |= UE::Landscape::Grass::EGrassWeightExporterFlags::NeedsGrassmap;
		}
		if (bGenerateHeightMap)
		{
			Flags |= UE::Landscape::Grass::EGrassWeightExporterFlags::NeedsHeightmap;
		}

		TUniquePtr<FLandscapeGrassWeightExporter> Exporter;
		{
			TArray<FName> GrassExporterGrassNames;
			if (InData.Settings->bTagDataWithLandscapeGrassTypeAssetNames)
			{
				// If bTagDataWithLandscapeGrassTypeAssetNames, translate the grass types names to the actual grass names, because that's what the grass weight exporter knows about : 
				GrassExporterGrassNames.Reserve(InData.Context->GrassNamesToExport.Num());
				Algo::Transform(InData.Context->GrassNamesToExport, GrassExporterGrassNames, [Context=InData.Context](const FName& InGrassTypeName)
				{
					const FName* FoundGrassName = Context->GrassTypeToGrassNameMap.Find(InGrassTypeName);
					check(FoundGrassName != nullptr);
					return *FoundGrassName;
				});
			}
			else
			{
				GrassExporterGrassNames = InData.Context->GrassNamesToExport;
			}

			Exporter.Reset(new FLandscapeGrassWeightExporter(
				InData.Landscape,
				InData.LandscapeComponents,
				Flags,
				GrassExporterGrassNames,
				/*InRequestedHeightMips = */TConstArrayView<int32>{}));
		}

		const FIntPoint& OutputTextureSize = Exporter->GetTargetSize();
		const int32 Max2DTextureDimension = GetMax2DTextureDimension();

		if (OutputTextureSize.X < 1 || OutputTextureSize.X > Max2DTextureDimension || OutputTextureSize.Y < 1 || OutputTextureSize.Y > Max2DTextureDimension)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidTextureSize", "Invalid output texture size ({0}, {1}), cannot exceed ({2}, {2}). Consider partitioning the component onto a smaller grid size."), OutputTextureSize.X, OutputTextureSize.Y, Max2DTextureDimension), InData.Context);
			return false;
		}

		const int32 ComponentSizeQuads = InData.Landscape->ComponentSizeQuads;
		const double QuadSizeWorld = InData.Context->LandscapeComponentExtent / ComponentSizeQuads;
		// Each corner of a quad in the landscape corresponds to one texel in the grass map. 
		const uint32 LandscapeComponentResolution = ComponentSizeQuads + 1;

		const FBox& TotalBounds = InData.LandscapeComponentBounds;
		const FBox& OutputBounds = InData.Context->OutputTextureBounds;

		UE_LOGF(LogPCG, Verbose, "PCGUnpackGrassMap: %ls, Output texture bounds (%.2f, %.2f -> %.2f, %.2f), Total bounds (%.2f, %.2f -> %.2f, %.2f), efficiency %.2f",
			*InData.Context->GetExecutionSourceName(),
			OutputBounds.Min.X, OutputBounds.Min.Y, OutputBounds.Max.X, OutputBounds.Max.Y,
			TotalBounds.Min.X, TotalBounds.Min.Y, TotalBounds.Max.X, TotalBounds.Max.Y,
			(OutputBounds.Max.X - OutputBounds.Min.X) * (OutputBounds.Max.Y - OutputBounds.Min.Y) / ((TotalBounds.Max.X - TotalBounds.Min.X) * (TotalBounds.Max.Y - TotalBounds.Min.Y)));

		FIntPoint TileGridStart, TileGridEnd;
		TileGridStart.X = FMath::Floor((TotalBounds.Min.X - OutputBounds.Min.X) / QuadSizeWorld);
		TileGridStart.Y = FMath::Floor((TotalBounds.Min.Y - OutputBounds.Min.Y) / QuadSizeWorld);
		TileGridEnd.X = FMath::CeilToInt((TotalBounds.Max.X - OutputBounds.Min.X) / QuadSizeWorld);
		TileGridEnd.Y = FMath::CeilToInt((TotalBounds.Max.Y - OutputBounds.Min.Y) / QuadSizeWorld);

		const FVector TotalLandscapeComponentExtent = InData.LandscapeComponentBounds.GetExtent() * 2;
		const uint32 NumTilesX = FMath::RoundToInt(TotalLandscapeComponentExtent.X / InData.Context->LandscapeComponentExtent);
		const uint32 NumTilesY = FMath::RoundToInt(TotalLandscapeComponentExtent.Y / InData.Context->LandscapeComponentExtent);

		const FIntPoint FullExportResolution((LandscapeComponentResolution - 1u) * NumTilesX + 1u, (LandscapeComponentResolution - 1u) * NumTilesY + 1u);
		UE_LOGF(LogPCG, Verbose, "PCGUnpackGrassMap: %ls, Export resolution %ls, Window %ls -> %ls, Window proportion %.2f",
			*InData.Context->GetExecutionSourceName(),
			*FullExportResolution.ToString(),
			*TileGridStart.ToString(),
			*TileGridEnd.ToString(),
			(InData.Context->OutputResolution.X * InData.Context->OutputResolution.Y) / double(FullExportResolution.X * FullExportResolution.Y));

		const FIntPoint DispatchMin(FMath::Max(0, TileGridStart.X), FMath::Max(0, TileGridStart.Y));
		const FIntPoint DispatchMax(FMath::Min(InData.Context->OutputResolution.X - 1, TileGridEnd.X), FMath::Min(InData.Context->OutputResolution.Y - 1, TileGridEnd.Y));
		const FIntPoint DispatchSize(DispatchMax.X - DispatchMin.X + 1, DispatchMax.Y - DispatchMin.Y + 1);
		if (DispatchSize.X <= 0 || DispatchSize.Y <= 0)
		{
			UE_LOGF(LogPCG, Error, "PCGGenerateLandscapeTextures: Landscape components for export do not overlap generation volume, should not happen.");
			return true;
		}

		UE::RenderCommandPipe::FSyncScope SyncScope;

#if !UE_BUILD_SHIPPING
		RenderCaptureInterface::FScopedCapture RenderCapture(PCGGenerateLandscapeTextures::GTriggerGPUCaptureDispatches > 0, TEXT("PCGLandscapeGrassmapCapture"));
		PCGGenerateLandscapeTextures::GTriggerGPUCaptureDispatches = FMath::Max(PCGGenerateLandscapeTextures::GTriggerGPUCaptureDispatches - 1, 0);
#endif  // !UE_BUILD_SHIPPING

		ENQUEUE_RENDER_COMMAND(PCGGenerateLandscapeTextures)(
			[ContextHandle = InData.Context->GetOrCreateHandle()
			, LandscapeComponentExtent = InData.Context->LandscapeComponentExtent
			, LandscapeTileCoords = MoveTemp(InData.LandscapeTileCoords)
			, LandscapeComponentKeys = MoveTemp(InData.LandscapeComponentKeys)
			, OutputIndexToExportIndex = InData.Context->OutputIndexToExportIndex
			, LandscapeComponentResolution
			, LandscapeRootZ = InData.Context->LandscapeRootZ
			, LandscapeScaleZ = InData.Context->LandscapeScaleZ
			, bGenerateGrassMaps
			, bGenerateHeightMap
			, NumTilesX
			, NumTilesY
			, OutputResolution = InData.Context->OutputResolution
			, TileGridStart
			, DispatchMin
			, DispatchSize
			, OutputTextureData = InData.Context->OutputTextureData
			, bLastGenerationPass = InData.bLastGenerationPass
			, Exporter = MoveTemp(Exporter)](FRHICommandListImmediate& RHICmdList)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGGenerateLandscapeTextures);
			LLM_SCOPE_BYTAG(PCG);

			FPCGContext::FSharedContext<FPCGGenerateLandscapeTexturesContext> SharedContext(ContextHandle);
			FPCGGenerateLandscapeTexturesContext* Context = SharedContext.Get();
			if (!Context)
			{
				return;
			}

			FRDGBuilder GraphBuilder(RHICmdList);

			FRDGTextureDesc GrassMapTextureDesc = FRDGTextureDesc::Create2D(
				Exporter->GetTargetSize(),
				PF_B8G8R8A8,
				FClearValueBinding(),
				ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource);

			FRDGTextureRef GrassMapTexture = GraphBuilder.CreateTexture(GrassMapTextureDesc, TEXT("PCGLandscapeGrassMapRenderTarget"));
			FRDGTextureSRVRef GrassMapTextureSRV = GraphBuilder.CreateSRV(GrassMapTexture);

			// Generate grass maps. All will be generated to a single texture.
			Exporter->RenderLandscapeComponentToTexture_RenderThread(GraphBuilder, GrassMapTexture);

			// Must have at least one slice to be a valid resource.
			const uint32 NumOutputSlices = FMath::Max(Context->GrassNamesToOutput.Num(), 1);

			FRDGTextureRef HeightMap = nullptr;
			bool bHeightMapRequiresExport = false;
			if (OutputTextureData->HeightMapHandle)
			{
				HeightMap = GraphBuilder.RegisterExternalTexture(OutputTextureData->HeightMapHandle, TEXT("PCGHeightMapExternal"));
			}
			else
			{
				FRDGTextureDesc HeightMapDesc = FRDGTextureDesc::Create2D(
					bGenerateHeightMap ? OutputResolution : FIntPoint(1, 1), // No default system texture usable for UAV, so always create but with 1x1 resolution
					PF_R32_FLOAT,
					FClearValueBinding(),
					ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);

				HeightMap = GraphBuilder.CreateTexture(HeightMapDesc, TEXT("PCGLandscapeHeightMapUnpacked"));

				// Only needs export if we're generating the heightmap.
				bHeightMapRequiresExport = bGenerateHeightMap;
			}


			// Output texture is array of textures, one per grass map.
			FRDGTextureRef GrassMap = nullptr;
			bool bGrassMapRequiresExport = false;
			if (OutputTextureData->GrassMapHandle)
			{
				GrassMap = GraphBuilder.RegisterExternalTexture(OutputTextureData->GrassMapHandle, TEXT("PCGGrassMapExternal"));
			}
			else
			{
				FRDGTextureDesc GrassMapDesc = FRDGTextureDesc::Create2DArray(
					bGenerateGrassMaps ? OutputResolution : FIntPoint(1, 1), // No default system texture usable for UAV, so always create but with 1x1 resolution
					PF_G8,
					FClearValueBinding(),
					ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV,
					NumOutputSlices);

				GrassMap = GraphBuilder.CreateTexture(GrassMapDesc, TEXT("PCGLandscapeGrassMapUnpacked"));

				// Only needs export if we're generating grass maps.
				bGrassMapRequiresExport = bGenerateGrassMaps;
			}

			// Unpack the generated results to simple world aligned textures.
			FPCGGrassMapUnpackerCS::FParameters* Parameters = GraphBuilder.AllocParameters<FPCGGrassMapUnpackerCS::FParameters>();
			Parameters->InPackedGrassMaps = GrassMapTextureSRV;
			Parameters->OutUnpackedGrassMaps = GraphBuilder.CreateUAV(GrassMap);
			Parameters->OutUnpackedHeight = GraphBuilder.CreateUAV(HeightMap);
			Parameters->InNumTiles = FUintVector2(NumTilesX, NumTilesY);
			Parameters->InLandscapeRootZ = static_cast<float>(LandscapeRootZ);
			Parameters->InLandscapeScaleZ = static_cast<float>(LandscapeScaleZ);
			Parameters->InLandscapeComponentResolution = LandscapeComponentResolution;
			Parameters->InOutputHeight = bGenerateHeightMap ? 1u : 0u; // No equivalent flag for grass maps, instead ComponentGrassmapOffsetAndChannelsBuffer will have invalid indices
			Parameters->InTileGridOrigin = TileGridStart;
			Parameters->InDispatchOffset = FUintVector2(DispatchMin.X, DispatchMin.Y);
			Parameters->InDispatchSize = FUintVector2(DispatchSize.X, DispatchSize.Y);
			Parameters->InNumExportedGrassMaps = Context->GrassNamesToExport.Num();

			// Initialize to invalid component indices.
			FMemory::Memset(&Parameters->InLinearTileIndexToComponentIndex[0], -1, FPCGGrassMapUnpackerCS::MaxNumLandscapeComponents * sizeof(int32) * 4);

			// Now write component mapping.
			check(LandscapeTileCoords.Num() <= FPCGGrassMapUnpackerCS::MaxNumLandscapeComponents);
			for (int Index = 0; Index < LandscapeTileCoords.Num(); ++Index)
			{
				const int32 LinearTileIndex = LandscapeTileCoords[Index].Y * NumTilesX + LandscapeTileCoords[Index].X;
				check((LinearTileIndex >= 0) && (LinearTileIndex < FPCGGrassMapUnpackerCS::MaxNumLandscapeComponents));
				Parameters->InLinearTileIndexToComponentIndex[LinearTileIndex].X = Index;
			}
			check(LandscapeTileCoords.Num() == LandscapeComponentKeys.Num());

			FRDGBufferRef ComponentHeightOffsetsBuffer;
			if (bGenerateHeightMap)
			{
				// X offset for where to find each component's height data in the grass texture (always in the .xy channels)
				TArray<int32> ComponentHeightOffsets;
				ComponentHeightOffsets.Reserve(LandscapeComponentKeys.Num());
				for (int ComponentIndex = 0; ComponentIndex < LandscapeComponentKeys.Num(); ++ComponentIndex)
				{
					TValueOrError<FIntRect /*OutputRect*/, FLandscapeGrassWeightExporter_RenderThread::EHeightmapTextureInfoError> TextureInfo =
						Exporter->GetTextureInfoForHeight(LandscapeComponentKeys[ComponentIndex]);
					// Not supposed to fail : 
					checkf(!TextureInfo.HasError(), TEXT("Failed to retrieve height map info on component %s : reason : %i"), *LandscapeComponentKeys[ComponentIndex].ToString(), EnumToUnderlyingType(TextureInfo.GetError()));
					ComponentHeightOffsets.Add(TextureInfo.GetValue().Min.X);
				}
				check(ComponentHeightOffsets.Num() == LandscapeComponentKeys.Num());
				ComponentHeightOffsetsBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("PCGGenerateLandscapeTexturesComponentHeightOffsets"), ComponentHeightOffsets);
			}
			else
			{
				ComponentHeightOffsetsBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(int32));
			}
			Parameters->InComponentHeightOffsets = GraphBuilder.CreateSRV(ComponentHeightOffsetsBuffer);

			FRDGBufferRef ComponentGrassmapOffsetAndChannelsBuffer;
			if (!Context->GrassNamesToExport.IsEmpty())
			{
				// For each component, for each grass type, X offset for where to find the grass data in the grass texture (.x) and on which channel (.y)
				//  It's possible a grass map is absent from a given component, in which case, the offset and channel will be set to -1
				TArray<FIntVector2> ComponentGrassmapOffsetAndChannels;
				ComponentGrassmapOffsetAndChannels.SetNumUninitialized(LandscapeComponentKeys.Num() * Context->GrassNamesToExport.Num());
				FMemory::Memset(ComponentGrassmapOffsetAndChannels.GetData(), -1, ComponentGrassmapOffsetAndChannels.Num() * sizeof(FIntVector2));

				for (int32 ComponentIndex = 0; ComponentIndex < LandscapeComponentKeys.Num(); ++ComponentIndex)
				{
					for (int32 GrassmapIndex = 0; GrassmapIndex < Context->GrassNamesToExport.Num(); ++GrassmapIndex)
					{
						FIntVector2& ComponentGrassmapOffsetAndChannel = ComponentGrassmapOffsetAndChannels[ComponentIndex * Context->GrassNamesToExport.Num() + GrassmapIndex];

						FName LandscapeExporterGrassName = Context->GrassNamesToExport[GrassmapIndex];
						// If we have a grass type name -> grass name mapping, that's because bTagDataWithLandscapeGrassTypeAssetNames was used, so we need to translate the grass type name into the grass name, 
						//  which is what the grass exporter understands :
						if (!Context->GrassTypeToGrassNameMap.IsEmpty())
						{
							const FName* FoundGrassName = Context->GrassTypeToGrassNameMap.Find(LandscapeExporterGrassName);
							check(FoundGrassName != nullptr);
							LandscapeExporterGrassName = *FoundGrassName;
						}

						TValueOrError<TPair<FIntRect /*OutputRect*/, uint8 /*ChannelIndex*/>, FLandscapeGrassWeightExporter_RenderThread::EGrassmapTextureInfoError> TextureInfo =
							Exporter->GetTextureInfoForGrass(LandscapeComponentKeys[ComponentIndex], LandscapeExporterGrassName);
						if (TextureInfo.HasError())
						{
							// It can fail, but only if that's because the grass map is absent from this component
							checkf(TextureInfo.GetError() == FLandscapeGrassWeightExporter_RenderThread::EGrassmapTextureInfoError::InvalidGrassName,
								TEXT("Failed to retrieve grass map (%s) info on component %s : reason : %i"), *LandscapeExporterGrassName.ToString() , *LandscapeComponentKeys[ComponentIndex].ToString(), EnumToUnderlyingType(TextureInfo.GetError()));
						}
						else
						{
							TPair<FIntRect /*OutputRect*/, uint8 /*ChannelIndex*/> TextureInfoValue = TextureInfo.GetValue();
							check((TextureInfoValue.Value >= 0) && (TextureInfoValue.Value < 4));
							ComponentGrassmapOffsetAndChannel = FIntVector2(TextureInfoValue.Key.Min.X, static_cast<int32>(TextureInfoValue.Value));
						}
					}
				}
				check(ComponentGrassmapOffsetAndChannels.Num() == LandscapeComponentKeys.Num() * Context->GrassNamesToExport.Num());
				ComponentGrassmapOffsetAndChannelsBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("PCGGenerateLandscapeTexturesComponentGrassmapOffsetAndChannels"), ComponentGrassmapOffsetAndChannels);
			}
			else
			{
				ComponentGrassmapOffsetAndChannelsBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FIntVector2));;
			}
			Parameters->InComponentGrassmapOffsetAndChannels = GraphBuilder.CreateSRV(ComponentGrassmapOffsetAndChannelsBuffer);

			FRDGBufferRef OutputIndexToExportIndexBuffer;
			if (!OutputIndexToExportIndex.IsEmpty())
			{
				OutputIndexToExportIndexBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("PCGGenerateLandscapeTexturesOutputIndexToExportIndex"), OutputIndexToExportIndex);
			}
			else
			{
				OutputIndexToExportIndexBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(int32));
			}
			Parameters->InOutputIndexToExportIndex = GraphBuilder.CreateSRV(OutputIndexToExportIndexBuffer);

			TShaderMapRef<FPCGGrassMapUnpackerCS> Shader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
			const int GroupCountX = FMath::DivideAndRoundUp<int>(DispatchSize.X, FPCGGrassMapUnpackerCS::ThreadGroupDim);
			const int GroupCountY = FMath::DivideAndRoundUp<int>(DispatchSize.Y, FPCGGrassMapUnpackerCS::ThreadGroupDim);
			const int GroupCountZ = NumOutputSlices;

			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("PCGUnpackGrassMap"), ERDGPassFlags::Compute, Shader, Parameters, FIntVector(GroupCountX, GroupCountY, GroupCountZ));

			UE_LOGF(LogPCG, Verbose, "PCGUnpackGrassMap: %ls, Components %d, OutputResolution %ls, OutputSlices %u",
				*Context->GetExecutionSourceName(), LandscapeTileCoords.Num(), *OutputResolution.ToString(), NumOutputSlices);

			// Export the output textures so they can be used downstream.
			if (bHeightMapRequiresExport)
			{
				OutputTextureData->HeightMapHandle = GraphBuilder.ConvertToExternalTexture(HeightMap);
			}

			if (bGrassMapRequiresExport)
			{
				OutputTextureData->GrassMapHandle = GraphBuilder.ConvertToExternalTexture(GrassMap);
			}

			if (OutputTextureData->HeightMapHandle)
			{
				GraphBuilder.SetTextureAccessFinal(HeightMap, ERHIAccess::SRVCompute);
			}

			if (OutputTextureData->GrassMapHandle)
			{
				GraphBuilder.SetTextureAccessFinal(GrassMap, ERHIAccess::SRVCompute);
			}

			GraphBuilder.Execute();

			if (bLastGenerationPass)
			{
				ExecuteOnGameThread(UE_SOURCE_LOCATION, [ContextHandle, OutputTextureData]()
				{
					LLM_SCOPE_BYTAG(PCG);

					FPCGContext::FSharedContext<FPCGGenerateLandscapeTexturesContext> SharedContext(ContextHandle);
					if (FPCGGenerateLandscapeTexturesContext* Context = SharedContext.Get())
					{
						check(OutputTextureData);
						OutputTextureData->bRenderingComplete = true;
						Context->bIsPaused = false;
					}
				});
			}
		});

		return true;
	}
}

UPCGGenerateLandscapeTexturesSettings::UPCGGenerateLandscapeTexturesSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		bGenerateHeightMap = true;
		bOutputDataForEachSelectedGrassType = true;
	}
}

void UPCGGenerateLandscapeTexturesSettings::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::OverridableLandscapeGrassTypes)
	{
		// To maintain backwards compatibility, we have to rely on landscape grass type asset names, since that used to be the way to address grass maps in the past and automatic deprecation is not possible
		//  until the graph runs (to gain access to the landscape component's NamedGrassTypes)
		bTagDataWithLandscapeGrassTypeAssetNames = true;
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
TArray<FPCGPreConfiguredSettingsInfo> UPCGGenerateLandscapeTexturesSettings::GetPreconfiguredInfo() const
{
	TArray<FPCGPreConfiguredSettingsInfo> PreconfiguredInfo;
	PreconfiguredInfo.Emplace(0, GetDefaultNodeTitle());
	PreconfiguredInfo.Emplace(1, LOCTEXT("GenerateGrassMapsNodeTitle", "Generate Grass Maps"));

	return PreconfiguredInfo;
}
#endif

FString UPCGGenerateLandscapeTexturesSettings::GetAdditionalTitleInformation() const
{
	return (bGenerateHeightMap ? LOCTEXT("GenerateGrassMapsNodeSubtitle_HeightAndGrass", "Height and Grass Maps") : LOCTEXT("GenerateGrassMapsNodeSubtitle_GrassOnly", "Grass Maps")).ToString();
}

void UPCGGenerateLandscapeTexturesSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo)
{
	if (PreconfigureInfo.PreconfiguredIndex == 1)
	{
		bGenerateHeightMap = false;
	}
}

TArray<FPCGPinProperties> UPCGGenerateLandscapeTexturesSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	// Single landscape only for now. In the future we should iterate over all landscapes and generate grass maps for each.
	FPCGPinProperties& PinProp = PinProperties.Emplace_GetRef(
		PCGGenerateLandscapeTextures::InputPinLabel,
		EPCGDataType::Landscape,
		/*bAllowMultipleConnections=*/true,
		/*bInAllowMultipleData=*/false);
	PinProp.SetRequiredPin();

	if (bOverrideFromInput)
	{
		FPCGPinProperties& GrassTypeOverrides = PinProperties.Emplace_GetRef(PCGGenerateLandscapeTextures::GrassTypeOverridesPinLabel, EPCGDataType::Param);
		GrassTypeOverrides.SetRequiredPin();
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGenerateLandscapeTexturesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	if (bGenerateHeightMap)
	{
		FPCGPinProperties& Pin = PinProperties.Emplace_GetRef(PCGGenerateLandscapeTextures::OutputHeightPinLabel, EPCGDataType::Texture);
		Pin.bAllowMultipleData = false; // Must be set after pin properties constructed
	}

	if (bOutputTextureArray)
	{
		PinProperties.Emplace(PCGGenerateLandscapeTextures::OutputGrassMapsPinLabel, FPCGDataTypeInfoTexture2DArray::AsId(), /*bAllowMultipleConnections=*/true, /*bInAllowMultipleData=*/false);
	}
	else
	{
		// todo_pcg: Could make single data if grass map mode is "include" and only a single entry is specified.
		PinProperties.Emplace(PCGGenerateLandscapeTextures::OutputGrassMapsPinLabel, EPCGDataType::Texture, /*bAllowMultipleConnections=*/true, /*bInAllowMultipleData=*/true);
	}

	return PinProperties;
}

FPCGElementPtr UPCGGenerateLandscapeTexturesSettings::CreateElement() const
{
	return MakeShared<FPCGGenerateLandscapeTexturesElement>();
}

#if WITH_EDITOR
EPCGChangeType UPCGGenerateLandscapeTexturesSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGenerateLandscapeTexturesSettings, bOverrideFromInput)
		|| InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGenerateLandscapeTexturesSettings, bGenerateHeightMap))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

void FPCGGenerateLandscapeTexturesContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	if (HeightTextureData)
	{
		Collector.AddReferencedObject(HeightTextureData);
	}

	if (GrassMapTextureArrayData)
	{
		Collector.AddReferencedObject(GrassMapTextureArrayData);
	}

	if (BlackTexture)
	{
		Collector.AddReferencedObject(BlackTexture);
	}

	Collector.AddReferencedObjects(GrassMapTextureDatas);
	Collector.AddReferencedObjects(TexturesToStream);
}

void FPCGGenerateLandscapeTexturesElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	// todo_pcg: Technically this could be fancier and hash the set of landscape components that overlap with the generation volume.
	if (const UPCGData* Data = InParams.ExecutionSource ? InParams.ExecutionSource->GetExecutionState().GetSelfData() : nullptr)
	{
		Crc.Combine(Data->GetOrComputeCrc(/*bFullDataCrc=*/false));
	}

	OutCrc = Crc;
}

FPCGContext* FPCGGenerateLandscapeTexturesElement::CreateContext()
{
	return new FPCGGenerateLandscapeTexturesContext();
}

bool FPCGGenerateLandscapeTexturesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGenerateLandscapeTexturesElement::ExecuteInternal);

	FPCGGenerateLandscapeTexturesContext* Context = static_cast<FPCGGenerateLandscapeTexturesContext*>(InContext);
	check(Context);

	const UPCGGenerateLandscapeTexturesSettings* Settings = Context->GetInputSettings<UPCGGenerateLandscapeTexturesSettings>();
	check(Settings);

	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	if (!ExecutionSource)
	{
		return true;
	}

	// 1. Select landscape components that overlap the given bounds.
	if (!Context->bLandscapeComponentsFiltered)
	{
		const UPCGLandscapeData* LandscapeData = nullptr;
		for (const FPCGTaggedData& Data : Context->InputData.TaggedData)
		{
			if (const UPCGLandscapeData* InputLandscapeData = Cast<UPCGLandscapeData>(Data.Data))
			{
				if (!LandscapeData)
				{
					LandscapeData = InputLandscapeData;
				}
				else
				{
					PCGLog::LogWarningOnGraph(LOCTEXT("MultipleLandscapesNotSpported", "Multiple landscape data inputs not currently supported, only first will be used."), Context);
				}
			}
		}

		if (!LandscapeData)
		{
			// No input landscape, done.
			return true;
		}

		if (!UE::Landscape::SupportsRuntimeGrassMapGeneration() && (!ExecutionSource->GetExecutionState().GetWorld() || ExecutionSource->GetExecutionState().GetWorld()->IsGameWorld()))
		{
			UE_LOGF(LogPCG, Warning, "Grass map generation is disabled outside of editor worlds. Try enabling the CVar 'grass.GrassMap.AlwaysBuildRuntimeGenerationResources'.");
			return true;
		}

		TArray<FName> SelectedGrassNames;

		if (!Settings->bOverrideFromInput)
		{
			SelectedGrassNames = Settings->SelectedGrassTypes;
		}
		else
		{
			const TArray<FPCGTaggedData> OverrideTaggedDatas = InContext->InputData.GetInputsByPin(PCGGenerateLandscapeTextures::GrassTypeOverridesPinLabel);
			TArray<FName> GrassNameOverrides;

			for (const FPCGTaggedData& OverrideTaggedData : OverrideTaggedDatas)
			{
				if (const UPCGData* OverrideData = OverrideTaggedData.Data)
				{
					const FPCGAttributePropertyInputSelector Selector = Settings->GrassTypesAttribute.CopyAndFixLast(OverrideData);

					if (PCGAttributeAccessorHelpers::ExtractAllValues(OverrideData, Selector, GrassNameOverrides, InContext))
					{
						SelectedGrassNames = std::move(GrassNameOverrides);
					}
					else
					{
						PCGLog::LogWarningOnGraph(LOCTEXT("FailExtractGrassTypeOverrides", "Failed to extract grass type overrides."), InContext);
					}
				}
			}
		}

		// If we're using an inclusion list, initialize the grass types to output to nulls, one for each selection entry. The entries will be replaced as grass types are encountered below.
		const bool bOutputDataForEachSelectedGrassType = !Settings->bExcludeSelectedGrassTypes && Settings->bOutputDataForEachSelectedGrassType;
		if (bOutputDataForEachSelectedGrassType)
		{
			Context->GrassNamesToOutput.SetNum(SelectedGrassNames.Num());

			Context->OutputIndexToExportIndex.SetNumUninitialized(SelectedGrassNames.Num());
			FMemory::Memset(Context->OutputIndexToExportIndex.GetData(), -1, Context->OutputIndexToExportIndex.Num() * Context->OutputIndexToExportIndex.GetTypeSize());
		}

		// Find the first valid landscape - only one supported right now.
		for (const TSoftObjectPtr<ALandscapeProxy>& LandscapeProxyPtr : LandscapeData->Landscapes)
		{
			ALandscapeProxy* LandscapeProxy = LandscapeProxyPtr.Get();
			if (!LandscapeProxy || !LandscapeProxy->GetLandscapeMaterial())
			{
				continue;
			}

			ALandscape* Landscape = LandscapeProxy->GetLandscapeActor();
			if (!Landscape)
			{
				continue;
			}

			if (!Context->Landscape.Get())
			{
				Context->Landscape = Landscape;
			}
			// TODO [jonathan.bard] : The grass map exporter doesn't support landscape components from different landscapes, simply skip those components for now : 
			else if (Context->Landscape.Get() != Landscape)
			{
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("LandscapeMismatch", "Encountered multiple landscapes which is not supported at the moment. Only the first landscape will be considered : {0} (current : {1})"),
					FText::FromString(Context->Landscape->GetActorNameOrLabel()), FText::FromString(Landscape->GetActorNameOrLabel())), Context);
			}
		}

		if (!Context->Landscape.Get() || !Context->Landscape->GetLandscapeInfo())
		{
			return true;
		}

		// Gather all landscape components overlapping our generation bounds.
		{
			Context->GenerationBounds = ExecutionSource->GetExecutionState().GetBounds();

			const FBox2D Extent2D = FBox2D(FVector2D(Context->GenerationBounds.Min), FVector2D(Context->GenerationBounds.Max));
			TMap<FIntPoint, ULandscapeComponent*> OverlappingComponents;
			FIntRect ComponentIndicesBoundingRect;
			Context->Landscape->GetLandscapeInfo()->GetOverlappedComponents(FTransform::Identity, Extent2D, OverlappingComponents, ComponentIndicesBoundingRect);

			// Grow storage to accommodate new components.
			const int32 NumToReserve = FMath::Min(OverlappingComponents.Num(), (int32)FPCGGrassMapUnpackerCS::MaxNumLandscapeComponents);
			Context->LandscapeComponents.Reserve(NumToReserve);

			TSet<FName> ProcessedGrassNames;

			for (auto& [Coordinate, LandscapeComponent] : OverlappingComponents)
			{
				if (ensure(LandscapeComponent))
				{
					// Only generate grass map if there is meaningful overlap with our domain of interest.
					const FBox LandscapeComponentBounds = LandscapeComponent->Bounds.GetBox();
					if (LandscapeComponentBounds.Overlap(Context->GenerationBounds).GetVolume() > UE_KINDA_SMALL_NUMBER)
					{
						if (Context->LandscapeComponents.Num() >= FPCGGrassMapUnpackerCS::MaxNumLandscapeComponents)
						{
							PCGLog::LogWarningOnGraph(LOCTEXT("MaxLandscapeComponentsExceeded", "Too many landscape components overlap the generation domain. Consider partitioning the component onto a smaller grid size."), Context);
							break;
						}

						LandscapeComponent->UpdateGrassTypes();

						TMap<FName, TObjectPtr<ULandscapeGrassType>> ValidNamedGrassTypes = UE::Landscape::Grass::FilterValidNamedGrassTypes(LandscapeComponent->GetNamedGrassTypes());
						for (const TPair<FName, TObjectPtr<ULandscapeGrassType>>& ItPair : ValidNamedGrassTypes)
						{
							check(!ItPair.Key.IsNone());

							FName GrassName = ItPair.Key;

							// If bTagDataWithLandscapeGrassTypeAssetNames is used, the grass name will be the grass type asset's name, so we store a mapping between this and the actual grass name, so that the landscape grass exporter can work with the latter :
							if (Settings->bTagDataWithLandscapeGrassTypeAssetNames)
							{
								if (ItPair.Value == nullptr)
								{
									continue;
								}

								FName GrassTypeName = ItPair.Value->GetFName();
								// In this case, use the grass type's name as the actual "PCG" grass name : 
								GrassName = GrassTypeName;

								// Register the grass type name -> grass name mapping or warn if that mapping has already been registered with a different grass name
								if (FName* FoundGrassName = Context->GrassTypeToGrassNameMap.Find(GrassTypeName))
								{
									if (*FoundGrassName != ItPair.Key)
									{
										PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("DuplicateGrassNamesForGrassTypeAsset", "Found more than one grass name associated with the same grass type asset's name. "
											"Found grass name {0} for Landscape Grass Type asset {1} while grass name {2} was already registered. This only happens because bTagDataWithLandscapeGrassTypeAssetNames is used. Consider unchecking this option and change "
											"the PCG graph accordingly in order to resolve the ambiguity, since only the grass name defines the actual grass map. "),
											FText::FromString(ItPair.Key.ToString()), FText::FromString(GrassTypeName.ToString()), FText::FromString(FoundGrassName->ToString())), Context);
									}
								}
								else
								{
									Context->GrassTypeToGrassNameMap.Add(GrassTypeName, ItPair.Key);
								}
							}

							if (ProcessedGrassNames.Contains(GrassName))
							{
								continue;
							}

							ProcessedGrassNames.Add(GrassName);
							const int32 IndexInSelection = SelectedGrassNames.IndexOfByKey(GrassName);

							const bool bIsSelectedLayer = IndexInSelection != INDEX_NONE;
							// Register the grass name if it is included or not excluded.
							if (bIsSelectedLayer != Settings->bExcludeSelectedGrassTypes)
							{
								const int32 SizeBefore = Context->GrassNamesToExport.Num();
								Context->GrassNamesToExport.AddUnique(GrassName);
								// Update grass name lists if this grass name is newly added.
								if (Context->GrassNamesToExport.Num() > SizeBefore)
								{
									if (bOutputDataForEachSelectedGrassType)
									{
										// If we need to follow the prescribed list of grass names, fill in the entry here.
										Context->GrassNamesToOutput[IndexInSelection] = GrassName;
										Context->OutputIndexToExportIndex[IndexInSelection] = Context->GrassNamesToExport.Num() - 1;
									}
									else
									{
										// Otherwise build list of grass names as we encounter them, if they are not filtered out by the include/exclude list.
										Context->GrassNamesToOutput.Add(GrassName);
										Context->OutputIndexToExportIndex.Add(Context->GrassNamesToExport.Num() - 1);
									}
								}
							}
						}

						Context->LandscapeComponents.Add(LandscapeComponent);
						Context->TotalLandscapeComponentBounds += LandscapeComponentBounds;

						if (Context->LandscapeComponents.Num() == 1)
						{
							Context->LandscapeComponentExtent = LandscapeComponent->Bounds.BoxExtent.X * 2.0;
							Context->TexelSizeWorld = Context->LandscapeComponentExtent / static_cast<double>(Context->Landscape->ComponentSizeQuads);

							if (!ensure(FMath::IsNearlyEqual(LandscapeComponent->Bounds.BoxExtent.X, LandscapeComponent->Bounds.BoxExtent.Y)))
							{
								return true;
							}

							if (!ensure(Context->LandscapeComponentExtent > 0.0))
							{
								return true;
							}
						}
						else
						{
							// Currently assuming all landscape have the same extents.
							ensure(FMath::IsNearlyEqual(Context->LandscapeComponentExtent, LandscapeComponent->Bounds.BoxExtent.X * 2.0));
						}

#if WITH_EDITOR
						if (PCGGenerateLandscapeTextures::CVarDebugDrawGeneratedComponents.GetValueOnGameThread())
						{
							DrawDebugBox(ExecutionSource->GetExecutionState().GetWorld(),
								LandscapeComponentBounds.GetCenter(),
								LandscapeComponentBounds.GetExtent(),
								FColor::MakeRandomSeededColor(Context->TaskId),
								/*bPersistentLines=*/false,
								/*LifeTime=*/3.0f);
						}
#endif
					}
				}
			}
		}

		if (Context->LandscapeComponents.IsEmpty() || (Context->GrassNamesToOutput.IsEmpty() && !Settings->bGenerateHeightMap) || !Context->TotalLandscapeComponentBounds.IsValid)
		{
			return true;
		}

		// Precompute some quantities that are used in each export pass.
		{
			ALandscape* Landscape = Context->Landscape.Get();
			check(Landscape);

			Context->LandscapeRootZ = Landscape->GetRootComponent()->GetRelativeLocation().Z;
			Context->LandscapeScaleZ = Landscape->GetRootComponent()->GetRelativeScale3D().Z;

			const double QuadSizeWorld = Context->LandscapeComponentExtent / Landscape->ComponentSizeQuads;
			check(!FMath::IsNearlyZero(QuadSizeWorld));

			// Texel-space offset of the generation bounds from the landscape component bounds.
			FVector2D OutputOffsetTexelsF;
			OutputOffsetTexelsF.X = (Context->GenerationBounds.Min.X - Context->TotalLandscapeComponentBounds.Min.X) / QuadSizeWorld;
			OutputOffsetTexelsF.Y = (Context->GenerationBounds.Min.Y - Context->TotalLandscapeComponentBounds.Min.Y) / QuadSizeWorld;
 
			// Round down so that we aren't in between texels.
			FVector2D OutputOffsetTexels;
			OutputOffsetTexels.X = FMath::FloorToInt(OutputOffsetTexelsF.X);
			OutputOffsetTexels.Y = FMath::FloorToInt(OutputOffsetTexelsF.Y);

			// A non-zero fraction means the generation bounds and landscape bounds don't lie on the same texel grid, so we'll have to grow the
			// output texture extents by one texel in order to cover the full generation bounds.
			const FVector2D OutputOffsetTexelsFrac = OutputOffsetTexelsF - OutputOffsetTexels;

			// Extents of the output should be the generation bounds, rounded up so that we aren't in between texels.
			// Need to grow the extents by one texel if the generation bounds don't line up with the landscape component bounds in texel space.
			FIntPoint OutputExtentTexels;
			OutputExtentTexels.X = FMath::CeilToInt((Context->GenerationBounds.Max.X - Context->GenerationBounds.Min.X) / QuadSizeWorld) + FMath::CeilToInt(OutputOffsetTexelsFrac.X);
			OutputExtentTexels.Y = FMath::CeilToInt((Context->GenerationBounds.Max.Y - Context->GenerationBounds.Min.Y) / QuadSizeWorld) + FMath::CeilToInt(OutputOffsetTexelsFrac.Y);

			// Dilate the horizontal extents by half a texel in both X and Y. This will place the texel centers on the vert grid.
			OutputOffsetTexels.X -= 0.5;
			OutputOffsetTexels.Y -= 0.5;
			OutputExtentTexels.X += 1;
			OutputExtentTexels.Y += 1;

			// Apply offset (in world space) to the landscape component bounds to find the output bounds.
			FVector2D OutputBoundsMin;
			OutputBoundsMin.X = (OutputOffsetTexels.X * QuadSizeWorld) + Context->TotalLandscapeComponentBounds.Min.X;
			OutputBoundsMin.Y = (OutputOffsetTexels.Y * QuadSizeWorld) + Context->TotalLandscapeComponentBounds.Min.Y;

			// Convert texel space back to world space.
			const FVector2D OutputExtents = OutputExtentTexels * QuadSizeWorld;

			FBox OutputBounds;
			OutputBounds.Min.X = OutputBoundsMin.X;
			OutputBounds.Min.Y = OutputBoundsMin.Y;
			OutputBounds.Max.X = OutputBoundsMin.X + OutputExtents.X;
			OutputBounds.Max.Y = OutputBoundsMin.Y + OutputExtents.Y;
			OutputBounds.Min.Z = Context->TotalLandscapeComponentBounds.Min.Z;
			OutputBounds.Max.Z = Context->TotalLandscapeComponentBounds.Max.Z;

			Context->OutputTextureTransform = FTransform(FQuat::Identity, OutputBounds.GetCenter(), OutputBounds.GetExtent());
			Context->OutputTextureBounds = OutputBounds;

			// Output resolution should be the dimensions (in texels) of the output bounds.
			Context->OutputResolution = OutputExtentTexels;
		}

		Context->bLandscapeComponentsFiltered = true;
	}

	// 2. Wait for landscape components to be ready for grass map rendering.
	if (!Context->bTextureStreamingRequested)
	{
		if (const UWorld* World = ExecutionSource->GetExecutionState().GetWorld())
		{
			for (TWeakObjectPtr<ULandscapeComponent> LandscapeComponentWeak : Context->LandscapeComponents)
			{
				if (ULandscapeComponent* LandscapeComponent = LandscapeComponentWeak.Get())
				{
					// Make list of textures to stream before generating.
					if (UTexture* HeightMap = LandscapeComponent->GetHeightmap())
					{
						Context->TexturesToStream.Add(HeightMap);
					}

					if (Context->GrassNamesToOutput.Num() > 0)
					{
						const ERHIFeatureLevel::Type FeatureLevel = World->GetFeatureLevel();
						for (UTexture2D* WeightmapTexture : LandscapeComponent->GetRenderedWeightmapTexturesForFeatureLevel(FeatureLevel))
						{
							if (WeightmapTexture)
							{
								Context->TexturesToStream.Add(WeightmapTexture);
							}
						}
					}
				}
			}

			for (UTexture* Texture : Context->TexturesToStream)
			{
				Texture->bForceMiplevelsToBeResident = true;
			}
		}

		Context->bTextureStreamingRequested = true;
	}

	if (!Context->bReadyToRender)
	{
		bool bAllReady = true;

		for (TWeakObjectPtr<ULandscapeComponent> LandscapeComponentWeak : Context->LandscapeComponents)
		{
			ULandscapeComponent* LandscapeComponent = LandscapeComponentWeak.Get();
			if (!LandscapeComponent)
			{
				PCGLog::LogErrorOnGraph(PCGGenerateLandscapeTextures::LandscapeComponentLostError, InContext);
				return true;
			}

			bAllReady &= LandscapeComponent->IsRegistered();
			bAllReady &= LandscapeComponent->CanRenderGrassMap();
		}

		if (bAllReady)
		{
			for (UTexture* Texture : Context->TexturesToStream)
			{
				const bool bStreamedIn = PCGGenerateLandscapeTextures::IsTextureFullyStreamedIn(Texture);
				bAllReady &= bStreamedIn;

				if (!bStreamedIn)
				{
					UE_LOGF(LogPCG, Verbose, "Waiting for landscape texture '%ls' to stream in.", *Texture->GetName());
					break;
				}
			}
		}
		
		if (!bAllReady)
		{
#if UE_ENABLE_DEBUG_DRAWING
			if (PCGSystemSwitches::CVarPCGDebugDrawGeneratedCells.GetValueOnGameThread())
			{
				FColor DebugColor = FColor::Yellow;
				PCGHelpers::DebugDrawGenerationVolume(InContext, &DebugColor);
			}
#endif

			// Sleep until next frame, no use spinning on this.
			Context->bIsPaused = true;
			FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = InContext->GetOrCreateHandle()]()
			{
				if (TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin())
				{
					if (FPCGContext* ContextPtr = SharedHandle->GetContext())
					{
						ContextPtr->bIsPaused = false;
					}
				}
			});

			return false;
		}

		Context->bReadyToRender = true;
	}

	// 3. Schedule grass map generation.
	if (!Context->bGenerationScheduled)
	{
		ALandscape* Landscape = Context->Landscape.IsValid() ? Context->Landscape.Get() : nullptr;
		if (!Landscape)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("LandscapeLost", "Reference to landscape actor lost, grass maps will not be generated."), InContext);
			return true;
		}

		TArray<ULandscapeComponent*> LandscapeComponents;
		LandscapeComponents.Reserve(FMath::Min(Context->LandscapeComponents.Num(), Settings->NumLandscapeComponentsGeneratedPerFrame));
		
		// Stores the global identifier for a landscape component so that we can query info about this component in the grass map exporter
		TArray<FIntPoint> LandscapeComponentKeys;
		LandscapeComponentKeys.Reserve(FMath::Min(Context->LandscapeComponents.Num(), Settings->NumLandscapeComponentsGeneratedPerFrame));

		FBox LandscapeComponentBounds = FBox(EForceInit::ForceInit);

		int32 NumComponentsToGenerate = 0;

		for (int32 Index = Context->NumGenerationPassesScheduled; Index < Context->LandscapeComponents.Num(); ++Index)
		{
			if (!Context->LandscapeComponents[Index].IsValid())
			{
				PCGLog::LogErrorOnGraph(PCGGenerateLandscapeTextures::LandscapeComponentLostError, InContext);
				return true;
			}

			ULandscapeComponent* LandscapeComponent = Context->LandscapeComponents[Index].Get();
			LandscapeComponents.Add(LandscapeComponent);
			LandscapeComponentKeys.Add(LandscapeComponent->GetComponentKey());
			LandscapeComponentBounds += LandscapeComponent->Bounds.GetBox();

			++NumComponentsToGenerate;
			if (NumComponentsToGenerate >= Settings->NumLandscapeComponentsGeneratedPerFrame)
			{
				break;
			}
		}

		if (!LandscapeComponentBounds.IsValid)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("LandscapeComonentBoundsInvalid", "Landscape component encountered with invalid bounds, grass maps will not be generated."), InContext);
			return true;
		}

		TArray<FIntVector2> LandscapeTileCoords;
		LandscapeTileCoords.Reserve(LandscapeComponents.Num());

		for (ULandscapeComponent* LandscapeComponent : LandscapeComponents)
		{
			check(LandscapeComponent);

			// Landscape components are not ordered, so store a 2d index of each component within the grass map.
			// TODO could likely sort the LandscapeComponents array so the order is known and no indices need looking up.
			LandscapeTileCoords.Add(FIntVector2(
				(LandscapeComponent->Bounds.Origin.X - LandscapeComponentBounds.Min.X) / Context->LandscapeComponentExtent,
				(LandscapeComponent->Bounds.Origin.Y - LandscapeComponentBounds.Min.Y) / Context->LandscapeComponentExtent));
		}

		if (!Context->OutputTextureData)
		{
			Context->OutputTextureData = MakeShared<FPCGGenerateLandscapeTexturesContext::FOutputTextureData>();
		}

		const bool bLastGenerationPass = (Context->NumGenerationPassesScheduled + NumComponentsToGenerate) >= Context->LandscapeComponents.Num();
		
		PCGGenerateLandscapeTextures::FGenerateComponentsData Data
		{
			.Context = Context,
			.LandscapeComponents = MoveTemp(LandscapeComponents),
			.Settings = Settings,
			.Landscape = Landscape,
			.LandscapeTileCoords = MoveTemp(LandscapeTileCoords),
			.LandscapeComponentKeys = MoveTemp(LandscapeComponentKeys),
			.LandscapeComponentBounds = LandscapeComponentBounds,
			.bLastGenerationPass = bLastGenerationPass,
		};

		Context->NumGenerationPassesScheduled += NumComponentsToGenerate;

		if (PCGGenerateLandscapeTextures::GenerateComponents(MoveTemp(Data)))
		{
			if (bLastGenerationPass)
			{
				Context->bGenerationScheduled = true;
				// Render command will wake this task up after completing.
				Context->bIsPaused = true;
			}
			else
			{
				// Sleep until next frame to timeslice generation passes across frames.
				Context->bIsPaused = true;
				FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
				{
					if (TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin())
					{
						if (FPCGContext* ContextPtr = SharedHandle->GetContext())
						{
							ContextPtr->bIsPaused = false;
						}
					}
				});
			}

			return false;
		}
		else
		{
			// Failed, terminate element.
			return true;
		}
	}

	// 4. Initialize texture data objects.

	if (Settings->bGenerateHeightMap && !Context->HeightTextureData)
	{
		Context->HeightTextureData = FPCGContext::NewObject_AnyThread<UPCGTextureData>(Context);
	}

	if (!Context->bCreatedOutputGrassTextures)
	{
		if (Settings->bOutputTextureArray)
		{
			Context->GrassMapTextureArrayData = FPCGContext::NewObject_AnyThread<UPCGTexture2DArrayData>(Context);
		}
		else
		{
			// Create the texture data objects if they haven't been created already. There should be one per selected grass name.
			for (int DataIndex = 0; DataIndex < Context->GrassNamesToOutput.Num(); ++DataIndex)
			{
				Context->GrassMapTextureDatas.Add(FPCGContext::NewObject_AnyThread<UPCGTextureData>(Context));
			}
		}

		Context->bCreatedOutputGrassTextures = true;
	}

	if (!Context->OutputTextureData || (!Context->OutputTextureData->GrassMapHandle.IsValid() && !Context->OutputTextureData->HeightMapHandle.IsValid()) || !ensure(Context->OutputTextureData->bRenderingComplete))
	{
		return true;
	}

	bool bAllTexturesInitialized = true;

	if (Context->OutputTextureData->HeightMapHandle)
	{
		check(Context->HeightTextureData);
		bAllTexturesInitialized &= Context->HeightTextureData->Initialize(Context->OutputTextureData->HeightMapHandle, /*TextureIndex=*/0, Context->OutputTextureTransform);
	}

	if (Context->OutputTextureData->GrassMapHandle)
	{
		if (Settings->bOutputTextureArray)
		{
			FPCGTexture2DArrayDataInitParams InitParams;
			InitParams.Transform = Context->OutputTextureTransform;

			bAllTexturesInitialized &= Context->GrassMapTextureArrayData->Initialize(Context->OutputTextureData->GrassMapHandle, InitParams);
		}
		else
		{
			for (int DataIndex = 0; DataIndex < Context->GrassNamesToOutput.Num(); ++DataIndex)
			{
				check(Context->GrassMapTextureDatas.IsValidIndex(DataIndex));
				UPCGTextureData* TextureData = Context->GrassMapTextureDatas[DataIndex];
				check(TextureData);

				const int32 ExportIndex = Context->GrassNamesToOutput[DataIndex].IsNone() ? INDEX_NONE : Context->GrassNamesToExport.IndexOfByKey(Context->GrassNamesToOutput[DataIndex]);

				if (ExportIndex != INDEX_NONE)
				{
					// Poll initialize (fine to be called even when initialization was already complete).
					bAllTexturesInitialized &= TextureData->Initialize(
						Context->OutputTextureData->GrassMapHandle,
						/*TextureIndex=*/DataIndex,
						Context->OutputTextureTransform);
				}
				else
				{
					if (!Context->BlackTexture)
					{
						// Expectation is this case should be rare, and black texture generally always loaded.
						Context->BlackTexture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EngineResources/Black.Black"));
					}

					bAllTexturesInitialized &= TextureData->Initialize(
						Context->BlackTexture,
						/*TextureIndex=*/0,
						Context->OutputTextureTransform);
				}
			}
		}
	}

	if (!bAllTexturesInitialized)
	{
		// Initialization not complete. Could be waiting on async texture processing. Sleep until next frame.
		Context->bIsPaused = true;
		FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
		{
			FPCGContext::FSharedContext<FPCGGenerateLandscapeTexturesContext> SharedContext(ContextHandle);
			if (FPCGGenerateLandscapeTexturesContext* ContextPtr = SharedContext.Get())
			{
				ContextPtr->bIsPaused = false;
			}
		});

		return false;
	}

	// 5. Emit texture data objects.
	if (Context->HeightTextureData)
	{
		if (Context->HeightTextureData->IsSuccessfullyInitialized())
		{
			FPCGTaggedData& OutTaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
			OutTaggedData.Data = Context->HeightTextureData;
			OutTaggedData.Pin = PCGGenerateLandscapeTextures::OutputHeightPinLabel;
		}
		else
		{
			PCGE_LOG(Warning, LogOnly, LOCTEXT("HeightTextureInitFailed", "Data could not be retrieved for height texture, initialization failed."));
		}
	}

	if (Settings->bOutputTextureArray)
	{
		FPCGTaggedData& OutTaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
		OutTaggedData.Pin = PCGGenerateLandscapeTextures::OutputGrassMapsPinLabel;
		OutTaggedData.Data = Context->GrassMapTextureArrayData;
	}
	else
	{
		FString TagString; // Used in loop but hoisted for efficiency.

		for (int DataIndex = 0; DataIndex < Context->GrassNamesToOutput.Num(); ++DataIndex)
		{
			check(Context->GrassMapTextureDatas.IsValidIndex(DataIndex));
			UPCGTextureData* TextureData = Context->GrassMapTextureDatas[DataIndex];
			check(TextureData);

			if (!TextureData->IsSuccessfullyInitialized())
			{
				PCGE_LOG(Warning, LogOnly, LOCTEXT("TextureInitFailed", "Data could not be retrieved for this texture, initialization failed."));
				continue;
			}

			FPCGTaggedData& OutTaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
			OutTaggedData.Pin = PCGGenerateLandscapeTextures::OutputGrassMapsPinLabel;
			OutTaggedData.Data = TextureData;

			FName GrassName = Context->GrassNamesToOutput[DataIndex];
			if (!GrassName.IsNone())
			{
				PCGComputeHelpers::GetPrefixedDataLabel(GrassName.ToString(), TagString);
				OutTaggedData.Tags.Add(TagString);
			}
			else if (!Settings->bExcludeSelectedGrassTypes && Settings->SelectedGrassTypes.IsValidIndex(DataIndex))
			{
				// We are using inclusion list, look up name
				PCGComputeHelpers::GetPrefixedDataLabel(Settings->SelectedGrassTypes[DataIndex].ToString(), TagString);
				OutTaggedData.Tags.Add(TagString);
			}
			else
			{
				PCGE_LOG(Warning, LogOnly, LOCTEXT("MissingGrassName",
					"Grass type name was missing, data could not be labeled. Make sure all GrassTypes in your landscape material have an asset associated."));
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
