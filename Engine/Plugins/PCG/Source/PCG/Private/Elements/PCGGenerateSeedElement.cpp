// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGenerateSeedElement.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGenerateSeedElement)

namespace PCGGeneratedSeedElement::Constants
{
	const FName DefaultGeneratedSeedAttributeName = TEXT("GeneratedSeed");
}

#define LOCTEXT_NAMESPACE "PCGGenerateSeedElement"

UPCGGenerateSeedSettings::UPCGGenerateSeedSettings()
{
	OutputTarget.SetAttributeName(PCGGeneratedSeedElement::Constants::DefaultGeneratedSeedAttributeName);
}

TArray<FPCGPinProperties> UPCGGenerateSeedSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any).SetRequiredPin();
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGenerateSeedSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
	return PinProperties;
}

FPCGElementPtr UPCGGenerateSeedSettings::CreateElement() const
{
	return MakeShared<FPCGGenerateSeedElement>();
}

bool FPCGGenerateSeedElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGenerateSeedElement::Execute);

	const UPCGGenerateSeedSettings* Settings = Context->GetInputSettings<UPCGGenerateSeedSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	const int32 InitialSeed = Context->GetSeed();
	FRandomStream RandomStream(InitialSeed);

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGMetadata* InputMetadata = Input.Data->ConstMetadata();
		check(InputMetadata);

		UPCGData* OutputData = Input.Data->DuplicateData(Context);
		UPCGMetadata* OutputMetadata = OutputData->MutableMetadata();
		check(OutputMetadata);

		FPCGAttributePropertyInputSelector SourceSelector = Settings->SeedSource.CopyAndFixLast(Input.Data);
		FPCGAttributePropertyOutputSelector OutputSelector = Settings->OutputTarget.CopyAndFixSource(&SourceSelector, Input.Data);
		// Validate that the attribute exists or create a new one. If it's not a basic attribute, it must exist already, so skip and let the accessor be validated later.
		if (OutputSelector.GetSelection() == EPCGAttributePropertySelection::Attribute && OutputSelector.IsBasicAttribute())
		{
			const FPCGMetadataAttribute<int32>* SeedAttribute = OutputMetadata->FindOrCreateAttribute<int32>(OutputSelector, InitialSeed);
			if (!SeedAttribute)
			{
				PCGLog::Metadata::LogFailToCreateAttributeError<int32>(OutputSelector.GetName(), Context);
				continue;
			}
		}

		const TUniquePtr<IPCGAttributeAccessor> OutputAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputData, OutputSelector);
		const TUniquePtr<IPCGAttributeAccessorKeys> OutputKeys = PCGAttributeAccessorHelpers::CreateKeys(OutputData, OutputSelector);
		if (!OutputAccessor || !OutputKeys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(OutputSelector, Context);
			continue;
		}

		// For users that want deterministic seeds across multiple data.
		if (Settings->bResetSeedPerInput)
		{
			RandomStream.Reset();
		}

		const int32 NumElements = OutputKeys->GetNum();
		TArray<int32> Seeds;
		Seeds.SetNumUninitialized(NumElements);
		for (int32 I = 0; I < NumElements; ++I)
		{
			// Random unsigned int to signed int is fine. Note: Seed is mutated.
			Seeds[I] = static_cast<int32>(RandomStream.GetUnsignedInt());
		}

		if (Settings->GenerationSource == EPCGGenerateSeedSource::HashStringConstant)
		{
			const uint32 Hash = PCG::Private::MetadataTraits<FString>::Hash(Settings->SourceString);
			for (int32& Seed : Seeds)
			{
				Seed = HashCombineFast(Hash, Seed);
			}
		}
		else if (Settings->GenerationSource == EPCGGenerateSeedSource::HashEachSourceAttribute)
		{
			const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, SourceSelector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, SourceSelector);
			if (!Accessor || !Keys)
			{
				PCGLog::Metadata::LogFailToGetAttributeError(SourceSelector, Context);
				break;
			}

			// Attribute hash is element-wise so apply on all seed attributes
			PCGMetadataAttribute::CallbackWithRightType(Accessor->GetUnderlyingType(), [&Seeds, &Accessor, &Keys]<typename AttributeType>(AttributeType)
			{
				PCGMetadataElementCommon::ApplyOnAccessor<AttributeType>(*Keys, *Accessor, [&Seeds](const AttributeType& Value, const int32 Index)
				{
					const uint32 Hash = PCG::Private::MetadataTraits<AttributeType>::Hash(Value);
					Seeds[Index] = HashCombineFast(Hash, Seeds[Index]);
				});
			});
		}

		if (!OutputAccessor->SetRange(MakeConstArrayView(Seeds), 0, *OutputKeys))
		{
			PCGLog::Metadata::LogFailToSetAttributeError<int32>(OutputSelector, OutputAccessor.Get(), Context);
			continue;
		}

		FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef(Input);
		Output.Pin = PCGPinConstants::DefaultOutputLabel;
		Output.Data = OutputData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
