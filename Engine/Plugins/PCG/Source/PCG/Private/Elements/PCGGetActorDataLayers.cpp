// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetActorDataLayers.h"

#include "PCGContext.h"
#include "PCGData.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGDataLayerHelpers.h"
#include "Utils/PCGLogErrors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetActorDataLayers)

#define LOCTEXT_NAMESPACE "PCGGetActorDataLayers"

UPCGGetActorDataLayersSettings::UPCGGetActorDataLayersSettings()
{
	ActorReferenceAttribute.SetAttributeName(PCGPointDataConstants::ActorReferenceAttribute);
	DataLayerReferenceAttribute.SetAttributeName(PCGDataLayerHelpers::Constants::DataLayerReferenceAttribute);
}

TArray<FPCGPinProperties> UPCGGetActorDataLayersSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::PointOrParam);
	InputPin.SetRequiredPin();
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGetActorDataLayersSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);
	return PinProperties;
}

FString UPCGGetActorDataLayersSettings::GetAdditionalTitleInformation() const
{
	return FString::Printf(TEXT("%s -> %s"), *ActorReferenceAttribute.GetDisplayText().ToString(), *DataLayerReferenceAttribute.GetDisplayText().ToString());
}

FPCGElementPtr UPCGGetActorDataLayersSettings::CreateElement() const
{
	return MakeShared<FPCGGetActorDataLayersElement>();
}

bool FPCGGetActorDataLayersElement::ExecuteInternal(FPCGContext* Context) const
{
	const UPCGGetActorDataLayersSettings* Settings = Context->GetInputSettings<UPCGGetActorDataLayersSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		if (!Input.Data)
		{
			continue;
		}

		// Add PCGParam Output with DataLayers
		FPCGTaggedData& DataLayerSetOutput = Context->OutputData.TaggedData.Emplace_GetRef(Input);
		UPCGParamData* OutputParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		DataLayerSetOutput.Pin = PCGPinConstants::DefaultOutputLabel;
		DataLayerSetOutput.Data = OutputParamData;

#if WITH_EDITOR
		FPCGMetadataAttribute<FSoftObjectPath>* DataLayersAttribute = OutputParamData->MutableMetadata()->CreateAttribute(Settings->DataLayerReferenceAttribute.GetName(), FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);

		TArray<FSoftObjectPath> DataLayerAssets = PCGDataLayerHelpers::GetDataLayerAssetsFromActorReferences(Context, Input.Data, Settings->ActorReferenceAttribute);
		for (const FSoftObjectPath& ActorDataLayer : DataLayerAssets)
		{
			PCGMetadataEntryKey Entry = OutputParamData->MutableMetadata()->AddEntry();
			DataLayersAttribute->SetValue(Entry, ActorDataLayer);
		}
#endif
	}

#if !WITH_EDITOR
	PCGLog::LogErrorOnGraph(LOCTEXT("GetActorDataLayersUnsupported", "Get Actor Data Layers is unsupported at runtime"), Context);
#endif

	return true;
}

#undef LOCTEXT_NAMESPACE
