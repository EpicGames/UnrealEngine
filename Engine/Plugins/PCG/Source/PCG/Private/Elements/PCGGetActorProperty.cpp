// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetActorProperty.h"

#include "GameFramework/Actor.h"
#include "PCGContext.h"
#include "PCGComponent.h"
#include "PCGParamData.h"
#include "Data/PCGPointData.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetActorProperty)

#define LOCTEXT_NAMESPACE "PCGPropertyToParamDataElement"

UPCGGetActorPropertySettings::UPCGGetActorPropertySettings(const FObjectInitializer& ObjectInitializer)
{
	// To not break previous behavior, we keep this false by default, but will force it to true for new nodes as it makes more sense to process them all,
	// as we have no guarantee on the order components are found. Also the new default is @Source for the output attribute
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		bProcessAllComponents = true;
		OutputAttributeName.SetAttributeName(PCGMetadataAttributeConstants::SourceAttributeName);
	}
	else
	{
		OutputAttributeName.SetAttributeName(NAME_None);
	}
}

#if WITH_EDITOR
void UPCGGetActorPropertySettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	OutKeysToSettings.FindOrAdd(ActorSelector.GetAssociatedKey()).Emplace(this, bTrackActorsOnlyWithinBounds);
}

void UPCGGetActorPropertySettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGGetActorPropertySettings, ActorSelector))
	{
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FPCGActorSelectorSettings, ActorSelection))
		{
			if (ActorSelector.ActorSelection != EPCGActorSelection::ByClass)
			{
				ActorSelector.ActorSelectionClass = TSubclassOf<AActor>();
			}
		}
	}
}

EPCGChangeType UPCGGetActorPropertySettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(PropertyChangedEvent) | EPCGChangeType::Cosmetic;

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGGetActorPropertySettings, ActorSelector))
	{
		// If we change from/to FromInput, this needs to trigger a graph recompilation especially for culling.
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FPCGActorSelectorSettings, ActorFilter))
		{
			ChangeType |= EPCGChangeType::Structural;
		}
	}

	return ChangeType;
}

#endif // WITH_EDITOR

void UPCGGetActorPropertySettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Migrate deprecated actor selection settings to struct if needed
	if (ActorSelection_DEPRECATED != EPCGActorSelection::ByTag ||
		ActorSelectionTag_DEPRECATED != NAME_None ||
		ActorSelectionName_DEPRECATED != NAME_None ||
		ActorSelectionClass_DEPRECATED != TSubclassOf<AActor>() ||
		ActorFilter_DEPRECATED != EPCGActorFilter::Self ||
		bIncludeChildren_DEPRECATED != false)
	{
		ActorSelector.ActorSelection = ActorSelection_DEPRECATED;
		ActorSelector.ActorSelectionTag = ActorSelectionTag_DEPRECATED;
		ActorSelector.ActorSelectionClass = ActorSelectionClass_DEPRECATED;
		ActorSelector.ActorFilter = ActorFilter_DEPRECATED;
		ActorSelector.bIncludeChildren = bIncludeChildren_DEPRECATED;

		ActorSelection_DEPRECATED = EPCGActorSelection::ByTag;
		ActorSelectionTag_DEPRECATED = NAME_None;
		ActorSelectionName_DEPRECATED = NAME_None;
		ActorSelectionClass_DEPRECATED = TSubclassOf<AActor>();
		ActorFilter_DEPRECATED = EPCGActorFilter::Self;
		bIncludeChildren_DEPRECATED = false;
	}

	if (ActorSelector.ActorSelection != EPCGActorSelection::ByClass)
	{
		ActorSelector.ActorSelectionClass = TSubclassOf<AActor>();
	}

	if (GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::OptionSanitizeOutputAttributeNamesPCG)
	{
		// For all previous nodes, we'll force this option to false. for retro-compatibility
		bSanitizeOutputAttributeName = false;
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bForceObjectAndStructExtraction)
	{
		bForceObjectAndStructExtraction = false;
		StructExtractorBehavior = EPCGStructExtractorBehavior::Extract;
		ObjectExtractorBehavior = EPCGObjectExtractorBehavior::Extract;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

FString UPCGGetActorPropertySettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	const FText Suffix = ActorSelector.GetTaskNameSuffix();
	return Suffix.IsEmpty()
		? FString::Printf(TEXT("%s, %s"), *ActorSelector.GetTaskName().ToString(), *PropertyName.ToString())
		: FString::Printf(TEXT("%s: %s, %s"), *ActorSelector.GetTaskName().ToString(), *Suffix.ToString(), *PropertyName.ToString());
#else
	return Super::GetAdditionalTitleInformation();
#endif
}

TArray<FPCGPinProperties> UPCGGetActorPropertySettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	if (ActorSelector.ActorFilter == EPCGActorFilter::FromInput)
	{
		PinProperties.Emplace(PCGPinConstants::DefaultInputLabel,
			EPCGDataType::Any,
			/*bAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true);
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGetActorPropertySettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FPCGDataTypeIdentifier UPCGGetActorPropertySettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	check(InPin);
	// Implementation note: since we can have an input pin in some instances, we can't rely on the
	// base class to provide the proper output type, as this will override the values set in the OutputPinProperties.
	return InPin->Properties.AllowedTypes;
}

FPCGElementPtr UPCGGetActorPropertySettings::CreateElement() const
{
	return MakeShared<FPCGGetActorPropertyElement>();
}

bool FPCGGetActorPropertyElement::IsCacheable(const UPCGSettings* InSettings) const
{
	if (const UPCGGetActorPropertySettings* Settings = Cast<UPCGGetActorPropertySettings>(InSettings))
	{
		return !Settings->bAlwaysRequeryActors;
	}
	else
	{
		return false;
	}
}

void FPCGGetActorPropertyElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	// If we track self or original, we are dependent on the actor data
	if (const UPCGGetActorPropertySettings* Settings = Cast<const UPCGGetActorPropertySettings>(InParams.Settings))
	{
		if (Settings->bAlwaysRequeryActors)
		{
			return;
		}

		if (InParams.ExecutionSource)
		{
			const bool bDependsOnSelfOrHierarchy = (Settings->ActorSelector.ActorFilter == EPCGActorFilter::Self || Settings->ActorSelector.ActorFilter == EPCGActorFilter::Original);
			IPCGGraphExecutionSource* ExecutionSourceToCheck = (Settings->ActorSelector.ActorFilter == EPCGActorFilter::Original) ? InParams.ExecutionSource->GetExecutionState().GetOriginalSource() : InParams.ExecutionSource;

			// If it is depending on Self, we need to CRC the tags (as they will be passed to the output data)
			// and also the owner path.
			if (bDependsOnSelfOrHierarchy && ExecutionSourceToCheck)
			{
				if (const AActor* Owner = ExecutionSourceToCheck->GetExecutionState().GetTypedTarget<AActor>())
				{
					Crc.Combine(UPCGActorHelpers::ComputeHashFromActorTagsAndReference(Owner, /*bIncludeTags=*/true, /*bIncludeActorReference=*/true));
				}
			}
		}
	}
}

bool FPCGGetActorPropertyElement::PrepareDataInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetActorPropertyElement::PrepareData);
	check(Context);

	const UPCGGetActorPropertySettings* Settings = Context->GetInputSettings<UPCGGetActorPropertySettings>();
	check(Settings);

	if (Settings->ActorSelector.ActorFilter == EPCGActorFilter::FromInput)
	{
		FPCGLoadObjectsFromPathContext* ThisContext = static_cast<FPCGLoadObjectsFromPathContext*>(Context);
		return ThisContext->InitializeAndRequestLoad(PCGPinConstants::DefaultInputLabel,
			Settings->ActorSelector.ActorReferenceSelector,
			{},
			/*bPersistAllData=*/false,
			/*bSilenceErrorOnEmptyObjectPath=*/true,
			/*bSynchronousLoad=*/false);
	}
	else
	{
		return true;
	}

}

bool FPCGGetActorPropertyElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetActorPropertyElement::Execute);

	check(InContext);
	FPCGLoadObjectsFromPathContext* Context = static_cast<FPCGLoadObjectsFromPathContext*>(InContext);

	const UPCGGetActorPropertySettings* Settings = Context->GetInputSettings<UPCGGetActorPropertySettings>();
	check(Settings);

	// Early out if arguments are not specified
	if (Settings->bSelectComponent && !Settings->ComponentClass)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("ParametersMissing", "Some parameters are missing, aborting"));
		return true;
	}

#if !WITH_EDITOR
	// If we have no output connected, nothing to do
	// Optimization possibly only in non-editor builds, otherwise we could poison the input-driven cache
	if (!Context->Node || !Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultOutputLabel))
	{
		PCGE_LOG(Verbose, LogOnly, LOCTEXT("UnconnectedNode", "Node is not connected, nothing to do"));
		return true;
	}
#endif

	// First find the actor depending on the selection
	UPCGComponent* OriginalComponent = UPCGBlueprintHelpers::GetOriginalComponent(*Context);
	TFunction<bool(const AActor*)> BoundsCheck = [](const AActor*) -> bool { return true; };
	auto NoSelfIgnoreCheck = [](const AActor*) -> bool { return true; };

	if (OriginalComponent && OriginalComponent->GetOwner() && Settings->ActorSelector.bMustOverlapSelf)
	{
		const FBox ActorBounds = PCGHelpers::GetActorBounds(OriginalComponent->GetOwner());
		BoundsCheck = [Settings, ActorBounds, OriginalComponent](const AActor* OtherActor) -> bool
		{
			const FBox OtherActorBounds = OtherActor ? PCGHelpers::GetGridBounds(OtherActor, OriginalComponent) : FBox(EForceInit::ForceInit);
			return ActorBounds.Intersect(OtherActorBounds);
		};
	}

	TArray<AActor*> ActorsFromInput;
	if (Settings->ActorSelector.ActorFilter == EPCGActorFilter::FromInput)
	{
		Algo::Transform(Context->PathsToObjectsAndDataIndex, ActorsFromInput, [](const TTuple<FSoftObjectPath, int32, int32>& InPath) { return Cast<AActor>(InPath.Get<0>().ResolveObject()); });
	}

	TArray<AActor*> FoundActors = PCGActorSelector::FindActors(Settings->ActorSelector, OriginalComponent, BoundsCheck, NoSelfIgnoreCheck, ActorsFromInput);

#if WITH_EDITOR
	// Setup dynamic tracking if needed
	if (Settings->ActorSelector.ActorFilter == EPCGActorFilter::FromInput)
	{
		FPCGDynamicTrackingHelper DynamicTracking;
		DynamicTracking.EnableAndInitialize(Context, Context->PathsToObjectsAndDataIndex.Num());
		for (const TTuple<FSoftObjectPath, int32, int32>& Path : Context->PathsToObjectsAndDataIndex)
		{
			// Skip tracking for references that weren't successfully resolved
			if (!Path.Get<0>().ResolveObject())
			{
				continue;
			}

			DynamicTracking.AddToTracking(FPCGSelectionKey::CreateFromPath(Path.Get<0>()), /*bIsCulled=*/Settings->bTrackActorsOnlyWithinBounds);
		}

		DynamicTracking.Finalize(Context);
	}
	else if (Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGGetActorPropertySettings, ActorSelector)))
	{
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, Settings->ActorSelector);
	}
#endif // WITH_EDITOR

	if (FoundActors.IsEmpty())
	{
		PCGE_LOG(Verbose, LogOnly, LOCTEXT("NoActorFound", "No matching actor was found"));
		return true;
	}

	for (AActor* FoundActor : FoundActors)
	{
		if (!FoundActor)
		{
			continue;
		}

		// From there, we either check the actor, or the component attached to it.
		TArray<UObject*> ObjectsToInspect;
		if (Settings->bSelectComponent)
		{
			TInlineComponentArray<UActorComponent*> ActorComponents;
			FoundActor->GetComponents(Settings->ComponentClass, ActorComponents);
			if (ActorComponents.IsEmpty())
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("ComponentDoesNotExist", "Component class '{0}' does not exist in the found actor {1}"), FText::FromString(Settings->ComponentClass->GetName()), FText::FromString(FoundActor->GetName())));
				return true;
			}

			if (Settings->bProcessAllComponents)
			{
				ObjectsToInspect.Append(ActorComponents);
			}
			else
			{
				ObjectsToInspect.Add(ActorComponents[0]);
			}
		}
		else
		{
			ObjectsToInspect.Add(FoundActor);
		}

		for (UObject* ObjectToInspect : ObjectsToInspect)
		{
			const FName OutputAttributeName = Settings->OutputAttributeName.GetName();

			PCGPropertyHelpers::FExtractorParameters Parameters(ObjectToInspect, ObjectToInspect->GetClass(), Settings->PropertyName.ToString(), OutputAttributeName, /*bPropertyNeedsToBeVisible=*/true);
			Parameters.StructExtractorBehavior = Settings->StructExtractorBehavior;
			Parameters.ContainerExtractorBehavior = Settings->ContainerExtractorBehavior;
			Parameters.ObjectExtractorBehavior = Settings->ObjectExtractorBehavior;
			Parameters.bStrictSanitizeOutputAttributeNames = Settings->bSanitizeOutputAttributeName;

			// Don't care for object traversed in non-editor build, since it is only useful for tracking.
			TSet<FSoftObjectPath>* ObjectTraversedPtr = nullptr;
#if WITH_EDITOR
			TSet<FSoftObjectPath> ObjectTraversed;
			ObjectTraversedPtr = &ObjectTraversed;
#endif // WITH_EDITOR

			if (UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Parameters, Context, ObjectTraversedPtr))
			{
				TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
				FPCGTaggedData& Output = Outputs.Emplace_GetRef();
				Output.Data = ParamData;
			
				for (FName ActorTag : FoundActor->Tags)
				{
					Output.Tags.Add(ActorTag.ToString());
				}

				if (Settings->bOutputActorReference)
				{
					check(ParamData->MutableMetadata());
					ParamData->MutableMetadata()->FindOrCreateAttribute(PCGPointDataConstants::ActorReferenceAttribute, FSoftObjectPath(FoundActor), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true);
				}
				
				if (Settings->bSelectComponent && Settings->bOutputComponentReference)
				{
					check(ParamData->MutableMetadata());
					ParamData->MutableMetadata()->FindOrCreateAttribute(TEXT("ComponentReference"), FSoftObjectPath(ObjectToInspect), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true);
				}
			}
			else
			{
				if(Parameters.PropertySelectors.IsEmpty() || Parameters.PropertySelectors[0].GetName() == NAME_None)
				{
					PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("FailedToExtractActor", "Fail to extract actor {0}."), FText::FromString(FoundActor->GetName())));
				}
				else
				{
					PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("FailedToExtract", "Fail to extract the property '{0}' on actor {1}."), Parameters.PropertySelectors[0].GetDisplayText(), FText::FromString(FoundActor->GetName())));
				}
			}

			// Register dynamic tracking
#if WITH_EDITOR
			if (!ObjectTraversed.IsEmpty())
			{
				FPCGDynamicTrackingHelper DynamicTracking;
				DynamicTracking.EnableAndInitialize(Context, ObjectTraversed.Num());
				for (const FSoftObjectPath& Path : ObjectTraversed)
				{
					DynamicTracking.AddToTracking(FPCGSelectionKey::CreateFromPath(Path), /*bCulled=*/Settings->bTrackActorsOnlyWithinBounds);
				}

				DynamicTracking.Finalize(Context);
			}
#endif // WITH_EDITOR
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
