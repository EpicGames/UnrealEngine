// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetBounds.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGSpatialData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetBounds)

#define LOCTEXT_NAMESPACE "PCGGetBoundsElement"

#if WITH_EDITOR
FText UPCGGetBoundsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Bounds");
}

FText UPCGGetBoundsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Computes the bounds of the inputs as attributes.");
}
#endif

TArray<FPCGPinProperties> UPCGGetBoundsSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spatial);
	InputPinProperty.SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGetBoundsSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);

	return PinProperties;
}

FPCGElementPtr UPCGGetBoundsSettings::CreateElement() const
{
	return MakeShared<FPCGGetBoundsElement>();
}

bool FPCGGetBoundsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetBoundsElement::Execute);

	const TArray<FPCGTaggedData>& Inputs = Context->InputData.TaggedData;
	bool bEmittedUnboundedDataWarning = false;

	for (int32 i = 0; i < Inputs.Num(); ++i)
	{
		const FPCGTaggedData& Input = Inputs[i];
		
		const UPCGSpatialData* InputSpatialData = Cast<UPCGSpatialData>(Input.Data);
		if (!InputSpatialData)
		{
			continue;
		}

		if (!InputSpatialData->IsBounded())
		{
			if (!bEmittedUnboundedDataWarning)
			{
				bEmittedUnboundedDataWarning = true;
				PCGE_LOG(Warning, GraphAndLog, LOCTEXT("UnboundedInput", "Skipped unbounded spatial data."));
			}

			continue;
		}

		const FBox InputBounds = InputSpatialData->GetBounds();
		if (!InputBounds.IsValid)
		{
			PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidBoundsOnInput", "Skipped data {0}, it has invalid bounds."), i));
			continue;
		}

		UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		check(ParamData && ParamData->Metadata);
		
		ParamData->Metadata->CreateVectorAttribute("BoundsMin", InputBounds.Min, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		ParamData->Metadata->CreateVectorAttribute("BoundsMax", InputBounds.Max, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		ParamData->Metadata->AddEntry();

		FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef(Input);
		Output.Data = ParamData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
