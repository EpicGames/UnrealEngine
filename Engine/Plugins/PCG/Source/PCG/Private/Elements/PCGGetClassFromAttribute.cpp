// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetClassFromAttribute.h"

#include "PCGCommon.h"
#include "PCGContext.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Utils/PCGLogErrors.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetClassFromAttribute)

#define LOCTEXT_NAMESPACE "PCGGetClassFromAttributeElement"

TArray<FPCGPinProperties> UPCGGetClassFromAttributeSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any).SetRequiredPin();
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGetClassFromAttributeSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
	return PinProperties;
}

FPCGElementPtr UPCGGetClassFromAttributeSettings::CreateElement() const
{
	return MakeShared<FPCGGetClassFromAttributeElement>();
}

UPCGGetClassFromAttributeSettings::UPCGGetClassFromAttributeSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	ClassPathOutputTarget.SetAttributeName("ClassPath");
}

#if WITH_EDITOR
FText UPCGGetClassFromAttributeSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Class From Attribute");
}

FText UPCGGetClassFromAttributeSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Reads a Soft Object Path, Soft Class Path, or String attribute on any PCG data entry (including data-level domain attributes), resolves the asset class without loading it, and writes the short class name as a string to the output attribute.");
}

FString UPCGGetClassFromAttributeSettings::GetAdditionalTitleInformation() const
{
	return FString::Printf(TEXT("%s -> %s"), *InputSource.GetDisplayText().ToString(), *OutputTarget.GetDisplayText().ToString());
}
#endif // WITH_EDITOR

bool FPCGGetClassFromAttributeElement::ExecuteInternal(FPCGContext* Context) const
{
	check(Context);
	const UPCGGetClassFromAttributeSettings* Settings = Context->GetInputSettings<UPCGGetClassFromAttributeSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

#if WITH_EDITOR
	const FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry"));
#endif // WITH_EDITOR

	for (int32 i = 0; i < Inputs.Num(); ++i)
	{
		const FPCGTaggedData& Input = Inputs[i];

		if (!ensure(Input.Data) || !Input.Data->ConstMetadata())
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InvalidData", "Input {0} has no metadata and cannot be processed. Skipped."), FText::AsNumber(i)));
			continue;
		}

		FPCGTaggedData Output = Input;

		// Fix @Last on the input selector (only needs Input.Data)
		const FPCGAttributePropertyInputSelector FixedInput = Settings->InputSource.CopyAndFixLast(Input.Data);

		TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, FixedInput);
		TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, FixedInput);

		if (!InputAccessor || !InputKeys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(FixedInput, Context);
			continue;
		}

		UPCGData* OutputData = Input.Data->DuplicateData(Context);
		check(OutputData);

		// Fix @Source on the output selector against the duplicated data
		const FPCGAttributePropertyOutputSelector FixedOutput = Settings->OutputTarget.CopyAndFixSource(&FixedInput, OutputData);

		// Create or find an FString attribute in the domain specified by the output selector
		FPCGMetadataDomain* OutputDomain = OutputData->MutableMetadata()->GetMetadataDomainFromSelector(FixedOutput);
		if (!OutputDomain)
		{
			PCGLog::Metadata::LogInvalidMetadataDomain(FixedOutput, Context);
			continue;
		}

		OutputDomain->FindOrCreateAttribute<FString>(FixedOutput.GetName(), FString(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);

		// Write values via accessor
		TUniquePtr<IPCGAttributeAccessor> OutputAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputData, FixedOutput);
		TUniquePtr<IPCGAttributeAccessorKeys> OutputKeys = PCGAttributeAccessorHelpers::CreateKeys(OutputData, FixedOutput);

		if (!OutputAccessor || !OutputKeys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(FixedOutput, Context);
			continue;
		}
		
		// Optional accessor for class path
		TUniquePtr<IPCGAttributeAccessor> OutputPathAccessor;
		TUniquePtr<IPCGAttributeAccessorKeys> OutputPathKeys;
		
		if (Settings->bAlsoOutputClassPath)
		{
			const FPCGAttributePropertyOutputSelector FixedClassPathOutput = Settings->ClassPathOutputTarget.CopyAndFixSource(&FixedInput, OutputData);
			
			OutputDomain->FindOrCreateAttribute<FSoftClassPath>(FixedClassPathOutput.GetName(), FSoftClassPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
			
			OutputPathAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutputData, FixedClassPathOutput);
			OutputPathKeys = PCGAttributeAccessorHelpers::CreateKeys(OutputData, FixedClassPathOutput);
			
			if (!OutputPathAccessor || !OutputPathKeys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(FixedClassPathOutput, Context);
				continue;
			}
		}

		// When the attribute stores FSoftClassPath, the resolved object IS the class itself.
		// Calling GetClass()->GetName() on a UClass returns "Class" (the meta-class name), so we detect this case upfront and use GetName() directly instead.
		// For FSoftObjectPath and FString the resolved object is an asset instance and GetClass()->GetName() correctly returns its type (e.g. "StaticMesh").

		// Read all paths (FSoftObjectPath exact match; FSoftClassPath and FString via constructible cast)
		const int32 NumEntries = InputKeys->GetNum();
		TArray<FSoftObjectPath> SoftPaths;
		SoftPaths.SetNum(NumEntries);
		if (!InputAccessor->GetRange<FSoftObjectPath>(SoftPaths, 0, *InputKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
		{
			PCGLog::Metadata::LogFailToGetAttributeError<FSoftObjectPath>(FixedInput, InputAccessor.Get(), Context);
			continue;
		}

		// Resolve class name for each entry (no load)
		TArray<FString> ClassNames;
		ClassNames.Reserve(NumEntries);

		TArray<FSoftClassPath> ClassPaths;
		ClassPaths.Reserve(Settings->bAlsoOutputClassPath ? NumEntries : 0);

		for (const FSoftObjectPath& Path : SoftPaths)
		{
			FSoftClassPath ClassPath{};
			FString ClassName;

			if (!Path.IsNull())
			{
				// Resolve already-loaded object — no asset load triggered
				if (const UObject* Object = Path.ResolveObject())
				{
					const UClass* Class = Object->IsA<UClass>() ? CastChecked<const UClass>(Object) : Object->GetClass();
					check(Class);
					ClassName = Class->GetName();
					ClassPath = FSoftClassPath(Class);
				}
#if WITH_EDITOR
				// Object not in memory — fall back to the Asset Registry (editor only).
				else
				{
					if (AssetRegistryModule)
					{
						const FAssetData AssetData = AssetRegistryModule->Get().GetAssetByObjectPath(Path);
						if (AssetData.IsValid())
						{
							ClassName = AssetData.AssetClassPath.GetAssetName().ToString();
							ClassPath = FSoftClassPath(AssetData.AssetClassPath.ToString());
						}
					}
				}
#endif // WITH_EDITOR
			}

			if (ClassName.IsEmpty() && !Settings->bSilenceClassNotFoundWarnings)
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("ClassNotFound", "Could not resolve a class name for path '{0}'. Output will be an empty string."), FText::FromString(Path.ToString())));
			}

			ClassNames.Add(MoveTemp(ClassName));
			if (Settings->bAlsoOutputClassPath)
			{
				ClassPaths.Add(MoveTemp(ClassPath));
			}
		}

		OutputAccessor->SetRange<FString>(ClassNames, 0, *OutputKeys);
		if (Settings->bAlsoOutputClassPath)
		{
			check(OutputPathAccessor && OutputPathKeys);
			OutputPathAccessor->SetRange<FSoftClassPath>(ClassPaths, 0, *OutputPathKeys);
		}

		Context->OutputData.TaggedData.Add_GetRef(Output).Data = OutputData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
