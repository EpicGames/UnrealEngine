// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetConsoleVariable.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetConsoleVariable)

#define LOCTEXT_NAMESPACE "PCGGetConsoleVariableElement"

TArray<FPCGPinProperties> UPCGGetConsoleVariableSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

#if WITH_EDITOR
FText UPCGGetConsoleVariableSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Reads the given console variable and writes the value to an attribute set.\n"
		"Note: Setting the console variable will not trigger a regeneration.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGGetConsoleVariableSettings::CreateElement() const
{
	return MakeShared<FPCGGetConsoleVariableElement>();
}

bool FPCGGetConsoleVariableElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetConsoleVariableElement::ExecuteInternal);

	check(Context);

	const UPCGGetConsoleVariableSettings* Settings = Context->GetInputSettings<UPCGGetConsoleVariableSettings>();
	check(Settings);

	IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*Settings->ConsoleVariableName.ToString());

	if (!ConsoleVariable)
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("FailedToFindCVar", "Failed to find console variable '{0}'."), FText::FromName(Settings->ConsoleVariableName)));
		return true;
	}

	UPCGParamData* OutParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	UPCGMetadata* Metadata = OutParamData->Metadata;
	check(Metadata);

	const PCGMetadataEntryKey EntryKey = OutParamData->Metadata->AddEntry();

	auto CreateAttribute = [Metadata, AttributeName = Settings->OutputAttributeName, EntryKey]<typename T>(const T& Value) -> bool
	{
		if (FPCGMetadataAttribute<T>* Attribute = Metadata->CreateAttribute<T>(AttributeName, Value, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false))
		{
			Attribute->SetValue(EntryKey, Value);
			return true;
		}
		else
		{
			return false;
		}
	};
	
	bool bCreatedAttribute = false;

	if (ConsoleVariable->IsVariableBool())
	{
		const bool Value = ConsoleVariable->GetBool();
		bCreatedAttribute |= CreateAttribute(Value);
	}
	else if (ConsoleVariable->IsVariableInt())
	{
		const int Value = ConsoleVariable->GetInt();
		bCreatedAttribute |= CreateAttribute(Value);
	}
	else if (ConsoleVariable->IsVariableFloat())
	{
		const float Value = ConsoleVariable->GetFloat();
		bCreatedAttribute |= CreateAttribute(Value);
	}
	else if (ConsoleVariable->IsVariableString())
	{
		const FString Value = ConsoleVariable->GetString();
		bCreatedAttribute |= CreateAttribute(Value);
	}
	else
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("UnsupportedCVarType", "Console variable '{0}' is not a supported type."), FText::FromName(Settings->ConsoleVariableName)));
		return true;
	}

	if (!bCreatedAttribute)
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("AttributeCreationFailed", "Failed to create attribute {0}."), FText::FromName(Settings->OutputAttributeName)));
		return true;
	}

	FPCGTaggedData& OutData = Context->OutputData.TaggedData.Emplace_GetRef();
	OutData.Data = OutParamData;

	return true;
}

#undef LOCTEXT_NAMESPACE