// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetDataInfo.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Helpers/PCGTagHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetDataInfo)

TArray<FPCGPinProperties> UPCGGetTagsSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FPCGElementPtr UPCGGetTagsSettings::CreateElement() const
{
	return MakeShared<FPCGGetTagsElement>();
}

bool FPCGGetTagsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGGetTagsSettings::Execute);
	check(Context);

	const UPCGGetTagsSettings* Settings = Context->GetInputSettings<UPCGGetTagsSettings>();

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		if (!Input.Data)
		{
			continue;
		}

		UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		check(ParamData && ParamData->Metadata);

		FPCGTaggedData& Output = Outputs.Emplace_GetRef(Input);
		Output.Data = ParamData;

		FPCGMetadataAttribute<FString>* Attribute = ParamData->Metadata->FindOrCreateAttribute<FString>(FName("Attribute"), FString(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true);
		FPCGMetadataAttribute<FString>* Values = nullptr;
		FPCGMetadataAttribute<FString>* Tags = nullptr;

		if (Settings->bExtractTagValues)
		{
			Values = ParamData->Metadata->FindOrCreateAttribute<FString>(FName("Values"), FString(), false, false, true);
		}

		if (Settings->bExtractFullTagStrings)
		{
			Tags = ParamData->Metadata->FindOrCreateAttribute<FString>(FName("Tags"), FString(), false, false, true);
		}

		for (const FString& Tag : Input.Tags)
		{
			PCG::Private::FParseTagResult TagData = PCG::Private::ParseTag(Tag);
			if (TagData.IsValid())
			{
				PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();
				Attribute->SetValue(EntryKey, TagData.GetOriginalAttribute());

				if (Values)
				{
					if (TagData.HasValue())
					{
						Values->SetValue(EntryKey, TagData.Value.GetValue());
					}
					else // Assume this was a boolean attribute then
					{
						Values->SetValue(EntryKey, FString("true"));
					}
				}

				if (Tags)
				{
					Tags->SetValue(EntryKey, Tag);
				}
			}
		}
	}

	return true;
}

TArray<FPCGPinProperties> UPCGGetAttributesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FPCGElementPtr UPCGGetAttributesSettings::CreateElement() const
{
	return MakeShared<FPCGGetAttributesElement>();
}

bool FPCGGetAttributesElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGGetTagsSettings::Execute);
	check(Context);

	const UPCGGetAttributesSettings* Settings = Context->GetInputSettings<UPCGGetAttributesSettings>();

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		if (!Input.Data)
		{
			continue;
		}

		const UPCGMetadata* InputMetadata = Input.Data->ConstMetadata();

		if (!InputMetadata)
		{
			continue;
		}

		UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		check(ParamData && ParamData->Metadata);

		FPCGTaggedData& Output = Outputs.Emplace_GetRef(Input);
		Output.Data = ParamData;

		TArray<FName> AttributeNames;
		TArray<EPCGMetadataTypes> AttributeTypes;
		InputMetadata->GetAttributes(AttributeNames, AttributeTypes);
		check(AttributeNames.Num() == AttributeTypes.Num());

		const UEnum* TypesEnum = StaticEnum<EPCGMetadataTypes>();

		FPCGMetadataAttribute<FName>* Attribute = ParamData->Metadata->FindOrCreateAttribute<FName>(FName("Attribute"), NAME_None, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true);
		FPCGMetadataAttribute<FName>* Types = nullptr;
		if (TypesEnum && Settings->bGetType)
		{
			Types = ParamData->Metadata->FindOrCreateAttribute<FName>(FName("Type"), NAME_None, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true);
		}

		FPCGMetadataAttribute<FString>* DefaultValues = nullptr;
		if (Settings->bGetDefaultValue)
		{
			DefaultValues = ParamData->Metadata->FindOrCreateAttribute<FString>(FName("DefaultValue"), FString(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true);
		}

		for (int AttributeIndex = 0; AttributeIndex < AttributeNames.Num(); ++AttributeIndex)
		{
			FName AttributeName = AttributeNames[AttributeIndex];
			EPCGMetadataTypes AttributeType = AttributeTypes[AttributeIndex];

			PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();
			Attribute->SetValue(EntryKey, AttributeName);

			if (Types && TypesEnum)
			{
				Types->SetValue(EntryKey, TypesEnum->GetNameByValue(static_cast<__underlying_type(EPCGMetadataTypes)>(AttributeType)));
			}

			if (DefaultValues)
			{
				TUniquePtr<const IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputMetadata->GetConstAttribute(AttributeName), InputMetadata, /*bQuiet=*/true);
				FPCGAttributeAccessorKeysEntries AttributeKeys(PCGInvalidEntryKey);
				FString DefaultValueString;

				if (AttributeAccessor && AttributeAccessor->Get<FString>(DefaultValueString, 0, AttributeKeys, EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible))
				{
					DefaultValues->SetValue(EntryKey, DefaultValueString);
				}
			}
		}
	}

	return true;
}