// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetExecutionContext.h"

#include "PCGParamData.h"
#include "PCGPin.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadata.h"

#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetExecutionContext)

#define LOCTEXT_NAMESPACE "PCGGetExecutionContextElement"

namespace PCGGetExecutionContextConstants
{
	static const FName AttributeName = TEXT("Info");
}

#if WITH_EDITOR
FText UPCGGetExecutionContextSettings::GetNodeTooltipText() const
{
	return LOCTEXT("GetExecutionContextTooltip", "Returns some context-specific common information.");
}

EPCGChangeType UPCGGetExecutionContextSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;
}
#endif

TArray<FPCGPinProperties> UPCGGetExecutionContextSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FString UPCGGetExecutionContextSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGGetExecutionContextMode>())
	{
		return FText::Format(LOCTEXT("AdditionalTitle", "Get {0}"), EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Mode))).ToString();
	}
	else
	{
		return FString();
	}
}

FPCGElementPtr UPCGGetExecutionContextSettings::CreateElement() const
{
	return MakeShared<FPCGGetExecutionContextElement>();
}

bool FPCGGetExecutionContextElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetExecutionContextElement::Execute);

	check(Context);

	const UPCGGetExecutionContextSettings* Settings = Context->GetInputSettings<UPCGGetExecutionContextSettings>();
	check(Settings);

	const EPCGGetExecutionContextMode Mode = Settings->Mode;

	UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	check(ParamData && ParamData->Metadata);

	FPCGTaggedData& OutputData = Context->OutputData.TaggedData.Emplace_GetRef();
	OutputData.Data = ParamData;

	const IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();

	auto IsBoolOutput = [](EPCGGetExecutionContextMode Mode)
	{
		return Mode != EPCGGetExecutionContextMode::TrackingPriority;
	};

	if (IsBoolOutput(Mode))
	{
		bool Value = false;
		const UWorld* SupportingWorld = ExecutionSource ? ExecutionSource->GetExecutionState().GetWorld() : nullptr;

		if (Mode == EPCGGetExecutionContextMode::IsEditor || Mode == EPCGGetExecutionContextMode::IsRuntime)
		{
			const bool bContextIsRuntimeOrOnRuntimeGenerationMode = (SupportingWorld && SupportingWorld->IsGameWorld()) || PCGHelpers::IsRuntimeGeneration(ExecutionSource);
			Value = (bContextIsRuntimeOrOnRuntimeGenerationMode == (Mode == EPCGGetExecutionContextMode::IsRuntime));
		}
		else if (Mode == EPCGGetExecutionContextMode::IsOriginal || Mode == EPCGGetExecutionContextMode::IsLocal)
		{
			Value = ExecutionSource && (ExecutionSource->GetExecutionState().IsLocalSource() == (Mode == EPCGGetExecutionContextMode::IsLocal));
		}
		else if (Mode == EPCGGetExecutionContextMode::IsPartitioned)
		{
			Value = ExecutionSource && ExecutionSource->GetExecutionState().IsPartitioned();
		}
		else if (Mode == EPCGGetExecutionContextMode::IsRuntimeGeneration)
		{
			Value = PCGHelpers::IsRuntimeGeneration(ExecutionSource);
		}
		else if (Mode == EPCGGetExecutionContextMode::IsDedicatedServer)
		{
			Value = PCGHelpers::IsDedicatedServer(SupportingWorld);
		}
		else if (Mode == EPCGGetExecutionContextMode::IsListenServer)
		{
			Value = PCGHelpers::IsListenServer(SupportingWorld);
		}
		else if (Mode == EPCGGetExecutionContextMode::HasAuthority)
		{
			Value = ExecutionSource && ExecutionSource->GetExecutionState().HasAuthority();
		}
		else if (Mode == EPCGGetExecutionContextMode::IsBuilder)
		{
#if WITH_EDITOR
			Value = PCGHelpers::IsInWorldPartitionBuilderCommandletContext();
#endif
		}
		else if (Mode == EPCGGetExecutionContextMode::IsGameWorld)
		{
			Value = SupportingWorld && SupportingWorld->IsGameWorld();
		}

		FPCGMetadataAttribute<bool>* Attribute = ParamData->Metadata->CreateAttribute<bool>(PCGGetExecutionContextConstants::AttributeName, Value, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	}
	else
	{
		ensure(Mode == EPCGGetExecutionContextMode::TrackingPriority);
		double TrackingPriority = 0.0;
#if WITH_EDITOR
		TrackingPriority = ExecutionSource ? ExecutionSource->GetExecutionState().GetDynamicTrackingPriority() : TrackingPriority;
#endif
		FPCGMetadataAttribute<double>* Attribute = ParamData->Metadata->CreateAttribute<double>(PCGGetExecutionContextConstants::AttributeName, TrackingPriority, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	}

	ParamData->Metadata->AddEntry();

	return true;
}

#undef LOCTEXT_NAMESPACE