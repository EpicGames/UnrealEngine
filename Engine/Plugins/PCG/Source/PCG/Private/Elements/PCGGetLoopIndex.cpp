// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetLoopIndex.h"

#include "PCGParamData.h"
#include "PCGPin.h"
#include "Graph/PCGStackContext.h"
#include "Metadata/PCGMetadata.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetLoopIndex)

#define LOCTEXT_NAMESPACE "PCGGetLoopIndexElement"

namespace PCGGetLoopIndexConstants
{
	static const FName LoopIndexAttributeName = TEXT("LoopIndex");
}

#if WITH_EDITOR
FText UPCGGetLoopIndexSettings::GetNodeTooltipText() const
{
	return LOCTEXT("GetLoopIndexTooltip", "Returns the index of the loop this subgraph is executing, if any.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGetLoopIndexSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FPCGElementPtr UPCGGetLoopIndexSettings::CreateElement() const
{
	return MakeShared<FPCGGetLoopIndexElement>();
}

bool FPCGGetLoopIndexElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetLoopIndexElement::Execute);

	check(Context);

	const UPCGGetLoopIndexSettings* Settings = Context->GetInputSettings<UPCGGetLoopIndexSettings>();
	check(Settings);

	const FPCGStack* Stack = Context->GetStack();
	if (!ensure(Stack))
	{
		PCGE_LOG(Error, LogOnly, LOCTEXT("ContextHasNoExecutionStack", "The execution context is malformed and has no call stack."));
		return true;
	}

	int LoopIndex = INDEX_NONE;

	const TArray<FPCGStackFrame>& StackFrames = Stack->GetStackFrames();
	// Last element in the stack should be the current subgraph; Second-to-last will be the loop index, if any.
	// WARNING: this is sensitive to stack frames changing.
	// This will get the loop index only if the current subgraph is executed as a loop, this will NOT find the "nearest" loop index.
	if (StackFrames.Num() >= 2)
	{
		LoopIndex = StackFrames.Last(1).LoopIndex;
	}

	if (LoopIndex == INDEX_NONE)
	{
		if (Settings->bWarnIfCalledOutsideOfLoop)
		{
			PCGE_LOG(Warning, GraphAndLog, LOCTEXT("NotRunningALoop", "GetLoopIndex node executed outside of a loop. If this is okay, consider turning off this warning."));
		}
		
		return true;
	}

	UPCGParamData* LoopIndexParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	check(LoopIndexParamData && LoopIndexParamData->Metadata);

	FPCGTaggedData& OutputData = Context->OutputData.TaggedData.Emplace_GetRef();
	OutputData.Data = LoopIndexParamData;

	FPCGMetadataAttribute<int>* LoopIndexAttribute = LoopIndexParamData->Metadata->CreateAttribute<int>(PCGGetLoopIndexConstants::LoopIndexAttributeName, LoopIndex, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	check(LoopIndexAttribute);

	LoopIndexParamData->Metadata->AddEntry();

	return true;
}

#undef LOCTEXT_NAMESPACE