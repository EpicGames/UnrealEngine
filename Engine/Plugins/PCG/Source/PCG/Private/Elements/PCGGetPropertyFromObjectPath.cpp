// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetPropertyFromObjectPath.h"

#include "PCGComponent.h"
#include "PCGParamData.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGPropertyHelpers.h"

#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetPropertyFromObjectPath)

#define LOCTEXT_NAMESPACE "PCGGetPropertyFromObjectPathElement"

UPCGGetPropertyFromObjectPathSettings::UPCGGetPropertyFromObjectPathSettings()
{
	// Default was None, but for new objects it will be @Source
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		OutputAttributeName.SetAttributeName(PCGMetadataAttributeConstants::SourceAttributeName);
	}
	else
	{
		OutputAttributeName.SetAttributeName(NAME_None);
	}
}

#if WITH_EDITOR
void UPCGGetPropertyFromObjectPathSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	// If input pin is connected, tracking is dynamic.
	const UPCGNode* Node = Cast<const UPCGNode>(GetOuter());
	if (Node && Node->IsInputPinConnected(PCGPinConstants::DefaultInputLabel))
	{
		return;
	}

	for (const FSoftObjectPath& ObjectPath : ObjectPathsToExtract)
	{
		if (ObjectPath.IsNull())
		{
			continue;
		}

		FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(ObjectPath);

		OutKeysToSettings.FindOrAdd(Key).Emplace(this, /*bCulling=*/false);
	}
}

FName UPCGGetPropertyFromObjectPathSettings::GetDefaultNodeName() const
{
	return FName(TEXT("GetPropertyFromObjectPath"));
}

FText UPCGGetPropertyFromObjectPathSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Property From Object Path");
}

bool UPCGGetPropertyFromObjectPathSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	return !InPin || (InPin->Properties.Label != PCGPinConstants::DefaultInputLabel) || InPin->IsConnected();
}

bool UPCGGetPropertyFromObjectPathSettings::CanEditChange(const FProperty* InProperty) const
{
	if (!InProperty || !Super::CanEditChange(InProperty))
	{
		return false;
	}

	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	const bool InPinIsConnected = Node ? Node->IsInputPinConnected(PCGPinConstants::DefaultInputLabel) : false;

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGGetPropertyFromObjectPathSettings, InputSource))
	{
		return InPinIsConnected;
	}
	else if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGGetPropertyFromObjectPathSettings, ObjectPathsToExtract))
	{
		return !InPinIsConnected;
	}

	return true;
}
#endif // WITH_EDITOR

void UPCGGetPropertyFromObjectPathSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::OptionSanitizeOutputAttributeNamesPCG)
	{
		// For all previous nodes, we'll force this option to false. for retro-compatibility
		bSanitizeOutputAttributeName = false;
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bForceObjectAndStructExtraction)
	{
		bForceObjectAndStructExtraction = false;
		StructExtractorBehavior = EPCGStructExtractorBehavior::Extract;
		ObjectExtractorBehavior = EPCGObjectExtractorBehavior::Extract;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

FString UPCGGetPropertyFromObjectPathSettings::GetAdditionalTitleInformation() const
{
	FString Path;

	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	const bool bInPinIsConnected = Node ? Node->IsInputPinConnected(PCGPinConstants::DefaultInputLabel) : false;

	// If the input pin is connected, don't display anything from the path.
	if (!bInPinIsConnected)
	{
		if (ObjectPathsToExtract.IsEmpty())
		{
			Path = LOCTEXT("MissingPath", "Missing Path").ToString();
		}
		else if (ObjectPathsToExtract.Num() == 1)
		{
			Path = ObjectPathsToExtract[0].ToString();
			Path = !Path.IsEmpty() ? Path : LOCTEXT("EmptyString", "None").ToString();
		}
		else
		{
			Path = LOCTEXT("MultiplePaths", "Multiple Paths").ToString();
		}

		Path += TEXT(", ");
	}

	return FString::Printf(TEXT("%s%s"), *Path, *PropertyName.ToString());
}

FPCGElementPtr UPCGGetPropertyFromObjectPathSettings::CreateElement() const
{
	return MakeShared<FPCGGetPropertyFromObjectPathElement>();
}

TArray<FPCGPinProperties> UPCGGetPropertyFromObjectPathSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Param);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGetPropertyFromObjectPathSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

bool FPCGGetPropertyFromObjectPathElement::PrepareDataInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetPropertyFromObjectPathElement::PrepareData);

	check(Context);

	const UPCGGetPropertyFromObjectPathSettings* Settings = Context->GetInputSettings<UPCGGetPropertyFromObjectPathSettings>();
	check(Settings);

	FPCGLoadObjectsFromPathContext* ThisContext = static_cast<FPCGLoadObjectsFromPathContext*>(Context);
	return ThisContext->InitializeAndRequestLoad(PCGPinConstants::DefaultInputLabel,
		Settings->InputSource,
		Settings->ObjectPathsToExtract,
		Settings->bPersistAllData,
		Settings->bSilenceErrorOnEmptyObjectPath,
		Settings->bSynchronousLoad);
}

bool FPCGGetPropertyFromObjectPathElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetPropertyFromObjectPathElement::Execute);

	check(Context);
	FPCGLoadObjectsFromPathContext* ThisContext = static_cast<FPCGLoadObjectsFromPathContext*>(Context);

	const UPCGGetPropertyFromObjectPathSettings* Settings = Context->GetInputSettings<UPCGGetPropertyFromObjectPathSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	// For any "invalid" entry that needs an empty param, just allocate one and use this one for all the invalids.
	const UPCGParamData* EmptyParam = nullptr;

	auto AddToOutput = [Context, &Inputs](const UPCGData* Data, int32 Index)
	{
		TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
		FPCGTaggedData& Output = Outputs.Emplace_GetRef();
		Output.Data = Data;
		Output.Pin = PCGPinConstants::DefaultOutputLabel;
		if (Index >= 0)
		{
			Output.Tags = Inputs[Index].Tags;
		}
	};

#if WITH_EDITOR
	FPCGDynamicTrackingHelper DynamicTrackingHelper;
	UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get());
	const bool IsDynamicallyTracking = (SourceComponent && Context->Node) ? Context->Node->IsInputPinConnected(PCGPinConstants::DefaultInputLabel) : false;
	if (IsDynamicallyTracking)
	{
		DynamicTrackingHelper.EnableAndInitialize(ThisContext, ThisContext->PathsToObjectsAndDataIndex.Num());
	}
#endif // WITH_EDITOR

	for (const TTuple<FSoftObjectPath, int32, int32>& SoftPathAndIndex : ThisContext->PathsToObjectsAndDataIndex)
	{
		const FSoftObjectPath& SoftPath = SoftPathAndIndex.Get<0>();
		const int32 Index = SoftPathAndIndex.Get<1>();

		if (SoftPath.IsNull() && Settings->bPersistAllData)
		{
			if (!Settings->bSilenceErrorOnEmptyObjectPath)
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("EmptyData", "Empty data on index {0}"), FText::AsNumber(Index)));
			}

			if (!EmptyParam)
			{
				EmptyParam = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
			}

			AddToOutput(EmptyParam, Index);
			continue;
		}

		const UObject* Object = SoftPath.ResolveObject();
		if (!Object)
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("FailedToLoad", "Failed to load object {0}"), FText::FromString(SoftPath.ToString())));
			continue;
		}

		const FName OutputAttributeName = Settings->OutputAttributeName.GetName();

		PCGPropertyHelpers::FExtractorParameters Parameters(Object, Object->GetClass(), Settings->PropertyName.ToString(), OutputAttributeName, /*bPropertyNeedsToBeVisible=*/true);
		Parameters.StructExtractorBehavior = Settings->StructExtractorBehavior;
		Parameters.ObjectExtractorBehavior = Settings->ObjectExtractorBehavior;
		Parameters.ContainerExtractorBehavior = Settings->ContainerExtractorBehavior;
		
		Parameters.bStrictSanitizeOutputAttributeNames = Settings->bSanitizeOutputAttributeName;

		if (UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Parameters, Context))
		{
			AddToOutput(ParamData, Index);
#if WITH_EDITOR
			DynamicTrackingHelper.AddToTracking(FPCGSelectionKey::CreateFromPath(SoftPath), /*bIsCulled=*/ false);
#endif // WITH_EDITOR
		}
		else
		{
			if(Parameters.PropertySelectors.IsEmpty() || Parameters.PropertySelectors[0].GetName() == NAME_None)
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("FailedToExtractObject", "Fail to extract object {0}."), FText::FromString(Object->GetName())));
			}
			else
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("FailedToExtract", "Fail to extract the property '{0}' on object {1}."), Parameters.PropertySelectors[0].GetDisplayText(), FText::FromString(Object->GetName())));
			}
		}
	}

#if WITH_EDITOR
	DynamicTrackingHelper.Finalize(ThisContext);
#endif // WITH_EDITOR

	return true;
}

#undef LOCTEXT_NAMESPACE
