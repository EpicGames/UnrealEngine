// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetResourcePath.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGResourceData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetResourcePath)

TArray<FPCGPinProperties> UPCGGetResourcePath::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Resource);
	InputPinProperty.SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGetResourcePath::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FPCGElementPtr UPCGGetResourcePath::CreateElement() const
{
	return MakeShared<FPCGGetResourcePathElement>();
}

bool FPCGGetResourcePathElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetResourcePathElement::Execute);

	check(Context);

	for (const FPCGTaggedData& Input : Context->InputData.TaggedData)
	{
		const UPCGResourceData* ResourceData = Cast<UPCGResourceData>(Input.Data);

		if (!ResourceData)
		{
			continue;
		}

		UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		check(ParamData);
		check(ParamData->Metadata);
		
		ParamData->Metadata->CreateSoftObjectPathAttribute(TEXT("ResourceReference"), ResourceData->GetResourcePath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
		ParamData->Metadata->AddEntry();

		FPCGTaggedData& Output = Context->OutputData.TaggedData.Add_GetRef(Input);
		Output.Data = ParamData;
	}

	return true;
}
