// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetSplineControlPoints.h"

#include "Data/PCGPolyLineData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGPolygon2DData.h"
#include "Data/PCGBasePointData.h"

#include "PCGContext.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetSplineControlPoints)

#define LOCTEXT_NAMESPACE "PCGGetSplineControlPointsElement"

#if WITH_EDITOR
FName UPCGGetSplineControlPointsSettings::GetDefaultNodeName() const
{
	return FName(TEXT("GetSplineControlPoints"));
}

FText UPCGGetSplineControlPointsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Spline Control Points");
}

FText UPCGGetSplineControlPointsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Extracts the control points from the spline(s) as point data.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGGetSplineControlPointsSettings::CreateElement() const
{
	return MakeShared<FPCGGetSplineControlPointsElement>();
}

TArray<FPCGPinProperties> UPCGGetSplineControlPointsSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::PolyLine).SetRequiredPin();	
	return Properties;
}

TArray<FPCGPinProperties> UPCGGetSplineControlPointsSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);	
	return Properties;
}

bool FPCGGetSplineControlPointsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetSplineControlPointsElement::Execute);

	check(InContext);

	const UPCGGetSplineControlPointsSettings* Settings = InContext->GetInputSettings<UPCGGetSplineControlPointsSettings>();
	check(Settings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		const UPCGPolyLineData* PolylineData = Cast<const UPCGPolyLineData>(Input.Data);

		if (!PolylineData)
		{
			continue;
		}

		const UPCGSplineData* SplineData = Cast<UPCGSplineData>(Input.Data);

		const int32 NumPoints = PolylineData->GetNumVertices();
		const TConstArrayView<PCGMetadataEntryKey> MetadataEntries = PolylineData->GetConstVerticesEntryKeys();

		UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(InContext);	
		FPCGInitializeFromDataParams Params(PolylineData);
		PointData->InitializeFromDataWithParams(Params);
		PointData->SetNumPoints(NumPoints);
		PointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::MetadataEntry);

		PointData->SetDensity(1.0f);
		PointData->SetSteepness(1.0f);

		FPCGMetadataDomain* ElementsMetadataDomain = PointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements);
		check(ElementsMetadataDomain);

		auto CreateAndValidateAttribute = [InContext, ElementsMetadataDomain]<typename T>(const FName AttributeName, T DefaultValue) -> FPCGMetadataAttribute<T>*
		{
			FPCGMetadataAttribute<T>* Attribute = ElementsMetadataDomain->FindOrCreateAttribute<T>(AttributeName, DefaultValue, /*bAllowInterpolation=*/true, /*bOverrideParent=*/true);
			if (!Attribute)
			{
				PCGLog::Metadata::LogFailToCreateAttributeError<T>(AttributeName, InContext);
			}

			return Attribute;
		};

		FPCGMetadataAttribute<FVector>* LeaveTangentAttribute = CreateAndValidateAttribute(Settings->LeaveTangentAttributeName, FVector::ZeroVector);
		FPCGMetadataAttribute<FVector>* ArriveTangentAttribute = CreateAndValidateAttribute(Settings->ArriveTangentAttributeName, FVector::ZeroVector);
		FPCGMetadataAttribute<int32>* InterpTypeAttribute = nullptr;
		
		TArray<FSplinePoint> SplinePoints;
		if (SplineData)
		{
			InterpTypeAttribute = CreateAndValidateAttribute(Settings->InterpTypeAttributeName, static_cast<int32>(ESplinePointType::CurveCustomTangent));
			
			if (InterpTypeAttribute)
			{
				SplinePoints = SplineData->GetSplinePoints();
			}
		}
		else if (const UPCGPolygon2DData* PolygonData = Cast<UPCGPolygon2DData>(Input.Data))
		{
			// All segments in a polygon are linear
			InterpTypeAttribute = CreateAndValidateAttribute(Settings->InterpTypeAttributeName, static_cast<int32>(ESplinePointType::Linear));
		}

		TPCGValueRange<FTransform> PointTransforms = PointData->GetTransformValueRange();
		TPCGValueRange<int> PointSeeds = PointData->GetSeedValueRange();
		TPCGValueRange<PCGMetadataEntryKey> PointMetadataEntries = PointData->GetMetadataEntryValueRange();

		for (int32 i = 0; i < NumPoints; ++i)
		{
			PointTransforms[i] = PolylineData->GetTransformAtDistance(i, 0);
			PointSeeds[i] = PCGHelpers::ComputeSeedFromPosition(PointTransforms[i].GetLocation());
			PointMetadataEntries[i] = MetadataEntries.IsValidIndex(i) ? MetadataEntries[i] : PCGInvalidEntryKey;
			ElementsMetadataDomain->InitializeOnSet(PointMetadataEntries[i]);

			if (LeaveTangentAttribute || ArriveTangentAttribute)
			{
				FVector LeaveTangent, ArriveTangent;
				PolylineData->GetTangentsAtSegmentStart(i, ArriveTangent, LeaveTangent);

				if (LeaveTangentAttribute)
				{
					LeaveTangentAttribute->SetValue(PointMetadataEntries[i], LeaveTangent);
				}

				if (ArriveTangentAttribute)
				{
					ArriveTangentAttribute->SetValue(PointMetadataEntries[i], ArriveTangent);
				}
			}

			if (SplineData && InterpTypeAttribute)
			{
				InterpTypeAttribute->SetValue(PointMetadataEntries[i], SplinePoints[i].Type);
			}
		}

		InContext->OutputData.TaggedData.Emplace_GetRef(Input).Data = PointData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
