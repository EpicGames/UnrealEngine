// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetStaticMeshResourceData.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#if WITH_EDITOR
#include "Helpers/PCGDynamicTrackingHelpers.h"
#endif // WITH_EDITOR

#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetStaticMeshResourceData)

#define LOCTEXT_NAMESPACE "PCGGetStaticMeshResourceDataElement"

namespace PCGGetStaticMeshResourceDataConstants
{
	const FName MeshOverridesPinLabel = TEXT("Meshes");
}

TArray<FPCGPinProperties> UPCGGetStaticMeshResourceDataSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	if (bOverrideFromInput)
	{
		FPCGPinProperties& MeshOverrides = Properties.Emplace_GetRef(PCGGetStaticMeshResourceDataConstants::MeshOverridesPinLabel, EPCGDataType::Param);
		MeshOverrides.SetRequiredPin();
	}

	return Properties;
}

TArray<FPCGPinProperties> UPCGGetStaticMeshResourceDataSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(
		PCGPinConstants::DefaultOutputLabel,
		EPCGDataType::StaticMeshResource,
		/*bInAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true);

	return Properties;
}

FPCGElementPtr UPCGGetStaticMeshResourceDataSettings::CreateElement() const
{
	return MakeShared<FPCGGetStaticMeshResourceDataElement>();
}

#if WITH_EDITOR
void UPCGGetStaticMeshResourceDataSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (bOverrideFromInput)
	{
		return;
	}
	
	for (const TSoftObjectPtr<UStaticMesh>& StaticMesh : StaticMeshes)
	{
		FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(StaticMesh.ToSoftObjectPath());
		OutKeysToSettings.FindOrAdd(MoveTemp(Key)).Emplace(this, /*bCulling=*/false);
	}
}

EPCGChangeType UPCGGetStaticMeshResourceDataSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGetStaticMeshResourceDataSettings, bOverrideFromInput))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR


bool FPCGGetStaticMeshResourceDataElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetStaticMeshResourceDataElement::Execute);
	check(InContext);

	const UPCGGetStaticMeshResourceDataSettings* Settings = InContext->GetInputSettings<UPCGGetStaticMeshResourceDataSettings>();
	check(Settings);

	TArray<TSoftObjectPtr<UStaticMesh>> StaticMeshes;

	if (!Settings->bOverrideFromInput)
	{
		StaticMeshes = Settings->StaticMeshes;
	}
	else
	{
		const TArray<FPCGTaggedData> OverrideTaggedDatas = InContext->InputData.GetInputsByPin(PCGGetStaticMeshResourceDataConstants::MeshOverridesPinLabel);
		TArray<FSoftObjectPath> MeshOverrides;

		for (const FPCGTaggedData& OverrideTaggedData : OverrideTaggedDatas)
		{
			if (const UPCGData* OverrideData = OverrideTaggedData.Data)
			{
				const FPCGAttributePropertyInputSelector MeshSelector = Settings->MeshAttribute.CopyAndFixLast(OverrideData);

				if (PCGAttributeAccessorHelpers::ExtractAllValues(OverrideData, MeshSelector, MeshOverrides, InContext))
				{
					for (const FSoftObjectPath& MeshOverride : MeshOverrides)
					{
						StaticMeshes.Emplace(MeshOverride);
					}
				}
				else
				{
					PCGLog::LogWarningOnGraph(LOCTEXT("FailExtractMeshOverrides", "Failed to extract static mesh overrides."), InContext);
				}
			}
		}
	}

	// We only want to create one resource data per unique mesh, so we'll cache them here while looping over the input.
	TMap<TSoftObjectPtr<UStaticMesh>, UPCGStaticMeshResourceData*> CreatedResourceDatas;

#if WITH_EDITOR
	FPCGDynamicTrackingHelper DynamicTracking;
	if (Settings->bOverrideFromInput)
	{
		DynamicTracking.EnableAndInitialize(InContext, StaticMeshes.Num());
	}
#endif // WITH_EDITOR

	for (const TSoftObjectPtr<UStaticMesh>& StaticMesh : StaticMeshes)
	{
		if (StaticMesh.IsNull())
		{
			continue;
		}

#if WITH_EDITOR
		DynamicTracking.AddToTracking(FPCGSelectionKey::CreateFromPath(StaticMesh.ToSoftObjectPath()), /*bCulled=*/false);
#endif // WITH_EDITOR
		
		UPCGStaticMeshResourceData** StaticMeshResourceDataPtr = CreatedResourceDatas.Find(StaticMesh);
		UPCGStaticMeshResourceData* StaticMeshResourceData = nullptr;

		if (StaticMeshResourceDataPtr)
		{
			StaticMeshResourceData = *StaticMeshResourceDataPtr;
		}
		else
		{
			StaticMeshResourceData = FPCGContext::NewObject_AnyThread<UPCGStaticMeshResourceData>(InContext);
			StaticMeshResourceData->Initialize(StaticMesh);

			CreatedResourceDatas.Add(StaticMesh, StaticMeshResourceData);
		}

		InContext->OutputData.TaggedData.Emplace_GetRef().Data = StaticMeshResourceData;
	}

#if WITH_EDITOR
	DynamicTracking.Finalize(InContext);
#endif // WITH_EDITOR

	return true;
}

#undef LOCTEXT_NAMESPACE
