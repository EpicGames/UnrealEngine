// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetSubgraphDepth.h"

#include "PCGGraph.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Graph/PCGStackContext.h"
#include "Metadata/PCGMetadata.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetSubgraphDepth)

#define LOCTEXT_NAMESPACE "PCGGetSubgraphDepthElement"

namespace PCGGetSubgraphDepthConstants
{
	static const FName DepthAttributeName = TEXT("Depth");
}

#if WITH_EDITOR
FText UPCGGetSubgraphDepthSettings::GetNodeTooltipText() const
{
	return LOCTEXT("GetSubgraphDepthTooltip", "Returns the call depth of this graph.");
}

EPCGChangeType UPCGGetSubgraphDepthSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGetSubgraphDepthSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FString UPCGGetSubgraphDepthSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGSubgraphDepthMode>())
	{
		if (Mode != EPCGSubgraphDepthMode::RecursiveDepth || DistanceRelativeToUpstreamGraph == 0)
		{
			return FText::Format(LOCTEXT("AdditionalTitle", "Get {0}"), EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Mode))).ToString();
		}
		else
		{
			return FText::Format(LOCTEXT("RelativeDistanceTitle", "Get Recursive Depth from {0}-Upstream Graph."), FText::AsNumber(DistanceRelativeToUpstreamGraph)).ToString();
		}
	}
	else
	{
		return FString();
	}
}

FPCGElementPtr UPCGGetSubgraphDepthSettings::CreateElement() const
{
	return MakeShared<FPCGGetSubgraphDepthElement>();
}

bool FPCGGetSubgraphDepthElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetSubgraphDepthElement::Execute);

	check(Context);

	const UPCGGetSubgraphDepthSettings* Settings = Context->GetInputSettings<UPCGGetSubgraphDepthSettings>();
	check(Settings);

	bool bRecursive = Settings->Mode == EPCGSubgraphDepthMode::RecursiveDepth;
	const UPCGGraph* TargetGraph = nullptr;

	const FPCGStack* Stack = Context->GetStack();
	if (!ensure(Stack))
	{
		PCGE_LOG(Error, LogOnly, LOCTEXT("ContextHasNoExecutionStack", "The execution context is malformed and has no call stack."));
		return true;
	}

	const TArray<FPCGStackFrame>& StackFrames = Stack->GetStackFrames();

	// Find target graph first
	int SkipGraphCount = 0;
	int Depth = 0;

	if (StackFrames.Num() > 0)
	{
		FGCScopeGuard Guard;
		for (int FrameIndex = StackFrames.Num() - 1; FrameIndex >= 0; --FrameIndex)
		{
			const FPCGStackFrame& StackFrame = StackFrames[FrameIndex];
			if (const UPCGGraph* Graph = StackFrame.GetObject_NoGuard<UPCGGraph>())
			{
				if (!bRecursive)
				{
					++Depth;
				}
				else
				{
					if (!TargetGraph)
					{
						if (SkipGraphCount == Settings->DistanceRelativeToUpstreamGraph)
						{
							TargetGraph = Graph;
							Depth = 1;
						}
						else
						{
							++SkipGraphCount;
						}
					}
					else if (TargetGraph == Graph)
					{
						++Depth;
					}
				}
			}
		}
	}

	if (Depth == 0 && !Settings->bQuietInvalidDepthQueries)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("InvalidDepthQuery", "Get depth query failed because of the relative distance to the upstream graph."));
	}

	// Finally, since we count everything, including the root and self, we need to remove one.
	Depth = FMath::Max(0, Depth - 1);

	UPCGParamData* DepthParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	check(DepthParamData && DepthParamData->Metadata);

	FPCGTaggedData& OutputData = Context->OutputData.TaggedData.Emplace_GetRef();
	OutputData.Data = DepthParamData;

	FPCGMetadataAttribute<int>* DepthAttribute = DepthParamData->Metadata->CreateAttribute<int>(PCGGetSubgraphDepthConstants::DepthAttributeName, Depth, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	check(DepthAttribute);

	DepthParamData->Metadata->AddEntry();

	return true;
}

#undef LOCTEXT_NAMESPACE