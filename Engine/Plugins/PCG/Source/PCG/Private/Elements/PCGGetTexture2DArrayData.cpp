// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetTexture2DArrayData.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Data/PCGTexture2DArrayData.h"

#if WITH_EDITOR
#include "Helpers/PCGDynamicTrackingHelpers.h"
#endif // WITH_EDITOR

#include "Engine/Texture2DArray.h"
#include "Engine/TextureRenderTarget2DArray.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetTexture2DArrayData)

#define LOCTEXT_NAMESPACE "PCGGetTexture2DArrayDataElement"

#if WITH_EDITOR
void UPCGGetTexture2DArrayDataSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGGetTexture2DArrayDataSettings, Texture)) || Texture.IsNull())
	{
		// Dynamic tracking or null settings
		return;
	}

	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(Texture.ToSoftObjectPath());

	OutKeysToSettings.FindOrAdd(Key).Emplace(this, /*bCulling=*/false);
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGetTexture2DArrayDataSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	FPCGPinProperties& Pin = Properties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeInfoTexture2DArray::AsId());
	Pin.bAllowMultipleData = false;

	return Properties;
}

FPCGElementPtr UPCGGetTexture2DArrayDataSettings::CreateElement() const
{
	return MakeShared<FPCGGetTexture2DArrayDataElement>();
}

bool FPCGGetTexture2DArrayDataElement::CanExecuteOnlyOnMainThread(FPCGContext* InContext) const
{
	// PrepareData does loading, which requires the game thread.
	return !InContext || InContext->CurrentPhase == EPCGExecutionPhase::PrepareData;
}

void FPCGGetTexture2DArrayDataElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	if (InParams.ExecutionSource)
	{
		if (const UPCGData* Data = InParams.ExecutionSource->GetExecutionState().GetSelfData())
		{
			Crc.Combine(Data->GetOrComputeCrc(/*bFullDataCrc=*/false));
		}
	}

	OutCrc = Crc;
}

bool FPCGGetTexture2DArrayDataElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetTexture2DArrayDataElement::PrepareDataInternal);
	check(InContext);

	FPCGGetTexture2DArrayDataContext* Context = static_cast<FPCGGetTexture2DArrayDataContext*>(InContext);

	const UPCGGetTexture2DArrayDataSettings* Settings = Context->GetInputSettings<UPCGGetTexture2DArrayDataSettings>();
	check(Settings);

	if (Settings->Texture.IsNull())
	{
		return true;
	}

	if (!Context->WasLoadRequested())
	{
		return Context->RequestResourceLoad(Context, { Settings->Texture.ToSoftObjectPath() }, !Settings->bSynchronousLoad);
	}

	return true;
}

bool FPCGGetTexture2DArrayDataElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetTexture2DArrayDataElement::Execute);
	check(Context);

	const UPCGGetTexture2DArrayDataSettings* Settings = Context->GetInputSettings<UPCGGetTexture2DArrayDataSettings>();
	check(Settings);

	if (Settings->Texture.IsNull())
	{
		return true;
	}

	UTexture* Texture = Settings->Texture.Get();

	if (!Texture)
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("CouldNotResolveTexture", "Texture at path '{0}' could not be loaded."), FText::FromString(Settings->Texture.ToString())));
		return true;
	}

	if (!Texture->IsA<UTexture2DArray>() && !Texture->IsA<UTextureRenderTarget2DArray>())
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InvalidType", "Texture at path '{0}' was not a valid type. Must be UTexture2DArray or UTextureRenderTarget2DArray."), FText::FromString(Settings->Texture.ToString())));
		return true;
	}

	FPCGTexture2DArrayDataInitParams InitParams;
	InitParams.Filter = Settings->Filter;
	InitParams.Transform = UPCGTexture2DBaseData::ComputeTransform(Context->ExecutionSource.Get());

	UPCGTexture2DArrayData* Texture2DArrayData = FPCGContext::NewObject_AnyThread<UPCGTexture2DArrayData>(Context);
	Texture2DArrayData->Initialize(Texture, InitParams);

	Context->OutputData.TaggedData.Emplace_GetRef().Data = Texture2DArrayData;

#if WITH_EDITOR
	// If we have an override, register for dynamic tracking.
	if (Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGGetTexture2DArrayDataSettings, Texture)))
	{
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, FPCGSelectionKey::CreateFromPath(Texture), /*bIsCulled=*/false);
	}
#endif // WITH_EDITOR

	return true;
}

#undef LOCTEXT_NAMESPACE
