// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetTextureInfo.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Compute/PCGComputeCommon.h"
#include "Data/PCGTexture2DBaseData.h"

#include "PixelFormat.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetTextureInfo)

#define LOCTEXT_NAMESPACE "PCGGetTextureInfoElement"

namespace PCGGetTextureInfoConstants
{
	const FName InputTextureLabel = TEXT("Texture");
}

#if WITH_EDITOR
FName UPCGGetTextureInfoSettings::GetDefaultNodeName() const
{
	return TEXT("GetTextureInfo");
}

FText UPCGGetTextureInfoSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Texture Info");
}

FText UPCGGetTextureInfoSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Returns texture metadata as attributes.");
}
#endif

TArray<FPCGPinProperties> UPCGGetTextureInfoSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	FPCGPinProperties& TexturePin = PinProperties.Emplace_GetRef(
		PCGGetTextureInfoConstants::InputTextureLabel,
		FPCGDataTypeInfoTexture2DBase::AsId(),
		/*bAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true);
	TexturePin.SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGetTextureInfoSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(
		PCGPinConstants::DefaultOutputLabel,
		EPCGDataType::Param,
		/*bAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("OutParamTooltip", "Attribute set containing the texture metadata"));

	return PinProperties;
}

FPCGElementPtr UPCGGetTextureInfoSettings::CreateElement() const
{
	return MakeShared<FPCGGetTextureInfoElement>();
}

bool FPCGGetTextureInfoElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetTextureInfoElement::Execute);

	check(Context);

	const UPCGGetTextureInfoSettings* Settings = Context->GetInputSettings<UPCGGetTextureInfoSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> TextureInputs = Context->InputData.GetInputsByPin(PCGGetTextureInfoConstants::InputTextureLabel);

	for (const FPCGTaggedData& Input : TextureInputs)
	{
		const UPCGTexture2DBaseData* TextureData = Cast<UPCGTexture2DBaseData>(Input.Data);
		if (!TextureData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("UnsupportedTextureType", "Input data is not a supported texture type."));
			continue;
		}

		const FIntPoint Resolution = TextureData->GetResolution();
		const EPixelFormat PixelFormat = TextureData->GetFormat();
		const EPCGRenderTargetFormat ComputeFormat = PCGComputeHelpers::GetPCGRenderTargetFormatFromPixelFormat(PixelFormat);

		UPCGParamData* OutputParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		UPCGMetadata* Metadata = OutputParamData->Metadata;
		check(Metadata);

		auto CreateAttribute = [Metadata, Context](bool bInEnabled, FName InAttributeName, auto InValue)
		{
			if (bInEnabled && !Metadata->CreateAttribute(InAttributeName, InValue, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false))
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FailedToCreateAttribute", "Failed to create attribute '{0}'."), FText::FromName(InAttributeName)), Context);
			}
		};

		CreateAttribute(Settings->bOutputWidth, Settings->WidthAttributeName, Resolution.X);
		CreateAttribute(Settings->bOutputHeight, Settings->HeightAttributeName, Resolution.Y);
		CreateAttribute(Settings->bOutputNumMips, Settings->NumMipsAttributeName, static_cast<int32>(TextureData->GetNumMips()));
		CreateAttribute(Settings->bOutputArraySize, Settings->ArraySizeAttributeName, static_cast<int32>(TextureData->GetArraySize()));
		CreateAttribute(Settings->bOutputFormat, Settings->FormatAttributeName, FName(GetPixelFormatString(PixelFormat)));
		CreateAttribute(Settings->bOutputFormatIndex, Settings->FormatIndexAttributeName, static_cast<int32>(PixelFormat));
		CreateAttribute(Settings->bOutputComputeFormat, Settings->ComputeFormatAttributeName, FName(StaticEnum<EPCGRenderTargetFormat>()->GetNameStringByValue(static_cast<int64>(ComputeFormat))));
		CreateAttribute(Settings->bOutputComputeFormatIndex, Settings->ComputeFormatIndexAttributeName, static_cast<int32>(ComputeFormat));

		Metadata->AddEntry();

		FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
		Output.Data = OutputParamData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
