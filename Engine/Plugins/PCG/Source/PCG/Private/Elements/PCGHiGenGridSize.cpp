// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGHiGenGridSize.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGPin.h"
#include "Data/PCGVolumeData.h"
#include "Helpers/PCGConversion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGHiGenGridSize)

#define LOCTEXT_NAMESPACE "PCGHiGenGridSizeElement"

namespace PCGHiGenGridSizeConstants
{
	const FName NodeName = TEXT("HiGenGridSize");
	const FName CellVolumeOutputLabel = TEXT("Grid Cell Volume");

	const FText SetSizeNodeTitle = LOCTEXT("SetSizeNodeTitle", "Set Grid Size");
	const FText ChangeSizeNodeTitle = LOCTEXT("ChangeSizeNodeTitle", "Change Grid Size");

	const int32 SetGridSizePreconfiguredIndex = 0;
	const int32 ChangeGridSizePreconfiguredIndex = 1;
}

uint32 UPCGHiGenGridSizeSettings::GetScaledGridSize() const
{
	if (HiGenGridSize == EPCGHiGenGrid::Unbounded)
	{
		return PCGHiGenGrid::UnboundedGridSize();
	}

	uint32 GridSize = PCGHiGenGrid::GridToGridSize(HiGenGridSize);

	if (UPCGGraph* Graph = GetTypedOuter<UPCGGraph>())
	{
		GridSize = FMath::RoundToInt(GridSize * Graph->GetGridSizeMultiplier());
	}

	return GridSize;
}

bool UPCGHiGenGridSizeSettings::ShouldOutputCellVolumeData() const
{
#if !WITH_EDITOR
	if (const UPCGNode* Node = Cast<UPCGNode>(GetOuter()))
	{
		return Node->IsOutputPinConnected(PCGHiGenGridSizeConstants::CellVolumeOutputLabel);
	}
	else
#endif
	{
		// In editor, we always want to generate this data for inspection & to prevent caching issues.
		return true;
	}
}

#if WITH_EDITOR
FName UPCGHiGenGridSizeSettings::GetDefaultNodeName() const
{
	return PCGHiGenGridSizeConstants::NodeName;
}

FText UPCGHiGenGridSizeSettings::GetDefaultNodeTitle() const
{
	return bShowInputPin ? PCGHiGenGridSizeConstants::ChangeSizeNodeTitle : PCGHiGenGridSizeConstants::SetSizeNodeTitle;
}

FText UPCGHiGenGridSizeSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Set the execution grid size for downstream nodes. Enables executing a single graph across a hierarchy of grids."
		"\n\nHas no effect if any of the following are true:"
		"\n\t* Generating PCG component is not set to Partitioned."
		"\n\t* Hierarchical Generation is disabled in the graph settings."
		"\n\t* Executed in a subgraph, as subgraphs are always invoked on parent grid level.");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGHiGenGridSizeSettings::GetPreconfiguredInfo() const
{
	TArray<FPCGPreConfiguredSettingsInfo> Infos;
	Infos.Emplace(PCGHiGenGridSizeConstants::SetGridSizePreconfiguredIndex, PCGHiGenGridSizeConstants::SetSizeNodeTitle);
	Infos.Emplace(PCGHiGenGridSizeConstants::ChangeGridSizePreconfiguredIndex, PCGHiGenGridSizeConstants::ChangeSizeNodeTitle);

	return Infos;
}
#endif

FPCGDataTypeIdentifier UPCGHiGenGridSizeSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	check(InPin);
	// Non-dynamically-typed pins
	if (!InPin->IsOutputPin() || InPin->Properties.Label == PCGHiGenGridSizeConstants::CellVolumeOutputLabel)
	{
		return InPin->Properties.AllowedTypes;
	}
	else
	{
		return Super::GetCurrentPinTypesID(InPin);
	}
}

TArray<FPCGPinProperties> UPCGHiGenGridSizeSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	if (bShowInputPin)
	{
		FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any);
		InputPinProperty.SetRequiredPin();
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGHiGenGridSizeSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	if (bShowInputPin)
	{
		PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
	}
	else
	{
		FPCGPinProperties& DepPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);
		DepPin.Usage = EPCGPinUsage::DependencyOnly;
	}

	PinProperties.Emplace(
		PCGHiGenGridSizeConstants::CellVolumeOutputLabel,
		EPCGDataType::Spatial,
		/*bAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("VolumeOutputPinTooltip", "The volume of the current grid cell."));

	return PinProperties;
}

FPCGElementPtr UPCGHiGenGridSizeSettings::CreateElement() const
{
	return MakeShared<FPCGHiGenGridSizeElement>();
}

FString UPCGHiGenGridSizeSettings::GetAdditionalTitleInformation() const
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return FString();
	}

	const UEnum* EnumPtr = StaticEnum<EPCGHiGenGrid>();
	if (ensure(EnumPtr))
	{
		FString AdditionalTitle = EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(HiGenGridSize)).ToString();

		const UPCGNode* Node = GetTypedOuter<UPCGNode>();
		const UPCGGraph* Graph = Node ? Node->GetGraph() : nullptr;

		if (Graph && Graph->GetGridSizeMultiplier() != 1.0)
		{
			AdditionalTitle = FString::Format(TEXT("{0} (Scaled: {1})"), { AdditionalTitle, GetScaledGridSize()});
		}

		return AdditionalTitle;
	}
	else
	{
		return FString::Printf(TEXT("%d"), static_cast<int32>(HiGenGridSize));
	}
}

void UPCGHiGenGridSizeSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	bShowInputPin = PreconfiguredInfo.PreconfiguredIndex == PCGHiGenGridSizeConstants::ChangeGridSizePreconfiguredIndex;
}

#if WITH_EDITOR
TArray<FPCGPreconfiguredInfo> UPCGHiGenGridSizeSettings::GetConversionInfo() const
{
	TArray<FPCGPreconfiguredInfo> Infos;

	if (bShowInputPin)
	{
		Infos.Emplace(PCGHiGenGridSizeConstants::SetGridSizePreconfiguredIndex, PCGHiGenGridSizeConstants::SetSizeNodeTitle);
	}
	else
	{
		Infos.Emplace(PCGHiGenGridSizeConstants::ChangeGridSizePreconfiguredIndex, PCGHiGenGridSizeConstants::ChangeSizeNodeTitle);
	}

	return Infos;
}
#endif // WITH_EDITOR

bool UPCGHiGenGridSizeSettings::ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo)
{
#if WITH_EDITOR
	const bool bWasShowingInputPin = bShowInputPin;
#endif
	bShowInputPin = ConversionInfo.PreconfiguredIndex == PCGHiGenGridSizeConstants::ChangeGridSizePreconfiguredIndex;

	if (UPCGNode* Node = Cast<UPCGNode>(GetOuter()))
	{
		// Grab the old output pin and break its edges so that they don't populate the new output pin.
		UPCGPin* RemovedOutputPin = bShowInputPin ? Node->GetOutputPin(PCGPinConstants::DefaultExecutionDependencyLabel) : Node->GetOutputPin(PCGPinConstants::DefaultOutputLabel);

		if (RemovedOutputPin)
		{
			RemovedOutputPin->BreakAllEdges();
		}
	}

#if WITH_EDITOR
	// Only broadcast an update if bShowInputPin actually changed.
	if (bWasShowingInputPin != bShowInputPin)
	{
		OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Settings | EPCGChangeType::Node | EPCGChangeType::GenerationGrid);
	}
#endif

	return true;
}

#if WITH_EDITOR
EPCGChangeType UPCGHiGenGridSizeSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;

	// Grid sizes are processed during graph compilation and is part of the graph structure.
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGHiGenGridSizeSettings, bEnabled)
		|| InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGHiGenGridSizeSettings, HiGenGridSize))
	{
		ChangeType |= EPCGChangeType::Structural | EPCGChangeType::GenerationGrid;
	}

	return ChangeType;
}
#endif

bool FPCGHiGenGridSizeElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGHiGenGridSizeElement::Execute);
	check(Context);

	if (!Context->ExecutionSource.IsValid())
	{
		UE_LOGF(LogPCG, Error, "FPCGHiGenGridSizeElement: Invalid execution source, aborting.");
		return true;
	}

	const UPCGHiGenGridSizeSettings* Settings = Context->GetInputSettings<UPCGHiGenGridSizeSettings>();
	check(Settings);

	const UPCGGraph* Graph = Context->Node ? Context->Node->GetGraph() : nullptr;
	if (Graph && Graph->IsHierarchicalGenerationEnabled() && (Graph->GetDefaultGridSize() < Settings->GetScaledGridSize()))
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("GridSizeLargerThanGraphGridSize", "Grid size is larger than graph default grid size and will be automatically clamped."));
	}

	// Trivial pass through. Will only execute on the prescribed grid.
	Context->OutputData = Context->InputData;
	for (FPCGTaggedData& Data : Context->OutputData.TaggedData)
	{
		Data.Pin = PCGPinConstants::DefaultOutputLabel;
		// Since data will downstream be used by potentially multiple partitions, we have to make sure that the data is flagged as used multiple times.
		Data.bIsUsedMultipleTimes = true;
	}

	if (Settings->ShouldOutputCellVolumeData())
	{
		UPCGVolumeData* VolumeData = FPCGContext::NewObject_AnyThread<UPCGVolumeData>(Context);
		check(VolumeData);
		VolumeData->Initialize(Context->ExecutionSource->GetExecutionState().GetBounds());

		FPCGTaggedData& VolumeOutput = Context->OutputData.TaggedData.Emplace_GetRef();
		VolumeOutput.Data = VolumeData;
		VolumeOutput.Pin = PCGHiGenGridSizeConstants::CellVolumeOutputLabel;
	}

	return true;
}

void FPCGHiGenGridSizeElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	if (const UPCGHiGenGridSizeSettings* Settings = Cast<UPCGHiGenGridSizeSettings>(InParams.Settings); Settings && Settings->ShouldOutputCellVolumeData())
	{
		// The grid cell volume output depends on the component transform.
		const UPCGData* ActorData = InParams.ExecutionSource ? InParams.ExecutionSource->GetExecutionState().GetSelfData() : nullptr;
		if (ActorData)
		{
			Crc.Combine(ActorData->GetOrComputeCrc(/*bFullDataCrc=*/false));
		}
	}

	OutCrc = Crc;
}

bool FPCGHiGenGridSizeElement::SupportsGPUResidentData(FPCGContext* InContext) const
{
	return PCGSystemSwitches::CVarPassGPUDataThroughGridLinks.GetValueOnAnyThread();
}

#undef LOCTEXT_NAMESPACE
