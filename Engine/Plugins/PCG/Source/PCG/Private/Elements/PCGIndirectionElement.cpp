// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGIndirectionElement.h"

#include "PCGPin.h"
#include "PCGSettings.h"
#include "Elements/PCGExecuteBlueprint.h" // Blueprint element class
#include "Helpers/PCGDynamicTrackingHelpers.h" // Dynamic tracking
#include "Helpers/PCGSettingsHelpers.h"   // Graph and log errors

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGIndirectionElement)

#define LOCTEXT_NAMESPACE "PCGIndirectionElement"

namespace PCGIndirectionSettings
{
	const FText DefaultNodeTitle = LOCTEXT("NodeTitle", "Proxy");
}

#if WITH_EDITOR
void UPCGIndirectionSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGIndirectionSettings, Settings)) || !Settings)
	{
		// Dynamic tracking or null settings
		return;
	}

	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(Settings);

	OutKeysToSettings.FindOrAdd(Key).Emplace(this, /*bCulling=*/false);
}

FName UPCGIndirectionSettings::GetDefaultNodeName() const
{
	return FName(TEXT("Proxy"));
}

FText UPCGIndirectionSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Proxy");
}

FText UPCGIndirectionSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Executes another settings object, which can be overridden.");
}
#endif // WITH_EDITOR

FString UPCGIndirectionSettings::GetAdditionalTitleInformation() const
{
	switch (ProxyInterfaceMode)
	{
	case EPCGProxyInterfaceMode::ByNativeElement:
		if (IsValid(SettingsClass))
		{
#if WITH_EDITOR
			const UPCGSettings* SettingsDefaultObject = CastChecked<UPCGSettings>(SettingsClass->GetDefaultObject());
			return SettingsDefaultObject->GetDefaultNodeTitle().ToString();
#else
			return SettingsClass->GetName();
#endif
		}
		break;
	case EPCGProxyInterfaceMode::ByBlueprintElement:
		if (IsValid(BlueprintElementClass))
		{
#if WITH_EDITOR
			return BlueprintElementClass->GetDisplayNameText().ToString();
#else
			return BlueprintElementClass->GetName();
#endif
		}
		break;
	case EPCGProxyInterfaceMode::BySettings:
		if (const UPCGSettings* SettingsPtr = Settings.Get())
		{
			return SettingsPtr->GetName();
		}
		break;

	default:
		checkNoEntry();
	}

	return LOCTEXT("MissingAsset", "Missing Asset").ToString();
}

TArray<FPCGPinProperties> UPCGIndirectionSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> InputProperties;
	bool bSetProperties = false;

	switch (ProxyInterfaceMode)
	{
		case EPCGProxyInterfaceMode::ByNativeElement:
			if (IsValid(SettingsClass))
			{
				const UPCGSettings* SettingsDefaultObject = CastChecked<UPCGSettings>(SettingsClass->GetDefaultObject());
				InputProperties = SettingsDefaultObject->DefaultInputPinProperties();
				bSetProperties = true;
			}
			break;
		case EPCGProxyInterfaceMode::ByBlueprintElement:
			if (IsValid(BlueprintElementClass))
			{
				const UPCGBlueprintBaseElement* BlueprintSettingsDefaultObject = CastChecked<UPCGBlueprintBaseElement>(BlueprintElementClass->GetDefaultObject());
				InputProperties = BlueprintSettingsDefaultObject->GetInputPins();
				bSetProperties = true;
			}
			break;
		case EPCGProxyInterfaceMode::BySettings:
			if (const UPCGSettings* SettingsPtr = Settings.Get())
			{
				InputProperties = SettingsPtr->DefaultInputPinProperties();
				bSetProperties = true;
			}
			break;

		default:
			checkNoEntry();
	}

	// If we couldn't get a source of pins, use the default pins instead.
	if (!bSetProperties)
	{
		InputProperties = Super::InputPinProperties();
		// However, unlike the default pins, we don't want to make those as required if the dynamic dispatch does not use them.
		for (FPCGPinProperties& PinProperty : InputProperties)
		{
			if (PinProperty.IsRequiredPin())
			{
				PinProperty.SetNormalPin();
			}
		}
	}

	return InputProperties;
}

TArray<FPCGPinProperties> UPCGIndirectionSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> OutputProperties;

	switch (ProxyInterfaceMode)
	{
		case EPCGProxyInterfaceMode::ByNativeElement:
			if (IsValid(SettingsClass))
			{
				const UPCGSettings* SettingsDefaultObject = CastChecked<UPCGSettings>(SettingsClass->GetDefaultObject());
				OutputProperties = SettingsDefaultObject->DefaultOutputPinProperties();
			}
			break;
		case EPCGProxyInterfaceMode::ByBlueprintElement:
			if (IsValid(BlueprintElementClass))
			{
				const UPCGBlueprintBaseElement* BlueprintSettingsDefaultObject = CastChecked<UPCGBlueprintBaseElement>(BlueprintElementClass->GetDefaultObject());
				OutputProperties = BlueprintSettingsDefaultObject->GetOutputPins();
			}
			break;
		case EPCGProxyInterfaceMode::BySettings:
			if (const UPCGSettings* SettingsPtr = Settings.Get())
			{
				OutputProperties = SettingsPtr->DefaultOutputPinProperties();
			}
			break;

		default:
			checkNoEntry();
	}

	if (OutputProperties.IsEmpty())
	{
		return Super::OutputPinProperties();
	}

	return OutputProperties;
}

FPCGElementPtr UPCGIndirectionSettings::CreateElement() const
{
	return MakeShared<FPCGIndirectionElement>();
}

bool FPCGIndirectionElement::CanExecuteOnlyOnMainThread(FPCGContext* InContext) const
{
	if (!InContext)
	{
		return true;
	}

	const FPCGIndirectionContext* Context = static_cast<FPCGIndirectionContext*>(InContext);

	if (Context->InnerElement)
	{
		return Context->InnerElement->CanExecuteOnlyOnMainThread(Context->InnerContext);
	}
	else
	{
		return true;
	}
}

FPCGIndirectionContext::~FPCGIndirectionContext()
{
	FPCGContext::Release(InnerContext);
	InnerContext = nullptr;
}

void FPCGIndirectionContext::SetupInnerContext()
{
	// Note: we need to pass a null node here + add the inner settings as part of the input so that they are retrieved properly
	FPCGDataCollection InnerInput = InputData;
	InnerInput.TaggedData.Emplace_GetRef().Data = InnerSettings;

	// TODO: there are some types of settings that might require a bit more information to be able to do their processing correctly
	// namely (dynamic) subgraphs, so YMMV.
	InnerContext = InnerElement->Initialize(FPCGInitializeElementParams(&InnerInput, ExecutionSource.Get(), nullptr));
	check(InnerContext);
	InitializeGraphExecutor(InnerContext);
	InnerContext->InitializeSettings();

	// Unclear whether we should give those new values
	InnerContext->TaskId = TaskId;
	InnerContext->CompiledTaskId = CompiledTaskId;
	InnerContext->DependenciesCrc = DependenciesCrc;

	InnerContext->AsyncState = AsyncState;
}

void FPCGIndirectionContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	if (InnerSettings)
	{
		Collector.AddReferencedObject(InnerSettings);
	}

	if (InnerContext)
	{
		InnerContext->AddExtraStructReferencedObjects(Collector);
	}
}

FPCGContext* FPCGIndirectionElement::CreateContext()
{
	return new FPCGIndirectionContext();
}

bool FPCGIndirectionElement::PrepareDataInternal(FPCGContext* InContext) const
{
	check(InContext);
	FPCGIndirectionContext* Context = static_cast<FPCGIndirectionContext*>(InContext);

	const UPCGIndirectionSettings* Settings = Context->GetInputSettings<UPCGIndirectionSettings>();
	check(Settings);

	Context->InnerSettings = Settings->Settings.Get();

	if(UPCGSettings* InnerSettings = Context->InnerSettings)
	{
		if (Settings->ProxyInterfaceMode == EPCGProxyInterfaceMode::ByNativeElement)
		{
			if (!Settings->SettingsClass || !InnerSettings->GetClass()->IsChildOf(Settings->SettingsClass))
			{
				Context->bShouldActAsPassthrough = true;
				PCGE_LOG_C(Error, GraphAndLog, Context, LOCTEXT("NativeProxySettingsInterfaceTypeMismatch", "The selected native settings template does not match the set or overridden settings!"));

				return true;
			}
		}
		else if (Settings->ProxyInterfaceMode == EPCGProxyInterfaceMode::ByBlueprintElement)
		{
			const UPCGBlueprintSettings* BlueprintSettings = Cast<UPCGBlueprintSettings>(InnerSettings);
			if (!Settings->BlueprintElementClass || !BlueprintSettings || !BlueprintSettings->GetElementType()->IsChildOf(Settings->BlueprintElementClass))
			{
				Context->bShouldActAsPassthrough = true;
				PCGE_LOG_C(Error, GraphAndLog, Context, LOCTEXT("BlueprintProxySettingsInterfaceTypeMismatch", "The selected blueprint settings template does not match the set or overridden settings!"));

				return true;
			}
		} 

		Context->InnerElement = InnerSettings->GetElement();
		check(Context->InnerElement);

		if (!Context->InnerContext)
		{
			Context->SetupInnerContext();
		}

		if (!Context->InnerElement->ConvertInputsIfNeeded(Context->InnerContext))
		{
			return false;
		}

#if WITH_EDITOR
		// If we have an override, register for dynamic tracking.
		if (Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGIndirectionSettings, Settings)))
		{
			FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, FPCGSelectionKey::CreateFromPath(Context->InnerSettings), /*bIsCulled=*/false);
		}
#endif // WITH_EDITOR
	}

	return true;
}

bool FPCGIndirectionElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGIndirectionElement::Execute);

	check(InContext);
	FPCGIndirectionContext* Context = static_cast<FPCGIndirectionContext*>(InContext);

	const UPCGIndirectionSettings* Settings = Context->GetInputSettings<UPCGIndirectionSettings>();
	check(Settings);

	// If Settings has not been set or overriden, act as passthrough
	if (Context->bShouldActAsPassthrough || !Context->InnerElement || !Context->InnerContext)
	{
		Context->OutputData = Context->InputData;
		return true;
	}

	if (Context->InnerSettings && Context->InnerSettings->ShouldExecuteOnGPU())
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("UnsupportedGPUProxyNode", "GPU nodes do not currently support execution via proxy."));
		Context->OutputData = Context->InputData;
		return true;
	}

	// TODO: use caching when possible
	// TODO: see what we can do for inspection data
	// TODO: support pausing in inner element, might require some upstream changes in the graph executor
	Context->InnerContext->AsyncState = Context->AsyncState;
	const bool bElementDone = Context->InnerElement->Execute(Context->InnerContext);

	// Implementation note: to make sure everything is clean vs. the root set, we need to copy the output data
	// regardless of whether the element is done or not
	Context->OutputData = Context->InnerContext->OutputData;

	// Finally, move the inner context pin data (which does not exist as-is on this node) to tags if required
	if (bElementDone && Settings->bTagOutputsBasedOnOutputPins)
	{
		for (FPCGTaggedData& TaggedData : Context->OutputData.TaggedData)
		{
			if (TaggedData.Pin != NAME_None)
			{
				TaggedData.Tags.Add(TaggedData.Pin.ToString());
			}
		}
	}

	return bElementDone;
}

#undef LOCTEXT_NAMESPACE