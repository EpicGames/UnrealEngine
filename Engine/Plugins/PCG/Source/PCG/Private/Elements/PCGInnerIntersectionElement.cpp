// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGInnerIntersectionElement.h"
#include "Data/PCGSpatialData.h"
#include "PCGContext.h"
#include "PCGPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGInnerIntersectionElement)

#define LOCTEXT_NAMESPACE "PCGInnerIntersectionElement"

#if WITH_EDITOR
FText UPCGInnerIntersectionSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltipText", "Spatial data will be generated as the result of intersecting with the other source inputs sequentially or no output if such an intersection does not exist. \nSee also: Intersection Node");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGInnerIntersectionSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& SourcePinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spatial);
	SourcePinProperty.SetRequiredPin();

#if WITH_EDITOR
	SourcePinProperty.Tooltip = LOCTEXT("SourcePinTooltip", "Source spatial data from which to conduct the intersection. Empty spatial data will be ignored.");
#endif // WITH_EDITOR
	
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGInnerIntersectionSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& OutputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spatial);

#if WITH_EDITOR
	OutputPinProperty.Tooltip = LOCTEXT("OutputPinTooltip", "The intersection created from all the source input data.");
#endif // WITH_EDITOR
	
	return PinProperties;
}

FPCGElementPtr UPCGInnerIntersectionSettings::CreateElement() const
{
	return MakeShared<FPCGInnerIntersectionElement>();
}

bool FPCGInnerIntersectionElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGInnerIntersectionElement::Execute);

	const UPCGInnerIntersectionSettings* Settings = Context->GetInputSettings<UPCGInnerIntersectionSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	const EPCGIntersectionDensityFunction DensityFunction = Settings->DensityFunction;
	const bool bKeepZeroDensityPoints = Settings->bKeepZeroDensityPoints;

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const UPCGSpatialData* FirstSpatialData = nullptr;
	UPCGIntersectionData* IntersectionData = nullptr;
	int32 IntersectionTaggedDataIndex = -1;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Input.Data);

		// Non-spatial data we're not going to touch
		if (!SpatialData)
		{
			Outputs.Add(Input);
			continue;
		}

		if (!FirstSpatialData)
		{
			FirstSpatialData = SpatialData;
			IntersectionTaggedDataIndex = Outputs.Num();
			Outputs.Add(Input);
			continue;
		}

		// Create a new intersection
		IntersectionData = (IntersectionData ? IntersectionData : FirstSpatialData)->IntersectWith(Context, SpatialData);
		// Propagate settings
		IntersectionData->DensityFunction = DensityFunction;
		IntersectionData->bKeepZeroDensityPoints = bKeepZeroDensityPoints;

		// Update tagged data
		FPCGTaggedData& IntersectionTaggedData = Outputs[IntersectionTaggedDataIndex];
		IntersectionTaggedData.Data = IntersectionData;
		IntersectionTaggedData.Tags.Append(Input.Tags);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE