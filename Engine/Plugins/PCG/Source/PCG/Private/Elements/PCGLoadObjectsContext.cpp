// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGLoadObjectsContext.h"

#include "PCGElement.h"
#include "PCGModule.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#define LOCTEXT_NAMESPACE "PCGLoadObjectsFromPathContext"

bool FPCGLoadObjectsFromPathContext::InitializeAndRequestLoad(
	FName InputPinName,
	const FPCGAttributePropertyInputSelector& InputAttributeSelector,
	const TArray<FSoftObjectPath>& StaticObjectPaths,
	bool bPersistAllData,
	bool bSilenceErrorOnEmptyObjectPath,
	bool bSynchronousLoad)
{
	if (!WasLoadRequested())
	{
		const bool InPinIsConnected = Node && Node->IsInputPinConnected(InputPinName);

		// First gather all the soft objects paths to extract
		TArray<FSoftObjectPath> ObjectsToLoad;
		if (!InPinIsConnected)
		{
			PathsToObjectsAndDataIndex.Reserve(StaticObjectPaths.Num());
			for (int32 PathIndex = 0; PathIndex < StaticObjectPaths.Num(); ++PathIndex)
			{
				const FSoftObjectPath& Path = StaticObjectPaths[PathIndex];
				if (!Path.IsNull())
				{
					ObjectsToLoad.AddUnique(Path);
				}

				PathsToObjectsAndDataIndex.Emplace(Path, -1, PathIndex);
			}
		}
		else
		{
			const TArray<FPCGTaggedData> Inputs = InputData.GetInputsByPin(InputPinName);
			for (int32 Index = 0; Index < Inputs.Num(); ++Index)
			{
				const FPCGTaggedData& Input = Inputs[Index];

				if (!Input.Data)
				{
					PCGE_LOG_C(Error, GraphAndLog, this, FText::Format(LOCTEXT("InvalidData", "Invalid data for input {0}"), FText::AsNumber(Index)));
					continue;
				}

				const FPCGAttributePropertyInputSelector AttributeSelector = InputAttributeSelector.CopyAndFixLast(Input.Data);
				const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, AttributeSelector);
				const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, AttributeSelector);

				if (!Accessor.IsValid() || !Keys.IsValid())
				{
					if (bPersistAllData)
					{
						// Special case for empty data. We need this case if we ever chain this node multiple times. An empty param (with no attributes and no entries) will generate another empty param.
						const UPCGMetadata* Metadata = Input.Data->ConstMetadata();
						if (Metadata && Metadata->GetAttributeCount() == 0 && Metadata->GetLocalItemCount() == 0)
						{
							// Emplace empty path for this input. Will generate an empty param data.
							PathsToObjectsAndDataIndex.Emplace(FSoftObjectPath(), Index, -1);
						}
					}

					if (!bSilenceErrorOnEmptyObjectPath)
					{
						PCGE_LOG_C(Error, GraphAndLog, this, FText::Format(LOCTEXT("AttributeNotFound", "Attribute/Property '{0}' does not exist on input {1}"), AttributeSelector.GetDisplayText(), FText::AsNumber(Index)));
					}

					continue;
				}

				const int32 NumElementsToAdd = Keys->GetNum();
				if (NumElementsToAdd == 0)
				{
					continue;
				}

				// Extract value as String to validate that a path is empty or ill-formed (because any ill-formed path will be null).
				TArray<FString> InputValues;
				InputValues.SetNum(NumElementsToAdd);
				if (Accessor->GetRange(MakeArrayView(InputValues), 0, *Keys, EPCGAttributeAccessorFlags::AllowConstructible | EPCGAttributeAccessorFlags::AllowBroadcast))
				{
					PathsToObjectsAndDataIndex.Reserve(PathsToObjectsAndDataIndex.Num() + NumElementsToAdd);
					for (int32 i = 0; i < InputValues.Num(); ++i)
					{
						FString& StringPath = InputValues[i];
						// Empty SoftObjectPath can convert to string to None and is treated as empty, so check that one too.
						const bool PathIsEmpty = StringPath.IsEmpty() || StringPath.Equals(TEXT("None"), ESearchCase::CaseSensitive);
						if (PathIsEmpty && bPersistAllData)
						{
							PathsToObjectsAndDataIndex.Emplace(FSoftObjectPath(), Index, i);
						}

						FSoftObjectPath Path(std::move(StringPath));

						if (!Path.IsNull())
						{
							ObjectsToLoad.AddUnique(Path);
							PathsToObjectsAndDataIndex.Emplace(std::move(Path), Index, i);
						}
						else
						{
							if (!PathIsEmpty || !bSilenceErrorOnEmptyObjectPath)
							{
								PCGE_LOG_C(Error, GraphAndLog, this, FText::Format(LOCTEXT("InvalidPath", "Value number {0} for Attribute/Property '{1}' on input {2} is not a valid path or is null. Will be ignored."), FText::AsNumber(i), AttributeSelector.GetDisplayText(), FText::AsNumber(Index)));
							}

							continue;
						}
					}
				}
				else
				{
					PCGE_LOG_C(Error, GraphAndLog, this, FText::Format(LOCTEXT("InvalidAttribute", "Attribute/Property '{0}'({1}) is not convertible to a SoftObjectPath on input {2}"), AttributeSelector.GetDisplayText(), PCG::Private::GetTypeNameText(Accessor->GetUnderlyingType()), FText::AsNumber(Index)));
					continue;
				}
			}
		}

		if (!ObjectsToLoad.IsEmpty())
		{
			return RequestResourceLoad(this, std::move(ObjectsToLoad), !bSynchronousLoad);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE