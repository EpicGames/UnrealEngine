// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGLoopElement.h"

#include "PCGComponent.h"
#include "PCGCustomVersion.h"
#include "PCGModule.h"
#include "PCGPin.h"
#include "Graph/PCGStackContext.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "Helpers/PCGHelpers.h"

#include "Algo/Copy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGLoopElement)

#define LOCTEXT_NAMESPACE "PCGLoopElement"

#if WITH_EDITOR
FText UPCGLoopSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Loop");
}

FText UPCGLoopSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Executes the specified Subgraph for each data on the loop pins (or on the first pin if no specific loop pins are provided), keeping the rest constant.");
}

void UPCGLoopSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	if (DataVersion < FPCGCustomVersion::UpdateGraphSettingsLoopPins)
	{
		bUseGraphDefaultPinUsage = LoopPins.IsEmpty();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DataVersion < FPCGCustomVersion::AttributesAndTagsCanContainSpaces)
	{
		bTokenizeOnWhiteSpace = true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Super::ApplyDeprecation(InOutNode);
}

bool UPCGLoopSettings::GetPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip) const
{
	if (bUseGraphDefaultPinUsage)
	{
		// Direct base class (Subgraph) removes these for readability, but here we explicitly want these
		return UPCGSettings::GetPinExtraIcon(InPin, OutExtraIcon, OutTooltip);
	}
	else if(InPin)
	{
		// In this case, we'll check if the given pin matches our list and get the extra from there
		TArray<FName> LoopPinNames;
		TArray<FName> FeedbackPinNames;
		GetLoopPinNames(nullptr, LoopPinNames, FeedbackPinNames, /*bQuiet=*/true);

		FPCGPinProperties PinProperties = InPin->Properties;
		PinProperties.Usage = EPCGPinUsage::Normal;

		if (LoopPinNames.Contains(PinProperties.Label))
		{
			PinProperties.Usage = EPCGPinUsage::Loop;
		}
		else if (FeedbackPinNames.Contains(PinProperties.Label))
		{
			PinProperties.Usage = EPCGPinUsage::Feedback;
		}

		return PCGPinPropertiesHelpers::GetDefaultPinExtraIcon(PinProperties, OutExtraIcon, OutTooltip);
	}

	return false;
}

void UPCGLoopSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Trigger a cosmetic change because all properties on this class might trigger a visual change on the pins.
	OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Cosmetic);
}
#endif // WITH_EDITOR

void UPCGLoopSettings::GetLoopPinNames(FPCGContext* Context, TArray<FName>& LoopPinNames, TArray<FName>& FeedbackPinNames, bool bQuiet) const
{
	if (bUseGraphDefaultPinUsage)
	{
		const UPCGGraph* Subgraph = GetSubgraph();

		if (!Subgraph)
		{
			return;
		}

		const UPCGNode* SubgraphInputNode = Subgraph->GetInputNode();
		check(SubgraphInputNode);

		TArray<FPCGPinProperties> InputPins = SubgraphInputNode->InputPinProperties();
		for (const FPCGPinProperties& InputPin : InputPins)
		{
			if (InputPin.Usage == EPCGPinUsage::Loop)
			{
				LoopPinNames.Add(InputPin.Label);
			}
			else if (InputPin.Usage == EPCGPinUsage::Feedback)
			{
				// Ignore feedback pins that have no connection.
				if (SubgraphInputNode->GetOutputPin(InputPin.Label)->EdgeCount() > 0)
				{
					FeedbackPinNames.Add(InputPin.Label);
				}
			}
		}
	}
	else
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TArray<FString> PinsFromSettings = bTokenizeOnWhiteSpace
			? PCGHelpers::GetStringArrayFromCommaSeparatedString(LoopPins, Context)
			: PCGHelpers::GetStringArrayFromCommaSeparatedList(LoopPins);

		for (const FString& PinLabel : PinsFromSettings)
		{
			LoopPinNames.Emplace(PinLabel);
		}

		PinsFromSettings = bTokenizeOnWhiteSpace
			? PCGHelpers::GetStringArrayFromCommaSeparatedString(FeedbackPins, Context)
			: PCGHelpers::GetStringArrayFromCommaSeparatedList(FeedbackPins);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		for (const FString& PinLabel : PinsFromSettings)
		{
			if (LoopPinNames.Contains(PinLabel))
			{
				if (!bQuiet && Context)
				{
					PCGE_LOG_C(Warning, GraphAndLog, Context, FText::Format(LOCTEXT("SameLabelAppearsTwice", "Label '{0}' appears in both the loop pins and the feedback pins"), FText::FromString(PinLabel)));
				}
			}
			else
			{
				FeedbackPinNames.Emplace(PinLabel);
			}
		}
	}

	// If no named pins have been provided, default to the first pin
	if (LoopPinNames.IsEmpty())
	{
		const UPCGSubgraphNode* LoopNode = nullptr;

		if (Context)
		{
			LoopNode = Cast<const UPCGSubgraphNode>(Context->Node);
		}
		else
		{
			LoopNode = Cast<const UPCGSubgraphNode>(GetOuter());
		}

		if (LoopNode)
		{
			if (const UPCGPin* LoopPin = LoopNode->GetFirstConnectedInputPin())
			{
				LoopPinNames.Add(LoopPin->Properties.Label);
			}
		}
	}
}

FPCGElementPtr UPCGLoopSettings::CreateElement() const
{
	return MakeShared<FPCGLoopElement>();
}

bool FPCGLoopElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGLoopElement::Execute);

	FPCGSubgraphContext* Context = static_cast<FPCGSubgraphContext*>(InContext);

	const UPCGLoopSettings* Settings = Context->GetInputSettings<UPCGLoopSettings>();
	check(Settings);

	// If we haven't scheduled the subgraph yet (1st call) - this is part of the FPCGSubgraphContext, see FPCGSubgraphElement for more info
	if (!Context->bScheduledSubgraph)
	{
		if (Settings->SubgraphOverride)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGLoopElement::ExecuteInternal::DynamicTrackingAndOverrides);

#if WITH_EDITOR
			FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, FPCGSelectionKey::CreateFromPath(Settings->SubgraphOverride), /*bIsCulled=*/false);
			// Also need to add statically tracked keys
			if (InContext && InContext->ExecutionSource.IsValid())
			{
				if (const UPCGGraph* Graph = Settings->SubgraphOverride->GetGraph())
				{
					InContext->ExecutionSource->GetExecutionState().RegisterDynamicTracking(Graph->GetTrackedSelectionKeysToSettings());
				}
			}
#endif // WITH_EDITOR

			Context->UpdateOverridesWithOverriddenGraph();
		}

		UPCGGraph* Subgraph = Settings->GetSubgraph();

		if (!Subgraph)
		{
			// No subgraph is equivalent to disabling the node
			Context->OutputData.TaggedData = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
			return true;
		}

		TArray<FPCGDataCollection> LoopDataCollection;
		TArray<FName> FeedbackPinNames;
		FPCGDataCollection FeedbackDataCollection;
		FPCGDataCollection FixedInputDataCollection;
		PrepareLoopDataCollections(Context, Settings, LoopDataCollection, FeedbackDataCollection, FeedbackPinNames, FixedInputDataCollection);

		// Early out if there are no data on the loop pin
		if (LoopDataCollection.IsEmpty() || LoopDataCollection[0].TaggedData.IsEmpty())
		{
			PCGE_LOG(Verbose, LogOnly, LOCTEXT("EmptyLoopCollection", "Loop data is empty - will not do anything."));
			return true;
		}

		const int32 NumberOfIterations = LoopDataCollection[0].TaggedData.Num();

		// Early out if the loop collection do not contain the same amount of entries
		for (int LoopDataIndex = 1; LoopDataIndex < LoopDataCollection.Num(); ++LoopDataIndex)
		{
			if (LoopDataCollection[LoopDataIndex].TaggedData.Num() != NumberOfIterations)
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("MismatchLoopCollections", "Data to loop on does not have the same number of entries for all pins!"));
				return true;
			}
		}

		FPCGDataCollection PreSubgraphDataCollection;
		PrepareSubgraphUserParameters(Settings, Context, PreSubgraphDataCollection);
		Context->AddToReferencedObjects(PreSubgraphDataCollection);

		FPCGElementPtr PreGraphElement = MakeShared<FPCGInputForwardingElement>(PreSubgraphDataCollection);

		FPCGDataCollection FixedDataCollection;
		PrepareSubgraphData(Settings, Context, FixedInputDataCollection, FixedDataCollection);

		// If we execute more than once, it's the same as if we were using the data multiple times for fixed input.
		if (NumberOfIterations > 1)
		{
			for (FPCGTaggedData& TaggedData : FixedDataCollection.TaggedData)
			{
				TaggedData.bIsUsedMultipleTimes = true;
			}
		}

		FPCGTaskId PreviousTaskId = InvalidPCGTaskId;
		const bool bIsFeedbackLoop = !(FeedbackPinNames.IsEmpty());

		// Dispatch the subgraph for each loop entries we have.
		// Implementation note: even if the execution gets cancelled, these tasks would get cancelled because they are associated to the current source component
		for (int EntryIndex = 0; EntryIndex < NumberOfIterations; ++EntryIndex)
		{
			// The final input for a given loop iteration is going to be the fixed data + the entry at the current loop index in all the gathered loop collections
			FPCGDataCollection InputDataCollection = FixedDataCollection;
			for (int LoopCollectionIndex = 0; LoopCollectionIndex < LoopDataCollection.Num(); ++LoopCollectionIndex)
			{
				InputDataCollection.TaggedData.Insert(LoopDataCollection[LoopCollectionIndex].TaggedData[EntryIndex], LoopCollectionIndex);
			}

			// If it's the first valid iteration, pass in the iteration data, otherwise it will be gathered by the loop input forwarding element
			if (PreviousTaskId == InvalidPCGTaskId && bIsFeedbackLoop)
			{
				InputDataCollection.TaggedData.Append(FeedbackDataCollection.TaggedData);
			}

			// Prepare the invocation stack - which is the stack up to this node, and then this node, then a loop index
			const FPCGStack* Stack = Context->GetStack();
			FPCGStack InvocationStack = ensure(Stack) ? *Stack : FPCGStack();

			TArray<FPCGStackFrame>& StackFrames = InvocationStack.GetStackFramesMutable();
			StackFrames.Reserve(StackFrames.Num() + 2);
			StackFrames.Emplace(Context->Node);
			StackFrames.Emplace(EntryIndex);

			TArray<FPCGTaskId> Dependencies = {};
			if (PreviousTaskId != InvalidPCGTaskId && bIsFeedbackLoop)
			{
				Dependencies.Add(PreviousTaskId);
			}

			Context->AddToReferencedObjects(InputDataCollection);

			FPCGElementPtr InputElement = MakeShared<FPCGLoopInputForwardingElement>(InputDataCollection, PreviousTaskId, FeedbackPinNames);
			FPCGTaskId SubgraphTaskId = Context->ScheduleGraph(FPCGScheduleGraphParams(
				Subgraph,
				Context->ExecutionSource.Get(),
				PreGraphElement,
				InputElement,
				Dependencies,
				&InvocationStack,
				/*bAllowHierarchicalGeneration=*/false));

			if (SubgraphTaskId != InvalidPCGTaskId)
			{
				if (bIsFeedbackLoop)
				{
					PreviousTaskId = SubgraphTaskId;
				}

				Context->SubgraphTaskIds.Add(SubgraphTaskId);
			}
		}

		// Add a trivial task after the output tasks that wakes up this task
		if (!Context->SubgraphTaskIds.IsEmpty())
		{
			Context->bScheduledSubgraph = true;
			Context->bIsPaused = true;

			// In the case of a feedback loop we only need to depend on the last scheduled task since all other tasks depend on each other
			if (bIsFeedbackLoop)
			{
				Context->DynamicDependencies.Add(Context->SubgraphTaskIds.Last());
			}
			else
			{
				Context->DynamicDependencies.Append(Context->SubgraphTaskIds);
			}

			return false;
		}
		else
		{
			// Nothing to do
			return true;
		}
	}
	else if (Context->bIsPaused)
	{
		// Should not happen once we skip it in the graph executor
		return false;
	}
	else
	{
		// when woken up, get the output data from the subgraph
		// and copy it to the current context output data, and finally return true
		// If this was running iterations, we need to build a list of pins we'll ignore for non-terminal tasks
		TArray<FName> LoopPinNames;
		TArray<FName> FeedbackPinNames;
		Settings->GetLoopPinNames(Context, LoopPinNames, FeedbackPinNames, /*bQuiet=*/true);

		for(int SubtaskIndex = 0; SubtaskIndex < Context->SubgraphTaskIds.Num(); ++SubtaskIndex)
		{
			FPCGTaskId SubgraphTaskId = Context->SubgraphTaskIds[SubtaskIndex];
			const bool bIsLastTask = (SubtaskIndex == Context->SubgraphTaskIds.Num() - 1);

			FPCGDataCollection SubgraphOutput;
			// While this should be always return true, if a scheduled task was cancelled, this can still happen because the dependency has been removed but this code isn't aware of this
			if (Context->GetOutputData(SubgraphTaskId, SubgraphOutput))
			{
				if (FeedbackPinNames.IsEmpty() || bIsLastTask)
				{
					Context->OutputData.TaggedData.Append(SubgraphOutput.TaggedData);
				}
				else
				{
					Algo::CopyIf(SubgraphOutput.TaggedData, Context->OutputData.TaggedData, [&FeedbackPinNames](const FPCGTaggedData& InTaggedData) { return !FeedbackPinNames.Contains(InTaggedData.Pin); });
				}
				Context->ClearOutputData(SubgraphTaskId);
			}
		}

		return true;
	}
}

void FPCGLoopElement::PrepareLoopDataCollections(FPCGContext* Context, const UPCGLoopSettings* Settings, TArray<FPCGDataCollection>& LoopDataCollection, FPCGDataCollection& FeedbackDataCollection, TArray<FName>& FeedbackPinNames, FPCGDataCollection& FixedInputDataCollection) const
{
	TArray<FName> LoopPinNames;
	check(Settings);
	Settings->GetLoopPinNames(Context, LoopPinNames, FeedbackPinNames, /*bQuiet=*/false);

	LoopDataCollection.Reset();
	LoopDataCollection.SetNum(LoopPinNames.Num());

	for (const FPCGTaggedData& TaggedData : Context->InputData.TaggedData)
	{
		int LoopPinIndex = LoopPinNames.IndexOfByKey(TaggedData.Pin);
		bool bIsFeedbackData = FeedbackPinNames.Contains(TaggedData.Pin);

		if (LoopPinIndex != INDEX_NONE)
		{
			LoopDataCollection[LoopPinIndex].TaggedData.Add(TaggedData);
		}
		else if (bIsFeedbackData)
		{
			FeedbackDataCollection.TaggedData.Add(TaggedData);
		}
		else
		{
			FixedInputDataCollection.TaggedData.Add(TaggedData);
		}
	}
}

FPCGLoopInputForwardingElement::FPCGLoopInputForwardingElement(const FPCGDataCollection& StaticInputToForward, FPCGTaskId InPreviousIterationTaskId, const TArray<FName>& InFeedbackPinNames)
	: FPCGInputForwardingElement(StaticInputToForward)
	, PreviousIterationTaskId(InPreviousIterationTaskId)
	, FeedbackPinNames(InFeedbackPinNames)
{
}

bool FPCGLoopInputForwardingElement::ExecuteInternal(FPCGContext* Context) const
{
	// Performs forwarding of static data
	while (!FPCGInputForwardingElement::ExecuteInternal(Context))
	{
	}

	// Retrieve data from previous task if needed
	if (PreviousIterationTaskId != InvalidPCGTaskId && !FeedbackPinNames.IsEmpty())
	{
		check(Context && Context->ExecutionSource.IsValid());
		FPCGDataCollection PreviousTaskOutput;
		if (Context->GetOutputData(PreviousIterationTaskId, PreviousTaskOutput))
		{
			Algo::CopyIf(PreviousTaskOutput.TaggedData, Context->OutputData.TaggedData, [this](const FPCGTaggedData& InTaggedData) { return FeedbackPinNames.Contains(InTaggedData.Pin); });
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE