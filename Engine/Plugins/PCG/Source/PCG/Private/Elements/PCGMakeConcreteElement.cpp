// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMakeConcreteElement.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMakeConcreteElement)

#define LOCTEXT_NAMESPACE "PCGMakeConcreteElement"

#if WITH_EDITOR
FText UPCGMakeConcreteSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Make Concrete");
}

FText UPCGMakeConcreteSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Concrete data is passed through (e.g. Point, Curve, Landscape). "
		"Spatial data (e.g. Intersection, Difference, Union) is collapsed to Point. "
		"Non-Spatial data (e.g. Attribute Set) is discarded.");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGMakeConcreteSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Concrete);

	return PinProperties;
}

FPCGElementPtr UPCGMakeConcreteSettings::CreateElement() const
{
	return MakeShared<FPCGMakeConcreteElement>();
}

bool FPCGMakeConcreteElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMakeConcreteElement::Execute);

	TArray<FPCGTaggedData> SpatialInputs = InContext->InputData.GetAllSpatialInputs();
	for (const FPCGTaggedData& Input : SpatialInputs)
	{
		const UPCGSpatialData* InputSpatialData = Cast<UPCGSpatialData>(Input.Data);
		// Verify data really Spatial and reported typed of data also within Spatial.
		if (!ensure(InputSpatialData) || !ensure(Input.Data->GetDataTypeId().IsChildOf<FPCGDataTypeInfoSpatial>()))
		{
			continue;
		}
		
		// Determine if we have data that needs to be made concrete.
		const bool bDataIsConcrete = Input.Data->GetDataTypeId().IsChildOf<FPCGDataTypeInfoConcrete>();

		// Add output, doing a to-point to make concrete if required.
		FPCGTaggedData& Output = InContext->OutputData.TaggedData.Emplace_GetRef(Input);
		Output.Pin = PCGPinConstants::DefaultOutputLabel;
		if (!bDataIsConcrete)
		{
			Output.Data = InputSpatialData->ToBasePointData(InContext);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
