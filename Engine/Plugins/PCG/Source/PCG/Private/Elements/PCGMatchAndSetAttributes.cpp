// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMatchAndSetAttributes.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGPointDataPartition.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include "Algo/AnyOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMatchAndSetAttributes)

#define LOCTEXT_NAMESPACE "PCGMatchAndSetAttributes"

namespace PCGMatchAndSetAttributesConstants
{
	const FName MatchDataLabel = TEXT("Match Data");
	const FName MaxDistanceLabel = TEXT("Max Match Distance");
}

UPCGMatchAndSetAttributesSettings::UPCGMatchAndSetAttributesSettings()
{
	// Weight attribute is None by default
	WeightAttribute.SetAttributeName(NAME_None);
	
	// The match attribute will be @Last, but leave None for not new object as it was the previous default.
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		MatchAttribute.SetAttributeName(PCGMetadataAttributeConstants::LastAttributeName);
	}
	else
	{
		MatchAttribute.SetAttributeName(NAME_None);
	}
}

#if WITH_EDITOR
FName UPCGMatchAndSetAttributesSettings::GetDefaultNodeName() const
{
	return FName(TEXT("MatchAndSetAttributes"));
}

FText UPCGMatchAndSetAttributesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Match And Set Attributes");
}

FText UPCGMatchAndSetAttributesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Matches or randomly assigns values from the Attribute Set to the input data. Supports N (input):1 (match data), or N:N configurations.");
}

void UPCGMatchAndSetAttributesSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	check(InOutNode);

	// Param | Point type was not explicitly defined in the data types, and therefore was not serialized correctly, resulting in an Input/Output pin serialized to None.
	// Restoring the right value here, before update pins.
	auto FixInvalidAllowedTypes = [](UPCGPin* InPin)
	{
		if (InPin && InPin->Properties.AllowedTypes == EPCGDataType::None)
		{
			InPin->Properties.AllowedTypes = EPCGDataType::PointOrParam;
		}
	};

	FixInvalidAllowedTypes(InOutNode->GetInputPin(PCGPinConstants::DefaultInputLabel));
	FixInvalidAllowedTypes(InOutNode->GetInputPin(PCGMatchAndSetAttributesConstants::MaxDistanceLabel));
	FixInvalidAllowedTypes(InOutNode->GetOutputPin(PCGPinConstants::DefaultOutputLabel));
}
#endif // WITH_EDITOR

bool UPCGMatchAndSetAttributesSettings::UseSeed() const
{
	// Minor TODO: could mark use seed true only if we don't use the input weight attribute
	return true;
}

TArray<FPCGPinProperties> UPCGMatchAndSetAttributesSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::PointOrParam);
	InputPinProperty.SetRequiredPin();

	PinProperties.Emplace_GetRef(PCGMatchAndSetAttributesConstants::MatchDataLabel,
		EPCGDataType::Param,
		/*bAllowMultipleConnection=*/true,
		/*bAllowMultipleData=*/true,
		LOCTEXT("MatchDataTooltip", "Input containing the data to match to, then copy the accompanying attribute values")
	).SetRequiredPin();

	if (bFindNearest && MaxDistanceMode == EPCGMatchMaxDistanceMode::AttributeMaxDistance)
	{
		PinProperties.Emplace_GetRef(PCGMatchAndSetAttributesConstants::MaxDistanceLabel,
			EPCGDataType::PointOrParam,
			/*bAllowMultipleConnections=*/true,
			/*bAllowMultipleData=*/true,
			LOCTEXT("MaxDistanceTooltip", "Input containing the maximum distance allowed for nearest search, selected by the Max Distance Attribute.")
		).SetRequiredPin();
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGMatchAndSetAttributesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::PointOrParam);

	return PinProperties;
}

#if WITH_EDITOR
EPCGChangeType UPCGMatchAndSetAttributesSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGMatchAndSetAttributesSettings, bFindNearest) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGMatchAndSetAttributesSettings, MaxDistanceMode))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGMatchAndSetAttributesSettings::CreateElement() const
{
	return MakeShared<FPCGMatchAndSetAttributesElement>();
}

class FPCGAttributeSetPartition
{
public:
	struct AttributeSetPartitionEntry
	{
		double TotalWeight = 0;
		TArray<PCGMetadataEntryKey> Keys;
		TArray<double> CumulativeWeight;

		void AddEntry(PCGMetadataEntryKey Key, double Weight)
		{
			if (Weight > 0)
			{
				Keys.Add(Key);
				TotalWeight += Weight;
				CumulativeWeight.Add(TotalWeight);
			}
		}
	};

	FPCGAttributeSetPartition() = default;

	FPCGAttributeSetPartition(FPCGContext* InContext, const UPCGParamData* InParamData, bool bPartitionByAttribute, FName AttributeName, bool bUseWeightAttribute, FName WeightAttributeName, bool bInFindNearest, EPCGMatchMaxDistanceMode InMaxDistanceMode, const FPCGMetadataTypesConstantStruct* InMaxDistanceForNearestMatch)
	{
		Initialize(InContext, InParamData, bPartitionByAttribute, AttributeName, bUseWeightAttribute, WeightAttributeName, bInFindNearest, InMaxDistanceMode, InMaxDistanceForNearestMatch);
	}

	bool Initialize(FPCGContext* Context, const UPCGParamData* InParamData, bool bPartitionByAttribute, FName AttributeName, bool bUseWeightAttribute, FName WeightAttributeName, bool bInFindNearest, EPCGMatchMaxDistanceMode InMaxDistanceMode, const FPCGMetadataTypesConstantStruct* InMaxDistanceForNearestMatch)
	{
		ParamData = InParamData;
		bFindNearest = bInFindNearest;
		MaxDistanceMode = InMaxDistanceMode;

		if (!ParamData || !ParamData->Metadata)
		{
			return false;
		}

		const UPCGMetadata* Metadata = ParamData->ConstMetadata();

		if (bPartitionByAttribute)
		{
			Attribute = Metadata->GetConstAttribute(AttributeName);

			if (!Attribute)
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("CannotFindAttribute", "Cannot find attribute '{0}' in the source Attribute Set."), FText::FromName(AttributeName)));
				return false;
			}
		}

		const FPCGMetadataAttributeBase* WeightAttribute = nullptr;
		if (bUseWeightAttribute)
		{
			WeightAttribute = Metadata->GetConstAttribute(WeightAttributeName);

			if (!WeightAttribute)
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("CannotFindWeightAttribute", "Cannot find weight attribute '{0}' in the source Attribute Set."), FText::FromName(WeightAttributeName)));
				return false;
			}

			if(!PCG::Private::IsOfTypes<int32, int64, float, double>(WeightAttribute->GetTypeId()))
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("InvalidWeightAttributeType", "Weight attribute '{0}' does not have the proper type (int32, int64, float or double)."), FText::FromName(WeightAttributeName)));
				return false;
			}
		}

		if (Attribute && MaxDistanceMode == EPCGMatchMaxDistanceMode::UseConstantMaxDistance)
		{
			auto ValidateAttributeSupportsDistance = [](auto AttributeDummyValue) -> bool
			{
				using AttributeType = decltype(AttributeDummyValue);
				return PCG::Private::MetadataTraits<AttributeType>::CanComputeDistance;
			};

			if (!PCGMetadataAttribute::CallbackWithRightType(Attribute->GetTypeId(), ValidateAttributeSupportsDistance))
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("AttributeDoesNotSupportDistance", "Attribute '{0}' does not support distance computation."), FText::FromName(Attribute->Name)));
				return false;
			}

			if (InMaxDistanceForNearestMatch)
			{
				auto CreateConstantAttribute = [this](auto&& Value)
				{
					using ConstantType = std::decay_t<decltype(Value)>;
					ConstantThreshold = MakeUnique<FPCGConstantValueAccessor<ConstantType>>(std::forward<decltype(Value)>(Value));
					ConstantKey = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();
				};

				InMaxDistanceForNearestMatch->Dispatcher(CreateConstantAttribute);

				if (!ConstantThreshold.IsValid() || !ConstantKey.IsValid())
				{
					PCGE_LOG_C(Error, GraphAndLog, Context, LOCTEXT("InvalidConstantThresholdAttribute", "Distance threshold is invalid."));
					return false;
				}
			}
		}

		auto GetWeightFromAttribute = [WeightAttribute](PCGMetadataEntryKey EntryKey) -> double
		{
			double Weight = 1.0;

			if (WeightAttribute)
			{
				auto GetValue = [WeightAttribute, EntryKey](auto Dummy) -> double
				{
					using ValueType = decltype(Dummy);
					if constexpr (PCG::Private::IsOfTypes<ValueType, int32, int64, float, double>())
					{
						return static_cast<const FPCGMetadataAttribute<ValueType>*>(WeightAttribute)->GetValueFromItemKey(EntryKey);
					}
					else
					{
						return 1.0;
					}
				};

				Weight = PCGMetadataAttribute::CallbackWithRightType(WeightAttribute->GetTypeId(), GetValue);
			}

			return Weight;
		};
		
		// Note: since we don't have an accessor to the entries from the metadata,
		// we're going to assume that they exist in a consecutive sequence, which should hold true for param data.
		const int64 FirstKey = Metadata->GetItemKeyCountForParent();
		const int64 KeyCount = Metadata->GetLocalItemCount();

		for (int64 EntryKey = FirstKey; EntryKey < FirstKey + KeyCount; ++EntryKey)
		{
			PCGMetadataValueKey ValueKey = PCGDefaultValueKey;
			if (Attribute)
			{
				ValueKey = Attribute->GetValueKey(EntryKey);
			}

			double Weight = GetWeightFromAttribute(EntryKey);
			TPair<PCGMetadataValueKey, AttributeSetPartitionEntry>* MatchingVK = nullptr;

			if (Attribute)
			{
				MatchingVK = PartitionData.FindByPredicate([this, ValueKey](const TPair<PCGMetadataValueKey, AttributeSetPartitionEntry>& Entry)
				{
					return Attribute->AreValuesEqual(Entry.Key, ValueKey);
				});
			}
			else if(!PartitionData.IsEmpty())
			{
				MatchingVK = &PartitionData[0];
			}

			if (!MatchingVK)
			{
				MatchingVK = &PartitionData.Emplace_GetRef(ValueKey, AttributeSetPartitionEntry());
			}

			MatchingVK->Value.AddEntry(EntryKey, Weight);
		}

#if WITH_EDITOR
		// Check for empty entries
		const bool bHasEmptyEntries = Algo::AnyOf(PartitionData, [](const TPair<PCGMetadataValueKey, AttributeSetPartitionEntry>& PartitionEntry) { return PartitionEntry.Value.TotalWeight == 0; });
		if (bHasEmptyEntries)
		{
			PCGE_LOG_C(Warning, GraphAndLog, Context, FText::Format(LOCTEXT("EmptyCategory", "Some match entries on Attribute '{0}' in the Attribute Set do not have any associated valid weight."), FText::FromName(Attribute ? Attribute->Name : NAME_None)));
		}
#endif // WITH_EDITOR

		// Normalize weights
		for (TPair<PCGMetadataValueKey, AttributeSetPartitionEntry>& PartitionEntry : PartitionData)
		{
			AttributeSetPartitionEntry& Entry = PartitionEntry.Value;
			if (Entry.TotalWeight > 0)
			{
				for (double& Weight : Entry.CumulativeWeight)
				{
					Weight /= Entry.TotalWeight;
				}

				Entry.TotalWeight = 1.0;
			}
		}
		
		bIsValid = true;
		return true;
	}

	bool InitializeForData(FPCGContext* Context, const UPCGData* InMaxDistanceData, const FPCGAttributePropertyInputSelector* InMaxDistanceSelector)
	{
		if (Attribute && MaxDistanceMode == EPCGMatchMaxDistanceMode::AttributeMaxDistance)
		{
			ConstantThreshold.Reset();
			ConstantKey.Reset();

			if (InMaxDistanceData && InMaxDistanceSelector)
			{
				ConstantThreshold = PCGAttributeAccessorHelpers::CreateConstAccessor(InMaxDistanceData, *InMaxDistanceSelector);
				ConstantKey = PCGAttributeAccessorHelpers::CreateConstKeys(InMaxDistanceData, *InMaxDistanceSelector);

				if (!ConstantThreshold.IsValid() || !ConstantKey.IsValid())
				{
					PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("InvalidThresholdAttribute", "Attribute '{0}' used for max distance is invalid."), FText::FromName(InMaxDistanceSelector->GetAttributeName())));
					return false;
				}
			}
			else
			{
				return false;
			}
		}

		if (Attribute && ConstantThreshold.IsValid() && ConstantKey.IsValid())
		{
			if (!PCG::Private::IsBroadcastableOrConstructible(ConstantThreshold->GetUnderlyingType(), Attribute->GetTypeId()))
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("InvalidThresholdAttributeType", "Distance threshold type is not compatible with attribute '{0}'."), FText::FromName(Attribute->Name)));
				return false;
			}

			auto ValidateTypeCanComputeDistance = [](auto AttributeDummyValue) -> bool
			{
				using AttributeType = decltype(AttributeDummyValue);
				return PCG::Private::MetadataTraits<AttributeType>::CanComputeDistance;
			};

			if (!PCGMetadataAttribute::CallbackWithRightType(Attribute->GetTypeId(), ValidateTypeCanComputeDistance))
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("AttributeDoesNotSupportMaxDistance", "The selected attribute '{0}' does not support computing distances."), FText::FromName(Attribute->Name)));
				return false;
			}
		}

		return true;
	}

	bool IsValid() const { return bIsValid; }

	TArray<int32> GetMatchingPartitionDataIndices(const TUniquePtr<const IPCGAttributeAccessor>& InputAttribute, const TUniquePtr<const IPCGAttributeAccessorKeys>& InputKeys, int32 PointsNum) const
	{
		TArray<int32> MatchingPartitionDataIndices;

		if (InputAttribute && Attribute)
		{
			check(InputKeys.IsValid() && InputKeys->GetNum() == PointsNum);
			auto FindMatchingValueKeyIndex = [this, &InputAttribute, &InputKeys, &MatchingPartitionDataIndices](auto AttributeDummyValue) -> bool
			{
				using AttributeType = decltype(AttributeDummyValue);

				// Get threshold value if we need it.
				void* ThresholdValuesPtr = nullptr;
				int ConstKeyCount = ConstantKey.IsValid() ? FMath::Max(1, ConstantKey->GetNum()) : 0;

				if constexpr (PCG::Private::MetadataTraits<AttributeType>::CanComputeDistance)
				{
					if (ConstantThreshold.IsValid() && ConstantKey.IsValid())
					{
						using DistanceType = typename PCG::Private::MetadataTraits<AttributeType>::DistanceType;
						DistanceType* TypedThresholdValues = new DistanceType[ConstKeyCount];
						if (TypedThresholdValues)
						{
							TArrayView<DistanceType> ThresholdValues(TypedThresholdValues, ConstKeyCount);
							if (!ConstantThreshold->GetRange(ThresholdValues, 0, *ConstantKey, EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible))
							{
								delete[] TypedThresholdValues;
								return false;
							}
						}
						else
						{
							// Couldn't allocate memory, exit early
							return false;
						}

						ThresholdValuesPtr = TypedThresholdValues;
					}
				}

				// Get the values to match against from the attribute
				const FPCGMetadataAttribute<AttributeType>* TypedAttribute = static_cast<const FPCGMetadataAttribute<AttributeType>*>(Attribute);
				TArray<AttributeType> AttributeValues;
				AttributeValues.Reserve(PartitionData.Num());

				for (const TPair<PCGMetadataValueKey, AttributeSetPartitionEntry>& PartitionEntry : PartitionData)
				{
					AttributeValues.Add(TypedAttribute->GetValue(PartitionEntry.Key));
				}

				bool bApplyOk = PCGMetadataElementCommon::ApplyOnAccessor<AttributeType>(*InputKeys, *InputAttribute, [this, &AttributeValues, &MatchingPartitionDataIndices, ThresholdValuesPtr, ConstKeyCount](const AttributeType& InValue, int32 InIndex)
				{
					int32 MatchingPartitionDataIndex = INDEX_NONE;
					bool bFoundEqualMatch = false;
					for (int32 AttributeValueIndex = 0; AttributeValueIndex < AttributeValues.Num(); ++AttributeValueIndex)
					{
						if (PCG::Private::MetadataTraits<AttributeType>::Equal(InValue, AttributeValues[AttributeValueIndex]))
						{
							MatchingPartitionDataIndex = AttributeValueIndex;
							bFoundEqualMatch = true;
							break;
						}
						else if (bFindNearest)
						{
							if constexpr (PCG::Private::MetadataTraits<AttributeType>::CanFindNearest)
							{
								if (MatchingPartitionDataIndex == INDEX_NONE || PCG::Private::MetadataTraits<AttributeType>::IsCloserTo(AttributeValues[AttributeValueIndex], AttributeValues[MatchingPartitionDataIndex], InValue))
								{
									MatchingPartitionDataIndex = AttributeValueIndex;
								}
							}
						}
					}

					// Finally, if we haven't found an equal match, we should compare against the distance threshold.
					if (!bFoundEqualMatch && ThresholdValuesPtr && ConstKeyCount > 0)
					{
						if constexpr (PCG::Private::MetadataTraits<AttributeType>::CanComputeDistance)
						{
							using DistanceType = typename PCG::Private::MetadataTraits<AttributeType>::DistanceType;

							DistanceType Distance = PCG::Private::MetadataTraits<AttributeType>::Distance(AttributeValues[MatchingPartitionDataIndex], InValue);
							const DistanceType& ThresholdValue = static_cast<DistanceType*>(ThresholdValuesPtr)[InIndex % ConstKeyCount];

							if (Distance >= ThresholdValue)
							{
								MatchingPartitionDataIndex = INDEX_NONE;
							}
						}
						else
						{
							MatchingPartitionDataIndex = INDEX_NONE;
						}
					}

					MatchingPartitionDataIndices.Add(MatchingPartitionDataIndex);
				}, EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible);

				// delete threshold value ptr
				if constexpr (PCG::Private::MetadataTraits<AttributeType>::CanComputeDistance)
				{
					if (ThresholdValuesPtr)
					{
						using DistanceType = typename PCG::Private::MetadataTraits<AttributeType>::DistanceType;
						delete[] static_cast<DistanceType*>(ThresholdValuesPtr);
						ThresholdValuesPtr = nullptr;
					}
				}

				return bApplyOk;
			};

			if (!PCGMetadataAttribute::CallbackWithRightType(Attribute->GetTypeId(), FindMatchingValueKeyIndex))
			{
				// Attribute isn't able to retrieve & compare - reset values
				MatchingPartitionDataIndices.Init(INDEX_NONE, InputKeys->GetNum());
			}
		}
		else if (!PartitionData.IsEmpty())
		{
			// There is only one entry
			MatchingPartitionDataIndices.Init(0, PointsNum);
		}
		else
		{
			// There are no entries in the partition data
			MatchingPartitionDataIndices.Init(INDEX_NONE, PointsNum);
		}

		return MatchingPartitionDataIndices;
	}

	PCGMetadataEntryKey GetWeightedEntry(int32 PartitionDataIndex, double RandomWeightedPick) const
	{
		if (PartitionDataIndex == INDEX_NONE)
		{
			return PCGInvalidEntryKey;
		}

		// Second, resolve weight-based entries
		const AttributeSetPartitionEntry& PartitionDataEntry = PartitionData[PartitionDataIndex].Value;
		int RandomPick = INDEX_NONE;

		if (PartitionDataEntry.Keys.Num() == 1)
		{
			RandomPick = 0;
		}
		else if (PartitionDataEntry.Keys.Num() > 1)
		{
			RandomPick = 0;
			while (RandomPick < PartitionDataEntry.CumulativeWeight.Num() && PartitionDataEntry.CumulativeWeight[RandomPick] <= RandomWeightedPick)
			{
				++RandomPick;
			}

			// If weight is outside of the unit range, then we can still take the last entry
			RandomPick = FMath::Min(RandomPick, PartitionDataEntry.CumulativeWeight.Num() - 1);
		}

		if (RandomPick != INDEX_NONE)
		{
			return PartitionDataEntry.Keys[RandomPick];
		}
		else
		{
			// No entry in partition data, which is unexpected, but possible if all entries were <= 0.
			return PCGInvalidEntryKey;
		}
	}

private:
	const UPCGParamData* ParamData = nullptr;
	const FPCGMetadataAttributeBase* Attribute = nullptr;
	bool bIsValid = false;
	bool bFindNearest = false;
	EPCGMatchMaxDistanceMode MaxDistanceMode = EPCGMatchMaxDistanceMode::NoMaxDistance;

	TUniquePtr<const IPCGAttributeAccessor> ConstantThreshold;
	TUniquePtr<const IPCGAttributeAccessorKeys> ConstantKey;

	TArray<TPair<PCGMetadataValueKey, AttributeSetPartitionEntry>> PartitionData;
};

class FPCGMatchAndSetPartition : public FPCGDataPartitionBase<FPCGMatchAndSetPartition, PCGMetadataValueKey>
{
public:
	FPCGMatchAndSetPartition(FPCGContext* InContext, const UPCGMatchAndSetAttributesSettings* InSettings, const IPCGGraphExecutionSource* InExecutionSource, const UPCGParamData* InParamData) 
		: FPCGDataPartitionBase<FPCGMatchAndSetPartition, PCGMetadataValueKey>()
		, Context(InContext)
		, Settings(InSettings)
		, ParamData(InParamData)
		, ExecutionSource(InExecutionSource)
	{
	}

	bool Initialize(const TMap<const UPCGData*, const UPCGData*>& InDataToMaxDistanceMap)
	{
		if (!Settings)
		{
			return false;
		}

		DataToMaxDistanceMap = InDataToMaxDistanceMap;

		AttributeSetPartition.Initialize(Context,
			ParamData,
			Settings->bMatchAttributes,
			Settings->MatchAttribute.CopyAndFixLast(ParamData).GetName(),
			Settings->bUseWeightAttribute,
			Settings->WeightAttribute.CopyAndFixLast(ParamData).GetName(),
			Settings->bFindNearest,
			Settings->MaxDistanceMode,
			&Settings->MaxDistanceForNearestMatch);

		return AttributeSetPartition.IsValid();
	}

	bool InitializeForData(const UPCGData* InData, UPCGData* OutData)
	{
		if (!InData || !InData->ConstMetadata() || !OutData || !OutData->MutableMetadata())
		{
			return false;
		}

		if (Settings->bMatchAttributes)
		{
			const FPCGAttributePropertyInputSelector InputAttributeSource = Settings->InputAttribute.CopyAndFixLast(InData);
			InputAttributeAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InData, InputAttributeSource);
			InputAttributeKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InData, InputAttributeSource);

			if (!InputAttributeAccessor.IsValid() || !InputAttributeKeys.IsValid())
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("MissingAttribute", "Input data does not have the input attribute '{0}'."), InputAttributeSource.GetDisplayText()));
				return false;
			}
		}

		const UPCGData** FoundMaxDistanceData = DataToMaxDistanceMap.Find(InData);
		const FPCGAttributePropertyInputSelector MaxDistanceSelector = Settings->MaxDistanceInputAttribute.CopyAndFixLast(FoundMaxDistanceData ? *FoundMaxDistanceData : nullptr);

		if (!AttributeSetPartition.InitializeForData(Context, FoundMaxDistanceData ? *FoundMaxDistanceData : nullptr, FoundMaxDistanceData ? &MaxDistanceSelector : nullptr))
		{
			return false;
		}

		if (Settings->bUseInputWeightAttribute)
		{
			const FPCGAttributePropertyInputSelector InputWeightAttributeSource = Settings->InputWeightAttribute.CopyAndFixLast(InData);
			InputWeightAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InData, InputWeightAttributeSource);

			if (!InputAttributeKeys)
			{
				InputAttributeKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InData, InputWeightAttributeSource);
			}

			if (!InputWeightAccessor.IsValid() || !InputAttributeKeys.IsValid())
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("MissingWeightAttribute", "Input data does not have the input weight attribute '{0}'."), InputWeightAttributeSource.GetDisplayText()));
				return false;
			}

			if (!PCG::Private::IsOfTypes<float, double>(InputWeightAccessor->GetUnderlyingType()))
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("InvalidInputWeightAttributeType", "Input weight attribute '{0}' does not have the proper type (float or double)."), InputWeightAttributeSource.GetDisplayText()));
				return false;
			}
		}

		AttributesToSet.Reset();

		// Prepare set of attributes to copy, i.e. create the attributes if we need to, make a 1:1 pair with the ones from the
		// param data. Note that we don't need to copy over the matched attribute if any, nor the weight.
		TArray<FName> ParamAttributeNames;
		TArray<EPCGMetadataTypes> ParamAttributeTypes;
		ParamData->ConstMetadata()->GetAttributes(ParamAttributeNames, ParamAttributeTypes);

		const FPCGAttributePropertyInputSelector MatchAttributeSelector = Settings->MatchAttribute.CopyAndFixLast(ParamData);
		const FPCGAttributePropertyInputSelector WeightAttributeSelector = Settings->WeightAttribute.CopyAndFixLast(ParamData);

		// Validate that the selector are basic attributes
		if (!MatchAttributeSelector.IsBasicAttribute() || !WeightAttributeSelector.IsBasicAttribute())
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("NotBasicAttribute", "Match attribute {0} and/or weight attribute {1} are not basic attributes. Not yet supported, we will only use the basic attribute."),
				MatchAttributeSelector.GetDisplayText(), WeightAttributeSelector.GetDisplayText()), Context);
		}

		const FName MatchAttribute = MatchAttributeSelector.GetName();
		const FName WeightAttribute = WeightAttributeSelector.GetName();

		UPCGMetadata* OutMetadata = OutData->MutableMetadata();
		check(OutMetadata);

		for (int32 AttributeIndex = 0; AttributeIndex < ParamAttributeNames.Num(); ++AttributeIndex)
		{
			const FName AttributeName = ParamAttributeNames[AttributeIndex];

			if ((Settings->bMatchAttributes && AttributeName == MatchAttribute) ||
				(Settings->bUseWeightAttribute && AttributeName == WeightAttribute))
			{
				continue;
			}

			const FPCGMetadataAttributeBase* ParamAttribute = ParamData->ConstMetadata()->GetConstAttribute(AttributeName);

			FPCGMetadataAttributeBase* PointAttribute = OutMetadata->GetMutableAttribute(AttributeName);
			if (PointAttribute && PointAttribute->GetTypeId() != ParamAttribute->GetTypeId())
			{
				bool bShouldDeleteAttribute = true;

				// If we're in the unmatched case, we'll have to cast the attribute if possible.
				// Otherwise, log a warning, since we don't want data loss here.
				if (Settings->bMatchAttributes && Settings->bKeepUnmatched)
				{
					if (PCG::Private::IsBroadcastableOrConstructible(PointAttribute->GetTypeId(), ParamAttribute->GetTypeId()))
					{
						if (OutMetadata->ChangeAttributeType(AttributeName, ParamAttribute->GetAttributeDesc()))
						{
							bShouldDeleteAttribute = false;

							// ChangeAttributeType will delete the attribute PointAttribute points to so we need to requery it.
							PointAttribute = OutMetadata->GetMutableAttribute(AttributeName);

							if (Settings->bWarnOnAttributeCast)
							{
								PCGE_LOG_C(Warning, GraphAndLog, Context, FText::Format(LOCTEXT("OperationRequiredAttributeCast", "Attribute '{0}' was cast to the match data type because of the 'Keep Unmatched' option. This warning can be turned off in the settings."), FText::FromName(AttributeName)));
							}
						}
					}
					else
					{
						PCGE_LOG_C(Warning, GraphAndLog, Context, FText::Format(LOCTEXT("CannotReuseDifferentTypedAttribute", "Attribute '{0}' on output data has a different type vs. the match data, and will reset the values of 'Keep Unmatched' data."), FText::FromName(AttributeName)));
					}
				}
				
				if(bShouldDeleteAttribute)
				{
					OutMetadata->DeleteAttribute(AttributeName);
					PointAttribute = nullptr;
				}
			}

			// In most cases, the attribute didn't exist or we're not keeping the unmatched entries, so just create a new attribute.
			if (!PointAttribute)
			{
				PointAttribute = OutMetadata->CopyAttribute(ParamAttribute, AttributeName, /*bKeepParent=*/false, /*bCopyEntries=*/false, /*bCopyValues=*/false);
			}

			if (!PointAttribute) // Failed to create attribute
			{
				PCGE_LOG_C(Warning, GraphAndLog, Context, FText::Format(LOCTEXT("UnableToCreateAttribute", "Unable to create attribute '{0}' on output data."), FText::FromName(AttributeName)));
				return false;
			}

			AttributesToSet.Emplace(ParamAttribute, PointAttribute);
		}

		const int32 NumElements = GetNumElements(InData);
		PartitionDataIndices = AttributeSetPartition.GetMatchingPartitionDataIndices(InputAttributeAccessor, InputAttributeKeys, NumElements);

		if (PartitionDataIndices.Num() != NumElements)
		{
			return false;
		}

		const UPCGBasePointData* InPointData = Cast<const UPCGBasePointData>(InData);
		SetupWeights(NumElements, InPointData);

		return Weights.Num() == NumElements;
	}

	void SetupWeights(const int32 NumElements, const UPCGBasePointData* PointData)
	{
		check(!PointData || NumElements == PointData->GetNumPoints());

		Weights.Reset(NumElements);

		if (InputWeightAccessor.IsValid() && InputAttributeKeys.IsValid())
		{
			Weights.SetNumUninitialized(NumElements);
			InputWeightAccessor->GetRange<double>(Weights, 0, *InputAttributeKeys, EPCGAttributeAccessorFlags::AllowConstructible);
		}
		else if (PointData)
		{
			const TConstPCGValueRange<int32> SeedRange = PointData->GetConstSeedValueRange();
			// Generate a random value from the seed
			for (int32 Index = 0; Index < SeedRange.Num(); ++Index)
			{
				Weights.Add(PCGHelpers::GetRandomStreamFromSeed(SeedRange[Index], Settings, ExecutionSource).FRand());
			}
		}
		else
		{
			FRandomStream RandomStream = PCGHelpers::GetRandomStreamFromSeed(0, Settings, ExecutionSource);
			for (int32 i = 0; i < NumElements; ++i)
			{
				Weights.Add(RandomStream.FRand());
			}
		}
	}

	FPCGDataPartitionBase::Element* Select(int32 Index)
	{
		return nullptr;
	}

	int32 GetNumElements(const UPCGData* InData)
	{
		check(InData);

		if (const UPCGBasePointData* InPointData = Cast<const UPCGBasePointData>(InData))
		{
			return InPointData->GetNumPoints();
		}
		else if (const UPCGMetadata* Metadata = InData->ConstMetadata())
		{
			return Metadata->GetItemCountForChild();
		}
		else
		{
			return 0;
		}
	}

	void Finalize(const UPCGData* InData, UPCGData* OutData)
	{
		check(InData && OutData);
		if (const UPCGBasePointData* InPointData = Cast<const UPCGBasePointData>(InData))
		{
			FinalizeInternal(InPointData, static_cast<UPCGBasePointData*>(OutData), InPointData->GetNumPoints());
		}
		else if (const UPCGMetadata* Metadata = InData->ConstMetadata())
		{
			FinalizeInternal(InData, OutData, Metadata->GetItemCountForChild());
		}
		else
		{
			checkNoEntry();
		}
	}

	template <typename T>
	void FinalizeInternal(const T* InData, T* OutData, int32 NumElements)
	{
		const UPCGMetadata* InMetadata = InData->ConstMetadata();
		UPCGMetadata* OutMetadata = OutData->MutableMetadata();
		check(InMetadata && OutMetadata);

		int32 NumWritten = 0;

		FConstPCGPointValueRanges InRanges;
		FPCGPointValueRanges OutRanges;
				
		if constexpr (std::is_same_v<T, UPCGBasePointData>)
		{
			const UPCGBasePointData* InPointData = Cast<UPCGBasePointData>(InData);
			InRanges = FConstPCGPointValueRanges(InPointData);

			UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(OutData);
			OutRanges = FPCGPointValueRanges(OutPointData, /*bAllocate=*/false);
		}

		for (int32 Index = 0; Index < NumElements; ++Index)
		{
			int32 PartitionDataIndex = PartitionDataIndices[Index];
			PCGMetadataEntryKey AttributeSetKey = PCGInvalidEntryKey;

			if (PartitionDataIndex != INDEX_NONE)
			{
				AttributeSetKey = AttributeSetPartition.GetWeightedEntry(PartitionDataIndex, Weights[Index]);
			}

			if (Settings->bKeepUnmatched || AttributeSetKey != PCGInvalidEntryKey)
			{
				PCGMetadataEntryKey PreviousKey = PCGMetadataEntryKey(Index);
				PCGMetadataEntryKey NewEntry;

				if constexpr (std::is_same_v<T, UPCGBasePointData>)
				{
					OutRanges.SetFromValueRanges(NumWritten, InRanges, Index);

					PreviousKey = OutRanges.MetadataEntryRange[NumWritten];

					if (AttributeSetKey != PCGInvalidEntryKey)
					{
						OutRanges.MetadataEntryRange[NumWritten] = OutMetadata->AddEntry(OutRanges.MetadataEntryRange[NumWritten]);
					}

					NewEntry = OutRanges.MetadataEntryRange[NumWritten];

					++NumWritten;
				}
				else
				{
					NewEntry = OutMetadata->AddEntry();
					OutMetadata->SetAttributes(PreviousKey, InMetadata, NewEntry);
				}

				if (AttributeSetKey != PCGInvalidEntryKey)
				{
					// This is similar to UPCGMetadata::SetAttributes but for a subset of attributes
					for (const TPair<const FPCGMetadataAttributeBase*, FPCGMetadataAttributeBase*>& AttributePair : AttributesToSet)
					{
						const FPCGMetadataAttributeBase* ParamAttribute = AttributePair.Key;
						FPCGMetadataAttributeBase* PointAttribute = AttributePair.Value;

						PointAttribute->SetValue(NewEntry, ParamAttribute, AttributeSetKey);
					}
				}
			}
		}

		if constexpr (std::is_same_v<T, UPCGBasePointData>)
		{
			UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(OutData);
			OutPointData->SetNumPoints(NumWritten);
		}
	}

private:
	FPCGContext* Context = nullptr;
	const UPCGMatchAndSetAttributesSettings* Settings = nullptr;
	const UPCGParamData* ParamData = nullptr;
	FPCGAttributeSetPartition AttributeSetPartition;
	const IPCGGraphExecutionSource* ExecutionSource = nullptr;

	// Per point data iteration data
	TUniquePtr<const IPCGAttributeAccessor> InputAttributeAccessor;
	TUniquePtr<const IPCGAttributeAccessor> InputWeightAccessor;
	TUniquePtr<const IPCGAttributeAccessorKeys> InputAttributeKeys;
	TArray<TPair<const FPCGMetadataAttributeBase*, FPCGMetadataAttributeBase*>> AttributesToSet;
	TMap<const UPCGData*, const UPCGData*> DataToMaxDistanceMap;
	TArray<int32> PartitionDataIndices;
	TArray<double> Weights;
};

FPCGMatchAndSetAttributesExecutionState::~FPCGMatchAndSetAttributesExecutionState()
{
	if (Partition)
	{
		delete Partition;
	}
}

FPCGMatchAndSetAttributesIterationState::~FPCGMatchAndSetAttributesIterationState()
{
	if (bIsPartitionOwner)
	{
		delete Partition;
	}
}

bool FPCGMatchAndSetAttributesElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMatchAndSetAttributesElement::PrepareDataInternal);

	const UPCGMatchAndSetAttributesSettings* Settings = InContext->GetInputSettings<UPCGMatchAndSetAttributesSettings>();
	check(Settings);

	FPCGMatchAndSetAttributesElement::ContextType* TimeSlicedContext = static_cast<FPCGMatchAndSetAttributesElement::ContextType*>(InContext);
	check(TimeSlicedContext);

	TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;
	TArray<FPCGTaggedData> ParamDataInputs = InContext->InputData.GetInputsByPin(PCGMatchAndSetAttributesConstants::MatchDataLabel);
	TArray<FPCGTaggedData> MaxDistanceDataInputs = InContext->InputData.GetInputsByPin(PCGMatchAndSetAttributesConstants::MaxDistanceLabel);

	auto CreatePartition = [Settings, Context = TimeSlicedContext, &Inputs, &MaxDistanceDataInputs](const UPCGParamData* ParamData, const TArray<FPCGTaggedData>& PartitionInputs, int PartitionInputIndexIfSingle, FPCGMatchAndSetPartition*& OutPartition) -> EPCGTimeSliceInitResult
	{
		if (PartitionInputs.IsEmpty())
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Validate that the param data is valid
		if (!ParamData)
		{
			if (Settings->bWarnIfNoMatchData)
			{
				PCGE_LOG_C(Warning, GraphAndLog, Context, LOCTEXT("NoMatchData", "Must have exactly one Attribute Set to match against"));
			}

			return EPCGTimeSliceInitResult::NoOperation;
		}

		FPCGMatchAndSetPartition* Partition = new FPCGMatchAndSetPartition(Context, Settings, Context->ExecutionSource.Get(), ParamData);
		check(Partition);

		// If there are provided max distance entries, we should have either 1 or the same cardinality as the inputs
		TMap<const UPCGData*, const UPCGData*> InputToMaxDistanceMapping;
		if (Settings->MaxDistanceMode == EPCGMatchMaxDistanceMode::AttributeMaxDistance)
		{
			check(PartitionInputs.Num() == 1 || MaxDistanceDataInputs.Num() == PartitionInputs.Num());

			if (MaxDistanceDataInputs.Num() == 1)
			{
				for (int DataIndex = 0; DataIndex < PartitionInputs.Num(); ++DataIndex)
				{
					InputToMaxDistanceMapping.Add(PartitionInputs[DataIndex].Data, MaxDistanceDataInputs[0].Data);
				}
			}
			else if (PartitionInputIndexIfSingle == INDEX_NONE && PartitionInputs.Num() == MaxDistanceDataInputs.Num())
			{
				for (int DataIndex = 0; DataIndex < PartitionInputs.Num(); ++DataIndex)
				{
					InputToMaxDistanceMapping.Add(PartitionInputs[DataIndex].Data, MaxDistanceDataInputs[DataIndex].Data);
				}
			}
			else if(PartitionInputIndexIfSingle != INDEX_NONE && PartitionInputs.Num() == 1 && MaxDistanceDataInputs.IsValidIndex(PartitionInputIndexIfSingle))
			{
				InputToMaxDistanceMapping.Add(PartitionInputs[0].Data, MaxDistanceDataInputs[PartitionInputIndexIfSingle].Data);
			}
			else
			{
				delete Partition;
				return EPCGTimeSliceInitResult::AbortExecution;
			}
		}

		if (Partition->Initialize(InputToMaxDistanceMapping))
		{
			OutPartition = Partition;
			return EPCGTimeSliceInitResult::Success;
		}
		else
		{
			delete Partition;
			return EPCGTimeSliceInitResult::AbortExecution;
		}
	};

	EPCGTimeSliceInitResult InitResult = TimeSlicedContext->InitializePerExecutionState([Settings, &ParamDataInputs, &Inputs, &MaxDistanceDataInputs, &CreatePartition](FPCGMatchAndSetAttributesElement::ContextType* Context, FPCGMatchAndSetAttributesExecutionState& OutState) -> EPCGTimeSliceInitResult
	{
		if (ParamDataInputs.IsEmpty())
		{
			if (Settings->bWarnIfNoMatchData)
			{
				PCGE_LOG_C(Warning, GraphAndLog, Context, LOCTEXT("NoMatchData", "Must have exactly one Attribute Set to match against"));
			}

			return EPCGTimeSliceInitResult::NoOperation;
		}

		if(ParamDataInputs.Num() != 1 && ParamDataInputs.Num() != Inputs.Num())
		{
			PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, PCGMatchAndSetAttributesConstants::MatchDataLabel, Context);
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		if (Settings->MaxDistanceMode == EPCGMatchMaxDistanceMode::AttributeMaxDistance && MaxDistanceDataInputs.Num() != Inputs.Num() && MaxDistanceDataInputs.Num() != 1)
		{
			PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, PCGMatchAndSetAttributesConstants::MaxDistanceLabel, Context);
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		// We will create a partition here only in the N:1 case.
		if (ParamDataInputs.Num() == 1)
		{
			EPCGTimeSliceInitResult Result = CreatePartition(Cast<const UPCGParamData>(ParamDataInputs[0].Data), Inputs, -1, OutState.Partition);

			if(Result == EPCGTimeSliceInitResult::AbortExecution)
			{
				PCGE_LOG_C(Warning, GraphAndLog, Context, LOCTEXT("CouldNotInitializeExecutionState", "Could not initialize per-execution timeslice state data"));
			}

			return Result;
		}
		else
		{
			return EPCGTimeSliceInitResult::Success;
		}
	});

	if (InitResult == EPCGTimeSliceInitResult::AbortExecution)
	{
		// Implementation note: the previous code paths already emit necessary warnings
		return true;
	}
	else if (InitResult == EPCGTimeSliceInitResult::NoOperation)
	{
		Outputs = Inputs;
		return true;
	}

	TimeSlicedContext->InitializePerIterationStates(Inputs.Num(), [&Inputs, &ParamDataInputs, &Outputs, InContext, &CreatePartition](FPCGMatchAndSetAttributesIterationState& OutState, const FPCGMatchAndSetAttributesExecutionState& ExecutionState, const uint32 IterationIndex) -> EPCGTimeSliceInitResult
	{
		// Create output data
		FPCGTaggedData& Output = Outputs.Add_GetRef(Inputs[IterationIndex]);

		OutState.InData = Inputs[IterationIndex].Data;
		if (!OutState.InData || !OutState.InData->ConstMetadata())
		{
			PCGE_LOG_C(Error, GraphAndLog, InContext, FText::Format(LOCTEXT("InvalidInputDataType", "Input {0}: Input data must be of type Point or Param"), FText::AsNumber(IterationIndex)));
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Create or assign partition as needed
		if (ParamDataInputs.Num() == 1)
		{
			OutState.Partition = ExecutionState.Partition;
			OutState.bIsPartitionOwner = false;
		}
		else
		{
			EPCGTimeSliceInitResult Result = CreatePartition(Cast<const UPCGParamData>(ParamDataInputs[IterationIndex].Data), { Inputs[IterationIndex] }, IterationIndex, OutState.Partition);
			if (Result == EPCGTimeSliceInitResult::Success)
			{
				OutState.bIsPartitionOwner = true;
			}
			else
			{
				if (Result == EPCGTimeSliceInitResult::AbortExecution)
				{
					PCGE_LOG_C(Warning, GraphAndLog, InContext, LOCTEXT("CouldNotInitializeStateData", "Could not initialize timeslice state data"));
				}
				
				return Result;
			}
		}

		if (const UPCGBasePointData* InPointData = Cast<const UPCGBasePointData>(OutState.InData))
		{
			UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(InContext);

			FPCGInitializeFromDataParams InitializeFromDataParams(InPointData);
			InitializeFromDataParams.bInheritSpatialData = false;
			OutPointData->InitializeFromDataWithParams(InitializeFromDataParams);
			OutPointData->SetNumPoints(InPointData->GetNumPoints());
			OutPointData->AllocateProperties(InPointData->GetAllocatedProperties() | EPCGPointNativeProperties::MetadataEntry);
			OutPointData->CopyUnallocatedPropertiesFrom(InPointData);

			OutState.OutData = OutPointData;
		}
		else if (const UPCGParamData* InParamData = Cast<const UPCGParamData>(OutState.InData))
		{
			UPCGParamData* OutParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InContext);
			PCGMetadataHelpers::InitializeMetadataWithDataDomainCopyAndElementsNoCopy(InParamData, OutParamData);

			OutState.OutData = OutParamData;
		}
		else
		{
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		Output.Data = OutState.OutData;

		return EPCGTimeSliceInitResult::Success;
	});

	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		TimeSlicedContext->OutputData.TaggedData.Empty();
		PCGE_LOG_C(Warning, GraphAndLog, InContext, LOCTEXT("CouldNotInitializeStateData", "Could not initialize timeslice state data"));
		return true;
	}

	return true;
}

bool FPCGMatchAndSetAttributesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMatchAndSetAttributesElement::ExecuteInternal);

	FPCGMatchAndSetAttributesElement::ContextType* TimeSlicedContext = static_cast<FPCGMatchAndSetAttributesElement::ContextType*>(InContext);
	check(TimeSlicedContext);

	TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	// Prepare data failed, no need to execute.
	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		return true;
	}

	// The context will iterate over per-iteration states and execute the lambda until it returns true
	return ExecuteSlice(TimeSlicedContext, [&Inputs](FPCGMatchAndSetAttributesElement::ContextType* Context, const FPCGMatchAndSetAttributesExecutionState& ExecState, FPCGMatchAndSetAttributesIterationState& IterState, const uint32 IterationIndex) -> bool
	{
		const EPCGTimeSliceInitResult InitResult = Context->GetIterationStateResult(IterationIndex);

		// This iteration resulted in an early out for no sampling operation. Early out with a passthrough.
		if (InitResult == EPCGTimeSliceInitResult::NoOperation)
		{
			return true;
		}

		// It should be guaranteed to be a success at this point
		check(InitResult == EPCGTimeSliceInitResult::Success);

		// Run the execution until the time slice is finished. We actually don't care about the max index since we won't select anything.
		return IterState.Partition->SelectMultiple(*Context, IterState.InData, IterState.CurrentIndex, /*MaxIndex*/ 0, IterState.OutData);
	});
}

#undef LOCTEXT_NAMESPACE
