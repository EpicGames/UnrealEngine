// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMergeAttributes.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Metadata/PCGMetadata.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMergeAttributes)

#define LOCTEXT_NAMESPACE "PCGMergeAttributesSettings"

#if WITH_EDITOR
FText UPCGMergeAttributesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Merge Attributes");
}

FText UPCGMergeAttributesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Merges multiple attribute sets in a single attribute set with multiple entries and all the provided attributes");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGMergeAttributesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param, /*bAllowMultipleConnections=*/true, /*bAllowMultipleData=*/false);

	return PinProperties;
}

FPCGElementPtr UPCGMergeAttributesSettings::CreateElement() const
{
	return MakeShared<FPCGMergeAttributesElement>();
}

FName UPCGMergeAttributesSettings::GetDynamicInputPinsBaseLabel() const
{
	return PCGPinConstants::DefaultInputLabel;
}

TArray<FPCGPinProperties> UPCGMergeAttributesSettings::StaticInputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	// Do not explicitly mark the static input pin as required, as data on any input pin should prevent culling.
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Param);
	return PinProperties;
}

#if WITH_EDITOR
void UPCGMergeAttributesSettings::AddDefaultDynamicInputPin()
{
	FPCGPinProperties SecondaryPinProperties(
		FName(GetDynamicInputPinsBaseLabel().ToString() + FString::FromInt(DynamicInputPinProperties.Num() + 2)),
		EPCGDataType::Param,
		/*bInAllowMultipleConnections=*/false);
	AddDynamicInputPin(std::move(SecondaryPinProperties));
}
#endif // WITH_EDITOR

bool FPCGMergeAttributesElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMergeAttributesElement::Execute);
	check(Context);

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	FPCGTaggedData* MergedOutput = nullptr;
	UPCGParamData* MergedAttributeSet = nullptr;

	const UPCGMergeAttributesSettings* Settings = Context->GetInputSettings<UPCGMergeAttributesSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Sources;
	int NumSourcesOnFirstPin = -1;
	for (FName PinLabel : Settings->GetNodeDefinedPinLabels())
	{
		Sources.Append(Context->InputData.GetInputsByPin(PinLabel));

		if (NumSourcesOnFirstPin == -1)
		{
			NumSourcesOnFirstPin = Sources.Num();
		}
	}

	for (int SourceIndex = 0; SourceIndex < Sources.Num(); ++SourceIndex)
	{
		const FPCGTaggedData& Source = Sources[SourceIndex];
		const UPCGParamData* SourceData = Cast<const UPCGParamData>(Source.Data);

		if (!SourceData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("UnsupportedDataType", "Unsupported data type in merge attributes"));
			continue;
		}

		const UPCGMetadata* SourceMetadata = SourceData->Metadata;
		const int64 ParamItemCount = SourceData->Metadata->GetLocalItemCount();

		if (ParamItemCount <= 0)
		{
			continue;
		}

		if (!MergedOutput)
		{
			MergedOutput = &Outputs.Add_GetRef(Source);

			if (Settings->MergeTagsMode == EPCGMergeTagsMode::None ||
				(Settings->MergeTagsMode == EPCGMergeTagsMode::SelectTagsFromFirstData && SourceIndex != 0) ||
				(Settings->MergeTagsMode == EPCGMergeTagsMode::MergeTagsFromFirstPin && SourceIndex >= NumSourcesOnFirstPin))
			{
				MergedOutput->Tags.Reset();
			}

			continue;
		}
		
		// When we're merging the 2nd element, create the actual merged attribute set
		if (!MergedAttributeSet)
		{
			MergedAttributeSet = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
			check(MergedOutput);
			MergedAttributeSet->Metadata->InitializeAsCopy(FPCGMetadataInitializeParams(MergedOutput->Data->ConstMetadata()));
			MergedOutput->Data = MergedAttributeSet;
		}

		// For all entries starting from the second:
		// - Add missing attributes
		MergedAttributeSet->Metadata->AddAttributes(SourceMetadata);

		// - Merge entries
		TArray<PCGMetadataEntryKey, TInlineAllocator<256>> SourceEntryKeys;
		SourceEntryKeys.SetNumUninitialized(ParamItemCount);
		for (int64 LocalItemKey = 0; LocalItemKey < ParamItemCount; ++LocalItemKey)
		{
			SourceEntryKeys[LocalItemKey] = LocalItemKey;
		}

		TArray<PCGMetadataEntryKey, TInlineAllocator<256>> EntryKeys;
		EntryKeys.Init(PCGInvalidEntryKey, ParamItemCount);

		MergedAttributeSet->Metadata->SetAttributes(SourceEntryKeys, SourceMetadata, EntryKeys, Context);

		// - Merge tags too (to be in line with the Merge points node)
		if (Settings->MergeTagsMode == EPCGMergeTagsMode::MergeAllTags ||
			(Settings->MergeTagsMode == EPCGMergeTagsMode::MergeTagsFromFirstPin && SourceIndex < NumSourcesOnFirstPin))
		{
			MergedOutput->Tags.Append(Source.Tags);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE