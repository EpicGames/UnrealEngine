// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMergeElement.h"

#include "Data/PCGBasePointData.h"
#include "PCGContext.h"
#include "PCGPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMergeElement)

#define LOCTEXT_NAMESPACE "PCGMergeElement"

#if WITH_EDITOR
FText UPCGMergeSettings::GetNodeTooltipText() const
{
	return LOCTEXT("MergeNodeTooltip", "Merges multiple data sources into a single data output.");
}
#endif

TArray<FPCGPinProperties> UPCGMergeSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point, /*bAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false);

	return PinProperties;
}

FPCGElementPtr UPCGMergeSettings::CreateElement() const
{
	return MakeShared<FPCGMergeElement>();
}

FName UPCGMergeSettings::GetDynamicInputPinsBaseLabel() const
{
	return PCGPinConstants::DefaultInputLabel;
}

TArray<FPCGPinProperties> UPCGMergeSettings::StaticInputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	// Do not explicitly mark the static input pin as required, as data on any input pin should prevent culling.
	Properties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	return Properties;
}

#if WITH_EDITOR
void UPCGMergeSettings::AddDefaultDynamicInputPin()
{
	FPCGPinProperties SecondaryPinProperties(
		FName(GetDynamicInputPinsBaseLabel().ToString() + FString::FromInt(DynamicInputPinProperties.Num() + 2)),
		EPCGDataType::Point,
		/*bInAllowMultipleConnections=*/false);
	AddDynamicInputPin(std::move(SecondaryPinProperties));
}
#endif // WITH_EDITOR

bool FPCGMergeElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMergeElement::Execute);
	check(Context);

	const UPCGMergeSettings* Settings = Context->GetInputSettings<UPCGMergeSettings>();
	check(Settings);
	const bool bMergeMetadata = Settings->bMergeMetadata;

	TArray<FPCGTaggedData> Sources;
	int NumSourcesOnFirstPin = -1;
	for (FName PinLabel : Settings->GetNodeDefinedPinLabels())
	{
		Sources.Append(Context->InputData.GetInputsByPin(PinLabel));

		if (NumSourcesOnFirstPin == -1)
		{
			NumSourcesOnFirstPin = Sources.Num();
		}
	}

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UPCGBasePointData* TargetPointData = nullptr;
	FPCGTaggedData* TargetTaggedData = nullptr;

	int32 TotalPointCount = 0;
	TArray<const UPCGBasePointData*> SourcePointDatas;

	// Prepare data & metadata
	// Done in two passes for futureproofing - expecting changes in the metadata attribute creation vs. usage in points
	for(int SourceIndex = 0; SourceIndex < Sources.Num(); ++SourceIndex)
	{
		const FPCGTaggedData& Source = Sources[SourceIndex];
		const UPCGBasePointData* SourcePointData = Cast<const UPCGBasePointData>(Source.Data);

		if (!SourcePointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("UnsupportedDataType", "Unsupported data type in merge"));
			continue;
		}

		if (SourcePointData->GetNumPoints() == 0)
		{
			continue;
		}

		SourcePointDatas.Add(SourcePointData);
		TotalPointCount += SourcePointData->GetNumPoints();

		// First data that's valid - will be the original output so we don't do a copy unless we actually need it
		if (!TargetTaggedData)
		{
			TargetTaggedData = &(Outputs.Add_GetRef(Source));

			if (Settings->MergeTagsMode == EPCGMergeTagsMode::None ||
				(Settings->MergeTagsMode == EPCGMergeTagsMode::SelectTagsFromFirstData && SourceIndex != 0) ||
				(Settings->MergeTagsMode == EPCGMergeTagsMode::MergeTagsFromFirstPin && SourceIndex >= NumSourcesOnFirstPin))
			{
				TargetTaggedData->Tags.Reset();
			}
		}
		else if (!TargetPointData)
		{
			// Second valid data - we'll create the actual merged data at this point
			check(TargetTaggedData);

			TargetPointData = FPCGContext::NewPointData_AnyThread(Context);
			FPCGInitializeFromDataParams InitializeFromDataParams(CastChecked<const UPCGBasePointData>(TargetTaggedData->Data));
			InitializeFromDataParams.bInheritSpatialData = false;
			InitializeFromDataParams.bInheritMetadata = bMergeMetadata;
			
			TargetPointData->InitializeFromDataWithParams(InitializeFromDataParams);
			TargetTaggedData->Data = TargetPointData;
		}

		if (TargetPointData)
		{
			if (bMergeMetadata)
			{
				TargetPointData->Metadata->AddAttributes(SourcePointData->Metadata);
			}
			
			check(TargetTaggedData);
			// By definition once we're here, it's the second data.
			if (Settings->MergeTagsMode == EPCGMergeTagsMode::MergeAllTags ||
				(Settings->MergeTagsMode == EPCGMergeTagsMode::MergeTagsFromFirstPin && SourceIndex < NumSourcesOnFirstPin))
			{
				TargetTaggedData->Tags.Append(Source.Tags);
			}
		}
	}

	// If there was no valid input or only one, there's nothing to do here
	if (!TargetPointData)
	{
		return true;
	}

	TargetPointData->SetNumPoints(TotalPointCount);
	TargetPointData->AllocateProperties(UPCGBasePointData::GetPropertiesToAllocateFromPointData(SourcePointDatas));

	int32 PointOffset = 0;

	for(int32 SourceDataIndex = 0; SourceDataIndex < SourcePointDatas.Num(); ++SourceDataIndex)
	{
		const UPCGBasePointData* SourcePointData = SourcePointDatas[SourceDataIndex];
		const int32 NumSourcePoints = SourcePointData->GetNumPoints();
		ensure(NumSourcePoints > 0);

		SourcePointData->CopyPointsTo(TargetPointData, 0, PointOffset, NumSourcePoints);

		if ((!bMergeMetadata || SourceDataIndex != 0))
		{
			TPCGValueRange<int64> TargetMetadataEntryRange = TargetPointData->GetMetadataEntryValueRange();
			for (int32 TargetIndex = PointOffset; TargetIndex < PointOffset + NumSourcePoints; ++TargetIndex)
			{
				TargetMetadataEntryRange[TargetIndex] = PCGInvalidEntryKey;
			}

			if (bMergeMetadata && TargetPointData->Metadata && SourcePointData->Metadata && SourcePointData->Metadata->GetAttributeCount() > 0)
			{
				// Extract the metadata entry keys from the in & out points
				TArray<PCGMetadataEntryKey, TInlineAllocator<256>> SourceKeys;
				SourceKeys.SetNumUninitialized(NumSourcePoints);

				const TConstPCGValueRange<int64> SourceMetadataEntryRange = SourcePointData->GetConstMetadataEntryValueRange();
				for (int32 SourceIndex = 0; SourceIndex < SourceKeys.Num(); ++SourceIndex)
				{
					SourceKeys[SourceIndex] = SourceMetadataEntryRange[SourceIndex];
				}
				
				TArray<PCGMetadataEntryKey, TInlineAllocator<256>> TargetKeys;
				TargetKeys.SetNumUninitialized(NumSourcePoints);
								
				for (int32 TargetIndex = 0; TargetIndex < TargetKeys.Num(); ++TargetIndex)
				{
					TargetKeys[TargetIndex] = PCGInvalidEntryKey;
				}

				TargetPointData->Metadata->SetAttributes(SourceKeys, SourcePointData->Metadata, TargetKeys, Context);

				// Write back
				for (int32 TargetIndex = 0; TargetIndex < TargetKeys.Num(); ++TargetIndex)
				{
					TargetMetadataEntryRange[PointOffset + TargetIndex] = TargetKeys[TargetIndex];
				}
			}
		}

		PointOffset += NumSourcePoints;
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
