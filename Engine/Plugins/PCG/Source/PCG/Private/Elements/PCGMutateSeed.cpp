// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMutateSeed.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMutateSeed)

#define LOCTEXT_NAMESPACE "PCGMutateSeedSettings"

namespace PCGMutateSeedConstants
{
	// TODO: Evaluate this value for optimization
	// An evolving best guess for the most optimized number of points to operate per thread per slice
	static constexpr int32 PointsPerChunk = 98304;
}

FPCGElementPtr UPCGMutateSeedSettings::CreateElement() const
{
	return MakeShared<FPCGMutateSeedElement>();
}

EPCGPointNativeProperties FPCGMutateSeedElement::GetPropertiesToAllocate(FPCGContext* Context) const
{
	return EPCGPointNativeProperties::Seed;
}

bool FPCGMutateSeedElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMutateSeedElement::Execute);

	check(Context);
	ContextType* MutateSeedContext = static_cast<ContextType*>(Context);

	return ExecutePointOperation(MutateSeedContext, [Seed = Context->GetSeed()](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
	{
		TConstPCGValueRange<FTransform> TransformRange = OutputData->GetConstTransformValueRange();
		TPCGValueRange<int32> SeedRange = OutputData->GetSeedValueRange();

		for (int32 Index = StartIndex; Index < (StartIndex + Count); ++Index)
		{
			SeedRange[Index] = PCGHelpers::ComputeSeed(PCGHelpers::ComputeSeedFromPosition(TransformRange[Index].GetLocation()), Seed, SeedRange[Index]);
		}

		return true;
	}, PCGMutateSeedConstants::PointsPerChunk);
}

#undef LOCTEXT_NAMESPACE
