// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGNormalToDensity.h"

#include "PCGContext.h"
#include "Compute/PCGKernelHelpers.h"
#include "Compute/Elements/PCGNormalToDensityKernel.h"
#include "Data/PCGPointData.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "Kismet/KismetMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGNormalToDensity)

#if WITH_EDITOR
void UPCGNormalToDensitySettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);
	PCGKernelHelpers::CreateKernel<UPCGNormalToDensityKernel>(InOutContext, CreateParams, OutKernels, OutEdges);
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGNormalToDensitySettings::CreateElement() const
{
	return MakeShared<FPCGNormalToDensityElement>();
}

EPCGPointNativeProperties FPCGNormalToDensityElement::GetPropertiesToAllocate(FPCGContext* Context) const
{
	return EPCGPointNativeProperties::Density;
}

bool FPCGNormalToDensityElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGNormalToDensityElement::Execute);

	ContextType* Context = static_cast<ContextType*>(InContext);

	const UPCGNormalToDensitySettings* Settings = Context->GetInputSettings<UPCGNormalToDensitySettings>();
	check(Settings);

	const FVector Normal = Settings->Normal.GetSafeNormal();
	const double Offset = Settings->Offset;
	const double Strength = Settings->Strength;
	const PCGNormalToDensityMode DensityMode = Settings->DensityMode;

	const double InvStrength = 1.0/FMath::Max(0.0001, Strength);

	const auto CalcValue = [Normal, Offset, InvStrength](const FTransform& InPointTransform)
	{
		const FVector Up = InPointTransform.GetUnitAxis(EAxis::Z);
		return FMath::Pow(FMath::Clamp(Up.Dot(Normal) + Offset, 0.0, 1.0), InvStrength);
	};

	const auto SetDensityLoop = [CalcValue](UPCGBasePointData* OutputData, int32 StartIndex, int32 Count, TFunctionRef<void(float, float&)> SetDensityFunc)
	{
		TConstPCGValueRange<FTransform> TransformRange = OutputData->GetConstTransformValueRange();
		TPCGValueRange<float> DensityRange = OutputData->GetDensityValueRange();

		for (int32 Index = StartIndex; Index < (StartIndex + Count); ++Index)
		{
			SetDensityFunc(CalcValue(TransformRange[Index]), DensityRange[Index]);
		}

		return true;
	};

	switch (DensityMode)
	{
		case PCGNormalToDensityMode::Set:
			return ExecutePointOperation(Context, [SetDensityLoop](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetDensityLoop(OutputData, StartIndex, Count, [](float InNormalDensity, float& OutDensity)
				{
					OutDensity = InNormalDensity;
				});
			});
			
		case PCGNormalToDensityMode::Minimum:
			return ExecutePointOperation(Context, [SetDensityLoop](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetDensityLoop(OutputData, StartIndex, Count, [](float InNormalDensity, float& OutDensity)
				{
					OutDensity = FMath::Min(OutDensity, InNormalDensity);
				});
			});

		case PCGNormalToDensityMode::Maximum:
			return ExecutePointOperation(Context, [SetDensityLoop](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetDensityLoop(OutputData, StartIndex, Count, [](float InNormalDensity, float& OutDensity)
				{
					OutDensity = FMath::Max(OutDensity, InNormalDensity);
				});
			});

		case PCGNormalToDensityMode::Add:
			return ExecutePointOperation(Context, [SetDensityLoop](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetDensityLoop(OutputData, StartIndex, Count, [](float InNormalDensity, float& OutDensity)
				{
					OutDensity += InNormalDensity;
				});
			});

		case PCGNormalToDensityMode::Subtract:
			return ExecutePointOperation(Context, [SetDensityLoop](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetDensityLoop(OutputData, StartIndex, Count, [](float InNormalDensity, float& OutDensity)
				{
					OutDensity -= InNormalDensity;
				});
			});

		case PCGNormalToDensityMode::Multiply:
			return ExecutePointOperation(Context, [SetDensityLoop](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetDensityLoop(OutputData, StartIndex, Count, [](float InNormalDensity, float& OutDensity)
				{
					OutDensity *= InNormalDensity;
				});
			});

		case PCGNormalToDensityMode::Divide:
			return ExecutePointOperation(Context, [SetDensityLoop](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetDensityLoop(OutputData, StartIndex, Count, [](float InNormalDensity, float& OutDensity)
				{
					OutDensity = UKismetMathLibrary::SafeDivide(OutDensity, InNormalDensity);
				});
			});
	}

	return true;
}
