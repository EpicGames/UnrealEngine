// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGNumberOfElements.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadata.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGNumberOfElements)

#define LOCTEXT_NAMESPACE "PCGNumberOfElementsSettings"

TArray<FPCGPinProperties> UPCGNumberOfElementsSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

FPCGElementPtr UPCGNumberOfElementsSettings::CreateElement() const
{
	return MakeShared<FPCGNumberOfElementsElement>();
}

bool FPCGNumberOfElementsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGNumberOfElementsBaseElement::Execute);

	check(Context);

	const UPCGNumberOfElementsSettings* Settings = Context->GetInputSettings<UPCGNumberOfElementsSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	UPCGParamData* OutputParamData = nullptr;
	FPCGMetadataAttribute<int32>* NewAttribute = nullptr;

	for (int32 i = 0; i < Inputs.Num(); ++i)
	{
		const UPCGData* InputData = Inputs[i].Data;

		if (!InputData)
		{
			PCGLog::InputOutput::LogInvalidInputDataError(Context);
			continue;
		}

		if (!OutputParamData)
		{
			OutputParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
			NewAttribute = OutputParamData->Metadata->CreateAttribute<int32>(Settings->OutputAttributeName, 0, /*bAllowInterpolation=*/true, /*bOverrideParent=*/false);

			if (!NewAttribute)
			{
				PCGLog::Metadata::LogFailToCreateAttributeError<int32>(Settings->OutputAttributeName, Context);
				return true;
			}
		}

		check(OutputParamData && NewAttribute);
		// Implementation note: since we might have multiple entries, we must set the actual value to a new entry
		NewAttribute->SetValue(OutputParamData->Metadata->AddEntry(), PCGHelpers::GetNumberOfElements(InputData));
	}

	if (OutputParamData)
	{
		FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
		Output.Data = OutputParamData;
		Output.Pin = PCGPinConstants::DefaultOutputLabel;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
