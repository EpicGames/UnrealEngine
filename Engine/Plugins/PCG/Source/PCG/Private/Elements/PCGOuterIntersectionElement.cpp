// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGOuterIntersectionElement.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "PCGSettingsWithDynamicInputs.h"
#include "Data/PCGSpatialData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGOuterIntersectionElement)

#define LOCTEXT_NAMESPACE "PCGOuterIntersectionElement"

TArray<FPCGPinProperties> UPCGOuterIntersectionSettings::StaticInputPinProperties() const
{
	TArray<FPCGPinProperties> StaticPinProperties;
	FPCGPinProperties& PrimaryPinProperties = StaticPinProperties.Emplace_GetRef(PCGIntersectionConstants::PrimaryLabel, EPCGDataType::Spatial);
	PrimaryPinProperties.SetRequiredPin();
	FPCGPinProperties& SecondaryPinProperties = StaticPinProperties.Emplace_GetRef(FName(PCGIntersectionConstants::SecondaryLabel.ToString() + FString(TEXT("1"))), EPCGDataType::Spatial);

#if WITH_EDITOR
	PrimaryPinProperties.Tooltip = LOCTEXT("PrimaryPinTooltip", "Each input on the primary pin will be used to calculate the output of the intersection operation. Assuming input data is provided, an output result will be generated for each input on this pin.");
	SecondaryPinProperties.Tooltip = PCGIntersectionConstants::SecondaryTooltip;
#endif // WITH_EDITOR

	return StaticPinProperties;
}

#if WITH_EDITOR
void UPCGOuterIntersectionSettings::AddDefaultDynamicInputPin()
{
	// Skip "Source 0", and also "Source 1" is always active, thus the label will be count+2
	FPCGPinProperties SecondaryPinProperties(FName(GetDynamicInputPinsBaseLabel().ToString() + FString::FromInt(DynamicInputPinProperties.Num() + 2)), EPCGDataType::Spatial);
	
#if WITH_EDITOR
	SecondaryPinProperties.Tooltip = PCGIntersectionConstants::SecondaryTooltip;
#endif // WITH_EDITOR

	AddDynamicInputPin(std::move(SecondaryPinProperties));
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGOuterIntersectionSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& OutputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spatial);

#if WITH_EDITOR
	//Source spatial data from which to construct the intersection. Incoming edges providing no data will be ignored
	OutputPinProperty.Tooltip = LOCTEXT("OutputPinTooltip", "The intersection created from all the source input data. One output will be generated for every input on the primary input pin.");
#endif // WITH_EDITOR
	
	return PinProperties;
}

#if WITH_EDITOR
FText UPCGOuterIntersectionSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltipText", "For each given source input on the primary pin, spatial data will be generated as the result of sequentially intersecting with the other source inputs (implicitly unioned), should an intersection exist. Additional pins maybe dynamically added and for each of these, all of the inputs into the same pin will be 'unioned' together automatically. Source pins receiving no or empty data will logically return an empty output, unless the 'Ignore Empty Secondary Input' flag has been set to 'true'.\nSee also: Inner Intersection Node, Union Node");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGOuterIntersectionSettings::CreateElement() const
{
	return MakeShared<FPCGOuterIntersectionElement>();
}

FName UPCGOuterIntersectionSettings::GetDynamicInputPinsBaseLabel() const
{
	return PCGIntersectionConstants::SecondaryLabel;
}

bool FPCGOuterIntersectionElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGOuterIntersectionElement::Execute);

	const UPCGOuterIntersectionSettings* Settings = Context->GetInputSettings<UPCGOuterIntersectionSettings>();
	check(Settings);
	
	const TArray<TObjectPtr<UPCGPin>>& InputPins = Context->Node->GetInputPins();
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	// Early out for empty primary spatial inputs
	if (Context->InputData.GetSpatialInputCountByPin(PCGIntersectionConstants::PrimaryLabel) == 0)
	{
		return true;
	}

	// If the user toggle is off, check for spatial pins that have no spatial data, for early out
	if (!Settings->bIgnorePinsWithNoInput)
	{
		for (const UPCGPin* InputPin : InputPins)
		{
			check(InputPin);
			// Only checking secondary source pins expecting spatial data
			if (InputPin->Properties.Label != PCGIntersectionConstants::PrimaryLabel
				&& !InputPin->Properties.IsDatalessPin()
				&& !!(InputPin->Properties.AllowedTypes & EPCGDataType::Spatial)
				&& Context->InputData.GetSpatialInputCountByPin(InputPin->Properties.Label) == 0)
			{
				UE_LOGF(LogPCG, Verbose, "Intersection resulted in empty output due to no spatial data on a dynamic input pin.");
				return true;
			}
		}
	}

	// Create unions for all the secondary source inputs with multiple connections/data
	TArray<const UPCGSpatialData*> SecondarySourceUnionArray;
	for (const UPCGPin* InputPin : InputPins)
	{
		const bool bIsPrimaryPin = InputPin->Properties.Label == PCGIntersectionConstants::PrimaryLabel;
		
		// Only checking pins expecting spatial data, not params, settings, etc
		if (bIsPrimaryPin || InputPin->Properties.IsDatalessPin() || !(InputPin->Properties.AllowedTypes & EPCGDataType::Spatial))
		{
			continue;
		}

		bool bDummy;
		if (const UPCGSpatialData* UnionData = Context->InputData.GetSpatialUnionOfInputsByPin(Context, InputPin->Properties.Label, bDummy))
		{
			SecondarySourceUnionArray.Push(UnionData);
		}
	}

	if (SecondarySourceUnionArray.IsEmpty())
	{
		if (Settings->bIgnorePinsWithNoInput)
		{
			Outputs = Context->InputData.GetInputsByPin(PCGIntersectionConstants::PrimaryLabel);
		}
		
		return true;
	}
	
	// Iterate through primary inputs, and create intersection data from the unioned
	for (const FPCGTaggedData& Input : Context->InputData.GetInputsByPin(PCGIntersectionConstants::PrimaryLabel))
	{
		const UPCGSpatialData* PrimarySpatialData = Cast<UPCGSpatialData>(Input.Data);

		// If this data isn't spatial, passthrough to the output
		if (!PrimarySpatialData)
		{
			Outputs.Add(Input);
			continue;
		}
		
		const UPCGSpatialData* IntersectionData = PrimarySpatialData;
		for (const UPCGSpatialData* SecondarySourceUnionData : SecondarySourceUnionArray)
		{
			UPCGIntersectionData* TempIntersectionData = IntersectionData->IntersectWith(Context, SecondarySourceUnionData);
			// Propagate settings
			TempIntersectionData->DensityFunction = Settings->DensityFunction;
			TempIntersectionData->bKeepZeroDensityPoints = Settings->bKeepZeroDensityPoints;

			IntersectionData = TempIntersectionData;
		}

		FPCGTaggedData& IntersectionTaggedData = Outputs.Emplace_GetRef(Input);
		IntersectionTaggedData.Data = IntersectionData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
