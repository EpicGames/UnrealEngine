// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Metadata/PCGParseString.h"

#include "PCGContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGParseString)

#define LOCTEXT_NAMESPACE "PCGParseStringElement"

#if WITH_EDITOR
FName UPCGParseStringSettings::GetDefaultNodeName() const
{
	return FName(TEXT("ParseString"));
}

FText UPCGParseStringSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Parse String");
}

FText UPCGParseStringSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Parse string passed as attribute into a compatible PCG type.\n"
		"Failed to parse will result to the value be set to Identity(for quat and transform) or 0 (for the rest).\n"
		"Int/Int64: Support base 10 or base 16 number (ie. 0xff). If int overflows, result is platform dependant. \n"
		"Float/Double: Support normal representation or exposant representation (ie. 13.5e-2f)\n"
		"Bool: [True/Yes/On] [False/No/Off], or 0 -> false and the rest -> true"
		"Vectors/Quat: X=<value> Y=<value> Z=<value> W=<value> OR <value>,<value>,<value>,<value>\n"
		"Rotator: P=<value> Y=<value> R=<value> or  Pitch=<value> Yaw=<value> Roll=<value> or <value,<value>,<value>\n"
		"Transform: <value>,<value>,<value>|<value>,<value>,<value>|<value>,<value>,<value> (Translation|Rotation|Scale)");
}
#endif // WITH_EDITOR

FString UPCGParseStringSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	const bool bIsTypeOverridden = IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGParseStringSettings, TargetType));
	const bool bSelectorOverridden = IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGParseStringSettings, InputSource)) || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGParseStringSettings, OutputTarget));

	if (bIsTypeOverridden)
	{
		return FString();
	}
	else
	{
		if (bSelectorOverridden)
		{
			return FText::Format(LOCTEXT("CastType", "To {0}"), PCG::Private::GetTypeNameText((uint16)TargetType)).ToString();
		}
	}
#endif

	return FText::Format(LOCTEXT("FullInfo", "{0} -> {1} as {2}"), InputSource.GetDisplayText(), OutputTarget.GetDisplayText(), PCG::Private::GetTypeNameText((uint16)TargetType)).ToString();
}

FPCGElementPtr UPCGParseStringSettings::CreateElement() const
{
	return MakeShared<FPCGParseStringElement>();
}

FPCGAttributePropertyInputSelector UPCGParseStringSettings::GetInputSource(uint32 Index) const
{
	ensure(Index == 0);
	return InputSource;
}

bool UPCGParseStringSettings::IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const
{
	ensure(InputIndex == 0);
	return PCG::Private::IsOfTypes<FName, FString>(TypeId);
}

TArray<uint16> UPCGParseStringSettings::GetAllOutputTypes() const
{
	return { static_cast<uint16>(TargetType) };
}

bool FPCGParseStringElement::DoOperation(PCGMetadataOps::FOperationData& OperationData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGParseStringElement::Execute);

	const UPCGParseStringSettings* Settings = CastChecked<UPCGParseStringSettings>(OperationData.Settings);

	return PCGMetadataAttribute::CallbackWithRightType(static_cast<uint16>(Settings->TargetType), [this, &OperationData]<typename T>(T Dummy) -> bool
	{
		return DoUnaryOp<FString>(OperationData, [](const FString& InValue) -> T
			{
				bool bSuccess;
				return PCG::Private::MetadataTraits<T>::ParseFromString(InValue, bSuccess);
			});
	});
}


#undef LOCTEXT_NAMESPACE
