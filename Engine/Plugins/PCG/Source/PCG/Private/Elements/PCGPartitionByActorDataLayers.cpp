// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGPartitionByActorDataLayers.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGDataLayerHelpers.h"
#include "Utils/PCGLogErrors.h"

#if WITH_EDITOR
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadataAccessor.h"
#include "Metadata/PCGMetadataPartitionCommon.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Algo/RemoveIf.h"
#include "GameFramework/Actor.h"
#include "Serialization/ArchiveCrc32.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPartitionByActorDataLayers)

#define LOCTEXT_NAMESPACE "PCGPartitionByActorDataLayers"

namespace PCGPartitionByActorDataLayers
{
	namespace Constants
	{
		const FName DataLayerPartitionsLabel = TEXT("DataLayerPartitions");
	}

#if WITH_EDITOR
	TArray<const UDataLayerAsset*> GetDataLayersFromActor(AActor* InActor, const TArray<TSoftObjectPtr<UDataLayerAsset>>& InIncludedDataLayerAssets, const TArray<TSoftObjectPtr<UDataLayerAsset>>& InExcludedDataLayerAssets)
	{
		TArray<const UDataLayerAsset*> DataLayerAssets = PCGDataLayerHelpers::GetDatalayerAssetsForActor(InActor);

		int32 RemoveIndex = Algo::RemoveIf(DataLayerAssets, [&InIncludedDataLayerAssets, &InExcludedDataLayerAssets](const UDataLayerAsset* InDataLayerAsset)
		{
			return (!InIncludedDataLayerAssets.IsEmpty() && !InIncludedDataLayerAssets.Contains(InDataLayerAsset))
				|| InExcludedDataLayerAssets.Contains(InDataLayerAsset);
		});

		DataLayerAssets.SetNum(RemoveIndex);
		DataLayerAssets.Sort([](const UDataLayerAsset& A, const UDataLayerAsset& B)
		{
			return A.GetPathName() < B.GetPathName();
		});

		return DataLayerAssets;
	}

	int32 GetDataLayersCrc(const TArray<const UDataLayerAsset*>& InDataLayerAssets)
	{
		FArchiveCrc32 Ar;
		for (const UDataLayerAsset* DataLayerAsset : InDataLayerAssets)
		{
			UDataLayerAsset* NonConstDataLayerAsset = const_cast<UDataLayerAsset*>(DataLayerAsset);
			Ar << NonConstDataLayerAsset;
		}
		return Ar.GetCrc();
	}
#endif
}

UPCGPartitionByActorDataLayersSettings::UPCGPartitionByActorDataLayersSettings()
{
	ActorReferenceAttribute.SetAttributeName(PCGPointDataConstants::ActorReferenceAttribute);
	IncludedDataLayers.Attribute.SetAttributeName(PCGDataLayerHelpers::Constants::IncludedDataLayersAttribute);
	ExcludedDataLayers.Attribute.SetAttributeName(PCGDataLayerHelpers::Constants::ExcludedDataLayersAttribute);
	DataLayerReferenceAttribute.SetAttributeName(PCGDataLayerHelpers::Constants::DataLayerReferenceAttribute);
}

TArray<FPCGPinProperties> UPCGPartitionByActorDataLayersSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	InputPin.SetRequiredPin();

	if (IncludedDataLayers.bAsInput)
	{
		FPCGPinProperties& IncludedDataLayersPin = PinProperties.Emplace_GetRef(PCGDataLayerHelpers::Constants::IncludedDataLayersAttribute, EPCGDataType::PointOrParam);
		IncludedDataLayersPin.SetRequiredPin();
	}

	if (ExcludedDataLayers.bAsInput)
	{
		FPCGPinProperties& ExcludedDataLayersPin = PinProperties.Emplace_GetRef(PCGDataLayerHelpers::Constants::ExcludedDataLayersAttribute, EPCGDataType::Param);
		ExcludedDataLayersPin.SetRequiredPin();
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGPartitionByActorDataLayersSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
	PinProperties.Emplace(PCGPartitionByActorDataLayers::Constants::DataLayerPartitionsLabel, EPCGDataType::Param);

	return PinProperties;
}

FPCGElementPtr UPCGPartitionByActorDataLayersSettings::CreateElement() const
{
	return MakeShared<FPCGPartitionByActorDataLayersElement>();
}

#if WITH_EDITOR
EPCGChangeType UPCGPartitionByActorDataLayersSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FPCGDataLayerReferenceSelector, bAsInput))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif

bool FPCGPartitionByActorDataLayersElement::ExecuteInternal(FPCGContext* Context) const
{
	const UPCGPartitionByActorDataLayersSettings* Settings = Context->GetInputSettings<UPCGPartitionByActorDataLayersSettings>();
	check(Settings);
	
	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

#if WITH_EDITOR
	TArray<TSoftObjectPtr<UDataLayerAsset>> IncludedDataLayerAssets = PCGDataLayerHelpers::GetDataLayerAssetsFromInput(Context, PCGDataLayerHelpers::Constants::IncludedDataLayersAttribute, Settings->IncludedDataLayers);
	TArray<TSoftObjectPtr<UDataLayerAsset>> ExcludedDataLayerAssets = PCGDataLayerHelpers::GetDataLayerAssetsFromInput(Context, PCGDataLayerHelpers::Constants::ExcludedDataLayersAttribute, Settings->ExcludedDataLayers);

	TMap<int32, UPCGBasePointData*> DataLayerCrcToOutputData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		// @todo_pcg: support ParamData
		const UPCGBasePointData* InData = Cast<UPCGBasePointData>(Input.Data);
		if (!InData)
		{
			continue;
		}

		// @todo_pcg: partitioning by actor reference will cause performance issues if we have too many actors, we should have a way in the partition api to specify a lambda to compare values where
		// we could compare the actor DataLayers Crcs directly
		FPCGAttributePropertySelector PartitionAttributeSource = Settings->ActorReferenceAttribute.CopyAndFixLast(InData);
		TArray<UPCGData*> PartitionDataArray = PCGMetadataPartitionCommon::AttributePartition(InData, PartitionAttributeSource, Context);

		for (UPCGData* PartitionData : PartitionDataArray)
		{
			UPCGBasePointData* PartitionPointData = CastChecked<UPCGBasePointData>(PartitionData);

			const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(PartitionData, PartitionAttributeSource);
			const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PartitionData, PartitionAttributeSource);

			if (!Accessor || !Keys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(PartitionAttributeSource, Context);
				continue;
			}

			if (Keys->GetNum() < 1)
			{
				continue;
			}

			FSoftObjectPath ActorSoftPath;
			if (!Accessor->Get<FSoftObjectPath>(ActorSoftPath, *Keys.Get(), EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::Metadata::LogFailToGetAttributeError(PartitionAttributeSource, Context);
				continue;
			}

			AActor* Actor = Cast<AActor>(ActorSoftPath.ResolveObject());
			if (!Actor)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("UnresolvedActor", "Could not resolve actor path '{0}'."), FText::FromString(ActorSoftPath.ToString())), Context);
				continue;
			}

			// Compute DL Crc
			TArray<const UDataLayerAsset*> ActorDataLayers = PCGPartitionByActorDataLayers::GetDataLayersFromActor(Actor, IncludedDataLayerAssets, ExcludedDataLayerAssets);
			int32 DataLayerCrc = PCGPartitionByActorDataLayers::GetDataLayersCrc(ActorDataLayers);

			UPCGBasePointData* PartitionOutputData = nullptr;
			if (UPCGBasePointData** FoundOutputData = DataLayerCrcToOutputData.Find(DataLayerCrc))
			{
				PartitionOutputData = *FoundOutputData;
			}
			else
			{
				PartitionOutputData = FPCGContext::NewPointData_AnyThread(Context);

				FPCGInitializeFromDataParams InitializeFromDataParams(PartitionPointData);
				InitializeFromDataParams.bInheritSpatialData = false;
				PartitionOutputData->InitializeFromDataWithParams(InitializeFromDataParams);

				DataLayerCrcToOutputData.Add(DataLayerCrc, PartitionOutputData);

				FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
				Output.Pin = PCGPinConstants::DefaultOutputLabel;
				Output.Data = PartitionOutputData;

				// Add PCGParam Output with DataLayers
				FPCGTaggedData& DataLayerSetOutput = Context->OutputData.TaggedData.Emplace_GetRef();
				UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
				DataLayerSetOutput.Pin = PCGPartitionByActorDataLayers::Constants::DataLayerPartitionsLabel;
				DataLayerSetOutput.Data = ParamData;

				FPCGMetadataAttribute<FSoftObjectPath>* DataLayersAttribute = ParamData->MutableMetadata()->CreateAttribute(Settings->DataLayerReferenceAttribute.GetName(), FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
				for (const UDataLayerAsset* ActorDataLayer : ActorDataLayers)
				{
					PCGMetadataEntryKey Entry = ParamData->MutableMetadata()->AddEntry();
					DataLayersAttribute->SetValue(Entry, FSoftObjectPath(ActorDataLayer));
				}
			}

			check(PartitionOutputData);
			const int32 NumPoints = PartitionOutputData->GetNumPoints();
			PartitionOutputData->SetNumPoints(NumPoints + PartitionPointData->GetNumPoints());
			PartitionOutputData->AllocateProperties(PartitionPointData->GetAllocatedProperties());
			PartitionPointData->CopyPointsTo(PartitionOutputData, 0, NumPoints, PartitionPointData->GetNumPoints());
		}
	}
#else
	for (const FPCGTaggedData& Input : Inputs)
	{
		Outputs.Add(Input);
		
		FPCGTaggedData& DataLayerSetOutput = Context->OutputData.TaggedData.Emplace_GetRef();
		UPCGParamData* ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		DataLayerSetOutput.Pin = PCGPartitionByActorDataLayers::Constants::DataLayerPartitionsLabel;
		DataLayerSetOutput.Data = ParamData;
	}

	PCGLog::LogErrorOnGraph(LOCTEXT("PartitionByActorDataLayersUnsupported", "Partition by Actor Data Layers is unsupported at runtime"), Context);
#endif // WITH_EDITOR

	return true;
}

#undef LOCTEXT_NAMESPACE

