// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGPathfindingElement.h"

#include "PCGContext.h"
#include "PCGComponent.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"
#include "SpatialAlgo/PCGAStar.h"

#include "Components/SplineComponent.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPathfindingElement)

#define LOCTEXT_NAMESPACE "PCGPathfindingElement"

namespace PCGPathfindingElement
{
	namespace Constants
	{
		const FName StartLocationsInputPinLabel = TEXT("Start");
		const FName GoalLocationsInputPinLabel = TEXT("Goal");
		static const TCHAR* CompletePathTag = TEXT("CompletePath");
		static const TCHAR* PartialPathTag = TEXT("PartialPath");
	}

	namespace Helpers
	{
		TArray<FSplinePoint> ConvertPathToSplinePoints(const PCGSpatialAlgo::AStar::FSearchState& SearchState, const TArrayView<PCGSpatialAlgo::AStar::FPointDescription> Path, const EPCGPathfindingSplineMode SplineMode)
		{
			using namespace PCGSpatialAlgo::AStar;
			ESplinePointType::Type SplineCurveMode = ESplinePointType::Type::Constant;
			switch (SplineMode)
			{
				case EPCGPathfindingSplineMode::Curve:
					SplineCurveMode = ESplinePointType::Curve;
					break;
				case EPCGPathfindingSplineMode::Linear:
					SplineCurveMode = ESplinePointType::Linear;
				default:
					break;
			}

			TArray<FSplinePoint> SplinePoints;
			SplinePoints.Reserve(Path.Num());

			int Index = 0;
			Algo::Transform(Path, SplinePoints, [&SearchState, &Index, SplineCurveMode](const FPointDescription& Point)
			{
				return FSplinePoint(
						Index++, // Spline points must be indexed in ascending order
						Point.GetLocation(),
						FVector::ZeroVector,
						FVector::ZeroVector,
						FRotator::ZeroRotator,
						FVector::OneVector,
						SplineCurveMode);
			});

			return SplinePoints;
		}
	}
} // namespace PCGPathfindingElement

UPCGPathfindingSettings::UPCGPathfindingSettings()
{
	StartLocationAttribute.SetPointProperty(EPCGPointProperties::Position);
	GoalLocationAttribute.SetPointProperty(EPCGPointProperties::Position);

	// In most cases, we're not going to be interested in checking for occlusion by the landscape itself, as we'll be pathfinding on the landscape.
	PathTraceParams.SelectLandscapeHits = EPCGWorldQuerySelectLandscapeHits::Exclude;
}

TArray<FPCGPinProperties> UPCGPathfindingSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point).SetRequiredPin();

	if (bStartLocationsAsInput)
	{
		Properties.Emplace_GetRef(PCGPathfindingElement::Constants::StartLocationsInputPinLabel, EPCGDataType::PointOrParam, /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false).SetRequiredPin();
	}

	if (bGoalLocationsAsInput)
	{
		Properties.Emplace_GetRef(PCGPathfindingElement::Constants::GoalLocationsInputPinLabel, EPCGDataType::PointOrParam, /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false).SetRequiredPin();
	}

	if (bUsePathTraces)
	{
		PathTraceParams.AddFilterPinIfNeeded(Properties);
	}

	return Properties;
}

TArray<FPCGPinProperties> UPCGPathfindingSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	if (bOutputAsSpline)
	{
		Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spline);
	}
	else
	{
		Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
	}

	return Properties;
}

FPCGElementPtr UPCGPathfindingSettings::CreateElement() const
{
	return MakeShared<FPCGPathfindingElement>();
}

#if WITH_EDITOR
EPCGChangeType UPCGPathfindingSettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	EPCGChangeType LocalChangeType = EPCGChangeType::Cosmetic;

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGPathfindingSettings, bOutputAsSpline)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UPCGPathfindingSettings, bStartLocationsAsInput)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UPCGPathfindingSettings, bGoalLocationsAsInput)
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGPathfindingSettings, PathTraceParams) &&
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(FPCGWorldCommonQueryParams, ActorFilterFromInput)))
	{
		LocalChangeType |= EPCGChangeType::Structural;
	}

	return Super::GetChangeTypeForProperty(PropertyChangedEvent) | LocalChangeType;
}
#endif // WITH_EDITOR

bool FPCGPathfindingElement::IsCacheable(const UPCGSettings* InSettings) const
{
	const UPCGPathfindingSettings* Settings = Cast<const UPCGPathfindingSettings>(InSettings);
	return !Settings || !Settings->bUsePathTraces;
}

bool FPCGPathfindingElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPathfindingElement::PrepareData);

	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context);

	const UPCGPathfindingSettings* Settings = Context->GetInputSettings<UPCGPathfindingSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> PointInputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	const TArray<FPCGTaggedData> FilterActorInputData = Context->InputData.GetInputsByPin(PCGWorldRayHitConstants::FilterActorPinLabel);

	if (PointInputs.IsEmpty())
	{
		return true;
	}

	const EPCGTimeSliceInitResult ExecResult = Context->InitializePerExecutionState([&Context = InContext, &PointInputs, &FilterActorInputData, Settings](ContextType* SlicedContext, ExecStateType& OutState)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::PrepareData::InitializePerExecutionState);

		if (PointInputs.IsEmpty())
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}
		
		if (Settings->bUsePathTraces && FilterActorInputData.Num() > 1 && FilterActorInputData.Num() != PointInputs.Num())
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidFilterActorInputCount", "Filter Actor input data count must be 1 or match the In pin data count."), SlicedContext);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		OutState.Settings = PCGSpatialAlgo::AStar::FSearchSettings
		{
			.SearchDistance = Settings->SearchDistance,
			.HeuristicWeight = Settings->HeuristicWeight,
			.bAcceptPartialPath = Settings->bAcceptPartialPath,
			.bCopyOriginatingPoints = Settings->bCopyOriginatingPoints
		};

		auto GetLocationsFromInput = [SlicedContext, Settings](FName Pin) -> const UPCGData*
		{
			const TArray<FPCGTaggedData> LocationsInputData = SlicedContext->InputData.GetInputsByPin(Pin);
			if (LocationsInputData.IsEmpty())
			{
				return nullptr;
			}

			if (LocationsInputData.Num() > 1)
			{
				PCGLog::InputOutput::LogFirstInputOnlyWarning(Pin, SlicedContext);
			}

			return LocationsInputData[0].Data;
		};

		bool bShouldCopyAllAttributesFromStartAndGoal = Settings->bCopyOriginatingPoints || Settings->CostFunctionMode==EPCGPathfindingCostFunctionMode::CostMultiplier || Settings->CostFunctionMode==EPCGPathfindingCostFunctionMode::FitnessScore;
		auto PopulateArray = [&Context, SlicedContext, bShouldCopyAllAttributesFromStartAndGoal](const UPCGData* Data, const FPCGAttributePropertyInputSelector& InSelector) -> FPCGPointInputRange
		{
			const FPCGAttributePropertyInputSelector Selector = InSelector.CopyAndFixLast(Data);

			bool AccessorIsPosition = (Selector.GetSelection() == EPCGAttributePropertySelection::Property && Selector.GetPointProperty() == EPCGPointProperties::Position);
			const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Data);

			// Simple case, reuse the original PointData.
			if (PointData && AccessorIsPosition)
			{
				return FPCGPointInputRange{PointData, 0, PointData->GetNumPoints()};
			}

			const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Data, Selector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Data, Selector);
		
			if (!Accessor || !Keys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, SlicedContext);
				return {};
			}

			UPCGBasePointData* OutPointData = nullptr; 
			if (PointData)
			{
				if (!PointData->IsEmpty())
				{
					OutPointData = FPCGContext::NewPointData_AnyThread(Context);
					UPCGBasePointData::SetPoints(PointData, OutPointData, /* InDataIndices=*/{}, /* bCopyAll=*/true);
				}
			}
			else if (Keys->GetNum() > 0)
			{
				OutPointData = FPCGContext::NewPointData_AnyThread(Context);
				OutPointData->SetNumPoints(Keys->GetNum());
			}

			if (OutPointData == nullptr)
			{
				return {};
			}
				
			SlicedContext->TrackObject(OutPointData);
			if (bShouldCopyAllAttributesFromStartAndGoal && PointData)
			{
				// We will want to copy properties from Start/Goal into final result, so it should follow.
				OutPointData->AllocateProperties(PointData->GetAllocatedProperties()|EPCGPointNativeProperties::Transform|EPCGPointNativeProperties::Seed);
				FPCGInitializeFromDataParams InitParams(PointData);
				InitParams.bInheritSpatialData = false;
				OutPointData->InitializeFromDataWithParams(InitParams);
			}
			else
			{
				OutPointData->AllocateProperties(EPCGPointNativeProperties::Transform|EPCGPointNativeProperties::Seed);
			}

			TArray<FVector> Locations;
			Locations.SetNumUninitialized(Keys->GetNum());
			Accessor->GetRange(TArrayView<FVector>(Locations), 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);

			check(Locations.Num() == OutPointData->GetNumPoints());
			TPCGValueRange<FTransform> TransformRange = OutPointData->GetTransformValueRange();
			TPCGValueRange<int32> SeedRange = OutPointData->GetSeedValueRange();
			for (int i = 0; i < OutPointData->GetNumPoints(); ++i)
			{
				TransformRange[i] = FTransform{Locations[i]};
				SeedRange[i] = PCGHelpers::ComputeSeedFromPosition(Locations[i]);
			}
			
			return FPCGPointInputRange{OutPointData, 0, OutPointData->GetNumPoints()};
		};

		if (Settings->bStartLocationsAsInput)
		{
			if (const UPCGData* Data = GetLocationsFromInput(PCGPathfindingElement::Constants::StartLocationsInputPinLabel))
			{
				OutState.StartPoints = PopulateArray(Data, Settings->StartLocationAttribute);
			}
		}
		else
		{
			UPCGBasePointData* StartPoints = FPCGContext::NewPointData_AnyThread(Context);
			SlicedContext->TrackObject(StartPoints);
			StartPoints->SetNumPoints(1);
			StartPoints->SetTransform(FTransform(Settings->Start));
			OutState.StartPoints = FPCGPointInputRange{StartPoints, 0, StartPoints->GetNumPoints()};
			
		}

		if (!OutState.StartPoints.PointData || OutState.StartPoints.RangeSize == 0)
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (Settings->bGoalLocationsAsInput)
		{
			if (const UPCGData* Data = GetLocationsFromInput(PCGPathfindingElement::Constants::GoalLocationsInputPinLabel))
			{
				OutState.GoalPoints = PopulateArray(Data, Settings->GoalLocationAttribute);
			}
		}
		else
		{
			UPCGBasePointData* GoalPoints = FPCGContext::NewPointData_AnyThread(Context);
			SlicedContext->TrackObject(GoalPoints);
			GoalPoints->SetNumPoints(1);
			GoalPoints->SetTransform(FTransform(Settings->Goal));
			OutState.GoalPoints = FPCGPointInputRange{GoalPoints, 0, GoalPoints->GetNumPoints()};
		}

		if (!OutState.GoalPoints.PointData || OutState.GoalPoints.RangeSize == 0)
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		OutState.IterationCount = Settings->GoalMappingMode == EPCGPathfindingGoalMappingMode::EachStartToEachGoal
			? OutState.StartPoints.RangeSize * OutState.GoalPoints.RangeSize
			: OutState.StartPoints.RangeSize;

		// Validate that if in N:N -> Start:Goal mode, the inputs are the correct cardinality. All other cases were validated in PrepareData and are acceptable.
		if (Settings->GoalMappingMode == EPCGPathfindingGoalMappingMode::EachStartToPairwiseGoal && OutState.StartPoints.RangeSize != OutState.GoalPoints.RangeSize)
		{
			PCGLog::LogErrorOnGraph(FText::Format(
				LOCTEXT("EachStartToPairwiseGoalInvalidMapping", "For 'Each Start To Pairwise Goal' pathfinding mode, there must be a one-to-one mapping between start locations and goal locations. The current input was {0}:{1} Start->Goal locations."), OutState.StartPoints.RangeSize, OutState.GoalPoints.RangeSize), SlicedContext);
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		return EPCGTimeSliceInitResult::Success;
	});

	if (ExecResult != EPCGTimeSliceInitResult::Success)
	{
		return true;
	}

	Context->InitializePerIterationStates(PointInputs.Num(), [&PointInputs, &FilterActorInputData, Settings, Context](IterStateType& OutState, const ExecStateType& ExecutionState, const uint32 IterationIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGClusterElement::PrepareData::InitializePerIterationStates);

		const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(PointInputs[IterationIndex].Data);
		if (!PointData || PointData->IsEmpty())
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		OutState.SearchState.OriginatingPointData = PointData;

		// Build cost attribute accessor if required
		TSharedPtr<const IPCGAttributeAccessor> CostAccessor = nullptr;

		if (Settings->CostFunctionMode != EPCGPathfindingCostFunctionMode::Distance)
		{
			//@todo_pcg The accessor will only work for PointData (It will not work on goal or start point data), we need to fix this without affecting the perf (i.e.: constructing an accessor every time we check the cost)
			const FPCGAttributePropertyInputSelector Selector = Settings->CostAttribute.CopyAndFixLast(PointData);
			CostAccessor = MakeShareable(PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, Selector).Release());

			FPCGPointTransform::ConstValueRange Keys = PointData->GetConstTransformValueRange();

			if (!CostAccessor)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, Context);
			}
			else if (!PCG::Private::IsBroadcastableOrConstructible(CostAccessor->GetUnderlyingType(), PCG::Private::MetadataTypes<double>::Id))
			{
				PCGLog::Metadata::LogFailToGetAttributeError<double>(Selector, CostAccessor.Get(), Context);
				CostAccessor = nullptr;
			}
		}

		TFunction<bool(const FVector&, const FVector&)> LineTraceTest = [](const FVector&, const FVector&) -> bool { return true; };

		if (Settings->bUsePathTraces)
		{
			const UPCGData* FilterActorsData = !FilterActorInputData.IsEmpty() ? FilterActorInputData[IterationIndex % FilterActorInputData.Num()].Data : nullptr;

			// Accept only point or param data
			if (FilterActorsData && !FilterActorsData->IsA<UPCGBasePointData>() && !FilterActorsData->IsA<UPCGParamData>())
			{
				PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::PointOrParam, PCGWorldRayHitConstants::FilterActorPinLabel, Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			// --- Gather filtered actors ---
			TSet<FObjectKey> CachedFilterObjects;
			if (!Settings->PathTraceParams.ExtractLoadedObjectFiltersIfNeeded(FilterActorsData, CachedFilterObjects, Context))
			{
				return EPCGTimeSliceInitResult::NoOperation;
			}
			
			if (UWorld* World = Context->ExecutionSource.Get() ? Context->ExecutionSource->GetExecutionState().GetWorld() : nullptr)
			{
				FPCGWorldRaycastQueryParams PathTraceParams = Settings->PathTraceParams;
				PathTraceParams.Initialize();

				TWeakInterfacePtr<IPCGGraphExecutionSource> OriginatingSource = Context->ExecutionSource.Get();
				FCollisionObjectQueryParams ObjectQueryParams(PathTraceParams.CollisionChannel);
				FCollisionQueryParams Params;
				Params.bTraceComplex = PathTraceParams.bTraceComplex;

				LineTraceTest = [World, ObjectQueryParams = MoveTemp(ObjectQueryParams), Params = MoveTemp(Params), OriginatingSource, PathTraceParams = MoveTemp(PathTraceParams), CachedFilterObjects = MoveTemp(CachedFilterObjects)](const FVector& StartPosition, const FVector& EndPosition) -> bool
				{
					TArray<FHitResult> OutHits;
					if (World->LineTraceMultiByObjectType(OutHits, StartPosition, EndPosition, ObjectQueryParams, Params))
					{
						TOptional<FHitResult> HitResult = PCGWorldQueryHelpers::FilterRayHitResults(PathTraceParams, OriginatingSource, OutHits, CachedFilterObjects);
						return !HitResult.IsSet();
					}
					else
					{
						return true;
					}
				};
			}
		}

		if (CostAccessor)
		{
			if (Settings->CostFunctionMode == EPCGPathfindingCostFunctionMode::FitnessScore)
			{
				const double MaxFitnessPenaltyFactor = FMath::Max(Settings->MaximumFitnessPenaltyFactor, 1.0);

				OutState.SearchState.CostFunction = [&SearchState = OutState.SearchState, FitnessAccessor = CostAccessor, MaxFitnessPenaltyFactor, PathTraceTest = MoveTemp(LineTraceTest)](const PCGSpatialAlgo::AStar::FPointDescription& PreviousNodePoint, const double DistanceToCurrentSquared, const PCGSpatialAlgo::AStar::FPointDescription& CurrentNodePoint)
				{
					if (!PathTraceTest(PreviousNodePoint.GetLocation(), CurrentNodePoint.GetLocation()))
					{
						return std::numeric_limits<double>::max();
					}

					double FitnessScore = 1.0;
					FPCGAttributeAccessorKeysPointIndices Key(CurrentNodePoint.SrcPointRange.PointData);

					FitnessAccessor->Get(FitnessScore, CurrentNodePoint.SrcPointRange.RangeStartIndex + CurrentNodePoint.PointIndexInRange, Key, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
					FitnessScore = FMath::Clamp(FitnessScore, 0.0, 1.0);

					return (1.0 - FitnessScore) * MaxFitnessPenaltyFactor * FMath::Sqrt(DistanceToCurrentSquared);
				};
			}
			else if (Settings->CostFunctionMode == EPCGPathfindingCostFunctionMode::CostMultiplier)
			{
				OutState.SearchState.CostFunction = [&SearchState = OutState.SearchState, MultiplierAccessor = CostAccessor, PathTraceTest = MoveTemp(LineTraceTest)](const PCGSpatialAlgo::AStar::FPointDescription& PreviousNodePoint, const double DistanceToCurrentSquared, const PCGSpatialAlgo::AStar::FPointDescription& CurrentNodePoint)
				{
					if (!PathTraceTest(PreviousNodePoint.GetLocation(), CurrentNodePoint.GetLocation()))
					{
						return std::numeric_limits<double>::max();
					}

					double Multiplier = 1.0;
					FPCGAttributeAccessorKeysPointIndices Key(CurrentNodePoint.SrcPointRange.PointData);

					MultiplierAccessor->Get(Multiplier, CurrentNodePoint.SrcPointRange.RangeStartIndex + CurrentNodePoint.PointIndexInRange, Key, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
					Multiplier = FMath::Max(Multiplier, 1.0);

					return Multiplier * FMath::Sqrt(DistanceToCurrentSquared);
				};
			}
			else
			{
				checkNoEntry();
			}
		}
		else if (Settings->bUsePathTraces)
		{
			// Use distance but with line trace
			OutState.SearchState.CostFunction = [&SearchState = OutState.SearchState, PathTraceTest = MoveTemp(LineTraceTest)](const PCGSpatialAlgo::AStar::FPointDescription& PreviousNodePoint, const double DistanceToCurrentSquared, const PCGSpatialAlgo::AStar::FPointDescription& CurrentNodePoint)
			{
				if (!PathTraceTest(PreviousNodePoint.GetLocation(), CurrentNodePoint.GetLocation()))
				{
					return std::numeric_limits<double>::max();
				}
				else
				{
					return PCGSpatialAlgo::AStar::Cost::CalculateCost_EuclideanDistance(PreviousNodePoint, DistanceToCurrentSquared, CurrentNodePoint);
				}
			};
		}

		return EPCGTimeSliceInitResult::Success;
	});

	return true;
}

bool FPCGPathfindingElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPathfindingElement::Execute);

	ContextType* TimeSlicedContext = static_cast<ContextType*>(InContext);
	check(TimeSlicedContext);

	if (!TimeSlicedContext->DataIsPreparedForExecution() || TimeSlicedContext->GetExecutionStateResult() == EPCGTimeSliceInitResult::NoOperation)
	{
		return true;
	}

	const UPCGPathfindingSettings* Settings = TimeSlicedContext->GetInputSettings<UPCGPathfindingSettings>();
	check(Settings);

	return ExecuteSlice(TimeSlicedContext, [Settings](ContextType* Context, const ExecStateType& ExecutionState, IterStateType& IterationState, const uint32 IterIndex)
	{
		if (Context->GetIterationStateResult(IterIndex) == EPCGTimeSliceInitResult::NoOperation)
		{
			return true;
		}

		const bool bCartesianMapping = Settings->GoalMappingMode == EPCGPathfindingGoalMappingMode::EachStartToEachGoal;

		while (IterationState.PathIterationIndex < ExecutionState.IterationCount)
		{
			using PCGSpatialAlgo::AStar::ESearchResult;
			TArray<PCGSpatialAlgo::AStar::FPointDescription> FinalPath;

			// Initialize the next iteration.
			if (IterationState.PathIterationIndex != IterationState.LastPathIterationIndex)
			{
				// Per start, per goal. GoalPoints was validated as not empty (0) in PrepareData.
				const int32 StartPointIndex = bCartesianMapping
					? IterationState.PathIterationIndex / ExecutionState.GoalPoints.RangeSize
					: IterationState.PathIterationIndex;
				IterationState.StartPoints = FPCGPointInputRange{ExecutionState.StartPoints.PointData, ExecutionState.StartPoints.RangeStartIndex + StartPointIndex, 1};


				if (Settings->GoalMappingMode == EPCGPathfindingGoalMappingMode::EachStartToNearestGoal)
				{
					IterationState.GoalPoints = ExecutionState.GoalPoints;
				}
				else // Other mapping modes will do a one-to-one comparison of a single start and single goal.
				{
					const int32 GoalPointIndex = bCartesianMapping
						? IterationState.PathIterationIndex % ExecutionState.GoalPoints.RangeSize
						: IterationState.PathIterationIndex;
					IterationState.GoalPoints = FPCGPointInputRange{ExecutionState.GoalPoints.PointData, ExecutionState.GoalPoints.RangeStartIndex + GoalPointIndex, 1};
				}

				PCGSpatialAlgo::AStar::Initialize(IterationState.StartPoints,IterationState.GoalPoints, IterationState.SearchState);
				IterationState.LastPathIterationIndex = IterationState.PathIterationIndex;
			}


			ESearchResult SearchResult;
			do
			{
				SearchResult = ExecuteSearchIteration(ExecutionState.Settings, IterationState.SearchState, FinalPath);

				if (SearchResult == ESearchResult::Processing && Context->ShouldStop())
				{
					return false;
				}
			}
			while (SearchResult == ESearchResult::Processing);

			if (SearchResult == ESearchResult::Invalid)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("InvalidSearch", "The search could not be completed. Check for invalid search settings or input."));
				return true;
			}

			// No path was found and partial paths were not enabled.
			if (FinalPath.IsEmpty())
			{
				check(!ExecutionState.Settings.bAcceptPartialPath && SearchResult != ESearchResult::Partial);
				return true;
			}

			FPCGTaggedData& OutputData = Context->OutputData.TaggedData.Emplace_GetRef();

			// Output the path as either a spline or points.
			if (Settings->bOutputAsSpline)
			{
				const TArray<FSplinePoint> SplinePoints = PCGPathfindingElement::Helpers::ConvertPathToSplinePoints(IterationState.SearchState, FinalPath, Settings->SplineMode);
				UPCGSplineData* SplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
				SplineData->Initialize(SplinePoints, /*bInClosedLoop=*/false, FTransform::Identity);
				OutputData.Data = SplineData;
			}
			else
			{
				UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(Context);
				if (ExecutionState.Settings.bCopyOriginatingPoints)
				{
					FPCGInitializeFromDataParams InitParams(IterationState.SearchState.OriginatingPointData);
					InitParams.bInheritSpatialData = false;
					OutputPointData->InitializeFromDataWithParams(InitParams);

					InitParams.Source = IterationState.SearchState.GoalPoints.PointData;
					OutputPointData->InitializeFromDataWithParams(InitParams);

					InitParams.Source = IterationState.StartPoints.PointData;
					OutputPointData->InitializeFromDataWithParams(InitParams);
				}

				OutputPointData->SetNumPoints(FinalPath.Num());
				const EPCGPointNativeProperties PropertiesToCopy = ExecutionState.Settings.bCopyOriginatingPoints ? EPCGPointNativeProperties::All : EPCGPointNativeProperties::AllProperties;
				for (int i = 0; i < FinalPath.Num(); i++)
				{
					const PCGSpatialAlgo::AStar::FPointDescription& Point = FinalPath[i];
					Point.SrcPointRange.PointData->CopyPropertiesTo(OutputPointData, Point.SrcPointRange.RangeStartIndex + Point.PointIndexInRange, i, 1, PropertiesToCopy);
				}

				OutputData.Data = OutputPointData;
			}

			if (ExecutionState.Settings.bAcceptPartialPath)
			{
				OutputData.Tags.Emplace(SearchResult == ESearchResult::Complete ? PCGPathfindingElement::Constants::CompletePathTag : PCGPathfindingElement::Constants::PartialPathTag);
			}

			++IterationState.PathIterationIndex;

			if (Context->ShouldStop())
			{
				return false;
			}
		}

		return true;
	});
}

#undef LOCTEXT_NAMESPACE
