// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGPointExtentsModifier.h"

#include "PCGContext.h"
#include "PCGPoint.h"
#include "Data/PCGBasePointData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPointExtentsModifier)

FPCGElementPtr UPCGPointExtentsModifierSettings::CreateElement() const
{
	return MakeShared<FPCGPointExtentsModifier>();
}

EPCGPointNativeProperties FPCGPointExtentsModifier::GetPropertiesToAllocate(FPCGContext* Context) const
{
	return EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax;
}

bool FPCGPointExtentsModifier::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPointExtentsModifier::Execute);

	ContextType* Context = static_cast<ContextType*>(InContext);
	const UPCGPointExtentsModifierSettings* Settings = Context->GetInputSettings<UPCGPointExtentsModifierSettings>();
	check(Settings);

	const EPCGPointExtentsModifierMode Mode = Settings->Mode;
	const FVector& Extents = Settings->Extents;

	auto SetExtentsLoop = [](UPCGBasePointData* OutputData, int32 StartIndex, int32 Count, TFunctionRef<void(const FVector&, FVector&, FVector&)> SetExtentsFunc)
	{
		TPCGValueRange<FVector> BoundsMinRange = OutputData->GetBoundsMinValueRange();
		TPCGValueRange<FVector> BoundsMaxRange = OutputData->GetBoundsMaxValueRange();

		for (int32 Index = StartIndex; Index < (StartIndex + Count); ++Index)
		{
			FVector& BoundsMin = BoundsMinRange[Index];
			FVector& BoundsMax = BoundsMaxRange[Index];

			FVector CurrentExtents = PCGPointHelpers::GetExtents(BoundsMin, BoundsMax);

			SetExtentsFunc(CurrentExtents, BoundsMin, BoundsMax);
		}

		return true;
	};

	switch (Mode)
	{
		case EPCGPointExtentsModifierMode::Minimum:
			return ExecutePointOperation(Context, [SetExtentsLoop, Extents](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetExtentsLoop(OutputData, StartIndex, Count, [Extents](const FVector& CurrentExtents, FVector& OutBoundsMin, FVector& OutBoundsMax)
				{
					PCGPointHelpers::SetExtents(FVector::Min(CurrentExtents, Extents), OutBoundsMin, OutBoundsMax);
				});
			});

		case EPCGPointExtentsModifierMode::Maximum:
			return ExecutePointOperation(Context, [SetExtentsLoop, Extents](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetExtentsLoop(OutputData, StartIndex, Count, [Extents](const FVector& CurrentExtents, FVector& OutBoundsMin, FVector& OutBoundsMax)
				{
					PCGPointHelpers::SetExtents(FVector::Max(CurrentExtents, Extents), OutBoundsMin, OutBoundsMax);
				});
			});

		case EPCGPointExtentsModifierMode::Add:
			return ExecutePointOperation(Context, [SetExtentsLoop, Extents](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetExtentsLoop(OutputData, StartIndex, Count, [Extents](const FVector& CurrentExtents, FVector& OutBoundsMin, FVector& OutBoundsMax)
				{
					PCGPointHelpers::SetExtents(CurrentExtents + Extents, OutBoundsMin, OutBoundsMax);
				});
			});

		case EPCGPointExtentsModifierMode::Multiply:
			return ExecutePointOperation(Context, [SetExtentsLoop, Extents](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetExtentsLoop(OutputData, StartIndex, Count, [Extents](const FVector& CurrentExtents, FVector& OutBoundsMin, FVector& OutBoundsMax)
				{
					PCGPointHelpers::SetExtents(CurrentExtents * Extents, OutBoundsMin, OutBoundsMax);
				});
			});

		case EPCGPointExtentsModifierMode::Set:
			return ExecutePointOperation(Context, [SetExtentsLoop, Extents](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
			{
				return SetExtentsLoop(OutputData, StartIndex, Count, [Extents](const FVector& CurrentExtents, FVector& OutBoundsMin, FVector& OutBoundsMax)
				{
					PCGPointHelpers::SetExtents(Extents, OutBoundsMin, OutBoundsMax);
				});
			});
	}

	return true;
}
