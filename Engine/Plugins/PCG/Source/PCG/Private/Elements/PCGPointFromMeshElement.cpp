// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGPointFromMeshElement.h"

#include "PCGComponent.h"
#include "PCGEdge.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"

#if WITH_EDITOR
#include "Helpers/PCGDynamicTrackingHelpers.h"
#endif // WITH_EDITOR

#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPointFromMeshElement)

#define LOCTEXT_NAMESPACE "PCGPointFromMeshElement"

void UPCGPointFromMeshSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (!StaticMesh.IsNull())
	{
		Mesh = StaticMesh;
		StaticMesh.Reset();
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
FText UPCGPointFromMeshSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Point From Mesh");
}

FText UPCGPointFromMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("PointFromMeshNodeTooltip", "Creates a single point at the origin with an attribute named MeshPathAttributeName containing a SoftObjectPath to the StaticMesh/SkeletalMesh.");
}

void UPCGPointFromMeshSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (Mesh.IsNull() || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGPointFromMeshSettings, Mesh)))
	{
		return;
	}

	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(Mesh.ToSoftObjectPath());

	OutKeysToSettings.FindOrAdd(MoveTemp(Key)).Emplace(this, /*bCulling=*/false);
}
#endif

FPCGElementPtr UPCGPointFromMeshSettings::CreateElement() const
{
	return MakeShared<FPCGPointFromMeshElement>();
}

bool FPCGPointFromMeshElement::PrepareDataInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPointFromMeshElement::PrepareData);

	check(Context);

	const UPCGPointFromMeshSettings* Settings = Context->GetInputSettings<UPCGPointFromMeshSettings>();
	check(Settings);

	if (Settings->Mesh.IsNull())
	{
		return true;
	}

	FPCGPointFromMeshContext* ThisContext = static_cast<FPCGPointFromMeshContext*>(Context);

	if (!ThisContext->WasLoadRequested())
	{
		return ThisContext->RequestResourceLoad(ThisContext, { Settings->Mesh.ToSoftObjectPath() }, !Settings->bSynchronousLoad);
	}

	return true;
}

bool FPCGPointFromMeshElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPointFromMeshElement::Execute);

	check(Context);

	const UPCGPointFromMeshSettings* Settings = Context->GetInputSettings<UPCGPointFromMeshSettings>();
	check(Settings);

	if (Settings->Mesh.IsNull())
	{
		return true;
	}

	const UObject* MeshObject = Settings->Mesh.Get();

	if (!MeshObject || (!MeshObject->IsA<UStaticMesh>() && !MeshObject->IsA<USkeletalMesh>()))
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("LoadStaticMeshFailed", "Failed to load StaticMesh/SkeletalMesh"));
		return true;
	}

#if WITH_EDITOR
	if (Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGPointFromMeshSettings, Mesh)))
	{
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, FPCGSelectionKey::CreateFromPath(Settings->Mesh.ToSoftObjectPath()), /*bIsCulled=*/false);
	}
#endif // WITH_EDITOR

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(Context);
	Outputs.Emplace_GetRef().Data = OutPointData;
	
	OutPointData->SetNumPoints(1);

	FBox MeshBounds{};
	if (const UStaticMesh* StaticMesh = Cast<UStaticMesh>(MeshObject))
	{
		MeshBounds = StaticMesh->GetBoundingBox();
	}
	else if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(MeshObject))
	{
		MeshBounds = SkeletalMesh->GetBounds().GetBox();
	}
	else
	{
		checkNoEntry();
	}

	OutPointData->SetBoundsMin(MeshBounds.Min);
	OutPointData->SetBoundsMax(MeshBounds.Max);

	// Write StaticMesh path to MeshPathAttribute
	check(OutPointData->Metadata);
	OutPointData->Metadata->CreateSoftObjectPathAttribute(Settings->MeshPathAttributeName, Settings->Mesh.ToSoftObjectPath(), /*bAllowsInterpolation=*/false);

	return true;
}

#undef LOCTEXT_NAMESPACE
