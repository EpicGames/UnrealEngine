// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGPointMatchAndSet.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGPin.h"
#include "Data/PCGPointData.h"
#include "Helpers/PCGHelpers.h"
#include "MatchAndSet/PCGMatchAndSetWeighted.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPointMatchAndSet)

#define LOCTEXT_NAMESPACE "PCGPointMatchAndSetElement"

UPCGPointMatchAndSetSettings::UPCGPointMatchAndSetSettings(const FObjectInitializer& ObjectInitializer)
{
	MatchAndSetType = UPCGMatchAndSetWeighted::StaticClass();

	if (!this->HasAnyFlags(RF_ClassDefaultObject))
	{
		MatchAndSetInstance = ObjectInitializer.CreateDefaultSubobject<UPCGMatchAndSetWeighted>(this, TEXT("DefaultMatchAndSet"));
	}
}

#if WITH_EDITOR
FText UPCGPointMatchAndSetSettings::GetNodeTooltipText() const
{
	return LOCTEXT("PointMatchAndSetNodeTooltip", "For all points, if a match is found (e.g. some attribute is equal to some value), sets a value on the point (e.g. another attribute).");
}

void UPCGPointMatchAndSetSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	if (DataVersion < FPCGCustomVersion::UpdateAttributePropertyInputSelector
		&& SetTarget.GetSelection() == EPCGAttributePropertySelection::Attribute
		&& SetTarget.GetAttributeName() == PCGMetadataAttributeConstants::SourceAttributeName)
	{
		// Previous behavior of the output target for this node was:
		// None => LastCreated
		SetTarget.SetAttributeName(PCGMetadataAttributeConstants::LastCreatedAttributeName);
	}

	Super::ApplyDeprecation(InOutNode);
}
#endif // WITH_EDITOR

bool UPCGPointMatchAndSetSettings::UseSeed() const
{
	return MatchAndSetInstance && MatchAndSetInstance->UsesRandomProcess();
}

TArray<FPCGPinProperties> UPCGPointMatchAndSetSettings::InputPinProperties() const
{
	// TODO Add param support?
	return Super::DefaultPointInputPinProperties();
}

TArray<FPCGPinProperties> UPCGPointMatchAndSetSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);

	return PinProperties;
}

FPCGElementPtr UPCGPointMatchAndSetSettings::CreateElement() const
{
	return MakeShared<FPCGPointMatchAndSetElement>();
}

void UPCGPointMatchAndSetSettings::PostLoad()
{
	Super::PostLoad();

	if (!MatchAndSetInstance)
	{
		RefreshMatchAndSet();
	}
	else
	{
		const EObjectFlags Flags = GetMaskedFlags(RF_PropagateToSubObjects) | RF_Transactional;
		MatchAndSetInstance->SetFlags(Flags);
	}

#if WITH_EDITOR
	if (SetTargetType == EPCGMetadataTypes::String)
	{
		if (SetTargetStringMode_DEPRECATED == EPCGMetadataTypesConstantStructStringMode::SoftObjectPath)
		{
			SetTargetType = EPCGMetadataTypes::SoftObjectPath;
		}
		else if (SetTargetStringMode_DEPRECATED == EPCGMetadataTypesConstantStructStringMode::SoftClassPath)
		{
			SetTargetType = EPCGMetadataTypes::SoftClassPath;
		}
	}
#endif
}

#if WITH_EDITOR

void UPCGPointMatchAndSetSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		const FName& PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGPointMatchAndSetSettings, MatchAndSetType))
		{
			RefreshMatchAndSet();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGPointMatchAndSetSettings, SetTarget))
		{
			bSetTargetIsAttribute = (SetTarget.GetSelection() == EPCGAttributePropertySelection::Attribute);

			// If not targetting an attribute, but a property, assign the new type to the OutputType accordingly
			if (!bSetTargetIsAttribute)
			{
				TUniquePtr<const IPCGAttributeAccessor> SetTargetAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(nullptr, SetTarget);
				if (SetTargetAccessor)
				{
					SetTargetType = static_cast<EPCGMetadataTypes>(SetTargetAccessor->GetUnderlyingType());

					if (MatchAndSetInstance)
					{
						MatchAndSetInstance->SetType(SetTargetType);
					}
				}
			}
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGPointMatchAndSetSettings, SetTargetType))
		{
			if (MatchAndSetInstance)
			{
				MatchAndSetInstance->SetType(SetTargetType);
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

void UPCGPointMatchAndSetSettings::SetMatchAndSetType(TSubclassOf<UPCGMatchAndSetBase> InMatchAndSetType)
{
	if (!MatchAndSetInstance || InMatchAndSetType != MatchAndSetType)
	{
		MatchAndSetType = InMatchAndSetType;
		RefreshMatchAndSet();
	}
}

void UPCGPointMatchAndSetSettings::RefreshMatchAndSet()
{
	if (MatchAndSetType)
	{
		ensure(IsInGameThread());

		// Forget previous instance
		if (MatchAndSetInstance)
		{
#if WITH_EDITOR
			MatchAndSetInstance->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
#endif
			MatchAndSetInstance->MarkAsGarbage();
			MatchAndSetInstance = nullptr;
		}

		const EObjectFlags Flags = GetMaskedFlags(RF_PropagateToSubObjects);
		MatchAndSetInstance = NewObject<UPCGMatchAndSetBase>(this, MatchAndSetType, NAME_None, Flags);
		check(MatchAndSetInstance);
		MatchAndSetInstance->SetType(SetTargetType);
	}
	else
	{
		MatchAndSetInstance = nullptr;
	}
}

bool FPCGPointMatchAndSetElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPointMatchAndSetElement::Execute);

	const UPCGPointMatchAndSetSettings* Settings = Context->GetInputSettings<UPCGPointMatchAndSetSettings>();
	check(Settings);

	const UPCGMatchAndSetBase* MatchAndSet = Settings->MatchAndSetInstance;

	if (!MatchAndSet)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidMatchAndSetInstance", "Invalid MatchAndSet instance, try recreating this node from the node palette"));
		return true;
	}

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		const UPCGPointData* InPointData = Cast<UPCGPointData>(Input.Data);
		if (!InPointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputDataType", "Input data must be of type Point"));
			continue;
		}

		// Perform validation - the MatchAndSet is responsible to validate if the input
		// conforms to what is expected (esp. on the metadata side)
		if (!MatchAndSet->ValidatePreconditions(InPointData))
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("PreconditionsFailed", "MatchAndSet failed to validate preconditions on input data"));
			continue;
		}

		UPCGPointData* OutPointData = FPCGContext::NewObject_AnyThread<UPCGPointData>(Context);
		Output.Data = OutPointData;

		OutPointData->InitializeFromData(InPointData);
		// Copy all points
		OutPointData->GetMutablePoints() = InPointData->GetPoints();

		// Apply MatchAndSet
		MatchAndSet->MatchAndSet(*Context, Settings, InPointData, OutPointData);

		if (MatchAndSet->ShouldMutateSeed())
		{
			TArray<FPCGPoint>& OutPoints = OutPointData->GetMutablePoints();
			for (int32 PointIndex = 0; PointIndex < OutPoints.Num(); ++PointIndex)
			{
				FPCGPoint& Point = OutPoints[PointIndex];
				Point.Seed = PCGHelpers::ComputeSeed(Point.Seed, PointIndex);
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
