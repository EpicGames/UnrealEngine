// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGPointNeighborhood.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPointNeighborhood)

#define LOCTEXT_NAMESPACE "PCGPointNeighborhoodElement"

namespace PCGPointNeighborhood
{
	template<typename T>
	void SetAttributeHelper(UPCGBasePointData* Data, const FName& AttributeName, const TArrayView<const T> Values)
	{
		// Discard None attributes
		if (AttributeName == NAME_None)
		{
			return;
		}

		ensure(Data->Metadata->FindOrCreateAttribute<T>(AttributeName));

		FPCGAttributePropertySelector AttributeSelector = FPCGAttributePropertySelector::CreateAttributeSelector(AttributeName);

		TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(Data, AttributeSelector);
		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(Data, AttributeSelector);
		
		if (Accessor.IsValid() && Keys.IsValid())
		{
			Accessor->SetRange(Values, 0, *Keys);
		}
		else
		{
			return;
		}
	}
}

FPCGElementPtr UPCGPointNeighborhoodSettings::CreateElement() const
{
	return MakeShared<FPCGPointNeighborhoodElement>();
}

bool FPCGPointNeighborhoodElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPointNeighborhoodElement::Execute);

	const UPCGPointNeighborhoodSettings* Settings = Context->GetInputSettings<UPCGPointNeighborhoodSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const double SearchDistance = Settings->SearchDistance;
	if (SearchDistance < UE_DOUBLE_SMALL_NUMBER)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidSearchDistance", "Search Distance must be greater than 0."));
		return true;
	}

	struct FProcessResults
	{
		TArray<double> Distances;
		TArray<FVector> AveragePositions;
	};

	FProcessResults OutputDataBuffers;

	for (int i = 0; i < Inputs.Num(); ++i)
	{
		const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(Inputs[i].Data);
		if (!InputPointData)
		{
			PCGE_LOG(Verbose, GraphAndLog, FText::Format(LOCTEXT("InvalidPointData", "Input {0} is not point data"), i));
			continue;
		}

		FPCGTaggedData& Output = Outputs.Add_GetRef(Inputs[i]);
		UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(Context);
		OutputPointData->InitializeFromData(InputPointData);

		EPCGPointNativeProperties PropertiesToAllocate = EPCGPointNativeProperties::None;

		if (Settings->SetDensity != EPCGPointNeighborhoodDensityMode::None)
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::Density;
		}

		if (Settings->bSetAveragePosition)
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::Transform;
		}

		if (Settings->bSetAverageColor)
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::Color;
		}

		if (Settings->bSetDistanceToAttribute || Settings->bSetAveragePositionToAttribute)
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::MetadataEntry;
		}

		OutputPointData->SetNumPoints(InputPointData->GetNumPoints(), /*bInitializeValues=*/false);
		if (!OutputPointData->HasSpatialDataParent())
		{
			OutputPointData->AllocateProperties(InputPointData->GetAllocatedProperties());
		}

		OutputPointData->AllocateProperties(PropertiesToAllocate);

		Output.Data = OutputPointData;
		
		const FBox SearchBounds(FVector(SearchDistance * -1.0), FVector(SearchDistance));

		FPCGProjectionParams Params{};
		Params.bProjectRotations = Params.bProjectScales = false;
		Params.ColorBlendMode = Settings->bSetAverageColor ? EPCGProjectionColorBlendMode::TargetValue : EPCGProjectionColorBlendMode::SourceValue;

		auto InitializeBuffers = [Settings, &OutputDataBuffers, Count = InputPointData->GetNumPoints()]()
		{
			if (Settings->bSetDistanceToAttribute)
			{
				OutputDataBuffers.Distances.SetNumUninitialized(Count);
			}

			if (Settings->bSetAveragePositionToAttribute)
			{
				OutputDataBuffers.AveragePositions.SetNumUninitialized(Count);
			}
		};

		auto ProcessPoint = [Settings, &OutputDataBuffers, &SearchBounds, &SearchDistance, &Params, InputPointData, OutputPointData](const int32 StartReadIndex, const int32 StartWriteIndex, const int32 Count)
		{
			const FConstPCGPointValueRanges InRanges(InputPointData);
			TPCGValueRange<float> DensityRange = Settings->SetDensity != EPCGPointNeighborhoodDensityMode::None ? OutputPointData->GetDensityValueRange(/*bAllocate=*/false) : TPCGValueRange<float>();
			TPCGValueRange<FTransform> TransformRange = Settings->bSetAveragePosition ? OutputPointData->GetTransformValueRange(/*bAllocate=*/false) : TPCGValueRange<FTransform>();
			TPCGValueRange<FVector4> ColorRange = Settings->bSetAverageColor ? OutputPointData->GetColorValueRange(/*bAllocate=*/false) : TPCGValueRange<FVector4>();

			if (!OutputPointData->HasSpatialDataParent())
			{
				InputPointData->CopyPointsTo(OutputPointData, StartReadIndex, StartWriteIndex, Count);
			}

			int32 NumWritten = 0;
			for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
			{
				const int32 WriteIndex = StartWriteIndex + NumWritten;

				FVector InLocation = InRanges.TransformRange[ReadIndex].GetLocation();
				const FTransform InTransform(InLocation);

				FPCGPoint ProjectionPoint = FPCGPoint();
				// Do not pass metadata as output as we don't care about attributes (Projected point will be discarded).
				// @todo_pcg: Maybe support the weighting average of attributes as an option.
				// @todo_pcg: Might be better to use ProjectPoints using a range here?
				InputPointData->ProjectPoint(InTransform, SearchBounds, Params, ProjectionPoint, /*OutMetadata=*/nullptr, Settings->bWeightedAverage);

				const double NormalizedDistance = FVector::Distance(InLocation, ProjectionPoint.Transform.GetLocation()) / SearchDistance;

				if (Settings->SetDensity == EPCGPointNeighborhoodDensityMode::SetNormalizedDistanceToDensity)
				{
					DensityRange[WriteIndex] = FMath::Clamp(NormalizedDistance, 0.0, 1.0);
				}
				else if (Settings->SetDensity == EPCGPointNeighborhoodDensityMode::SetAverageDensity)
				{
					DensityRange[WriteIndex] = ProjectionPoint.Density;
				}

				if (Settings->bSetDistanceToAttribute)
				{
					OutputDataBuffers.Distances[WriteIndex] = FVector::Distance(InLocation, ProjectionPoint.Transform.GetLocation());
				}

				if (Settings->bSetAveragePosition)
				{
					TransformRange[WriteIndex].SetLocation(ProjectionPoint.Transform.GetLocation());
				}

				if (Settings->bSetAveragePositionToAttribute)
				{
					OutputDataBuffers.AveragePositions[WriteIndex] = ProjectionPoint.Transform.GetLocation();
				}

				if (Settings->bSetAverageColor)
				{
					ColorRange[WriteIndex] = ProjectionPoint.Color;
				}

				++NumWritten;
			}

			check(NumWritten == Count);
			return Count;
		};

		FPCGAsync::AsyncProcessingOneToOneRangeEx(&Context->AsyncState, InputPointData->GetNumPoints(), InitializeBuffers, ProcessPoint, /* bEnableTimeSlicing=*/false);

		if (Settings->bSetDistanceToAttribute)
		{
			PCGPointNeighborhood::SetAttributeHelper<double>(OutputPointData, Settings->DistanceAttribute, OutputDataBuffers.Distances);
		}
		if (Settings->bSetAveragePositionToAttribute)
		{
			PCGPointNeighborhood::SetAttributeHelper<FVector>(OutputPointData, Settings->AveragePositionAttribute, OutputDataBuffers.AveragePositions);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
