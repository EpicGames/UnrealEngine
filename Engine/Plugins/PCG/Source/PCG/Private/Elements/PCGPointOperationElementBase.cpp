// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGPointOperationElementBase.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGPointData.h"
#include "Helpers/PCGAsync.h"

#define LOCTEXT_NAMESPACE "PCGPointOperationElementBase"

bool FPCGPointOperationElementBase::PrepareDataInternal(FPCGContext* Context) const
{
	check(Context);
	ContextType* PointProcessContext = static_cast<ContextType*>(Context);
	check(PointProcessContext);

	// Prepares the context for time slicing
	return PreparePointOperationData(PointProcessContext);
}

bool FPCGPointOperationElementBase::PreparePointOperationData(ContextType* InContext, FName InputPinLabel) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPointOperationElementBase::PreparePointProcessing);
	check(InContext);

	const TArray<FPCGTaggedData>& Inputs = InContext->InputData.GetInputsByPin(InputPinLabel);
	TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;

	// There is no execution state, so this just flags that its okay to continue
	InContext->InitializePerExecutionState();

	// Prepare the 'per iteration' time slice context state and allocate output point data
	InContext->InitializePerIterationStates(Inputs.Num(),
		[this, &InContext, &Inputs, &Outputs](IterStateType& OutState, const ExecStateType& ExecState, const uint32 IterationIndex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPointOperationElementBase::InitializePerIterationStates);

			FPCGTaggedData& Output = Outputs.Add_GetRef(Inputs[IterationIndex]);

			const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Inputs[IterationIndex].Data);

			if (!SpatialData)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("InputMissingSpatialData", "Unable to get Spatial data from input"), InContext);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			OutState.InputData = SupportsBasePointDataInputs(InContext) ? SpatialData->ToBasePointData(InContext) : SpatialData->ToPointData(InContext);
			if (!OutState.InputData)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("InputMissingPointData", "Unable to get Point data from input"), InContext);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			OutState.NumPoints = OutState.InputData->GetNumPoints();

			// Create and initialize the output points
			OutState.OutputData = SupportsBasePointDataInputs(InContext) ? FPCGContext::NewPointData_AnyThread(InContext) : FPCGContext::NewObject_AnyThread<UPCGPointData>(InContext);
			OutState.OutputData->InitializeFromData(OutState.InputData);
			OutState.OutputData->SetNumPoints(OutState.NumPoints, /*bInitializeValues=*/false);

			// Allocate properties that we are going to modify
			EPCGPointNativeProperties PropertiesToAllocate = GetPropertiesToAllocate(InContext);
			
			// If data doesn't support parenting also allocate properties we are going to copy from input
			if (!OutState.OutputData->HasSpatialDataParent())
			{
				PropertiesToAllocate |= OutState.InputData->GetAllocatedProperties();
			}

			OutState.OutputData->AllocateProperties(PropertiesToAllocate);

			Output.Data = OutState.OutputData;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OutState.InputPointData = Cast<UPCGPointData>(OutState.InputData);
			OutState.OutputPointData = Cast<UPCGPointData>(OutState.OutputData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			return EPCGTimeSliceInitResult::Success;
		});

	return true;
}

namespace PCGPointOperationElementBase
{
	bool ExecuteSliceOneToOne(FPCGPointOperationElementBase::ContextType* Context, const FPCGPointOperationElementBase::IterStateType& IterState, TFunctionRef<bool(const FPCGPoint& InPoint, FPCGPoint& OutPoint)> Callback, int32 PointsPerChunk, bool bShouldCopyPoints)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		check(IterState.InputPointData && IterState.OutputPointData);
		const TArray<FPCGPoint>& InPoints = IterState.InputPointData->GetPoints();
		TArray<FPCGPoint>& OutPoints = IterState.OutputPointData->GetMutablePoints();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Conversion lambda from index to point ref for ease of use
		auto InternalPointFunction = [&Callback, &InPoints, &OutPoints, bShouldCopyPoints](int32 ReadIndex, int32 WriteIndex) -> bool
		{
			if(bShouldCopyPoints)
			{
				OutPoints[WriteIndex] = InPoints[ReadIndex];
			}

			return Callback(InPoints[ReadIndex], OutPoints[WriteIndex]);
		};

		return FPCGAsync::AsyncProcessingOneToOneEx(
			&Context->AsyncState,
			IterState.NumPoints,
			/*InitializeFunc=*/[] {}, // Not useful for this context, since its preferred to initialize in PrepareDataInternal, so empty lambda
			std::move(InternalPointFunction),
			Context->TimeSliceIsEnabled(),
			PointsPerChunk);
	}

	bool ExecuteSliceRange(FPCGPointOperationElementBase::ContextType* Context, const FPCGPointOperationElementBase::IterStateType& IterState, TFunctionRef<bool(const UPCGBasePointData* InPointData, UPCGBasePointData* OutPointData, int32 InStartIndex, int32 InCount)> Callback, int32 PointsPerChunk, bool bShouldCopyPoints)
	{
		const UPCGBasePointData* InputData = IterState.InputData;
		UPCGBasePointData* OutputData = IterState.OutputData;

		auto ProcessRangeFunc = [&Callback, &InputData, &OutputData, bShouldCopyPoints](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			check(StartReadIndex == StartWriteIndex);
			if(bShouldCopyPoints && !OutputData->HasSpatialDataParent())
			{
				InputData->CopyPointsTo(OutputData, StartReadIndex, StartReadIndex, Count);
			}
						
			Callback(InputData, OutputData, StartReadIndex, Count);
			return Count;
		};

		return FPCGAsync::AsyncProcessingOneToOneRangeEx(
			&Context->AsyncState,
			IterState.NumPoints,
			/*InitializeFunc=*/[] {}, // Not useful for this context, since its preferred to initialize in PrepareDataInternal, so empty lambda
			std::move(ProcessRangeFunc),
			Context->TimeSliceIsEnabled(),
			PointsPerChunk);
	}

	template <typename Func>
	bool ExecutePointOperation(const FPCGPointOperationElementBase* Element, FPCGPointOperationElementBase::ContextType* Context, Func Callback, int32 PointsPerChunk, bool bShouldCopyPoints)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPointOperationElementBase::ExecutePointProcessing);
		check(Context);

		// Standard check that the time slice state has been prepared. If the result is NoOp or Failure, result in no output
		if (!Context->DataIsPreparedForExecution() || Context->GetExecutionStateResult() != EPCGTimeSliceInitResult::Success)
		{
			Context->OutputData.TaggedData.Empty();
			return true;
		}

		return Element->ExecuteSlice(Context, [&Callback, &PointsPerChunk, &bShouldCopyPoints](FPCGPointOperationElementBase::ContextType* Context, const FPCGPointOperationElementBase::ExecStateType& ExecState, const FPCGPointOperationElementBase::IterStateType& IterState, const uint32 IterIndex)
		{
			// If this input created an error, result in no output
			if (Context->GetIterationStateResult(IterIndex) != EPCGTimeSliceInitResult::Success)
			{
				if (IterState.OutputData)
				{
					IterState.OutputData->SetNumPoints(0);
				}
				
				return true;
			}

			bool bAsyncDone = false;
			if constexpr (std::is_invocable_v<Func, const FPCGPoint&, FPCGPoint&>)
			{
				bAsyncDone = ExecuteSliceOneToOne(Context, IterState, Callback, PointsPerChunk, bShouldCopyPoints);
			}
			else
			{
				bAsyncDone = ExecuteSliceRange(Context, IterState, Callback, PointsPerChunk, bShouldCopyPoints);
			}

			if (bAsyncDone)
			{
				PCGE_LOG_C(Verbose, LogOnly, Context, FText::Format(LOCTEXT("PointProcessInfo", "Processed {0} points"), IterState.NumPoints));
			}

			return bAsyncDone;
		});
	}
}

bool FPCGPointOperationElementBase::ExecutePointOperationWithPoints(ContextType* Context, TFunctionRef<bool(const FPCGPoint& InPoint, FPCGPoint& OutPoint)> Callback, int32 PointsPerChunk) const
{
	return PCGPointOperationElementBase::ExecutePointOperation(this, Context, Callback, PointsPerChunk, ShouldCopyPoints());
}

bool FPCGPointOperationElementBase::ExecutePointOperationWithIndices(ContextType* Context, TFunctionRef<bool(const UPCGBasePointData* InPointData, UPCGBasePointData* OutPointData, int32 InStartIndex, int32 InCount)> Callback, int32 PointsPerChunk) const
{
	return PCGPointOperationElementBase::ExecutePointOperation(this, Context, Callback, PointsPerChunk, ShouldCopyPoints());
}

#undef LOCTEXT_NAMESPACE
