// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGPointProcessingElementBase.h"

#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Helpers/PCGAsync.h"
#include "PCGContext.h"

#define LOCTEXT_NAMESPACE "PCGPointProcessingElementBase"

void FPCGPointProcessingElementBase::ProcessPoints(FPCGContext* Context, const TArray<FPCGTaggedData>& Inputs, TArray<FPCGTaggedData>& Outputs, const TFunction<bool(const FPCGPoint&, FPCGPoint&)>& PointFunc) const
{
	// TODO: embarassingly parallel loop
	for (const FPCGTaggedData& Input : Inputs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPointProcessingElement::Execute::InputLoop);
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Input.Data);

		if (!SpatialData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InputMissingSpatialData", "Unable to get Spatial data from input"));
			continue;
		}

		const UPCGPointData* PointData = SpatialData->ToPointData(Context);

		if (!PointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InputMissingPointData", "Unable to get Point data from input"));
			continue;
		}

		const TArray<FPCGPoint>& Points = PointData->GetPoints();

		UPCGPointData* OutputData = FPCGContext::NewObject_AnyThread<UPCGPointData>(Context);
		OutputData->InitializeFromData(PointData);
		TArray<FPCGPoint>& OutputPoints = OutputData->GetMutablePoints();
		Output.Data = OutputData;

		FPCGAsync::AsyncPointProcessing(Context, Points, OutputPoints, PointFunc);
	}
}

#undef LOCTEXT_NAMESPACE
