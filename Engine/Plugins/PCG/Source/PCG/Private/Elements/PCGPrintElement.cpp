// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGPrintElement.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGModule.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPrintElement)

#define LOCTEXT_NAMESPACE "PCGPrintElement"

namespace PCGPrintElementConstants
{
	FText Delimiter = LOCTEXT("Delimiter", "::");
}

namespace PCGPrintElementHelpers
{
	void CleanUpMessage(uint64& OutMessageHashKey)
	{
#if WITH_EDITOR
		if (GEngine && (OutMessageHashKey != (uint64)-1) && GEngine->OnScreenDebugMessageExists(OutMessageHashKey))
		{
			GEngine->RemoveOnScreenDebugMessage(OutMessageHashKey);
			OutMessageHashKey = (uint64)-1;
		}
#endif // WITH_EDITOR
	}
}

bool UPCGManagedDebugStringMessageKey::Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	PCGPrintElementHelpers::CleanUpMessage(HashKey);
	return Super::Release(bHardRelease, OutActorsToDelete);
}

TArray<FPCGPinProperties> UPCGPrintElementSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any);
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGPrintElementSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
	return PinProperties;
}

FPCGElementPtr UPCGPrintElementSettings::CreateElement() const
{
	return MakeShared<FPCGPrintElement>();
}

bool FPCGPrintElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPrintElement::Execute);
	Context->OutputData.TaggedData = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	const UPCGPrintElementSettings* Settings = Context->GetInputSettings<UPCGPrintElementSettings>();
	check(Settings);

	if (!Settings->bEnablePrint
		|| (Settings->Verbosity == EPCGPrintVerbosity::NoLogging
#if WITH_EDITOR
			&& !Settings->bPrintToScreen
#endif // WITH_EDITOR
		))
	{
		return true;
	}

	TArray<FString> Prefixes;

	// Build the message prefixes
	UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get());
	const UPCGComponent* Component = SourceComponent ? SourceComponent->GetOriginalComponent() : nullptr;
	if (Settings->bPrintPerComponent && Component)
	{
		if (Settings->bPrefixWithOwner)
		{
#if WITH_EDITOR
			Prefixes.Emplace(Component->GetOwner() ? Component->GetOwner()->GetActorLabel() : LOCTEXT("MissingOwner", "Missing Owner").ToString());
#else
			Prefixes.Emplace(Component->GetOwner() ? Component->GetOwner()->GetName() : LOCTEXT("MissingOwner", "Missing Owner").ToString());
#endif // WITH_EDITOR
		}

		if (Settings->bPrefixWithComponent)
		{
			Prefixes.Emplace(Component->GetName());
		}

		if (Settings->bPrefixWithGraph)
		{
			Prefixes.Emplace(Component->GetGraph() ? Component->GetGraph()->GetName() : LOCTEXT("MissingGraph", "Missing Graph").ToString());
		}
	}

	if (Settings->bPrefixWithNode)
	{
		Prefixes.Emplace(Context->Node ? Context->Node->GetName() : LOCTEXT("MissingNode", "Missing Node").ToString());
	}

	FString ObjectPrefix;
	if (!Prefixes.IsEmpty())
	{
		ObjectPrefix = FString::Printf(TEXT("[%s]: "), *FString::Join(Prefixes, *PCGPrintElementConstants::Delimiter.ToString()));
	}

	const FString FinalString = Settings->CustomPrefix + ObjectPrefix + Settings->PrintString;

	switch (Settings->Verbosity)
	{
		case EPCGPrintVerbosity::Error:
			if (Settings->bDisplayOnNode)
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, FText::FromString(FinalString));
			}
			else
			{ //-V523
				PCGE_LOG_C(Error, LogOnly, Context, FText::FromString(FinalString));
			}
			break;
		case EPCGPrintVerbosity::Warning:
			if (Settings->bDisplayOnNode)
			{
				PCGE_LOG_C(Warning, GraphAndLog, Context, FText::FromString(FinalString));
			}
			else
			{ //-V523
				PCGE_LOG_C(Warning, LogOnly, Context, FText::FromString(FinalString));
			}
			break;
		case EPCGPrintVerbosity::Log:
			PCGE_LOG_C(Log, LogOnly, Context, FText::FromString(FinalString));
			break;
		case EPCGPrintVerbosity::Display:
			PCGE_LOG_C(Display, LogOnly, Context, FText::FromString(FinalString));
			break;
		case EPCGPrintVerbosity::NoLogging:
			break;
		default:
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidVerbosity", "Invalid verbosity selected. Please use Log, Display, Warning, or Error. See EPCGPrintVerbosity for more details."), Context);
			return true;
	}

#if WITH_EDITOR
	if (Settings->bPrintToScreen && GEngine && SourceComponent)
	{
		if (UEditorEngine* Editor = static_cast<UEditorEngine*>(GEngine))
		{
			uint32 HashKey32 = HashCombine(GetTypeHash(Context->Node->GetFName()), Context->Node->GetUniqueID());

			if (Settings->bPrintPerComponent)
			{
				HashKey32 = HashCombine(HashKey32, Component->GetUniqueID());
			}

			// Add the stack hash to include the loop indices
			HashKey32 = HashCombine(HashKey32, GetTypeHash(Context->GetStack()));

			check(SourceComponent);
			UPCGManagedDebugStringMessageKey* ManagedMessageKey = FPCGContext::NewObject_AnyThread<UPCGManagedDebugStringMessageKey>(Context, SourceComponent);
			ManagedMessageKey->HashKey = static_cast<uint64>(HashKey32);

			SourceComponent->AddToManagedResources(ManagedMessageKey);

			// Count 0 as infinite duration to match other debug features in PCG
			const double Duration = Settings->PrintToScreenDuration > 0.0 ? Settings->PrintToScreenDuration : std::numeric_limits<double>::max();
			Editor->AddOnScreenDebugMessage(ManagedMessageKey->HashKey, Duration, Settings->PrintToScreenColor, FinalString);
		}
	}
#endif // WITH_EDITOR
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING

	return true;
}

#undef LOCTEXT_NAMESPACE
