// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGProjectionElement.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGEdge.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGProjectionData.h"
#include "Data/PCGSpatialData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGProjectionElement)

#define LOCTEXT_NAMESPACE "PCGProjectionElement"

TArray<FPCGPinProperties> UPCGProjectionSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Concrete, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true,  LOCTEXT("ProjectionSourcePinTooltip", "The data to project."));
	InputPinProperty.SetRequiredPin();

	FPCGPinProperties& ProjectionPinProperty = PinProperties.Emplace_GetRef(PCGProjectionConstants::ProjectionTargetLabel, EPCGDataType::Concrete, /*bAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false, LOCTEXT("ProjectionTargetPinTooltip", "The projection target."));
	ProjectionPinProperty.SetRequiredPin();

	return PinProperties;
}

void UPCGProjectionSettings::PostLoad()
{
	Super::PostLoad();

	ProjectionParams.ApplyDeprecation();
}

#if WITH_EDITOR
FText UPCGProjectionSettings::GetNodeTooltipText() const
{
	return LOCTEXT("ProjectionNodeTooltip", "Projects each of the inputs connected to In onto the Projection Target and concatenates all of the results to Out.");
}
#endif

FPCGDataTypeIdentifier UPCGProjectionSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	if (InPin->IsOutputPin())
	{
		if (bForceCollapseToPoint || GetTypeUnionIDOfIncidentEdges(PCGPinConstants::DefaultInputLabel) == EPCGDataType::Point)
		{
			return FPCGDataTypeIdentifier{ EPCGDataType::Point };
		}
		else
		{
			// Return type here, because falling back to the super call will return the incident of the types on the primary input.
			return FPCGDataTypeIdentifier{ EPCGDataType::Spatial };
		}
	}

	return Super::GetCurrentPinTypesID(InPin);
}

FPCGElementPtr UPCGProjectionSettings::CreateElement() const
{
	return MakeShared<FPCGProjectionElement>();
}

bool FPCGProjectionElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGProjectionElement::Execute);
	check(Context);

	const UPCGProjectionSettings* Settings = Context->GetInputSettings<UPCGProjectionSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Sources = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData> Targets = Context->InputData.GetInputsByPin(PCGProjectionConstants::ProjectionTargetLabel);

	// If there are no sources or no targets, then nothing to do.
	if (Sources.Num() == 0 || Targets.Num() == 0)
	{
		return true;
	}

	// Ensure we have spatial data to project onto
	const UPCGSpatialData* ProjectionTarget = Cast<UPCGSpatialData>(Targets[0].Data);
	if (!ProjectionTarget)
	{
		return true;
	}

	const FPCGProjectionParams& ProjectionParams = Settings->ProjectionParams;

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const bool bKeepZeroDensityPoints = Settings->bKeepZeroDensityPoints;

	for (const FPCGTaggedData& Source : Sources)
	{
		const UPCGSpatialData* ProjectionSource = Cast<UPCGSpatialData>(Source.Data);

		if (!ProjectionSource)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidProjectionSource", "Invalid projection source data input found (non-Spatial data). Input will be ignored."));
			continue;
		}

		UPCGSpatialData* ProjectionData = ProjectionSource->ProjectOn(Context, ProjectionTarget, ProjectionParams);
		ProjectionData->bKeepZeroDensityPoints = bKeepZeroDensityPoints;

		const UPCGSpatialData* OutputData = ProjectionData;

		if (Settings->bForceCollapseToPoint || ProjectionSource->IsA<UPCGBasePointData>())
		{
			OutputData = OutputData->ToBasePointData(Context);
		}
		else if (OutputData->RequiresCollapseToSample())
		{
			// Calling ToBasePointData will populate the point cache. Doing so here means we can pass in the Context object, which
			// means the operation will be multi-threaded. This primes the cache in the most efficient way.
			OutputData->ToBasePointData(Context);
		}

		FPCGTaggedData& ProjectionTaggedData = Outputs.Emplace_GetRef(Source);
		ProjectionTaggedData.Data = OutputData;

		if (ProjectionParams.TagMergeOperation == EPCGProjectionTagMergeMode::Target)
		{
			ProjectionTaggedData.Tags = Targets[0].Tags;
		}
		else if(ProjectionParams.TagMergeOperation == EPCGProjectionTagMergeMode::Both)
		{
			ProjectionTaggedData.Tags.Append(Targets[0].Tags);
		}
	}

	return true;
}

#if WITH_EDITOR
void UPCGProjectionSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	check(InOutNode);

	if (DataVersion < FPCGCustomVersion::SplitProjectionNodeInputs)
	{
		// Split first pin inputs across two pins. The last edge connected to the first pin becomes the projection target.

		// Loose check we have at least projection source and target pins. If not then this migration code is not valid for this version and should
		// be guarded against.
		check(InOutNode->GetInputPins().Num() >= 2);

		UPCGPin* SourcePin = InOutNode->GetInputPins()[0];
		check(SourcePin);

		if (SourcePin->EdgeCount() > 1)
		{
			UPCGPin* TargetPin = InOutNode->GetInputPins()[1];
			check(TargetPin);

			UPCGEdge* ProjectionTargetEdge = SourcePin->Edges.Last();
			check(ProjectionTargetEdge);

			UPCGPin* UpstreamPin = ProjectionTargetEdge->InputPin;
			check(UpstreamPin);

			UpstreamPin->BreakEdgeTo(SourcePin);
			UpstreamPin->AddEdgeTo(TargetPin);
		}
	}

	Super::ApplyDeprecation(InOutNode);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
