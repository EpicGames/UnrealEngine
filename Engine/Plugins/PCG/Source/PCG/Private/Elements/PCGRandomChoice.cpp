// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGRandomChoice.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGRandomChoice)

#define LOCTEXT_NAMESPACE "PCGRandomChoiceElement"

namespace PCGRandomChoice
{
	UPCGData* ChoosePointData(const UPCGData* InData, TArrayView<int32> InIndexes, FPCGContext* InContext)
	{
		// We know it is a point data
		const UPCGBasePointData* InPointData = CastChecked<const UPCGBasePointData>(InData);
		UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(InContext);

		FPCGInitializeFromDataParams InitializeFromDataParams(InPointData);
		InitializeFromDataParams.bInheritSpatialData = false;
		OutPointData->InitializeFromDataWithParams(InitializeFromDataParams);

		OutPointData->SetNumPoints(InIndexes.Num());
		OutPointData->AllocateProperties(InPointData->GetAllocatedProperties());
		OutPointData->CopyUnallocatedPropertiesFrom(InPointData);

		// Order needs to be stable to sort this part of the array
		Algo::Sort(InIndexes);

		int32 WriteIndex = 0;
		const FConstPCGPointValueRanges InRanges(InPointData);
		FPCGPointValueRanges OutRanges(OutPointData, /*bAllocate=*/false);

		for (int32 ReadIndex : InIndexes)
		{
			OutRanges.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);
			++WriteIndex;
		}

		return OutPointData;
	}

	UPCGData* ChooseParamData(const UPCGData* InData, TArrayView<int32> InIndexes, FPCGContext* InContext)
	{
		// We know it is a param data
		const UPCGParamData* InParamData = CastChecked<const UPCGParamData>(InData);

		UPCGParamData* OutParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InContext);

		// Order needs to be stable to sort this part of the array
		Algo::Sort(InIndexes);

		TArray<PCGMetadataEntryKey> SelectedEntries;
		Algo::Transform(InIndexes, SelectedEntries, [](int32 In) -> PCGMetadataEntryKey { return In; });

		OutParamData->Metadata->InitializeAsCopy(FPCGMetadataInitializeParams(InParamData->Metadata, &SelectedEntries));

		return OutParamData;
	}
}

#if WITH_EDITOR
FName UPCGRandomChoiceSettings::GetDefaultNodeName() const
{
	return FName(TEXT("RandomChoice"));
}

FText UPCGRandomChoiceSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Random Choice");
}

FText UPCGRandomChoiceSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Chooses entries randomly through ratio or a fixed number of entries.\n"
		"Chosen/Discarded entries will be in the same order than they appear in the input data.");
}
#endif

TArray<FPCGPinProperties> UPCGRandomChoiceSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::PointOrParam).SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGRandomChoiceSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGRandomChoiceConstants::ChosenEntriesLabel, EPCGDataType::PointOrParam);

	if (bOutputDiscardedEntries)
	{
		PinProperties.Emplace(PCGRandomChoiceConstants::DiscardedEntriesLabel, EPCGDataType::PointOrParam);
	}

	return PinProperties;
}

FPCGElementPtr UPCGRandomChoiceSettings::CreateElement() const
{
	return MakeShared<FPCGRandomChoiceElement>();
}

bool FPCGRandomChoiceElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRandomChoiceElement::Execute);

	check(Context);

	const UPCGRandomChoiceSettings* Settings = Context->GetInputSettings<UPCGRandomChoiceSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (int i = 0; i < Inputs.Num(); ++i)
	{
		const FPCGTaggedData& CurrentInput = Inputs[i];

		int NumElements = 0;
		int32 Seed = Context->GetSeed();

		UPCGData* (*ChooseFunc)(const UPCGData*, TArrayView<int32>, FPCGContext*) = nullptr;

		if (const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(CurrentInput.Data))
		{
			const TConstPCGValueRange<int32> SeedRange = InputPointData->GetConstSeedValueRange();
			NumElements = InputPointData->GetNumPoints();
			ChooseFunc = &PCGRandomChoice::ChoosePointData;

			// By default, combine the seed with the first point's seed so that multiple data produces different results.
			if (!Settings->bHasCustomSeedSource && NumElements > 0)
			{
				Seed = PCGHelpers::ComputeSeed(Seed, SeedRange[0]);
			}
		}
		else if (const UPCGParamData* InputParamData = Cast<UPCGParamData>(CurrentInput.Data))
		{
			NumElements = InputParamData->Metadata ? InputParamData->Metadata->GetLocalItemCount() : 0;
			ChooseFunc = &PCGRandomChoice::ChooseParamData;
		}
		else
		{
			PCGE_LOG(Verbose, GraphAndLog, FText::Format(LOCTEXT("InvalidData", "Input {0} is not a supported data"), i));
			continue;
		}

		if (NumElements > 0 && Settings->bHasCustomSeedSource)
		{
			FPCGAttributePropertyInputSelector SeedSelector = Settings->CustomSeedSource.CopyAndFixLast(CurrentInput.Data);
			const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(CurrentInput.Data, SeedSelector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(CurrentInput.Data, SeedSelector);
			if (!Accessor || !Keys)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(SeedSelector, Context);
				continue;
			}

			PCGMetadataAttribute::CallbackWithRightType(Accessor->GetUnderlyingType(), [&Accessor, &Keys, &Seed, Settings]<typename T>(const T&)
			{
				PCGMetadataElementCommon::ApplyOnAccessor<T>(*Keys, *Accessor, [&Seed](const T& Value, int32)
				{
					const uint32 Hash = PCG::Private::MetadataTraits<T>::Hash(Value);
					Seed = HashCombineFast(Seed, Hash);
				},
				EPCGAttributeAccessorFlags::StrictType,
				PCGMetadataElementCommon::DefaultChunkSize,
				/*Count=*/Settings->bUseFirstAttributeOnly ? 1 : /*UseAllKeys*/-1);
			});
		}

		check(ChooseFunc);

		int NumOfElementsToKeep = 0;

		if (Settings->bFixedMode)
		{
			NumOfElementsToKeep = FMath::Clamp(Settings->FixedNumber, 0, NumElements);
		}
		else
		{
			NumOfElementsToKeep = FMath::CeilToInt(NumElements * FMath::Clamp(Settings->Ratio, 0, 1));
		}

		if (NumOfElementsToKeep == 0)
		{
			// We keep no entries, forward the input to the Discarded Entries, and create an empty data on chosen for parity.
			if (Settings->bOutputDiscardedEntries)
			{
				FPCGTaggedData& DiscardedOutput = Outputs.Add_GetRef(CurrentInput);
				DiscardedOutput.Pin = PCGRandomChoiceConstants::DiscardedEntriesLabel;
			}

			FPCGTaggedData& ChosenOutput = Outputs.Add_GetRef(CurrentInput);
			ChosenOutput.Data = ChooseFunc(CurrentInput.Data, {}, Context);
			ChosenOutput.Pin = PCGRandomChoiceConstants::ChosenEntriesLabel;
			continue;
		}
		else if (NumOfElementsToKeep == NumElements)
		{
			// We keep all the points, forward the input to the Chosen Points, and create an empty point data on discarded for parity.
			FPCGTaggedData& ChosenOutput = Outputs.Add_GetRef(CurrentInput);
			ChosenOutput.Pin = PCGRandomChoiceConstants::ChosenEntriesLabel;

			if (Settings->bOutputDiscardedEntries)
			{
				FPCGTaggedData& DiscardedOutput = Outputs.Add_GetRef(CurrentInput);
				DiscardedOutput.Data = ChooseFunc(CurrentInput.Data, {}, Context);
				DiscardedOutput.Pin = PCGRandomChoiceConstants::DiscardedEntriesLabel;
			}

			continue;
		}

		// TODO: While shuffling is the most intuitive way of selecting randomly points, it is still inefficient in memory, especially if we have a lot of points and want to select just a few of them.
		// Perhaps we could chose another algorithm in that case.
		// Like for example:
		// * Pick a number in [0, n[
		// * Pick a number in[0, n - 1[, then for each previously selected number, if it's larger, add +1
		// It's O(n) cpu + O(n) memory vs O(s^2) cpu + O(s) memory. (n = total number of points, s = number of points to keep)

		FRandomStream RandStream(Seed);
		TArray<int32> ShuffledIndexes;
		ShuffledIndexes.Reserve(NumElements);

		// Instead of swapping directly, we swap the indexes since it cheaper than copying points around
		for (int j = 0; j < NumElements; ++j)
		{
			ShuffledIndexes.Add(j);
		}

		// We only have to shuffle until we reached the number of elements to keep.
		for (int j = 0; j < NumOfElementsToKeep; ++j)
		{
			const int RandomElement = RandStream.RandRange(j, NumElements - 1);
			if (RandomElement != j)
			{
				Swap(ShuffledIndexes[j], ShuffledIndexes[RandomElement]);
			}
		}

		FPCGTaggedData& ChosenOutput = Outputs.Add_GetRef(CurrentInput);
		ChosenOutput.Data = ChooseFunc(CurrentInput.Data, MakeArrayView(ShuffledIndexes.GetData(), NumOfElementsToKeep), Context);
		ChosenOutput.Pin = PCGRandomChoiceConstants::ChosenEntriesLabel;

		if (Settings->bOutputDiscardedEntries)
		{
			FPCGTaggedData& DiscardedOutput = Outputs.Add_GetRef(CurrentInput);
			DiscardedOutput.Data = ChooseFunc(CurrentInput.Data, MakeArrayView(ShuffledIndexes.GetData() + NumOfElementsToKeep, NumElements - NumOfElementsToKeep), Context);
			DiscardedOutput.Pin = PCGRandomChoiceConstants::DiscardedEntriesLabel;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
