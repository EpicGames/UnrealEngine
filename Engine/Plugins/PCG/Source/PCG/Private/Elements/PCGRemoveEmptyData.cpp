// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGRemoveEmptyData.h"

#include "PCGContext.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGRemoveEmptyData)

#define LOCTEXT_NAMESPACE "PCGRemoveEmptyDataElement"

#if WITH_EDITOR
FName UPCGRemoveEmptyDataSettings::GetDefaultNodeName() const
{
	return FName(TEXT("RemoveEmptyData"));
}

FText UPCGRemoveEmptyDataSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Remove Empty Data");
}

FText UPCGRemoveEmptyDataSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Remove all data in the input pin that is empty.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGRemoveEmptyDataSettings::CreateElement() const
{
	return MakeShared<FPCGRemoveEmptyDataElement>();
}

TArray<FPCGPinProperties> UPCGRemoveEmptyDataSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any).SetRequiredPin();	
	return Properties;
}

TArray<FPCGPinProperties> UPCGRemoveEmptyDataSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);	
	return Properties;
}

bool FPCGRemoveEmptyDataElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRemoveEmptyDataElement::Execute);

	check(InContext);

	const UPCGRemoveEmptyDataSettings* Settings = InContext->GetInputSettings<UPCGRemoveEmptyDataSettings>();
	check(Settings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if (!Input.Data)
		{
			continue;
		}

		if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(Input.Data))
		{
			if (Proxy->GetNumElements() == FIntVector4::ZeroValue)
			{
				continue;
			}
		}
		else
		{
			// Get the keys on the index, if it doesn't exist or is empty, discard the data.
			const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index));
			if (!Keys || Keys->GetNum() == 0)
			{
				continue;
			}
		}

		InContext->OutputData.TaggedData.Add(Input);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
