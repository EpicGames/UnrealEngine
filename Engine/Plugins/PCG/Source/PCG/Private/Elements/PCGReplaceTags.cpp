// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGReplaceTags.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGData.h"
#include "PCGModule.h"
#include "PCGPin.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGReplaceTags)

#define LOCTEXT_NAMESPACE "PCGReplaceTagsElement"

#if WITH_EDITOR
void UPCGReplaceTagsSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DataVersion < FPCGCustomVersion::AttributesAndTagsCanContainSpaces)
	{
		bTokenizeOnWhiteSpace = true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Super::ApplyDeprecation(InOutNode);
}

FName UPCGReplaceTagsSettings::GetDefaultNodeName() const
{
	return TEXT("ReplaceTags");
}

FText UPCGReplaceTagsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Replace Tags");
}
#endif // WITH_EDITOR

FString UPCGReplaceTagsSettings::GetAdditionalTitleInformation() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FString> TagsToReplace = bTokenizeOnWhiteSpace
		? PCGHelpers::GetStringArrayFromCommaSeparatedString(SelectedTags)
		: PCGHelpers::GetStringArrayFromCommaSeparatedList(SelectedTags);
	const TArray<FString> TagsToProcess = bTokenizeOnWhiteSpace
		? PCGHelpers::GetStringArrayFromCommaSeparatedString(ReplacedTags)
		: PCGHelpers::GetStringArrayFromCommaSeparatedList(ReplacedTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const FText MultipleEntry = LOCTEXT("MultipleEntry", "Multiple");

	if(TagsToReplace.Num() >= 1 && TagsToProcess.Num() >= 1)
	{
		return FString::Printf(TEXT("%s -> %s"), (TagsToReplace.Num() > 1 ? *MultipleEntry.ToString() : *TagsToReplace[0]), (TagsToProcess.Num() > 1 ? *MultipleEntry.ToString() : *TagsToProcess[0]));
	}
	else
	{
		return FString();
	}
}

TArray<FPCGPinProperties> UPCGReplaceTagsSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGReplaceTagsSettings::CreateElement() const
{
	return MakeShared<FPCGReplaceTagsElement>();
}

bool FPCGReplaceTagsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGReplaceTagsElement::Execute);
	check(Context);

	const UPCGReplaceTagsSettings* Settings = Context->GetInputSettings<UPCGReplaceTagsSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FString> SelectedTags = Settings->bTokenizeOnWhiteSpace
		? PCGHelpers::GetStringArrayFromCommaSeparatedString(Settings->SelectedTags)
		: PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->SelectedTags);
	const TArray<FString> ReplacedTags = Settings->bTokenizeOnWhiteSpace
		? PCGHelpers::GetStringArrayFromCommaSeparatedString(Settings->ReplacedTags)
		: PCGHelpers::GetStringArrayFromCommaSeparatedList(Settings->ReplacedTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (SelectedTags.IsEmpty() && ReplacedTags.IsEmpty())
	{
		// Forward the input and early out
		Outputs = Inputs;
		PCGE_LOG(Verbose, LogOnly, LOCTEXT("InvalidNumberOfTags", "No selected or replaced tags, nothing to do."));
		return true;
	}

	if (ReplacedTags.Num() > 1 && SelectedTags.Num() != ReplacedTags.Num())
	{
		Outputs = Inputs;
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidMappingSupport", "Input only supports 1:1, N:1, and N:N mappings from source to replaced tags, Input data discarded."));
		return true;
	}

	const bool bNToNTagMapping = SelectedTags.Num() == ReplacedTags.Num();
	const bool bNToOneMapping = ReplacedTags.Num() == 1;
	
	for (const FPCGTaggedData& Input : Inputs)
	{
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		for (int i = 0; i < SelectedTags.Num(); ++i)
		{
			if (Input.Tags.Contains(SelectedTags[i]))
			{
				Output.Tags.Remove(SelectedTags[i]);

				if (bNToNTagMapping)
				{
					Output.Tags.Add(ReplacedTags[i]);
				}
				else if (bNToOneMapping)
				{
					Output.Tags.Add(ReplacedTags[0]);
				}
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
