// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGReroute.h"

#include "PCGContext.h"
#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "Compute/PCGDataDescription.h"
#include "Helpers/PCGConversion.h"
#include "Helpers/PCGSettingsHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGReroute)

#define LOCTEXT_NAMESPACE "PCGRerouteElement"

namespace PCGReroute::Constants::Conversion
{
	constexpr int32 RerouteToNamedDeclarationIndex = 0;
	constexpr int32 RerouteToNamedPairingIndex = 1;
	constexpr int32 NamedDeclarationToRerouteIndex = 0;
	constexpr int32 NamedUsageToRerouteIndex = 0;

	const FText ToRerouteLabel = LOCTEXT("ToRerouteLabel", "Reroute Node");
	const FText ToNamedDeclarationLabel = LOCTEXT("ToNamedDeclarationLabel", "Named Declaration");
	const FText ToNamedPairingLabel = LOCTEXT("ToNamedPairingLabel", "Named Pairing");

	const FText ToRerouteTooltip = LOCTEXT("ToRerouteTooltip", "Converts the named reroute to a normal reroute node.");
	const FText ToNamedDeclarationTooltip = LOCTEXT("ToNamedDeclarationTooltip", "Converts the reroute node to a named reroute declaration.");
	const FText ToNamedPairingTooltip = LOCTEXT("ToNamedPairingTooltip", "Converts the reroute node to a named reroute pair (declaration & usage).");

	const FName DefaultRerouteTitle(LOCTEXT("DefaultRerouteTitle", "Reroute").ToString());
	const FName DefaultNamedRerouteTitle(LOCTEXT("DefaultNamedRerouteTitle", "NamedReroute").ToString());
}

UPCGRerouteSettings::UPCGRerouteSettings()
{
#if WITH_EDITORONLY_DATA
	bExposeToLibrary = false;

	// Reroutes don't support disabling or debugging.
	bDisplayDebuggingProperties = false;
	DebugSettings.bDisplayProperties = false;
	bEnabled = true;
	bDebug = false;
#endif
}

FPCGDataTypeIdentifier UPCGRerouteSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	// Since we can have only a single input pin, just forward the type (with its potential subtype)
	if (InPin && InPin->IsOutputPin())
	{		
		const UPCGNode* Node = Cast<const UPCGNode>(GetOuter());
		const UPCGPin* FirstPin = Node && !Node->GetInputPins().IsEmpty() ? Node->GetInputPins()[0] : nullptr;

		if (const UPCGPin* OtherPin = (FirstPin && !FirstPin->Edges.IsEmpty() && FirstPin->Edges[0]) ? FirstPin->Edges[0]->InputPin : nullptr)
		{
			// Use the scope to avoid recomputing it multiple times
			PCGSettingsHelpers::FPinTypeScopeHelper PinTypeHelper;
			return PinTypeHelper.GetCurrentPinTypeID(OtherPin);
		}
	}
	
	return Super::GetCurrentPinTypesID(InPin);
}

TOptional<FName> UPCGRerouteSettings::GetCollisionFreeNodeName(const UPCGGraph* InGraph, FName BaseName)
{
	FName NamedRerouteTitle(BaseName);

	uint64 TitleIteration = 1;
	while (InGraph->FindNodeByTitleName(NamedRerouteTitle))
	{
		constexpr uint64 MaxIterationsBeforeAbort = 100;
		if (TitleIteration > MaxIterationsBeforeAbort)
		{
			return {};
		}

		NamedRerouteTitle.SetNumber(++TitleIteration);
	}

	return NamedRerouteTitle;
}

TArray<FPCGPinProperties> UPCGRerouteSettings::InputPinProperties() const
{
	FPCGPinProperties PinProperties;
	PinProperties.Label = PCGPinConstants::DefaultInputLabel;
	PinProperties.SetAllowMultipleConnections(false);
	PinProperties.AllowedTypes = EPCGDataType::Any;

	return { PinProperties };
}

TArray<FPCGPinProperties> UPCGRerouteSettings::OutputPinProperties() const
{
	FPCGPinProperties PinProperties;
	PinProperties.Label = PCGPinConstants::DefaultOutputLabel;
	PinProperties.AllowedTypes = EPCGDataType::Any;

	return { PinProperties };
}

FPCGElementPtr UPCGRerouteSettings::CreateElement() const
{
	return MakeShared<FPCGRerouteElement>();
}

#if WITH_EDITOR
TArray<FPCGPreconfiguredInfo> UPCGRerouteSettings::GetConversionInfo() const
{
	using namespace PCGReroute::Constants::Conversion;
	TArray<FPCGPreconfiguredInfo> ConversionInfo;

	ConversionInfo.Emplace(RerouteToNamedDeclarationIndex, ToNamedDeclarationLabel, ToNamedDeclarationTooltip);
	ConversionInfo.Emplace(RerouteToNamedPairingIndex, ToNamedPairingLabel, ToNamedPairingTooltip);

	return ConversionInfo;
}
#endif // WITH_EDITOR

bool UPCGRerouteSettings::ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo)
{
	using namespace PCGReroute::Constants::Conversion;
	UPCGNode* Node = CastChecked<UPCGNode>(GetOuter());
	UPCGGraph* Graph = CastChecked<UPCGGraph>(Node->GetOuter());

	switch (ConversionInfo.PreconfiguredIndex)
	{
		case RerouteToNamedDeclarationIndex:
		{
			if (TOptional<FName> NamedRerouteTitle = GetCollisionFreeNodeName(Graph, DefaultNamedRerouteTitle))
			{
				Node->NodeTitle = *NamedRerouteTitle;
			}
			else
			{
				return false;
			}

			FPCGSingleNodeConverter NodeConverter(Node, UPCGNamedRerouteDeclarationSettings::StaticClass());
			if (!NodeConverter.IsGraphInitialized() || !NodeConverter.IsSourceInitialized())
			{
				PCGLog::Settings::LogInvalidPreconfigurationWarning(ConversionInfo.PreconfiguredIndex, ToRerouteLabel);
				return false;
			}

			NodeConverter.PrepareData();
			NodeConverter.ApplyStructural();
			NodeConverter.Finalize();

			if (NodeConverter.IsComplete())
			{
				return true;
			}
			else
			{
				// Back out the title change
				Node->NodeTitle = NAME_None;
				return false;
			}
		}

		case RerouteToNamedPairingIndex:
		{
			TOptional<FName> NamedRerouteTitle = GetCollisionFreeNodeName(Graph, DefaultNamedRerouteTitle);
			if (!NamedRerouteTitle)
			{
				return false;
			}

			FPCGReroutePairNodeConverter NodeConverter(Node, *NamedRerouteTitle);
			if (!NodeConverter.IsGraphInitialized() || !NodeConverter.IsSourceInitialized())
			{
				PCGLog::Settings::LogInvalidPreconfigurationWarning(ConversionInfo.PreconfiguredIndex, ToRerouteLabel);
				return false;
			}

			NodeConverter.PrepareData();

			UPCGNamedRerouteDeclarationSettings* DeclarationSettings = CastChecked<UPCGNamedRerouteDeclarationSettings>(NodeConverter.GetGeneratedDeclarationSettings());
			// TODO: Intentionally handled here as it will be in the future, when the builder replaces the converter class. Remove this comment at that time.
			for (UPCGSettings* Settings : NodeConverter.GetGeneratedUsageSettings())
			{
				UPCGNamedRerouteUsageSettings* UsageSettings = CastChecked<UPCGNamedRerouteUsageSettings>(Settings);
				UsageSettings->Declaration = DeclarationSettings;
			}

			NodeConverter.ApplyStructural();
			NodeConverter.Finalize();

			return NodeConverter.IsComplete();
		}

		default:
			break;
	}

	return false;
}

UPCGNamedRerouteBaseSettings::UPCGNamedRerouteBaseSettings()
{
#if WITH_EDITORONLY_DATA
	// While reroutes don't support debugging, it makes sense for named reroutes to support it.
	bDisplayDebuggingProperties = true;
	DebugSettings.bDisplayProperties = true;
#endif
}

#if WITH_EDITOR
EPCGChangeType UPCGNamedRerouteBaseSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);
	
	// Reroute nodes are culled at the compilation, making them not debuggable.
	// If we want to debug we need to force the element to exist, and this means recompiling the graph.
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSettingsInterface, bDebug))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGNamedRerouteDeclarationSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	// Default visible pin
	PinProperties.Emplace(
		PCGPinConstants::DefaultOutputLabel, 
		EPCGDataType::Any, 
		/*bAllowMultipleConnections=*/true, 
		/*bAllowMultipleData=*/true);

	FPCGPinProperties& InvisiblePin = PinProperties.Emplace_GetRef(
		PCGNamedRerouteConstants::InvisiblePinLabel,
		EPCGDataType::Any,
		/*bAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true);
	InvisiblePin.bInvisiblePin = true;
	
	return PinProperties;
}

#if WITH_EDITOR
TArray<FPCGPreconfiguredInfo> UPCGNamedRerouteDeclarationSettings::GetConversionInfo() const
{
	using namespace PCGReroute::Constants::Conversion;
	TArray<FPCGPreconfiguredInfo> ConversionInfo;

	ConversionInfo.Emplace(NamedDeclarationToRerouteIndex, ToRerouteLabel, ToRerouteLabel);

	return ConversionInfo;
}
#endif // WITH_EDITOR

bool UPCGNamedRerouteDeclarationSettings::ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo)
{
	using namespace PCGReroute::Constants::Conversion;
	UPCGNode* Node = CastChecked<UPCGNode>(GetOuter());
	if (ConversionInfo.PreconfiguredIndex == NamedDeclarationToRerouteIndex)
	{
		FPCGRerouteDeclarationConverter NodeConverter(Node, DefaultRerouteTitle);
		if (!NodeConverter.IsGraphInitialized() || !NodeConverter.IsSourceInitialized())
		{
			PCGLog::Settings::LogInvalidPreconfigurationWarning(ConversionInfo.PreconfiguredIndex, ToNamedDeclarationLabel);
			return false;
		}

		NodeConverter.PrepareData();
		NodeConverter.ApplyStructural();
		NodeConverter.Finalize();

		return NodeConverter.IsComplete();
	}

	return false;
}

TArray<FPCGPinProperties> UPCGNamedRerouteUsageSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties = Super::InputPinProperties();
	if (ensure(PinProperties.Num() == 1))
	{
		PinProperties[0].bInvisiblePin = true;
	}

	return PinProperties;
}

FPCGDataTypeIdentifier UPCGNamedRerouteUsageSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	// Defer to declaration if possible
	return Declaration ? Declaration->GetCurrentPinTypesID(InPin) : Super::GetCurrentPinTypesID(InPin);
}

bool FPCGRerouteElement::ExecuteInternal(FPCGContext* Context) const
{
	check(Context);

	const UPCGSettings* Settings = Context->GetInputSettings<UPCGSettings>();
	check(Settings);
	
	// Reroute elements are culled during graph compilation unless they have no inbound edge or in debug.
	// If they have no inbound edge, this is a good place to log an error for user to deal with.
#if WITH_EDITOR
	if (!Settings->bDebug)
#endif
	{
		PCGE_LOG(Error, GraphAndLog, NSLOCTEXT("PCGRerouteSettings", "DetachedReroute", "Reroute is not linked to anything. Reconnect to recreate to fix the error."));
	}

	// Need to forward the output to the output pin, but also to the invisible pin in case of named reroute declaration node.
	TArray<FName, TInlineAllocator<2>> PinsToForwardTo = {PCGPinConstants::DefaultOutputLabel};
	if (Settings->IsA<UPCGNamedRerouteDeclarationSettings>())
	{
		PinsToForwardTo.Add(PCGNamedRerouteConstants::InvisiblePinLabel);
	}

	Context->OutputData.TaggedData.Reserve(Context->OutputData.TaggedData.Num() + PinsToForwardTo.Num() * Context->InputData.TaggedData.Num());

	for (const FName PinToForwardTo : PinsToForwardTo)
	{
		for (const FPCGTaggedData& Input : Context->InputData.TaggedData)
		{
			Context->OutputData.TaggedData.Emplace_GetRef(Input).Pin = PinToForwardTo;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
