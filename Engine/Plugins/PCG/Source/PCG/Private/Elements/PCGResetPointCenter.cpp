// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGResetPointCenter.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGResetPointCenter)

#define LOCTEXT_NAMESPACE "PCGResetPointCenterElement"

FPCGElementPtr UPCGResetPointCenterSettings::CreateElement() const
{
	return MakeShared<FPCGResetPointCenterElement>();
}

EPCGPointNativeProperties FPCGResetPointCenterElement::GetPropertiesToAllocate(FPCGContext* Context) const
{
	return EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax;
}

bool FPCGResetPointCenterElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGResetPointCenterElement::Execute);
	check(Context);

	FPCGResetPointCenterElement::ContextType* PointCenterContext = static_cast<FPCGResetPointCenterElement::ContextType*>(Context);

	const UPCGResetPointCenterSettings* Settings = Context->GetInputSettings<UPCGResetPointCenterSettings>();
	check(Settings);

	return ExecutePointOperation(PointCenterContext, [&PointCenterLocation = Settings->PointCenterLocation](const UPCGBasePointData* InputData, UPCGBasePointData* OutputData, int32 StartIndex, int32 Count)
	{
		TPCGValueRange<FTransform> TransformRange = OutputData->GetTransformValueRange();
		TPCGValueRange<FVector> BoundsMinRange = OutputData->GetBoundsMinValueRange();
		TPCGValueRange<FVector> BoundsMaxRange = OutputData->GetBoundsMaxValueRange();
		
		for (int32 Index = StartIndex; Index < (StartIndex + Count); ++Index)
		{
			PCGPointHelpers::ResetPointCenter(PointCenterLocation, TransformRange[Index], BoundsMinRange[Index], BoundsMaxRange[Index]);
		}

		return true;
	});
}

#undef LOCTEXT_NAMESPACE
