// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSampleTexture.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGTextureData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSampleTexture)

#define LOCTEXT_NAMESPACE "PCGSampleTextureElement"

TArray<FPCGPinProperties> UPCGSampleTextureSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	FPCGPinProperties& InputPin = PinProperties.Emplace_GetRef(PCGSampleTextureConstants::InputPointLabel,
		EPCGDataType::Point,
		/*bAllowMultipleConnections=*/true,
		/*bAllowMultipleData=*/true);
	InputPin.SetRequiredPin();

	PinProperties.Emplace(PCGSampleTextureConstants::InputTextureLabel,
		EPCGDataType::BaseTexture,
		/*bAllowMultipleConnections=*/false,
		/*bAllowMultipleData=*/false);

	return PinProperties;
}

FPCGElementPtr UPCGSampleTextureSettings::CreateElement() const
{
	return MakeShared<FPCGSampleTextureElement>();
}

bool FPCGSampleTextureElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSampleTextureElement::Execute);
	check(Context);

	const UPCGSampleTextureSettings* Settings = Context->GetInputSettings<UPCGSampleTextureSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> PointInputs = Context->InputData.GetInputsByPin(PCGSampleTextureConstants::InputPointLabel);
	const TArray<FPCGTaggedData> BaseTextureInput = Context->InputData.GetInputsByPin(PCGSampleTextureConstants::InputTextureLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	if (BaseTextureInput.Num() > 1)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("InvalidNumberOfTextureData", "Only 1 texture input is allowed."));
	}

	const UPCGTexture2DSingleBaseData* BaseTextureData = !BaseTextureInput.IsEmpty() ? Cast<UPCGTexture2DSingleBaseData>(BaseTextureInput[0].Data) : nullptr;
	if (!BaseTextureData)
	{
		return true;
	}

	auto DensityMergeFunc = PCGHelpers::GetDensityMergeFunction(Settings->DensityMergeFunction);

	for (int i = 0; i < PointInputs.Num(); ++i)
	{
		const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(PointInputs[i].Data);
		if (!PointData)
		{
			PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidPointData", "Point Input {0} is not point data."), i));
			continue;
		}

		TUniquePtr<const IPCGAttributeAccessor> InputAccessor = nullptr;
		TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = nullptr;

		if (Settings->TextureMappingMethod == EPCGTextureMappingMethod::UVCoordinates)
		{
			const FPCGAttributePropertyInputSelector UVSource = Settings->UVCoordinatesAttribute.CopyAndFixLast(PointData);

			InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, UVSource);
			InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, UVSource);
			if (!InputAccessor || !InputKeys)
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidUVAccessor", "Could not create coordinate accessor {0} for Point Input {1}."), FText::FromName(UVSource.GetName()), i));
				continue;
			}

			if (!PCG::Private::IsOfTypes<FVector, FVector2D>(InputAccessor->GetUnderlyingType()))
			{
				PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("InvalidAccessorType", "Accessor {0} must be of type Vector2 or Vector3"), FText::FromName(UVSource.GetName())));
				continue;
			}
		}

		UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(Context);

		FPCGInitializeFromDataParams InitializeFromDataParams(PointData);
		InitializeFromDataParams.bInheritSpatialData = false;
		OutPointData->InitializeFromDataWithParams(InitializeFromDataParams);

		auto InitializeFunc = [OutPointData, PointData, Settings]()
		{
			OutPointData->SetNumPoints(PointData->GetNumPoints());

			EPCGPointNativeProperties PropertiesToAllocate = PointData->GetAllocatedProperties();

			// This is based of the UPCGTexture2DSingleBaseData::SamplePoint code, we should probably add a virtual to UPCGSpatialData to return the sampled properties to better futurproof this code
			PropertiesToAllocate |= (EPCGPointNativeProperties::Color | EPCGPointNativeProperties::Density);
			if (Settings->TextureMappingMethod != EPCGTextureMappingMethod::UVCoordinates)
			{
				PropertiesToAllocate |= EPCGPointNativeProperties::Transform;
			}

			OutPointData->AllocateProperties(PropertiesToAllocate);
			OutPointData->CopyUnallocatedPropertiesFrom(PointData);
		};

		auto MoveDataRangeFunc = [OutPointData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
		{
			OutPointData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		};

		auto FinishedFunc = [OutPointData](int32 NumPoints)
		{
			OutPointData->SetNumPoints(NumPoints);
		};

		auto ProcessRangeFunc = [Settings, BaseTextureData, &InputAccessor, &InputKeys, PointData, &DensityMergeFunc, OutPointData](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			int32 NumWritten = 0;

			const FConstPCGPointValueRanges InRanges(PointData);
			FPCGPointValueRanges OutRanges(OutPointData, /*bAllocate=*/false);

			TArray<FVector> OutSampledPositions;
			OutSampledPositions.SetNumUninitialized(Count);
			if (Settings->TextureMappingMethod == EPCGTextureMappingMethod::UVCoordinates)
			{
				InputAccessor->GetRange<FVector>(OutSampledPositions, StartReadIndex, *InputKeys, EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible);
			}

			for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
			{
				const int32 WriteIndex = StartWriteIndex + NumWritten;

				if (Settings->TextureMappingMethod == EPCGTextureMappingMethod::UVCoordinates)
				{
					FVector& OutSamplePosition = OutSampledPositions[ReadIndex - StartReadIndex];
					
					if (Settings->TilingMode == EPCGTextureAddressMode::Clamp)
					{
						OutSamplePosition.X = FMath::Clamp(OutSamplePosition.X, 0.0, 1.0);
						OutSamplePosition.Y = FMath::Clamp(OutSamplePosition.Y, 0.0, 1.0);
					}

					float SampleDensity = 1.0f;
					FVector4 OutColor;
					if (BaseTextureData->SamplePointLocal(FVector2D(OutSamplePosition), OutColor, SampleDensity))
					{
						OutRanges.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);
						OutRanges.ColorRange[WriteIndex] = OutColor;

						float ComputedDensity = DensityMergeFunc(OutRanges.DensityRange[WriteIndex], SampleDensity);

						if (Settings->bClampOutputDensity)
						{
							ComputedDensity = FMath::Clamp(ComputedDensity, 0.0f, 1.0f);
						}

						OutRanges.DensityRange[WriteIndex] = ComputedDensity;
						++NumWritten;
					}
				}
				else
				{
					FPCGPoint OutPoint = InRanges.GetPoint(ReadIndex);
					if (BaseTextureData->SamplePoint(OutPoint.Transform, OutPoint.GetLocalBounds(), OutPoint, OutPointData->Metadata))
					{
						OutRanges.SetFromPoint(WriteIndex, OutPoint);

						float ComputedDensity = DensityMergeFunc(InRanges.DensityRange[ReadIndex], OutRanges.DensityRange[WriteIndex]);

						if (Settings->bClampOutputDensity)
						{
							ComputedDensity = FMath::Clamp(ComputedDensity, 0.0f, 1.0f);
						}

						OutRanges.DensityRange[WriteIndex] = ComputedDensity;
						++NumWritten;
					}
				}
			}

			return NumWritten;
		};

		FPCGTaggedData& Output = Outputs.Add_GetRef(PointInputs[i]);
		Output.Data = OutPointData;

		FPCGAsync::AsyncProcessingRangeEx(
			&Context->AsyncState,
			PointData->GetNumPoints(),
			InitializeFunc,
			ProcessRangeFunc,
			MoveDataRangeFunc,
			FinishedFunc,
			/*bEnableTimeSlicing=*/false);

	}

	return true;
}

#undef LOCTEXT_NAMESPACE
