// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSanityCheckPointData.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGPin.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSanityCheckPointData)

#define LOCTEXT_NAMESPACE "PCGSanityCheckPointDataElement"

	
TArray<FPCGPinProperties> UPCGSanityCheckPointDataSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSanityCheckPointDataSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);

	return PinProperties;
}

FPCGElementPtr UPCGSanityCheckPointDataSettings::CreateElement() const
{
	return MakeShared<FPCGSanityCheckPointDataElement>();
}

bool FPCGSanityCheckPointDataElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSanityCheckPointDataElement::Execute);

	const UPCGSanityCheckPointDataSettings* Settings = Context->GetInputSettings<UPCGSanityCheckPointDataSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Sources = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	int32 InputPointCount = 0;

	// first find the total Input bounds which will determine the size of each cell
	for (const FPCGTaggedData& Source : Sources) 
	{
		const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Source.Data);

		if (!PointData)
		{
			continue;
		}

		InputPointCount += PointData->GetNumPoints();
	}

	if (InputPointCount < Settings->MinPointCount)
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("NotEnoughPoints", "Expected at least {0} Points, found {1}"), Settings->MinPointCount, InputPointCount));

		if (Context->ExecutionSource.IsValid())
		{
			Context->ExecutionSource->GetExecutionState().Cancel();
		}
	}
	else if (InputPointCount > Settings->MaxPointCount)
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("TooManyPoints", "Expected less than {0} Points, found {1}"), Settings->MaxPointCount, InputPointCount));

		if (Context->ExecutionSource.IsValid())
		{
			Context->ExecutionSource->GetExecutionState().Cancel();
		}
	}
	else
	{
		// if valid, pass through data
		Context->OutputData = Context->InputData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
