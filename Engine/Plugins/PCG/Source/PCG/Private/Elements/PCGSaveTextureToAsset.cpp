// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSaveTextureToAsset.h"

#include "PCGAssetExporterUtils.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "PCGTextureReadback.h"
#include "Data/PCGRenderTargetData.h"
#include "Data/PCGTextureData.h"
#include "Metadata/PCGMetadata.h"

#include "PixelFormat.h"
#include "RHIStaticStates.h"
#include "Async/ParallelFor.h"
#include "Engine/Texture2D.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSaveTextureToAsset)

#define LOCTEXT_NAMESPACE "PCGSaveTextureToAsset"

namespace PCGSaveTextureToAssetConstants
{
	const FName OutAssetPathLabel = TEXT("OutAssetPath");
}

namespace PCGSaveTextureToAssetHelpers
{
	/** Maps an EPixelFormat to an ETextureSourceFormat.
	 * Returns TSF_Invalid if no direct mapping exists.
	 */
	ETextureSourceFormat GetTextureSourceFormat(EPixelFormat InFormat)
	{
		switch (InFormat)
		{
		case PF_G8:              return TSF_G8;
		case PF_B8G8R8A8:        return TSF_BGRA8;
		case PF_G16:             return TSF_G16;
		case PF_R16F:            return TSF_R16F;
		case PF_FloatRGBA:       return TSF_RGBA16F;
		case PF_R32_FLOAT:       return TSF_R32F;
		case PF_A32B32G32R32F:   return TSF_RGBA32F;
		default:                 return TSF_Invalid;
		}
	}

	/** Returns a readback-compatible EPixelFormat for the given input format.
	 * Prefers the input format when it supports UAV and has a direct ETextureSourceFormat mapping.
	 * Falls back to a format at matching precision and channel count otherwise.
	 */
	EPixelFormat GetReadbackFormat(EPixelFormat InFormat)
	{
		if (UE::PixelFormat::HasCapabilities(InFormat, EPixelFormatCapabilities::UAV) && GetTextureSourceFormat(InFormat) != TSF_Invalid)
		{
			return InFormat;
		}

		// Fallback: pick a format with matching precision. ETextureSourceFormat has no 2-channel entries, so 2/3-channel formats are promoted to 4-channel.
		const FPixelFormatInfo& FormatInfo = GPixelFormats[InFormat];
		const int32 BytesPerComponent = FormatInfo.BlockBytes / FMath::Max(FormatInfo.NumComponents, 1);

		if (BytesPerComponent >= 4)
		{
			// 32-bit
			return (FormatInfo.NumComponents <= 1) ? PF_R32_FLOAT : PF_A32B32G32R32F;
		}
		else if (BytesPerComponent >= 2 || IsFloatFormat(InFormat) || IsHDR(InFormat))
		{
			// 16-bit
			return (FormatInfo.NumComponents <= 1) ? PF_R16F : PF_FloatRGBA;
		}
		else
		{
			// 8-bit
			return (FormatInfo.NumComponents <= 1) ? PF_G8 : PF_B8G8R8A8;
		}
	}

	/** Sets compression settings on a texture to match its source format.
	 * TC_Default uses LDR compression which is lossy for HDR and grayscale formats.
	 */
	void ApplyCompressionSettings(UTexture2D* Texture, ETextureSourceFormat InFormat)
	{
		switch (InFormat)
		{
		// HDR formats: sRGB is forced off because the sRGB transfer function clamps to [0,1] and applies gamma, which is incompatible with linear float data.
		// Uncompressed formats are preferred because BC6H is RGB-only and discards the alpha channel.
		case TSF_RGBA16F:
			Texture->CompressionSettings = TC_HDR;
			Texture->SRGB = false;
			break;
		case TSF_RGBA32F:
			Texture->CompressionSettings = TC_HDR_F32;
			Texture->SRGB = false;
			break;
		case TSF_R16F:
			Texture->CompressionSettings = TC_HalfFloat;
			Texture->SRGB = false;
			break;
		case TSF_R32F:
			Texture->CompressionSettings = TC_SingleFloat;
			Texture->SRGB = false;
			break;
		case TSF_G8:
		case TSF_G16:
			Texture->CompressionSettings = TC_Grayscale;
			break;
		case TSF_BGRA8:
		default:
			Texture->CompressionSettings = TC_Default;
			break;
		}
	}
}

TArray<FPCGPinProperties> UPCGSaveTextureToAssetSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	FPCGPinProperties& InputTexturePinProperties = Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::BaseTexture, /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false);
	InputTexturePinProperties.SetRequiredPin();

	return Properties;
}

TArray<FPCGPinProperties> UPCGSaveTextureToAssetSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	FPCGPinProperties& Pin = Properties.Emplace_GetRef(PCGSaveTextureToAssetConstants::OutAssetPathLabel, EPCGDataType::Param, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/false);
	Pin.AllowedTypes.CustomSubtype = static_cast<int32>(EPCGMetadataTypes::SoftObjectPath);

	return Properties;
}

FPCGElementPtr UPCGSaveTextureToAssetSettings::CreateElement() const
{
	return MakeShared<FPCGSaveTextureToAssetElement>();
}

FPCGSaveTextureToAssetContext::~FPCGSaveTextureToAssetContext()
{
	FMemory::Free(RawReadbackData);
}

void FPCGSaveTextureToAssetContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ExportedTexture);
}

bool FPCGSaveTextureToAssetElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSaveTextureToAssetElement::Execute);
	check(InContext);

#if WITH_EDITOR
	FPCGSaveTextureToAssetContext* Context = static_cast<FPCGSaveTextureToAssetContext*>(InContext);

	const UPCGSaveTextureToAssetSettings* Settings = Context->GetInputSettings<UPCGSaveTextureToAssetSettings>();
	check(Settings);

	auto SleepUntilNextFrame = [Context]()
	{
		Context->bIsPaused = true;
		FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
		{
			if (TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin())
			{
				if (FPCGContext* ContextPtr = SharedHandle->GetContext())
				{
					ContextPtr->bIsPaused = false;
				}
			}
		});
	};

	if (!Context->InputTextureData)
	{
		const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

		if (Inputs.IsEmpty())
		{
			return true;
		}

		if (Inputs.Num() > 1)
		{
			PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGPinConstants::DefaultInputLabel, InContext);
		}

		Context->InputTextureData = Cast<const UPCGTexture2DSingleBaseData>(Inputs[0].Data);

		if (!Context->InputTextureData)
		{
			PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::BaseTexture, PCGPinConstants::DefaultInputLabel, InContext);
			return true;
		}
	}

	// Poll readback of input texture. Sleep each tick until readback is complete.
	if (!ReadbackInputTexture(Context))
	{
		SleepUntilNextFrame();
		return false;
	}

	if (!Context->ExportedTexture)
	{
#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("ExportingTexture", "Exporting texture asset from PCG"), Context->ExecutionSource.Get() && Context->ExecutionSource->GetExecutionState().UseTransactions());
#endif

		// Create the texture asset.
		UTexture2D* ExportedTexture = nullptr;

		UPCGAssetExporterUtils::CreateAsset(
			UTexture2D::StaticClass(),
			Settings->ExporterParams,
			[&ExportedTexture](const FString& PackagePath, UObject* Asset) -> bool
			{
				ExportedTexture = Cast<UTexture2D>(Asset);
				return ExportedTexture != nullptr;
			},
			Context);

		if (ExportedTexture)
		{
			// Initialize exported texture with the readback data in the format matching the input texture.
			const ETextureSourceFormat TexSrcFormat = PCGSaveTextureToAssetHelpers::GetTextureSourceFormat(Context->ReadbackFormat);

			ExportedTexture->Modify();
			ExportedTexture->Source.Init(Context->ReadbackWidth, Context->ReadbackHeight, /*NewNumSlices=*/1, /*NewNumMips=*/1, TexSrcFormat, Context->RawReadbackData);

			PCGSaveTextureToAssetHelpers::ApplyCompressionSettings(ExportedTexture, TexSrcFormat);

			ExportedTexture->UpdateResource();

			Context->ExportedTexture = ExportedTexture;
		}
		else
		{
			if (Settings->ExporterParams.AssetPath.IsEmpty() || Settings->ExporterParams.AssetName.IsEmpty())
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("CreateAssetFailedNoParams", "Failed to create texture asset. Invalid asset path or name was provided."), InContext);
			}
			else
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("CreateAssetFailedWithParams", "Failed to create texture asset at path '{0}' with name '{1}'."),
					FText::FromString(Settings->ExporterParams.AssetPath),
					FText::FromString(Settings->ExporterParams.AssetName)), InContext);
			}

			return true;
		}
	}

	if (Context->ExportedTexture->HasPendingInitOrStreaming())
	{
		SleepUntilNextFrame();
		return false;
	}

	// Create an attribute set output holding the exported texture asset path.
	UPCGParamData* OutputParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	OutputParamData->Metadata->CreateAttribute<FSoftObjectPath>(TEXT("AssetPath"), FSoftObjectPath(Context->ExportedTexture), /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	OutputParamData->Metadata->AddEntry();

	Context->OutputData.TaggedData.Emplace_GetRef().Data = OutputParamData;
#else
	PCGLog::LogErrorOnGraph(LOCTEXT("CannotExportInNonEditor", "Texture cannot be saved to an asset in non-editor builds"), InContext);
#endif // WITH_EDITOR

	return true;
}

bool FPCGSaveTextureToAssetElement::ReadbackInputTexture(FPCGSaveTextureToAssetContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSaveTextureToAssetElement::ReadbackInputTexture);

	// Only dispatch readback if it hasn't already been submitted.
	if (!InContext->bReadbackDispatched)
	{
		if (!InContext->InputTextureData)
		{
			return true;
		}

		if (InContext->InputTextureData->GetTextureResourceType() == EPCGTextureResourceType::TextureObject)
		{
			UTexture* InputTexture = InContext->InputTextureData->GetTexture();

			if (!InputTexture)
			{
				return true;
			}

			// Render target data already has a fully initialized GPU resource.
			// Calling UpdateResource() on it would re-create the resource and discard the content.
			if (!InContext->bUpdatedReadbackTextureResource && !InContext->InputTextureData->IsA<UPCGRenderTargetData>())
			{
				InputTexture->UpdateResource();

				InContext->bUpdatedReadbackTextureResource = true;
			}

			if (InputTexture->HasPendingInitOrStreaming())
			{
				return false;
			}
		}

		FTextureRHIRef TextureRHI = InContext->InputTextureData->GetTextureRHI();

		if (!ensure(TextureRHI))
		{
			// If the texture could not be acquired, readback cannot continue.
			return true;
		}

		const FIntVector TextureSize = TextureRHI->GetDesc().GetSize();

		FPCGTextureReadbackDispatchParams Params;
		Params.SourceTexture = TextureRHI;
		Params.SourceSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Params.SourceTextureIndex = InContext->InputTextureData->GetTextureSlice();
		Params.SourceDimensions = FIntPoint(TextureSize.X, TextureSize.Y);
		Params.OutputFormat = PCGSaveTextureToAssetHelpers::GetReadbackFormat(InContext->InputTextureData->GetFormat());

		InContext->ReadbackFormat = Params.OutputFormat;
		InContext->bReadbackDispatched = true;

		FPCGTextureReadbackInterface::Dispatch(Params, [ContextHandle = InContext->GetOrCreateHandle(), SourceDimensions = Params.SourceDimensions](void* OutBuffer, int32 ReadbackWidth, int32 ReadbackHeight)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSaveTextureToAssetElement::ReadbackInputTexture::DispatchCallback);

			TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin();
			FPCGSaveTextureToAssetContext* Context = SharedHandle ? static_cast<FPCGSaveTextureToAssetContext*>(SharedHandle->GetContext()) : nullptr;

			if (!Context)
			{
				return;
			}

			Context->ReadbackWidth = SourceDimensions.X;
			Context->ReadbackHeight = SourceDimensions.Y;

			const int32 BytesPerPixel = GPixelFormats[Context->ReadbackFormat].BlockBytes;
			const uint32 DstPitch = SourceDimensions.X * BytesPerPixel;
			const uint32 SrcPitch = ReadbackWidth * BytesPerPixel;
			const uint32 BufferSizeBytes = DstPitch * SourceDimensions.Y;
			Context->RawReadbackData = static_cast<uint8*>(FMemory::Malloc(BufferSizeBytes));

			if (ReadbackWidth == SourceDimensions.X)
			{
				FMemory::Memcpy(Context->RawReadbackData, OutBuffer, BufferSizeBytes);
			}
			else if (ensure(ReadbackWidth > SourceDimensions.X))
			{
				// Readback rows may be padded for GPU alignment (e.g. D3D12 256-byte row pitch).
				// Copy row-by-row, stripping the padding to produce a tightly packed buffer.
				ParallelFor(SourceDimensions.Y, [&](int32 Row)
				{
					FMemory::Memcpy(
						Context->RawReadbackData + Row * DstPitch,
						static_cast<uint8*>(OutBuffer) + Row * SrcPitch,
						DstPitch);
				});
			}
			else
			{
				FMemory::Memzero(Context->RawReadbackData, BufferSizeBytes);
			}

			Context->bReadbackComplete = true;
			Context->bIsPaused = false;
		});
	}

	return InContext->bReadbackComplete;
}

#undef LOCTEXT_NAMESPACE
