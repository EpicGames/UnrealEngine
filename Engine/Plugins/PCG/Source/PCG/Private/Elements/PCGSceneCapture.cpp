// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSceneCapture.h"

#include "Data/PCGRenderTargetData.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "EngineUtils.h"
#include "Algo/AnyOf.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSceneCapture)

#define LOCTEXT_NAMESPACE "PCGSceneCaptureElement"

namespace PCGSceneCaptureConstants
{
	const FName BoundingShapeLabel = TEXT("BoundingShape");
}

#if WITH_EDITOR
void UPCGSceneCaptureSettings::PostLoad()
{
	Super::PostLoad();

	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	const UPCGPin* BoundingShapePin = Node ? Node->GetInputPin(PCGSceneCaptureConstants::BoundingShapeLabel) : nullptr;

	if (BoundingShapePin && BoundingShapePin->IsConnected())
	{
		OrientationMode = EPCGSceneCaptureOrientationMode::FromBoundingShape;
	}
}
#endif // WITH_EDITOR

bool UPCGSceneCaptureSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	return OrientationMode == EPCGSceneCaptureOrientationMode::Explicit || (InPin
		&& (InPin->Properties.Label != GET_MEMBER_NAME_CHECKED(UPCGSceneCaptureSettings, CaptureLocation)
		&& InPin->Properties.Label != GET_MEMBER_NAME_CHECKED(UPCGSceneCaptureSettings, CaptureRotation)
		&& InPin->Properties.Label != GET_MEMBER_NAME_CHECKED(UPCGSceneCaptureSettings, CaptureHalfExtents)));
}

TArray<FPCGPinProperties> UPCGSceneCaptureSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	if (OrientationMode == EPCGSceneCaptureOrientationMode::FromBoundingShape)
	{
		FPCGPinProperties& BoundsPin = Properties.Emplace_GetRef(PCGSceneCaptureConstants::BoundingShapeLabel, EPCGDataType::Spatial);
		BoundsPin.SetRequiredPin();
#if WITH_EDITOR
		BoundsPin.Tooltip = LOCTEXT("BoundingShapePinTooltip", "Optional bounding shape to use instead of the PCG actor or explicit transform. Note, bounding shape overrides will always be an axis aligned bounding box with top down projection (orientation is ignored).");
#endif
	}

	return Properties;
}

TArray<FPCGPinProperties> UPCGSceneCaptureSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	FPCGPinProperties& Pin = Properties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::RenderTarget);
	Pin.bAllowMultipleData = false; // Must be set after pin properties constructed

	return Properties;
}

FPCGElementPtr UPCGSceneCaptureSettings::CreateElement() const
{
	return MakeShared<FPCGSceneCaptureElement>();
}

#if WITH_EDITOR
EPCGChangeType UPCGSceneCaptureSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSceneCaptureSettings, OrientationMode))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

void FPCGSceneCaptureContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SceneCaptureComponent);
	Collector.AddReferencedObject(RenderTarget);
}

bool FPCGSceneCaptureElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSceneCaptureElement::Execute);
	check(InContext);

	FPCGSceneCaptureContext* Context = static_cast<FPCGSceneCaptureContext*>(InContext);

	const UPCGSceneCaptureSettings* Settings = InContext->GetInputSettings<UPCGSceneCaptureSettings>();
	check(Settings);

	if (!Context->bSubmittedSceneCapture)
	{
		check(InContext->ExecutionSource.Get());

		const IPCGGraphExecutionState& ExecutionState = InContext->ExecutionSource->GetExecutionState();
		UWorld* World = ExecutionState.GetWorld();

		if (!World)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidWorld", "Invalid world. Cannot create USceneCaptureComponent."), InContext);
			return true;
		}

		if (Settings->TexelSize <= UE_DOUBLE_SMALL_NUMBER)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidTexelSize", "Invalid texel size {0}. Must be greater than zero."), Settings->TexelSize), InContext);
			return true;
		}

		// Default values in case Settings->OrientationMode == EPCGSceneCaptureOrientationMode::Explicit
		FVector CaptureLocation = Settings->CaptureLocation;
		FQuat CaptureRotation = Settings->CaptureRotation;
		FVector CaptureHalfExtents = Settings->CaptureHalfExtents;

		if (Settings->OrientationMode == EPCGSceneCaptureOrientationMode::FromBoundingShape)
		{
			bool bUnionWasCreated = false;

			if (const UPCGSpatialData* BoundingShape = Context->InputData.GetSpatialUnionOfInputsByPin(Context, PCGSceneCaptureConstants::BoundingShapeLabel, bUnionWasCreated))
			{
				BoundingShape->GetBounds().GetCenterAndExtents(CaptureLocation, CaptureHalfExtents);

				// Spatial data does not have a transform, so scene captures through an overridden bounds must be top down and axis aligned.
				CaptureRotation = FQuat::Identity;
			}
			else
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MissingBoundingShape", "No spatial data found on pin '{0}'."), FText::FromName(PCGSceneCaptureConstants::BoundingShapeLabel)), InContext);
				return true;
			}
		}
		else if (Settings->OrientationMode == EPCGSceneCaptureOrientationMode::FromExecutionSource)
		{
			const FBox OriginalActorBounds = ExecutionState.GetOriginalBounds();
			const FBox ActorBounds = ExecutionState.GetBounds();
			const FBox ExecutionBounds = OriginalActorBounds.Overlap(ActorBounds);
			const FTransform ActorTransform = ExecutionState.GetTransform();

			ExecutionBounds.GetCenterAndExtents(CaptureLocation, CaptureHalfExtents);
			CaptureRotation = ActorTransform.GetRotation();
		}

		if (CaptureHalfExtents.Size() <= UE_DOUBLE_SMALL_NUMBER)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidCaptureExtents", "Invalid capture extents. Volume must be greater than zero."), InContext);
			return true;
		}

		CaptureRotation = CaptureRotation.GetNormalized(); // Zero quat will return identity when normalized.
		const FQuat CaptureComponentRotation = FQuat(CaptureRotation.GetUpVector(), FMath::DegreesToRadians(-90.0f)) * FQuat(CaptureRotation.GetRightVector(), FMath::DegreesToRadians(90.0f)) * CaptureRotation;

		if (Settings->bCaptureFromTopOfExtents)
		{
			CaptureLocation += CaptureRotation.GetUpVector() * CaptureHalfExtents.Z;
		}

		const float OrthoWidth = FMath::Max(CaptureHalfExtents.X, CaptureHalfExtents.Y) * 2.0f;
		const int32 MaxTextureDimension = GetMax2DTextureDimension();
		const int32 RenderTargetDimensions = FMath::RoundToInt(OrthoWidth / Settings->TexelSize);

		Context->RenderTargetTransform = FTransform(CaptureRotation, CaptureLocation, FVector(OrthoWidth / 2.0, OrthoWidth / 2.0, CaptureHalfExtents.Z));

		if (RenderTargetDimensions <= 0 || RenderTargetDimensions > MaxTextureDimension)
		{
			PCGLog::LogErrorOnGraph(
				FText::Format(LOCTEXT("InvalidDimensions", "Invalid render target dimensions ({0}, {1}), must be between (1, 1) and ({2}, {2})."),
					RenderTargetDimensions,
					RenderTargetDimensions,
					MaxTextureDimension),
				InContext);

			return true;
		}

		// Initialize render target.
		Context->RenderTarget = NewObject<UTextureRenderTarget2D>();
		Context->RenderTarget->SizeX = RenderTargetDimensions;
		Context->RenderTarget->SizeY = RenderTargetDimensions;
		Context->RenderTarget->RenderTargetFormat = Settings->PixelFormat;
		Context->RenderTarget->ClearColor = FLinearColor::Black;
		Context->RenderTarget->UpdateResource();

		// Setup scene capture.
		// @todo_pcg: To improve perf we could avoid relying on a USceneCaptureComponent2D, and do the scene capture ourselves.
		Context->SceneCaptureComponent = NewObject<USceneCaptureComponent2D>(GetTransientPackage(), NAME_None, RF_Transient);
		Context->SceneCaptureComponent->bTickInEditor = false;
		Context->SceneCaptureComponent->SetComponentTickEnabled(false);
		Context->SceneCaptureComponent->SetVisibility(true);
		Context->SceneCaptureComponent->bCaptureEveryFrame = false;
		Context->SceneCaptureComponent->bCaptureOnMovement = false;
		Context->SceneCaptureComponent->TextureTarget = Context->RenderTarget;
		Context->SceneCaptureComponent->CaptureSource = Settings->CaptureSource;
		Context->SceneCaptureComponent->RegisterComponentWithWorld(World);
		Context->SceneCaptureComponent->ProjectionType = ECameraProjectionMode::Orthographic;
		Context->SceneCaptureComponent->OrthoWidth = OrthoWidth;
		Context->SceneCaptureComponent->SetWorldLocation(CaptureLocation);
		Context->SceneCaptureComponent->SetWorldRotation(CaptureComponentRotation);

		const bool bCaptureDepth = Settings->CaptureSource == ESceneCaptureSource::SCS_SceneColorSceneDepth || Settings->CaptureSource == ESceneCaptureSource::SCS_SceneDepth;
		const bool bUNorm = Settings->PixelFormat == ETextureRenderTargetFormat::RTF_R8
			|| Settings->PixelFormat == ETextureRenderTargetFormat::RTF_RG8
			|| Settings->PixelFormat == ETextureRenderTargetFormat::RTF_RGBA8
			|| Settings->PixelFormat == ETextureRenderTargetFormat::RTF_RGBA8_SRGB
			|| Settings->PixelFormat == ETextureRenderTargetFormat::RTF_RGB10A2;

		if (bCaptureDepth && bUNorm)
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("IncompatibleFormats", "Performing scene capture for '{0}', but render target format is '{1}'. Depth will be clamped to 1."),
				StaticEnum<ESceneCaptureSource>()->GetDisplayNameTextByValue((int64)Settings->CaptureSource),
				StaticEnum<ETextureRenderTargetFormat>()->GetDisplayNameTextByValue((int64)Settings->PixelFormat)));
		}

		// Gather actors and primitives for inclusion/exclusion filtering.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FilterPrimitivesForCapture);

			Context->SceneCaptureComponent->PrimitiveRenderMode = Settings->bOnlyCaptureContentMatchingIncludedTags ? ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList : ESceneCapturePrimitiveRenderMode::PRM_RenderScenePrimitives;

			for (TActorIterator<AActor> It(ExecutionState.GetWorld()); It; ++It)
			{
				AActor* Actor = *It;

				if (IsValid(Actor))
				{
					// Hide any actors marked for cleanup or exclusion.
					if (Actor->ActorHasTag(PCGHelpers::MarkedForCleanupPCGTag)
						|| (Settings->bExcludePCGContent && Actor->ActorHasTag(PCGHelpers::DefaultPCGActorTag))
						|| Algo::AnyOf(Settings->ExcludedTags, [Actor](const FName& InTag) { return Actor->ActorHasTag(InTag); }))
					{
						Context->SceneCaptureComponent->HiddenActors.Add(Actor);
					}
					// Show any actors marked for inclusion.
					else if (Settings->bOnlyCaptureContentMatchingIncludedTags && Algo::AnyOf(Settings->IncludedTags, [Actor](const FName& InTag) { return Actor->ActorHasTag(InTag); }))
					{
						Context->SceneCaptureComponent->ShowOnlyActors.Add(Actor);
					}

					TInlineComponentArray<UPrimitiveComponent*> Components(Actor);

					for (UPrimitiveComponent* Component : Components)
					{
						if (IsValid(Component))
						{
							// Hide any components marked for cleanup or exclusion.
							if (Component->ComponentHasTag(PCGHelpers::MarkedForCleanupPCGTag)
								|| (Settings->bExcludePCGContent && Component->ComponentHasTag(PCGHelpers::DefaultPCGTag))
								|| Algo::AnyOf(Settings->ExcludedTags, [Component](const FName& InTag) { return Component->ComponentHasTag(InTag); }))
							{
								Context->SceneCaptureComponent->HiddenComponents.Add(Component);
							}
							// Show any components marked for inclusion.
							else if (Settings->bOnlyCaptureContentMatchingIncludedTags && Algo::AnyOf(Settings->IncludedTags, [Component](const FName& InTag) { return Component->ComponentHasTag(InTag); }))
							{
								Context->SceneCaptureComponent->ShowOnlyComponents.Add(Component);
							}
						}
					}
				}
			}
		}

		// Perform scene capture.
		Context->SceneCaptureComponent->CaptureSceneDeferred();
		Context->bSubmittedSceneCapture = true;
		Context->bIsPaused = true;

		// Scene capture will be processed alongside the next render frame, so pause for one tick.
		FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
		{
			if (TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin())
			{
				if (FPCGContext* Context = SharedHandle->GetContext())
				{
					Context->bIsPaused = false;
				}
			}
		});

		return false;
	}

	// Cleanup scene capture.
	Context->SceneCaptureComponent->TextureTarget = nullptr;
	Context->SceneCaptureComponent->UnregisterComponent();

	// Initialize render target data.
	UPCGRenderTargetData* RenderTargetData = FPCGContext::NewObject_AnyThread<UPCGRenderTargetData>(InContext);
	RenderTargetData->Filter = Settings->Filter;
	RenderTargetData->Initialize(Context->RenderTarget, Context->RenderTargetTransform, /*bInTakeOwnershipOfRenderTarget=*/true);

	FPCGTaggedData& OutputData = InContext->OutputData.TaggedData.Emplace_GetRef();
	OutputData.Data = RenderTargetData;

	return true;
}

#undef LOCTEXT_NAMESPACE
