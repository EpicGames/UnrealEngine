// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSelectPoints.h"

#include "PCGContext.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"

#include "Math/RandomStream.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSelectPoints)

#define LOCTEXT_NAMESPACE "PCGSelectPointsElement"

#if WITH_EDITOR
FText UPCGSelectPointsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Selects a stable random subset of the input points.");
}
#endif

FPCGElementPtr UPCGSelectPointsSettings::CreateElement() const
{
	return MakeShared<FPCGSelectPointsElement>();
}

bool FPCGSelectPointsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelectPointsElement::Execute);
	// TODO: make time-sliced implementation
	const UPCGSelectPointsSettings* Settings = Context->GetInputSettings<UPCGSelectPointsSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const float Ratio = Settings->Ratio;
#if WITH_EDITOR
	const bool bKeepZeroDensityPoints = Settings->bKeepZeroDensityPoints;
#else
	const bool bKeepZeroDensityPoints = false;
#endif

	const int Seed = Context->GetSeed();

	const bool bNoSampling = (Ratio <= 0.0f);
	const bool bTrivialSampling = (Ratio >= 1.0f);

	// Early exit when nothing will be generated out of this sampler
	if (bNoSampling && !bKeepZeroDensityPoints)
	{
		PCGE_LOG(Verbose, LogOnly, LOCTEXT("AllInputsRejected", "Skipped - all inputs rejected"));
		return true;
	}

	// TODO: embarassingly parallel loop
	for (const FPCGTaggedData& Input : Inputs)
	{
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		if (!Input.Data || Cast<UPCGSpatialData>(Input.Data) == nullptr)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data"));
			continue;
		}

		// Skip processing if the transformation would be trivial
		if (bTrivialSampling)
		{
			PCGE_LOG(Verbose, LogOnly, LOCTEXT("SkippedTrivialSampling", "Skipped - trivial sampling"));
			continue;
		}

		const UPCGBasePointData* OriginalData = Cast<UPCGSpatialData>(Input.Data)->ToBasePointData(Context);

		if (!OriginalData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("NoPointDataInInput", "Unable to get point data from input"));
			continue;
		}
				
		const int OriginalPointCount = OriginalData->GetNumPoints();

		UPCGBasePointData* SampledData = FPCGContext::NewPointData_AnyThread(Context);
		
		FPCGInitializeFromDataParams InitializeFromDataParams(OriginalData);
		InitializeFromDataParams.bInheritSpatialData = false;
		
		SampledData->InitializeFromDataWithParams(InitializeFromDataParams);
		
		Output.Data = SampledData;

		// TODO: randomize on the fractional number of points
#if WITH_EDITOR
		int TargetNumPoints = (bKeepZeroDensityPoints ? OriginalPointCount : OriginalPointCount * Ratio);
#else
		int TargetNumPoints = OriginalPointCount * Ratio;
#endif

		// Early out
		if (TargetNumPoints == 0)
		{
			PCGE_LOG(Verbose, LogOnly, LOCTEXT("SkippedAllPointsRejected", "Skipped - all points rejected"));
			continue;
		}
		else
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelectPointsElement::Execute::SelectPoints);

			auto InitializeFunc = [SampledData, OriginalData]()
			{
				SampledData->SetNumPoints(OriginalData->GetNumPoints(), /*bInitializeValues=*/false);
				SampledData->AllocateProperties(OriginalData->GetAllocatedProperties() | EPCGPointNativeProperties::Density);
				SampledData->CopyUnallocatedPropertiesFrom(OriginalData);
			};

			auto AsyncProcessRangeFunc = [SampledData, OriginalData, Seed, Ratio, bKeepZeroDensityPoints](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
			{
				int32 NumWritten = 0;

				// @todo_pcg: could probably build a list of indices and then use CopyPoints instead of writing this boiler plate code
				FPCGPointValueRanges SampledRanges(SampledData, /*bAllocate=*/false);
				const FConstPCGPointValueRanges OriginalRanges(OriginalData);
								
				for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
				{
					const int32 WriteIndex = StartWriteIndex + NumWritten;
					const int32 OriginalSeed = OriginalRanges.SeedRange[ReadIndex];

					// Apply a high-pass filter based on selected ratio
					FRandomStream RandomSource(PCGHelpers::ComputeSeed(Seed, OriginalSeed));
					float Chance = RandomSource.FRand();

					if (Chance < Ratio)
					{
						SampledRanges.SetFromValueRanges(WriteIndex, OriginalRanges, ReadIndex);
						SampledRanges.DensityRange[WriteIndex] = OriginalRanges.DensityRange[ReadIndex];
						++NumWritten;
					}
#if WITH_EDITOR
					else if (bKeepZeroDensityPoints)
					{
						SampledRanges.SetFromValueRanges(WriteIndex, OriginalRanges, ReadIndex);
						SampledRanges.DensityRange[WriteIndex] = 0.f;
						++NumWritten;
					}
#endif
				}
							
				return NumWritten;
			};

			auto MoveDataRangeFunc = [SampledData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
			{
				SampledData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
			};

			auto FinishedFunc = [SampledData](int32 NumWritten)
			{
				SampledData->SetNumPoints(NumWritten);
			};

			FPCGAsyncState* AsyncState = Context ? &Context->AsyncState : nullptr;
			const bool bDone = FPCGAsync::AsyncProcessingRangeEx(
				AsyncState,
				OriginalPointCount,
				InitializeFunc,
				AsyncProcessRangeFunc,
				MoveDataRangeFunc,
				FinishedFunc,
				/*bEnableTimeSlicing=*/false);

			PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("GenerationInfo", "Generated {0} points from {1} source points"), SampledData->GetNumPoints(), OriginalPointCount));
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
