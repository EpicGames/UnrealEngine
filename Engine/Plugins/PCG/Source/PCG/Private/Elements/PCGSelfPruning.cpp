// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSelfPruning.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGAttributeExtractor.h"

#include "CollisionShape.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/ImplicitObject.h"
#include "PhysicsEngine/BodyInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSelfPruning)

#define LOCTEXT_NAMESPACE "PCGSelfPruningElement"

namespace PCGSelfPruningAlgorithms
{
	bool RandomSort(const UPCGBasePointData* PointData, const PCGPointOctree::FPointRef& A, const PCGPointOctree::FPointRef& B)
	{
		const int32 SeedA = PointData->GetSeed(A.Index);
		const int32 SeedB = PointData->GetSeed(B.Index);

		return SeedA < SeedB;
	}

	bool SortSmallToLargeNoRandom(const UPCGBasePointData* PointData, const PCGPointOctree::FPointRef& A, const PCGPointOctree::FPointRef& B, FVector::FReal SquaredRadiusEquality)
	{
		return A.Bounds.BoxExtent.SquaredLength() * SquaredRadiusEquality < B.Bounds.BoxExtent.SquaredLength();
	}

	bool SortSmallToLargeWithRandom(const UPCGBasePointData* PointData, const PCGPointOctree::FPointRef& A, const PCGPointOctree::FPointRef& B, FVector::FReal SquaredRadiusEquality)
	{
		const FVector::FReal SqrLenA = A.Bounds.BoxExtent.SquaredLength();
		const FVector::FReal SqrLenB = B.Bounds.BoxExtent.SquaredLength();
		if (SqrLenA * SquaredRadiusEquality < SqrLenB)
		{
			return true;
		}
		else if (SqrLenB * SquaredRadiusEquality < SqrLenA)
		{
			return false;
		}
		else
		{
			return RandomSort(PointData, A, B);
		}
	}

	/**
	* Keep Extents sort alive since it is a bit faster than the SortByAttribute.
	*/
	void SortExtents(const UPCGBasePointData* PointData, TArray<PCGPointOctree::FPointRef>& SortedPointRefs, const FPCGSelfPruningParameters& Parameters, FVector::FReal SquaredRadiusEquality)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelfPruningElement::Execute::SortExtents);

		if (Parameters.PruningType == EPCGSelfPruningType::LargeToSmall)
		{
			if (Parameters.bRandomizedPruning)
			{
				Algo::Sort(SortedPointRefs, [PointData, SquaredRadiusEquality](const PCGPointOctree::FPointRef& A, const PCGPointOctree::FPointRef& B) { return !PCGSelfPruningAlgorithms::SortSmallToLargeWithRandom(PointData, A, B, SquaredRadiusEquality); });
			}
			else
			{
				Algo::Sort(SortedPointRefs, [PointData, SquaredRadiusEquality](const PCGPointOctree::FPointRef& A, const PCGPointOctree::FPointRef& B) { return !PCGSelfPruningAlgorithms::SortSmallToLargeNoRandom(PointData, A, B, SquaredRadiusEquality); });
			}
		}
		else //if (Parameters.PruningType == EPCGSelfPruningType::SmallToLarge)
		{
			if (Parameters.bRandomizedPruning)
			{
				Algo::Sort(SortedPointRefs, [PointData, SquaredRadiusEquality](const PCGPointOctree::FPointRef& A, const PCGPointOctree::FPointRef& B) { return PCGSelfPruningAlgorithms::SortSmallToLargeWithRandom(PointData, A, B, SquaredRadiusEquality); });
			}
			else
			{
				Algo::Sort(SortedPointRefs, [PointData, SquaredRadiusEquality](const PCGPointOctree::FPointRef& A, const PCGPointOctree::FPointRef& B) { return PCGSelfPruningAlgorithms::SortSmallToLargeNoRandom(PointData, A, B, SquaredRadiusEquality); });
			}
		}
	}
}

namespace PCGSelfPruningElement
{
	void FPointBitSet::Initialize(int32 NumPoints)
	{
		Bits.SetNumZeroed(NumPoints / 32 + 1);
	}

	void FPointBitSet::Add(int32 Index)
	{
		const uint32 BitsIndex = Index / 32;
		const uint32 Bit = Index % 32;

		Bits[BitsIndex] |= (1 << Bit);
	}

	bool FPointBitSet::Contains(int32 Index)
	{
		const uint32 BitsIndex = Index / 32;
		const uint32 Bit = Index % 32;

		return (Bits[BitsIndex] & (1 << Bit)) != 0;
	}

	static constexpr int32 TimeSliceFrequencyCheck = 255;

	bool ShouldStop(FPCGContext* InOptionalContext)
	{
		return InOptionalContext && InOptionalContext->TimeSliceIsEnabled() && InOptionalContext->ShouldStop();
	}

	bool DensityBoundsExclusion(FIterationState& IterationState, FPCGContext* InOptionalContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelfPruningElement::Execute::DensityBoundsExclusion);

		check(IterationState.InputData);
		const PCGPointOctree::FPointOctree& Octree = IterationState.InputData->GetPointOctree();

		int32 CheckTimeSlicingCount = 0;

		while (IterationState.CurrentPointIndex < IterationState.SortedPointRefs.Num())
		{
			// Don't check too many times. Pre-increment will make sure we always at least process "TimeSliceFrequencyCheck" elements at each iteration.
			if (++CheckTimeSlicingCount >= TimeSliceFrequencyCheck)
			{
				if (ShouldStop(InOptionalContext))
				{
					return false;
				}

				CheckTimeSlicingCount = 0;
			}

			const PCGPointOctree::FPointRef& PointRef = IterationState.SortedPointRefs[IterationState.CurrentPointIndex++];
			if (IterationState.ExcludedPoints.Contains(PointRef.Index))
			{
				continue;
			}

			IterationState.ExclusionPoints.Add(PointRef.Index);

			Octree.FindElementsWithBoundsTest(FBoxCenterAndExtent(PointRef.Bounds.Origin, PointRef.Bounds.BoxExtent), [&IterationState](const PCGPointOctree::FPointRef& InPointRef)
			{
				// TODO: check on an oriented-box basis?
				if (!IterationState.ExclusionPoints.Contains(InPointRef.Index))
				{
					IterationState.ExcludedPoints.Add(InPointRef.Index);
				}
			});
		}

		return true;
	}

	/** Self-pruning driven by use of collision shapes. Implementation is in practice just a secondary step after the octree query to filter out points if their collisions don`t intersect. */
	bool CollisionExclusion(FIterationState& IterationState, FPCGContext* InOptionalContext, EPCGCollisionQueryFlag InCollisionQueryFlag)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelfPruningElement::Execute::CollisionExclusion);

		check(IterationState.InputData);
		const PCGPointOctree::FPointOctree& Octree = IterationState.InputData->GetPointOctree();
		const TConstPCGValueRange<FTransform> TransformRange = IterationState.InputData->GetConstTransformValueRange();

		int32 CheckTimeSlicingCount = 0;

		TArray<const PCGPointOctree::FPointRef*, TInlineAllocator<256>> ElementsToTest;

		auto SetupQueryInfo = [&IterationState, &TransformRange](const PCGPointOctree::FPointRef& PointRef, FBodyInstance* OtherBodyInstance, EPCGCollisionQueryFlag CollisionQueryFlag, FBodyInstance*& OutInstance, PhysicsInterfaceTypes::FInlineShapeArray& OutShapes, FCollisionShape& OutSimpleShape)
		{
			const int32 EntryIndex = PointRef.Index;
			OutInstance = IterationState.CollisionWrapper.GetBodyInstance(EntryIndex);
			bool bHasComplexShapes = false;

			if (OutInstance)
			{
				if (OutInstance == OtherBodyInstance)
				{
					if (FBodyInstance** TemporaryInstance = IterationState.TemporaryBodyInstances.Find(OtherBodyInstance))
					{
						OutInstance = *TemporaryInstance;
					}
					else
					{
						check(OtherBodyInstance->GetBodySetup());
						OutInstance = new FBodyInstance();
						OutInstance->bAutoWeld = false;
						OutInstance->bSimulatePhysics = false;
						OutInstance->InitBody(OtherBodyInstance->GetBodySetup(), FTransform::Identity, nullptr, nullptr);

						IterationState.TemporaryBodyInstances.Add(OtherBodyInstance, OutInstance);
					}
				}

				OutInstance->UpdateBodyScale(TransformRange[PointRef.Index].GetScale3D());
				const bool bFirstChoice = FPCGCollisionWrapper::GetShapeArray(OutInstance, CollisionQueryFlag, OutShapes);

				if (OutShapes.IsEmpty())
				{
					OutInstance = nullptr;
				}
				else
				{
					bHasComplexShapes = (CollisionQueryFlag == EPCGCollisionQueryFlag::Complex) ||
						(CollisionQueryFlag == EPCGCollisionQueryFlag::ComplexFirst && bFirstChoice) ||
						(CollisionQueryFlag == EPCGCollisionQueryFlag::SimpleFirst && !bFirstChoice);

					// Special identification step: in some instances (when users mark their complex collision as simple...) it's possible that we get in a case here
					// where simple collisions contain triangle meshes, which are going to break the tests below as the Physics API does not allow casting/testing for overlaps
					// on triangle mesh - triangle mesh cases.
					if (!bHasComplexShapes)
					{
						for (const FPhysicsShapeHandle& Shape : OutShapes)
						{
							FPhysicsGeometryCollection Collection = FPhysicsInterface::GetGeometryCollection(Shape);
							const Chaos::FImplicitObject& ShapeGeom = Collection.GetGeometry();

							if (ShapeGeom.IsUnderlyingMesh())
							{
								bHasComplexShapes = true;
								break;
							}
						}
					}

					// Special verification step - if at this point we had asked *specifically* for simple shapes, we must discard triangle meshes if we had found some.
					if (CollisionQueryFlag == EPCGCollisionQueryFlag::Simple && bHasComplexShapes)
					{
						OutInstance = nullptr;
						bHasComplexShapes = false;
					}
				}
			}

			if (!OutInstance)
			{
				OutShapes.Reset();
				OutSimpleShape.SetBox(FVector3f(PointRef.Bounds.BoxExtent));
			}

			return bHasComplexShapes;
		};

		while (IterationState.CurrentPointIndex < IterationState.SortedPointRefs.Num())
		{
			// Don't check too many times. Pre-increment will make sure we always at least process "TimeSliceFrequencyCheck" elements at each iteration.
			if (++CheckTimeSlicingCount >= TimeSliceFrequencyCheck)
			{
				if (ShouldStop(InOptionalContext))
				{
					return false;
				}

				CheckTimeSlicingCount = 0;
			}

			const PCGPointOctree::FPointRef& PointRef = IterationState.SortedPointRefs[IterationState.CurrentPointIndex++];
			if (IterationState.ExcludedPoints.Contains(PointRef.Index))
			{
				continue; // Point discarded from previous iteration
			}

			// Select point
			IterationState.ExclusionPoints.Add(PointRef.Index);

			// 1. Gather point refs to test against - similar to the DensityBoundsExclusion, except we write to an array temporarily
			ElementsToTest.Reset();
			Octree.FindElementsWithBoundsTest(FBoxCenterAndExtent(PointRef.Bounds.Origin, PointRef.Bounds.BoxExtent), [&IterationState, &ElementsToTest](const PCGPointOctree::FPointRef& OtherPointRef)
			{
				if (!IterationState.ExclusionPoints.Contains(OtherPointRef.Index) && !IterationState.ExcludedPoints.Contains(OtherPointRef.Index))
				{
					ElementsToTest.Add(&OtherPointRef);
				}
			});

			// For perf reasons, we won't do the body instance setup (scale...) if there's nothing to test against
			if (ElementsToTest.IsEmpty())
			{
				continue;
			}

			// Implementation note: this is a deconstruction of FBodyInstance::OverlapTestForBodiesImpl
			FBodyInstance* ThisInstance = nullptr;
			PhysicsInterfaceTypes::FInlineShapeArray ThisShapes;
			FCollisionShape ThisSimpleShape;

			FBodyInstance* OtherInstance = nullptr;
			PhysicsInterfaceTypes::FInlineShapeArray OtherShapes;
			FCollisionShape OtherSimpleShape;

			const FTransform& PointTransform = TransformRange[PointRef.Index];
			const bool bThisHasComplexShapes = SetupQueryInfo(PointRef, nullptr, InCollisionQueryFlag, ThisInstance, ThisShapes, ThisSimpleShape);
			FTransform TransformNoScale = FTransform(PointTransform.GetRotation(), PointTransform.GetLocation());

			// We must force the other collision flag to simple if we have complex shapes in the leading shape here, because complex-complex overlaps aren't supported.
			EPCGCollisionQueryFlag OtherCollisionQueryFlag = (bThisHasComplexShapes ? EPCGCollisionQueryFlag::Simple : InCollisionQueryFlag);

			for (const PCGPointOctree::FPointRef* OtherPointRef : ElementsToTest)
			{
				bool bOverlaps = false;

				bool bOtherHasComplexShapes = SetupQueryInfo(*OtherPointRef, ThisInstance, OtherCollisionQueryFlag, OtherInstance, OtherShapes, OtherSimpleShape);

				const FTransform& OtherPointTransform = TransformRange[OtherPointRef->Index];
				FTransform OtherTransformNoScale = FTransform(OtherPointTransform.GetRotation(), OtherPointTransform.GetLocation());

				// Four cases here:
				// 1 - shapes vs shapes
				if (!ThisShapes.IsEmpty() && !OtherShapes.IsEmpty())
				{
					auto CheckForOverlap = [](const PhysicsInterfaceTypes::FInlineShapeArray& ComplexShapes, const FTransform& ComplexShapesTransform, const PhysicsInterfaceTypes::FInlineShapeArray& SimpleShapes, const FTransform& SimpleShapesTransform)
					{
						FTransform RelativeTransform = SimpleShapesTransform.GetRelativeTransform(ComplexShapesTransform);

						for (const FPhysicsShapeHandle& ComplexShape : ComplexShapes)
						{
							for (const FPhysicsShapeHandle& SimpleShape : SimpleShapes)
							{
								FPhysicsGeometryCollection SimpleShapeCollection = FPhysicsInterface::GetGeometryCollection(SimpleShape);
								const Chaos::FImplicitObject& SimpleShapeGeom = SimpleShapeCollection.GetGeometry();

								if (Chaos::Utilities::CastHelper(SimpleShapeGeom, RelativeTransform, [&ComplexShape](const auto& Downcast, const auto& FullGeomTransform) { return Chaos::OverlapQuery(*ComplexShape.Shape->GetGeometry(), FTransform::Identity, Downcast, FullGeomTransform); }))
								{
									return true;
								}
							}
						}

						return false;
					};

					if (!bOtherHasComplexShapes)
					{
						bOverlaps = CheckForOverlap(ThisShapes, TransformNoScale, OtherShapes, OtherTransformNoScale);
					}
					else
					{
						bOverlaps = CheckForOverlap(OtherShapes, OtherTransformNoScale, ThisShapes, TransformNoScale);
					}
				}
				// 2 - shapes vs simple shape
				// 3 - simple shape vs shapes
				else if (!ThisShapes.IsEmpty() || !OtherShapes.IsEmpty())
				{
					auto CheckForOverlap = [](const PhysicsInterfaceTypes::FInlineShapeArray& Shapes, const FTransform& ShapesTransform, const FCollisionShape& CollShape, const FTransform& CollTransform)
					{
						FTransform RelativeTransform = CollTransform.GetRelativeTransform(ShapesTransform);
						FPhysicsShapeAdapter CollAdapter(RelativeTransform.GetRotation(), CollShape);
						const FPhysicsGeometry& Geom = CollAdapter.GetGeometry();
						FTransform GeomTransform = CollAdapter.GetGeomPose(RelativeTransform.GetLocation());

						for (const FPhysicsShapeHandle& Shape : Shapes)
						{
							if (Chaos::Utilities::CastHelper(Geom, GeomTransform, [&Shape](const auto& Downcast, const auto& FullGeomTransform) { return Chaos::OverlapQuery(*Shape.Shape->GetGeometry(), FTransform::Identity, Downcast, FullGeomTransform); }))
							{
								return true;
							}
						}

						return false;
					};

					if(!ThisShapes.IsEmpty())
					{
						bOverlaps = CheckForOverlap(ThisShapes, TransformNoScale, OtherSimpleShape, OtherTransformNoScale);
					}
					else
					{
						check(!OtherShapes.IsEmpty());
						bOverlaps = CheckForOverlap(OtherShapes, OtherTransformNoScale, ThisSimpleShape, TransformNoScale);
					}
				}
				// 4 - simple shape vs simple shape <- default case.
				else
				{
					FTransform RelativeTransform = OtherTransformNoScale.GetRelativeTransform(TransformNoScale);
					FPhysicsShapeAdapter ThisAdapter(FQuat::Identity, ThisSimpleShape);
					FPhysicsShapeAdapter OtherAdapter(RelativeTransform.GetRotation(), OtherSimpleShape);

					bOverlaps = Chaos::Utilities::CastHelper(OtherAdapter.GetGeometry(), OtherAdapter.GetGeomPose(RelativeTransform.GetTranslation()), [&ThisAdapter](const auto& Downcast, const auto& FullGeomTransform) { return Chaos::OverlapQuery(ThisAdapter.GetGeometry(), ThisAdapter.GetGeomPose(FVector::ZeroVector), Downcast, FullGeomTransform, /*Thickness=*/0); });
				}

				if (bOverlaps)
				{
					IterationState.ExcludedPoints.Add(OtherPointRef->Index);
				}
			}
		}

		// Clean up temporary instances
		if (IterationState.CurrentPointIndex == IterationState.SortedPointRefs.Num())
		{
			for (TPair<FBodyInstance*, FBodyInstance*>& TemporaryInstance : IterationState.TemporaryBodyInstances)
			{
				delete TemporaryInstance.Value;
			}
			IterationState.TemporaryBodyInstances.Reset();
		}

		return true;
	}

	bool DuplicatePointsExclusion(FIterationState& IterationState, FPCGContext* InOptionalContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelfPruningElement::Execute::DuplicatePointsExclusion);

		check(IterationState.InputData);
		const PCGPointOctree::FPointOctree& Octree = IterationState.InputData->GetPointOctree();

		int32 CheckTimeSlicingCount = 0;

		const TConstPCGValueRange<FTransform> TransformRange = IterationState.InputData->GetConstTransformValueRange();
		const TConstPCGValueRange<FVector> BoundsMinRange = IterationState.InputData->GetConstBoundsMinValueRange();
		const TConstPCGValueRange<FVector> BoundsMaxRange = IterationState.InputData->GetConstBoundsMaxValueRange();

		while (IterationState.CurrentPointIndex < IterationState.SortedPointRefs.Num())
		{
			// Don't check too many times. Pre-increment will make sure we always at least process "TimeSliceFrequencyCheck" elements at each iteration.
			if (++CheckTimeSlicingCount >= TimeSliceFrequencyCheck)
			{
				if (ShouldStop(InOptionalContext))
				{
					return false;
				}

				CheckTimeSlicingCount = 0;
			}

			const PCGPointOctree::FPointRef& PointRef = IterationState.SortedPointRefs[IterationState.CurrentPointIndex++];
			if (IterationState.ExcludedPoints.Contains(PointRef.Index))
			{
				continue;
			}

			IterationState.ExclusionPoints.Add(PointRef.Index);
			const FTransform& PointTransform = TransformRange[PointRef.Index];
			const FVector PointLocalCenter = PCGPointHelpers::GetLocalCenter(BoundsMinRange[PointRef.Index], BoundsMaxRange[PointRef.Index]);

			Octree.FindElementsWithBoundsTest(FBoxCenterAndExtent(PointTransform.TransformPosition(PointLocalCenter), FVector::Zero()), [&IterationState, &PointTransform, &TransformRange](const PCGPointOctree::FPointRef InPointRef)
			{
				const FTransform& OtherPointTransform = TransformRange[InPointRef.Index];
				if ((PointTransform.GetLocation() - OtherPointTransform.GetLocation()).SquaredLength() <= SMALL_NUMBER && !IterationState.ExclusionPoints.Contains(InPointRef.Index))
				{
					IterationState.ExcludedPoints.Add(InPointRef.Index);
				}
			});
		}

		return true;
	}

	void Execute(FPCGContext* Context, EPCGSelfPruningType PruningType, float RadiusSimilarityFactor, bool bRandomizedPruning)
	{
		FPCGSelfPruningParameters Parameters;
		Parameters.PruningType = PruningType;
		Parameters.RadiusSimilarityFactor = RadiusSimilarityFactor;
		Parameters.bRandomizedPruning = bRandomizedPruning;
		Parameters.ComparisonSource.SetPointProperty(EPCGPointProperties::Extents);
		return Execute(Context, Parameters);
	}

	void Execute(FPCGContext* Context, const FPCGSelfPruningParameters& InParameters)
	{
		check(Context);

		// Early out: if pruning is disabled
		if (InParameters.PruningType == EPCGSelfPruningType::None)
		{
			Context->OutputData = Context->InputData;
			PCGE_LOG_C(Verbose, LogOnly, Context, LOCTEXT("TypeNotSpecified", "Skipped - Type is None"));
			return;
		}

		TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
		for (const FPCGTaggedData& Input : Inputs)
		{
			const UPCGSpatialData* SpatialInput = Cast<UPCGSpatialData>(Input.Data);

			if (!SpatialInput)
			{
				PCGE_LOG_C(Error, GraphAndLog, Context, LOCTEXT("InvalidInputData", "Invalid input data"));
				continue;
			}

			const UPCGBasePointData* InputData = SpatialInput->ToBasePointData(Context);
			FIterationState IterationState;
			IterationState.InputData = InputData;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			IterationState.InputPointData = Cast<UPCGPointData>(InputData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			while (!ExecuteSlice(IterationState, InParameters, Context)) {}

			FPCGTaggedData& Output = Context->OutputData.TaggedData.Add_GetRef(Input);
			if (IterationState.OutputData != nullptr)
			{
				Output.Data = IterationState.OutputData;
			}
		}
	}

	bool ExecuteSlice(FIterationState& InState, const FPCGSelfPruningParameters& InParameters, FPCGContext* InOptionalContext)
	{
		// Early out: if pruning is disabled
		if (InParameters.PruningType == EPCGSelfPruningType::None)
		{
			if (InOptionalContext)
			{
				PCGE_LOG_C(Verbose, LogOnly, InOptionalContext, LOCTEXT("TypeNotSpecified", "Skipped - Type is None"));
			}
			return true;
		}

		const FVector::FReal RadiusEquality = 1.0f + InParameters.RadiusSimilarityFactor;
		const FVector::FReal SquaredRadiusEquality = FMath::Square(RadiusEquality);
		const int32 InputNumPoints = InState.InputData->GetNumPoints();

		// Force octree computation, and check if we need to stop after that if it was dirty in the first place.
		const bool bOctreeWasDirty = InState.InputData->IsPointOctreeDirty();
		const PCGPointOctree::FPointOctree& Octree = InState.InputData->GetPointOctree();
		if (bOctreeWasDirty && ShouldStop(InOptionalContext))
		{
			return false;
		}

		// Self-pruning will be done as follows:
		// For each point:
		//  if in its vicinity, there is >=1 non-rejected point with a radius significantly larger
		//  or in its range + has a randomly assigned index -> we'll look at its seed
		//  then remove this point
		if (!InState.bSortDone)
		{
			// In the case of the collision-driven self-pruning, we have to populate the sorted points array earlier since we're playing with the bounds.
			InState.SortedPointRefs.Empty();
			InState.SortedPointRefs.Reserve(InputNumPoints);

			const TConstPCGValueRange<FTransform> TransformRange = InState.InputData->GetConstTransformValueRange();
			const TConstPCGValueRange<FVector> BoundsMinRange = InState.InputData->GetConstBoundsMinValueRange();
			const TConstPCGValueRange<FVector> BoundsMaxRange = InState.InputData->GetConstBoundsMaxValueRange();
			const TConstPCGValueRange<float> SteepnessRange = InState.InputData->GetConstSteepnessValueRange();
			const TConstPCGValueRange<int32> SeedRange = InState.InputData->GetConstSeedValueRange();

			for (int32 PointIndex = 0; PointIndex < InputNumPoints; ++PointIndex)
			{
				InState.SortedPointRefs.Emplace(PCGPointOctree::FPointRef(PointIndex, PCGPointHelpers::GetDensityBounds(TransformRange[PointIndex], SteepnessRange[PointIndex], BoundsMinRange[PointIndex], BoundsMaxRange[PointIndex])));
			}

			FPCGAttributePropertySelector ComparisonSource = InParameters.ComparisonSource.CopyAndFixLast(InState.InputData);

			// First sort the points if we need sorting. 
			// Randomize for other cases than LargeToSmall and SmallToLarge are just a seed sort, so special case for that.
			// Special case for extents too, since the SortByAttribute is slightly slower (~10%) due to the overhead of working with accessors.
			// To not impact existing graphs (and since it is already one of the slowest operations in the graph), we will keep the old sorting directly on the extents.
			// Note that sorting is not time sliced.
			// TODO: Might be a good thing to have a future there.
			bool bSortAttribute = InParameters.PruningType == EPCGSelfPruningType::LargeToSmall || InParameters.PruningType == EPCGSelfPruningType::SmallToLarge;
			if (InParameters.bRandomizedPruning && !bSortAttribute)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelfPruningElement::Execute::SortingSeeds);
				Algo::Sort(InState.SortedPointRefs, [&InState](const PCGPointOctree::FPointRef& A, const PCGPointOctree::FPointRef& B) { return PCGSelfPruningAlgorithms::RandomSort(InState.InputData, A, B); });
			}
			else if (ComparisonSource.GetSelection() == EPCGAttributePropertySelection::Property && ComparisonSource.GetPointProperty() == EPCGPointProperties::Extents && ComparisonSource.GetExtraNames().IsEmpty())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelfPruningElement::Execute::SortingExtents);
				PCGSelfPruningAlgorithms::SortExtents(InState.InputData, InState.SortedPointRefs, InParameters, SquaredRadiusEquality);
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelfPruningElement::Execute::SortingGeneric);

				const bool bAscending = InParameters.PruningType != EPCGSelfPruningType::LargeToSmall;

				TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InState.InputData, ComparisonSource);
				TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InState.InputData, FPCGAttributePropertyInputSelector());

				if (!InputAccessor.IsValid())
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidAttribute", "Attribute/Property '{0}' was not found."), ComparisonSource.GetDisplayText()), InOptionalContext);
					return true;
				}

				// For vector attributes, collapse them to their square length for comparison (was the previous behavior with extents).
				if (PCG::Private::IsOfTypes<FVector2D, FVector, FVector4>(InputAccessor->GetUnderlyingType()))
				{
					// Force squared length extraction on vectors
					ComparisonSource.GetExtraNamesMutable().Add(PCGAttributeExtractorConstants::VectorSquaredLength.ToString());
					InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InState.InputData, ComparisonSource);
				}

				if (!PCG::Private::IsOfTypes<int32, int64, float, double, uint8>(InputAccessor->GetUnderlyingType()))
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NotNumericAttribute", "Attribute/Property '{0}' is not numeric ({1})"), ComparisonSource.GetDisplayText(), PCG::Private::GetTypeNameText(InputAccessor->GetUnderlyingType())), InOptionalContext);
					return true;
				}

				if (InParameters.bRandomizedPruning)
				{
					auto CompareLessWithRandom = [SquaredRadiusEquality, SeedRange](const auto& A, const auto& B, int32 IndexA, int32 IndexB, bool)
					{
						static_assert(std::is_same_v<decltype(A), decltype(B)>);

						using ValueType = std::remove_const_t<std::remove_reference_t<decltype(A)>>;

						if constexpr (std::is_arithmetic_v<ValueType>)
						{
							ValueType ACopy = static_cast<ValueType>(static_cast<double>(A) * SquaredRadiusEquality);
							if (PCG::Private::MetadataTraits<ValueType>::Less(ACopy, B))
							{
								return true;
							}

							ValueType BCopy = static_cast<ValueType>(static_cast<double>(B) * SquaredRadiusEquality);
							if (PCG::Private::MetadataTraits<ValueType>::Less(BCopy, A))
							{
								return false;
							}
						}
						else
						{
							if (PCG::Private::MetadataTraits<ValueType>::Less(A, B))
							{
								return true;
							}
							else if (PCG::Private::MetadataTraits<ValueType>::Less(B, A))
							{
								return false;
							}
						}

						return SeedRange[IndexA] < SeedRange[IndexB];
					};

					PCGAttributeAccessorHelpers::SortByAttribute(*InputAccessor, *InputKeys, InState.SortedPointRefs, bAscending, PCGAttributeAccessorHelpers::Private::DefaultIndexGetter, std::move(CompareLessWithRandom));
				}
				else
				{
					auto CompareLess = [SquaredRadiusEquality](const auto& A, const auto& B, int32, int32, bool)
					{
						static_assert(std::is_same_v<decltype(A), decltype(B)>);

						using ValueType = std::remove_const_t<std::remove_reference_t<decltype(A)>>;
						using CompareType = std::conditional_t<std::is_arithmetic_v<ValueType>, ValueType, const ValueType&>;

						CompareType ACopy = A;

						if constexpr (std::is_arithmetic_v<ValueType>)
						{
							ACopy = static_cast<ValueType>(static_cast<double>(A) * SquaredRadiusEquality);
						}

						return PCG::Private::MetadataTraits<ValueType>::Less(ACopy, B);
					};

					PCGAttributeAccessorHelpers::SortByAttribute(*InputAccessor, *InputKeys, InState.SortedPointRefs, bAscending, PCGAttributeAccessorHelpers::Private::DefaultIndexGetter, std::move(CompareLess));
				}
			}

			InState.ExclusionPoints.Initialize(InputNumPoints);
			InState.ExcludedPoints.Initialize(InputNumPoints);

			InState.bSortDone = true;

			// After sorting, allow ourselves to stop if needed.
			if (ShouldStop(InOptionalContext))
			{
				return false;
			}
		}

		// Find excluded/duplicate points. Time sliced.
		const bool bIsDuplicateTest = (InParameters.PruningType == EPCGSelfPruningType::RemoveDuplicates);
		bool bIsDone = false;
		if (bIsDuplicateTest)
		{
			bIsDone = PCGSelfPruningElement::DuplicatePointsExclusion(InState, InOptionalContext);
		}
		else if (InParameters.bUseCollisionAttribute)
		{
			bIsDone = PCGSelfPruningElement::CollisionExclusion(InState, InOptionalContext, InParameters.CollisionQueryFlag);
		}
		else
		{
			bIsDone = PCGSelfPruningElement::DensityBoundsExclusion(InState, InOptionalContext);
		}

		// Finally, output all points that are present in the ExclusionPoints. This part is not time sliced, should it be too?
		if (bIsDone)
		{
			UPCGBasePointData* PrunedData = FPCGContext::NewPointData_AnyThread(InOptionalContext);

			FPCGInitializeFromDataParams InitializeFromDataParams(InState.InputData);
			InitializeFromDataParams.bInheritSpatialData = false;

			PrunedData->InitializeFromDataWithParams(InitializeFromDataParams);

			InState.OutputData = PrunedData;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			InState.OutputPointData = Cast<UPCGPointData>(PrunedData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			
			TArray<int32> Indices;
			Indices.Reserve(InputNumPoints);
			for (int32 PointIndex = 0; PointIndex < InputNumPoints; ++PointIndex)
			{
				if (InState.ExclusionPoints.Contains(PointIndex))
				{
					Indices.Add(PointIndex);
				}
			}

			if (Indices.Num() > 0)
			{
				UPCGBasePointData::SetPoints(InState.InputData, PrunedData, Indices, /*bCopyAll=*/false);
			}

			if (InOptionalContext)
			{
				if (bIsDuplicateTest)
				{
					PCGE_LOG_C(Verbose, LogOnly, InOptionalContext, FText::Format(LOCTEXT("GenerationInfoDuplicate", "Removed {0} duplicate points from {1} source points"), InputNumPoints - PrunedData->GetNumPoints(), InputNumPoints));
				}
				else
				{
					PCGE_LOG_C(Verbose, LogOnly, InOptionalContext, FText::Format(LOCTEXT("GenerationInfo", "Generated {0} points from {1} source points"), PrunedData->GetNumPoints(), InputNumPoints));
				}
			}
		}

		return bIsDone;
	}
}

void FPCGSelfPruningParameters::PostLoad()
{
#if WITH_EDITOR
	if (bUseComplexCollision_DEPRECATED)
	{
		CollisionQueryFlag = EPCGCollisionQueryFlag::Complex;
		bUseComplexCollision_DEPRECATED = false;
	}
#endif
}

UPCGSelfPruningSettings::UPCGSelfPruningSettings()
{
	// Previous Default behavior was Extents
	Parameters.ComparisonSource.SetPointProperty(EPCGPointProperties::Extents);
}

void UPCGSelfPruningSettings::PostLoad()
{
	Super::PostLoad();

	Parameters.PostLoad();

#if WITH_EDITOR
	if (PruningType_DEPRECATED != EPCGSelfPruningType::LargeToSmall)
	{
		Parameters.PruningType = PruningType_DEPRECATED;
		PruningType_DEPRECATED = EPCGSelfPruningType::LargeToSmall;
	}

	if (RadiusSimilarityFactor_DEPRECATED != 0.25f)
	{
		Parameters.RadiusSimilarityFactor = RadiusSimilarityFactor_DEPRECATED;
		RadiusSimilarityFactor_DEPRECATED = 0.25;
	}

	if (!bRandomizedPruning_DEPRECATED)
	{
		Parameters.bRandomizedPruning = bRandomizedPruning_DEPRECATED;
		bRandomizedPruning_DEPRECATED = true;
	}
#endif // WITH_EDITOR
}

FPCGElementPtr UPCGSelfPruningSettings::CreateElement() const
{
	return MakeShared<FPCGSelfPruningElement>();
}

bool FPCGSelfPruningElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelfPruningElement::PrepareDataInternal);

	FPCGSelfPruningElement::ContextType* TimeSlicedContext = static_cast<FPCGSelfPruningElement::ContextType*>(InContext);
	check(TimeSlicedContext);

	const UPCGSelfPruningSettings* Settings = TimeSlicedContext->GetInputSettings<UPCGSelfPruningSettings>();
	check(Settings);

	TimeSlicedContext->SetTimeSliceIsEnabled(true);

	TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	// No global execution state.
	EPCGTimeSliceInitResult InitResult = TimeSlicedContext->InitializePerExecutionState();

	TimeSlicedContext->InitializePerIterationStates(Inputs.Num(), [&Inputs, Settings, InContext](PCGSelfPruningElement::FIterationState& OutState, const FPCGSelfPruningElement::ExecStateType&, int32 Index) -> EPCGTimeSliceInitResult
	{
		const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Inputs[Index].Data);
		if (!SpatialData)
		{
			PCGE_LOG_C(Error, GraphAndLog, InContext, FText::Format(LOCTEXT("InvalidInputDataType", "Input {0}: Input data must be of type Spatial"), FText::AsNumber(Index)));
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Retro-compatibility
		OutState.InputData = SpatialData->ToBasePointData(InContext);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OutState.InputPointData = Cast<UPCGPointData>(OutState.InputData);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (!OutState.InputData)
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (Settings->Parameters.PruningType != EPCGSelfPruningType::RemoveDuplicates && Settings->Parameters.bUseCollisionAttribute)
		{
			FPCGAttributePropertyInputSelector InputSelector = Settings->Parameters.CollisionAttribute.CopyAndFixLast(OutState.InputData);

			TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OutState.InputData, InputSelector);
			TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(OutState.InputData, InputSelector);

			TArray<FSoftObjectPath> Meshes;
			if (OutState.CollisionWrapper.Prepare(InputAccessor.Get(), InputKeys.Get(), Meshes))
			{
				OutState.CollisionWrapper.CreateBodyInstances(Meshes);
			}
		}

		return EPCGTimeSliceInitResult::Success;
	});

	return true;
}

bool FPCGSelfPruningElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSelfPruningElement::Execute);

	const UPCGSelfPruningSettings* Settings = Context->GetInputSettings<UPCGSelfPruningSettings>();
	check(Settings);

	FPCGSelfPruningElement::ContextType* TimeSlicedContext = static_cast<FPCGSelfPruningElement::ContextType*>(Context);
	check(TimeSlicedContext);

	// Prepare data failed, no need to execute. Return an empty output
	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		TimeSlicedContext->OutputData.TaggedData.Empty();
		return true;
	}

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	return ExecuteSlice(TimeSlicedContext, [Settings, &Inputs](FPCGSelfPruningElement::ContextType* Context, const FPCGSelfPruningElement::ExecStateType&, PCGSelfPruningElement::FIterationState& IterState, const uint32 IterationIndex) -> bool
	{
		const bool bIsDone = PCGSelfPruningElement::ExecuteSlice(IterState, Settings->Parameters, Context);

		if (bIsDone)
		{
			FPCGTaggedData& Output = Context->OutputData.TaggedData.Add_GetRef(Inputs[IterationIndex]);
			if (IterState.OutputData != nullptr)
			{
				Output.Data = IterState.OutputData;
			}
		}

		return bIsDone;
	});
}

bool FPCGSelfPruningElement::CanExecuteOnlyOnMainThread(FPCGContext* Context) const
{
	if (const UPCGSelfPruningSettings* Settings = (Context ? Context->GetInputSettings<UPCGSelfPruningSettings>() : nullptr))
	{
		return Settings->Parameters.bUseCollisionAttribute;
	}
	else
	{
		return false;
	}
}

#undef LOCTEXT_NAMESPACE
