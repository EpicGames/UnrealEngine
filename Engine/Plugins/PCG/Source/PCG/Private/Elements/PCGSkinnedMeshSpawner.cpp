// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSkinnedMeshSpawner.h"

#include "PCGCustomVersion.h"
#include "PCGManagedResource.h"
#include "PCGManagedResourceContainer.h"
#include "Compute/PCGKernelHelpers.h"
#include "Compute/BuiltInKernels/PCGCountUniqueAttributeValuesKernel.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGSkinnedMeshSpawnerContext.h"
#include "Elements/PCGSkinnedMeshSpawnerKernel.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "InstanceDataPackers/PCGSkinnedMeshInstanceDataPackerBase.h"
#include "MeshSelectors/PCGSkinnedMeshSelector.h"

#include "Components/InstancedSkinnedMeshComponent.h"
#include "Engine/SkinnedAsset.h"
#include "Materials/MaterialInterface.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSkinnedMeshSpawner)

#define LOCTEXT_NAMESPACE "PCGSkinnedMeshSpawnerElement"

UPCGSkinnedMeshSpawnerSettings::UPCGSkinnedMeshSpawnerSettings(const FObjectInitializer &ObjectInitializer)
{
	// Implementation note: this should not have been done here (it should have been null), as it causes issues with copy & paste
	// when the thing to paste does not have that class for its instance.
	// However, removing it makes it that any object actually using the instance created by default would be lost.
	if (!this->HasAnyFlags(RF_ClassDefaultObject))
	{
		MeshSelectorParameters = ObjectInitializer.CreateDefaultSubobject<UPCGSkinnedMeshSelector>(this, TEXT("DefaultSelectorInstance"));
	}
}

#if WITH_EDITOR
void UPCGSkinnedMeshSpawnerSettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);

	const bool bProduceOutputPoints = !InNode || InNode->IsOutputPinConnected(PCGPinConstants::DefaultOutputLabel) || bIsInspecting || bDebug;

	if (!bProduceOutputPoints)
	{
		CreateParams.NodeOutputPinsToWire.Empty();
	}

	UPCGSkinnedMeshSpawnerKernel* SpawnerKernel = PCGKernelHelpers::CreateKernel<UPCGSkinnedMeshSpawnerKernel>(InOutContext, CreateParams, OutKernels, OutEdges);
	SpawnerKernel->SetProduceOutputPoints(bProduceOutputPoints);

	// If doing by-attribute selection, add analysis kernel that will count how many instances of each mesh are present.
	if (const UPCGSkinnedMeshSelector* Selector = Cast<UPCGSkinnedMeshSelector>(MeshSelectorParameters); ensure(Selector))
	{
		// Don't wire count kernel to node output pin, wire manually to the spawner kernel below.
		CreateParams.NodeOutputPinsToWire.Empty();

		UPCGCountUniqueAttributeValuesKernel* CountKernel = PCGKernelHelpers::CreateKernel<UPCGCountUniqueAttributeValuesKernel>(InOutContext, CreateParams, OutKernels, OutEdges);
		CountKernel->SetAttributeName(MeshSelectorParameters->MeshAttribute.GetName());
		// We operate across all input data rather than spawning for each input data separately.
		CountKernel->SetEmitPerDataCounts(false);
		CountKernel->SetMaxNumUniqueValuesSource(EPCGMaxNumUniqueValuesSource::StringKeyValues);

		OutEdges.Emplace(FPCGPinReference(CountKernel, PCGPinConstants::DefaultOutputLabel), FPCGPinReference(SpawnerKernel, PCGSkinnedMeshSpawnerConstants::InstanceCountsPinLabel));
	}
}

FText UPCGSkinnedMeshSpawnerSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Instanced Skinned Mesh Spawner");
}

void UPCGSkinnedMeshSpawnerSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	check(InOutNode);
	Super::ApplyDeprecation(InOutNode);
}
#endif

TArray<FPCGPinProperties> UPCGSkinnedMeshSpawnerSettings::InputPinProperties() const
{
	// Note: If executing on the GPU, we need to prevent multiple connections on inputs, since it is not supported at this time.
	// Also note: Since the ShouldExecuteOnGPU() is already tied to structural changes, we don't need to implement any logic for this in GetChangeTypeForProperty()
	const bool bAllowMultipleConnections = !ShouldExecuteOnGPU();

	TArray<FPCGPinProperties> Properties;
	FPCGPinProperties& InputPinProperty = Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point, bAllowMultipleConnections);
	InputPinProperty.SetRequiredPin();

	return Properties;
}

FPCGElementPtr UPCGSkinnedMeshSpawnerSettings::CreateElement() const
{
	return MakeShared<FPCGSkinnedMeshSpawnerElement>();
}

FPCGContext* FPCGSkinnedMeshSpawnerElement::CreateContext()
{
	return new FPCGSkinnedMeshSpawnerContext();
}

bool FPCGSkinnedMeshSpawnerElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSkinnedMeshSpawnerElement::PrepareDataInternal);
	// TODO : time-sliced implementation
	FPCGSkinnedMeshSpawnerContext* Context = static_cast<FPCGSkinnedMeshSpawnerContext*>(InContext);
	const UPCGSkinnedMeshSpawnerSettings* Settings = Context->GetInputSettings<UPCGSkinnedMeshSpawnerSettings>();
	check(Settings);

	if (!Settings->MeshSelectorParameters)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidMeshSelectorInstance", "Invalid MeshSelector instance, try reselecting the MeshSelector type"));
		return true;
	}

	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	FPCGManagedResourceContainerHelper ManagedResourcesContainerHelper(ExecutionSource);

#if WITH_EDITOR
	// In editor, we always want to generate this data for inspection & to prevent caching issues
	const bool bGenerateOutput = true;
#else
	const bool bGenerateOutput = Context->Node && Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultOutputLabel);
#endif

	// Check if we can reuse existing resources
	bool& bSkippedDueToReuse = Context->bSkippedDueToReuse;

	if (!Context->bReuseCheckDone)
	{
		// Compute CRC if it has not been computed (it likely isn't, but this is to futureproof this)
		if (!Context->DependenciesCrc.IsValid())
		{
			FPCGDataCollection TempCollection;
			const FPCGDataCollection* InputDataCollectionPtr = &Context->InputData;

			// If the collection didn't have any Data crc information yet, we need to compute it, but can't do it on the input directly.
			if (Context->InputData.DataCrcs.Num() != Context->InputData.TaggedData.Num())
			{
				TempCollection = Context->InputData;
				TempCollection.ComputeCrcs(ShouldComputeFullOutputDataCrc(Context));
				InputDataCollectionPtr = &TempCollection;
			}

			GetDependenciesCrc(FPCGGetDependenciesCrcParams(InputDataCollectionPtr, Settings, Context->ExecutionSource.Get()), Context->DependenciesCrc);
		}
		
		if (Context->DependenciesCrc.IsValid() && ManagedResourcesContainerHelper.IsValid())
		{
			TArray<UPCGManagedISKMComponent*> MISKMCs;
			ManagedResourcesContainerHelper.ForEachManagedResource([&MISKMCs, &Context, Settings](UPCGManagedResource* InResource)
			{
				if (UPCGManagedISKMComponent* Resource = Cast<UPCGManagedISKMComponent>(InResource))
				{
					if (Resource->GetCrc().IsValid() && Resource->GetCrc() == Context->DependenciesCrc)
					{
						MISKMCs.Add(Resource);
					}
				}
			});

			for (UPCGManagedISKMComponent* MISKMC : MISKMCs)
			{
				if (!MISKMC->IsMarkedUnused() && Settings->bWarnOnIdenticalSpawn)
				{
					// TODO: Revisit if the stack is added to the managed components at creation
					PCGLog::LogWarningOnGraph(LOCTEXT("IdenticalABMCSpawn", "Identical Instanced Skinned Mesh Component spawn occurred. It may be beneficial to re-check graph logic for identical spawn conditions (same mesh descriptor at same location, etc) or repeated nodes."), Context);
				}

				MISKMC->MarkAsReused();
			}

			if (!MISKMCs.IsEmpty())
			{
				bSkippedDueToReuse = true;
			}
		}

		Context->bReuseCheckDone = true;
	}

	// Early out - if we've established we could reuse resources and there is no need to generate an output, quit now
	if (!bGenerateOutput && bSkippedDueToReuse)
	{
		return true;
	}

	// perform mesh selection
	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	while(Context->CurrentInputIndex < Inputs.Num())
	{
		if (!Context->bCurrentInputSetup)
		{
			const FPCGTaggedData& Input = Inputs[Context->CurrentInputIndex];
			const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Input.Data);

			if (!SpatialData)
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data"));
				++Context->CurrentInputIndex;
				continue;
			}

			const UPCGPointData* PointData = SpatialData->ToPointData(Context);
			if (!PointData)
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("NoPointDataInInput", "Unable to get point data from input"));
				++Context->CurrentInputIndex;
				continue;
			}

			AActor* TargetActor = Settings->TargetActor.Get() ? Settings->TargetActor.Get() : Context->GetTypedExecutionTarget<AActor>();
			if (!TargetActor)
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidTargetActor", "Invalid target actor. Ensure TargetActor member is initialized when creating SpatialData."));
				++Context->CurrentInputIndex;
				continue;
			}

			if (bGenerateOutput)
			{
				FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

				UPCGPointData* OutputPointData = FPCGContext::NewObject_AnyThread<UPCGPointData>(Context);
				OutputPointData->InitializeFromData(PointData);

				Output.Data = OutputPointData;
				check(!Context->CurrentOutputPointData);
				Context->CurrentOutputPointData = OutputPointData;
			}

			FPCGSkinnedMeshSpawnerContext::FPackedInstanceListData& InstanceListData = Context->MeshInstancesData.Emplace_GetRef();
			InstanceListData.TargetActor = TargetActor;
			InstanceListData.SpatialData = PointData;

			Context->CurrentPointData = PointData;
			Context->bCurrentInputSetup = true;
		}

		// TODO: If we know we re-use the mesh components, we should not run the Selection, as it can be pretty costly.
		// At the moment, the selection is filling the output point data, so it is necessary to run it. But we should just hit the cache in that case.
		if (!Context->bSelectionDone)
		{
			check(Context->CurrentPointData);
			Context->bSelectionDone = Settings->MeshSelectorParameters->SelectInstances(*Context, Settings, Context->CurrentPointData, Context->MeshInstancesData.Last().MeshInstances, Context->CurrentOutputPointData);
		}

		if (!Context->bSelectionDone)
		{
			return false;
		}

		// If we need the output but would otherwise skip the resource creation, we don't need to run the instance packing part of the processing
		if (!bSkippedDueToReuse)
		{
			TArray<FPCGSkinnedMeshPackedCustomData>& PackedCustomData = Context->MeshInstancesData.Last().PackedCustomData;
			const TArray<FPCGSkinnedMeshInstanceList>& MeshInstances = Context->MeshInstancesData.Last().MeshInstances;

			if (PackedCustomData.Num() != MeshInstances.Num())
			{
				PackedCustomData.SetNum(MeshInstances.Num());
			}

			if (Settings->InstanceDataPackerParameters)
			{
				for (int32 InstanceListIndex = 0; InstanceListIndex < MeshInstances.Num(); ++InstanceListIndex)
				{
					Settings->InstanceDataPackerParameters->PackInstances(*Context, Context->CurrentPointData, MeshInstances[InstanceListIndex], PackedCustomData[InstanceListIndex]);
				}
			}
		}

		// We're done - cleanup for next iteration if we still have time
		++Context->CurrentInputIndex;
		Context->ResetInputIterationData();

		// Continue on to next iteration if there is time left, otherwise, exit here
		if (Context->AsyncState.ShouldStop() && Context->CurrentInputIndex < Inputs.Num())
		{
			return false;
		}
	}

	IPCGAsyncLoadingContext* AsyncLoadingContext = static_cast<IPCGAsyncLoadingContext*>(Context);

	if (Context->CurrentInputIndex == Inputs.Num() && !AsyncLoadingContext->WasLoadRequested() && !Context->MeshInstancesData.IsEmpty() && !Settings->bSynchronousLoad)
	{
		TArray<FSoftObjectPath> ObjectsToLoad;
		for (const FPCGSkinnedMeshSpawnerContext::FPackedInstanceListData& InstanceData : Context->MeshInstancesData)
		{
			for (const FPCGSkinnedMeshInstanceList& MeshInstanceList : InstanceData.MeshInstances)
			{
				if (!MeshInstanceList.Descriptor.SkinnedAsset.IsNull())
				{
					ObjectsToLoad.AddUnique(MeshInstanceList.Descriptor.SkinnedAsset.ToSoftObjectPath());
				}

			#if 0 // AB-TODO
				for (const TSoftObjectPtr<UMaterialInterface>& OverrideMaterial : MeshInstanceList.Descriptor.OverrideMaterials)
				{
					if (!OverrideMaterial.IsNull())
					{
						ObjectsToLoad.AddUnique(OverrideMaterial.ToSoftObjectPath());
					}
				}

			#endif
			}
		}

		return AsyncLoadingContext->RequestResourceLoad(Context, std::move(ObjectsToLoad), /*bAsynchronous=*/true);
	}

	return true;
}

bool FPCGSkinnedMeshSpawnerElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSkinnedMeshSpawnerElement::Execute);
	FPCGSkinnedMeshSpawnerContext* Context = static_cast<FPCGSkinnedMeshSpawnerContext*>(InContext);
	const UPCGSkinnedMeshSpawnerSettings* Settings = Context->GetInputSettings<UPCGSkinnedMeshSpawnerSettings>();
	check(Settings && !Settings->ShouldExecuteOnGPU());

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("AddISKM", "Adding ISKM/instances to an ISKM from PCG"), Context->ExecutionSource.Get() && Context->ExecutionSource->GetExecutionState().UseTransactions());
#endif

	while(!Context->MeshInstancesData.IsEmpty())
	{
		const FPCGSkinnedMeshSpawnerContext::FPackedInstanceListData& InstanceList = Context->MeshInstancesData.Last();
		check(Context->bSkippedDueToReuse || InstanceList.MeshInstances.Num() == InstanceList.PackedCustomData.Num());

		const bool bTargetActorValid = (InstanceList.TargetActor && IsValid(InstanceList.TargetActor));

		if (bTargetActorValid)
		{
			while (Context->CurrentDataIndex < InstanceList.MeshInstances.Num())
			{
				const FPCGSkinnedMeshInstanceList& MeshInstance = InstanceList.MeshInstances[Context->CurrentDataIndex];
				// We always have mesh instances, but if we are in re-use, we don't compute the packed custom data.
				const FPCGSkinnedMeshPackedCustomData* PackedCustomData = InstanceList.PackedCustomData.IsValidIndex(Context->CurrentDataIndex) ? &InstanceList.PackedCustomData[Context->CurrentDataIndex] : nullptr;
				SpawnSkinnedMeshInstances(Context, MeshInstance, InstanceList.TargetActor, PackedCustomData);

				// Now that the mesh is loaded/spawned, set the bounds to out points if requested.
				if (MeshInstance.Descriptor.SkinnedAsset && Settings->bApplyMeshBoundsToPoints)
				{
					if (TMap<UPCGPointData*, TArray<int32>>* OutPointDataToPointIndex = Context->MeshToOutPoints.Find(MeshInstance.Descriptor.SkinnedAsset))
					{
						const FBox Bounds = MeshInstance.Descriptor.SkinnedAsset->GetBounds().GetBox();
						for (TPair<UPCGPointData*, TArray<int32>>& It : *OutPointDataToPointIndex)
						{
							check(It.Key);
							TArray<FPCGPoint>& OutPoints = It.Key->GetMutablePoints();
							for (int32 Index : It.Value)
							{
								FPCGPoint& Point = OutPoints[Index];
								Point.BoundsMin = Bounds.Min;
								Point.BoundsMax = Bounds.Max;
							}
						}
					}
				}

				++Context->CurrentDataIndex;

				if (Context->AsyncState.ShouldStop())
				{
					break;
				}
			}
		}

		if (!bTargetActorValid || Context->CurrentDataIndex == InstanceList.MeshInstances.Num())
		{
			Context->MeshInstancesData.RemoveAtSwap(Context->MeshInstancesData.Num() - 1);
			Context->CurrentDataIndex = 0;
		}

		if (Context->AsyncState.ShouldStop())
		{
			break;
		}
	}

	const bool bFinishedExecution = Context->MeshInstancesData.IsEmpty();
	if (bFinishedExecution)
	{
		if (AActor* TargetActor = Settings->TargetActor.Get() ? Settings->TargetActor.Get() : Context->GetTypedExecutionTarget<AActor>())
		{
			for (UFunction* Function : PCGHelpers::FindUserFunctions(TargetActor->GetClass(), Settings->PostProcessFunctionNames, { UPCGFunctionPrototypes::GetPrototypeWithNoParams() }, Context))
			{
				TargetActor->ProcessEvent(Function, nullptr);
			}
		}
	}

	return bFinishedExecution;
}

bool FPCGSkinnedMeshSpawnerElement::CanExecuteOnlyOnMainThread(FPCGContext* Context) const
{
	// PrepareData can call UPCGManagedComponent::MarkAsReused which registers the mesh component, which can go into Chaos code that asserts if not on main thread.
	// TODO: We can likely re-enable multi-threading for PrepareData if we move the call to MarkAsReused to Execute. There should hopefully not be
	// wider contention on resources resources are not shared across nodes and are also per-component.
	return !Context || Context->CurrentPhase == EPCGExecutionPhase::Execute || Context->CurrentPhase == EPCGExecutionPhase::PrepareData;
}

void FPCGSkinnedMeshSpawnerElement::SpawnSkinnedMeshInstances(FPCGSkinnedMeshSpawnerContext* Context, const FPCGSkinnedMeshInstanceList& InstanceList, AActor* TargetActor, const FPCGSkinnedMeshPackedCustomData* InPackedCustomData) const
{
	// Populate the mesh component from the previously prepared entries
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSkinnedMeshSpawnerElement::Execute::PopulateAB);

	if (InstanceList.Instances.Num() == 0)
	{
		return;
	}

	// Will be synchronously loaded if not loaded. But by default it should already have been loaded asynchronously in PrepareData, so this is free.
	USkinnedAsset* LoadedMesh = InstanceList.Descriptor.SkinnedAsset.LoadSynchronous();

	if (!LoadedMesh)
	{
		// Either we have no mesh (so nothing to do) or the mesh couldn't be loaded
		if (InstanceList.Descriptor.SkinnedAsset.IsValid())
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("MeshLoadFailed", "Unable to load mesh '{0}'"), FText::FromString(InstanceList.Descriptor.SkinnedAsset.ToString())));
		}

		return;
	}

	// Don't spawn meshes if we reuse the ISMCs, but we still want to be sure that the mesh is loaded at least (for operations downstream).
	if (Context->bSkippedDueToReuse)
	{
		return;
	}

#if 0 // AB-TODO
	for (TSoftObjectPtr<UMaterialInterface> OverrideMaterial : InstanceList.Descriptor.OverrideMaterials)
	{
		// Will be synchronously loaded if not loaded. But by default it should already have been loaded asynchronously in PrepareData, so this is free.
		if (OverrideMaterial.IsValid() && !OverrideMaterial.LoadSynchronous())
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("OverrideMaterialLoadFailed", "Unable to load override material '{0}'"), FText::FromString(OverrideMaterial.ToString())));
			return;
		}
	}
#endif

	// If we spawn the meshes, we should have computed a packed custom data.
	if (!ensure(InPackedCustomData))
	{
		return;
	}

	const FPCGSkinnedMeshPackedCustomData& PackedCustomData = *InPackedCustomData;

	FPCGSkinnedMeshComponentBuilderParams Params;
	Params.Descriptor = InstanceList.Descriptor;
	Params.NumCustomDataFloats = PackedCustomData.NumCustomDataFloats;

	// If the root actor we're binding to is movable, then the component should be movable by default
	if (USceneComponent* SceneComponent = TargetActor->GetRootComponent())
	{
		Params.Descriptor.Mobility = SceneComponent->Mobility;
	}

	const UPCGSkinnedMeshSpawnerSettings* Settings = Context->GetInputSettings<UPCGSkinnedMeshSpawnerSettings>();
	check(Settings);

	Params.SettingsCrc = Settings->GetSettingsCrc();
	ensure(Params.SettingsCrc.IsValid());

	// Implementation note: in order to prevent components from merging together when they have tags, we need to incorporate the tags hash
	// into the settings crc. This will make sure we don't select component that don't match up with what we're expecting.
	// However, this validation is potentially too restrictive, as the order and composition of the tags ultimately don't really matter.
	// TODO- improve this, however since this stems from a path where we build those, the ordering of tags could be sorted in other ways.
	if (Params.Descriptor.HasTags())
	{
		const uint32 TagsArrayHash = Params.Descriptor.ComponentTags.IsEmpty() ? 1 : GetTypeHash(Params.Descriptor.ComponentTags);
		const uint32 AdditionalTagsHash = Params.Descriptor.AdditionalCommaSeparatedTags.IsEmpty() ? 1 : GetTypeHash(Params.Descriptor.AdditionalCommaSeparatedTags);
		Params.SettingsCrc.Combine(HashCombine(TagsArrayHash, AdditionalTagsHash));
	}

	UPCGManagedISKMComponent* MISKMC = UPCGActorHelpers::GetOrCreateManagedABMC(TargetActor, Context->ExecutionSource.Get(), Params, Context);

	check(MISKMC);
	MISKMC->SetCrc(Context->DependenciesCrc);

	// Keep track of all touched resources in the context, because if the execution is cancelled during the SMS execution
	// we cannot easily guarantee that the state (esp. vs CRCs) is going to be entirely valid
	Context->TouchedResources.Emplace(MISKMC);

	UInstancedSkinnedMeshComponent* ISKMC = MISKMC->GetComponent();
	check(ISKMC);

	const int32 PreExistingInstanceCount = ISKMC->GetInstanceCount();
	const int32 NewInstanceCount = InstanceList.Instances.Num();
	const int32 NumCustomDataFloats = PackedCustomData.NumCustomDataFloats;

	check((ISKMC->GetNumCustomDataFloats() == 0 && PreExistingInstanceCount == 0) || ISKMC->GetNumCustomDataFloats() == NumCustomDataFloats);
	ISKMC->SetNumCustomDataFloats(NumCustomDataFloats);

	TArray<FTransform> Transforms;
	TArray<int32> AnimationIndices;

	Transforms.Reserve(NewInstanceCount);
	AnimationIndices.Reserve(NewInstanceCount);

	// TODO: Remove allocs/copies
	for (const FPCGSkinnedMeshInstance& Instance : InstanceList.Instances)
	{
		Transforms.Emplace(Instance.Transform);
		AnimationIndices.Emplace(Instance.AnimationIndex);
	}

	// Populate the instances
	TArray<FPrimitiveInstanceId> NewIds = ISKMC->AddInstances(Transforms, AnimationIndices, /*bShouldReturnIndices=*/true, /*bWorldSpace=*/true);

	// Copy new CustomData into the ABMC PerInstanceSMCustomData
	if (NumCustomDataFloats > 0)
	{
		for (int32 NewIndex = 0; NewIndex < NewInstanceCount; ++NewIndex)
		{
			ISKMC->SetCustomData(NewIds[NewIndex], MakeArrayView(&PackedCustomData.CustomData[NewIndex * NumCustomDataFloats], NumCustomDataFloats));
		}
	}

	ISKMC->UpdateBounds();
	ISKMC->OptimizeInstanceData();

	{
		PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("GenerationInfo", "Added {0} instances of '{1}' on actor '{2}'"),
			InstanceList.Instances.Num(), FText::FromString(InstanceList.Descriptor.SkinnedAsset->GetFName().ToString()), FText::FromString(TargetActor->GetFName().ToString())));
	}
}

void FPCGSkinnedMeshSpawnerElement::AbortInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSkinnedMeshSpawnerElement::AbortInternal);
	// It is possible to Abort a ready task with no context yet
	if (!InContext)
	{
		return;
	}

	FPCGSkinnedMeshSpawnerContext* Context = static_cast<FPCGSkinnedMeshSpawnerContext*>(InContext);

	// Any resources we've touched during the execution of this node can potentially be in a "not-quite complete state" especially if we have multiple sources of data writing to the same ISMC.
	// In this case, we're aiming to mark the resources as "Unused" so they are picked up to be removed during the component's OnProcessGraphAborted, which is why we call Release here.
	for (TWeakObjectPtr<UPCGManagedISKMComponent> ManagedResource : Context->TouchedResources)
	{
		if(ManagedResource.IsValid())
		{
			TSet<TSoftObjectPtr<AActor>> Dummy;
			ManagedResource->Release(/*bHardRelease=*/false, Dummy);
		}
	}
}

void UPCGSkinnedMeshSpawnerSettings::PostLoad()
{
	Super::PostLoad();

	const EObjectFlags Flags = GetMaskedFlags(RF_PropagateToSubObjects) | RF_Transactional;

	if (!MeshSelectorParameters)
	{
		MeshSelectorParameters = NewObject<UPCGSkinnedMeshSelector>(this, UPCGSkinnedMeshSelector::StaticClass(), NAME_None, GetMaskedFlags(RF_PropagateToSubObjects));
	}
	else
	{
		MeshSelectorParameters->SetFlags(Flags);
	}

	if (!InstanceDataPackerParameters)
	{
		RefreshInstancePacker();
	}
	else
	{
		InstanceDataPackerParameters->SetFlags(Flags);
	}
}

#if WITH_EDITOR
void UPCGSkinnedMeshSpawnerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) 
{
	if (PropertyChangedEvent.Property)
	{
		const FName& PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGSkinnedMeshSpawnerSettings, InstanceDataPackerType))
		{
			RefreshInstancePacker();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UPCGSkinnedMeshSpawnerSettings::CanEditChange(const FProperty* InProperty) const
{
	return Super::CanEditChange(InProperty);
}
#endif

void UPCGSkinnedMeshSpawnerSettings::SetInstancePackerType(TSubclassOf<UPCGSkinnedMeshInstanceDataPackerBase> InInstancePackerType)
{
	if (!InstanceDataPackerParameters || InInstancePackerType != InstanceDataPackerType)
	{
		if (InInstancePackerType != InstanceDataPackerType)
		{
			InstanceDataPackerType = InInstancePackerType;
		}
		
		RefreshInstancePacker();
	}
}

void UPCGSkinnedMeshSpawnerSettings::RefreshInstancePacker()
{
	if (InstanceDataPackerType)
	{
		ensure(IsInGameThread());

		if (InstanceDataPackerParameters)
		{
		#if WITH_EDITOR
			InstanceDataPackerParameters->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
		#endif
			InstanceDataPackerParameters->MarkAsGarbage();
			InstanceDataPackerParameters = nullptr;
		}

		const EObjectFlags Flags = GetMaskedFlags(RF_PropagateToSubObjects);
		InstanceDataPackerParameters = NewObject<UPCGSkinnedMeshInstanceDataPackerBase>(this, InstanceDataPackerType, NAME_None, Flags);
	}
	else
	{
		InstanceDataPackerParameters = nullptr;
	}
}

FPCGSkinnedMeshSpawnerContext::FPackedInstanceListData::FPackedInstanceListData() = default;
FPCGSkinnedMeshSpawnerContext::FPackedInstanceListData::~FPackedInstanceListData() = default;

#undef LOCTEXT_NAMESPACE
