// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSkinnedMeshSpawnerContext.h"

void FPCGSkinnedMeshSpawnerContext::ResetInputIterationData()
{
	bCurrentInputSetup = false;
	bSelectionDone = false;
	bPartitionDone = false;
	CurrentPointData = nullptr;
	CurrentOutputPointData = nullptr;
	MaterialOverrideHelper.Reset();
	CurrentPointIndex = 0;
	MeshToValueKey.Reset();
	AttributeOverridePartition.Reset();
	OverriddenDescriptors.Reset();
}