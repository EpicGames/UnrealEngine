// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSkinnedMeshSpawnerKernel.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Components/PCGProceduralISMComponent.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/BuiltInKernels/PCGCountUniqueAttributeValuesKernel.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/DataInterfaces/PCGInstanceDataInterface.h"
#include "Compute/DataInterfaces/Elements/PCGSkinnedMeshSpawnerDataInterface.h"
#include "Elements/PCGSkinnedMeshSpawner.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "InstanceDataPackers/PCGSkinnedMeshInstanceDataPackerBase.h"
#include "MeshSelectors/PCGSkinnedMeshSelector.h"

#include "ShaderCompilerCore.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSkinnedMeshSpawnerKernel)

#define LOCTEXT_NAMESPACE "PCGSkinnedMeshSpawnerKernel"

namespace PCGSkinnedMeshSpawnerKernel
{
	static const FText NoMeshEntriesFormat = LOCTEXT("NoMeshEntries", "No mesh entries provided in weighted mesh selector.");
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGSkinnedMeshSpawnerKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	// Code assumes single output pin.
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	// Forward data from In to Out.
	const FPCGKernelPin InputKernelPin(GetKernelIndex(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InBinding->ComputeKernelPinDataDesc(InputKernelPin);

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeSharedFrom(InputDesc);

	const UPCGSkinnedMeshSpawnerSettings* SMSettings = CastChecked<UPCGSkinnedMeshSpawnerSettings>(GetSettings());
	if (SMSettings->bApplyMeshBoundsToPoints)
	{
		OutDataDesc->AllocatePropertiesForAllData(EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax);
	}

	return OutDataDesc;
}

int UPCGSkinnedMeshSpawnerKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const TSharedPtr<const FPCGDataCollectionDesc> InputPinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	return ensure(InputPinDesc) ? InputPinDesc->ComputeTotalElementCount() : 0;
}

#if WITH_EDITOR
FString UPCGSkinnedMeshSpawnerKernel::GetCookedSource(FPCGGPUCompilationContext& InOutContext) const
{
	FString SourceFile;
	ensure(LoadShaderSourceFile(GetSourceFilePath(), EShaderPlatform::SP_PCD3D_SM5, &SourceFile, nullptr));

	TMap<FString, FStringFormatArg> Defines =
	{
		{ TEXT("ProduceOutputPoints"), GetProduceOutputPoints() ? 1 : 0 },
	};

	return FString::Format(*SourceFile, Defines);
}

void UPCGSkinnedMeshSpawnerKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGSkinnedMeshSpawnerDataInterface> NodeDI = InOutContext.NewObject_AnyThread<UPCGSkinnedMeshSpawnerDataInterface>(InObjectOuter);
	NodeDI->SetProducerKernel(this);
	OutDataInterfaces.Add(NodeDI);
}

void UPCGSkinnedMeshSpawnerKernel::CreateAdditionalOutputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalOutputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	UPCGInstanceDataInterface* InstanceDI = InOutContext.NewObject_AnyThread<UPCGInstanceDataInterface>(InObjectOuter);
	InstanceDI->SetProducerKernel(this);
	InstanceDI->SetInputPinProvidingData(PCGPinConstants::DefaultInputLabel);
	OutDataInterfaces.Add(InstanceDI);
}
#endif

void UPCGSkinnedMeshSpawnerKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	OutPins.Emplace(PCGSkinnedMeshSpawnerConstants::InstanceCountsPinLabel, EPCGDataType::Param);
}

void UPCGSkinnedMeshSpawnerKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	if (GetProduceOutputPoints())
	{
		OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
	}
}

#if WITH_EDITOR
bool UPCGSkinnedMeshSpawnerKernel::PerformStaticValidation()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSkinnedMeshSpawnerKernel::PerformStaticValidation);

	if (!Super::PerformStaticValidation())
	{
		return false;
	}

	const UPCGSkinnedMeshSpawnerSettings* SMSettings = CastChecked<UPCGSkinnedMeshSpawnerSettings>(GetSettings());

	// Currently instance packers must be able to specify a full list of attribute names upfront, to build the attribute table at compile time.
	// TODO: We should be able to augment a static attribute table with new attributes at execution time, which will allow other types like regex.
	if (SMSettings->InstanceDataPackerParameters && !SMSettings->InstanceDataPackerParameters->GetAttributeNames(/*OutNames=*/nullptr))
	{
#if PCG_KERNEL_LOGGING_ENABLED
		AddStaticLogEntry(LOCTEXT("InvalidInstancePacker", "Selected instance packer does not support GPU execution."), EPCGKernelLogVerbosity::Error);
#endif
		return false;
	}
	
	return true;
}
#endif

#undef LOCTEXT_NAMESPACE
