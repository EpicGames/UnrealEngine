// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSortAttributes.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGHelpers.h"
#include "Utils/PCGLogErrors.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSortAttributes)

#define LOCTEXT_NAMESPACE "PCGSortAttributesElement"

namespace PCGSortAttributes::Helpers
{
	using FCompareFunc = TFunction<int32(int32, int32)>;

	// Snapshots every value of the attribute into a cache owned by the returned closure, so the comparator can out live
	// the accessor and keys. Returns an unbound FCompareFunc if the attribute's underlying type is not comparable.
	FCompareFunc BuildCachedAttributeComparator(const IPCGAttributeAccessor& Accessor, const IPCGAttributeAccessorKeys& Keys)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGSortAttributes::Helpers::BuildCachedAttributeComparator);

		FCompareFunc CompareFunc;
		PCGMetadataAttribute::CallbackWithRightType(Accessor.GetUnderlyingType(), [&Accessor, &Keys, &CompareFunc]<typename T>(T)
		{
			if constexpr (PCG::Private::MetadataTraits<T>::CanCompare)
			{
				TSharedRef<TArray<T>> Cache = MakeShared<TArray<T>>();
				if constexpr (std::is_trivially_copyable_v<T>)
				{
					Cache->SetNumUninitialized(Keys.GetNum());
				}
				else
				{
					Cache->SetNum(Keys.GetNum());
				}

				Accessor.GetRange(TArrayView<T>(*Cache), 0, Keys);

				CompareFunc = [Cache](int32 A, int32 B) -> int32
				{
					const T& ValueA = (*Cache)[A];
					const T& ValueB = (*Cache)[B];
					if (PCG::Private::MetadataTraits<T>::Equal(ValueA, ValueB))
					{
						return 0;
					}

					return PCG::Private::MetadataTraits<T>::Less(ValueA, ValueB) ? -1 : 1;
				};
			}
		});

		return CompareFunc;
	}
} // namespace PCGSortAttributes::Helpers

UPCGSortAttributesSettings::UPCGSortAttributesSettings()
{
	// For deprecation of older behavior, in which case sorting was unstable. New nodes should default to stable sort.
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		bUseStableSort = true;
	}
}

void UPCGSortAttributesSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if ((InputSource != FPCGAttributePropertyInputSelector()) || (SortMethod != EPCGSortMethod::Ascending))
	{
		SortAttributes.Empty();
		FPCGSortAttributeEntry& Entry = SortAttributes.AddDefaulted_GetRef();
		Entry.InputSource = InputSource;
		Entry.SortMethod = SortMethod;

		InputSource = FPCGAttributePropertyInputSelector();
		SortMethod = EPCGSortMethod::Ascending;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

#if WITH_EDITOR
FName UPCGSortAttributesSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SortAttributes"));
}

FText UPCGSortAttributesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Sort Attributes");
}

FText UPCGSortAttributesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Sorts data by one or more attributes.");
}

TArray<FText> UPCGSortAttributesSettings::GetNodeTitleAliases() const
{
	// Re-use old LOCTEXT name if it was already localized.
	return { NSLOCTEXT("PCGSortPointsElement", "NodeTitle", "Sort Points") };
}
#endif // WITH_EDITOR

FString UPCGSortAttributesSettings::GetAdditionalTitleInformation() const
{
	return FString::JoinBy(SortAttributes, TEXT(", "), [](const FPCGSortAttributeEntry& Entry)
	{
		return Entry.InputSource.ToString();
	});
}

TArray<FPCGPinProperties> UPCGSortAttributesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGSortAttributesSettings::CreateElement() const
{
	return MakeShared<FPCGSortAttributesElement>();
}

bool FPCGSortAttributesElement::ExecuteInternal(FPCGContext* Context) const
{
	using namespace PCGSortAttributes;
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSortAttributesElement::Execute);

	check(Context);

	const UPCGSortAttributesSettings* Settings = Context->GetInputSettings<UPCGSortAttributesSettings>();
	check(Settings);

	const TArray<FPCGSortAttributeEntry>& EffectiveAttributes = Settings->SortAttributes;

	if (EffectiveAttributes.IsEmpty())
	{
		Context->OutputData = Context->InputData;
		return true;
	}

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSortAttributesElement::InputLoop);

		const FPCGTaggedData& Input = Inputs[InputIndex];
		const UPCGData* InputData = Input.Data;
		if (!InputData)
		{
			continue;
		}

		// @todo_pcg: Only the elements metadata domain is supported for now.
		for (const FPCGSortAttributeEntry& Entry : EffectiveAttributes)
		{
			const FPCGAttributePropertyInputSelector Selector = Entry.InputSource.CopyAndFixLast(InputData);
			FPCGMetadataDomainID DomainID = InputData->GetMetadataDomainIDFromSelector(Selector);
			if (DomainID.IsDefault())
			{
				DomainID = InputData->GetDefaultMetadataDomainID();
			}

			if (DomainID != PCGMetadataDomainID::Elements)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("UnsupportedDomain", "Sort attributes must target the elements metadata domain."), Context);
				return true;
			}
		}

		TArray<Helpers::FCompareFunc> AttributeCompares;
		AttributeCompares.Reserve(EffectiveAttributes.Num());
		int32 NumKeys = 0;
		bool bValid = true;

		for (int32 AttributeIndex = 0; AttributeIndex < EffectiveAttributes.Num(); ++AttributeIndex)
		{
			FPCGAttributePropertyInputSelector Selector = EffectiveAttributes[AttributeIndex].InputSource.CopyAndFixLast(InputData);
			TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InputData, Selector);
			TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InputData, Selector);

			if (!Accessor.IsValid() || !Keys.IsValid())
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, Context);
				bValid = false;
				break;
			}

			if (AttributeIndex == 0)
			{
				NumKeys = Keys->GetNum();
			}

			Helpers::FCompareFunc Compare = Helpers::BuildCachedAttributeComparator(*Accessor, *Keys);
			if (!Compare)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidTypeAccessor", "Attribute '{0}' exists but is not of a comparable type ({1})."), Selector.GetDisplayText(), PCG::Private::GetTypeNameText(Accessor->GetUnderlyingType())), Context);
				bValid = false;
				break;
			}

			AttributeCompares.Add(MoveTemp(Compare));
		}

		if (!bValid)
		{
			continue;
		}

		TArray<int32> SortedIndices;
		SortedIndices.SetNumUninitialized(NumKeys);
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			SortedIndices[KeyIndex] = KeyIndex;
		}

		auto Sort = [&AttributeCompares, &EffectiveAttributes](int32 A, int32 B) -> bool
		{
			for (int32 AttributeIndex = 0; AttributeIndex < EffectiveAttributes.Num(); ++AttributeIndex)
			{
				const int32 Result = AttributeCompares[AttributeIndex](A, B);
				if (Result != 0)
				{
					return EffectiveAttributes[AttributeIndex].SortMethod == EPCGSortMethod::Ascending ? Result < 0 : Result > 0;
				}
			}

			return false;
		};

		if (Settings->bUseStableSort)
		{
			SortedIndices.StableSort(Sort);
		}
		else
		{
			SortedIndices.Sort(Sort);
		}

		UPCGData* OutputData = nullptr;
		if (const UPCGBasePointData* InputPointData = Cast<const UPCGBasePointData>(InputData))
		{
			UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(Context);

			FPCGInitializeFromDataParams InitializeFromDataParams(InputPointData);
			InitializeFromDataParams.bInheritSpatialData = false;
			OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);

			UPCGBasePointData::SetPoints(InputPointData, OutputPointData, SortedIndices, /*bCopyAll=*/false);

			OutputData = OutputPointData;
		}
		else if (InputData->ConstMetadata())
		{
			OutputData = InputData->DuplicateData(Context, /*bInitializeMetdata=*/false);
			UPCGMetadata* OutMetadata = OutputData->MutableMetadata();

			if (!ensure(OutMetadata))
			{
				continue;
			}

			TArray<PCGMetadataEntryKey> Entries;
			Entries.Reserve(SortedIndices.Num());
			for (int32 Index : SortedIndices)
			{
				Entries.Add(Index);
			}

			OutMetadata->InitializeAsCopy(FPCGMetadataInitializeParams(InputData->ConstMetadata(), &Entries));
		}

		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);
		Output.Data = OutputData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
