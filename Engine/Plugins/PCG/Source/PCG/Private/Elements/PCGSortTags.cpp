// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSortTags.h"

#include "PCGContext.h"
#include "Helpers/PCGTagHelpers.h"

#include "Algo/StableSort.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSortTags)

#define LOCTEXT_NAMESPACE "PCGSortTagsElement"

#if WITH_EDITOR
// The label the node is known as internally.
FName UPCGSortTagsSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SortTags"));
}

// Default node name shown in the graph editor. Include spaces.
FText UPCGSortTagsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Sort Data By Tag Value");
}

// Default tooltip for the node
FText UPCGSortTagsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Sorts data by tag value (i.e. with tags of the form Tag:Value)");
}
#endif //WITH_EDITOR

TArray<FPCGPinProperties> UPCGSortTagsSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

FPCGElementPtr UPCGSortTagsSettings::CreateElement() const
{
	return MakeShared<FPCGSortTagsElement>();
}

bool FPCGSortTagsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSortTagsElement::Execute);

	check(Context);

	const UPCGSortTagsSettings* Settings = Context->GetInputSettings<UPCGSortTagsSettings>();
	check(Settings);

	const FString TargetTag = Settings->Tag.ToString();
	const bool bSortAscending = (Settings->SortMethod == EPCGSortMethod::Ascending);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	// Build list of parse tag results
	TMap<const FPCGTaggedData*, PCG::Private::FParseTagResult> TagResults;
	TArray<const FPCGTaggedData*> SortedInputs;
	bool bSortUsingNumericValues = true;

	TagResults.Reserve(Inputs.Num());
	SortedInputs.Reserve(Inputs.Num());

	for (const FPCGTaggedData& Input : Inputs)
	{
		SortedInputs.Add(&Input);

		bool bHasFoundMatch = false;
		for (const FString& Tag : Input.Tags)
		{
			PCG::Private::FParseTagResult TagData = PCG::Private::ParseTag(Tag);
			if (TagData.IsValid() && TagData.GetOriginalAttribute() == TargetTag)
			{
				if (TagData.HasValue() && !TagData.HasNumericValue())
				{
					// We'll use numeric values only if they are all using numeric values
					bSortUsingNumericValues = false;
				}

				TagResults.Add(&Input, MoveTemp(TagData));
				bHasFoundMatch = true;

				break;
			}
		}

		// Tag not found - add empty tag data.
		if (!bHasFoundMatch)
		{
			TagResults.Add(&Input, PCG::Private::FParseTagResult());
		}
	}
	
	// Sort according to our criteria
	Algo::StableSort(SortedInputs, [bSortUsingNumericValues, bSortAscending, &TagResults](const FPCGTaggedData* const& A, const FPCGTaggedData* const& B) -> bool
	{
		check(A && B);
		const PCG::Private::FParseTagResult& TagA = TagResults[A];
		const PCG::Private::FParseTagResult& TagB = TagResults[B];

		// Three stages of values here:
		// IsValid -> tag exists, always placed at the end
		if (!TagA.IsValid() || !TagB.IsValid())
		{
			return TagA.IsValid();
		}

		// Has Value -> binary tag
		if (!TagA.HasValue() || !TagB.HasValue())
		{
			return TagA.HasValue();
		}

		// Value -> compare values (either string or numerical)
		if (bSortUsingNumericValues)
		{
			return bSortAscending == (TagA.NumericValue.GetValue() < TagB.NumericValue.GetValue());
		}
		else
		{
			return bSortAscending == (TagA.Value.GetValue() < TagB.Value.GetValue());
		}
	});
	
	// Finally, write back sorted data
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	for (const FPCGTaggedData* SortedInput : SortedInputs)
	{
		Outputs.Add(*SortedInput);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
