// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSpatialNoise.h"

#include "Kismet/KismetMathLibrary.h"
#include "GameFramework/Actor.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGPin.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSpatialNoise)

#define LOCTEXT_NAMESPACE "PCGSpatialNoise"

UPCGSpatialNoiseSettings::UPCGSpatialNoiseSettings()
{
	ValueTarget.SetPointProperty(EPCGPointProperties::Density);
}

#if WITH_EDITOR
void UPCGSpatialNoiseSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	// Important note: this version change is not related to the reason we do deprecation here, but
	// this happened in a close timeframe and we can close the gap here. The original issue is that
	// we've added seed usage on the SpatialNoise settings at a slightly later version (a few days),
	// but since this applies at the CDO level too, this was adding seed usage also to the data prior to
	// this change, which had an effect on generated results.
	if (DataVersion < FPCGCustomVersion::NoMoreSpatialDataConversionToPointDataByDefaultOnNonPointPins)
	{
		bForceNoUseSeed = true;
	}

	Super::ApplyDeprecation(InOutNode);
}
#endif // WITH_EDITOR

bool UPCGSpatialNoiseSettings::UseSeed() const
{
	return !bForceNoUseSeed;
}

TArray<FPCGPinProperties> UPCGSpatialNoiseSettings::InputPinProperties() const
{
	return Super::DefaultPointInputPinProperties();
}

TArray<FPCGPinProperties> UPCGSpatialNoiseSettings::OutputPinProperties() const
{
	return Super::DefaultPointOutputPinProperties();
}

FPCGElementPtr UPCGSpatialNoiseSettings::CreateElement() const
{
	return MakeShared<FPCGSpatialNoise>();
}

namespace PCGSpatialNoise
{
	// the useful ranges here are very small so make input easier
	static const double MAGIC_SCALE_FACTOR = 0.0001;

	inline double Fract(double X)
	{
		return X - FMath::Floor(X);
	}

	inline FVector2D Floor(FVector2D X)
	{
		return FVector2D(FMath::Floor(X.X), FMath::Floor(X.Y));
	}

	inline FInt32Point FloorInt(FVector2D X)
	{
		return FInt32Point(FMath::FloorToInt(X.X), FMath::FloorToInt(X.Y));
	}

	inline FVector2D ToVec2D(FInt32Point V)
	{
		return FVector2D(double(V.X), double(V.Y));
	}

	inline FVector2D Fract(FVector2D V)
	{
		return FVector2D(Fract(V.X), Fract(V.Y));
	}

	inline double ValueHash(FVector2D Position)
	{
		Position = 50.0 * Fract(Position * 0.3183099 + FVector2D(0.71, 0.113));
		return -1.0 + 2.0 * Fract(Position.X * Position.Y * (Position.X + Position.Y));
	}

	inline double Noise2D(FVector2D Position)
	{
		const FVector2D FloorPosition = Floor(Position);
		const FVector2D Fraction = Position - FloorPosition;
		const FVector2D U = Fraction * Fraction * (FVector2D(3.0, 3.0) - 2.0 * Fraction);

		return FMath::Lerp(
			FMath::Lerp(ValueHash(FloorPosition), ValueHash(FloorPosition + FVector2D(1.0, 0.0)), U.X),
			FMath::Lerp(ValueHash(FloorPosition + FVector2D(0.0, 1.0)), ValueHash(FloorPosition + FVector2D(1.0, 1.0)), U.X),
			U.Y
		);
	}

	inline FVector2D MultiplyMatrix2D(FVector2D Point, const FVector2D (&Mat2)[2])
	{
		return Point.X * Mat2[0] + Point.Y * Mat2[1];
	}

	// takes a cell id and produces a pseudo random vector offset
	inline FVector2D VoronoiHash2D(FInt32Point Cell)
	{
		const FVector2D P = ToVec2D(Cell);

		// this is some arbitrary large random scale+rotation+skew
		const FVector2D P2 = FVector2D(
			P.Dot(FVector2D(127.1, 311.7)),
			P.Dot(FVector2D(269.5, 183.3))
		);

		// further scale the results by a big number
		return FVector2D(
			Fract(FMath::Sin(P2.X) * 17.1717) - 0.5,
			Fract(FMath::Sin(P2.Y) * 17.1717) - 0.5
		);
	}

	struct FVoronoi2DResult
	{
		double DistanceToEdge = 8.0;
		double ID = -1.0;

		FVector2D CellEdgeDirection = FVector2D::ZeroVector;
	};

	template<typename Func>
	FVoronoi2DResult CalcVoronoi2D(const FVector2D Position, Func&& HashFunc)
	{
		const FInt32Point WorldIdx = FloorInt(Position);

		FVector2D CellHashValue = FVector2D::ZeroVector;
		FVector2D CellPosition = FVector2D::ZeroVector;
		FInt32Point CellIdx = FInt32Point(0,0);

		// first pass, find our closest cell point (which is all voronoi really is)
		{
			double MinDistanceSq = TNumericLimits<double>::Max();
			for (int32 Y = WorldIdx.Y-1; Y <= WorldIdx.Y+1; ++Y)
			{
				for (int32 X = WorldIdx.X-1; X <= WorldIdx.X+1; ++X)
				{
					const FInt32Point ThisCellIdx(X, Y);
					const FVector2D ThisCellHashValue = HashFunc(ThisCellIdx);
					const FVector2D ThisCellPosition = ToVec2D(ThisCellIdx) + ThisCellHashValue;

					const double CellDistSquared = FVector2D::DistSquared(ThisCellPosition, Position);
					if (CellDistSquared > MinDistanceSq)
					{
						continue;
					}

					MinDistanceSq = CellDistSquared;
					CellHashValue = ThisCellHashValue;
					CellPosition = ThisCellPosition;
					CellIdx = ThisCellIdx;
				}
			}
		}

		// second pass, compute distance from Position to the nearest cell edge
		FVoronoi2DResult Result;
		Result.ID =  0.5+0.5*FMath::Cos((CellHashValue.X+CellHashValue.Y)*6.2831);
		Result.DistanceToEdge = TNumericLimits<double>::Max();

		for (int32 Y = CellIdx.Y-2; Y <= CellIdx.Y+2; ++Y)
		{
			for (int32 X = CellIdx.X-2; X <= CellIdx.X+2; ++X)
			{
				const FInt32Point ThisCellIdx(X, Y);

				if (ThisCellIdx == CellIdx)
				{
					continue;
				}

				const FVector2D ThisCellPosition = ToVec2D(ThisCellIdx) + HashFunc(ThisCellIdx);
				const FVector2D EdgeDirection = ThisCellPosition - CellPosition;

				if (EdgeDirection.IsNearlyZero())
				{
					continue;
				}

				const FVector2D PlaneNormal = EdgeDirection.GetSafeNormal();
				const FVector2D PointOnPlane = CellPosition + EdgeDirection * 0.5;

				const double DistanceToEdge = FMath::Abs((Position-PointOnPlane).Dot(PlaneNormal));
				if (DistanceToEdge > Result.DistanceToEdge)
				{
					continue;
				}

				Result.DistanceToEdge = DistanceToEdge;
				Result.CellEdgeDirection = PlaneNormal;
			}
		}

		return Result;
	}

	double CalcFractionalBrownian2D(FVector2D Position, int32 Iterations)
	{
		double Z = 0.5;
		double Result = 0.0;

		// just some fixed random rotation and scale numbers
		const FVector2D RotScale[] = {{1.910673, -0.5910404}, {0.5910404, 1.910673}};

		for (int32 I = 0; I < Iterations; ++I)
		{
			Result += FMath::Abs(Noise2D(Position)) * Z;
			Z *= 0.5;
			Position = MultiplyMatrix2D(Position, RotScale);
		}

		return Result;
	}

	double CalcCaustic2D(FVector2D Position, int32 Iterations)
	{
		const FVector2D P = Fract(Position * 0.2) * (PI * 2.0) - FVector2D(250.0, 250.0);
		FVector2D I = P;
		double Value = 0.0;

		for (int32 N = 0; N < Iterations; ++N)
		{
			const double T = 1.0 - (3.5 / double(N+1));
			I = P + FVector2D(FMath::Cos(T - I.X) + FMath::Sin(T + I.Y), FMath::Sin(T - I.Y) + FMath::Cos(T + I.X));

			const double S = FMath::Sin(I.X + T);
			const double C = FMath::Cos(I.Y + T);

			const FVector2D Temp = FVector2D(
				FMath::Abs(S) > 0.0001 ? (P.X / S) : 1000.0,
				FMath::Abs(C) > 0.0001 ? (P.Y / C) : 1000.0
			);

			Value += FMath::InvSqrt(Temp.SquaredLength());
		}

		return Value / (0.003 * double(Iterations));
	}

	const FVector2D PerlinM[] = {{1.6, 1.2}, {-1.2, 1.6}};

	double CalcPerlin2D(FVector2D Position, int32 Iterations)
	{
		double Value = 0.0;

		double Strength = 1.0;

		for (int32 N = 0; N < Iterations; ++N)
		{
			Strength *= 0.5;
			Value += Strength * Noise2D(Position);
			Position = MultiplyMatrix2D(Position, PerlinM);
		}

		return 0.5 + 0.5 * Value;		
	}

	double ApplyContrast(double Value, double Contrast)
	{
		// early out for default 1.0 contrast, the math should be the same
		if (Contrast == 1.0)
		{
			return Value;
		}

		if (Contrast <= 0.0)
		{
			return 0.5;
		}

		Value = FMath::Clamp(Value, 0.0, 1.0);

		if (Value == 1.0)
		{
			return 1.0;
		}

		return 1.0 / (1.0 + FMath::Pow(Value / (1.0 - Value), -Contrast));
	}

	FLocalCoordinates2D CalcLocalCoordinates2D(const FBox& LocalBox, const FTransform& TransformInverse, FVector2D Scale, const FVector& InPosition)
	{
		if (!LocalBox.IsValid)
		{
			return {};
		}

		const FVector2D LocalPosition = FVector2D(TransformInverse.TransformPosition(InPosition));

		const double LeftDist = LocalPosition.X-LocalBox.Min.X;
		const double RightDist = LocalPosition.X-LocalBox.Max.X;

		const double TopDist = LocalPosition.Y-LocalBox.Min.Y;
		const double BottomDist = LocalPosition.Y-LocalBox.Max.Y;

		FLocalCoordinates2D Result;

		Result.X0 = LeftDist * Scale.X;
		Result.X1 = RightDist * Scale.X;
		Result.Y0 = TopDist * Scale.Y;
		Result.Y1 = BottomDist * Scale.Y;

		Result.FracX = FMath::Clamp(LeftDist / (LocalBox.Max.X - LocalBox.Min.X), 0.0, 1.0);
		Result.FracY = FMath::Clamp(TopDist / (LocalBox.Max.Y - LocalBox.Min.Y), 0.0, 1.0);

		return Result;
	};

	double CalcEdgeBlendAmount2D(const FLocalCoordinates2D& LocalCoords, double EdgeBlendDistance)
	{
		if (EdgeBlendDistance < 0.01)
		{
			return 1.0;
		}

		const double UseX = FMath::Abs(LocalCoords.X0) < FMath::Abs(LocalCoords.X1) ? LocalCoords.X0 : LocalCoords.X1;
		const double UseY = FMath::Abs(LocalCoords.Y0) < FMath::Abs(LocalCoords.Y1) ? LocalCoords.Y0 : LocalCoords.Y1;

		const double CurrentEdgeAmount = FMath::Min(FMath::Abs(UseX), FMath::Abs(UseY));

		return EdgeBlendDistance > CurrentEdgeAmount ? FMath::Clamp((EdgeBlendDistance-CurrentEdgeAmount)/EdgeBlendDistance, 0.0, 1.0) : 0.0;		
	}

	struct FSharedParams
	{
		FPCGContext* Context = nullptr;
		FTransform TransformInverse;
		FBox LocalBox;
		FTransform Transform;

		double Brightness;
		double Contrast;
		int32 Iterations;
		bool bTiling;
	};

	struct FBufferParams
	{
		const UPCGBasePointData* InputPointData = nullptr;
		UPCGBasePointData* OutputPointData = nullptr;
	};

	template<typename T>
	void SetAttributeHelper(UPCGSpatialData* Data, const FPCGAttributePropertySelector& PropertySelector, const TArrayView<const T> Values)
	{
		if (PropertySelector.GetSelection() == EPCGAttributePropertySelection::Attribute)
		{
			// Discard None attributes
			const FName AttributeName = PropertySelector.GetAttributeName();
			if (AttributeName == NAME_None)
			{
				return;
			}

			Data->Metadata->FindOrCreateAttribute<T>(AttributeName);
		}

		TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateAccessor(Data, PropertySelector);
		if (!Accessor)
		{
			return;
		}

		TUniquePtr<IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateKeys(Data, PropertySelector);
		if (!Keys)
		{
			return;
		}

		Accessor->SetRange(Values, 0, *Keys);
	}

	template<typename FractalNoiseFunc>
	void DoFractal2D(const FSharedParams& SharedParams, const FBufferParams& BufferParams, const UPCGSpatialNoiseSettings& Settings, FractalNoiseFunc&& FractalNoise)
	{
		const int32 NumPoints = BufferParams.InputPointData->GetNumPoints();

		struct FProcessResults
		{
			TArray<double> Values;
		};

		FProcessResults Results;

		FPCGAsync::AsyncProcessingOneToOneRangeEx(
			SharedParams.Context ? &SharedParams.Context->AsyncState : nullptr,
			NumPoints,
			[&Results, NumPoints]()
			{
				// initialize
				Results.Values.SetNumUninitialized(NumPoints);
			},
			[&SharedParams, &BufferParams, &Results, &FractalNoise](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
			{
				int32 NumWritten = 0;

				if (!BufferParams.OutputPointData->HasSpatialDataParent())
				{
					BufferParams.InputPointData->CopyPointsTo(BufferParams.OutputPointData, StartReadIndex, StartWriteIndex, Count);
				}

				const TConstPCGValueRange<FTransform> InTransformRange = BufferParams.InputPointData->GetConstTransformValueRange();

				for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
				{	
					const FVector PointPos = InTransformRange[ReadIndex].GetTranslation();

					double Value = 0.0;

					if (SharedParams.bTiling)
					{
						const FLocalCoordinates2D LocalCoords = CalcLocalCoordinates2D(
							SharedParams.LocalBox,
							SharedParams.TransformInverse,
							FVector2D(SharedParams.Transform.GetScale3D()),
							PointPos
						);

						Value = FMath::BiLerp(
							FractalNoise(FVector2D(LocalCoords.X0, LocalCoords.Y0), SharedParams.Iterations),
							FractalNoise(FVector2D(LocalCoords.X1, LocalCoords.Y0), SharedParams.Iterations),
							FractalNoise(FVector2D(LocalCoords.X0, LocalCoords.Y1), SharedParams.Iterations),
							FractalNoise(FVector2D(LocalCoords.X1, LocalCoords.Y1), SharedParams.Iterations),
							LocalCoords.FracX,
							LocalCoords.FracY
						);
					}
					else
					{
						Value = FractalNoise(FVector2D(SharedParams.Transform.TransformPosition(PointPos)), SharedParams.Iterations);
					}

					Value = ApplyContrast(SharedParams.Brightness + Value, SharedParams.Contrast);

					const int32 WriteIndex = StartWriteIndex + NumWritten;
					Results.Values[WriteIndex] = Value;
					++NumWritten;
				}

				check(NumWritten == Count);
				return NumWritten;
			},
			/*bTimeSliceEnabled=*/false);

		SetAttributeHelper<double>(BufferParams.OutputPointData, Settings.ValueTarget, Results.Values);
	}

	template<typename FractalNoiseFunc>
	void DoEdgeMask2D(const FSharedParams& SharedParams, const FBufferParams& BufferParams, const UPCGSpatialNoiseSettings& Settings, FractalNoiseFunc&& FractalNoise)
	{
		const int32 NumPoints = BufferParams.InputPointData->GetNumPoints();

		struct FProcessResults
		{
			TArray<double> Values;
		};

		FProcessResults Results;

		FPCGAsync::AsyncProcessingOneToOneRangeEx(
			SharedParams.Context ? &SharedParams.Context->AsyncState : nullptr,
			NumPoints,
			[&Results, NumPoints]()
			{
				// initialize
				Results.Values.SetNumUninitialized(NumPoints);
			},
			[&SharedParams, &BufferParams, &Results, &FractalNoise, &Settings](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
			{
				int32 NumWritten = 0;

				if (!BufferParams.OutputPointData->HasSpatialDataParent())
				{
					BufferParams.InputPointData->CopyPointsTo(BufferParams.OutputPointData, StartReadIndex, StartWriteIndex, Count);
				}

				const TConstPCGValueRange<FTransform> InTransformRange = BufferParams.InputPointData->GetConstTransformValueRange();

				for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
				{
					const FVector InPosition = InTransformRange[ReadIndex].GetTranslation();

					const FLocalCoordinates2D LocalCoords = CalcLocalCoordinates2D(
						SharedParams.LocalBox,
						SharedParams.TransformInverse,
						FVector2D(SharedParams.Transform.GetScale3D()),
						InPosition
					);

					const double EdgeBlendAmount = CalcEdgeBlendAmount2D(LocalCoords, Settings.EdgeBlendDistance);

					double Value = 1.0;

					if (EdgeBlendAmount > 0.0001)
					{
						const double NoiseValue = FractalNoise(FVector2D(SharedParams.Transform.TransformPosition(InPosition)), SharedParams.Iterations);
						const double RemappedNoiseValue = ApplyContrast(NoiseValue, Settings.EdgeBlendCurveIntensity);
						const double OffsetAmount = FMath::Pow(EdgeBlendAmount, Settings.EdgeBlendCurveOffset);

						const double NoisedAmount = FMath::Lerp(
							FMath::Min(OffsetAmount, RemappedNoiseValue * OffsetAmount),
							1.0 - FMath::Min(1.0 - OffsetAmount, (1.0 - RemappedNoiseValue) * (1.0 - OffsetAmount)),
							OffsetAmount
						);

						Value = 1.0 - FMath::Clamp(ApplyContrast(NoisedAmount + SharedParams.Brightness, SharedParams.Contrast), 0.0, 1.0);
					}

					const int32 WriteIndex = StartWriteIndex + NumWritten;
					Results.Values[WriteIndex] = Value;
					++NumWritten;
				}

				check(NumWritten == Count);
				return NumWritten;
			},
			/*bTimeSliceEnabled=*/false);

		SetAttributeHelper<double>(BufferParams.OutputPointData, Settings.ValueTarget, Results.Values);
	}

	template<typename VoronoiHashFunc>
	void DoVoronoi2DInternal(const FSharedParams& SharedParams, const FBufferParams& BufferParams, const UPCGSpatialNoiseSettings& Settings, VoronoiHashFunc&& HashFunc)
	{
		const int32 NumPoints = BufferParams.InputPointData->GetNumPoints();

		struct FProcessResults
		{
			TArray<double> DistanceToEdge;
			TArray<double> CellID;
		};

		FProcessResults Results;

		FPCGAsync::AsyncProcessingOneToOneRangeEx(
			SharedParams.Context ? &SharedParams.Context->AsyncState : nullptr,
			NumPoints,
			[&BufferParams, &Results, NumPoints, &Settings]()
			{
				// initialize
				Results.DistanceToEdge.SetNumUninitialized(NumPoints);
				Results.CellID.SetNumUninitialized(NumPoints);

				if (Settings.bVoronoiOrientSamplesToCellEdge)
				{
					BufferParams.OutputPointData->AllocateProperties(EPCGPointNativeProperties::Transform);
				}
			},
			[&SharedParams, &BufferParams, &HashFunc, &Results, &Settings](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
			{
				int32 NumWritten = 0;

				if (!BufferParams.OutputPointData->HasSpatialDataParent())
				{
					BufferParams.InputPointData->CopyPointsTo(BufferParams.OutputPointData, StartReadIndex, StartWriteIndex, Count);
				}

				const TConstPCGValueRange<FTransform> InTransformRange = BufferParams.InputPointData->GetConstTransformValueRange();
				TPCGValueRange<FTransform> OutTransformRange = Settings.bVoronoiOrientSamplesToCellEdge ? BufferParams.OutputPointData->GetTransformValueRange(/*bAllocate=*/false) : TPCGValueRange<FTransform>();

				for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
				{
					const FVector2D Position = FVector2D(SharedParams.Transform.TransformPosition(InTransformRange[ReadIndex].GetTranslation()));

					FVoronoi2DResult Result = CalcVoronoi2D(Position, HashFunc);

					const double Value = ApplyContrast(SharedParams.Brightness + Result.DistanceToEdge / MAGIC_SCALE_FACTOR, SharedParams.Contrast);

					const int32 WriteIndex = StartWriteIndex + NumWritten;
					Results.DistanceToEdge[WriteIndex] = Value;
					Results.CellID[WriteIndex] = Result.ID;
				
					if (Settings.bVoronoiOrientSamplesToCellEdge)
					{
						OutTransformRange[WriteIndex].SetRotation(
							FQuat(UKismetMathLibrary::FindLookAtRotation(
								FVector::ZeroVector,
								FVector(Result.CellEdgeDirection.X, Result.CellEdgeDirection.Y, 0.0)
							))
						);
					}

					++NumWritten;
				}

				check(NumWritten == Count);
				return NumWritten;
			},
			/*bTimeSliceEnabled=*/false);

		SetAttributeHelper<double>(BufferParams.OutputPointData, Settings.ValueTarget, Results.DistanceToEdge);
		SetAttributeHelper<double>(BufferParams.OutputPointData, Settings.VoronoiCellIDTarget, Results.CellID);
	}

	void DoVoronoi2D(const FSharedParams& SharedParams, const FBufferParams& BufferParams, const UPCGSpatialNoiseSettings& Settings)
	{
		if (SharedParams.bTiling)
		{
			/**

			 This voronoi method uses a fixed integer cell subdivision which means it can tile seamlessly
			 it also has the ability to randomize the interior cell positions with a different seed so the 
			 end result is variations of things that still tile with each other.

			 */
			const FInt32Point InteriorOffset = FloorInt(FVector2D(SharedParams.Transform.GetTranslation()));
			const int32 TiledVoronoiResolution = FMath::Max(1, Settings.TiledVoronoiResolution);
			const int32 MaxBlendingTileIndex = TiledVoronoiResolution - Settings.TiledVoronoiEdgeBlendCellCount;
			const FVector LocalBoxSize = SharedParams.LocalBox.GetSize();

			// copy this so the new transform converts to the cell index
			FSharedParams UseSharedParams = SharedParams;

			UseSharedParams.Transform = SharedParams.TransformInverse * FTransform(
				FRotator::ZeroRotator,
				FVector(double(TiledVoronoiResolution) * 0.5 + 0.5, double(TiledVoronoiResolution) * 0.5 + 0.5, 1.0), // offset to the center
				FVector(double(TiledVoronoiResolution)/LocalBoxSize.X, double(TiledVoronoiResolution)/LocalBoxSize.Y, 1.0) // scale to the range of 0 .. TiledVoronoiResolution
			);

			DoVoronoi2DInternal(UseSharedParams, BufferParams, Settings,
				[
					InteriorOffset,
					MaxBlendingTileIndex,
					TiledVoronoiResolution,
					EdgeOffset = FloorInt(FVector2D(Settings.Transform.GetTranslation())),
					EdgeBlendCellCount = Settings.TiledVoronoiEdgeBlendCellCount,
					VoronoiCellRandomness = Settings.VoronoiCellRandomness					
				] (const FInt32Point Cell)
				{
					if (EdgeBlendCellCount > 0 && (Cell.X < EdgeBlendCellCount || Cell.Y < EdgeBlendCellCount || Cell.X >= MaxBlendingTileIndex || Cell.Y >= MaxBlendingTileIndex))
					{
						// global seed for edge cells
						return VoronoiHash2D(FInt32Point((Cell.X + EdgeOffset.X) % TiledVoronoiResolution, (Cell.Y + EdgeOffset.Y) % TiledVoronoiResolution)) * VoronoiCellRandomness;
					}
					else
					{
						return VoronoiHash2D(Cell + InteriorOffset) * VoronoiCellRandomness;
					}
				}
			);	
		}
		else
		{
			DoVoronoi2DInternal(SharedParams, BufferParams, Settings,
				[VoronoiCellRandomness = Settings.VoronoiCellRandomness](const FInt32Point CellIdx) {
					return VoronoiHash2D(CellIdx) * VoronoiCellRandomness;
				}
			);
		}		
	}
}

bool FPCGSpatialNoise::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSpatialNoise::Execute);

	const UPCGSpatialNoiseSettings* Settings = Context->GetInputSettings<UPCGSpatialNoiseSettings>();
	check(Settings);

	FRandomStream RandomSource(Context->GetSeed());

	const FVector RandomOffset = Settings->RandomOffset * FVector(RandomSource.GetFraction(), RandomSource.GetFraction(), RandomSource.GetFraction());

	PCGSpatialNoise::FSharedParams SharedParams;
	SharedParams.Context = Context;

	IPCGGraphExecutionSource* ExecutionSource = Cast<IPCGGraphExecutionSource>(Context->ExecutionSource.Get());
	if (!ExecutionSource)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("NoExecutionSource", "No Execution Source."));
		return true;
	}

	{
		SharedParams.LocalBox = ExecutionSource->GetExecutionState().GetLocalSpaceBounds();

		const FTransform Transform = ExecutionSource->GetExecutionState().GetTransform();
		SharedParams.LocalBox.Min *= Transform.GetScale3D();
		SharedParams.LocalBox.Max *= Transform.GetScale3D();

		const FTransform ActorTransform = FTransform(Transform.Rotator(), Transform.GetTranslation(), FVector::One());
		SharedParams.TransformInverse = ActorTransform.Inverse();
	}

	SharedParams.Transform = Settings->Transform * FTransform(FRotator::ZeroRotator, RandomOffset, FVector(PCGSpatialNoise::MAGIC_SCALE_FACTOR));
	SharedParams.Brightness = Settings->Brightness;
	SharedParams.Contrast = Settings->Contrast;
	SharedParams.bTiling = Settings->bTiling;
	SharedParams.Iterations = FMath::Max(1, Settings->Iterations); // clamped in meta properties but things will crash if it's < 1
	
	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);	
	for (const FPCGTaggedData& Input : Inputs)
	{
		PCGSpatialNoise::FBufferParams BufferParams;

		BufferParams.InputPointData = Cast<UPCGBasePointData>(Input.Data);

		if (!BufferParams.InputPointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data (only supports point data)."));
			continue;
		}

		BufferParams.OutputPointData = FPCGContext::NewPointData_AnyThread(Context);
		BufferParams.OutputPointData->InitializeFromData(BufferParams.InputPointData);
		BufferParams.OutputPointData->SetNumPoints(BufferParams.InputPointData->GetNumPoints());

		if (!BufferParams.OutputPointData->HasSpatialDataParent())
		{
			BufferParams.OutputPointData->AllocateProperties(BufferParams.InputPointData->GetAllocatedProperties());
		}

		Context->OutputData.TaggedData.Add_GetRef(Input).Data = BufferParams.OutputPointData;

		switch (Settings->Mode)
		{
		case PCGSpatialNoiseMode::Caustic2D:
			{
				DoFractal2D(SharedParams, BufferParams, *Settings, &PCGSpatialNoise::CalcCaustic2D);
			}
			break;

		case PCGSpatialNoiseMode::Perlin2D:
			{
				DoFractal2D(SharedParams, BufferParams, *Settings, &PCGSpatialNoise::CalcPerlin2D);
			}
			break;

		case PCGSpatialNoiseMode::FractionalBrownian2D:
			{
				DoFractal2D(SharedParams, BufferParams, *Settings, &PCGSpatialNoise::CalcFractionalBrownian2D);
			}
			break;

		case PCGSpatialNoiseMode::Voronoi2D:
			{
				DoVoronoi2D(SharedParams, BufferParams, *Settings);
			}
			break;

		case PCGSpatialNoiseMode::EdgeMask2D:
			{
				switch (Settings->EdgeMask2DMode)
				{
				case PCGSpatialNoiseMask2DMode::Perlin:
					DoEdgeMask2D(SharedParams, BufferParams, *Settings, &PCGSpatialNoise::CalcPerlin2D);
					break;

				case PCGSpatialNoiseMask2DMode::Caustic:
					DoEdgeMask2D(SharedParams, BufferParams, *Settings, &PCGSpatialNoise::CalcCaustic2D);
					break;

				case PCGSpatialNoiseMask2DMode::FractionalBrownian:
					DoEdgeMask2D(SharedParams, BufferParams, *Settings, &PCGSpatialNoise::CalcFractionalBrownian2D);
					break;

				default:
					checkNoEntry();
					break;
				}
			}
			break;

		default:
			checkNoEntry();
			break;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
