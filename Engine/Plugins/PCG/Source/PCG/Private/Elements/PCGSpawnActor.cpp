// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSpawnActor.h"

#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGManagedResource.h"
#include "PCGSubsystem.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Graph/PCGGraphExecutor.h"
#include "Graph/PCGStackContext.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGPointDataPartition.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#include "Misc/Crc.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSpawnActor)

#define LOCTEXT_NAMESPACE "PCGSpawnActorElement"

static TAutoConsoleVariable<bool> CVarAllowActorReuse(
	TEXT("pcg.Actor.AllowReuse"),
	true,
	TEXT("Controls whether PCG spawned actors can be reused and skipped when re-executing"));

class FPCGSpawnActorPartitionByAttribute : public FPCGDataPartitionBase<FPCGSpawnActorPartitionByAttribute, TSubclassOf<AActor>>
{
public:
	FPCGSpawnActorPartitionByAttribute(FName InSpawnAttribute, FPCGContext* InContext)
		: FPCGDataPartitionBase<FPCGSpawnActorPartitionByAttribute, TSubclassOf<AActor>>()
		, SpawnAttribute(InSpawnAttribute)
		, Context(InContext)
	{
	}

	bool InitializeForData(const UPCGData* InData, UPCGData* OutData)
	{
		if (!InData || !InData->IsA<UPCGBasePointData>())
		{
			return false;
		}

		FPCGAttributePropertyInputSelector InputSource;
		InputSource.SetAttributeName(SpawnAttribute);
		InputSource = InputSource.CopyAndFixLast(InData);
		SpawnAttributeAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InData, InputSource);
		SpawnAttributeKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InData, InputSource);

		return SpawnAttributeAccessor.IsValid() && SpawnAttributeKeys.IsValid();
	}

	void Finalize(const UPCGData* InData, UPCGData* OutData)
	{
		const UPCGBasePointData* InPointData = Cast<UPCGBasePointData>(InData);
		if (!InPointData)
		{
			return;
		}

		for (auto& KeyValuePair : ElementMap)
		{
			FPCGDataPartitionBase::Element& Element = KeyValuePair.Value;
			
			if (!Element.Indices.IsEmpty())
			{
				check(!Element.PartitionData);

				UPCGBasePointData* PartitionPointData = FPCGContext::NewPointData_AnyThread(Context);
				Element.PartitionData = PartitionPointData;

				FPCGInitializeFromDataParams InitializeFromDataParams(InPointData);
				InitializeFromDataParams.bInheritSpatialData = false;
				
				PartitionPointData->InitializeFromDataWithParams(InitializeFromDataParams);
				PartitionPointData->SetPointsFrom(InPointData, Element.Indices);
			}
		}
	}
		
	FPCGDataPartitionBase::Element* Select(int32 Index)
	{
		FSoftClassPath ActorPath;
		TSoftClassPtr<AActor> ActorClassSoftPtr;

		if (SpawnAttributeAccessor->Get<FSoftClassPath>(ActorPath, Index, *SpawnAttributeKeys))
		{
			ActorClassSoftPtr = TSoftClassPtr<AActor>(ActorPath);
		}
		else
		{
			FString ActorPathString;
			if (SpawnAttributeAccessor->Get<FString>(ActorPathString, Index, *SpawnAttributeKeys))
			{
				ActorPath = FSoftClassPath(ActorPathString);
				ActorClassSoftPtr = TSoftClassPtr<AActor>(ActorPath);
			}
		}

		if (!ActorPath.IsValid())
		{
			return nullptr;
		}

		UClass* ActorClass = ActorClassSoftPtr.LoadSynchronous();

		if (!ActorClass)
		{
			UBlueprint* Blueprint = Cast<UBlueprint>(ActorPath.TryLoad());
			if (Blueprint)
			{
				ActorClass = Blueprint->GeneratedClass.Get();
			}
		}

		// Finally, we'll try to take the path and check if it matches a class path instead, because the (BP) asset will not be available in packaged builds.
		if (!ActorClass)
		{
			ActorPath = FSoftClassPath(ActorPath.ToString() + TEXT("_C"));
			ActorClassSoftPtr = TSoftClassPtr<AActor>(ActorPath);
			ActorClass = ActorClassSoftPtr.LoadSynchronous();
		}
		
		if (ActorClass && ActorClass->IsChildOf<AActor>())
		{
			return &ElementMap.FindOrAdd(ActorClass);
		}

		PCGLog::LogErrorOnGraph(LOCTEXT("InvalidActorClass", "Invalid actor class in Spawn Actor by Attribute"), Context);
		return nullptr;
	}

	// Disables time-slicing altogether because the code isn't setup for this yet
	int32 TimeSlicingCheckFrequency() const { return std::numeric_limits<int>::max(); }

public:
	TUniquePtr<const IPCGAttributeAccessor> SpawnAttributeAccessor;
	TUniquePtr<const IPCGAttributeAccessorKeys> SpawnAttributeKeys;
	FName SpawnAttribute = NAME_None;
	FPCGContext* Context = nullptr;
};

UPCGSpawnActorSettings::UPCGSpawnActorSettings(const FObjectInitializer& ObjectInitializer)
	: UPCGBaseSubgraphSettings(ObjectInitializer)
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		Option = EPCGSpawnActorOption::NoMerging;
		AttachOptions = EPCGAttachOptions::InFolder;
	}
}

UPCGNode* UPCGSpawnActorSettings::CreateNode() const
{
	return NewObject<UPCGSpawnActorNode>();
}

void UPCGSpawnActorSettings::SetTemplateActorClass(const TSubclassOf<AActor>& InTemplateActorClass)
{
#if WITH_EDITOR
	TeardownBlueprintEvent();
#endif // WITH_EDITOR

	TemplateActorClass = InTemplateActorClass;

#if WITH_EDITOR
	SetupBlueprintEvent();
	RefreshTemplateActor();
#endif // WITH_EDITOR
}

void UPCGSpawnActorSettings::SetAllowTemplateActorEditing(bool bInAllowTemplateActorEditing)
{
	bAllowTemplateActorEditing = bInAllowTemplateActorEditing;

#if WITH_EDITOR
	RefreshTemplateActor();
#endif // WITH_EDITOR
}

bool UPCGSpawnActorSettings::GetAllowTemplateActorEditing() const
{
	return Option != EPCGSpawnActorOption::CollapseActors && bAllowTemplateActorEditing;
}

FPCGElementPtr UPCGSpawnActorSettings::CreateElement() const
{
	return MakeShared<FPCGSpawnActorElement>();
}

UPCGGraphInterface* UPCGSpawnActorSettings::GetGraphInterfaceFromActorSubclass(TSubclassOf<AActor> InTemplateActorClass)
{
	if (!InTemplateActorClass || InTemplateActorClass->HasAnyClassFlags(CLASS_Abstract))
	{
		return nullptr;
	}

	UPCGGraphInterface* Result = nullptr;

	AActor::ForEachComponentOfActorClassDefault<UPCGComponent>(InTemplateActorClass, [&](const UPCGComponent* PCGComponent)
	{
		// If there is no graph, there is no graph instance
		if (PCGComponent->GetGraph() && PCGComponent->bActivated)
		{
			Result = PCGComponent->GetGraphInstance();
			return false;
		}
		
		return true;
	});

	return Result;
}

UPCGGraphInterface* UPCGSpawnActorSettings::GetSubgraphInterface() const
{
	return GetGraphInterfaceFromActorSubclass(TemplateActorClass);
}

void UPCGSpawnActorSettings::BeginDestroy()
{
#if WITH_EDITOR
	TeardownBlueprintEvent();
#endif // WITH_EDITOR

	Super::BeginDestroy();
}

#if WITH_EDITOR
void UPCGSpawnActorSettings::SetupBlueprintEvent()
{
	if (UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(TemplateActorClass))
	{
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UPCGSpawnActorSettings::OnObjectsReplaced);
	}
}

void UPCGSpawnActorSettings::TeardownBlueprintEvent()
{
	if (UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(TemplateActorClass))
	{
		FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	}
}
#endif

void UPCGSpawnActorSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Since the template actor editing is set to false by default, this needs to be corrected on post-load for proper deprecation
	// Note: Before we were not allowing template editing if we were spawning as attribute, so force it at false in that case.
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGAllowTemplateWhenSpawningFromAttribute && bSpawnByAttribute)
	{
		bAllowTemplateActorEditing = false;
	}
	else if (TemplateActor && Option != EPCGSpawnActorOption::CollapseActors)
	{
		bAllowTemplateActorEditing = true;
	}

	SetupBlueprintEvent();

	if (TemplateActorClass)
	{
		if (TemplateActor)
		{
			TemplateActor->ConditionalPostLoad();
		}
	}

	RefreshTemplateActor();
#endif // WITH_EDITOR
}

#if WITH_EDITOR
EPCGChangeType UPCGSpawnActorSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSpawnActorSettings, TemplateActorClass) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSpawnActorSettings, Option) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSpawnActorSettings, bSpawnByAttribute))
	{
		ChangeType |= EPCGChangeType::Structural;
	}
	
	ChangeType |= DataLayerSettings.GetChangeTypeForProperty(InPropertyName);

	return ChangeType;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGSpawnActorSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties = DefaultPointInputPinProperties();
	PinProperties.Append(DataLayerSettings.InputPinProperties());
	return PinProperties;
}

TObjectPtr<UPCGGraphInterface> UPCGSpawnActorNode::GetSubgraphInterface() const
{
	TObjectPtr<UPCGSpawnActorSettings> Settings = Cast<UPCGSpawnActorSettings>(GetSettings());
	return (Settings && Settings->Option != EPCGSpawnActorOption::NoMerging) ? Settings->GetSubgraphInterface() : nullptr;
}

#if WITH_EDITOR
void UPCGSpawnActorSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGSpawnActorSettings, TemplateActorClass))
	{
		TeardownBlueprintEvent();
	}
}

void UPCGSpawnActorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		const FName& PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGSpawnActorSettings, TemplateActorClass))
		{
			SetupBlueprintEvent();
			RefreshTemplateActor();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGSpawnActorSettings, bAllowTemplateActorEditing))
		{
			RefreshTemplateActor();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPCGSpawnActorSettings::PreEditUndo()
{
	TeardownBlueprintEvent();

	Super::PreEditUndo();
}

void UPCGSpawnActorSettings::PostEditUndo()
{
	Super::PostEditUndo();

	SetupBlueprintEvent();
	RefreshTemplateActor();
}

void UPCGSpawnActorSettings::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances)
{
	if (!TemplateActor)
	{
		return;
	}

	if (UObject* NewObject = InOldToNewInstances.FindRef(TemplateActor))
	{
		TemplateActor = Cast<AActor>(NewObject);
		DirtyCache();
		OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Settings);
	}
}

void UPCGSpawnActorSettings::RefreshTemplateActor()
{
	// Implementation note: this is similar to the child actor component implementation
	if (TemplateActorClass && GetAllowTemplateActorEditing())
	{
		const bool bCreateNewTemplateActor = (!TemplateActor || TemplateActor->GetClass() != TemplateActorClass);

		if (bCreateNewTemplateActor)
		{
			AActor* NewTemplateActor = NewObject<AActor>(GetTransientPackage(), TemplateActorClass, NAME_None, RF_ArchetypeObject | RF_Transactional | RF_Public);

			if (TemplateActor)
			{
				UEngine::FCopyPropertiesForUnrelatedObjectsParams Options;
				Options.bNotifyObjectReplacement = true;
				UEngine::CopyPropertiesForUnrelatedObjects(TemplateActor, NewTemplateActor, Options);

				TemplateActor->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

				TMap<UObject*, UObject*> OldToNew;
				OldToNew.Emplace(TemplateActor, NewTemplateActor);
				GEngine->NotifyToolsOfObjectReplacement(OldToNew);

				TemplateActor->MarkAsGarbage();
			}

			TemplateActor = NewTemplateActor;

			// Record initial object state in case we're in a transaction context.
			TemplateActor->Modify();

			// Outer to this object
			TemplateActor->Rename(nullptr, this, REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
	}
	else
	{
		if (TemplateActor)
		{
			TemplateActor->MarkAsGarbage();
		}

		TemplateActor = nullptr;
	}
}

#endif // WITH_EDITOR

bool FPCGSpawnActorElement::ExecuteInternal(FPCGContext* InContext) const
{
	FPCGSubgraphContext* Context = static_cast<FPCGSubgraphContext*>(InContext);

	const UPCGSpawnActorSettings* Settings = Context->GetInputSettings<UPCGSpawnActorSettings>();
	check(Settings);

	if (!Context->bScheduledSubgraph)
	{
		return SpawnAndPrepareSubgraphs(Context, Settings);
	}
	else if (Context->bIsPaused)
	{
		// Should not happen once we skip it in the graph executor
		return false;
	}
	else
	{
		// TODO: Currently, we don't gather results from subgraphs, but we could (in a single pin).
		return true;
	}
}

bool FPCGSpawnActorElement::SpawnAndPrepareSubgraphs(FPCGSubgraphContext* Context, const UPCGSpawnActorSettings* Settings) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCSpawnActorElement::Execute);

	// Early out
	if (!Settings->bSpawnByAttribute)
	{
		if (!Settings->TemplateActorClass || Settings->TemplateActorClass->HasAnyClassFlags(CLASS_Abstract))
		{
			const FText ClassName = Settings->TemplateActorClass ? FText::FromString(Settings->TemplateActorClass->GetFName().ToString()) : FText::FromName(NAME_None);
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InvalidTemplateActorClass", "Invalid template actor class '{0}'"), ClassName));
			return true;
		}

		if (!ensure(!Settings->TemplateActor || Settings->TemplateActor->IsA(Settings->TemplateActorClass)))
		{
			return true;
		}
	}

	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	if (!ExecutionSource)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidSource", "Invalid Source."));
		return true;
	}

	FPCGManagedResourceContainerHelper ManagedResourcesContainerHelper(ExecutionSource);

	// Check if we can reuse existing resources - note that this is done on a per-settings basis when collapsed,
	// Otherwise we'll check against merged crc 
	bool bFullySkippedDueToReuse = false;

	if (CVarAllowActorReuse.GetValueOnAnyThread() && ManagedResourcesContainerHelper.IsValid())
	{
		// Compute CRC if it has not been computed (it likely isn't, but this is to futureproof this)
		if (!Context->DependenciesCrc.IsValid())
		{
			GetDependenciesCrc(FPCGGetDependenciesCrcParams(&Context->InputData, Settings, Context->ExecutionSource.Get()), Context->DependenciesCrc);
		}

		if (Context->DependenciesCrc.IsValid())
		{
			if (Settings->Option == EPCGSpawnActorOption::CollapseActors)
			{
				TArray<UPCGManagedISMComponent*> MISMCs;
				ManagedResourcesContainerHelper.ForEachManagedResource([&MISMCs, &Context](UPCGManagedResource* InResource)
				{
					if (UPCGManagedISMComponent* Resource = Cast<UPCGManagedISMComponent>(InResource))
					{
						if (Resource->GetCrc().IsValid() && Resource->GetCrc() == Context->DependenciesCrc)
						{
							MISMCs.Add(Resource);
						}
					}
				});

				for (UPCGManagedISMComponent* MISMC : MISMCs)
				{
					if (!MISMC->IsMarkedUnused() && Settings->bWarnOnIdenticalSpawn)
					{
						// TODO: Revisit if the stack is added to the managed components at creation
						PCGLog::LogWarningOnGraph(LOCTEXT("IdenticalISMCSpawn", "Identical ISM Component spawn occurred. It may be beneficial to re-check graph logic for identical spawn conditions (same actor at same location, etc) or repeated nodes."), Context);
					}

					MISMC->MarkAsReused();
				}

				if (!MISMCs.IsEmpty())
				{
					bFullySkippedDueToReuse = true;
				}
			}
		}
	}

	// Pass-through exclusions & settings
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

#if WITH_EDITOR
	const bool bGenerateOutputsWithActorReference = (Settings->Option != EPCGSpawnActorOption::CollapseActors);
#else
	const bool bGenerateOutputsWithActorReference = (Settings->Option != EPCGSpawnActorOption::CollapseActors) && Context->Node && Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultOutputLabel);
#endif

	const bool bHasAuthority = ExecutionSource->GetExecutionState().HasAuthority();

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("SpawnActors", "Spawning actors from PCG"), ExecutionSource->GetExecutionState().UseTransactions());
#endif

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Input.Data);

		if (!SpatialData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data"));
			continue;
		}

		AActor* TargetActor = Settings->RootActor.Get() ? Settings->RootActor.Get() : Context->GetTypedExecutionTarget<AActor>();
		UObject* TargetWorldObject = TargetActor ? TargetActor : Context->GetExecutionTarget();

		if (!TargetActor && Settings->Option == EPCGSpawnActorOption::CollapseActors)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidTargetActor", "Invalid target actor. Ensure TargetActor member is initialized when creating SpatialData."));
			continue;
		}

		if (!TargetWorldObject || !TargetWorldObject->GetWorld())
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidTargetObject", "Invalid target object. Will be unable to spawn actors since we cannot resolve a world."));
			continue;
		}

		// First, create target instance transforms
		const UPCGBasePointData* PointData = SpatialData->ToBasePointData(Context);

		if (!PointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("NoPointDataInInput", "Unable to get point data from input"));
			continue;
		}

		if (PointData->GetNumPoints() == 0)
		{
			PCGE_LOG(Verbose, LogOnly, LOCTEXT("SkippedNoPoints", "Skipped - no points"));
			continue;
		}

		auto SpawnOrCollapse = [this, bHasAuthority, &Context, &TargetActor, &TargetWorldObject, &Settings](TSubclassOf<AActor> TemplateActorClass, AActor* TemplateActor, FPCGTaggedData& Output, const UPCGBasePointData* PointData, UPCGBasePointData* OutPointData)
		{
			const bool bSpawnedActorsRequireAuthority = (TemplateActor ? TemplateActor->GetIsReplicated() : CastChecked<AActor>(TemplateActorClass->GetDefaultObject())->GetIsReplicated());

			if (Settings->Option == EPCGSpawnActorOption::CollapseActors)
			{
				CollapseIntoTargetActor(Context, TargetActor, TemplateActorClass, PointData);
			}
			else if (bHasAuthority || !bSpawnedActorsRequireAuthority)
			{
				SpawnActors(Context, TargetWorldObject, TemplateActorClass, TemplateActor, Output, PointData, OutPointData);
			}
		};

		UPCGBasePointData* OutPointData = nullptr;
		if (bGenerateOutputsWithActorReference)
		{
			OutPointData = FPCGContext::NewPointData_AnyThread(Context);

			FPCGInitializeFromDataParams InitializeFromDataParams(PointData);
			InitializeFromDataParams.bInheritSpatialData = false;

			OutPointData->InitializeFromDataWithParams(InitializeFromDataParams);
		}

		FPCGTaggedData Output = Input;

		if (Settings->bSpawnByAttribute && (!bFullySkippedDueToReuse || bGenerateOutputsWithActorReference))
		{
			FPCGSpawnActorPartitionByAttribute Selector(Settings->SpawnAttribute, Context);
			int32 CurrentPointIndex = 0;

			// Selection is still needed if are fully skipped in order to write to the OutPointData.
			Selector.SelectMultiple(*Context, PointData, CurrentPointIndex, PointData->GetNumPoints(), OutPointData);

			if (!bFullySkippedDueToReuse)
			{
				for (auto& Element : Selector.ElementMap)
				{
					FPCGTaggedData PartialInput = Input;
					PartialInput.Data = Element.Value.PartitionData;

					SpawnOrCollapse(Element.Key, Settings->TemplateActor, PartialInput, Cast<UPCGBasePointData>(Element.Value.PartitionData), OutPointData);

					// Exception case here: if we've spawned actors but are merging the PCG inputs,
					// normally this node is taken as a subgraph node (e.g. no need to do anything more than forwarding the inputs)
					// However, if we`re in the spawn by attribute case, we need to dispatch it here.
					if (Settings->Option != EPCGSpawnActorOption::NoMerging)
					{
						// TODO: maybe consider a version that would support multi PCG
						if (UPCGGraphInterface* GraphInterface = UPCGSpawnActorSettings::GetGraphInterfaceFromActorSubclass(Element.Key))
						{
							FPCGDataCollection SubgraphInputData;
							SubgraphInputData.TaggedData.Add(PartialInput);

							Context->AddToReferencedObjects(SubgraphInputData);

							// Prepare the invocation stack - which is the stack up to this node, and then this node, then a loop index
							const FPCGStack* Stack = Context->GetStack();
							FPCGStack InvocationStack = ensure(Stack) ? *Stack : FPCGStack();

							UPCGGraph* Graph = GraphInterface->GetGraph();

							FPCGTaskId SubgraphTaskId = Context->ScheduleGraph(FPCGScheduleGraphParams(
								Graph,
								Context->ExecutionSource.Get(),
								MakeShared<FPCGPreGraphElement>(GraphInterface),
								MakeShared<FPCGInputForwardingElement>(SubgraphInputData),
								/*Dependencies=*/{},
								&InvocationStack,
								/*bAllowHierarchicalGeneration=*/false));

							if (SubgraphTaskId != InvalidPCGTaskId)
							{
								Context->SubgraphTaskIds.Add(SubgraphTaskId);
							}
						}
					}
				}
			}
		}
		else if(!bFullySkippedDueToReuse)
		{
			// Spawn actors/populate ISM
			FPCGTaggedData InputCopy = Input;
			SpawnOrCollapse(Settings->TemplateActorClass, Settings->TemplateActor, InputCopy, PointData, OutPointData);
		}

		// Update the data in the output to the final data gathered
		if (OutPointData)
		{
			Output.Data = OutPointData;
		}

		// Finally, pass through the input, in all cases: 
		// - if it's not merged, will be the input points directly
		// - if it's merged but there is no subgraph, will be the input points directly
		// - if it's merged and there is a subgraph, we'd need to pass the data for it to be given to the subgraph
		Outputs.Add(Output);
	}

	// If we've dispatched dynamic execution, we should queue a task here to wait for those
	if (!Context->SubgraphTaskIds.IsEmpty())
	{
		Context->bScheduledSubgraph = true;
		Context->bIsPaused = true;
		Context->DynamicDependencies.Append(Context->SubgraphTaskIds);

		return false;
	}
	else
	{
		return true;
	}
}

void FPCGSpawnActorElement::CollapseIntoTargetActor(FPCGSubgraphContext* Context, AActor* TargetActor, TSubclassOf<AActor> TemplateActorClass, const UPCGBasePointData* PointData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSpawnActorElement::ExecuteInternal::CollapseActors);
	check(Context && TargetActor && PointData);

	const int32 NumPoints = PointData->GetNumPoints();
	if (NumPoints == 0)
	{
		return;
	}

	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	if (!ExecutionSource)
	{
		return;
	}

	const UPCGSpawnActorSettings* Settings = Context->GetInputSettings<UPCGSpawnActorSettings>();
	check(Settings);
	
	FPCGCrc SettingsCrc = Settings->GetSettingsCrc();
	ensure(SettingsCrc.IsValid());

	TMap<FPCGISMComponentBuilderParams, TArray<FTransform>> MeshDescriptorTransforms;

	AActor::ForEachComponentOfActorClassDefault<UStaticMeshComponent>(TemplateActorClass, [&MeshDescriptorTransforms, SettingsCrc](const UStaticMeshComponent* StaticMeshComponent)
	{
		FPCGISMComponentBuilderParams Params;
		Params.Descriptor.InitFrom(StaticMeshComponent);
		Params.SettingsCrc = SettingsCrc;
		// TODO: No custom data float support?

		TArray<FTransform>& Transforms = MeshDescriptorTransforms.FindOrAdd(Params);

		if (const UInstancedStaticMeshComponent* InstancedStaticMeshComponent = Cast<UInstancedStaticMeshComponent>(StaticMeshComponent))
		{
			const int32 NumInstances = InstancedStaticMeshComponent->GetInstanceCount();
			Transforms.Reserve(Transforms.Num() + NumInstances);

			for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; InstanceIndex++)
			{
				FTransform InstanceTransform;
				if (InstancedStaticMeshComponent->GetInstanceTransform(InstanceIndex, InstanceTransform))
				{
					Transforms.Add(InstanceTransform);
				}
			}
		}
		else
		{
			Transforms.Add(StaticMeshComponent->GetRelativeTransform());
		}

		return true;
	});

	const TConstPCGValueRange<FTransform> TransformRange = PointData->GetConstTransformValueRange();

	for (const TPair<FPCGISMComponentBuilderParams, TArray<FTransform>>& ISMCBuilderTransforms : MeshDescriptorTransforms)
	{
		const FPCGISMComponentBuilderParams& ISMCParams = ISMCBuilderTransforms.Key;

		UPCGManagedISMComponent* MISMC = UPCGActorHelpers::GetOrCreateManagedISMC(TargetActor, ExecutionSource, ISMCParams, Context);
		if (!MISMC)
		{
			continue;
		}

		MISMC->SetCrc(Context->DependenciesCrc);

		UInstancedStaticMeshComponent* ISMC = MISMC->GetComponent();
		check(ISMC);

		const TArray<FTransform>& ISMCTransforms = ISMCBuilderTransforms.Value;

		TArray<FTransform> Transforms;
		Transforms.Reserve(NumPoints * ISMCTransforms.Num());
		for (int32 PointIndex = 0; PointIndex < NumPoints; PointIndex++)
		{
			for (int32 TransformIndex = 0; TransformIndex < ISMCTransforms.Num(); TransformIndex++)
			{
				const FTransform& Transform = ISMCTransforms[TransformIndex];
				Transforms.Add(Transform * TransformRange[PointIndex]);
			}
		}

		// Fill in custom data (?)
		ISMC->AddInstances(Transforms, false, true);
		ISMC->UpdateBounds();

		PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("InstanceCreationInfo", "Added {0} instances of mesh '{1}' to ISMC '{2}' on actor '{3}'"),
			Transforms.Num(), FText::FromString(ISMC->GetStaticMesh().GetName()), FText::FromString(ISMC->GetName()), FText::FromString(TargetActor->GetActorNameOrLabel())));
	}
}

void FPCGSpawnActorElement::SpawnActors(FPCGSubgraphContext* Context, UObject* TargetWorldObject, TSubclassOf<AActor> InTemplateActorClass, AActor* InTemplateActor, FPCGTaggedData& Output, const UPCGBasePointData* PointData, UPCGBasePointData* OutPointData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSpawnActorElement::ExecuteInternal::SpawnActors);
	check(Context && TargetWorldObject && PointData);
	check(IsInGameThread());

	if (PointData->GetNumPoints() == 0 || !InTemplateActorClass)
	{
		return;
	}

	// Verify first if the template actor and template class matches. If they are not, the spawn will fail.
	if (InTemplateActor && !InTemplateActorClass->IsChildOf(InTemplateActor->GetClass()))
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidTemplateClass", "Tried to spawn an actor of class {0}, while providing a template actor of type {1}. This is invalid. The template actor must be of a parent class of the spawned actor class."),
			FText::FromName(InTemplateActorClass->GetFName()),
			FText::FromName(InTemplateActor->GetClass()->GetFName())));
		return;
	}

	int32 OutPointOffset = 0;
	FPCGMetadataAttribute<FSoftObjectPath>* ActorReferenceAttribute = nullptr;

	if (OutPointData)
	{
		OutPointOffset = OutPointData->GetNumPoints();
		OutPointData->SetNumPoints(OutPointOffset + PointData->GetNumPoints());
		PointData->CopyPointsTo(OutPointData, 0, OutPointOffset, PointData->GetNumPoints());
		ActorReferenceAttribute = OutPointData->MutableMetadata()->FindOrCreateAttribute<FSoftObjectPath>(PCGPointDataConstants::ActorReferenceAttribute, FSoftObjectPath(), /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/false);
	}

	const UPCGSpawnActorSettings* Settings = Context->GetInputSettings<UPCGSpawnActorSettings>();
	check(Settings && Settings->Option != EPCGSpawnActorOption::CollapseActors);

	const bool bForceDisableActorParsing = (Settings->bForceDisableActorParsing);

	AActor* TemplateActor = nullptr;
	// We can use the template actor as-is if the classes matches perfectly.
	const bool bClassMismatch = InTemplateActor && InTemplateActorClass != InTemplateActor->GetClass();

	if (InTemplateActor && !bClassMismatch)
	{
		if (Settings->SpawnedActorPropertyOverrideDescriptions.IsEmpty())
		{
			TemplateActor = InTemplateActor;
		}
		else
		{
			TemplateActor = DuplicateObject(InTemplateActor, GetTransientPackage());
		}
	}
	else
	{
		if (Settings->SpawnedActorPropertyOverrideDescriptions.IsEmpty() && !bClassMismatch)
		{
			TemplateActor = Cast<AActor>(InTemplateActorClass->GetDefaultObject());
		}
		else
		{
			TemplateActor = NewObject<AActor>(GetTransientPackage(), InTemplateActorClass, NAME_None, RF_ArchetypeObject);

			// If we have a class mismatch, copy the properties from the template actor.
			if (bClassMismatch)
			{
				UEngine::CopyPropertiesForUnrelatedObjects(InTemplateActor, TemplateActor);
			}
		}
	}

	check(TemplateActor);

	FPCGObjectOverrides ActorOverrides(TemplateActor);
	ActorOverrides.Initialize(Settings->SpawnedActorPropertyOverrideDescriptions, TemplateActor, PointData, Context);

	FActorSpawnParameters SpawnParams;
	SpawnParams.Template = TemplateActor;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

	AActor* TargetActor = Cast<AActor>(TargetWorldObject);
	SpawnParams.OverrideLevel = TargetActor ? TargetActor->GetLevel() : Cast<ULevel>(TargetWorldObject);

	IPCGGraphExecutionSource* Source = Context->ExecutionSource.Get();

	if (!Source)
	{
		return; // won't be able to create resources correctly
	}

	UPCGActorHelpers::FSpawnDefaultActorParams SpawnDefaultActorParams(TargetWorldObject->GetWorld(), InTemplateActorClass, FTransform::Identity, SpawnParams);
	SpawnDefaultActorParams.bForceStaticMobility = false; // Always respect the actor's mobility
	SpawnDefaultActorParams.bIsRuntime = (Source && Source->GetExecutionState().IsManagedByRuntimeGenSystem());
#if WITH_EDITOR
	SpawnDefaultActorParams.bIsPreviewActor = (Source && Source->GetExecutionState().IsInPreviewMode());

	int32 DataLayerCrc = 0;
	if (TargetActor)
	{
		SpawnDefaultActorParams.DataLayerInstances = PCGDataLayerHelpers::GetDataLayerInstancesAndCrc(Context, Settings->DataLayerSettings, TargetActor, DataLayerCrc);
	}
	

	int32 HLODLayerCrc = 0;
	if (TargetActor)
	{
		SpawnDefaultActorParams.HLODLayer = PCGHLODHelpers::GetHLODLayerAndCrc(Context, Settings->HLODSettings, TargetActor, TemplateActor, HLODLayerCrc);
	}
#endif

	const bool bForceCallGenerate = (Settings->GenerationTrigger == EPCGSpawnActorGenerationTrigger::ForceGenerate);
#if WITH_EDITOR
	const bool bOnLoadCallGenerate = (Settings->GenerationTrigger == EPCGSpawnActorGenerationTrigger::Default);
#else
	const bool bOnLoadCallGenerate = (Settings->GenerationTrigger == EPCGSpawnActorGenerationTrigger::Default ||
		Settings->GenerationTrigger == EPCGSpawnActorGenerationTrigger::DoNotGenerateInEditor);
#endif
	UPCGSubsystem* Subsystem = UWorld::GetSubsystem<UPCGSubsystem>(Source ? Source->GetExecutionState().GetWorld() : nullptr);

	FPCGManagedResourceContainerHelper ManagedResourcesContainerHelper(Source);
	
	if (!ManagedResourcesContainerHelper.IsValid())
	{
		return; // won't be able to manage resources correctly
	}

	// Try to reuse actors if they are preexisting
	UPCGManagedActors* ReusedManagedActorsResource = nullptr;
	FPCGCrc InputDependenciesCrc;
	if (CVarAllowActorReuse.GetValueOnAnyThread())
	{
		FPCGDataCollection SingleInputCollection;
		SingleInputCollection.TaggedData.Emplace_GetRef().Data = PointData;
		// Need to do a full CRC here as the PointData might not be the original input (if there was some partitioning because of spawning by attribute). 
		// Since it is spawning by attribute, all point data will be different.
		SingleInputCollection.ComputeCrcs(/*bFullDataCrc=*/true);

		GetDependenciesCrc(FPCGGetDependenciesCrcParams(&SingleInputCollection, Settings, Context->ExecutionSource.Get()), InputDependenciesCrc);

#if WITH_EDITOR
		if (DataLayerCrc != 0)
		{
			InputDependenciesCrc.Combine(DataLayerCrc);
		}

		if (HLODLayerCrc != 0)
		{
			InputDependenciesCrc.Combine(HLODLayerCrc);
		}
#endif

		if (InputDependenciesCrc.IsValid())
		{
			ManagedResourcesContainerHelper.ForEachManagedResource([&ReusedManagedActorsResource, &InputDependenciesCrc, &Context, NumPoints = PointData->GetNumPoints(), bIsPreview = SpawnDefaultActorParams.bIsPreviewActor](UPCGManagedResource* InResource)
			{
				if (ReusedManagedActorsResource)
				{
					return;
				}

				if (UPCGManagedActors* Resource = Cast<UPCGManagedActors>(InResource))
				{
#if WITH_EDITOR
					if (Resource->IsPreview() != bIsPreview)
					{
						return;
					}
#endif

					// We can only re-use the resource if it matches the number of points (if actor failed to spawned for whatever reason, we won't know which point is associated with the fail)
					if (Resource->GetCrc().IsValid() && Resource->GetCrc() == InputDependenciesCrc && Resource->GetConstGeneratedActors().Num() == NumPoints)
					{
						ReusedManagedActorsResource = Resource;
					}
				}
			});
		}
	}

	TArray<AActor*> ProcessedActors;
	const bool bActorsHavePCGComponents = (UPCGSpawnActorSettings::GetGraphInterfaceFromActorSubclass(InTemplateActorClass) != nullptr);

	if (ReusedManagedActorsResource)
	{
		// If the actors are fully independent, we might need to make sure to call Generate if the underlying graph has changed - e.g. if the actor is dirty
		ReusedManagedActorsResource->MarkAsReused();

		// If we're in the no-merge case, keep track of these actors to generate.
		// Also set to the output data the actor reference.
		if (Settings->Option == EPCGSpawnActorOption::NoMerging)
		{
			TPCGValueRange<int64> MetadataEntryRange = OutPointData ? OutPointData->GetMetadataEntryValueRange() : TPCGValueRange<int64>();

			const TArray<TSoftObjectPtr<AActor>>& GeneratedActors = ReusedManagedActorsResource->GetConstGeneratedActors();
			for (int32 i = 0; i < GeneratedActors.Num(); ++i)
			{
				const TSoftObjectPtr<AActor>& ManagedActorPtr = GeneratedActors[i];

				// Write to out data the actor reference
				if (OutPointData && ActorReferenceAttribute)
				{
					int64& MetadataEntry = MetadataEntryRange[i + OutPointOffset];
					OutPointData->Metadata->InitializeOnSet(MetadataEntry);
					ActorReferenceAttribute->SetValue(MetadataEntry, ManagedActorPtr.ToSoftObjectPath());
				}

				if (bActorsHavePCGComponents)
				{
					if (AActor* ManagedActor = ManagedActorPtr.Get())
					{
						ProcessedActors.Add(ManagedActor);
					}
				}
			}
		}
	}
	else
	{
		TArray<FName> NewActorTags = GetNewActorTags(Context, TargetActor, Settings->bInheritActorTags, Settings->TagsToAddOnActors);

		// Create managed resource for actor tracking
		UPCGManagedActors* ManagedActors = NewObject<UPCGManagedActors>(Cast<UObject>(Source));
#if WITH_EDITOR
		ManagedActors->SetIsPreview(SpawnDefaultActorParams.bIsPreviewActor);
#endif
		ManagedActors->SetCrc(InputDependenciesCrc);
		ManagedActors->bSupportsReset = !Settings->bDeleteActorsBeforeGeneration;

		// If generated actors are not directly attached, place them in a subfolder for tidiness.
		FString GeneratedActorsFolderPath;
#if WITH_EDITOR
		PCGHelpers::GetGeneratedActorsFolderPath(TargetActor, Context, Settings->AttachOptions, GeneratedActorsFolderPath);
#endif

		const UFunction* FunctionPrototypeWithNoParams = UPCGFunctionPrototypes::GetPrototypeWithNoParams();
		const UFunction* FunctionPrototypeWithPointAndMetadata = UPCGFunctionPrototypes::GetPrototypeWithPointAndMetadata();

		const TArray<UFunction*> PostSpawnFunctions = PCGHelpers::FindUserFunctions(
			InTemplateActorClass,
			Settings->PostSpawnFunctionNames,
			{ FunctionPrototypeWithNoParams, FunctionPrototypeWithPointAndMetadata },
			Context);

		bool bAllActorOverridesSucceeded = true;

#if WITH_EDITOR
		// Since the actor params take a string view as a parameter, we need a temporary string on the stack.
		FString TempLabelString;
#endif

		const FConstPCGPointValueRanges ValueRanges(PointData);

		TPCGValueRange<int64> OutMetadataEntryRange = OutPointData ? OutPointData->GetMetadataEntryValueRange() : TPCGValueRange<int64>();

		for (int32 i = 0; i < PointData->GetNumPoints(); ++i)
		{
			bAllActorOverridesSucceeded &= ActorOverrides.Apply(i);

			SpawnDefaultActorParams.Transform = ValueRanges.TransformRange[i];

#if WITH_EDITOR
			if (!Settings->ActorLabel.IsEmpty())
			{
				TempLabelString = FString::Format(TEXT("{0}{1}"), { Settings->ActorLabel, i });
				SpawnDefaultActorParams.SpawnParams.InitialActorLabel = TempLabelString;
			}
#endif

			AActor* GeneratedActor = UPCGActorHelpers::SpawnDefaultActor(SpawnDefaultActorParams);

			if (!GeneratedActor)
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("ActorSpawnFailed", "Failed to spawn actor on point with index {0}"), i));
				continue;
			}

			// HACK: until UE-62747 is fixed, we have to force set the scale after spawning the actor
			GeneratedActor->SetActorRelativeScale3D(ValueRanges.TransformRange[i].GetScale3D());
			GeneratedActor->Tags.Append(NewActorTags);
			PCGHelpers::AttachToParent(GeneratedActor, TargetActor, Settings->AttachOptions, Context, GeneratedActorsFolderPath);

			for (UFunction* PostSpawnFunction : PostSpawnFunctions)
			{
				if (PostSpawnFunction->IsSignatureCompatibleWith(FunctionPrototypeWithNoParams))
				{
					GeneratedActor->ProcessEvent(PostSpawnFunction, nullptr);
				}
				else if (PostSpawnFunction->IsSignatureCompatibleWith(FunctionPrototypeWithPointAndMetadata))
				{
					TPair<FPCGPoint, const UPCGMetadata*> PointAndMetadata = { ValueRanges.GetPoint(i), PointData->ConstMetadata()};
					GeneratedActor->ProcessEvent(PostSpawnFunction, &PointAndMetadata);
				}
			}

			ManagedActors->GetMutableGeneratedActors().AddUnique(GeneratedActor);

			if (bActorsHavePCGComponents)
			{
				ProcessedActors.Add(GeneratedActor);
			}

			// Write to out data the actor reference
			if (OutPointData && ActorReferenceAttribute)
			{
				int64& MetadataEntry = OutMetadataEntryRange[i + OutPointOffset];
				OutPointData->Metadata->InitializeOnSet(MetadataEntry);
				ActorReferenceAttribute->SetValue(MetadataEntry, FSoftObjectPath(GeneratedActor));
			}
		}

		if (!bAllActorOverridesSucceeded)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("ActorOverridesFailed", "At least one actor property override failed."));
		}

		ManagedResourcesContainerHelper.AddManagedResource(ManagedActors);

		PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("GenerationInfo", "Generated {0} actors"), PointData->GetNumPoints()));
	}

	UPCGComponent* SourceComponent = Cast<UPCGComponent>(Source);

	// Setup & Generate on PCG components if needed
	for (AActor* Actor : ProcessedActors)
	{
		TInlineComponentArray<UPCGComponent*, 1> PCGComponents;
		Actor->GetComponents(PCGComponents);

		for (UPCGComponent* PCGComponent : PCGComponents)
		{
#if WITH_EDITOR
			// For both pre-existing and new actors, we need to make sure we're inline with loading/generation as needed
			if (SourceComponent && PCGComponent->GetEditingMode() != SourceComponent->GetEditingMode())
			{
				PCGComponent->SetEditingMode(/*CurrentEditingMode=*/SourceComponent->GetEditingMode(), /*SerializedEditingMode=*/SourceComponent->GetEditingMode());
				PCGComponent->ChangeTransientState(SourceComponent->GetEditingMode());
			}
#endif // WITH_EDITOR

			if (Settings->Option == EPCGSpawnActorOption::NoMerging)
			{
				if (bForceDisableActorParsing)
				{
					PCGComponent->bParseActorComponents = false;
				}

				if (bForceCallGenerate || (bOnLoadCallGenerate && PCGComponent->GenerationTrigger == EPCGComponentGenerationTrigger::GenerateOnLoad))
				{
					if (Subsystem)
					{
						Subsystem->RegisterOrUpdateExecutionSource(PCGComponent);
					}

					FPCGTaskId SubgraphTaskId = Subsystem ? Subsystem->ScheduleGraph(PCGComponent, {}) : PCGComponent->GenerateLocalGetTaskId(/*bForce=*/true);
					if (SubgraphTaskId != InvalidPCGTaskId)
					{
						Context->SubgraphTaskIds.Add(SubgraphTaskId);
					}
				}
			}
			else // otherwise, they will be taken care of as-if a subgraph (either dynamically or statically)
			{
				PCGComponent->bActivated = false;
			}
		}
	}
}

TArray<FName> FPCGSpawnActorElement::GetNewActorTags(FPCGContext* Context, AActor* TargetActor, bool bInheritActorTags, const TArray<FName>& AdditionalTags) const
{
	TArray<FName> NewActorTags;
	// Prepare actor tags
	if (bInheritActorTags && TargetActor)
	{
		// Special case: if the current target actor is a partition, we'll reach out
		// and find the original actor tags
		if (APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(TargetActor))
		{
			// @todo_pcg review for execution source
			if (UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get()))
			{
				if (UPCGComponent* OriginalComponent = PartitionActor->GetOriginalComponent(SourceComponent))
				{
					check(OriginalComponent->GetOwner());
					NewActorTags = OriginalComponent->GetOwner()->Tags;
				}
			}
		}
		else
		{
			NewActorTags = TargetActor->Tags;
		}
	}

	NewActorTags.AddUnique(PCGHelpers::DefaultPCGActorTag);

	for (const FName& AdditionalTag : AdditionalTags)
	{
		NewActorTags.AddUnique(AdditionalTag);
	}

	return NewActorTags;
}

#if WITH_EDITOR
EPCGSettingsType UPCGSpawnActorSettings::GetType() const
{
	return GetSubgraph() ? EPCGSettingsType::Subgraph : EPCGSettingsType::Spawner;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
