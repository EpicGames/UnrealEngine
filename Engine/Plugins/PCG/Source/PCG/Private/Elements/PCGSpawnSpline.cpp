// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSpawnSpline.h"

#include "PCGContext.h"
#include "PCGManagedResource.h"
#include "PCGManagedResourceContainer.h"
#include "Data/PCGSplineData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGHelpers.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSpawnSpline)

#define LOCTEXT_NAMESPACE "PCGSpawnSplineElement"

#if WITH_EDITOR
FName UPCGSpawnSplineSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SpawnSplineComponent"));
}

FText UPCGSpawnSplineSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Spawn Spline Component");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGSpawnSplineSettings::CreateElement() const
{
	return MakeShared<FPCGSpawnSplineElement>();
}

TArray<FPCGPinProperties> UPCGSpawnSplineSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spline).SetRequiredPin();
	return Properties;
}

TArray<FPCGPinProperties> UPCGSpawnSplineSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spline);
	return Properties;
}

bool FPCGSpawnSplineElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSpawnSplineElement::Execute);

	check(InContext);

	const UPCGSpawnSplineSettings* Settings = InContext->GetInputSettings<UPCGSpawnSplineSettings>();
	check(Settings);

	IPCGGraphExecutionSource* ExecutionSource = InContext->ExecutionSource.Get();
	UObject* ExecutionSourceObject = Cast<UObject>(ExecutionSource);
	FPCGManagedResourceContainerHelper ManagedResourcesContainerHelper(ExecutionSource);

	if (!*Settings->SplineComponent || !ExecutionSource || !ManagedResourcesContainerHelper.IsValid())
	{
		return true;
	}

	AActor* TargetActor = Settings->TargetActor.Get() ? Settings->TargetActor.Get() : InContext->GetTypedExecutionTarget<AActor>();
	if (!TargetActor)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("InvalidTargetActor", "Invalid target actor. Ensure TargetActor member is initialized when creating SpatialData."), InContext);
		return true;
	}

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	
	for (const FPCGTaggedData& InputData : Inputs)
	{
		const UPCGSplineData* InSplineData = Cast<const UPCGSplineData>(InputData.Data);
		if (!InSplineData)
		{
			continue;
		}

		TSubclassOf<USplineComponent> SplineComponentToSpawn = Settings->SplineComponent;
		if (Settings->bSpawnComponentFromAttribute)
		{
			const FPCGAttributePropertyInputSelector InputSelector = Settings->SpawnComponentFromAttributeName.CopyAndFixLast(InSplineData);
			TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InSplineData, InputSelector);
			if (!Accessor)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(InputSelector, InContext);
				continue;
			}

			FPCGAttributeAccessorKeysEntries DefaultEntry(PCGInvalidEntryKey);
			FSoftObjectPath SplineComponentClassPath;
			if (!Accessor->Get<FSoftObjectPath>(SplineComponentClassPath, DefaultEntry, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
			{
				PCGLog::Metadata::LogFailToGetAttributeError<FSoftObjectPath>(InputSelector, Accessor.Get(), InContext);
				continue;
			}

			SplineComponentToSpawn = Cast<UClass>(SplineComponentClassPath.TryLoad());
		}

		if (!*SplineComponentToSpawn)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidSplineComponent", "Invalid spline component subclass."), InContext);
			continue;
		}

#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("AddSplineComponent", "Adding spline component to actor from PCG"), ExecutionSource->GetExecutionState().UseTransactions());
#endif

		check(IsInGameThread());
		USplineComponent* SplineComponent = NewObject<USplineComponent>(TargetActor, *SplineComponentToSpawn);

		SplineComponent->RegisterComponent();
		TargetActor->AddInstanceComponent(SplineComponent);
		// Spline Data apply to Spline Component is done in world coordinate, so keep world location.
		if (!SplineComponent->AttachToComponent(TargetActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, false)))
		{
			PCGLog::Component::LogComponentAttachmentFailedWarning(InContext);
		}

		InSplineData->ApplyTo(SplineComponent);

		UPCGManagedComponent* ManagedComponent = NewObject<UPCGManagedComponent>(ExecutionSourceObject);
		ManagedComponent->GeneratedComponent = SplineComponent;
		ManagedResourcesContainerHelper.AddManagedResource(ManagedComponent);

		if (!Settings->PropertyOverrideDescriptions.IsEmpty())
		{
			FPCGObjectOverrides DescriptorOverride(SplineComponent);
			DescriptorOverride.Initialize(Settings->PropertyOverrideDescriptions, SplineComponent, InSplineData, InContext);
			if (DescriptorOverride.IsValid() && !DescriptorOverride.Apply(0))
			{
				const int32 Index = static_cast<int32>(&InputData - Inputs.GetData());
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("FailOverride", "Failed to override descriptor for input {0}"), Index));
			}

			// Apply 0, will be the default values for the spline
			DescriptorOverride.Apply(0);
		}

		// Add tags, both the default ones (source name + PCG Generated) and the ones from the override pin.
		// We do this after the override in the case that the override was writing the tags.
		if (ExecutionSourceObject)
		{
			SplineComponent->ComponentTags.Add(ExecutionSourceObject->GetFName());
		}
		
		SplineComponent->ComponentTags.Add(PCGHelpers::DefaultPCGTag);
		for (const FName& Tag : Settings->TagsToAddOnComponents)
		{
			SplineComponent->ComponentTags.AddUnique(Tag);
		}

		// Also propagate the tags on the tagged data to the component.
		Algo::Transform(InputData.Tags, SplineComponent->ComponentTags, [](const FString& Tag) { return FName(Tag); });

		// Execute PostProcess Functions
		if (TargetActor)
		{
			for (UFunction* Function : PCGHelpers::FindUserFunctions(TargetActor->GetClass(), Settings->PostProcessFunctionNames, { UPCGFunctionPrototypes::GetPrototypeWithNoParams() }, InContext))
			{
				TargetActor->ProcessEvent(Function, nullptr);
			}
		}
		
		if (Settings->bOutputSplineComponentReference)
		{
			UPCGSplineData* OutputSplineData = CastChecked<UPCGSplineData>(InSplineData->DuplicateData(InContext));

			if (!PCGMetadataElementCommon::ClearOrCreateAttribute<FSoftObjectPath>(OutputSplineData->Metadata, Settings->ComponentReferenceAttributeName, FSoftObjectPath(SplineComponent)))
			{
				PCGLog::Metadata::LogFailToCreateAttributeError<FSoftObjectPath>(Settings->ComponentReferenceAttributeName, InContext);
			}
			
			InContext->OutputData.TaggedData.Emplace_GetRef(InputData).Data = OutputSplineData;
		}
		else
		{
			InContext->OutputData.TaggedData.Emplace(InputData);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
