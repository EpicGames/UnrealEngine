// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSpawnSplineMesh.h"

#include "PCGContext.h"
#include "PCGManagedResource.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "Data/PCGLandscapeSplineData.h"
#include "Data/PCGSplineData.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"

#include "DrawDebugHelpers.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "Materials/MaterialInterface.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSpawnSplineMesh)

#define LOCTEXT_NAMESPACE "PCGCreateSplineMeshElement"

void UPCGSpawnSplineMeshSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// TODO: Remove if/when FBodyInstance is updated or replaced
	// Necessary to update the collision Response Container from the Response Array
	SplineMeshDescriptor.PostLoadFixup(this);
#endif // WITH_EDITOR
}

#if WITH_EDITOR
FText UPCGSpawnSplineMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("CreateSplineMeshTooltip", "Create a USplineMeshComponent for each segment along a given spline.");
}
#endif

TArray<FPCGPinProperties> UPCGSpawnSplineMeshSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::PolyLine).SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSpawnSplineMeshSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::PolyLine);

	return PinProperties;
}

FPCGElementPtr UPCGSpawnSplineMeshSettings::CreateElement() const
{
	return MakeShared<FPCGSpawnSplineMeshElement>();
}

bool FPCGSpawnSplineMeshElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreateSplineMeshElement::PrepareDataInternal);

	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context);

	const UPCGSpawnSplineMeshSettings* Settings = Context->GetInputSettings<UPCGSpawnSplineMeshSettings>();
	check(Settings);
	
	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	const FSoftSplineMeshComponentDescriptor& Descriptor = Settings->SplineMeshDescriptor;

	if (!Context->WasLoadRequested())
	{
		TArray<FSoftObjectPath> ObjectsToLoad = { Descriptor.StaticMesh.ToSoftObjectPath() };

		if (!Descriptor.OverlayMaterial.IsNull())
		{
			ObjectsToLoad.Emplace(Descriptor.OverlayMaterial.ToSoftObjectPath());
		}

		for (int MaterialIndex = 0; MaterialIndex < Descriptor.OverrideMaterials.Num(); ++MaterialIndex)
		{
			if (!Descriptor.OverrideMaterials[MaterialIndex].IsNull())
			{
				ObjectsToLoad.Emplace(Descriptor.OverrideMaterials[MaterialIndex].ToSoftObjectPath());
			}
		}

		for (int RVTIndex = 0; RVTIndex < Descriptor.RuntimeVirtualTextures.Num(); ++RVTIndex)
		{
			if (!Descriptor.RuntimeVirtualTextures[RVTIndex].IsNull())
			{
				ObjectsToLoad.Emplace(Descriptor.RuntimeVirtualTextures[RVTIndex].ToSoftObjectPath());
			}
		}

		EPCGTimeSliceInitResult ExecResult = Context->InitializePerExecutionState([Settings](ContextType* Context, FPCGSpawnSplineMeshPerExecutionState& OutState)
		{
			OutState.TargetActor = Settings->TargetActor.Get() ? Settings->TargetActor.Get() : Context->GetTypedExecutionTarget<AActor>();

			if (!OutState.TargetActor)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("InvalidTargetActor", "Invalid target actor."), Context);
				return EPCGTimeSliceInitResult::AbortExecution;
			}
			else
			{
				return EPCGTimeSliceInitResult::Success;
			}
		});

		if (ExecResult == EPCGTimeSliceInitResult::AbortExecution)
		{
			return true;
		}
		
		Context->InitializePerIterationStates(Inputs.Num(), [&Inputs, &Context, Settings, &ObjectsToLoad](FPCGSpawnSplineMeshPerIterationState& OutState, const ExecStateType&, const uint32 IterationIndex) -> EPCGTimeSliceInitResult
		{
			const FPCGTaggedData& Input = Inputs[IterationIndex];
			OutState.SplineData = Cast<UPCGPolyLineData>(Input.Data);
			if (!OutState.SplineData)
			{
				return EPCGTimeSliceInitResult::NoOperation;
			}

			// Forward input
			Context->OutputData.TaggedData.Add(Input);

			// If the spline is empty we have nothing else to do
			if (OutState.SplineData->GetNumSegments() <= 0)
			{
				return EPCGTimeSliceInitResult::NoOperation;
			}
			
			OutState.LandscapeSplineData = Cast<UPCGLandscapeSplineData>(OutState.SplineData);
			OutState.SMCBuilderParams.SplineMeshParams = Settings->SplineMeshParams;

			// Initialize the override for the descriptor, and extract all the objects to load
			OutState.DescriptionOverrides.Initialize(Settings->SplineMeshOverrideDescriptions, &OutState.SMCBuilderParams.Descriptor, OutState.SplineData, Context);
			if (OutState.DescriptionOverrides.IsValid())
			{
				OutState.DescriptionOverrides.GatherAllOverridesToLoad(ObjectsToLoad);
			}

			// Also initialize the override for the params
			OutState.ParamsOverrides.Initialize(Settings->SplineMeshParamsOverride, &OutState.SMCBuilderParams.SplineMeshParams, OutState.SplineData, Context);

			// And initialize the override with the CDO component for now, it will be updated with the right component when it'll be spawned.
			if (const TSubclassOf<USplineMeshComponent> SplineMeshComponentClass = Settings->SplineMeshDescriptor.ComponentClass; SplineMeshComponentClass)
			{
				OutState.ComponentOverrides.Initialize(Settings->SplineMeshComponentOverride, SplineMeshComponentClass->GetDefaultObject<USplineMeshComponent>(), OutState.SplineData, Context);
			}

			return EPCGTimeSliceInitResult::Success;
		});

		return Context->RequestResourceLoad(Context, std::move(ObjectsToLoad), !Settings->bSynchronousLoad);
	}

	return true;
}

bool FPCGSpawnSplineMeshElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreateSplineMeshElement::Execute);

	ContextType* Context = static_cast<ContextType*>(InContext);
	const UPCGSpawnSplineMeshSettings* Settings = Context->GetInputSettings<UPCGSpawnSplineMeshSettings>();
	check(Settings);

	if (!Context->DataIsPreparedForExecution())
	{
		return true;
	}

	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	if (!ExecutionSource)
	{
		return true;
	}

	return ExecuteSlice(Context, [this, Settings](ContextType* Context, const ExecStateType& ExecState, IterStateType& IterState, const uint32 IterIndex)
	{
		if (Context->GetIterationStateResult(IterIndex) != EPCGTimeSliceInitResult::Success)
		{
			return true;
		}

		const UPCGPolyLineData* SplineData = IterState.SplineData;
		
		const int NumSegments = SplineData->GetNumSegments();
		const bool bIsClosed = SplineData->IsClosed();

		const bool bStartRollOverridden = Settings->SplineMeshParamsOverride.ContainsByPredicate([](const FPCGObjectPropertyOverrideDescription& It) { return It.PropertyTarget == GET_MEMBER_NAME_CHECKED(FPCGSplineMeshParams, StartRollDegrees); });
		const bool bEndRollOverridden = Settings->SplineMeshParamsOverride.ContainsByPredicate([](const FPCGObjectPropertyOverrideDescription& It) { return It.PropertyTarget == GET_MEMBER_NAME_CHECKED(FPCGSplineMeshParams, EndRollDegrees); });

		if (IterState.ElementIndex == 0)
		{
			// Copy the descriptor, for the overrides
			IterState.SMCBuilderParams.Descriptor = FSplineMeshComponentDescriptor(Settings->SplineMeshDescriptor);
			IterState.SMCBuilderParams.SplineMeshParams = Settings->SplineMeshParams;
			IterState.DescriptionOverrides.UpdateTemplateObject(&IterState.SMCBuilderParams.Descriptor);
			IterState.ParamsOverrides.UpdateTemplateObject(&IterState.SMCBuilderParams.SplineMeshParams);
		}

		while (IterState.ElementIndex < NumSegments)
		{
			if (IterState.DescriptionOverrides.IsValid() && !IterState.DescriptionOverrides.Apply(IterState.ElementIndex))
			{
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("FailOverrideDescriptor", "Failed to override descriptor for input {0} element {1}"), IterIndex, IterState.ElementIndex));
			}
			
			UStaticMesh* StaticMesh = IterState.SMCBuilderParams.Descriptor.StaticMesh.Get();

			if (!StaticMesh || !IterState.SMCBuilderParams.Descriptor.ComponentClass)
			{
				IterState.ElementIndex++;
				continue;
			}

			if (IterState.ParamsOverrides.IsValid() && !IterState.ParamsOverrides.Apply(IterState.ElementIndex))
			{
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("FailOverrideParams", "Failed to override params for input {0} element {1}"), IterIndex, IterState.ElementIndex));
			}
			
			const FVector MeshExtents = StaticMesh->GetBounds().BoxExtent;
			
			const int NextIndex = (bIsClosed && IterState.ElementIndex == NumSegments - 1) ? 0 : (IterState.ElementIndex + 1);

			FBox Bounds, NextBounds;
			FTransform Transform = SplineData->GetTransformAtDistance(IterState.ElementIndex, /*Distance=*/0.0, /*bWorldSpace=*/true, &Bounds);
			FTransform NextTransform = SplineData->GetTransformAtDistance(IterState.ElementIndex, SplineData->GetSegmentLength(IterState.ElementIndex), /*bWorldSpace=*/true, &NextBounds);

			if (IterState.LandscapeSplineData && !IterState.SMCBuilderParams.SplineMeshParams.bScaleMeshToLandscapeSplineFullWidth)
			{
				Bounds = FBox::BuildAABB(FVector::ZeroVector, FVector::OneVector);
				NextBounds = FBox::BuildAABB(FVector::ZeroVector, FVector::OneVector);
			}

			FVector LeaveTangent, ArriveTangent, Dummy;
			SplineData->GetTangentsAtSegmentStart(IterState.ElementIndex, Dummy, LeaveTangent);
			SplineData->GetTangentsAtSegmentStart(NextIndex, ArriveTangent, Dummy);

			// Need to apply the transform to the tangents too, as we are working in world space (and tangents are in local space)
			const FTransform SplineTransform = SplineData->GetTransform();
			LeaveTangent = SplineTransform.TransformVector(LeaveTangent);
			ArriveTangent = SplineTransform.TransformVector(ArriveTangent);

			// Set Position
			IterState.SMCBuilderParams.SplineMeshParams.StartPosition = Transform.GetLocation();
			IterState.SMCBuilderParams.SplineMeshParams.StartTangent = LeaveTangent;
			IterState.SMCBuilderParams.SplineMeshParams.EndPosition = NextTransform.GetLocation();
			IterState.SMCBuilderParams.SplineMeshParams.EndTangent = ArriveTangent;

			// Set Roll (Rotation)
			if (!bStartRollOverridden || !bEndRollOverridden)
			{
				const FQuat StartRotation = Transform.GetRotation();
				const FQuat EndRotation = NextTransform.GetRotation();

				if (!bStartRollOverridden)
				{
					IterState.SMCBuilderParams.SplineMeshParams.StartRollDegrees = StartRotation.Rotator().Roll;
				}
				
				if (!bEndRollOverridden)
				{
					IterState.SMCBuilderParams.SplineMeshParams.EndRollDegrees = EndRotation.Rotator().Roll;
				}
			}

			// Set Scale
			FVector StartScale = Transform.GetScale3D() * Bounds.GetExtent();
			FVector EndScale = NextTransform.GetScale3D() * NextBounds.GetExtent();

			if (IterState.SMCBuilderParams.SplineMeshParams.bScaleMeshToBounds)
			{
				FVector::FReal ScaleFactorY = 1.0f;
				FVector::FReal ScaleFactorZ = 1.0f;

				// We only scale in two dimensions since we are extruding along one of the axes. Scale on the two axes we are not extruding along.
				if (IterState.SMCBuilderParams.SplineMeshParams.ForwardAxis == EPCGSplineMeshForwardAxis::X)
				{
					ScaleFactorY = MeshExtents.Y;
					ScaleFactorZ = MeshExtents.Z;
				}
				else if (IterState.SMCBuilderParams.SplineMeshParams.ForwardAxis == EPCGSplineMeshForwardAxis::Y)
				{
					ScaleFactorY = MeshExtents.X;
					ScaleFactorZ = MeshExtents.Z;
				}
				else if (IterState.SMCBuilderParams.SplineMeshParams.ForwardAxis == EPCGSplineMeshForwardAxis::Z)
				{
					ScaleFactorY = MeshExtents.X;
					ScaleFactorZ = MeshExtents.Y;
				}

				StartScale.Y /= ScaleFactorY;
				StartScale.Z /= ScaleFactorZ;
				EndScale.Y /= ScaleFactorY;
				EndScale.Z /= ScaleFactorZ;
			}

			IterState.SMCBuilderParams.SplineMeshParams.StartScale = FVector2D(StartScale.Y, StartScale.Z);
			IterState.SMCBuilderParams.SplineMeshParams.EndScale = FVector2D(EndScale.Y, EndScale.Z);
			IterState.SMCBuilderParams.SettingsCrc = Settings->GetSettingsCrc();
			ensure(IterState.SMCBuilderParams.SettingsCrc.IsValid());

			IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
			check(ExecutionSource); // not likely to happen since it was tested at the beginning of the execute.

#if WITH_EDITOR
			FScopedTransaction Transaction(LOCTEXT("AddSplineMeshComponent", "Adding spline mesh component to actor from PCG"), ExecutionSource->GetExecutionState().UseTransactions());
#endif

			if (USplineMeshComponent* SplineMeshComponent = UPCGActorHelpers::GetOrCreateSplineMeshComponent(ExecState.TargetActor, ExecutionSource, IterState.SMCBuilderParams, Context))
			{
				// Apply the overrides on the spawned component
				if (IterState.ComponentOverrides.IsValid())
				{
					IterState.ComponentOverrides.UpdateTemplateObject(SplineMeshComponent);
					IterState.ComponentOverrides.Apply(IterState.ElementIndex);

					// Since the component overrides can override the component tags, we need to ensure that the default tags are still present
					if (UObject* ExecutionSourceObject = Cast<UObject>(ExecutionSource))
					{
						SplineMeshComponent->ComponentTags.AddUnique(ExecutionSourceObject->GetFName());
					}
					
					SplineMeshComponent->ComponentTags.AddUnique(PCGHelpers::DefaultPCGTag);
				}

				// Apply tags on the Spline Mesh Component
				for (const FName& Tag : Settings->TagsToAddOnComponents)
				{
					SplineMeshComponent->ComponentTags.AddUnique(Tag);
				}

				// TODO: Write out the geometry to a dynamic mesh type.
				//SplineMeshComponent->BodySetup->TriMeshGeometries
			}

			IterState.ElementIndex++;

			if (Context->ShouldStop())
			{
				break;
			}
		}

		const bool bDone = IterState.ElementIndex == NumSegments;
		
		// Execute PostProcess Functions
		if (bDone && ExecState.TargetActor)
		{
			for (UFunction* Function : PCGHelpers::FindUserFunctions(ExecState.TargetActor->GetClass(), Settings->PostProcessFunctionNames, { UPCGFunctionPrototypes::GetPrototypeWithNoParams() }, Context))
			{
				ExecState.TargetActor->ProcessEvent(Function, nullptr);
			}
		}

		return bDone;
	});
}

#undef LOCTEXT_NAMESPACE
