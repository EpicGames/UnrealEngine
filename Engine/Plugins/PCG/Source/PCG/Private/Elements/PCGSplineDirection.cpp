// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSplineDirection.h"

#include "PCGContext.h"
#include "Data/PCGSplineData.h"

#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplineDirection)

#define LOCTEXT_NAMESPACE "PCGSplineDirectionElement"

namespace PCGSplineDirection
{
	bool IsClockwiseXY(const UPCGSplineData* InputSplineData)
	{
		check(InputSplineData);

		double CumulativeArea = 0.0;
		const int NumPoints = InputSplineData->SplineStruct.GetSplinePointsPosition().Points.Num();

		if (NumPoints <= 1)
		{
			return true;
		}

		if (NumPoints == 2)
		{
			FVector ArriveTangent{}, LeaveTangent{}, DummyTangent;
			InputSplineData->GetTangentsAtSegmentStart(0, DummyTangent, LeaveTangent);
			InputSplineData->GetTangentsAtSegmentStart(1, ArriveTangent, DummyTangent);
			const double CrossProduct = (ArriveTangent.X * LeaveTangent.Y - ArriveTangent.Y * LeaveTangent.X);
			return CrossProduct <= 0;
		}

		// Shoelace formula - https://en.wikipedia.org/wiki/Shoelace_formula.
		// It's not clear the algorithm gives the right answer if we consider the spline not closed, so always consider the spline is
		// closed for this.
		int NumIter = NumPoints;

		for (int32 Index = 0; Index < NumIter; ++Index)
		{
			const FVector& ThisPoint = InputSplineData->SplineStruct.GetSplinePointsPosition().Points[Index % NumPoints].OutVal;
			const FVector& NextPoint = InputSplineData->SplineStruct.GetSplinePointsPosition().Points[(Index + 1) % NumPoints].OutVal;

			const double CrossProduct = (NextPoint.X * ThisPoint.Y - ThisPoint.X * NextPoint.Y);

			CumulativeArea += CrossProduct;
		}

		return CumulativeArea <= 0;
	}

	UPCGSplineData* Reverse(const UPCGSplineData* InputSplineData, FPCGContext* Context)
	{
		check(InputSplineData);

		const FInterpCurveVector& ControlPointsPosition = InputSplineData->SplineStruct.GetSplinePointsPosition();
		const FInterpCurveQuat& ControlPointsRotation = InputSplineData->SplineStruct.GetSplinePointsRotation();
		const FInterpCurveVector& ControlPointsScale = InputSplineData->SplineStruct.GetSplinePointsScale();
		TArray<PCGMetadataEntryKey> ControlPointKeys(InputSplineData->SplineStruct.GetConstControlPointsEntryKeys());

		TArray<FSplinePoint> NewControlPoints;
		NewControlPoints.Reserve(ControlPointsPosition.Points.Num());

		const float FirstKeyValue = ControlPointsPosition.Points.IsEmpty() ? 0.0f : ControlPointsPosition.Points[0].InVal;
		const float LastKeyValue = ControlPointsPosition.Points.IsEmpty() ? 0.0f : ControlPointsPosition.Points.Last().InVal;

		for (int i = ControlPointsPosition.Points.Num() - 1; i >= 0; --i)
		{
			/* Implementation Note: Segment interpolation is determined by the interpolation mode of the preceding
			 * control point. When inverting order of control points, we can decay the interpolation mode by setting all
			 * modes to Custom Tangent to use the pre-calculated tangents as-is, so long as they were actually
			 * calculated. This is a slightly destructive process and some information will be lost. Also, its worth
			 * noting that since each spline segment is calculated from [0..1) there is a slight inconsistency when
			 * evaluated in reverse order, as effectively the reverse is [1..0) per segment.
			 */
			const int PreviousPointIndex = (i > 0 ? i - 1 : (InputSplineData->IsClosed() ? ControlPointsPosition.Points.Num() - 1 : 0));
			ESplinePointType::Type NewControlPointType = ConvertInterpCurveModeToSplinePointType(ControlPointsPosition.Points[PreviousPointIndex].InterpMode);

			if ((NewControlPointType == ESplinePointType::Curve || NewControlPointType == ESplinePointType::CurveClamped || NewControlPointType == ESplinePointType::CurveCustomTangent))
			{
				NewControlPointType = ESplinePointType::CurveCustomTangent;
			}

			NewControlPoints.Emplace(FirstKeyValue + LastKeyValue - ControlPointsPosition.Points[i].InVal,
				ControlPointsPosition.Points[i].OutVal,
				-ControlPointsPosition.Points[i].LeaveTangent, // Tangents are inverted and swapped
				-ControlPointsPosition.Points[i].ArriveTangent,
				ControlPointsRotation.Points[i].OutVal.Rotator(),
				ControlPointsScale.Points[i].OutVal,
				NewControlPointType);
		}

		UPCGSplineData* NewSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
		NewSplineData->InitializeFromData(InputSplineData);
		NewSplineData->Initialize(NewControlPoints, InputSplineData->IsClosed(), InputSplineData->GetTransform(), std::move(ControlPointKeys));

		// Set the key offset too based on the original spline.
		if (InputSplineData->IsClosed())
		{
			NewSplineData->SplineStruct.SetLoopKeyOffset(InputSplineData->GetLoopKeyOffset());
		}
		
		return NewSplineData;
	}
}

void UPCGReverseSplineSettings::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITOR
	// UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID) is located in parent class UPCGSettings::Serialize
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGSplineDirectionClockwiseFix)
	{
		bFlipClockwiseComputationResult = true;
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
FName UPCGReverseSplineSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SplineDirection"));
}

FText UPCGReverseSplineSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Spline Direction");
}

EPCGChangeType UPCGReverseSplineSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGReverseSplineSettings, Operation))
	{
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGReverseSplineSettings::CreateElement() const
{
	return MakeShared<FPCGSplineDirectionElement>();
}

FString UPCGReverseSplineSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGReverseSplineOperation>())
	{
		return EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation)).ToString();
	}

	return {};
}

TArray<FPCGPinProperties> UPCGReverseSplineSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spline).SetRequiredPin();
	return Properties;
}

TArray<FPCGPinProperties> UPCGReverseSplineSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spline);
	return Properties;
}

bool FPCGSplineDirectionElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSplineDirectionElement::Execute);

	check(InContext);

	const UPCGReverseSplineSettings* Settings = InContext->GetInputSettings<UPCGReverseSplineSettings>();
	check(Settings);

	// Only warn for the deprecated algorithm once.
	bool bHasWarned = false;

	for (const FPCGTaggedData& InputData : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		FPCGTaggedData& Output = InContext->OutputData.TaggedData.Emplace_GetRef(InputData);

		const UPCGSplineData* InputSplineData = Cast<const UPCGSplineData>(InputData.Data);
		if (!InputSplineData || InputSplineData->SplineStruct.GetNumberOfSplineSegments() < 1)
		{
			continue;
		}

		bool bShouldReverse = true;

		if (Settings->Operation == EPCGReverseSplineOperation::ForceClockwise || Settings->Operation == EPCGReverseSplineOperation::ForceCounterClockwise)
		{
			bShouldReverse = PCGSplineDirection::IsClockwiseXY(InputSplineData) != (Settings->Operation == EPCGReverseSplineOperation::ForceClockwise);
			if (Settings->bFlipClockwiseComputationResult)
			{
				bShouldReverse = !bShouldReverse;
				if (!bHasWarned)
				{
					PCGLog::LogWarningOnGraph(LOCTEXT("WarningUpdateClockwiseAlgorithm", "The clockwise detecting algorithm has been updated. Replace with a new copy of the node to remove this warning."), InContext);
					bHasWarned = true;
				}
			}
		}
		else if (Settings->Operation != EPCGReverseSplineOperation::Reverse)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidOperation", "Invalid operation enum value"), InContext);
			return true;
		}

		if (bShouldReverse)
		{
			Output.Data = PCGSplineDirection::Reverse(InputSplineData, InContext);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
