// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSplineIntersection.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "PCA3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplineIntersection)

#define LOCTEXT_NAMESPACE "PCGSplineIntersectionElement"

namespace PCGSplineIntersection
{
	namespace Constants
	{
		const FName OriginatingSplineIndexAttributeName = TEXT("OriginatingSplineIndex");
		const FName IntersectingSplineIndexAttributeName = TEXT("IntersectingSplineIndex");
		const FName IntersectionTypeAttributeName = TEXT("IntersectionType");
		constexpr double KeyDifferenceThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER;
	}

	namespace Helpers
	{
		double GetSplineKeyLength(const UPCGSplineData* InSpline)
		{
			check(InSpline);
			if (InSpline->SplineStruct.GetSplinePointsPosition().Points.IsEmpty())
			{
				return 0;
			}
			else 
			{
				const double LastKey = InSpline->SplineStruct.GetSplinePointsPosition().Points.Last().InVal;
				return LastKey + (InSpline->IsClosed() ? InSpline->GetLoopKeyOffset() : 0.0);
			}
		}
	}

	// Data tracking structure to encapsulate discretization of the spline
	struct FInputData
	{
		void InitializeBounds(const FVector& HalfRadiusTolerance)
		{
			// Implementation note: we add half the distance tolerance to the bounds so we never need to update them after, 
			// as we'll use them for bounds-bounds tests only.
			check(Spline);
			Bounds = Spline->GetBounds().ExpandBy(HalfRadiusTolerance, HalfRadiusTolerance);

			SegmentsNum = Spline->GetNumSegments();
			SegmentBounds.SetNumUninitialized(SegmentsNum);
			for (int32 SegmentIndex = 0; SegmentIndex < SegmentBounds.Num(); ++SegmentIndex)
			{
				SegmentBounds[SegmentIndex] = Spline->SplineStruct.GetSegmentBounds(SegmentIndex).TransformBy(Spline->SplineStruct.Transform).ExpandBy(HalfRadiusTolerance, HalfRadiusTolerance);
			}
		}

		const TArray<FVector>& GetSamples(int SegmentIndex, double DiscretizationError) const
		{
			check(Spline && SegmentIndex >= 0 && SegmentIndex < SegmentsNum);
			TArray<FVector>& Samples = Sampling.FindOrAdd(SegmentIndex);

			if (Samples.IsEmpty())
			{
				Spline->SplineStruct.ConvertSplineSegmentToPolyLine(SegmentIndex, ESplineCoordinateSpace::World, DiscretizationError, Samples);
			}

			return Samples;
		}

		const UPCGSplineData* Spline = nullptr;
		int InputIndex = INDEX_NONE;
		FBox Bounds = FBox(EForceInit::ForceInit);
		TArray<FBox> SegmentBounds;
		mutable TMap<int, TArray<FVector>> Sampling;
		int SegmentsNum;
	};

	// Represents an intersection on a spline, with a start & stop.
	struct FIntersectionRange
	{
		const UPCGSplineData* Spline = nullptr;
		const FInputData* Input = nullptr;
		double InputKeyStart = 0.0;
		double InputKeyEnd = 0.0;

		TArray<const UPCGSplineData*, TInlineAllocator<1>> OtherSplines;
		TArray<FVector, TInlineAllocator<2>> Points;

		bool Overlaps(const FIntersectionRange& Other) const
		{
			if (Spline != Other.Spline)
			{
				return false;
			}

			// Special treatment for looping ranges
			if (Spline->IsClosed() && !Spline->SplineStruct.GetSplinePointsPosition().Points.IsEmpty())
			{
				const float SplineKeyLength = Helpers::GetSplineKeyLength(Spline);

				// Assumption here that the key end will be at most the spline key length, otherwise we've made an error
				ensure(InputKeyEnd <= SplineKeyLength && Other.InputKeyStart <= SplineKeyLength);

				if (InputKeyStart + SplineKeyLength <= Other.InputKeyEnd + Constants::KeyDifferenceThreshold ||
					Other.InputKeyStart + SplineKeyLength <= InputKeyEnd + Constants::KeyDifferenceThreshold)
				{
					return true;
				}
			}

			// Normal overlapping case
			return Spline == Other.Spline && !(InputKeyStart > Other.InputKeyEnd + Constants::KeyDifferenceThreshold || Other.InputKeyStart > InputKeyEnd + Constants::KeyDifferenceThreshold);
		}

		void Merge(const FIntersectionRange& Other)
		{
			check(Spline == Other.Spline);

			// Special loop management:
			// In this case, we'll support cases where key start > key end to represent the loop part.
			if (Spline->IsClosed())
			{
				const float SplineKeyLength = Helpers::GetSplineKeyLength(Spline);
				const bool bInvertedA = (InputKeyEnd < InputKeyStart);
				const bool bInvertedB = (Other.InputKeyEnd < Other.InputKeyStart);
				const bool bShiftA = (InputKeyStart + SplineKeyLength <= Other.InputKeyEnd + Constants::KeyDifferenceThreshold);
				const bool bShiftB = !bShiftA && Other.InputKeyStart + SplineKeyLength <= InputKeyEnd + Constants::KeyDifferenceThreshold;

				double StartA = InputKeyStart + (bShiftA ? SplineKeyLength : 0);
				double EndA = InputKeyEnd + (bShiftA ? SplineKeyLength : 0) + (bInvertedA ? SplineKeyLength : 0);
				double StartB = Other.InputKeyStart + (bShiftB ? SplineKeyLength : 0);
				double EndB = Other.InputKeyEnd + (bShiftB ? SplineKeyLength : 0) + (bInvertedB ? SplineKeyLength : 0);

				InputKeyStart = FMath::Min(StartA, StartB);
				InputKeyEnd = FMath::Max(EndA, EndB);

				// Finally, special case for a complete loop
				if (InputKeyEnd - InputKeyStart + Constants::KeyDifferenceThreshold >= SplineKeyLength)
				{
					InputKeyStart = 0;
					InputKeyEnd = SplineKeyLength;
				}
				// Finally, correct end to keep in the real range of keys
				else if (InputKeyEnd >= SplineKeyLength)
				{
					InputKeyEnd -= SplineKeyLength;
				}
			}
			else
			{
				InputKeyStart = FMath::Min(InputKeyStart, Other.InputKeyStart);
				InputKeyEnd = FMath::Max(InputKeyEnd, Other.InputKeyEnd);
			}

			for (const UPCGSplineData* OtherSpline : Other.OtherSplines)
			{
				OtherSplines.AddUnique(OtherSpline);
			}

			Points.Append(Other.Points);
		}

		FVector GetVector() const
		{
			check(Points.Num() >= 2);
			FVector A = Points[0];
			FVector B = Points[1];

			// Find limits on a kind of triangle inequality.
			for (int PointIndex = 2; PointIndex < Points.Num(); ++PointIndex)
			{
				// Basically, if the 3rd point is on the "same" side of both A & B, then it is a more extreme point.
				// Note: this does somewhat assume linearity.
				const FVector& C = Points[PointIndex];

				const FVector CA = C - A;
				const FVector CB = C - B;
				if ((CA | CB) > 0)
				{
					if (CA.SquaredLength() < CB.SquaredLength())
					{
						A = C;
					}
					else
					{
						B = C;
					}
				}
			}

			return B - A;
		}
	};

	// Intersection observed between two splines
	struct FIntersectionPoint
	{
		FIntersectionRange A;
		FIntersectionRange B;
	};

	// Retrieves & merges the intersection ranges for a specific spline.
	TArray<FIntersectionRange> FilterAndMergeIntersections(const TArray<FIntersectionPoint>& Intersections, const UPCGSplineData* Spline)
	{
		TArray<FIntersectionRange> Output;

		// Select intersection ranges only for the spline we're interested in
		for (const FIntersectionPoint& Intersection : Intersections)
		{
			if (Intersection.A.Spline == Spline)
			{
				Output.Add(Intersection.A);
			}

			if (Intersection.B.Spline == Spline)
			{
				Output.Add(Intersection.B);
			}
		}

		// Merge ranges based on their key values
		for (int I = 0; I < Output.Num(); ++I)
		{
			FIntersectionRange& Current = Output[I];
			for (int J = I + 1; J < Output.Num(); ++J)
			{
				FIntersectionRange& Other = Output[J];

				if (Current.Overlaps(Other))
				{
					Current.Merge(Other);
					Output.RemoveAt(J);
					I--; // restart the loop
					break;
				}
			}
		}

		return Output;
	}

	struct FCollapsedIntersectionPoint
	{
		TArray<FIntersectionRange> Ranges;
		FTransform Transform = FTransform::Identity;
		FBox Bounds = FBox(ForceInit);

		void ComputeSpatialData(bool bUsePCA)
		{
			// Perform PCA on all points to get bounds for the intersection to add.
			TArray<FVector> Points;
			for (const FIntersectionRange& Range : Ranges)
			{
				Points.Append(Range.Points);
			}

			// note: might need to perturb points a bit to account for things happening on a plane
			UE::Geometry::FPCA3d PCA;
			if (bUsePCA && PCA.Compute(Points))
			{
				Transform = FTransform(PCA.Eigenvectors[0], PCA.Eigenvectors[1], PCA.Eigenvectors[2], PCA.Mean);
			}
			else
			{
				// Default & Degenerate case (collocated or collinear).
				// Find out per-range the extreme points - this will be the primary axis (X).
				// After that, use Z-up, and compute resulting Y axis.
				FVector PrimaryAxis = FVector::ZeroVector;
				for (const FIntersectionRange& Range : Ranges)
				{
					const FVector RangeVector = Range.GetVector();
					if (RangeVector.SquaredLength() > PrimaryAxis.SquaredLength())
					{
						PrimaryAxis = RangeVector;
					}
				}

				if (PrimaryAxis.SquaredLength() > UE_DOUBLE_SMALL_NUMBER)
				{
					FVector MeanPosition = FVector::ZeroVector;

					if (Points.Num() > 0)
					{
						for (const FVector& Point : Points)
						{
							MeanPosition += Point;
						}

						MeanPosition /= static_cast<double>(Points.Num());
					}

					// Normalize primary axis and compute a proper 'z' vector.
					PrimaryAxis = PrimaryAxis.GetSafeNormal(UE_DOUBLE_SMALL_NUMBER, FVector::XAxisVector);

					FVector TentativeZAxis = FVector::ZAxisVector;
					if (FMath::IsNearlyEqual(FMath::Abs(PrimaryAxis | TentativeZAxis), 1.0, UE_KINDA_SMALL_NUMBER))
					{
						TentativeZAxis = FVector::YAxisVector;
					}

					// todo_pcg: z axis here is wrong. we need another way to get the axis
					Transform = FTransform(FRotationMatrix::MakeFromXZ(PrimaryAxis, TentativeZAxis).ToQuat(), MeanPosition, FVector::OneVector);
				}
			}

			// compute local box from points.
			for (const FVector& Point : Points)
			{
				Bounds += Transform.InverseTransformPosition(Point);
			}

			// Finally, make sure that the bounds extents are non-zero.
			constexpr double MinBoundsSize = 1.0;
			const FVector OriginalExtent = Bounds.GetExtent();
			FVector Expansion(FMath::Max(0.0, MinBoundsSize - OriginalExtent[0]),
				FMath::Max(0.0, MinBoundsSize - OriginalExtent[1]),
				FMath::Max(0.0, MinBoundsSize - OriginalExtent[2]));

			Bounds = Bounds.ExpandBy(Expansion);
		}
	};

	// Collapsed the intersections until all intersections are not mergeable (based on the standard intersection merge ratios).
	TArray<FCollapsedIntersectionPoint> CollapseIntersections(const TArray<FIntersectionPoint>& Intersections, bool bUsePCA)
	{
		TArray<FCollapsedIntersectionPoint> Output;
		TArray<bool> ConsumedIntersections;
		ConsumedIntersections.SetNumZeroed(Intersections.Num());

		for (int IntersectionIndex = 0; IntersectionIndex < Intersections.Num(); ++IntersectionIndex)
		{
			if (ConsumedIntersections[IntersectionIndex])
			{
				continue;
			}

			ConsumedIntersections[IntersectionIndex] = true;

			const FIntersectionPoint& CurrentIntersection = Intersections[IntersectionIndex];

			// @todo_pcg: this will not work for self-intersections so we might need to have a slightly different approach here.
			// Break down intersection in its ranges
			TMap<const UPCGSplineData*, FIntersectionRange> RangesPerSpline;
			RangesPerSpline.Add(CurrentIntersection.A.Spline, CurrentIntersection.A);
			RangesPerSpline.Add(CurrentIntersection.B.Spline, CurrentIntersection.B);

			bool bDone = false;
			while (!bDone)
			{
				bDone = true;

				for (int OtherIntersectionIndex = IntersectionIndex + 1; OtherIntersectionIndex < Intersections.Num(); ++OtherIntersectionIndex)
				{
					if (ConsumedIntersections[OtherIntersectionIndex])
					{
						continue;
					}

					const FIntersectionPoint& OtherIntersection = Intersections[OtherIntersectionIndex];

					FIntersectionRange* MatchingRangeA = RangesPerSpline.Find(OtherIntersection.A.Spline);
					FIntersectionRange* MatchingRangeB = RangesPerSpline.Find(OtherIntersection.B.Spline);

					// The condition here is a bit convoluted:
					// - All ranges we've taken so far (for specific splines) must overlap
					// *but* we're fine if we hadn't seen that spline before.
					if ((!MatchingRangeA && !MatchingRangeB) ||
						(MatchingRangeA && !MatchingRangeA->Overlaps(OtherIntersection.A)) ||
						(MatchingRangeB && !MatchingRangeB->Overlaps(OtherIntersection.B)))
					{
						continue;
					}

					ConsumedIntersections[OtherIntersectionIndex] = true;

					if (MatchingRangeA)
					{
						MatchingRangeA->Merge(OtherIntersection.A);
					}
					else
					{
						RangesPerSpline.Add(OtherIntersection.A.Spline, OtherIntersection.A);
					}

					if (MatchingRangeB)
					{
						MatchingRangeB->Merge(OtherIntersection.B);
					}
					else
					{
						RangesPerSpline.Add(OtherIntersection.B.Spline, OtherIntersection.B);
					}

					bDone = false;
					break;
				}
			}

			// Finally push to output
			FCollapsedIntersectionPoint& Intersection = Output.Emplace_GetRef();
			for (auto& Range : RangesPerSpline)
			{
				Intersection.Ranges.Add(MoveTemp(Range.Value));
			}

			Intersection.ComputeSpatialData(bUsePCA);
		}

		return Output;
	}

	// Considering two segments (AB and CD), a tolerance (SqrDistanceThreshold), compute:
	// - if the segments overlap in any way (return value - true if overlaps)
	// - at what ratio the overlaps start/end (ABStart - ABEnd; CDStart - CDEnd).
	bool FindOverlapRange(const FVector& A, const FVector& B, const FVector& C, const FVector& D, const double SqrDistanceThreshold, double& ABStart, double& ABEnd, double& CDStart, double& CDEnd)
	{
		ABStart = 0.0;
		ABEnd = 1.0;
		CDStart = 0.0;
		CDEnd = 1.0;

		// Test segment to segment - if that isn't successful, there's nothing to do.
		FVector X, Y;
		FMath::SegmentDistToSegment(A, B, C, D, X, Y);

		if ((X - Y).SquaredLength() > SqrDistanceThreshold)
		{
			return false;
		}

		// TODO : we should early test out trivial cases
		auto FindRange = [&SqrDistanceThreshold](const FVector& A, const FVector& B, const FVector& C, const FVector& D, const double& Hint, double& MinRange, double& MaxRange)
		{
			const FVector& Start = A;
			const double Len = (B - A).Length();
			check(Len >= UE_DOUBLE_SMALL_NUMBER);

			const FVector Dir = (B - A) / Len;

			auto FindEnd = [&SqrDistanceThreshold, &Start, &Len, &Dir, &C, &D, &Hint](bool bFindMin)
			{
				constexpr double RangeThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER;
				double Low = bFindMin ? 0.0 : Hint;
				double High = bFindMin ? Hint : 1.0;

				while (High - Low > RangeThreshold)
				{
					const double Mid = 0.5 * (Low + High);
					const FVector Pt = Start + Dir * Mid * Len;

					if ((FMath::PointDistToSegmentSquared(Pt, C, D) < SqrDistanceThreshold) == bFindMin)
					{
						High = Mid;
					}
					else
					{
						Low = Mid;
					}
				}

				return 0.5 * (Low + High);
			};

			MinRange = FindEnd(/*bFindMin=*/true);
			MaxRange = FindEnd(/*bFindMin=*/false);
		};

		const double ABLen = (B - A).Length();
		if (ABLen >= UE_DOUBLE_SMALL_NUMBER)
		{
			const double ABHint = (X - A).Length() / ABLen;
			FindRange(A, B, C, D, ABHint, ABStart, ABEnd);
		}
		else
		{
			ABStart = 0.0;
			ABEnd = 1.0;
		}

		const double CDLen = (D - C).Length();
		if (CDLen >= UE_DOUBLE_SMALL_NUMBER)
		{
			const double CDHint = (Y - C).Length() / CDLen;
			FindRange(C, D, A, B, CDHint, CDStart, CDEnd);
		}
		else
		{
			CDStart = 0.0;
			CDEnd = 1.0;
		}

		check(ABStart <= ABEnd);
		check(CDStart <= CDEnd);
		return true;
	}
}

UPCGSplineIntersectionSettings::UPCGSplineIntersectionSettings()
{
	OriginatingSplineIndexAttribute.SetAttributeName(PCGSplineIntersection::Constants::OriginatingSplineIndexAttributeName);
	IntersectingSplineIndexAttribute.SetAttributeName(PCGSplineIntersection::Constants::IntersectingSplineIndexAttributeName);
	IntersectionTypeAttribute.SetAttributeName(PCGSplineIntersection::Constants::IntersectionTypeAttributeName);
}

#if WITH_EDITOR
FName UPCGSplineIntersectionSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SplineIntersection"));
}

FText UPCGSplineIntersectionSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Spline Intersection");
}

EPCGChangeType UPCGSplineIntersectionSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSplineIntersectionSettings, Output))
	{
		// This can change the output pin types, so we need a structural change
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGSplineIntersectionSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spline);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSplineIntersectionSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	if (Output == EPCGSplineIntersectionOutput::IntersectionPointsOnly)
	{
		PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
	}
	else
	{
		PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spline);
	}

	return PinProperties;
}

FPCGElementPtr UPCGSplineIntersectionSettings::CreateElement() const
{
	return MakeShared<FPCGSplineIntersectionElement>();
}

bool FPCGSplineIntersectionElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSplineIntersectionElement::Execute);

	const UPCGSplineIntersectionSettings* Settings = Context->GetInputSettings<UPCGSplineIntersectionSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	TArray<PCGSplineIntersection::FInputData> InputSplines;
	const FVector HalfDistanceRadius(0.5 * Settings->DistanceThreshold);
	const double SqrDistanceThreshold = FMath::Square(Settings->DistanceThreshold);

	for (int InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		const FPCGTaggedData& Input = Inputs[InputIndex];
		if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(Input.Data))
		{
			PCGSplineIntersection::FInputData& SplineInputData = InputSplines.Add_GetRef(PCGSplineIntersection::FInputData{ .Spline = SplineData, .InputIndex = InputIndex });
			SplineInputData.InitializeBounds(HalfDistanceRadius);
		}
	}
	
	// Early out if we have no work to do.
	if (InputSplines.IsEmpty() || Settings->DistanceThreshold <= 0)
	{
		return true;
	}

	// Compute intersections...
	TArray<PCGSplineIntersection::FIntersectionPoint> Intersections;

	TArray<int> SplinesToTest;
	for (int SplineIndex = 0; SplineIndex < InputSplines.Num(); ++SplineIndex)
	{
		SplinesToTest.Reset();

		if (Settings->Type == EPCGSplineIntersectionType::Self)
		{
			SplinesToTest.Add(SplineIndex);
		}
		else
		{
			for (int OtherSplineIndex = SplineIndex + 1; OtherSplineIndex < InputSplines.Num(); ++OtherSplineIndex)
			{
				SplinesToTest.Add(OtherSplineIndex);
			}
		}

		const PCGSplineIntersection::FInputData& CurrentSplineData = InputSplines[SplineIndex];
		const FBox& CurrentBounds = CurrentSplineData.Bounds;
		
		for (int SplineToTest : SplinesToTest)
		{
			const PCGSplineIntersection::FInputData& OtherSplineData = InputSplines[SplineToTest];
			const FBox& OtherBounds = OtherSplineData.Bounds;
			const bool bIsSameData = (OtherSplineData.Spline == CurrentSplineData.Spline);

			// Early test
			if (!CurrentBounds.Intersect(OtherBounds))
			{
				continue;
			}

			// Test on a segment-segment basis
			for (int32 SegmentIndex = 0; SegmentIndex < CurrentSplineData.SegmentBounds.Num(); ++SegmentIndex)
			{
				const FBox& CurrentSegmentBounds = CurrentSplineData.SegmentBounds[SegmentIndex];

				// First early out if the segment does not intersect with the other spline bounds
				if (!CurrentSegmentBounds.Intersect(OtherBounds))
				{
					continue;
				}

				const TArray<FVector>& CurrentSamples = CurrentSplineData.GetSamples(SegmentIndex, Settings->DistanceThreshold);

				// Then iterate through potential segments
				for (int32 OtherSegmentIndex = 0; OtherSegmentIndex < OtherSplineData.SegmentBounds.Num(); ++OtherSegmentIndex)
				{
					// Test against bounds first
					if (!CurrentSegmentBounds.Intersect(OtherSplineData.SegmentBounds[OtherSegmentIndex]))
					{
						continue;
					}

					const TArray<FVector>& OtherSamples = OtherSplineData.GetSamples(OtherSegmentIndex, Settings->DistanceThreshold);
					
					// At this point, we are in intersection-possible territory.
					// We don't have an analytical solution here, but there are a few things to consider:
					// It's possible that two segments intersect in more than one place (up to 4 assumedly),
					// but it's also possible that segments are collinear.
					for (int CurrentSampleIndex = 0; CurrentSampleIndex < CurrentSamples.Num() - 1; ++CurrentSampleIndex)
					{
						const FVector& A = CurrentSamples[CurrentSampleIndex];
						const FVector& B = CurrentSamples[CurrentSampleIndex + 1];

						const int OtherStartSampleIndex = bIsSameData ? CurrentSampleIndex + 1 : 0;
						for (int OtherSampleIndex = OtherStartSampleIndex; OtherSampleIndex < OtherSamples.Num() - 1; ++OtherSampleIndex)
						{
							const FVector& C = OtherSamples[OtherSampleIndex];
							const FVector& D = OtherSamples[OtherSampleIndex + 1];

							double ABStart, ABEnd, CDStart, CDEnd;
							if (PCGSplineIntersection::FindOverlapRange(A, B, C, D, SqrDistanceThreshold, ABStart, ABEnd, CDStart, CDEnd))
							{
								const FVector StartAB = A + (B - A) * ABStart;
								const FVector EndAB = A + (B - A) * ABEnd;
								const FVector StartCD = C + (D - C) * CDStart;
								const FVector EndCD = C + (D - C) * CDEnd;

								PCGSplineIntersection::FIntersectionPoint& Intersection = Intersections.Emplace_GetRef();
								Intersection.A.Spline = CurrentSplineData.Spline;
								Intersection.A.Input = &CurrentSplineData;
								Intersection.A.InputKeyStart = CurrentSplineData.Spline->SplineStruct.FindInputKeyOnSegmentClosestToWorldLocation(StartAB, SegmentIndex);
								Intersection.A.InputKeyEnd = CurrentSplineData.Spline->SplineStruct.FindInputKeyOnSegmentClosestToWorldLocation(EndAB, SegmentIndex);
								Intersection.A.OtherSplines.Add(OtherSplineData.Spline);
								Intersection.A.Points.Add(StartAB);
								Intersection.A.Points.Add(EndAB);

								Intersection.B.Spline = OtherSplineData.Spline;
								Intersection.B.Input = &OtherSplineData;
								Intersection.B.InputKeyStart = OtherSplineData.Spline->SplineStruct.FindInputKeyOnSegmentClosestToWorldLocation(StartCD, OtherSegmentIndex);
								Intersection.B.InputKeyEnd = OtherSplineData.Spline->SplineStruct.FindInputKeyOnSegmentClosestToWorldLocation(EndCD, OtherSegmentIndex);
								Intersection.B.OtherSplines.Add(CurrentSplineData.Spline);
								Intersection.B.Points.Add(StartCD);
								Intersection.B.Points.Add(EndCD);
							}
						}
					}
				}
			}
		}
	}

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	auto LogErrorOnAttributeCreation = [Context](const FName IndexName)
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FailedCreateIndexAttribute", "Failed to create the attribute '{0}'."), FText::FromName(IndexName)), Context);
	};

	// Finally, build output data.
	if (Settings->Output == EPCGSplineIntersectionOutput::IntersectionPointsOnly)
	{
		// Merge intersection points on a global basis, regardless of source spline
		TArray<PCGSplineIntersection::FCollapsedIntersectionPoint> CollapsedIntersections = PCGSplineIntersection::CollapseIntersections(Intersections, Settings->bUseBestFitBox);

		FPCGTaggedData& Output = Outputs.Emplace_GetRef();
		
		UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(Context);
		Output.Data = PointData;

		PointData->SetNumPoints(CollapsedIntersections.Num());

		const EPCGPointNativeProperties PropertiesToAllocate = EPCGPointNativeProperties::Transform | 
			EPCGPointNativeProperties::BoundsMin | 
			EPCGPointNativeProperties::BoundsMax | 
			((Settings->bOutputSplineIndices || Settings->bOutputIntersectionType) ? EPCGPointNativeProperties::MetadataEntry : EPCGPointNativeProperties::None);
		PointData->AllocateProperties(PropertiesToAllocate);

		// Implementation note: there's no point in outputting the intersection type, since these are always intersections with their appropriate sizes.
		// @todo_pcg: add array support for index attributes maybe
		FPCGMetadataAttribute<int32>* FirstIndexAttribute = nullptr;
		FPCGMetadataAttribute<int32>* SecondIndexAttribute = nullptr;

		if (Settings->bOutputSplineIndices)
		{
			auto CreateIndexAttribute = [PointData, &LogErrorOnAttributeCreation](const FName IndexName) -> FPCGMetadataAttribute<int32>*
			{
				FPCGMetadataAttribute<int32>* IndexAttribute = PointData->MutableMetadata()->FindOrCreateAttribute<int32>(IndexName, -1, /*bAllowInterpolation=*/false);
				if (!IndexAttribute)
				{
					LogErrorOnAttributeCreation(IndexName);
				}

				return IndexAttribute;
			};

			FirstIndexAttribute = CreateIndexAttribute(Settings->OriginatingSplineIndexAttribute.GetAttributeName());
			SecondIndexAttribute = CreateIndexAttribute(Settings->IntersectingSplineIndexAttribute.GetAttributeName());
		}

		FPCGPointValueRanges OutRanges(PointData, /*bAllocate=*/false);

		for (int Index = 0; Index < CollapsedIntersections.Num(); ++Index)
		{
			const PCGSplineIntersection::FCollapsedIntersectionPoint& Point = CollapsedIntersections[Index];
			check(Point.Ranges.Num() >= 1);

			OutRanges.TransformRange[Index] = Point.Transform;
			OutRanges.BoundsMinRange[Index] = Point.Bounds.Min;
			OutRanges.BoundsMaxRange[Index] = Point.Bounds.Max;

			PCGMetadataEntryKey* CurrentEntryKey = nullptr;

			if (FirstIndexAttribute || SecondIndexAttribute)
			{
				CurrentEntryKey = &OutRanges.MetadataEntryRange[Index];
				PointData->Metadata->InitializeOnSet(*CurrentEntryKey);
			}

			if (FirstIndexAttribute)
			{
				FirstIndexAttribute->SetValue(*CurrentEntryKey, Point.Ranges[0].Input->InputIndex);
			}

			if (Point.Ranges.Num() > 1 && SecondIndexAttribute)
			{
				SecondIndexAttribute->SetValue(*CurrentEntryKey, Point.Ranges[1].Input->InputIndex);
			}
		}
	}
	else if (Settings->Output == EPCGSplineIntersectionOutput::OriginalSplinesWithIntersections)
	{
		for (const PCGSplineIntersection::FInputData& InputData : InputSplines)
		{
			const UPCGSplineData* InputSpline = InputData.Spline;

			// Filter points for the current spline and collapse points that would be too close.
			// Note: the average position in this case will be when points are collapsed on the spline itself, not against the other splines
			TArray<PCGSplineIntersection::FIntersectionRange> CollapsedPoints = PCGSplineIntersection::FilterAndMergeIntersections(Intersections, InputSpline);
			// Sort by key
			CollapsedPoints.Sort([](const auto& A, const auto& B) { return A.InputKeyStart < B.InputKeyStart; });

			// At this point, we've successfully filtered/collapsed points if it was needed,
			// Or we had no new points - in which case we have nothing to do.
			FPCGTaggedData& Output = Outputs.Add_GetRef(Inputs[InputData.InputIndex]);
			if (!CollapsedPoints.IsEmpty())
			{
				UPCGSplineData* NewSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
				NewSplineData->InitializeFromData(InputData.Spline);

				// implementation note: this does not support typecasting, will just overwrite the attribute.
				FPCGMetadataAttribute<int32>* IndexAttribute = nullptr;
				FPCGMetadataAttribute<int32>* IntersectionTypeAttribute = nullptr;

				if (Settings->bOutputSplineIndices)
				{
					FName IndexAttributeName = Settings->IntersectingSplineIndexAttribute.GetAttributeName();
					IndexAttribute = NewSplineData->MutableMetadata()->FindOrCreateAttribute(IndexAttributeName, -1, /*bAllowInterpolation=*/false);

					if (!IndexAttribute)
					{
						LogErrorOnAttributeCreation(IndexAttributeName);
					}
				}

				if (Settings->bOutputIntersectionType)
				{
					IntersectionTypeAttribute = NewSplineData->MutableMetadata()->FindOrCreateAttribute<int32>(Settings->IntersectionTypeAttribute.GetAttributeName(), static_cast<int32>(EPCGSplineIntersectionPointType::NotAnIntersection), /*bAllowInterpolation=*/false);
					if (!IntersectionTypeAttribute)
					{
						LogErrorOnAttributeCreation(Settings->IntersectionTypeAttribute.GetAttributeName());
					}
				}

				// Implementation note: with the old spline representation, it's not possible to add control points in between other points, 
				// because there are some places in the code where we rely on the index being equal to the key (esp. around the ReparamTable).
				// Not respecting this makes some operations fail, so we need to conform to that expectation at this point in time.
				// However, this means we basically need to have custom tangents everywhere.
				TArray<FSplinePoint> SplinePoints = InputData.Spline->GetSplinePoints();
				TArray<int64> SplinePointsEntryKeys = InputData.Spline->GetMetadataEntryKeysForSplinePoints();
				check(SplinePoints.Num() == SplinePointsEntryKeys.Num() || SplinePointsEntryKeys.IsEmpty());
				const bool bHasMetadata = !SplinePointsEntryKeys.IsEmpty();
				const bool bNeedsMetadata = (IndexAttribute != nullptr || IntersectionTypeAttribute != nullptr);

				// Prepare metadata entry keys if we're going to use them later
				if (bNeedsMetadata && !bHasMetadata)
				{
					SplinePointsEntryKeys.SetNumUninitialized(SplinePoints.Num());
					for (int64& SplinePointEntryKey : SplinePointsEntryKeys)
					{
						SplinePointEntryKey = PCGInvalidEntryKey;
					}
				}

				auto OptionalWriteIntersectionMetadata = [bNeedsMetadata, NewSplineData, IndexAttribute, IntersectionTypeAttribute, &InputData, &InputSplines](int64& EntryKey, const PCGSplineIntersection::FIntersectionRange& IntersectionPoint, EPCGSplineIntersectionPointType IntersectionType)
				{
					if (bNeedsMetadata)
					{
						NewSplineData->Metadata->InitializeOnSet(EntryKey);

						if (IndexAttribute)
						{
							check(!IntersectionPoint.OtherSplines.IsEmpty());
							const PCGSplineIntersection::FInputData* IntersectedSplineData = InputSplines.FindByPredicate([&IntersectionPoint](const PCGSplineIntersection::FInputData& Data) { return Data.Spline == IntersectionPoint.OtherSplines[0]; });
							if(IntersectedSplineData)
							{
								IndexAttribute->SetValue(EntryKey, IntersectedSplineData->InputIndex);
							}
						}

						if (IntersectionTypeAttribute)
						{
							IntersectionTypeAttribute->SetValue(EntryKey, IntersectionType);
						}
					}
				};
				
				constexpr double MinKeyDifferenceThreshold = PCGSplineIntersection::Constants::KeyDifferenceThreshold;
				
				for (const PCGSplineIntersection::FIntersectionRange& PointToAdd : CollapsedPoints)
				{
					TArray<double, TInlineAllocator<2>> Keys;
					TArray<EPCGSplineIntersectionPointType, TInlineAllocator<2>> KeyTypes;

					double IntersectionKeyStart = PointToAdd.InputKeyStart;
					double IntersectionKeyEnd = PointToAdd.InputKeyEnd;
					double LoopKeyShift = 0;

					if (IntersectionKeyEnd < IntersectionKeyStart)
					{
						ensure(PointToAdd.Spline->IsClosed());
						LoopKeyShift = PCGSplineIntersection::Helpers::GetSplineKeyLength(InputSpline);
						IntersectionKeyEnd += LoopKeyShift;
					}

					if ((IntersectionKeyEnd - IntersectionKeyStart) > MinKeyDifferenceThreshold && Settings->bOutputComplexIntersections)
					{
						Keys.Add(IntersectionKeyStart);
						Keys.Add(IntersectionKeyEnd - LoopKeyShift);
						KeyTypes.Add(EPCGSplineIntersectionPointType::IntersectionStart);
						KeyTypes.Add(EPCGSplineIntersectionPointType::IntersectionEnd);
					}
					else
					{
						double AverageKey = 0.5 * (IntersectionKeyStart + IntersectionKeyEnd);
						if (AverageKey > LoopKeyShift)
						{
							AverageKey -= LoopKeyShift;
						}

						Keys.Add(AverageKey);
						KeyTypes.Add(EPCGSplineIntersectionPointType::SinglePointIntersection);
					}

					for(int KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
					{
						const double& Key = Keys[KeyIndex];
						const FVector Position = InputData.Spline->SplineStruct.GetLocationAtSplineInputKey(Key, ESplineCoordinateSpace::Local);

						// Find insertion index @todo_pcg: use bisection
						int InsertionIndex = SplinePoints.Num();
						for (int SPIndex = 1; SPIndex < SplinePoints.Num(); ++SPIndex)
						{
							if (SplinePoints[SPIndex].InputKey > Key)
							{
								InsertionIndex = SPIndex;
								break;
							}
						}

						bool bIsOnLastSegment = (InsertionIndex == SplinePoints.Num());

						// Skip points that are on control points - need to check against previous and next.
						FSplinePoint& PreviousPoint = SplinePoints[InsertionIndex - 1];
						FSplinePoint& NextPoint = SplinePoints[bIsOnLastSegment ? 0 : InsertionIndex];

						const float NextPointVirtualKey = (bIsOnLastSegment ? FMath::CeilToInt(PreviousPoint.InputKey + 1.0e-4f) : NextPoint.InputKey);

						if ((FMath::Abs(PreviousPoint.InputKey - Key) <= MinKeyDifferenceThreshold && (PreviousPoint.Position - Position).SquaredLength() <= SqrDistanceThreshold))
						{
							if (bNeedsMetadata)
							{
								// Mark previous point as an intersection
								OptionalWriteIntersectionMetadata(SplinePointsEntryKeys[InsertionIndex - 1], PointToAdd, KeyTypes[KeyIndex]);
							}

							continue;
						}
						else if ((FMath::Abs(NextPointVirtualKey - Key) <= MinKeyDifferenceThreshold && (NextPoint.Position - Position).SquaredLength() <= SqrDistanceThreshold))
						{
							if (bNeedsMetadata)
							{
								// Mark next point as an intersection
								OptionalWriteIntersectionMetadata(SplinePointsEntryKeys[bIsOnLastSegment ? 0 : InsertionIndex], PointToAdd, KeyTypes[KeyIndex]);
							}

							continue;
						}

						FSplinePoint SplinePoint;
						SplinePoint.InputKey = Key;
						SplinePoint.Position = Position;
						// @todo_pcg: we don't need to add points that are on the control point, however, we might want to do metadata for those
						SplinePoint.ArriveTangent = InputData.Spline->SplineStruct.GetTangentAtSplineInputKey(Key, ESplineCoordinateSpace::Local);
						SplinePoint.LeaveTangent = SplinePoint.ArriveTangent;
						SplinePoint.Rotation = InputData.Spline->SplineStruct.GetQuaternionAtSplineInputKey(Key, ESplineCoordinateSpace::Local).Rotator();
						SplinePoint.Scale = InputData.Spline->SplineStruct.GetScaleAtSplineInputKey(Key);
						SplinePoint.Type = ESplinePointType::CurveCustomTangent;

						// Adjust tangents
						const double KeyRange = FMath::Max(NextPointVirtualKey - PreviousPoint.InputKey, 1.0e-4);
						const double PreviousRatio = (Key - PreviousPoint.InputKey) / KeyRange;
						const double NextRatio = (NextPointVirtualKey - Key) / KeyRange;

						PreviousPoint.LeaveTangent *= PreviousRatio;
						NextPoint.ArriveTangent *= NextRatio;

						// The spline point tangent is "always" taken from the original spline's perspective,
						// so we will assume that the range is always one, so we need to scale accordingly.
						// @todo_pcg: get key at the beginning of the segment
						SplinePoint.ArriveTangent *= (Key - PreviousPoint.InputKey);
						SplinePoint.LeaveTangent *= (NextPointVirtualKey - Key);

						SplinePoints.Insert(SplinePoint, InsertionIndex);
						if (bHasMetadata || bNeedsMetadata)
						{
							PCGMetadataEntryKey EntryKey = PCGInvalidEntryKey;

							if (bHasMetadata)
							{
								InputData.Spline->WriteMetadataToEntry(Key, EntryKey, NewSplineData->MutableMetadata());
							}

							OptionalWriteIntersectionMetadata(EntryKey, PointToAdd, KeyTypes[KeyIndex]);
							SplinePointsEntryKeys.Insert(EntryKey, InsertionIndex);
						}
					}
				}

				// Mark all tangents as custom, and reset the keys
				for(int SPIndex = 0; SPIndex < SplinePoints.Num(); ++SPIndex)
				{
					FSplinePoint& SplinePoint = SplinePoints[SPIndex];
					SplinePoint.InputKey = static_cast<float>(SPIndex);
					if (SplinePoint.Type == ESplinePointType::Curve || SplinePoint.Type == ESplinePointType::CurveClamped)
					{
						SplinePoints[SPIndex].Type = ESplinePointType::CurveCustomTangent;
					}
				}

				// Initialize data now
				NewSplineData->Initialize(SplinePoints,
					InputData.Spline->SplineStruct.IsClosedLoop(),
					InputData.Spline->SplineStruct.GetTransform(),
					SplinePointsEntryKeys);

				Output.Data = NewSplineData;
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
