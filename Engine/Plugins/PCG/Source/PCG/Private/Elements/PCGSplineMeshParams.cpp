// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSplineMeshParams.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplineMeshParams)

uint32 GetTypeHash(const FPCGSplineMeshParams& Key)
{
	if (!Key.Hash.IsSet())
	{
		Key.Hash = Key.ComputeHash();
	}

	return Key.Hash.GetValue();
}

bool FPCGSplineMeshParams::operator==(const FPCGSplineMeshParams& Other) const
{
	return ForwardAxis == Other.ForwardAxis
		&& bScaleMeshToBounds == Other.bScaleMeshToBounds
		&& bScaleMeshToLandscapeSplineFullWidth == Other.bScaleMeshToLandscapeSplineFullWidth
		&& bSmoothInterpRollScale == Other.bSmoothInterpRollScale
		&& FMath::IsNearlyEqual(NaniteClusterBoundsScale, Other.NaniteClusterBoundsScale)
		&& FMath::IsNearlyEqual(SplineBoundaryMin, Other.SplineBoundaryMin)
		&& FMath::IsNearlyEqual(SplineBoundaryMax, Other.SplineBoundaryMax)
		&& FMath::IsNearlyEqual(StartRollDegrees, Other.StartRollDegrees)
		&& FMath::IsNearlyEqual(EndRollDegrees, Other.EndRollDegrees)
		&& (StartPosition - Other.StartPosition).IsNearlyZero()
		&& (StartTangent - Other.StartTangent).IsNearlyZero()
		&& (EndPosition - Other.EndPosition).IsNearlyZero()
		&& (EndTangent - Other.EndTangent).IsNearlyZero()
		&& (StartScale - Other.StartScale).IsNearlyZero()
		&& (EndScale - Other.EndScale).IsNearlyZero();
}

uint32 FPCGSplineMeshParams::ComputeHash() const
{
	auto HashCombineAll = [](auto... Values)
	{
		uint32 HashVal = 0;
		([&HashVal, &Values]()
		{
			HashVal = HashCombine(HashVal, GetTypeHash(Values));
		}(), ...);
		return HashVal;
	};
	
	return HashCombineAll(
		ForwardAxis,
		bScaleMeshToBounds,
		bScaleMeshToLandscapeSplineFullWidth,
		SplineUpDir,
		NaniteClusterBoundsScale,
		SplineBoundaryMin,
		SplineBoundaryMax,
		bSmoothInterpRollScale,
		StartPosition,
		StartTangent,
		StartRollDegrees,
		StartScale,
		EndPosition,
		EndTangent,
		EndRollDegrees,
		EndScale);
}
