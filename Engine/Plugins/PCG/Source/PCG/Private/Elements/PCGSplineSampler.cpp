// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSplineSampler.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "Data/PCGIntersectionData.h"
#include "Data/PCGLandscapeSplineData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPolyLineData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGSpatialData.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "Async/ParallelFor.h"
#include "Components/SplineComponent.h"
#include "Voronoi/Voronoi.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplineSampler)

#define LOCTEXT_NAMESPACE "PCGSplineSamplerElement"

namespace PCGSplineSamplerHelpers
{
	/** Intersects a line segment with all line segments of a polygon. May return duplicates if the segment intersects a vertex of the polygon. */
	TArray<FVector2D> SegmentPolygonIntersection2D(const FVector2D& SegmentStart, const FVector2D& SegmentEnd, const UE::Geometry::FPolygon2d& InPolygon)
	{
		const int32 PointCount = InPolygon.VertexCount();

		if (PointCount < 3)
		{
			return {};
		}

		TArray<FVector2D> IntersectionPoints;

		// Taken from TPolygon2::FindIntersections, with a slight modification to handle the edge case where the segment intersects the polygon
		// exactly at one of the polygon vertices
		const UE::Geometry::FSegment2d InSegment{SegmentStart, SegmentEnd};
		TArray<FVector2D> PolygonVerticesIntersects;
		
		for (UE::Geometry::FSegment2d PolygonSegment : InPolygon.Segments())
		{
			if (PolygonSegment.Intersects(InSegment))
			{
				UE::Geometry::FIntrSegment2Segment2d Intersection(PolygonSegment, InSegment);
				if (Intersection.Find())
				{
					TArray<FVector2D, TFixedAllocator<2>> TentativePoints = {Intersection.Point0};
					if (Intersection.Quantity == 2)
					{
						TentativePoints.Add(Intersection.Point1);
					}

					// For each tentative point, check if the intersection is a vertex. If so, discard it if we already found it.
					// Make sure to handle collocated vertices.
					TArray<FVector2D, TFixedAllocator<2>> PolygonVertices = {PolygonSegment.StartPoint()};
					if (!FMath::IsNearlyZero(PolygonSegment.Extent))
					{
						PolygonVertices.Add(PolygonSegment.EndPoint());
					}
					
					for (const FVector2D& TentativePoint : TentativePoints)
					{
						bool bDiscardPoint = false;
						for (const FVector2D& PolygonVertex : PolygonVertices)
						{
							if (PolygonVertex.Equals(TentativePoint))
							{
								if (PolygonVerticesIntersects.Contains(PolygonVertex))
								{
									bDiscardPoint = true;
								}
								else
								{
									PolygonVerticesIntersects.Add(PolygonVertex);
								}
							}
						}

						if (!bDiscardPoint)
						{
							IntersectionPoints.Add(TentativePoint);
						}
					}
				}
			}
		}

		return IntersectionPoints;
	}

	bool PointInsidePolygon2D(const UE::Geometry::FPolygon2d& InPolygon, const FVector2D& Point) 
	{
		if (InPolygon.VertexCount() < 3)
		{
			return false;
		}

		return InPolygon.Contains(Point);
	}

	/** Projects a point in space onto the approximated surface defined by a closed spline. */
	FVector::FReal ProjectOntoSplineInteriorSurface(const TArray<FVector>& SplinePoints, const FVector& PointToProject)
	{
		// Compute average Z value weighted by 1 / Distance^2
		FVector::FReal SumZ = 0.f;
		FVector::FReal SumWeights = 0.f;

		for (int32 PointIndex = 0; PointIndex < SplinePoints.Num(); ++PointIndex)
		{
			const FVector& Point = SplinePoints[PointIndex];
			// TODO: It would be more accurate to use distance to the polyline instead of distance to the polyline points,
			// however it would also be much more expensive. Perhaps worth investigating when Params.bTreatSplineAsPolyline is true.
			const FVector::FReal DistanceSquared = FVector::DistSquaredXY(PointToProject, Point); 

			// If sample point overlaps exactly with a border point, then that must be the height.
			if (FMath::IsNearlyZero(DistanceSquared))
			{
				return Point.Z;
			}

			// TODO[UE-205462]: It would be ideal to find a better constraint than inverse squared distance, which produces boundary artifacts for undersampled splines.
			const FVector::FReal Weight = 1.f / DistanceSquared;

			SumWeights += Weight;
			SumZ += Point.Z * Weight;
		}

		// If the weights sum to zero then there were no points to sample, and therefore nothing to project to.
		// TODO: This is not robust for interior points which are enormously far away from the spline and could collapse to 0 incorrectly.
		return FMath::IsNearlyZero(SumWeights) ? 0.0 : SumZ / SumWeights;
	}

	struct FSamplerResult
	{
		FTransform LocalTransform;
		FBox Box = FBox::BuildAABB(FVector::ZeroVector, FVector::OneVector);
		FVector::FReal Curvature = 0;
		int SampleIndex = 0;
		int SegmentIndex = 0;
		int SubsegmentIndex = 0;
		float InputKey = 0.0f;
		FVector ArriveTangent = FVector::Zero();
		FVector LeaveTangent = FVector::Zero();
		FVector::FReal PreviousDeltaAngle = 0;
		FVector::FReal NextDeltaAngle = 0;
		FVector::FReal Alpha = 0.0f;
		FVector::FReal Distance = 0.0f;
	};

	void SetSeed(int32& OutSeed, const FTransform& InTransform, const FVector& LSPosition, const FPCGSplineSamplerParams& Params, int ExternalSeed, int SampleIndex)
	{
		if (Params.SeedingMode == EPCGSplineSamplingSeedingMode::SeedFromPosition)
		{
			if (!Params.bSeedFromLocalPosition)
			{
				if (!Params.bSeedFrom2DPosition)
				{
					OutSeed = PCGHelpers::ComputeSeedFromPosition(InTransform.GetLocation());
				}
				else
				{
					const FVector WSPosition = InTransform.GetLocation();
					OutSeed = PCGHelpers::ComputeSeed((int)WSPosition.X, (int)WSPosition.Y);
				}
			}
			else // Use provided local position
			{
				if (!Params.bSeedFrom2DPosition)
				{
					OutSeed = PCGHelpers::ComputeSeedFromPosition(LSPosition);
				}
				else
				{
					OutSeed = PCGHelpers::ComputeSeed((int)LSPosition.X, (int)LSPosition.Y);
				}
			}
		}
		else
		{
			OutSeed = PCGHelpers::ComputeSeed(ExternalSeed, SampleIndex);
		}
	}

	struct FStepSampler
	{
		FStepSampler(const UPCGPolyLineData* InLineData, const FPCGSplineSamplerParams& Params, int InSeed)
			: LineData(InLineData)
			, Seed(InSeed)
			, bComputeCurvature(Params.bComputeCurvature)
			, bComputeTangents(Params.bComputeTangents)
			, bComputeAlpha(Params.bComputeAlpha)
			, bComputeDistance(Params.bComputeDistance)
		{
			check(LineData);
			CurrentSegmentIndex = 0;
		}

		virtual void Step(FSamplerResult& OutSamplerResult) = 0;
		virtual bool IsDone() const = 0;

		const UPCGPolyLineData* LineData = nullptr;
		int Seed = 0;
		int SampleIndex = 0;
		int CurrentSegmentIndex = 0;
		FVector::FReal DistanceToCurrentSegment = 0.0;
		bool bComputeCurvature = false;
		bool bComputeTangents = false;
		bool bComputeAlpha = false;
		bool bComputeDistance = false;
	};

	struct FSubdivisionStepSampler : public FStepSampler
	{
		FSubdivisionStepSampler(const UPCGPolyLineData* InLineData, const FPCGSplineSamplerParams& Params, int InSeed)
			: FStepSampler(InLineData, Params, InSeed)
		{
			NumSegments = LineData->GetNumSegments();
			SubdivisionsPerSegment = Params.SubdivisionsPerSegment;

			CurrentSegmentIndex = 0;
			SubpointIndex = 0;
		}

		virtual void Step(FSamplerResult& OutResult) override
		{
			const int PreviousSegmentIndex = (CurrentSegmentIndex > 0 ? CurrentSegmentIndex : NumSegments) - 1;

			// To capture the last key point on the spline, we sample the point at the end of the previous segment.
			const bool bLastKeyPoint = CurrentSegmentIndex == LineData->GetNumSegments();
			const int SegmentIndex = bLastKeyPoint ? PreviousSegmentIndex : CurrentSegmentIndex;

			const FVector::FReal SegmentLength = LineData->GetSegmentLength(SegmentIndex);
			const FVector::FReal SegmentStep = SegmentLength / (SubdivisionsPerSegment + 1);
			const FVector::FReal DistanceAlongSegment = bLastKeyPoint ? SegmentLength : SubpointIndex * SegmentStep;

			FBox& OutBox = OutResult.Box;
			FTransform& OutTransform = OutResult.LocalTransform;
			OutTransform = LineData->GetTransformAtDistance(SegmentIndex, DistanceAlongSegment, /*bWorldSpace=*/false, &OutBox);
			OutResult.SampleIndex = SampleIndex++;
			OutResult.SegmentIndex = LineData->IsClosed() ? CurrentSegmentIndex : SegmentIndex;
			OutResult.SubsegmentIndex = SubpointIndex;
			OutResult.InputKey = LineData->GetInputKeyAtDistance(SegmentIndex, DistanceAlongSegment);

			if (bComputeCurvature)
			{
				OutResult.Curvature = LineData->GetCurvatureAtDistance(SegmentIndex, DistanceAlongSegment);
			}

			if (bComputeTangents)
			{
				// Control points have actual Arrive and Leave tangents
				if (SubpointIndex == 0)
				{
					LineData->GetTangentsAtSegmentStart(CurrentSegmentIndex, OutResult.ArriveTangent, OutResult.LeaveTangent);
				}
				else
				{
					// For a non-control-point, we can get the normalized tangent at least.
					const FVector Forward = OutTransform.GetRotation().GetForwardVector();
					OutResult.ArriveTangent = Forward;
					OutResult.LeaveTangent = Forward;
				}
			}

			if (bComputeAlpha)
			{
				OutResult.Alpha = LineData->GetAlphaAtDistance(SegmentIndex, DistanceAlongSegment);
			}

			if (bComputeDistance)
			{
				// When we step onto a new segment, add the length of the previous segment onto the distance to our current segment.
				if (SegmentIndex > 0 && !bLastKeyPoint && SubpointIndex == 0)
				{
					DistanceToCurrentSegment += LineData->GetSegmentLength(SegmentIndex - 1);
				}

				OutResult.Distance = DistanceToCurrentSegment + DistanceAlongSegment;
			}

			const double ScaleFactor = 0.5 * SegmentStep / (FMath::IsNearlyZero(OutTransform.GetScale3D().X) ? UE_DOUBLE_SMALL_NUMBER : OutTransform.GetScale3D().X);

			if (SubpointIndex == 0)
			{
				const FVector::FReal PreviousSegmentLength = LineData->GetSegmentLength(PreviousSegmentIndex);
				FTransform PreviousSegmentEndTransform = LineData->GetTransformAtDistance(PreviousSegmentIndex, PreviousSegmentLength, /*bWorldSpace=*/false);

				if ((PreviousSegmentEndTransform.GetLocation() - OutTransform.GetLocation()).Length() <= KINDA_SMALL_NUMBER)
				{
					OutBox.Min.X *= 0.5 * PreviousSegmentLength / ((FMath::IsNearlyZero(PreviousSegmentEndTransform.GetScale3D().X) ? UE_DOUBLE_SMALL_NUMBER : PreviousSegmentEndTransform.GetScale3D().X) * (SubdivisionsPerSegment + 1));
				}
				else
				{
					OutBox.Min.X *= ScaleFactor;
				}
			}
			else
			{
				OutBox.Min.X *= ScaleFactor;
			}

			OutBox.Max.X *= ScaleFactor;

			++SubpointIndex;
			if (SubpointIndex > SubdivisionsPerSegment || bLastKeyPoint)
			{
				SubpointIndex = 0;
				++CurrentSegmentIndex;
			}
		}

		virtual bool IsDone() const override
		{
			// Subdivision Sampler is not done until it captures the final control point, which does not have a valid segment associated unless the spline is a closed loop.
			const int NumSegmentsWhenDone = ((LineData && LineData->IsClosed()) || NumSegments == 0) ? NumSegments : NumSegments + 1;
			return CurrentSegmentIndex >= NumSegmentsWhenDone;
		}

		int NumSegments = 0;
		int SubdivisionsPerSegment = 0;
		int SubpointIndex = 0;
	};

	struct FDistanceStepSampler : public FStepSampler
	{
		FDistanceStepSampler(const UPCGPolyLineData* InLineData, const FPCGSplineSamplerParams& Params, int InSeed)
			: FStepSampler(InLineData, Params, InSeed)
		{
			StartOffset = FMath::Max(0, Params.StartOffset);
			const FVector::FReal EndOffset = FMath::Max(0, Params.EndOffset);

			CurrentDistance = StartOffset;
			TotalDistance = InLineData->GetLength();
			EndDistance = TotalDistance - EndOffset;

			const FVector::FReal TotalLength = EndDistance - StartOffset;

			if (Params.Mode == EPCGSplineSamplingMode::NumberOfSamples)
			{
				TotalNumSamples = Params.NumSamples;
			}
			else if (Params.Mode == EPCGSplineSamplingMode::Distance && Params.bFitToCurve)
			{
				// In Distance mode we can cover the full spline by finding the nearest whole number of samples that would fit, and treating the mode as NumberOfSamples instead.
				TotalNumSamples = (Params.DistanceIncrement > 0) ? (TotalLength / Params.DistanceIncrement) : 0;
			}

			if (TotalNumSamples > 0)
			{
				// Compute an increment which evenly distributes sample points along the length of the curve.
				DistanceIncrement = TotalLength / (LineData->IsClosed() ? TotalNumSamples : FMath::Max(1, TotalNumSamples - 1));
			}
			else
			{
				DistanceIncrement = Params.DistanceIncrement;
			}

			MaxRandomOffset = FMath::Max(0.0, Params.MaxRandomOffsetNormalized) * DistanceIncrement / 2.0;
			bUseRandomOffset = !FMath::IsNearlyZero(MaxRandomOffset);

			if (bUseRandomOffset)
			{
				RandomSource.Initialize(Seed);
			}
		}

		virtual void Step(FSamplerResult& OutResult) override
		{
			const int NumSegments = LineData->GetNumSegments();
			// If the current segment exceeds the number of segments, assume the segment is the last valid segment.
			if (CurrentSegmentIndex >= NumSegments)
			{
				CurrentSegmentIndex = NumSegments - 1;
				// Adjust the current distances relative to the start of the final segment, so that the final control point is sampled.
				CurrentDistance = LineData->GetSegmentLength(CurrentSegmentIndex);
				DistanceToCurrentSegment = LineData->GetDistanceAtSegmentStart(CurrentSegmentIndex);
			}

			FVector::FReal OffsetDistance = CurrentDistance + (bUseRandomOffset ? RandomSource.FRandRange(-MaxRandomOffset, MaxRandomOffset) : 0.0);

			// Prevent samples from wrapping around on open splines.
			if (!LineData->IsClosed() && DistanceToCurrentSegment + OffsetDistance < StartOffset)
			{
				OffsetDistance = StartOffset - DistanceToCurrentSegment;
			}

			if (!LineData->IsClosed() && DistanceToCurrentSegment + OffsetDistance >= EndDistance)
			{
				OffsetDistance = EndDistance - DistanceToCurrentSegment;
			}

			FVector::FReal CurrentSegmentLength = LineData->GetSegmentLength(CurrentSegmentIndex);
			FTransform& OutTransform = OutResult.LocalTransform;
			FBox& OutBox = OutResult.Box;
			OutTransform = LineData->GetTransformAtDistance(CurrentSegmentIndex, OffsetDistance, /*bWorldSpace=*/false, &OutBox);
			OutResult.SampleIndex = SampleIndex++;
			OutResult.SegmentIndex = CurrentSegmentIndex;
			OutResult.InputKey = LineData->GetInputKeyAtDistance(CurrentSegmentIndex, OffsetDistance);

			// Set min/max to half of extent
			const double ScaleFactor = 0.5 * DistanceIncrement / (FMath::IsNearlyZero(OutTransform.GetScale3D().X) ? UE_DOUBLE_SMALL_NUMBER : OutTransform.GetScale3D().X);
			OutBox.Min.X *= ScaleFactor;
			OutBox.Max.X *= ScaleFactor;

			if (bComputeCurvature)
			{
				OutResult.Curvature = LineData->GetCurvatureAtDistance(CurrentSegmentIndex, OffsetDistance);
			}

			if (bComputeTangents)
			{
				const FVector Forward = OutTransform.GetRotation().GetForwardVector();
				OutResult.ArriveTangent = Forward;
				OutResult.LeaveTangent = Forward;
			}

			if (bComputeAlpha)
			{
				OutResult.Alpha = LineData->GetAlphaAtDistance(CurrentSegmentIndex, OffsetDistance);
			}

			if (bComputeDistance)
			{
				OutResult.Distance = DistanceToCurrentSegment + OffsetDistance;
			}

			// Increment the current distance to get our next sample location. Note that we don't use the offset distance, since the new sample doesn't care
			// about the previous sample location.
			CurrentDistance += DistanceIncrement;
			++CurrentNumSamples;

			while(CurrentDistance > CurrentSegmentLength)
			{
				CurrentDistance -= CurrentSegmentLength;
				++CurrentSegmentIndex;

				DistanceToCurrentSegment += CurrentSegmentLength;

				if (IsDone() || CurrentSegmentLength <= 0)
				{
					break;
				}
				else
				{
					CurrentSegmentLength = LineData->GetSegmentLength(CurrentSegmentIndex);
				}
			}
		}

		virtual bool IsDone() const override
		{
			return (DistanceToCurrentSegment + CurrentDistance > EndDistance + UE_DOUBLE_SMALL_NUMBER) || (CurrentNumSamples == TotalNumSamples);
		}

		FVector::FReal CurrentDistance = 0.0;
		FVector::FReal DistanceIncrement = 0.0;
		FVector::FReal StartOffset = 0.0;
		FVector::FReal EndDistance = 0.0;
		FVector::FReal TotalDistance = 0.0;
		FVector::FReal MaxRandomOffset = 0.0;
		FRandomStream RandomSource;
		bool bUseRandomOffset = false;

		int CurrentNumSamples = 0;
		int TotalNumSamples = -1;
	};

	struct FDimensionSampler
	{
		FDimensionSampler(const UPCGPolyLineData* InLineData, const UPCGSpatialData* InBoundingShapeData, const UPCGSpatialData* InProjectionTarget, const FPCGProjectionParams& InProjectionParams, const FPCGSplineSamplerParams& InParams, int InSeed, UPCGBasePointData* OutPointData)
			: Params(InParams)
			, Seed(InSeed)
		{
			check(InLineData);
			LineData = InLineData;
			BoundingShapeData = InBoundingShapeData;
			ProjectionTargetData = InProjectionTarget;
			ProjectionParams = InProjectionParams;

			UPCGMetadata* Metadata = OutPointData ? OutPointData->Metadata.Get() : nullptr;

			// Initialize metadata accessors if needed
			if (Metadata)
			{
				constexpr double DefaultValue = 0.0;

				bSetMetadata = false;

				if (Params.bComputeDirectionDelta)
				{
					NextDirectionDeltaAttribute = Metadata->FindOrCreateAttribute<double>(Params.NextDirectionDeltaAttribute, DefaultValue);
					bSetMetadata |= (NextDirectionDeltaAttribute != nullptr);
				}

				if (Params.bComputeCurvature)
				{
					CurvatureAttribute = Metadata->FindOrCreateAttribute<double>(Params.CurvatureAttribute, DefaultValue);
					bSetMetadata |= (CurvatureAttribute != nullptr);
				}

				if (Params.bComputeSegmentIndex)
				{
					SegmentIndexAttribute = Metadata->FindOrCreateAttribute<int>(Params.SegmentIndexAttribute, static_cast<int>(DefaultValue));
					bSetMetadata |= (SegmentIndexAttribute != nullptr);
				}

				if (Params.bComputeSubsegmentIndex && Params.Mode == EPCGSplineSamplingMode::Subdivision)
				{
					SubsegmentIndexAttribute = Metadata->FindOrCreateAttribute<int>(Params.SubsegmentIndexAttribute, static_cast<int>(DefaultValue));
					bSetMetadata |= (SubsegmentIndexAttribute != nullptr);
				}

				if (Params.bComputeTangents)
				{
					ArriveTangentAttribute = Metadata->FindOrCreateAttribute<FVector>(Params.ArriveTangentAttribute, FVector::Zero());
					bSetMetadata |= (ArriveTangentAttribute != nullptr);

					LeaveTangentAttribute = Metadata->FindOrCreateAttribute<FVector>(Params.LeaveTangentAttribute, FVector::Zero());
					bSetMetadata |= (LeaveTangentAttribute != nullptr);
				}

				if (Params.bComputeAlpha)
				{
					AlphaAttribute = Metadata->FindOrCreateAttribute<double>(Params.AlphaAttribute, DefaultValue);
					bSetMetadata |= (AlphaAttribute != nullptr);
				}

				if (Params.bComputeDistance)
				{
					DistanceAttribute = Metadata->FindOrCreateAttribute<double>(Params.DistanceAttribute, DefaultValue);
					bSetMetadata |= (DistanceAttribute != nullptr);
				}

				if (Params.bComputeInputKey)
				{
					InputKeyAttribute = Metadata->FindOrCreateAttribute<float>(Params.InputKeyAttribute, static_cast<float>(DefaultValue));
					bSetMetadata |= (InputKeyAttribute != nullptr);
				}
			}
		}

		virtual ~FDimensionSampler() = default;

		virtual void SetMetadata(const FSamplerResult& InResult, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata)
		{
			if (bSetMetadata)
			{
				OutMetadata->InitializeOnSet(OutPoint.MetadataEntry);

				if (NextDirectionDeltaAttribute)
				{
					NextDirectionDeltaAttribute->SetValue(OutPoint.MetadataEntry, InResult.NextDeltaAngle);
				}

				if (CurvatureAttribute)
				{
					CurvatureAttribute->SetValue(OutPoint.MetadataEntry, InResult.Curvature);
				}

				if (SegmentIndexAttribute)
				{
					SegmentIndexAttribute->SetValue(OutPoint.MetadataEntry, InResult.SegmentIndex);
				}
			
				if (SubsegmentIndexAttribute)
				{
					SubsegmentIndexAttribute->SetValue(OutPoint.MetadataEntry, InResult.SubsegmentIndex);
				}

				if (ArriveTangentAttribute)
				{
					ArriveTangentAttribute->SetValue(OutPoint.MetadataEntry, InResult.ArriveTangent);
				}

				if (LeaveTangentAttribute)
				{
					LeaveTangentAttribute->SetValue(OutPoint.MetadataEntry, InResult.LeaveTangent);
				}

				if (AlphaAttribute)
				{
					AlphaAttribute->SetValue(OutPoint.MetadataEntry, InResult.Alpha);
				}

				if (DistanceAttribute)
				{
					DistanceAttribute->SetValue(OutPoint.MetadataEntry, InResult.Distance);
				}

				if (InputKeyAttribute)
				{
					InputKeyAttribute->SetValue(OutPoint.MetadataEntry, InResult.InputKey);
				}
			}
		}

		virtual void Sample(const FSamplerResult& InResult, UPCGBasePointData* OutPointData)
		{
			// Mimic PCGSplineData sampling.
			const FTransform Transform = InResult.LocalTransform * LineData->GetTransform();

			bool bValid = true;

			FPCGPoint OutPoint;
			OutPoint.Density = 1.0f;
			OutPoint.SetLocalBounds(InResult.Box);
			OutPoint.Steepness = Params.PointSteepness;
			
			LineData->WriteMetadataToPoint(InResult.InputKey, OutPoint, OutPointData->Metadata);

			if (ProjectionTargetData)
			{
				FPCGPoint ProjPoint;
				bValid = ProjectionTargetData->ProjectPoint(Transform, InResult.Box, ProjectionParams, ProjPoint, OutPointData->Metadata);
				OutPoint.Transform = ProjPoint.Transform;
			}
			else
			{
				OutPoint.Transform = Transform;
			}

			// Only add valid points within bounds
			FPCGPoint BoundsTestPoint;
			if (bValid && (!BoundingShapeData || BoundingShapeData->SamplePoint(Transform, InResult.Box, BoundsTestPoint, nullptr)))
			{
				SetSeed(OutPoint.Seed, OutPoint.Transform, InResult.LocalTransform.GetLocation(), Params, Seed, InResult.SampleIndex);
				SetMetadata(InResult, OutPoint, OutPointData->Metadata);
				
				// @todo_pcg: ideally we could preallocate the number of points here if the Samplers could give a hint on number of points
				const int32 NumPoints = OutPointData->GetNumPoints();
				OutPointData->SetNumPoints(NumPoints+1);
				// @todo_pcg: be more precise about allocation (sampler could provide properties to allocate)
				OutPointData->AllocateProperties(EPCGPointNativeProperties::All);

				FPCGPointValueRanges OutRanges(OutPointData, /*bAllocate=*/false);
				OutRanges.SetFromPoint(NumPoints, OutPoint);
			}
		}

		const FPCGSplineSamplerParams Params;
		const UPCGPolyLineData* LineData = nullptr;
		const UPCGSpatialData* BoundingShapeData = nullptr;
		const UPCGSpatialData* ProjectionTargetData = nullptr;
		FPCGProjectionParams ProjectionParams;
		int Seed = 0;

		bool bSetMetadata = false;
		FPCGMetadataAttribute<double>* NextDirectionDeltaAttribute = nullptr;
		FPCGMetadataAttribute<double>* CurvatureAttribute = nullptr;
		FPCGMetadataAttribute<int>* SegmentIndexAttribute = nullptr;
		FPCGMetadataAttribute<int>* SubsegmentIndexAttribute = nullptr;
		FPCGMetadataAttribute<FVector>* LeaveTangentAttribute = nullptr;
		FPCGMetadataAttribute<FVector>* ArriveTangentAttribute = nullptr;
		FPCGMetadataAttribute<double>* AlphaAttribute = nullptr;
		FPCGMetadataAttribute<double>* DistanceAttribute = nullptr;
		FPCGMetadataAttribute<float>* InputKeyAttribute = nullptr;
	};

	/** Samples in a volume surrounding the poly line. */
	struct FVolumeSampler : public FDimensionSampler
	{
		FVolumeSampler(const UPCGPolyLineData* InLineData, const UPCGSpatialData* InBoundingShapeData, const UPCGSpatialData* InProjectionTarget, const FPCGProjectionParams& InProjectionParams, const FPCGSplineSamplerParams& Params, int InSeed, UPCGBasePointData* OutPointData)
			: FDimensionSampler(InLineData, InBoundingShapeData, InProjectionTarget, InProjectionParams, Params, InSeed, OutPointData)
		{
			Fill = Params.Fill;
			NumPlanarSteps = 1 + ((Params.Dimension == EPCGSplineSamplingDimension::OnVertical) ? 0 : Params.NumPlanarSubdivisions);
			NumHeightSteps = 1 + ((Params.Dimension == EPCGSplineSamplingDimension::OnHorizontal) ? 0 : Params.NumHeightSubdivisions);
		}

		virtual void Sample(const FSamplerResult& InResult, UPCGBasePointData* OutPointData) override
		{
			const FTransform& TransformLS = InResult.LocalTransform;
			FTransform TransformWS = TransformLS * LineData->GetTransform();
			FBox InBox = InResult.Box;

			// We're assuming that we can scale against the origin in this method so this should always be true.
			// We will also assume that we can separate the curve into 4 ellipse sections for radius checks
			check(InBox.Max.Y > 0 && InBox.Min.Y < 0 && InBox.Max.Z > 0 && InBox.Min.Z < 0);

			const FVector::FReal YHalfStep = 0.5 * (InBox.Max.Y - InBox.Min.Y) / (FVector::FReal)NumPlanarSteps;
			const FVector::FReal ZHalfStep = 0.5 * (InBox.Max.Z - InBox.Min.Z) / (FVector::FReal)NumHeightSteps;

			FBox SubBox = InBox;
			SubBox.Min /= FVector(1.0, NumPlanarSteps, NumHeightSteps);
			SubBox.Max /= FVector(1.0, NumPlanarSteps, NumHeightSteps);

			// TODO: we can optimize this if we are in the "edges only case" to only pick boundary values.
			FVector::FReal CurrentZ = (InBox.Min.Z + ZHalfStep);
			while (CurrentZ <= InBox.Max.Z - ZHalfStep + KINDA_SMALL_NUMBER)
			{
				// Compute inner/outer distance Z contribution (squared value since we'll compare against 1)
				const FVector::FReal InnerZ = ((NumHeightSteps > 1) ? (CurrentZ - FMath::Sign(CurrentZ) * ZHalfStep) : 0);
				const FVector::FReal OuterZ = ((NumHeightSteps > 1) ? (CurrentZ + FMath::Sign(CurrentZ) * ZHalfStep) : 0);

				// TODO: based on the current Z, we can compute the "unit" circle (as seen below) so we don't run outside of it by design, which would be a bit more efficient
				//  care needs to be taken to make sure we are on the same "steps" though
				for (FVector::FReal CurrentY = (InBox.Min.Y + YHalfStep); CurrentY <= (InBox.Max.Y - YHalfStep + KINDA_SMALL_NUMBER); CurrentY += 2.0 * YHalfStep)
				{
					// Compute inner/outer distance Y contribution
					const FVector::FReal InnerY = ((NumPlanarSteps > 1) ? (CurrentY - FMath::Sign(CurrentY) * YHalfStep) : 0);
					const FVector::FReal OuterY = ((NumPlanarSteps > 1) ? (CurrentY + FMath::Sign(CurrentY) * YHalfStep) : 0);

					const FVector::FReal InnerDistance = FMath::Square((CurrentZ >= 0) ? (InnerZ / InBox.Max.Z) : (InnerZ / InBox.Min.Z)) + FMath::Square((CurrentY >= 0) ? (InnerY / InBox.Max.Y) : (InnerY / InBox.Min.Y));
					const FVector::FReal OuterDistance = FMath::Square((CurrentZ >= 0) ? (OuterZ / InBox.Max.Z) : (OuterZ / InBox.Min.Z)) + FMath::Square((CurrentY >= 0) ? (OuterY / InBox.Max.Y) : (OuterY / InBox.Min.Y));

					// Check if we should consider this point based on the fill mode / the position in the iteration
					// If the normalized z^2 + y^2 > 1, then there's no point in testing this point
					if (InnerDistance >= 1.0 + KINDA_SMALL_NUMBER)
					{
						continue; // fully outside the unit circle
					}
					else if (Fill == EPCGSplineSamplingFill::EdgesOnly && OuterDistance < 1.0 - KINDA_SMALL_NUMBER)
					{
						continue; // Not the edge point
					}

					FVector TentativeLocationLS = FVector(0.0, CurrentY, CurrentZ);
					FTransform TentativeTransform = TransformWS;
					TentativeTransform.SetLocation(TransformWS.TransformPosition(TentativeLocationLS));

					// Sample spline to get density.
					FPCGPoint OutPoint;
					if (!LineData->SamplePoint(TentativeTransform, SubBox, OutPoint, OutPointData->Metadata))
					{
						continue;
					}

					// Project point if projection target provided.
					if (ProjectionTargetData)
					{
						FPCGPoint ProjPoint;
						if (!ProjectionTargetData->ProjectPoint(OutPoint.Transform, OutPoint.GetLocalBounds(), ProjectionParams, ProjPoint, OutPointData->Metadata))
						{
							continue;
						}
						OutPoint.Transform = ProjPoint.Transform;
					}

					// Prune points outside of bounds
					FPCGPoint BoundsTestPoint;
					if (BoundingShapeData && !BoundingShapeData->SamplePoint(OutPoint.Transform, OutPoint.GetLocalBounds(), BoundsTestPoint, nullptr))
					{
						continue;
					}

					SetSeed(OutPoint.Seed, OutPoint.Transform, TentativeLocationLS, Params, Seed, InResult.SampleIndex);
					OutPoint.Steepness = Params.PointSteepness;
					SetMetadata(InResult, OutPoint, OutPointData->Metadata);

					// @todo_pcg: ideally we could preallocate the number of points here if the Samplers could give a hint on number of points
					const int32 NumPoints = OutPointData->GetNumPoints();
					OutPointData->SetNumPoints(NumPoints + 1);
					// @todo_pcg: be more precise about allocation (sampler could provide properties to allocate)
					OutPointData->AllocateProperties(EPCGPointNativeProperties::All);
					
					FPCGPointValueRanges OutRanges(OutPointData, /*bAllocate=*/false);
					OutRanges.SetFromPoint(NumPoints, OutPoint);
				}

				CurrentZ += 2.0 * ZHalfStep;
			}
		}

		EPCGSplineSamplingFill Fill;
		int NumPlanarSteps;
		int NumHeightSteps;
	};

	void SampleLineData(FPCGContext* Context, const UPCGPolyLineData* LineData, const UPCGSpatialData* InBoundingShapeData, const UPCGSpatialData* InProjectionTarget, const FPCGProjectionParams& InProjectionParams, const FPCGSplineSamplerParams& Params, UPCGBasePointData* OutPointData)
	{
		check(LineData && OutPointData);

		bool bIsClosedSpline = false;

		if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(LineData))
		{
			const FPCGSplineStruct* Spline = &SplineData->SplineStruct;

			if (Spline && Spline->IsClosedLoop())
			{
				bIsClosedSpline = true;
			}
		}

		const int32 Seed = Context ? Context->GetSeed() : PCGValueConstants::DefaultSeed;

		FSubdivisionStepSampler SubdivisionSampler(LineData, Params, Seed);
		FDistanceStepSampler DistanceSampler(LineData, Params, Seed);

		FStepSampler* Sampler = ((Params.Mode == EPCGSplineSamplingMode::Subdivision) ? static_cast<FStepSampler*>(&SubdivisionSampler) : static_cast<FStepSampler*>(&DistanceSampler));


		FDimensionSampler TrivialDimensionSampler(LineData, InBoundingShapeData, InProjectionTarget, InProjectionParams, Params, Seed, OutPointData);
		FVolumeSampler VolumeSampler(LineData, InBoundingShapeData, InProjectionTarget, InProjectionParams, Params, Seed, OutPointData);


		FDimensionSampler* ExtentsSampler = ((Params.Dimension == EPCGSplineSamplingDimension::OnSpline) ? &TrivialDimensionSampler : static_cast<FDimensionSampler*>(&VolumeSampler));

		bool bHasPreviousPoint = false;
		FSamplerResult Results[2];
		FSamplerResult* PreviousResult = &Results[0];
		FSamplerResult* CurrentResult = &Results[1];

		if (!Sampler->IsDone())
		{
			Sampler->Step(*PreviousResult);
			bHasPreviousPoint = true;
		}

		const FSamplerResult FirstResult = *PreviousResult;
		
		while(!Sampler->IsDone())
		{
			// Sample point on spline proper
			Sampler->Step(*CurrentResult);

			// Get unsigned angle difference between the two points
			const FVector::FReal DeltaSinAngle = ((CurrentResult->LocalTransform.GetUnitAxis(EAxis::X) ^ PreviousResult->LocalTransform.GetUnitAxis(EAxis::X)) | PreviousResult->LocalTransform.GetUnitAxis(EAxis::Z));
			// Normalize value to be between -1 and 1
			FVector::FReal DeltaAngle = FMath::Asin(DeltaSinAngle) / UE_HALF_PI;

			PreviousResult->NextDeltaAngle = DeltaAngle;
			CurrentResult->PreviousDeltaAngle = -DeltaAngle;
			// Perform samples "around" the spline depending on the settings
			ExtentsSampler->Sample(*PreviousResult, OutPointData);

			// Prepare for next iteration
			Swap(PreviousResult, CurrentResult);
			*CurrentResult = FSamplerResult();
		}
		
		if (bHasPreviousPoint)
		{
			// Get the DeltaAngle between last and first points if we are operating on a closed spline
			if (bIsClosedSpline)
			{
				// Get unsigned angle difference between the two points
				const FVector::FReal DeltaSinAngle = ((FirstResult.LocalTransform.GetUnitAxis(EAxis::X) ^ PreviousResult->LocalTransform.GetUnitAxis(EAxis::X)) | PreviousResult->LocalTransform.GetUnitAxis(EAxis::Z));
				// Normalize value to be between -1 and 1
				PreviousResult->NextDeltaAngle = FMath::Asin(DeltaSinAngle) / UE_HALF_PI;
			}

			ExtentsSampler->Sample(*PreviousResult, OutPointData);
		}
	}
 
	void SampleInteriorData(FPCGContext* Context, const UPCGPolyLineData* LineData, const UPCGSpatialData* InBoundingShape, const UPCGSpatialData* InProjectionTarget, const FPCGProjectionParams& InProjectionParams, const FPCGSplineSamplerParams& Params, UPCGBasePointData* OutPointData)
	{
		using namespace UE::Geometry;

		check(LineData && OutPointData);

		const FPCGSplineStruct* Spline = nullptr;

		// Add more information in order to detect which spline it might be
		auto EmitLog = [Context, LineData](const FText& ErrorMessage, bool bError = true)
		{
			const UPCGMetadata* Metadata = LineData ? LineData->ConstMetadata() : nullptr;
			const FPCGMetadataDomain* MetadataDomain = Metadata ? Metadata->GetConstMetadataDomain(PCGMetadataDomainID::Data) : nullptr;
			const FPCGMetadataAttribute<FSoftObjectPath>* ActorRefefrenceAttribute = MetadataDomain ? MetadataDomain->GetConstTypedAttribute<FSoftObjectPath>(PCGAttributeNameConstants::ActorReferenceAttribute) : nullptr;
			const FText ActorReferenceHint = ActorRefefrenceAttribute ? FText::Format(LOCTEXT("ActorReferenceHint", "\nPossible actor source: {0}"), FText::FromString(ActorRefefrenceAttribute->GetValue(PCGFirstEntryKey).ToString())) : FText{};

			const FText SplineTransformHint = LineData ? FText::Format(LOCTEXT("TransfromHint", "\nSpline world position: {0}"), FText::FromString(LineData->GetTransform().GetLocation().ToString())) : FText{};
			const FText FinalMessage = FText::Format(INVTEXT("{0}{1}{2}"), ErrorMessage, ActorReferenceHint, SplineTransformHint);

			if (bError)
			{
				PCGLog::LogErrorOnGraph(FinalMessage, Context);
			}
			else
			{
				PCGLog::LogWarningOnGraph(FinalMessage, Context);
			}
		};

		if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(LineData))
		{
			Spline = &SplineData->SplineStruct;
		}
		else if (const UPCGLandscapeSplineData* LandscapeSplineData = Cast<UPCGLandscapeSplineData>(LineData))
		{
			EmitLog(LOCTEXT("LandscapeSplinesNotSupported", "Input data of type Landscape Spline are not supported for interior sampling"));
			return;
		}
		else
		{
			EmitLog(LOCTEXT("CouldNotCreateSplineData", "Could not create UPCGSplineData from LineData"));
			return;
		}

		check(Spline);

		if (!Spline->IsClosedLoop())
		{
			EmitLog(LOCTEXT("ShapeNotClosed", "Interior sampling only generates for closed shapes, enable the 'Closed Loop' setting on the spline"));
			return;
		}

		const FBoxSphereBounds SplineLocalBounds = Spline->LocalBounds;
		FVector MinPoint = SplineLocalBounds.Origin - SplineLocalBounds.BoxExtent;
		FVector MaxPoint = SplineLocalBounds.Origin + SplineLocalBounds.BoxExtent;

		const FVector::FReal MaxDimension = FMath::Max(Spline->Bounds.BoxExtent.X, Spline->Bounds.BoxExtent.Y) * 2.f;
		const FVector::FReal MaxDimensionSquared = MaxDimension * MaxDimension;

		const int NumSegments = LineData->GetNumSegments();

		const FRichCurve* DensityFalloffCurve = Params.InteriorDensityFalloffCurve.GetRichCurveConst();
		const bool bComputeDensityFalloff = DensityFalloffCurve != nullptr && DensityFalloffCurve->GetNumKeys() > 0 && NumSegments > 1;
		const bool bFindNearestSplineKey = Params.InteriorOrientation == EPCGSplineSamplingInteriorOrientation::FollowCurvature || (bComputeDensityFalloff && !Params.bTreatSplineAsPolyline);
		const bool bProjectOntoSurface = Params.bProjectOntoSurface || bFindNearestSplineKey;

		const FVector::FReal BoundExtents = Params.InteriorBorderSampleSpacing * 0.5f;
		const FVector BoundsMin = FVector::One() * -BoundExtents;
		const FVector BoundsMax = FVector::One() * BoundExtents;

		const FVector::FReal SplineLength = Spline->GetSplineLength();

		TArray<FVector> SplineSamplePoints;

		if (Params.bTreatSplineAsPolyline)
		{
			// Treat spline interface points as vertices of a polyline
			SplineSamplePoints.Reserve(NumSegments);

			for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
			{
				SplineSamplePoints.Add(LineData->GetLocationAtDistance(SegmentIndex, 0, /*bWorldSpace=*/false));
			}
		}
		else if(Params.InteriorBorderSampleSpacing > 0)
		{
			SplineSamplePoints.Reserve(1 + SplineLength / Params.InteriorBorderSampleSpacing);

			// Get sample points along the spline that are higher resolution than our PolyLine
			for (FVector::FReal Length = 0.f; Length < SplineLength; Length += Params.InteriorBorderSampleSpacing)
			{
				SplineSamplePoints.Add(Spline->GetLocationAtDistanceAlongSpline(Length, ESplineCoordinateSpace::Local));
			}
		}
		else
		{
			return;
		}

		// Flat polygon representation of our spline points
		TArray<FVector2D> SplineSamplePoints2D;
		SplineSamplePoints2D.Reserve(SplineSamplePoints.Num());

		for (const FVector& Point : SplineSamplePoints)
		{
			SplineSamplePoints2D.Add(FVector2D(Point));

			MinPoint = FVector::Min(MinPoint, Point);
			MaxPoint = FVector::Max(MaxPoint, Point);
		}

		FPolygon2d PolygonFromSpline(MoveTemp(SplineSamplePoints2D));
		if (PolygonFromSpline.IsClockwise())
		{
			PolygonFromSpline.Reverse();
		}

		TArray<TTuple<FVector2D, FVector2D>> MedialAxisEdges;
		FVector2D Centroid;

		if (bComputeDensityFalloff)
		{
			// Compute the Medial Axis as a subset of the Voronoi Diagram of the PolyLine points. Only works for >= 4 points.
			if (NumSegments >= 4)
			{
				TArray<FVector> PolygonPoints; // Top-down 2D projection polygon of the spline points
				const FPolygon2d* PolygonPoints2DPtr = nullptr;
				FPolygon2d PolygonPoint2D;

				if (Params.bTreatSplineAsPolyline)
				{
					// If we already computed the polygon, use a copy instead of generating it again
					PolygonPoints.Reserve(SplineSamplePoints.Num());
					for (const FVector& SplinePoint : SplineSamplePoints)
					{
						FVector& PolygonPoint = PolygonPoints.Add_GetRef(SplinePoint);
						PolygonPoint.Z = MinPoint.Z;
					}

					PolygonPoints2DPtr = &PolygonFromSpline;
				}
				else
				{
					TArray<FVector2D> TempPolygonPoints2D;
					PolygonPoints.Reserve(NumSegments);
					TempPolygonPoints2D.Reserve(NumSegments);

					for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
					{
						FVector& PolygonPoint = PolygonPoints.Add_GetRef(LineData->GetLocationAtDistance(SegmentIndex, 0, /*bWorldSpace=*/false));
						PolygonPoint.Z = MinPoint.Z;
						TempPolygonPoints2D.Add(FVector2D(PolygonPoint));
					}

					PolygonPoint2D = FPolygon2d{MoveTemp(TempPolygonPoints2D)};
					if (PolygonPoint2D.IsClockwise())
					{
						PolygonPoint2D.Reverse();
					}

					PolygonPoints2DPtr = &PolygonPoint2D;
				}

				check(PolygonPoints2DPtr)

				TArray<TTuple<FVector, FVector>> VoronoiEdges;
				TArray<int32> CellMember;

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSplineSamplerElement::Execute::GetVoronoiEdges);
					GetVoronoiEdges(PolygonPoints, FBox(MinPoint, FVector(MaxPoint.X, MaxPoint.Y, MinPoint.Z)), VoronoiEdges, CellMember);
				}

				// Find the subset of the Voronoi Diagram which composes the Medial Axis
				for (const TTuple<FVector, FVector>& Edge : VoronoiEdges)
				{
					// Discard any edges which intersect the polygon
					bool bDiscard = false;
					for (const FSegment2d& Segment : PolygonPoints2DPtr->Segments())
					{
						const FSegment2d Edge2D{FVector2D(Edge.Get<0>()), FVector2D(Edge.Get<1>())};

						if (Segment.Intersects(Edge2D))
						{
							bDiscard = true;
							break;
						}
					}

					if (bDiscard)
					{
						continue;
					}

					// If either of the points lies within the polygon, the segment must lie within the polygon
					if (PCGSplineSamplerHelpers::PointInsidePolygon2D(*PolygonPoints2DPtr, FVector2D(Edge.Get<0>())))
					{
						MedialAxisEdges.Add(Edge);
					}
				}
			}
			else if (NumSegments == 3) // If we only have 3 points, use the centroid instead of the medial axis.
			{
				const FVector2D ControlPoint1 = FVector2D(LineData->GetLocationAtDistance(/*SegmentIndex=*/0, /*Distance=*/0.0f, /*bWorldSpace=*/false));
				const FVector2D ControlPoint2 = FVector2D(LineData->GetLocationAtDistance(/*SegmentIndex=*/1, /*Distance=*/0.0f, /*bWorldSpace=*/false));
				const FVector2D ControlPoint3 = FVector2D(LineData->GetLocationAtDistance(/*SegmentIndex=*/2, /*Distance=*/0.0f, /*bWorldSpace=*/false));
				Centroid = (ControlPoint1 + ControlPoint2 + ControlPoint3) / 3.0f;
			}
			else if (NumSegments == 2) // If we only have 2 points, compute a third point, and use the centroid instead of the medial axis.
			{
				// Note: The midpoint of the first segment may not be collinear if the control point has meaningful tangents.
				// Note: The centroid of these three points may not always lie inside the spline. This can happen if the spline is self-intersecting.
				// TODO: It may be worth investigating super-sampling the spline to get >= 4 points and then compute the medial axis instead.
				const FVector2D ControlPoint1 = FVector2D(LineData->GetLocationAtDistance(/*SegmentIndex=*/0, /*Distance=*/0.0f, /*bWorldSpace=*/false));
				const FVector2D ControlPoint2 = FVector2D(LineData->GetLocationAtDistance(/*SegmentIndex=*/0, /*Distance=*/0.5f, /*bWorldSpace=*/false));
				const FVector2D ControlPoint3 = FVector2D(LineData->GetLocationAtDistance(/*SegmentIndex=*/1, /*Distance=*/0.0f, /*bWorldSpace=*/false));
				Centroid = (ControlPoint1 + ControlPoint2 + ControlPoint3) / 3.0f;
			}
		}

		// RayPadding should be large enough to avoid potential misclassifications in SegmentPolygonIntersection2D
		const FVector::FReal RayPadding = 100.f;
		const FVector::FReal MinY = FMath::CeilToDouble(MinPoint.Y / Params.InteriorSampleSpacing) * Params.InteriorSampleSpacing;
		const FVector::FReal MaxY = FMath::FloorToDouble(MaxPoint.Y / Params.InteriorSampleSpacing) * Params.InteriorSampleSpacing;

		constexpr int32 MinIterationPerDispatch = 4;
		const int32 NumIterations = (MaxY + UE_KINDA_SMALL_NUMBER - MinY) / Params.InteriorSampleSpacing;
		int32 NumDispatch = Context ? (NumIterations / MinIterationPerDispatch) : 1;
		if (Context && Context->AsyncState.NumAvailableTasks > 0)
		{
			NumDispatch = FMath::Min(Context->AsyncState.NumAvailableTasks, NumDispatch);
		}
		NumDispatch = FMath::Max(1, NumDispatch);

		const int32 NumIterationsPerDispatch = NumIterations / NumDispatch;
		const FBox GeneratedPointBounds = FBox(-FVector::OneVector * Params.InteriorSampleSpacing / 2.0f, FVector::OneVector * Params.InteriorSampleSpacing / 2.0f);

		TArray<TArray<TTuple<FTransform, FVector, float>>> InteriorSplinePointData;
		InteriorSplinePointData.SetNum(NumDispatch);

		const FTransform LineDataTransform = LineData->GetTransform();
		bool bAnyRowFailedToSample = false;

		ParallelFor(NumDispatch, [&](int32 DispatchIndex)
		{
			LLM_SCOPE_BYTAG(PCG);
			const bool bIsLastIteration = (DispatchIndex == (NumDispatch - 1));
			const int32 StartIterationIndex = DispatchIndex * NumIterationsPerDispatch;
			const int32 EndIterationIndex = bIsLastIteration ? NumIterations : (StartIterationIndex + NumIterationsPerDispatch);

			const FVector::FReal LocalMinY = MinY + StartIterationIndex * Params.InteriorSampleSpacing;
			const FVector::FReal LocalMaxY = bIsLastIteration ? (MaxY + UE_KINDA_SMALL_NUMBER) : (MinY + EndIterationIndex * Params.InteriorSampleSpacing);

			// Point sampling
			for (FVector::FReal Y = LocalMinY; Y < LocalMaxY; Y += Params.InteriorSampleSpacing)
			{
				const FVector2D RayMin(MinPoint.X - RayPadding, Y);
				const FVector2D RayMax(MaxPoint.X + RayPadding, Y);

				TArray<FVector2D> Intersections = SegmentPolygonIntersection2D(RayMin, RayMax, PolygonFromSpline);

				if (Intersections.Num() % 2 != 0)
				{
					bAnyRowFailedToSample = true;
					continue;
				}

				Intersections.Sort([](const FVector2D& LHS, const FVector2D& RHS) { return LHS.X < RHS.X; });

				// TODO: async processing
				// Each pair of intersections defines a range in which point samples may lie
				for (int32 RangeIndex = 0; RangeIndex < Intersections.Num(); RangeIndex += 2)
				{
					const FVector::FReal MinX = FMath::CeilToDouble(Intersections[RangeIndex].X / Params.InteriorSampleSpacing) * Params.InteriorSampleSpacing;
					const FVector::FReal MaxX = FMath::FloorToDouble(Intersections[RangeIndex + 1].X / Params.InteriorSampleSpacing) * Params.InteriorSampleSpacing;

					if (MaxX - MinX < Params.InteriorSampleSpacing)
					{
						continue;
					}

					InteriorSplinePointData[DispatchIndex].Reserve(2 + FMath::Max(MaxX - MinX, 0) / Params.InteriorSampleSpacing);

					for (FVector::FReal X = MinX; X < MaxX + UE_KINDA_SMALL_NUMBER; X += Params.InteriorSampleSpacing)
					{
						const FVector2D SampleLocation = FVector2D(X, Y);
						FVector SurfaceLocation = FVector(SampleLocation, MinPoint.Z);

						if (bProjectOntoSurface)
						{
							SurfaceLocation.Z = ProjectOntoSplineInteriorSurface(SplineSamplePoints, SurfaceLocation);
						}

						// if bTreatAsPolyline, then we shouldnt use this, we should use nearest point on the polygon line segments
						float Dummy;
						const float NearestSplineKey = bFindNearestSplineKey ? Spline->GetSplinePointsPosition().InaccurateFindNearest(SurfaceLocation, Dummy) : 0.f;

						const FVector PointLocationLS = (Params.bProjectOntoSurface ? FVector(SurfaceLocation) : FVector(SampleLocation, MinPoint.Z));

						FTransform TransformLS = FTransform::Identity;
						TransformLS.SetLocation(PointLocationLS);

						if (Params.InteriorOrientation == EPCGSplineSamplingInteriorOrientation::FollowCurvature)
						{
							TransformLS.SetRotation(Spline->GetQuaternionAtSplineInputKey(NearestSplineKey, ESplineCoordinateSpace::Local));
						}

						// Calculate density fall off
						float Density = 1.0f;

						if (bComputeDensityFalloff)
						{
							FVector::FReal SmallestDistSquared = MaxDimensionSquared;

							if (MedialAxisEdges.Num() > 0)
							{
								// Find distance from SampleLocation to MedialAxis
								for (const TTuple<FVector2D, FVector2D>& Edge : MedialAxisEdges)
								{
									const FVector::FReal DistSquared = FMath::PointDistToSegmentSquared(FVector(SampleLocation, 0), FVector(Edge.Get<0>(), 0), FVector(Edge.Get<1>(), 0));

									if (DistSquared < SmallestDistSquared)
									{
										SmallestDistSquared = DistSquared;
									}
								}
							}
							else if (NumSegments == 2 || NumSegments == 3) // If a centroid was computed instead of the medial axis, fallback to that.
							{
								SmallestDistSquared = FVector2D::DistSquared(SampleLocation, Centroid);
							}

							const FVector::FReal SmallestDist = FMath::Sqrt(SmallestDistSquared);

							FVector::FReal PointToSplineDist = 0.f;

							if (Params.bTreatSplineAsPolyline)
							{
								SmallestDistSquared = MaxDimensionSquared;
								for (int32 PointIndex = 0; PointIndex < SplineSamplePoints.Num(); ++PointIndex)
								{
									const FVector::FReal DistSquared = FMath::PointDistToSegmentSquared(SurfaceLocation, SplineSamplePoints[PointIndex], SplineSamplePoints[(PointIndex + 1) % SplineSamplePoints.Num()]);

									if (DistSquared < SmallestDistSquared)
									{
										SmallestDistSquared = DistSquared;
									}
								}

								PointToSplineDist = FMath::Sqrt(SmallestDistSquared);
							}
							else
							{
								const FVector NearestSplineLocation = Spline->GetLocationAtSplineInputKey(NearestSplineKey, ESplineCoordinateSpace::Local);
								PointToSplineDist = FVector2D::Distance(SampleLocation, FVector2D(NearestSplineLocation.X, NearestSplineLocation.Y));
							}

							// Linear fall off in the range [0, 1]
							const FVector::FReal T = SmallestDist / (SmallestDist + PointToSplineDist + UE_KINDA_SMALL_NUMBER);
							Density = DensityFalloffCurve->Eval(T);
						}

						FTransform TransformWS = TransformLS * LineDataTransform;
						if (InProjectionTarget)
						{
							FPCGPoint ProjectedPoint;
							if (!InProjectionTarget->ProjectPoint(TransformWS, FBox(BoundsMin, BoundsMax), InProjectionParams, ProjectedPoint, OutPointData->Metadata))
							{
								continue;
							}

							if (InProjectionParams.bProjectPositions)
							{
								TransformWS.SetLocation(ProjectedPoint.Transform.GetLocation());
							}

							if (InProjectionParams.bProjectRotations)
							{
								TransformWS.SetRotation(ProjectedPoint.Transform.GetRotation());
							}

							if (InProjectionParams.bProjectScales)
							{
								TransformWS.SetScale3D(ProjectedPoint.Transform.GetScale3D());
							}
						}

						// Prune points outside of bounds
						FPCGPoint BoundsTestPoint;
						if (InBoundingShape && !InBoundingShape->SamplePoint(TransformWS, GeneratedPointBounds, BoundsTestPoint, nullptr))
						{
							continue;
						}
						
						InteriorSplinePointData[DispatchIndex].Emplace(TransformWS, PointLocationLS, Density);
					}
				}
			}
		});

		if (bAnyRowFailedToSample)
		{
			EmitLog(LOCTEXT("IntersectionTestFailed", "One or more rows of the spline interior failed to sample (intersection test failed). Ensure the spline points are not overlapping."), /*bError=*/false);
		}

		// Finally, gather the data and push to the points
		int32 PointCount = 0;
		for (const TArray<TTuple<FTransform, FVector, float>>& InteriorData : InteriorSplinePointData)
		{
			PointCount += InteriorData.Num();
		}

		const int32 Seed = Context ? Context->GetSeed() : PCGValueConstants::DefaultSeed;
		int32 SampleIndex = 0;

		// TODO: should we parallel for this too?
		OutPointData->SetNumPoints(PointCount);
		OutPointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Density);

		// Set Constant values
		OutPointData->SetBoundsMin(BoundsMin);
		OutPointData->SetBoundsMax(BoundsMax);
		OutPointData->SetSteepness(Params.PointSteepness);
				
		FPCGPointValueRanges OutRanges(OutPointData, /*bAllocate=*/false);

		int32 NumWritten = 0;

		for (const TArray<TTuple<FTransform, FVector, float>>& InteriorData : InteriorSplinePointData)
		{
			for (const TTuple<FTransform, FVector, float>& InteriorPoint : InteriorData)
			{
				OutRanges.TransformRange[NumWritten] = InteriorPoint.Get<0>();
				SetSeed(OutRanges.SeedRange[NumWritten], OutRanges.TransformRange[NumWritten], InteriorPoint.Get<1>(), Params, Seed, ++SampleIndex);
				OutRanges.DensityRange[NumWritten] = InteriorPoint.Get<2>();
				++NumWritten;
			}
		}
		check(NumWritten == PointCount);
	}

	const UPCGPolyLineData* GetPolyLineData(const UPCGSpatialData* InSpatialData)
	{
		if (!InSpatialData)
		{
			return nullptr;
		}

		if (const UPCGPolyLineData* LineData = Cast<const UPCGPolyLineData>(InSpatialData))
		{
			return LineData;
		}
		else if (const UPCGSplineProjectionData* SplineProjectionData = Cast<const UPCGSplineProjectionData>(InSpatialData))
		{
			return SplineProjectionData->GetSpline();
		}
		else if (const UPCGIntersectionData* Intersection = Cast<const UPCGIntersectionData>(InSpatialData))
		{
			if (const UPCGPolyLineData* IntersectionA = GetPolyLineData(Intersection->A))
			{
				return IntersectionA;
			}
			else if (const UPCGPolyLineData* IntersectionB = GetPolyLineData(Intersection->B))
			{
				return IntersectionB;
			}
		}

		return nullptr;
	}
}

UPCGSplineSamplerSettings::UPCGSplineSamplerSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		SamplerParams.PointSteepness = 1.0f;
	}
}

#if WITH_EDITOR
FText UPCGSplineSamplerSettings::GetNodeTooltipText() const
{
	return LOCTEXT("SplineSamplerNodeTooltip", "Generates points along the given Spline, and within the Bounding Shape if provided.");
}
#endif

TArray<FPCGPinProperties> UPCGSplineSamplerSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& SplinePinProperty = PinProperties.Emplace_GetRef(PCGSplineSamplerConstants::SplineLabel, EPCGDataType::PolyLine, /*bAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);
	SplinePinProperty.SetRequiredPin();

	// Only one connection/data allowed. To avoid ambiguity, samplers should require users to union or intersect multiple shapes.
	PinProperties.Emplace(PCGSplineSamplerConstants::BoundingShapeLabel, EPCGDataType::Spatial, /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false, LOCTEXT("SplineSamplerBoundingShapePinTooltip",
		"Optional. All sampled points must be contained within this shape."
	));

	return PinProperties;
}

FPCGElementPtr UPCGSplineSamplerSettings::CreateElement() const
{
	return MakeShared<FPCGSplineSamplerElement>();
}

void FPCGSplineSamplerElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	if (const UPCGSplineSamplerSettings* Settings = Cast<UPCGSplineSamplerSettings>(InParams.Settings))
	{
		bool bUnbounded;
		PCGSettingsHelpers::GetOverrideValue(*InParams.InputData, Settings, GET_MEMBER_NAME_CHECKED(FPCGSplineSamplerParams, bUnbounded), Settings->SamplerParams.bUnbounded, bUnbounded);
		const bool bBoundsConnected = InParams.InputData->GetInputsByPin(PCGSplineSamplerConstants::BoundingShapeLabel).Num() > 0;

		// If we're operating in bounded mode then we'll use actor bounds, and therefore take dependency on actor data.
		if (!bUnbounded && !bBoundsConnected && InParams.ExecutionSource)
		{
			if (const UPCGData* Data = InParams.ExecutionSource->GetExecutionState().GetSelfData())
			{
				Crc.Combine(Data->GetOrComputeCrc(/*bFullDataCrc=*/false));
			}
		}
	}

	OutCrc = Crc;
}

bool FPCGSplineSamplerElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSplineSamplerElement::Execute);

	const UPCGSplineSamplerSettings* Settings = Context->GetInputSettings<UPCGSplineSamplerSettings>();
	check(Settings);

	TArray<FPCGTaggedData> SplineInputs = Context->InputData.GetInputsByPin(PCGSplineSamplerConstants::SplineLabel);
	TArray<FPCGTaggedData> BoundingShapeInputs = Context->InputData.GetInputsByPin(PCGSplineSamplerConstants::BoundingShapeLabel);

	const FPCGSplineSamplerParams& SamplerParams = Settings->SamplerParams;

	const UPCGSpatialData* BoundingShape = nullptr;

	// If unbounded, specific samplers will not be constrained by a nullptr BoundingShape
	if(!SamplerParams.bUnbounded)
	{
		// TODO: Once we support time-slicing, put this in the context and root (see FPCGSurfaceSamplerContext)
		bool bUnionCreated = false;
		// Grab the Bounding Shape input if there is one.
		BoundingShape = Context->InputData.GetSpatialUnionOfInputsByPin(Context, PCGSplineSamplerConstants::BoundingShapeLabel, bUnionCreated);

		// Fallback to getting bounds from actor
		if (!BoundingShape && Context->ExecutionSource.IsValid())
		{
			BoundingShape = Cast<UPCGSpatialData>(Context->ExecutionSource->GetExecutionState().GetSelfData());
		}
	}
	else if (BoundingShapeInputs.Num() > 0)
	{
		PCGE_LOG(Verbose, LogOnly, LOCTEXT("BoundsIgnored", "The bounds of the Bounding Shape input pin will be ignored because the Unbounded option is enabled."));
	}

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (FPCGTaggedData& Input : SplineInputs)
	{
		const UPCGSpatialData* SpatialData = Cast<const UPCGSpatialData>(Input.Data);
		if (!SpatialData)
		{
			continue;
		}

		// TODO: do something for point data approximations
		const UPCGPolyLineData* LineData = PCGSplineSamplerHelpers::GetPolyLineData(SpatialData);
		if (!LineData)
		{
			continue;
		}

		const UPCGSpatialData* ProjectionTarget = nullptr;
		FPCGProjectionParams ProjectionParams;
		if (const UPCGSplineProjectionData* SplineProjection = Cast<const UPCGSplineProjectionData>(Input.Data))
		{
			ProjectionTarget = SplineProjection->GetSurface();
			ProjectionParams = SplineProjection->GetProjectionParams();
		}

		FPCGTaggedData& Output = Outputs.Emplace_GetRef(Input);

		UPCGBasePointData* SampledPointData = FPCGContext::NewPointData_AnyThread(Context);

		FPCGInitializeFromDataParams InitializeFromDataParams(SpatialData);
		InitializeFromDataParams.bInheritSpatialData = false;

		SampledPointData->InitializeFromDataWithParams(InitializeFromDataParams);
		Output.Data = SampledPointData;

		if (SamplerParams.Dimension == EPCGSplineSamplingDimension::OnInterior)
		{
			PCGSplineSamplerHelpers::SampleInteriorData(Context, LineData, BoundingShape, ProjectionTarget, ProjectionParams, SamplerParams, SampledPointData);
		}
		else
		{
			PCGSplineSamplerHelpers::SampleLineData(Context, LineData, BoundingShape, ProjectionTarget, ProjectionParams, SamplerParams, SampledPointData);
		}
	}

	return true;
}

#if WITH_EDITOR
void UPCGSplineSamplerSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	check(InOutNode);
	
	if (DataVersion < FPCGCustomVersion::SplineSamplerBoundedByDefault)
	{
		UE_LOGF(LogPCG, Log, "Spline Sampler node migrated from an older version. Defaulting to 'Unbounded' to match previous behavior.");
		SamplerParams.bUnbounded = true;
	}
	
	Super::ApplyDeprecation(InOutNode);
}

void UPCGSplineSamplerSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	if (DataVersion < FPCGCustomVersion::SplineSamplerUpdatedNodeInputs && ensure(InOutNode))
	{
		if (InputPins.Num() > 0 && InputPins[0])
		{
			// First pin renamed in this version. Rename here so that edges won't get culled in UpdatePins later.
			InputPins[0]->Properties.Label = PCGSplineSamplerConstants::SplineLabel;
		}
	}

	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
