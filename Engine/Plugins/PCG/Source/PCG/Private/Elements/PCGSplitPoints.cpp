// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSplitPoints.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"
#include "Metadata/PCGMetadataAccessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplitPoints)

#define LOCTEXT_NAMESPACE "PCGSplitPointsElement"

namespace PCGSplitPointsConstants
{
	const FName OutputALabel = TEXT("Before Split");
	const FName OutputBLabel = TEXT("After Split");
}

TArray<FPCGPinProperties> UPCGSplitPointsSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGSplitPointsConstants::OutputALabel, 
		EPCGDataType::Point, 
		/*bAllowMultipleConnections=*/true, 
		/*bAllowMultipleData=*/true, 
		LOCTEXT("PinATooltip", "The portion of each point before the split plane."));

	PinProperties.Emplace(PCGSplitPointsConstants::OutputBLabel, 
		EPCGDataType::Point, 
		/*bAllowMultipleConnections=*/true, 
		/*bAllowMultipleData=*/true, 
		LOCTEXT("PinBTooltip", "The portion of each point after the split plane."));

	return PinProperties;
}

FPCGElementPtr UPCGSplitPointsSettings::CreateElement() const
{
	return MakeShared<FPCGSplitPointsElement>();
}

bool FPCGSplitPointsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSplitPointsElement::Execute);
	check(Context);

	const UPCGSplitPointsSettings* Settings = Context->GetInputSettings<UPCGSplitPointsSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const float SplitPosition = FMath::Clamp(Settings->SplitPosition, 0.0f, 1.0f);

	for (int i = 0; i < Inputs.Num(); ++i)
	{
		const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(Inputs[i].Data);
		if (!InputPointData)
		{
			PCGE_LOG(Verbose, GraphAndLog, FText::Format(LOCTEXT("InvalidPointData", "Input {0} is not point data"), i));
			continue;
		}

		FPCGTaggedData& OutputA = Outputs.Add_GetRef(Inputs[i]);
		UPCGBasePointData* OutPointDataA = FPCGContext::NewPointData_AnyThread(Context);

		OutPointDataA->InitializeFromData(InputPointData);

		OutPointDataA->SetNumPoints(InputPointData->GetNumPoints());

		if (!OutPointDataA->HasSpatialDataParent())
		{
			OutPointDataA->AllocateProperties(InputPointData->GetAllocatedProperties());
			OutPointDataA->CopyUnallocatedPropertiesFrom(InputPointData);
		}

		OutPointDataA->AllocateProperties(EPCGPointNativeProperties::BoundsMax);

		OutputA.Data = OutPointDataA;
		OutputA.Pin = PCGSplitPointsConstants::OutputALabel;

		FPCGTaggedData& OutputB = Outputs.Add_GetRef(Inputs[i]);
		UPCGBasePointData* OutPointDataB = FPCGContext::NewPointData_AnyThread(Context);

		OutPointDataB->InitializeFromData(InputPointData);

		OutPointDataB->SetNumPoints(InputPointData->GetNumPoints());

		if (!OutPointDataB->HasSpatialDataParent())
		{
			OutPointDataB->AllocateProperties(InputPointData->GetAllocatedProperties());
			OutPointDataB->CopyUnallocatedPropertiesFrom(InputPointData);
		}

		OutPointDataB->AllocateProperties(EPCGPointNativeProperties::BoundsMin);

		OutputB.Data = OutPointDataB;
		OutputB.Pin = PCGSplitPointsConstants::OutputBLabel;

		auto ProcessPoint = [Settings, SplitPosition, InputPointData, OutPointDataA, OutPointDataB](const int32 StartReadIndex, const int32 StartWriteIndex, const int32 Count)
		{
			const FConstPCGPointValueRanges InRanges(InputPointData);
			FPCGPointValueRanges OutRangesA(OutPointDataA, /*bAllocate=*/false);
			FPCGPointValueRanges OutRangesB(OutPointDataB, /*bAllocate=*/false);

			int32 NumWritten = 0;

			for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
			{
				const int32 WriteIndex = StartWriteIndex + NumWritten;
				const FVector SplitterPosition = (InRanges.BoundsMaxRange[ReadIndex] - InRanges.BoundsMinRange[ReadIndex]) * SplitPosition;
				const FVector MinPlusSplit = InRanges.BoundsMinRange[ReadIndex] + SplitterPosition;

				FVector SplitValues = FVector::ZeroVector;
				const int AxisIndex = static_cast<int>(Settings->SplitAxis);
				if (ensure(AxisIndex >= 0 && AxisIndex <= 2))
				{
					SplitValues[AxisIndex] = 1.0;
				}

				if (!OutPointDataA->HasSpatialDataParent())
				{
					OutRangesA.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);
				}

				// Execution for PointsA portion
				OutRangesA.BoundsMaxRange[WriteIndex] = OutRangesA.BoundsMaxRange[WriteIndex] + SplitValues * (MinPlusSplit - OutRangesA.BoundsMaxRange[WriteIndex]);
				
				if(!OutPointDataB->HasSpatialDataParent())
				{
					OutRangesB.SetFromValueRanges(WriteIndex, InRanges, ReadIndex);
				}

				// Execution of the PointsB portion
				OutRangesB.BoundsMinRange[WriteIndex] = OutRangesB.BoundsMinRange[WriteIndex] + SplitValues * (MinPlusSplit - OutRangesB.BoundsMinRange[WriteIndex]);
				
				++NumWritten;
			}

			check(NumWritten == Count);
			return Count;
		};

		FPCGAsync::AsyncProcessingOneToOneRangeEx(&Context->AsyncState, InputPointData->GetNumPoints(), /*Initialize=*/[]() {}, ProcessPoint, /*bEnableTimeSlicing=*/false);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
