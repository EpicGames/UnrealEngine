// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSplitSplines.h"

#include "PCGContext.h"
#include "Data/PCGSplineData.h"
#include "Elements/PCGSplineIntersection.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplitSplines)

#define LOCTEXT_NAMESPACE "PCGSplitSplinesElement"

namespace PCGSplitSplineConstants
{
	const FName SplitInfoLabel = TEXT("SplitInfo");
}

namespace PCGSplitSpline
{
	EStatus SplitSpline(const UPCGSplineData* Spline, UPCGSplineData* OutSplitSpline, TConstArrayView<FSplinePoint> SplinePoints, TConstArrayView<PCGMetadataEntryKey> SplinePointsEntryKeys, double StartKey, double EndKey)
	{
		check(Spline);
		check(OutSplitSpline);

		if (SplinePoints.IsEmpty())
		{
			return EStatus::Skipped;
		}

		if (!Spline->IsClosed() && StartKey > EndKey)
		{
			return EStatus::Skipped;
		}

		const double LoopKeyLength = Spline->IsClosed() ? Spline->GetLoopKeyOffset() : 1.0;
		const double SplineKeyLength = SplinePoints.Last().InputKey + (Spline->IsClosed() ? LoopKeyLength : 0.0);
		constexpr double KeyEpsilon = 1.0e-6;

		// Normalize start and end keys so that they are valid.
		if (!Spline->IsClosed())
		{
			StartKey = FMath::Max(StartKey, 0.0);
			EndKey = FMath::Min(EndKey, SplineKeyLength);

			// Spline is fully clipped, so nothing to do.
			if (StartKey >= EndKey)
			{
				return EStatus::Skipped;
			}
		}
		else
		{
			// Validate that the end key and the start key are in valid positions with respect to each other.
			// Basically, if the end key is smaller than the start key, we'll bump it once and once only.
			// After that, if the difference is negative, then the range was too large (e.g. > spline key length)
			if (EndKey < StartKey)
			{
				EndKey += SplineKeyLength;
			}

			// Validate length of split
			double SplitLength = EndKey - StartKey;

			if ((SplitLength <= -KeyEpsilon) || (SplitLength >= SplineKeyLength + KeyEpsilon))
			{
				return EStatus::Error;
			}

			SplitLength = FMath::Clamp(SplitLength, 0, SplineKeyLength);

			// 1. Get into [0, spline key length[ range.
			StartKey -= SplineKeyLength * FMath::Floor(StartKey / SplineKeyLength);

			if (StartKey < 0)
			{
				StartKey += SplineKeyLength;
			}

			// 2. Get the end key in the [0, 2 * spline keyLength[ range.
			EndKey = StartKey + SplitLength;
		}

		TArray<FSplinePoint> SplitSplinePoints;
		TArray<int64> SplitSplinePointsEntryKeys;

		auto AddExistingControlPoint = [&SplitSplinePoints, &SplitSplinePointsEntryKeys, &SplinePoints, &SplinePointsEntryKeys](const int ControlPointIndex)
		{
			SplitSplinePoints.Add(SplinePoints[ControlPointIndex]);
			// Add metadata if needed, e.g. if the original data had entry keys.
			if (!SplinePointsEntryKeys.IsEmpty())
			{
				SplitSplinePointsEntryKeys.Add(SplinePointsEntryKeys[ControlPointIndex]);
			}
		};

		auto AddSplitPoint = [&OutSplitSpline, &SplitSplinePoints, &SplitSplinePointsEntryKeys, &Spline, &SplinePointsEntryKeys](const double Key, const double NormalizedKey, TEnumAsByte<ESplinePointType::Type> PointType, const double PreviousKey, const double NextKey)
		{
			check(Key >= PreviousKey && Key <= NextKey && PreviousKey <= NextKey);

			FSplinePoint SplitPoint;
			SplitPoint.InputKey = Key;
			SplitPoint.Position = Spline->SplineStruct.GetLocationAtSplineInputKey(NormalizedKey, ESplineCoordinateSpace::Local);
			SplitPoint.ArriveTangent = Spline->SplineStruct.GetTangentAtSplineInputKey(NormalizedKey, ESplineCoordinateSpace::Local);
			SplitPoint.LeaveTangent = SplitPoint.ArriveTangent;
			SplitPoint.Rotation = Spline->SplineStruct.GetQuaternionAtSplineInputKey(NormalizedKey, ESplineCoordinateSpace::Local).Rotator();
			SplitPoint.Scale = Spline->SplineStruct.GetScaleAtSplineInputKey(NormalizedKey);
			SplitPoint.Type = PointType;

			// The tangent is always computed from the original spline's perspective so we will assume the key range is always one.
			const double Ratio = (NextKey - PreviousKey) > UE_SMALL_NUMBER ? (Key - PreviousKey) / (NextKey - PreviousKey) : 0.5;
			SplitPoint.ArriveTangent *= Ratio;
			SplitPoint.LeaveTangent *= 1.0 - Ratio;

			SplitSplinePoints.Add(SplitPoint);

			// Add metadata if needed, e.g. if the original data had entry keys.
			if (!SplinePointsEntryKeys.IsEmpty())
			{
				PCGMetadataEntryKey& EntryKey = SplitSplinePointsEntryKeys.Add_GetRef(PCGInvalidEntryKey);
				Spline->WriteMetadataToEntry(NormalizedKey, EntryKey, OutSplitSpline->MutableMetadata());
			}
		};

		auto GetPointTypeForTouchedPoint = [](TEnumAsByte<ESplinePointType::Type> FirstType) -> TEnumAsByte<ESplinePointType::Type>
		{
			if (FirstType == ESplinePointType::Curve ||
				FirstType == ESplinePointType::CurveClamped ||
				FirstType == ESplinePointType::CurveCustomTangent)
			{
				return ESplinePointType::CurveCustomTangent;
			}
			else
			{
				return FirstType;
			}
		};

		auto PrepareControlPointsInfo = [&SplinePoints, Spline, SplineKeyLength]()
		{
			// start key, end key, original point index
			TArray<TTuple<double, double, int>> ControlPointsInfo;

			for (int Index = 0; Index < SplinePoints.Num(); ++Index)
			{
				ControlPointsInfo.Emplace(SplinePoints[Index].InputKey, 0.0, Index);
			}

			if (Spline->IsClosed())
			{
				for (int Index = 0; Index < SplinePoints.Num(); ++Index)
				{
					ControlPointsInfo.Emplace(SplinePoints[Index].InputKey + SplineKeyLength, 0.0, Index);
				}
			}

			for(int Index = 0; Index < ControlPointsInfo.Num() - 1; ++Index)
			{
				ControlPointsInfo[Index].Get<1>() = ControlPointsInfo[Index + 1].Get<0>();
			}
			ControlPointsInfo.Last().Get<1>() = (Spline->IsClosed() ? 2.0 : 1.0) * SplineKeyLength;

			return ControlPointsInfo;
		};

		// Prepare control points information so we can so a simple traversal
		// 0: start key, 1: end key, 2: original point index
		TArray<TTuple<double, double, int>> ControlPointsInfo = PrepareControlPointsInfo();

		double CurrentKey = StartKey;
		int CurrentSegmentIndex = 0;
		double StartTangentAdjustmentRatio = -1.0;
		double EndTangentAdjustmentRatio = -1.0;

		// Iterate through control points and interpolate points when needed
		while (CurrentKey <= EndKey && CurrentSegmentIndex < ControlPointsInfo.Num())
		{
			const TTuple<double, double, int>& Segment = ControlPointsInfo[CurrentSegmentIndex];
			++CurrentSegmentIndex;
			EndTangentAdjustmentRatio = -1.0;

			ensure(CurrentKey >= Segment.Get<0>() - KeyEpsilon);
			if (CurrentSegmentIndex < ControlPointsInfo.Num() - 1 && CurrentKey >= Segment.Get<1>() - KeyEpsilon)
			{
				continue;
			}

			const int StartControlPointIndex = Segment.Get<2>();
			const int EndControlPointIndex = (StartControlPointIndex + 1) % SplinePoints.Num();

			// Now the current key can either be:
			// (1) on the start control point
			const double SegmentStartDelta = (CurrentKey - Segment.Get<0>());
			if (FMath::Abs(SegmentStartDelta) <= KeyEpsilon)
			{
				AddExistingControlPoint(StartControlPointIndex);
			}
			// (2) in between both
			else
			{
				const double Ratio = (Segment.Get<1>() - Segment.Get<0>() > 0) ? (CurrentKey - Segment.Get<0>()) / (Segment.Get<1>() - Segment.Get<0>()) : 0.0;

				if (SplitSplinePoints.IsEmpty())
				{
					StartTangentAdjustmentRatio = Ratio;
				}

				TEnumAsByte<ESplinePointType::Type> SplitPointType = GetPointTypeForTouchedPoint(SplinePoints[StartControlPointIndex].Type);
				AddSplitPoint(CurrentKey, FMath::Fmod(CurrentKey, SplineKeyLength), SplitPointType, Segment.Get<0>(), Segment.Get<1>());

				EndTangentAdjustmentRatio = Ratio;
			}

			if (CurrentKey < EndKey)
			{
				if (Segment.Get<1>() <= EndKey)
				{
					CurrentKey = Segment.Get<1>();
				}
				else
				{
					CurrentKey = EndKey;
					// Since we pre-incremented, we need to rollback once when we get to the final point, which would lie in the same segment as the last point.
					--CurrentSegmentIndex;
				}
			}
			else
			{
				break;
			}
		}

		// Adjust tangents if required
		if (SplitSplinePoints.Num() >= 2)
		{
			if(StartTangentAdjustmentRatio >= 0.0)
			{
				FSplinePoint& FirstPreexistingPoint = SplitSplinePoints[1];
				FirstPreexistingPoint.ArriveTangent *= (1.0 - StartTangentAdjustmentRatio);
				FirstPreexistingPoint.Type = GetPointTypeForTouchedPoint(FirstPreexistingPoint.Type);
			}

			if(EndTangentAdjustmentRatio >= 0.0)
			{
				FSplinePoint& LastPreexistingPoint = SplitSplinePoints[SplitSplinePoints.Num() - 2];
				LastPreexistingPoint.LeaveTangent *= EndTangentAdjustmentRatio;
				LastPreexistingPoint.Type = GetPointTypeForTouchedPoint(LastPreexistingPoint.Type);
			}
		}

		// Finally, mark all tangents as custom and reset the keys.
		for (int SPIndex = 0; SPIndex < SplitSplinePoints.Num(); ++SPIndex)
		{
			FSplinePoint& SplinePoint = SplitSplinePoints[SPIndex];
			SplinePoint.InputKey = static_cast<float>(SPIndex);
		}

		if (SplitSplinePoints.Num() > 0)
		{
			SplitSplinePoints[0].Type = GetPointTypeForTouchedPoint(SplitSplinePoints[0].Type);
			SplitSplinePoints.Last().Type = GetPointTypeForTouchedPoint(SplitSplinePoints.Last().Type);
		}

		// Initialize data now
		OutSplitSpline->Initialize(SplitSplinePoints,
			/*bIsClosedLoop*/false,
			Spline->SplineStruct.GetTransform(),
			SplitSplinePointsEntryKeys);

		return EStatus::Split;
	}

	EStatus SplitSpline(const UPCGSplineData* Spline, UPCGSplineData* OutSplitSpline, double StartKey, double EndKey)
	{
		check(Spline);
		check(OutSplitSpline);

		TArray<FSplinePoint> SplinePoints = Spline->GetSplinePoints();
		TConstArrayView<PCGMetadataEntryKey> SplinePointsEntryKeys = Spline->GetConstVerticesEntryKeys();

		return SplitSpline(Spline, OutSplitSpline, SplinePoints, SplinePointsEntryKeys, StartKey, EndKey);
	}
}

UPCGSplitSplinesSettings::UPCGSplitSplinesSettings()
{
	OutputOriginatingSplineIndex = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyOutputSelector>(TEXT("OriginatingSplineIndex"), TEXT("Data"));
}

#if WITH_EDITOR
FName UPCGSplitSplinesSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SplitSpline"));
}

FText UPCGSplitSplinesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Split Spline");
}

EPCGChangeType UPCGSplitSplinesSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSplitSplinesSettings, Mode) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSplitSplinesSettings, bUseConstant))
	{
		// This can change the output pin types, so we need a structural change
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGSplitSplinesSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spline).SetRequiredPin();

	if(!bUseConstant && Mode != EPCGSplitSplineMode::ByPredicateOnControlPoints)
	{
		PinProperties.Emplace_GetRef(PCGSplitSplineConstants::SplitInfoLabel, EPCGDataType::Param).SetRequiredPin();
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSplitSplinesSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spline);

	return PinProperties;
}

FPCGElementPtr UPCGSplitSplinesSettings::CreateElement() const
{
	return MakeShared<FPCGSplitSplineElement>();
}

bool FPCGSplitSplineElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSplitSplineElement::Execute);

	const UPCGSplitSplinesSettings* Settings = Context->GetInputSettings<UPCGSplitSplinesSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData> SplitInfo = Context->InputData.GetInputsByPin(PCGSplitSplineConstants::SplitInfoLabel);
	const EPCGSplitSplineMode Mode = Settings->Mode;

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	// Validate inputs:
	// If we are in the ByKey / ByDistance, we need to validate cardinality here
	if(!Settings->bUseConstant && Mode != EPCGSplitSplineMode::ByPredicateOnControlPoints)
	{
		if (SplitInfo.Num() != 1 && SplitInfo.Num() != Inputs.Num())
		{
			PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, PCGSplitSplineConstants::SplitInfoLabel, Context);
			return true;
		}
	}

	// Goal: Cut the spline at specific key locations where these depend on the input data (fixed key; fixed distance; specified keys, specified distances, boolean predicate on control points)
	for (int InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		const FPCGTaggedData& Input = Inputs[InputIndex];
		const UPCGSplineData* SplineData = Cast<UPCGSplineData>(Input.Data);

		// Input data isn't supported, just discard the original input
		if (!SplineData)
		{
			continue;
		}

		TArray<FSplinePoint> SplinePoints = SplineData->GetSplinePoints();

		if (SplinePoints.Num() < 2)
		{
			Outputs.Add(Input);
			continue;
		}

		TConstArrayView<PCGMetadataEntryKey> SplinePointsEntryKeys = SplineData->GetConstVerticesEntryKeys();
		check(SplinePoints.Num() == SplinePointsEntryKeys.Num() || SplinePointsEntryKeys.IsEmpty());

		// Build keys list
		TArray<double> SplitKeys;
		if (Mode == EPCGSplitSplineMode::ByPredicateOnControlPoints)
		{
			TArray<bool> Predicates;
			if (!PCGAttributeAccessorHelpers::ExtractAllValues(SplineData, Settings->Attribute, Predicates, Context))
			{
				Outputs.Add(Input);
				continue;
			}

			if (Predicates.Num() != SplinePoints.Num())
			{
				FPCGAttributePropertyInputSelector ResolvedAttribute = Settings->Attribute.CopyAndFixLast(SplineData);

				// Mismatch in predicate vs. the expected number of values
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MismatchInPredicateCardinality", "Property {0} is not a proper predicate for splitting the spline."), FText::FromName(ResolvedAttribute.GetAttributeName())), Context);
				Outputs.Add(Input);
				continue;
			}

			// @todo_pcg: Spline implementation assumption, i.e. that control points are on integer keys.
			for (int ControlPointIndex = 0; ControlPointIndex < Predicates.Num(); ++ControlPointIndex)
			{
				if (Predicates[ControlPointIndex])
				{
					SplitKeys.Add(static_cast<double>(ControlPointIndex));
				}
			}
		}
		else
		{
			TArray<double, TInlineAllocator<128>> Values;
			if (Settings->bUseConstant)
			{
				Values.Add(Settings->Constant);
			}
			else
			{
				int SplitInfoIndex = InputIndex % SplitInfo.Num();
				const UPCGData* SplitInfoData = SplitInfo[SplitInfoIndex].Data;

				if (!PCGAttributeAccessorHelpers::ExtractAllValues(SplitInfoData, Settings->Attribute, Values, Context))
				{
					Outputs.Add(Input);
					continue;
				}
			}

			if (Mode == EPCGSplitSplineMode::ByKey)
			{
				SplitKeys = MoveTemp(Values);
			}
			else if (Mode == EPCGSplitSplineMode::ByDistance)
			{
				Algo::Transform(Values, SplitKeys, [SplineData](const double& InValue) { return SplineData->SplineStruct.GetInputKeyAtDistanceAlongSpline(InValue); });
			}
			else if (Mode == EPCGSplitSplineMode::ByAlpha)
			{
				Algo::Transform(Values, SplitKeys, [SplineData](const double& InValue) { return SplineData->GetInputKeyAtAlpha(InValue); });
			}
			else
			{
				check(0); // Invalid mode
			}
		}

		constexpr double KeyEpsilon = 1.0e-6f;
		const bool bIsClosed = SplineData->IsClosed();

		// Remove values that are outside of the spline
		const float FirstKey = SplinePoints.IsEmpty() ? 0.0f : SplinePoints[0].InputKey;
		const float LastKey = SplinePoints.IsEmpty() ? 1.0f : (SplinePoints.Last().InputKey + (bIsClosed ? SplineData->GetLoopKeyOffset() : 0.0f));

		if (bIsClosed)
		{
			// Closed loops can be a bit more flexible on the input keys, but have a specific requirement:
			// the range of keys should not be more than ONE spline length.
			float MinKey = SplitKeys.IsEmpty() ? 0.0f : SplitKeys[0];
			float MaxKey = MinKey;
			for (int SplitIndex = 1; SplitIndex < SplitKeys.Num(); ++SplitIndex)
			{
				MinKey = FMath::Min(MinKey, SplitKeys[SplitIndex]);
				MaxKey = FMath::Max(MaxKey, SplitKeys[SplitIndex]);
			}

			if (MaxKey - MinKey > (LastKey - FirstKey) + KeyEpsilon)
			{
				PCGLog::LogWarningOnGraph(LOCTEXT("InvalidKeyRangeForClosedLoop", "Specified split keys are not valid for a closed spline in this operation."), Context);
				continue;
			}
		}
		else
		{
			for (int SplitIndex = SplitKeys.Num() - 1; SplitIndex >= 0; --SplitIndex)
			{
				// Implementation note: in open splines, there's no point in splitting at the first or last keys
				if (SplitKeys[SplitIndex] <= FirstKey || SplitKeys[SplitIndex] >= LastKey)
				{
					SplitKeys.RemoveAtSwap(SplitIndex);
				}
			}
		}

		// If there are no valid values, then there is nothing to do.
		if (SplitKeys.IsEmpty())
		{
			if (Settings->CutDataBehavior == EPCGSplitSplineDataBehavior::KeepAllSplines)
			{
				Outputs.Add(Input);
			}

			continue;
		}

		// Sort values
		SplitKeys.Sort([](const double& A, const double& B) { return A < B; });

		// Remove duplicates/values too close to each other
		for (int SplitIndex = SplitKeys.Num() - 1; SplitIndex > 0; --SplitIndex)
		{
			if ((SplitKeys[SplitIndex] - SplitKeys[SplitIndex - 1] < KeyEpsilon) ||
				(bIsClosed && SplitIndex == SplitKeys.Num() - 1 && (LastKey - SplitKeys[SplitIndex]) < KeyEpsilon))
			{
				SplitKeys.RemoveAt(SplitIndex);
			}
		}

		// Once we've removed split keys that are outside their normal range, we'll add keys so we can process segments by pairs of segment keys.
		// For closed splines: we'll add a duplicate of the first key to the end of the range to do the full loop.
		// Note that it will mean that the segments will always start from a split point. [split key -> ... -> split key]
		// For open splines: we'll add a starting key (assumedly 0 in most cases) and a finishing key so we have proper segments [0 -> split key] ... [split key -> end]
		if (bIsClosed)
		{
			const double FirstSplitKey = SplitKeys[0];
			SplitKeys.Add(FirstSplitKey + LastKey);
		}
		else if(Settings->CutDataBehavior != EPCGSplitSplineDataBehavior::KeepInteriorSplinesOnly)
		{
			SplitKeys.Insert(SplinePoints[0].InputKey, 0);
			SplitKeys.Add(SplinePoints.Last().InputKey);
		}

		for (int CurrentSplitPair = 0; CurrentSplitPair < SplitKeys.Num() - 1; ++CurrentSplitPair)
		{
			const double StartKey = SplitKeys[CurrentSplitPair];
			const double EndKey = SplitKeys[CurrentSplitPair+1];

			UPCGSplineData* NewSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
			NewSplineData->InitializeFromData(SplineData);

			if (PCGSplitSpline::SplitSpline(SplineData, NewSplineData, SplinePoints, SplinePointsEntryKeys, StartKey, EndKey) == PCGSplitSpline::EStatus::Split)
			{
				// Write originating spline index if required
				if (Settings->bShouldOutputOriginatingSplineIndex)
				{
					const FName IndexAttributeName = Settings->OutputOriginatingSplineIndex.GetName();
					FPCGMetadataAttribute<int32>* OriginatingIndexAttribute = nullptr;

					if (FPCGMetadataDomain* OutputMetadataDomain = NewSplineData->MutableMetadata()->GetMetadataDomainFromSelector(Settings->OutputOriginatingSplineIndex))
					{
						OriginatingIndexAttribute = OutputMetadataDomain->FindOrCreateAttribute(IndexAttributeName, InputIndex, /*bAllowInterpolation=*/false);
					}

					if (!OriginatingIndexAttribute)
					{
						PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FailedCreateIndexAttribute", "Failed to create the index attribute '{0}'."), FText::FromName(IndexAttributeName)), Context);
					}
				}
			
				Outputs.Add_GetRef(Input).Data = NewSplineData;
			}
		}
	}

	return true;
}

EPCGElementExecutionLoopMode FPCGSplitSplineElement::ExecutionLoopMode(const UPCGSettings* InSettings) const
{
	const UPCGSplitSplinesSettings* Settings = Cast<UPCGSplitSplinesSettings>(InSettings);
	// @todo_pcg: Currently we don't have the concept of N:N or N:1 primary pins in the execution loop modes
	return (Settings && (Settings->bUseConstant || Settings->Mode == EPCGSplitSplineMode::ByPredicateOnControlPoints)) ? EPCGElementExecutionLoopMode::SinglePrimaryPin : EPCGElementExecutionLoopMode::NotALoop;
}

#undef LOCTEXT_NAMESPACE