// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGStaticMeshSpawner.h"

#include "PCGCustomVersion.h"
#include "PCGManagedResource.h"
#include "PCGManagedResourceContainer.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGKernelHelpers.h"
#include "Compute/BuiltInKernels/PCGSMSpawnerAnalysisKernel.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGStaticMeshSpawnerContext.h"
#include "Elements/PCGStaticMeshSpawnerKernel.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "InstanceDataPackers/PCGInstanceDataPackerBase.h"
#include "MeshSelectors/PCGMeshSelectorBase.h"
#include "MeshSelectors/PCGMeshSelectorByAttribute.h"
#include "MeshSelectors/PCGMeshSelectorPrimitiveData.h"
#include "MeshSelectors/PCGMeshSelectorWeighted.h"

#include "GrassInstancedStaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Graph/DataOverride/PCGDataOverride.h"
#include "Materials/MaterialInterface.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGStaticMeshSpawner)

#define LOCTEXT_NAMESPACE "PCGStaticMeshSpawnerElement"

namespace PCGStaticMeshSpawner
{
	/**
	 * ISM Reuse (all this applies if CVarAllowISMReuse is enabled)
	 * 
	 * There are 3 ways a SMS (Static Mesh Spawner) can do actual reuse:
	 * 1. Full Reuse: Dependencies Crc matches the last execution (all inputs, settings, etc) meaning we can reuse all our resources and skip most processing except for generating an output when needed.
	 * 2. Partial Reuse: 1-N input Crc matches the last execution in which case the resources from those inputs can be reused and the others generated.
	 * 3. Component Reuse: No inputs match but the settings/descriptor crc on the components match and so we can reuse the resource but its instances will be reset and rebuild.
	 * 
	 * Also of note: bAllowMergeDifferentDataInSameInstancedComponents is a setting that will allow different SMS to output instances to the same components if those components share the same settings/descriptor crc.
	 * 
	 * When this 'bAllowMergeDifferentDataInSameInstancedComponents' is enabled and 2 or more SMS do indeed share some resources then actual reusing of Component+Instances becomes impossible as we do not support cleaning up invalid instances.
	 * Ex: SMS1 outputs instances to Component1 and SMS2 also outputs instances to Component1, on the next execution SMS1 inputs changed and doesn't need to output to Component1 anymore but SMS2 still does. If we were to reuse Component1 as is
	 * then instances from the previous executon of SMS1 would still be present in the result which is invalid.
	 * 
	 * So when a resource is used for more than one SMS then we only allow option 3 which reuses the component but requires that all SMS re-add their instances. This has a negative impact on performance but merging allows a great reduction on
	 * number of generated resources for large scale generations. We could eventually support "cleaning" up instances from previous generations to get the best of both worlds.
	 * 
	 * How do we make sure that 1 and 2 can be achieved?
	 * 
	 * 1. Full Reuse uses
	 *		- DependenciesCrc
	 *		- FPCGStaticMeshSpawnerContext::UsedResources 
	 *		- UPCGManagedISMComponent::CrcReuseCount
	 *		- UPCGManagedISMComponent::Crc
	 * 
	 *		In FPCGStaticMeshSpawnerElement::PrepareDataInternal:
	 *		- Compute 'DependenciesCrc'.
	 *		- Iterate through all existing resources and accumulate all resources that have the same Crc.
	 *		- If the accumulated resource count is equal to each of the individual resource CrcReuseCount it means we can do a full reuse,
	 *			if the reuse count doesn't match it means one of our resources might have been used by another SMS making full reuse impossible.
	 *      - Once we know we can do full reuse, we also push those resources into: UsedResources so we can update their CrcReuseCount at the end of execution and we also push
	 *			those resources into the DataCrcToUsedResources (per input) since we can conclude that if we can do full reuse than our resoures would also be valid for partial reuse.
	 *			The reason why we do this is that if the next execution makes full reuse impossible, we can still fallback on partial reuse as our resource will have been updated properly.
	 * 
	 * 2. Partial Reuse uses
	 *		- DataCrc (per input Crc)
	 *		- FPCGStaticMeshSpawnerContext::DataCrcToUsedResources
	 *		- UPCGManagedISMComponent::DataCrcReuseCount
	 *		- UPCGManagedISMComponent::DataCrc
	 * 
	 *		Still in FPCGStaticMeshSpawnerElement::PrepareDataInternal: (if we failed doing full reuse)
	 *		- Iterate through inputs computing a per input DataCrc
	 *		- On each input iterate all existing resources and accumulate all resources that have the same DataCrc (and settings crc)
	 *		- If the accumulated resource count is equal to each of the individual resource DataCrcReuseCount it means we can do reuse for this input,
	 *			if the reuse count doesn't match it means one of our resources might have been used by another SMS making partial reuse impossible.
	 *		- Once we know we can do partial reuse, we also push those resources into: DataCrcToUsedResources so we can update their DataCrcReuseCount at the end of execution and we also push
	 *			those resources into the UsedResources (full reuse) to update their CrcReuseCount and set the DependenciesCrc to make full reuse possible on next execution if nothing changes.
	 * 
	 *	This sums up the code that handles the reuse. If we are not reusing then it is important to still compute valid CrcReuseCount/DataCrcReuseCount/Crc/DataCrc for each execution of a SMS so that reuse can happen
	 *  on subsequent executions.
	 * 
	 *  In FPCGStaticMeshSpawnerElement::SpawnStaticMeshInstances is where we do this:
	 *		- This methods is responsible for setting the Crc (DependenciesCrc) on a resource.
	 *		- If that resource already has a Crc that differs from our current DependenciesCrc it means that another SMS used this resource in which case we set UsedResources.bMergedResources = true.
	 *		- We also update that resource's Crc by combining both Crcs so that no matches can happen anymore on this resource.
	 *		- That flag will be read later one to impact the CrcReuseCount of that resource.
	 *		- We also add that resource to the UsedResource so that in the end we have a list of all the resources needed for a full reuse.
	 *		- This method also sets the DataCrc (per input) and adds that resource to the DataCrcToUsedResources map using the DataCrc as a key.
	 *		- If that resource already has a DataCrc it means that another SMS used this resource in which case we set the bMergedResources to true for its previous DataCrc and the new DataCrc.
	 *		- This is subtil but it allows flagging other resources assigned to the same input at the end of execution so that they also can't be reused.
	 *		- We also add that resource to the DataCrcToUsedResources using the new DataCrc as the key.
	 * 
	 *	At this point either through FPCGStaticMeshSpawnerElement::PrepareDataInternal or FPCGStaticMeshSpawnerElement::SpawnStaticMeshInstances, we've gathered and updated resources for reuse.
	 *  
	 *	The last step is at the end of execution in FPCGStaticMeshSpawnerElement::ExecuteInternal where we are going to go through both UsedResources and DataCrcToUsedResources and based on the bMergedResources flag for each
	 *  set of resources, are going to set the CrcReuseCount or DataCrcReuseCount to INDEX_NONE if that set has bMergedResources = true or to the actual number of resources if bMergedResources = false.
	 * 
	 *  In the case 'bAllowMergeDifferentDataInSameInstancedComponents' is false prior to calling UPCGActorHelpers::GetOrCreateManagedISMC we set DataCrc in the params struct so that we don't reuse an existing resource
	 *  with the same settings crc.
	 */
	static TAutoConsoleVariable<bool> CVarAllowISMReuse(
		TEXT("pcg.ISM.AllowReuse"),
		true,
		TEXT("Controls whether ISMs can be reused and skipped when re-executing"));

	static TAutoConsoleVariable<bool> CVarUseRawDataForAnalysis(
		TEXT("pcg.RuntimeGeneration.ISM.UseRawDataForGPUAnalysis"),
		true,
		TEXT("Read back raw data in GPU primitive instancing path rather than constructing an attribute set, to save CPU time. Read-only, must be set from ini."),
		ECVF_ReadOnly // Structural change, could flush graph compilation cache, but trickier to flush compute graph instance pool in UPCGGraph currently.
	);

	static int32 SanitizeCullGridSize(int32 In)
	{
		return FMath::RoundUpToPowerOfTwo(FMath::Max(In, 64));
	}
};

UPCGStaticMeshSpawnerSettings::UPCGStaticMeshSpawnerSettings(const FObjectInitializer &ObjectInitializer)
{
	MeshSelectorType = UPCGMeshSelectorWeighted::StaticClass();
	// Implementation note: this should not have been done here (it should have been null), as it causes issues with copy & paste
	// when the thing to paste does not have that class for its instance.
	// However, removing it makes it that any object actually using the instance created by default would be lost.
	if (!this->HasAnyFlags(RF_ClassDefaultObject))
	{
		MeshSelectorParameters = ObjectInitializer.CreateDefaultSubobject<UPCGMeshSelectorWeighted>(this, TEXT("DefaultSelectorInstance"));
	}
}

#if WITH_EDITOR
void UPCGStaticMeshSpawnerSettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	const UPCGMeshSelectorPrimitiveData* SelectorPrimitiveData = Cast<UPCGMeshSelectorPrimitiveData>(MeshSelectorParameters);

	// Only the by-primitive-data selector supports the raw buffer optimization, for which the analysis data contains table indices.
	// For by-attribute spawning the analysis data contains upstream string keys which we can't currently remap to the downstream graph string keys.
	const bool bUseRawBuffer = SelectorPrimitiveData && PCGStaticMeshSpawner::CVarUseRawDataForAnalysis.GetValueOnAnyThread();

	UPCGStaticMeshSpawnerKernel* SpawnerKernel = nullptr;

	// Create spawner kernel.
	{
		PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);

		// If spawning by-primitive-data, wire up the primitive data.
		if (const UPCGMeshSelectorPrimitiveData* Selector = Cast<UPCGMeshSelectorPrimitiveData>(MeshSelectorParameters))
		{
			CreateParams.NodeInputPinsToWire.Add(PCGStaticMeshSpawner::PrimitiveTablePinLabel);
		}

		const bool bProduceOutputPoints = !InNode || InNode->IsOutputPinConnected(PCGPinConstants::DefaultOutputLabel) || bIsInspecting || bDebug;

		if (!bProduceOutputPoints)
		{
			CreateParams.NodeOutputPinsToWire.Empty();
		}

		SpawnerKernel = PCGKernelHelpers::CreateKernel<UPCGStaticMeshSpawnerKernel>(InOutContext, CreateParams, OutKernels, OutEdges);
		SpawnerKernel->SetUseRawInstanceCountsBuffer(bUseRawBuffer);
		SpawnerKernel->SetProduceOutputPoints(bProduceOutputPoints);
		SpawnerKernel->SetTargetCullingGridSize(GetCullingCellSize());
	}

	// If doing by-attribute selection, add analysis kernel that will count how many instances of each mesh are present.
	const UPCGMeshSelectorByAttribute* SelectorByAttribute = Cast<UPCGMeshSelectorByAttribute>(MeshSelectorParameters);
	if (SelectorByAttribute || SelectorPrimitiveData)
	{
		PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);

		// Don't wire count kernel to node output pin, wire manually to the spawner kernel below.
		CreateParams.NodeOutputPinsToWire.Empty();

		UPCGSMSpawnerAnalysisKernel* CountKernel = PCGKernelHelpers::CreateKernel<UPCGSMSpawnerAnalysisKernel>(InOutContext, CreateParams, OutKernels, OutEdges);
		CountKernel->SetAttributeName(SelectorByAttribute ? SelectorByAttribute->AttributeName : SelectorPrimitiveData->PrimitiveIndexAttribute);
		// We operate across all input data rather than spawning for each input data separately.
		CountKernel->SetEmitPerDataCounts(false);
		CountKernel->SetOutputRawBuffer(bUseRawBuffer);
		CountKernel->SetCullingCellExtent(GetCullingCellSize());

		if (SelectorPrimitiveData)
		{
			// The max number of unique primitives indices is the num elements in the input primitive data table.
			CountKernel->SetMaxNumUniqueValuesSource(EPCGMaxNumUniqueValuesSource::UniqueValueTable);
			CountKernel->SetMeshAttributeName(SelectorPrimitiveData->MeshAttribute);
			OutEdges.Emplace(FPCGPinReference(PCGStaticMeshSpawner::PrimitiveTablePinLabel), FPCGPinReference(CountKernel, PCGAttributeAnalysisKernelConstants::UniqueValueTablePinLabel));
		}
		else
		{
			CountKernel->SetMaxNumUniqueValuesSource(EPCGMaxNumUniqueValuesSource::StringKeyValues);
		}

		OutEdges.Emplace(FPCGPinReference(CountKernel, PCGPinConstants::DefaultOutputLabel), FPCGPinReference(SpawnerKernel, PCGStaticMeshSpawnerKernelConstants::InstanceCountsPinLabel));
		OutEdges.Emplace(FPCGPinReference(CountKernel, PCGSMSpawnerAnalysisConstants::CullingCellMinMaxPositionsPinLabel), FPCGPinReference(SpawnerKernel, PCGStaticMeshSpawnerKernelConstants::CullingCellMinMaxPositionsPinLabel));
	}
}

FText UPCGStaticMeshSpawnerSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Static Mesh Spawner");
}

void UPCGStaticMeshSpawnerSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	check(InOutNode);

	if (DataVersion < FPCGCustomVersion::StaticMeshSpawnerApplyMeshBoundsToPointsByDefault)
	{
		UE_LOGF(LogPCG, Log, "Static Mesh Spawner node migrated from an older version. Disabling 'ApplyMeshBoundsToPoints' by default to match previous behavior.");
		bApplyMeshBoundsToPoints = false;
	}

	Super::ApplyDeprecation(InOutNode);
}

EPCGChangeType UPCGStaticMeshSpawnerSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGStaticMeshSpawnerSettings, MeshSelectorType))
	{
		// Mesh selectors can drive node pins.
		ChangeType |= EPCGChangeType::Structural | EPCGChangeType::Cosmetic;
	}

	return ChangeType;
}
#endif

int32 UPCGStaticMeshSpawnerSettings::GetCullingCellSize() const
{
	return bSetupCullingCells ? PCGStaticMeshSpawner::SanitizeCullGridSize(CullingCellSize) : 0;
}

TArray<FPCGPinProperties> UPCGStaticMeshSpawnerSettings::InputPinProperties() const
{
	// Note: If executing on the GPU, we need to prevent multiple connections on inputs, since it is not supported at this time.
	// Also note: Since the ShouldExecuteOnGPU() is already tied to structural changes, we don't need to implement any logic for this in GetChangeTypeForProperty()
	const bool bAllowMultipleConnections = !ShouldExecuteOnGPU();

	TArray<FPCGPinProperties> Properties;
	FPCGPinProperties& InputPinProperty = Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point, bAllowMultipleConnections);
	InputPinProperty.SetRequiredPin();

	if (MeshSelectorType && MeshSelectorType->IsChildOf(UPCGMeshSelectorPrimitiveData::StaticClass()))
	{
		FPCGPinProperties& PrimitiveDataPin = Properties.Emplace_GetRef(PCGStaticMeshSpawner::PrimitiveTablePinLabel, EPCGDataType::Param, /*bAllowMultipleConnections=*/false);
		PrimitiveDataPin.SetRequiredPin();
#if WITH_EDITOR
		PrimitiveDataPin.Tooltip = LOCTEXT("PrimitiveDataPinTooltip", "Table of attributes that drive primitive properties. The primitive index attribute on the input points index into rows of this table. This data is only used on the CPU; GPU data will be read back to CPU.");
#endif
	}

	return Properties;
}

FPCGElementPtr UPCGStaticMeshSpawnerSettings::CreateElement() const
{
	return MakeShared<FPCGStaticMeshSpawnerElement>();
}

FPCGContext* FPCGStaticMeshSpawnerElement::CreateContext()
{
	return new FPCGStaticMeshSpawnerContext();
}

bool FPCGStaticMeshSpawnerElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGStaticMeshSpawnerElement::PrepareDataInternal);
	// TODO : time-sliced implementation
	FPCGStaticMeshSpawnerContext* Context = static_cast<FPCGStaticMeshSpawnerContext*>(InContext);
	const UPCGStaticMeshSpawnerSettings* Settings = Context->GetInputSettings<UPCGStaticMeshSpawnerSettings>();
	check(Settings);

	if (!Settings->MeshSelectorParameters)
	{
		PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidMeshSelectorInstance", "Invalid MeshSelector instance, try reselecting the MeshSelector type"));
		return true;
	}

	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	if (!ExecutionSource)
	{
		return true;
	}

	FPCGManagedResourceContainerHelper ManagedResourcesContainerHelper(ExecutionSource);

#if WITH_EDITOR
	// In editor, we always want to generate this data for inspection & to prevent caching issues
	const bool bGenerateOutput = true;
#else
	const bool bGenerateOutput = Context->Node && Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultOutputLabel);
#endif

	// Check if we can reuse existing resources on an "all-or-nothing" basis
	bool& bSkippedDueToReuse = Context->bSkippedDueToReuse;

	if (!Context->bReuseCheckDone && PCGStaticMeshSpawner::CVarAllowISMReuse.GetValueOnAnyThread() && ManagedResourcesContainerHelper.IsValid())
	{
		// Compute CRC if it has not been computed (it likely isn't, but this is to futureproof this)
		if (!Context->DependenciesCrc.IsValid())
		{
			FPCGDataCollection TempCollection;
			const FPCGDataCollection* InputDataCollectionPtr = &Context->InputData;

			// If the collection didn't have any Data crc information yet, we need to compute it, but can't do it on the input directly.
			if (Context->InputData.DataCrcs.Num() != Context->InputData.TaggedData.Num())
			{
				TempCollection = Context->InputData;
				TempCollection.ComputeCrcs(ShouldComputeFullOutputDataCrc(Context));
				InputDataCollectionPtr = &TempCollection;
			}

			GetDependenciesCrc(FPCGGetDependenciesCrcParams(InputDataCollectionPtr, Settings, Context->ExecutionSource.Get()), Context->DependenciesCrc);
		}
		
		if (Context->DependenciesCrc.IsValid())
		{
			int32 CrcReuseCount = 0;
			TArray<UPCGManagedISMComponent*> MISMCs;
			ManagedResourcesContainerHelper.ForEachManagedResource([&MISMCs, &Context, Settings, &CrcReuseCount](UPCGManagedResource* InResource)
			{
				if (UPCGManagedISMComponent* Resource = Cast<UPCGManagedISMComponent>(InResource))
				{
					if (Resource->GetCrc().IsValid() && Resource->GetCrc() == Context->DependenciesCrc && Resource->GetCrcReuseCount() != INDEX_NONE)
					{
						if (ensure(CrcReuseCount == 0 || CrcReuseCount == Resource->GetCrcReuseCount()))
						{
							CrcReuseCount = Resource->GetCrcReuseCount();
							MISMCs.Add(Resource);
						}
					}
				}
			});

			if (CrcReuseCount > 0 && CrcReuseCount == MISMCs.Num())
			{
				for (UPCGManagedISMComponent* MISMC : MISMCs)
				{
					if (!MISMC->IsMarkedUnused() && Settings->bWarnOnIdenticalSpawn)
					{
						// TODO: Revisit if the stack is added to the managed components at creation
						PCGLog::LogWarningOnGraph(LOCTEXT("IdenticalISMCSpawn", "Identical ISM Component spawn occurred. It may be beneficial to re-check graph logic for identical spawn conditions (same mesh descriptor at same location, etc) or repeated nodes."), Context);
					}

					MISMC->MarkAsReused();
					Context->UsedResources.Resources.Add(MISMC);

					if (ensure(MISMC->GetDataCrc().IsValid()))
					{
						Context->DataCrcToUsedResources.FindOrAdd(MISMC->GetDataCrc().GetValue()).Resources.Add(MISMC);
					}
				}
				bSkippedDueToReuse = true;
			}
		}

		Context->bReuseCheckDone = true;
	}

	// Early out - if we've established we could reuse resources and there is no need to generate an output, quit now
	if (!bGenerateOutput && bSkippedDueToReuse)
	{
		return true;
	}

	// perform mesh selection
	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData> PrimitiveTableInputs = Context->InputData.GetInputsByPin(PCGStaticMeshSpawner::PrimitiveTablePinLabel);

	if (PrimitiveTableInputs.Num() > 1)
	{
		PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGStaticMeshSpawner::PrimitiveTablePinLabel, Context);
	}

	const FPCGTaggedData* PrimitiveTableInput = !PrimitiveTableInputs.IsEmpty() ? &PrimitiveTableInputs[0] : nullptr;

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	while(Context->CurrentInputIndex < Inputs.Num())
	{
		if (!Context->bCurrentInputSetup)
		{
			const FPCGTaggedData& Input = Inputs[Context->CurrentInputIndex];
			const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Input.Data);

			if (!SpatialData)
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidInputData", "Invalid input data"));
				++Context->CurrentInputIndex;
				continue;
			}

			const UPCGBasePointData* PointData = SpatialData->ToBasePointData(Context);
			if (!PointData)
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("NoPointDataInInput", "Unable to get point data from input"));
				++Context->CurrentInputIndex;
				continue;
			}

			AActor* TargetActor = Settings->TargetActor.Get() ? Settings->TargetActor.Get() : Context->GetTypedExecutionTarget<AActor>();
			if (!TargetActor)
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidTargetActor", "Invalid target actor. Ensure TargetActor member is initialized when creating SpatialData."));
				++Context->CurrentInputIndex;
				continue;
			}

			// Prior to selection, if we have MISMC resources that have the same settings crc & the same data crc,
			// We can skip the selection here, unless we need to output the data (again, could/should be cached instead).
			FPCGCrc DataCrc;
			if(!bSkippedDueToReuse && PCGStaticMeshSpawner::CVarAllowISMReuse.GetValueOnAnyThread() && ManagedResourcesContainerHelper.IsValid())
			{
				ensure(Context->DependenciesCrc.IsValid()); // should have been done earlier

				const FPCGCrc SettingsCrc = Settings->GetSettingsCrc();
				ensure(SettingsCrc.IsValid());

				// Compute this specific data crc as-if it were alone
				{
					FPCGDataCollection SubCollection;
					SubCollection.TaggedData.Add(Input);
					if (PrimitiveTableInput)
					{
						SubCollection.TaggedData.Add(*PrimitiveTableInput);
					}

					SubCollection.ComputeCrcs(ShouldComputeFullOutputDataCrc(Context));

					GetDependenciesCrc(FPCGGetDependenciesCrcParams(&SubCollection, Settings, Context->ExecutionSource.Get()), DataCrc);
				}

				int32 DataCrcReuseCount = 0;
				TArray<UPCGManagedISMComponent*> MISMCs;
				ManagedResourcesContainerHelper.ForEachManagedResource([&MISMCs, &SettingsCrc, &DataCrc, &Context, &DataCrcReuseCount](UPCGManagedResource* InResource)
				{
					if (UPCGManagedISMComponent* Resource = Cast<UPCGManagedISMComponent>(InResource))
					{
						// All inputs are reused so just make sure to track per data crcs used resources and update data crcs
						if (Resource->GetSettingsCrc().IsValid() && Resource->GetSettingsCrc() == SettingsCrc &&
							Resource->GetDataCrc().IsValid() && Resource->GetDataCrc() == DataCrc &&
							Resource->GetDataCrcReuseCount() != INDEX_NONE)
						{
							if (ensure(DataCrcReuseCount == 0 || DataCrcReuseCount == Resource->GetDataCrcReuseCount()))
							{
								DataCrcReuseCount = Resource->GetDataCrcReuseCount();
								MISMCs.Add(Resource);
							}
						}
					}
				});

				if (DataCrcReuseCount > 0 && MISMCs.Num() == DataCrcReuseCount)
				{
					for (UPCGManagedISMComponent* MISMC : MISMCs)
					{
						if (!MISMC->IsMarkedUnused() && Settings->bWarnOnIdenticalSpawn)
						{
							// TODO: Revisit if the stack is added to the managed components at creation
							PCGLog::LogWarningOnGraph(LOCTEXT("IdenticalISMCSpawn", "Identical ISM Component spawn occurred. It may be beneficial to re-check graph logic for identical spawn conditions (same mesh descriptor at same location, etc) or repeated nodes."), Context);
						}

						MISMC->MarkAsReused();

						// Update global crc, otherwise these resources wouldn't get picked up in a subsequent update
						MISMC->SetCrc(Context->DependenciesCrc);

						Context->DataCrcToUsedResources.FindOrAdd(DataCrc.GetValue()).Resources.Emplace(MISMC);
						Context->UsedResources.Resources.Add(MISMC);
					}

					Context->bCurrentDataSkippedDueToReuse = true;
				}
			}

			if (bGenerateOutput)
			{
				FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

				UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(Context);

				// @todo_pcg: this could probably be inherited, since not all selectors output all points this works for now.
				FPCGInitializeFromDataParams InitializeFromDataParams(PointData);
				InitializeFromDataParams.bInheritSpatialData = false;
				OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);

				OutputPointData->SetNumPoints(PointData->GetNumPoints());
				OutputPointData->AllocateProperties(PointData->GetAllocatedProperties());

				if (OutputPointData->Metadata->HasAttribute(Settings->OutAttributeName))
				{
					OutputPointData->Metadata->DeleteAttribute(Settings->OutAttributeName);
					PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("AttributeOverwritten", "Metadata attribute '{0}' is being overwritten in the output data"), FText::FromName(Settings->OutAttributeName)));
				}

				OutputPointData->Metadata->CreateStringAttribute(Settings->OutAttributeName, FName(NAME_None).ToString(), /*bAllowsInterpolation=*/false);

				Output.Data = OutputPointData;
				check(!Context->CurrentOutputPointData);
				Context->CurrentOutputPointData = OutputPointData;
			}

			if (bGenerateOutput || !Context->bCurrentDataSkippedDueToReuse)
			{
				FPCGStaticMeshSpawnerContext::FPackedInstanceListData& InstanceListData = Context->MeshInstancesData.Emplace_GetRef();
				InstanceListData.TargetActor = TargetActor;
				InstanceListData.SpatialData = PointData;
				InstanceListData.DataCrc = DataCrc;
				InstanceListData.bSkippedDueToReuse = Context->bSkippedDueToReuse || Context->bCurrentDataSkippedDueToReuse;

				Context->CurrentPointData = PointData;
				Context->bCurrentInputSetup = true;
			}
			else
			{
				// skip selection
				Context->bSelectionDone = true;
			}
		}

		// TODO: If we know we re-use the ISMCs, we should not run the Selection, as it can be pretty costly.
		// At the moment, the selection is filling the output point data, so it is necessary to run it. But we should just hit the cache in that case.
		if (!Context->bSelectionDone)
		{
			check(Context->CurrentPointData);
			Context->bSelectionDone = Settings->MeshSelectorParameters->SelectMeshInstances(*Context, Settings, Context->CurrentPointData, Context->MeshInstancesData.Last().MeshInstances, Context->CurrentOutputPointData);
		}

		if (!Context->bSelectionDone)
		{
			return false;
		}

		// If we need the output but would otherwise skip the resource creation, we don't need to run the instance packing part of the processing
		if (!bSkippedDueToReuse && !Context->bCurrentDataSkippedDueToReuse)
		{
			TArray<FPCGPackedCustomData>& PackedCustomData = Context->MeshInstancesData.Last().PackedCustomData;
			const TArray<FPCGMeshInstanceList>& MeshInstances = Context->MeshInstancesData.Last().MeshInstances;

			if (PackedCustomData.Num() != MeshInstances.Num())
			{
				PackedCustomData.SetNum(MeshInstances.Num());
			}

			if (Settings->InstanceDataPackerParameters)
			{
				for (int32 InstanceListIndex = 0; InstanceListIndex < MeshInstances.Num(); ++InstanceListIndex)
				{
					Settings->InstanceDataPackerParameters->PackInstances(*Context, Context->CurrentPointData, MeshInstances[InstanceListIndex], PackedCustomData[InstanceListIndex]);
				}
			}
		}

		// We're done - cleanup for next iteration if we still have time
		++Context->CurrentInputIndex;
		Context->ResetInputIterationData();

		// Continue on to next iteration if there is time left, otherwise, exit here
		if (Context->AsyncState.ShouldStop() && Context->CurrentInputIndex < Inputs.Num())
		{
			return false;
		}
	}

	IPCGAsyncLoadingContext* AsyncLoadingContext = static_cast<IPCGAsyncLoadingContext*>(Context);

	if (Context->CurrentInputIndex == Inputs.Num() && !AsyncLoadingContext->WasLoadRequested() && !Context->MeshInstancesData.IsEmpty() && !Settings->bSynchronousLoad)
	{
		TArray<FSoftObjectPath> ObjectsToLoad;
		for (const FPCGStaticMeshSpawnerContext::FPackedInstanceListData& InstanceData : Context->MeshInstancesData)
		{
			for (const FPCGMeshInstanceList& MeshInstanceList : InstanceData.MeshInstances)
			{
				if (!MeshInstanceList.Descriptor.StaticMesh.IsNull())
				{
					ObjectsToLoad.AddUnique(MeshInstanceList.Descriptor.StaticMesh.ToSoftObjectPath());
				}

				for (const TSoftObjectPtr<UMaterialInterface>& OverrideMaterial : MeshInstanceList.Descriptor.OverrideMaterials)
				{
					if (!OverrideMaterial.IsNull())
					{
						ObjectsToLoad.AddUnique(OverrideMaterial.ToSoftObjectPath());
					}
				}
			}
		}

		return AsyncLoadingContext->RequestResourceLoad(Context, std::move(ObjectsToLoad), /*bAsynchronous=*/true);
	}

	return true;
}

bool FPCGStaticMeshSpawnerElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGStaticMeshSpawnerElement::Execute);
	FPCGStaticMeshSpawnerContext* Context = static_cast<FPCGStaticMeshSpawnerContext*>(InContext);
	const UPCGStaticMeshSpawnerSettings* Settings = Context->GetInputSettings<UPCGStaticMeshSpawnerSettings>();
	check(Settings && !Settings->ShouldExecuteOnGPU());

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("AddISM", "Adding ISM/instances to an ISM from PCG"), Context->ExecutionSource.Get() && Context->ExecutionSource->GetExecutionState().UseTransactions());
#endif

	while(!Context->MeshInstancesData.IsEmpty())
	{
		const FPCGStaticMeshSpawnerContext::FPackedInstanceListData& InstanceList = Context->MeshInstancesData.Last();
		check(InstanceList.bSkippedDueToReuse || InstanceList.MeshInstances.Num() == InstanceList.PackedCustomData.Num());

		const bool bTargetActorValid = (InstanceList.TargetActor && IsValid(InstanceList.TargetActor));

		if (bTargetActorValid)
		{
			while (Context->CurrentDataIndex < InstanceList.MeshInstances.Num())
			{
				const FPCGMeshInstanceList& MeshInstance = InstanceList.MeshInstances[Context->CurrentDataIndex];
				// We always have mesh instances, but if we are in re-use, we don't compute the packed custom data.
				const FPCGPackedCustomData* PackedCustomData = InstanceList.PackedCustomData.IsValidIndex(Context->CurrentDataIndex) ? &InstanceList.PackedCustomData[Context->CurrentDataIndex] : nullptr;
				SpawnStaticMeshInstances(Context, InstanceList, MeshInstance, PackedCustomData);

				// Now that the mesh is loaded/spawned, set the bounds to out points if requested.
				if (MeshInstance.Descriptor.StaticMesh && Settings->bApplyMeshBoundsToPoints)
				{
					if (TMap<UPCGBasePointData*, TArray<int32>>* OutPointDataToPointIndex = Context->MeshToOutPoints.Find(MeshInstance.Descriptor.StaticMesh))
					{
						const FBox Bounds = MeshInstance.Descriptor.StaticMesh->GetBoundingBox();
						for (TPair<UPCGBasePointData*, TArray<int32>>& It : *OutPointDataToPointIndex)
						{
							check(It.Key);

							TPCGValueRange<FVector> BoundsMinRange = It.Key->GetBoundsMinValueRange();
							TPCGValueRange<FVector> BoundsMaxRange = It.Key->GetBoundsMaxValueRange();

							for (int32 Index : It.Value)
							{
								BoundsMinRange[Index] = Bounds.Min;
								BoundsMaxRange[Index] = Bounds.Max;
							}
						}
					}
				}

				++Context->CurrentDataIndex;

				if (Context->AsyncState.ShouldStop())
				{
					break;
				}
			}
		}

		if (!bTargetActorValid || Context->CurrentDataIndex == InstanceList.MeshInstances.Num())
		{
			Context->MeshInstancesData.RemoveAtSwap(Context->MeshInstancesData.Num() - 1);
			Context->CurrentDataIndex = 0;
		}

		if (Context->AsyncState.ShouldStop())
		{
			break;
		}
	}

	const bool bFinishedExecution = Context->MeshInstancesData.IsEmpty();
	if (bFinishedExecution)
	{
		{
			// We finished executing, make sure to add information needed to compare against full reuse
			const int32 CrcReuseCount = Context->UsedResources.bMergedResources ? INDEX_NONE : Context->UsedResources.Resources.Num();

			for (TWeakObjectPtr<UPCGManagedISMComponent> WeakManagedResource : Context->UsedResources.Resources)
			{
				if (UPCGManagedISMComponent* ManagedResource = WeakManagedResource.Get())
				{
					ManagedResource->SetCrcReuseCount(CrcReuseCount);
				}
			}
		}

		{
			// We finished executing, make sure to add information needed to compare against per input reuse
			for (auto& [DataCrc, UsedResources] : Context->DataCrcToUsedResources)
			{
				const int32 DataCrcReuseCount = UsedResources.bMergedResources ? INDEX_NONE : UsedResources.Resources.Num();

				for (TWeakObjectPtr<UPCGManagedISMComponent> WeakManagedResource : UsedResources.Resources)
				{
					if (UPCGManagedISMComponent* ManagedResource = WeakManagedResource.Get())
					{
						ManagedResource->SetDataCrcReuseCount(DataCrcReuseCount);
					}
				}
			}
		}

		if (AActor* TargetActor = Settings->TargetActor.Get() ? Settings->TargetActor.Get() : Context->GetTypedExecutionTarget<AActor>())
		{
			for (UFunction* Function : PCGHelpers::FindUserFunctions(TargetActor->GetClass(), Settings->PostProcessFunctionNames, { UPCGFunctionPrototypes::GetPrototypeWithNoParams() }, Context))
			{
				TargetActor->ProcessEvent(Function, nullptr);
			}
		}
	}

	return bFinishedExecution;
}

bool FPCGStaticMeshSpawnerElement::CanExecuteOnlyOnMainThread(FPCGContext* Context) const
{
	// PrepareData can call UPCGManagedComponent::MarkAsReused which registers the ISMC, which can go into Chaos code that asserts if not on main thread.
	// TODO: We can likely re-enable multi-threading for PrepareData if we move the call to MarkAsReused to Execute. There should hopefully not be
	// wider contention on resources resources are not shared across nodes and are also per-component.
	return !Context || Context->CurrentPhase == EPCGExecutionPhase::Execute || Context->CurrentPhase == EPCGExecutionPhase::PrepareData;
}

void FPCGStaticMeshSpawnerElement::SpawnStaticMeshInstances(FPCGStaticMeshSpawnerContext* Context, const FPCGStaticMeshSpawnerContext::FPackedInstanceListData& InstanceListData, const FPCGMeshInstanceList& InstanceList, const FPCGPackedCustomData* InPackedCustomData) const
{
	// Populate the (H)ISM from the previously prepared entries
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGStaticMeshSpawnerElement::Execute::PopulateISMs);

	AActor* TargetActor = InstanceListData.TargetActor;
	const FPCGCrc& DataCrc = InstanceListData.DataCrc;
	const bool bSkippedDueToReuse = InstanceListData.bSkippedDueToReuse;

	if (InstanceList.Instances.Num() == 0)
	{
		return;
	}
	
	if (InstanceList.Descriptor.ComponentClass && InstanceList.Descriptor.ComponentClass->IsChildOf<UGrassInstancedStaticMeshComponent>())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NoSupportForGrassComponent", "Grass Instanced Static Mesh Component are not meant to be used outside of the landscape grass system, and are not supported by PCG."), Context);
		return;
	}

	// Will be synchronously loaded if not loaded. But by default it should already have been loaded asynchronously in PrepareData, so this is free.
	UStaticMesh* LoadedMesh = InstanceList.Descriptor.StaticMesh.LoadSynchronous();

	if (!LoadedMesh)
	{
		// Either we have no mesh (so nothing to do) or the mesh couldn't be loaded
		if (InstanceList.Descriptor.StaticMesh.IsValid())
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("MeshLoadFailed", "Unable to load mesh '{0}'"), FText::FromString(InstanceList.Descriptor.StaticMesh.ToString())));
		}

		return;
	}

	// Don't spawn meshes if we reuse the ISMCs, but we still want to be sure that the mesh is loaded at least (for operations downstream).
	if (bSkippedDueToReuse)
	{
		return;
	}

	for (TSoftObjectPtr<UMaterialInterface> OverrideMaterial : InstanceList.Descriptor.OverrideMaterials)
	{
		// Will be synchronously loaded if not loaded. But by default it should already have been loaded asynchronously in PrepareData, so this is free.
		if (OverrideMaterial.IsValid() && !OverrideMaterial.LoadSynchronous())
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("OverrideMaterialLoadFailed", "Unable to load override material '{0}'"), FText::FromString(OverrideMaterial.ToString())));
			return;
		}
	}

	// If we spawn the meshes, we should have computed a packed custom data.
	if (!ensure(InPackedCustomData))
	{
		return;
	}

	const FPCGPackedCustomData& PackedCustomData = *InPackedCustomData;

	FPCGISMComponentBuilderParams Params;
	Params.Descriptor = InstanceList.Descriptor;
	Params.NumCustomDataFloats = PackedCustomData.NumCustomDataFloats;
	Params.CustomPrimitiveData = MakeConstArrayView(InstanceList.CustomPrimitiveData);

	// If the root actor we're binding to is movable, then the ISMC should be movable by default
	if (USceneComponent* SceneComponent = TargetActor->GetRootComponent())
	{
		Params.Descriptor.Mobility = SceneComponent->Mobility;
	}

	const UPCGStaticMeshSpawnerSettings* Settings = Context->GetInputSettings<UPCGStaticMeshSpawnerSettings>();
	check(Settings);

	Params.SettingsCrc = Settings->GetSettingsCrc();
	ensure(Params.SettingsCrc.IsValid());

	// Enforce selection limitation when getting the MISMC based on whether we want to share the ISM or not.
	if (!Settings->bAllowMergeDifferentDataInSameInstancedComponents)
	{
		Params.DataCrc = DataCrc;
	}

	// Implementation note: in order to prevent components from merging together when they have tags, we need to incorporate the tags hash
	// into the settings crc. This will make sure we don't select component that don't match up with what we're expecting.
	// However, this validation is potentially too restrictive, as the order and composition of the tags ultimately don't really matter.
	// TODO- improve this, however since this stems from a path where we build those, the ordering of tags could be sorted in other ways.
	if (Params.Descriptor.HasTags())
	{
		const uint32 TagsArrayHash = Params.Descriptor.ComponentTags.IsEmpty() ? 1 : GetTypeHash(Params.Descriptor.ComponentTags);
		const uint32 AdditionalTagsHash = Params.Descriptor.AdditionalCommaSeparatedTags.IsEmpty() ? 1 : GetTypeHash(Params.Descriptor.AdditionalCommaSeparatedTags);
		Params.SettingsCrc.Combine(HashCombine(TagsArrayHash, AdditionalTagsHash));
	}

	Params.bAllowDescriptorChanges = Settings->bAllowDescriptorChanges;

	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	if (!ensure(ExecutionSource))
	{
		return;
	}

	UPCGManagedISMComponent* MISMC = UPCGActorHelpers::GetOrCreateManagedISMC(TargetActor, ExecutionSource, Params, Context);

	check(MISMC);

	// When spawning from different inputs, the dependencies crc will be the same on the same spawner but across multiple spawners it will change and needs
	// to be updated to disable full reuse across multiple spawners.
	if (MISMC->GetCrc().IsValid() && MISMC->GetCrc() != Context->DependenciesCrc)
	{
		FPCGCrc UpdatedCrc = MISMC->GetCrc();
		UpdatedCrc.Combine(Context->DependenciesCrc);
		MISMC->SetCrc(UpdatedCrc);
		Context->UsedResources.bMergedResources = true;
	}
	else
	{
		MISMC->SetCrc(Context->DependenciesCrc);
	}

	ensure(Settings->bAllowMergeDifferentDataInSameInstancedComponents || !MISMC->GetDataCrc().IsValid() && MISMC->GetComponent() && MISMC->GetComponent()->GetNumInstances() == 0);

	// Mutate MISMC data crc, so this will ensure that when we write multiple data to the same ISM that we don't trigger reuse.
	if (MISMC->GetDataCrc().IsValid())
	{
		FPCGCrc CurrentDataCrc = MISMC->GetDataCrc();

		// This means we have different inputs sharing the same ISM in which case we can't do partial reuse (on both inputs, previous crc and current crc)
		Context->DataCrcToUsedResources.FindOrAdd(CurrentDataCrc.GetValue()).bMergedResources = true;
		Context->DataCrcToUsedResources.FindOrAdd(DataCrc.GetValue()).bMergedResources = true;

		CurrentDataCrc.Combine(DataCrc);
		MISMC->SetDataCrc(CurrentDataCrc);
	}
	else
	{
		MISMC->SetDataCrc(DataCrc);
	}

	Context->DataCrcToUsedResources.FindOrAdd(DataCrc.GetValue()).Resources.Emplace(MISMC);
	Context->UsedResources.Resources.Emplace(MISMC);

	// Keep track of all touched resources in the context, because if the execution is cancelled during the SMS execution
	// we cannot easily guarantee that the state (esp. vs CRCs) is going to be entirely valid
	Context->TouchedResources.Emplace(MISMC);

	UInstancedStaticMeshComponent* ISMC = MISMC->GetComponent();
	check(ISMC);

	const int32 PreExistingInstanceCount = ISMC->GetInstanceCount();
	const int32 NewInstanceCount = InstanceList.Instances.Num();
	const int32 NumCustomDataFloats = PackedCustomData.NumCustomDataFloats;

	check((ISMC->NumCustomDataFloats == 0 && PreExistingInstanceCount == 0) || ISMC->NumCustomDataFloats == NumCustomDataFloats);
	ISMC->SetNumCustomDataFloats(NumCustomDataFloats);

	// The index in ISMC PerInstanceSMCustomData where we should pick up to begin inserting new floats
	const int32 PreviousCustomDataOffset = PreExistingInstanceCount * NumCustomDataFloats;

	// Populate the ISM instances
	ISMC->AddInstances(InstanceList.Instances, /*bShouldReturnIndices=*/false, /*bWorldSpace=*/true);

	// Copy new CustomData into the ISMC PerInstanceSMCustomData
	if (NumCustomDataFloats > 0)
	{
		check(PreviousCustomDataOffset + PackedCustomData.CustomData.Num() == ISMC->PerInstanceSMCustomData.Num());
		for (int32 NewIndex = 0; NewIndex < NewInstanceCount; ++NewIndex)
		{
			ISMC->SetCustomData(PreExistingInstanceCount + NewIndex, MakeArrayView(&PackedCustomData.CustomData[NewIndex * NumCustomDataFloats], NumCustomDataFloats));
		}
	}

	ISMC->UpdateBounds();

	{
		PCGE_LOG(Verbose, LogOnly, FText::Format(LOCTEXT("GenerationInfo", "Added {0} instances of '{1}' on actor '{2}'"),
			InstanceList.Instances.Num(), FText::FromString(InstanceList.Descriptor.StaticMesh->GetFName().ToString()), FText::FromString(TargetActor->GetFName().ToString())));
	}
}

EPCGDataOverridePhase FPCGStaticMeshSpawnerElement::GetDataOverridePhase() const
{
	return EPCGDataOverridePhase::PrepareData;
}

void FPCGStaticMeshSpawnerElement::AbortInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGStaticMeshSpawnerElement::AbortInternal);
	// It is possible to Abort a ready task with no context yet
	if (!InContext)
	{
		return;
	}

	FPCGStaticMeshSpawnerContext* Context = static_cast<FPCGStaticMeshSpawnerContext*>(InContext);

	// Any resources we've touched during the execution of this node can potentially be in a "not-quite complete state" especially if we have multiple sources of data writing to the same ISMC.
	// In this case, we're aiming to mark the resources as "Unused" so they are picked up to be removed during the component's OnProcessGraphAborted, which is why we call Release here.
	for (TWeakObjectPtr<UPCGManagedISMComponent> ManagedResource : Context->TouchedResources)
	{
		if(ManagedResource.IsValid())
		{
			TSet<TSoftObjectPtr<AActor>> Dummy;
			ManagedResource->Release(/*bHardRelease=*/false, Dummy);
		}
	}
}

void UPCGStaticMeshSpawnerSettings::PostLoad()
{
	Super::PostLoad();

	const EObjectFlags Flags = GetMaskedFlags(RF_PropagateToSubObjects) | RF_Transactional;
	
	if (!MeshSelectorParameters)
	{
		RefreshMeshSelector();
	}
	else
	{
		MeshSelectorParameters->SetFlags(Flags);
	}

	if (!InstanceDataPackerParameters)
	{
		RefreshInstancePacker();
	}
	else
	{
		InstanceDataPackerParameters->SetFlags(Flags);
	}
}

#if WITH_EDITOR
void UPCGStaticMeshSpawnerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) 
{
	if (PropertyChangedEvent.Property)
	{
		const FName& PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGStaticMeshSpawnerSettings, MeshSelectorType))
		{
			RefreshMeshSelector();
		} 
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGStaticMeshSpawnerSettings, InstanceDataPackerType))
		{
			RefreshInstancePacker();
		}
	}

	CullingCellSize = PCGStaticMeshSpawner::SanitizeCullGridSize(CullingCellSize);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UPCGStaticMeshSpawnerSettings::CanEditChange(const FProperty* InProperty) const
{
	// TODO: In place temporarily, until the other two modes are supported
	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGStaticMeshSpawnerSettings, StaticMeshComponentPropertyOverrides))
	{
		if (!MeshSelectorType->IsChildOf(UPCGMeshSelectorByAttribute::StaticClass()))
		{
			return false;
		}
	}

	return Super::CanEditChange(InProperty);
}
#endif

void UPCGStaticMeshSpawnerSettings::SetMeshSelectorType(TSubclassOf<UPCGMeshSelectorBase> InMeshSelectorType) 
{
	if (!MeshSelectorParameters || InMeshSelectorType != MeshSelectorType)
	{
		if (InMeshSelectorType != MeshSelectorType)
		{
			MeshSelectorType = InMeshSelectorType;
		}
		
		RefreshMeshSelector();
	}
}

void UPCGStaticMeshSpawnerSettings::SetInstancePackerType(TSubclassOf<UPCGInstanceDataPackerBase> InInstancePackerType) 
{
	if (!InstanceDataPackerParameters || InInstancePackerType != InstanceDataPackerType)
	{
		if (InInstancePackerType != InstanceDataPackerType)
		{
			InstanceDataPackerType = InInstancePackerType;
		}
		
		RefreshInstancePacker();
	}
}

void UPCGStaticMeshSpawnerSettings::RefreshMeshSelector()
{
	if (MeshSelectorType)
	{
		ensure(IsInGameThread());

		if (MeshSelectorParameters)
		{
#if WITH_EDITOR
			MeshSelectorParameters->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
#endif
			MeshSelectorParameters->MarkAsGarbage();
			MeshSelectorParameters = nullptr;
		}

		const EObjectFlags Flags = GetMaskedFlags(RF_PropagateToSubObjects);
		MeshSelectorParameters = NewObject<UPCGMeshSelectorBase>(this, MeshSelectorType, NAME_None, Flags);
	}
	else
	{
		MeshSelectorParameters = nullptr;
	}
}

void UPCGStaticMeshSpawnerSettings::RefreshInstancePacker()
{
	if (InstanceDataPackerType)
	{
		ensure(IsInGameThread());

		if (InstanceDataPackerParameters)
		{
#if WITH_EDITOR
			InstanceDataPackerParameters->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
#endif
			InstanceDataPackerParameters->MarkAsGarbage();
			InstanceDataPackerParameters = nullptr;
		}

		const EObjectFlags Flags = GetMaskedFlags(RF_PropagateToSubObjects);
		InstanceDataPackerParameters = NewObject<UPCGInstanceDataPackerBase>(this, InstanceDataPackerType, NAME_None, Flags);
	}
	else
	{
		InstanceDataPackerParameters = nullptr;
	}
}

FPCGStaticMeshSpawnerContext::FPackedInstanceListData::FPackedInstanceListData() = default;
FPCGStaticMeshSpawnerContext::FPackedInstanceListData::~FPackedInstanceListData() = default;

#undef LOCTEXT_NAMESPACE
