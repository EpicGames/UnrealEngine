// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGStaticMeshSpawnerContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGStaticMeshSpawnerContext)

void FPCGStaticMeshSpawnerContext::ResetInputIterationData()
{
	bCurrentInputSetup = false;
	bSelectionDone = false;
	bPartitionDone = false;
	bCurrentDataSkippedDueToReuse = false;
	CurrentPointData = nullptr;
	CurrentOutputPointData = nullptr;
	MaterialOverrideHelper.Reset();
	CurrentPointIndex = 0;
	CurrentWriteIndex = 0;
	WeightedMeshInstances.Reset();
	MeshToValueKey.Reset();
	CumulativeWeights.Reset();
	CategoryEntryToInstancesAndWeights.Reset();
	AttributeOverridePartition.Reset();
	OverriddenDescriptors.Reset();
}
