// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGStaticMeshSpawnerKernel.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Components/PCGProceduralISMComponent.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGDataBinding.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/BuiltInKernels/PCGCountUniqueAttributeValuesKernel.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Compute/DataInterfaces/PCGInstanceDataInterface.h"
#include "Compute/DataInterfaces/Elements/PCGStaticMeshSpawnerDataInterface.h"
#include "Elements/PCGStaticMeshSpawner.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "InstanceDataPackers/PCGInstanceDataPackerBase.h"
#include "MeshSelectors/PCGMeshSelectorByAttribute.h"
#include "MeshSelectors/PCGMeshSelectorPrimitiveData.h"
#include "MeshSelectors/PCGMeshSelectorWeighted.h"

#include "ShaderCompilerCore.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGStaticMeshSpawnerKernel)

#define LOCTEXT_NAMESPACE "PCGStaticMeshSpawnerKernel"

namespace PCGStaticMeshSpawnerKernel
{
	static const FText NoMeshEntriesFormat = LOCTEXT("NoMeshEntries", "No mesh entries provided in weighted mesh selector.");
}

TSharedPtr<const FPCGDataCollectionDesc> UPCGStaticMeshSpawnerKernel::ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const
{
	check(InBinding);

	// Code assumes single output pin.
	if (!ensure(InOutputPinLabel == PCGPinConstants::DefaultOutputLabel))
	{
		return nullptr;
	}

	// Forward data from In to Out.
	const FPCGKernelPin InputKernelPin(GetKernelIndex(), PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	const TSharedPtr<const FPCGDataCollectionDesc> InputDesc = InBinding->ComputeKernelPinDataDesc(InputKernelPin);

	TSharedPtr<FPCGDataCollectionDesc> OutDataDesc = FPCGDataCollectionDesc::MakeSharedFrom(InputDesc);

	// Add output attribute (selected mesh).
	{
		const UPCGStaticMeshSpawnerSettings* SMSettings = CastChecked<UPCGStaticMeshSpawnerSettings>(GetSettings());

		TArray<int32> UniqueStringKeys;

		// Create unique value keys for the output string.
		if (UPCGMeshSelectorWeighted* SelectorWeighted = Cast<UPCGMeshSelectorWeighted>(SMSettings->MeshSelectorParameters))
		{
			// Weighted selection - add explicit strings from settings.
			for (const FPCGMeshSelectorWeightedEntry& Entry : SelectorWeighted->MeshEntries)
			{
				const FString Value = Entry.Descriptor.StaticMesh.ToString();
				const int32 StringIndex = InBinding->GetStringTable().IndexOfByKey(Value);
				if (ensureAlways(StringIndex != INDEX_NONE))
				{
					UniqueStringKeys.AddUnique(StringIndex);
				}
			}

			OutDataDesc->AddAttributeToAllData(FPCGKernelAttributeKey(SMSettings->OutAttributeName, EPCGKernelAttributeType::StringKey), InBinding, &UniqueStringKeys);
		}
		else if (UPCGMeshSelectorByAttribute* SelectorByAttribute = Cast<UPCGMeshSelectorByAttribute>(SMSettings->MeshSelectorParameters))
		{
			// By-attribute selection - pass on strings from input attribute.
			if (SelectorByAttribute->AttributeName != NAME_None)
			{
				TArray<int32> StringKeys;

				for (FPCGDataDesc& DataDesc : OutDataDesc->GetDataDescriptionsMutable())
				{
					StringKeys.Empty();

					for (const FPCGKernelAttributeDesc& AttrDesc : DataDesc.GetAttributeDescriptions())
					{
						if (AttrDesc.GetAttributeKey().GetIdentifier().Name == SelectorByAttribute->AttributeName)
						{
							StringKeys = AttrDesc.GetUniqueStringKeys();
							break;
						}
					}

					for (int32 StringKey : StringKeys)
					{
						UniqueStringKeys.AddUnique(StringKey);
					}

					DataDesc.AddAttribute(FPCGKernelAttributeKey(SMSettings->OutAttributeName, EPCGKernelAttributeType::StringKey), InBinding, &StringKeys);
				}
			}
		}
		else if (UPCGMeshSelectorPrimitiveData* SelectorPrimitiveData = Cast<UPCGMeshSelectorPrimitiveData>(SMSettings->MeshSelectorParameters))
		{
			TSharedPtr<FPCGContextHandle> ContextHandle = InBinding->GetContextHandle().Pin();
			FPCGContext* Context = ContextHandle ? ContextHandle->GetContext() : nullptr;

			// Collect output string keys from primitive table.
			const int32 PrimitiveTableIndex = InBinding->GetFirstInputDataIndex(this, PCGStaticMeshSpawnerKernelConstants::PrimitiveTablePinLabel);
			const bool bIndexValid = InBinding->GetInputDataCollection().TaggedData.IsValidIndex(PrimitiveTableIndex);
			const UPCGParamData* PrimitiveData = bIndexValid ? Cast<UPCGParamData>(InBinding->GetInputDataCollection().TaggedData[PrimitiveTableIndex].Data) : nullptr;
			const UPCGMetadata* PrimitiveMetadata = PrimitiveData ? PrimitiveData->ConstMetadata() : nullptr;

			if (!PrimitiveMetadata)
			{
				PCG_KERNEL_VALIDATION_ERR(Context, SMSettings, LOCTEXT("NoPrimitiveData", "No primitive table data found, no primitives will spawn."));
				return OutDataDesc;
			}

			const FPCGMetadataAttributeBase* MeshAttributeBase = PrimitiveMetadata->GetConstAttribute(SelectorPrimitiveData->MeshAttribute);
			const bool bIsString = MeshAttributeBase && MeshAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FString>::Id;
			const bool bIsSoftObjectPath = MeshAttributeBase && MeshAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FSoftObjectPath>::Id;
			if (!MeshAttributeBase || (!bIsString && !bIsSoftObjectPath))
			{
				PCG_KERNEL_VALIDATION_WARN(Context, SMSettings,
					FText::Format(LOCTEXT("MeshAttributeInvalid", "Mesh attribute {0} not found in primitive table data, or is not of required type Soft Object Path or String."), FText::FromName(SelectorPrimitiveData->MeshAttribute)));
				return OutDataDesc;
			}

			const FPCGMetadataAttribute<FString>* MeshAttributeString = bIsString ? static_cast<const FPCGMetadataAttribute<FString>*>(MeshAttributeBase) : nullptr;
			const FPCGMetadataAttribute<FSoftObjectPath>* MeshAttributeSoftObjectPath = bIsSoftObjectPath ? static_cast<const FPCGMetadataAttribute<FSoftObjectPath>*>(MeshAttributeBase) : nullptr;

			const int32 NumPrimitiveDataElements = PrimitiveMetadata->GetItemCountForChild();

			for (int32 TableIndex = 0; TableIndex < NumPrimitiveDataElements; ++TableIndex)
			{
				const PCGMetadataValueKey ValueKey = MeshAttributeString ? MeshAttributeString->GetValueKey(TableIndex) : MeshAttributeSoftObjectPath->GetValueKey(TableIndex);
				const FString MeshPath = MeshAttributeString ? MeshAttributeString->GetValue(TableIndex) : MeshAttributeSoftObjectPath->GetValue(TableIndex).ToString();
				const int32 StringKey = InBinding->GetStringTable().IndexOfByKey(MeshPath);
				UniqueStringKeys.AddUnique(StringKey);
			}

			for (FPCGDataDesc& DataDesc : OutDataDesc->GetDataDescriptionsMutable())
			{
				DataDesc.AddAttribute(FPCGKernelAttributeKey(SMSettings->OutAttributeName, EPCGKernelAttributeType::StringKey), InBinding, &UniqueStringKeys);
			}
		}
		else if (SMSettings->MeshSelectorParameters)
		{
			UE_LOGF(LogPCG, Error, "Mesh selector not supported by GPU Static Mesh Spawner: %ls", *SMSettings->MeshSelectorParameters->GetName());
		}

		// Allocate bounds if the spawner will write more than one unique value for the mesh bounds.
		if (SMSettings->bApplyMeshBoundsToPoints && UniqueStringKeys.Num() > 1)
		{
			OutDataDesc->AllocatePropertiesForAllData(EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax);
		}
	}

	return OutDataDesc;
}

int UPCGStaticMeshSpawnerKernel::ComputeThreadCount(const UPCGDataBinding* InBinding) const
{
	const TSharedPtr<const FPCGDataCollectionDesc> InputPinDesc = InBinding->GetCachedKernelPinDataDesc(this, PCGPinConstants::DefaultInputLabel, /*bIsInput=*/true);
	return ensure(InputPinDesc) ? InputPinDesc->ComputeTotalElementCount() : 0;
}

#if WITH_EDITOR
FString UPCGStaticMeshSpawnerKernel::GetCookedSource(FPCGGPUCompilationContext& InOutContext) const
{
	FString SourceFile;
	ensure(LoadShaderSourceFile(GetSourceFilePath(), EShaderPlatform::SP_PCD3D_SM5, &SourceFile, nullptr));

	TMap<FString, FStringFormatArg> Defines =
	{
		{ TEXT("ProduceOutputPoints"), GetProduceOutputPoints() ? 1 : 0 },
	};

	return FString::Format(*SourceFile, Defines);
}

void UPCGStaticMeshSpawnerKernel::CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalInputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	TObjectPtr<UPCGStaticMeshSpawnerDataInterface> NodeDI = InOutContext.NewObject_AnyThread<UPCGStaticMeshSpawnerDataInterface>(InObjectOuter);
	NodeDI->SetProducerKernel(this);
	OutDataInterfaces.Add(NodeDI);
}

void UPCGStaticMeshSpawnerKernel::CreateAdditionalOutputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const
{
	Super::CreateAdditionalOutputDataInterfaces(InOutContext, InObjectOuter, OutDataInterfaces);

	UPCGInstanceDataInterface* InstanceDI = InOutContext.NewObject_AnyThread<UPCGInstanceDataInterface>(InObjectOuter);
	InstanceDI->SetProducerKernel(this);
	InstanceDI->SetInputPinProvidingData(PCGPinConstants::DefaultInputLabel);
	InstanceDI->SetCullingCellExtent(GetTargetCullingGridSize());
	OutDataInterfaces.Add(InstanceDI);
}

bool UPCGStaticMeshSpawnerKernel::UploadInputPinDataToGPU(FName InPinLabel) const
{
	// Primitive table is not actually needed on GPU, is only used statically on CPU for scheduling and primitive setup.
	return InPinLabel != PCGStaticMeshSpawnerKernelConstants::PrimitiveTablePinLabel;
}
#endif

void UPCGStaticMeshSpawnerKernel::AddStaticCreatedStrings(TArray<FString>& InOutStringTable) const
{
	const UPCGStaticMeshSpawnerSettings* SMSettings = CastChecked<UPCGStaticMeshSpawnerSettings>(GetSettings());
	if (UPCGMeshSelectorWeighted* SelectorWeighted = Cast<UPCGMeshSelectorWeighted>(SMSettings->MeshSelectorParameters))
	{
		for (const FPCGMeshSelectorWeightedEntry& Entry : SelectorWeighted->MeshEntries)
		{
			InOutStringTable.AddUnique(Entry.Descriptor.StaticMesh.ToString());
		}
	}
}

void UPCGStaticMeshSpawnerKernel::GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const
{
	// Register the attribute this node creates.
	const UPCGStaticMeshSpawnerSettings* SMSettings = CastChecked<UPCGStaticMeshSpawnerSettings>(GetSettings());
	OutKeys.AddUnique(FPCGKernelAttributeKey(SMSettings->OutAttributeName, EPCGKernelAttributeType::StringKey));
}

void UPCGStaticMeshSpawnerKernel::GetInputPins(TArray<FPCGPinProperties>& OutPins) const
{
	OutPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);

	// Used to receive analysis data when using by-attribute spawning.
	OutPins.Emplace(PCGStaticMeshSpawnerKernelConstants::InstanceCountsPinLabel, bUseRawInstanceCountsBuffer ? FPCGDataTypeInfoRawBuffer::AsId() : FPCGDataTypeInfoParam::AsId());

	// Used when spawning from primitive table. This data is not actually used by the kernel/on the GPU.
	OutPins.Emplace(PCGStaticMeshSpawnerKernelConstants::PrimitiveTablePinLabel, FPCGDataTypeInfoParam::AsId());

	// Used to receive per-culling-cell min/max position data.
	OutPins.Emplace(PCGStaticMeshSpawnerKernelConstants::CullingCellMinMaxPositionsPinLabel, FPCGDataTypeInfoRawBuffer::AsId());
}

void UPCGStaticMeshSpawnerKernel::GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const
{
	if (GetProduceOutputPoints())
	{
		OutPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
	}
}

#if WITH_EDITOR
bool UPCGStaticMeshSpawnerKernel::PerformStaticValidation()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGStaticMeshSpawnerKernel::PerformStaticValidation);

	if (!Super::PerformStaticValidation())
	{
		return false;
	}

	const UPCGStaticMeshSpawnerSettings* SMSettings = CastChecked<UPCGStaticMeshSpawnerSettings>(GetSettings());

	if (UPCGMeshSelectorWeighted* SelectorWeighted = Cast<UPCGMeshSelectorWeighted>(SMSettings->MeshSelectorParameters))
	{
		if (SelectorWeighted->MeshEntries.IsEmpty())
		{
#if PCG_KERNEL_LOGGING_ENABLED
			AddStaticLogEntry(PCGStaticMeshSpawnerKernel::NoMeshEntriesFormat, EPCGKernelLogVerbosity::Error);
#endif
			return false;
		}

		for (const FPCGMeshSelectorWeightedEntry& Entry : SelectorWeighted->MeshEntries)
		{
			if (Entry.Descriptor.StaticMesh.IsNull())
			{
#if PCG_KERNEL_LOGGING_ENABLED
				AddStaticLogEntry(LOCTEXT("UnassignedMesh", "Unassigned mesh."), EPCGKernelLogVerbosity::Error);
#endif
				return false;
			}
		}
	}
	else if (!SMSettings->MeshSelectorParameters || (!SMSettings->MeshSelectorParameters->IsA<UPCGMeshSelectorByAttribute>() && !SMSettings->MeshSelectorParameters->IsA<UPCGMeshSelectorPrimitiveData>()))
	{
#if PCG_KERNEL_LOGGING_ENABLED
		AddStaticLogEntry(LOCTEXT("InvalidMeshSelector", "Currently GPU Static Mesh Spawner nodes must use PCGMeshSelectorWeighted or UPCGMeshSelectorByAttribute or UPCGMeshSelectorPrimitiveData as the mesh selector type."), EPCGKernelLogVerbosity::Error);
#endif
		return false;
	}

	// Currently instance packers must be able to specify a full list of attribute names upfront, to build the attribute table at compile time.
	// TODO: We should be able to augment a static attribute table with new attributes at execution time, which will allow other types like regex.
	if (SMSettings->InstanceDataPackerParameters && !SMSettings->InstanceDataPackerParameters->GetAttributeNames(/*OutNames=*/nullptr))
	{
#if PCG_KERNEL_LOGGING_ENABLED
		AddStaticLogEntry(LOCTEXT("InvalidInstancePacker", "Selected instance packer does not support GPU execution."), EPCGKernelLogVerbosity::Error);
#endif
		return false;
	}
	
	return true;
}
#endif

#undef LOCTEXT_NAMESPACE
