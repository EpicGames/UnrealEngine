// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSurfaceSampler.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGCustomVersion.h"
#include "PCGData.h"
#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGPin.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSurfaceData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "HAL/UnrealMemory.h"
#include "Math/RandomStream.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSurfaceSampler)

#define LOCTEXT_NAMESPACE "PCGSurfaceSamplerElement"

namespace PCGSurfaceSampler
{
	void FSurfaceSamplerParams::InitializeFromSettings(const UPCGSurfaceSamplerSettings* Settings)
	{
		check(Settings);
		// Compute used values
		bUseLegacyGridCreationMethod = Settings->bUseLegacyGridCreationMethod;
		PointsPerSquaredMeter = Settings->PointsPerSquaredMeter;
		PointExtents = Settings->PointExtents;
		Looseness = Settings->Looseness;
		bApplyDensityToPoints = Settings->bApplyDensityToPoints;
		PointSteepness = Settings->PointSteepness;
#if WITH_EDITOR
		bKeepZeroDensityPoints = Settings->bKeepZeroDensityPoints;
#endif
	}

	bool FSurfaceSamplerData::Initialize(const UPCGSurfaceSamplerSettings* Settings, const FPCGContext* Context, const FBox& InEffectiveGridBounds, const FTransform& InSurfaceTransform)
	{
		Params.InitializeFromSettings(Settings);
		return Initialize(Context, InEffectiveGridBounds, InSurfaceTransform);
	}

	bool FSurfaceSamplerData::Initialize(const FPCGContext* Context, const FBox& InEffectiveGridBounds, const FTransform& InSurfaceTransform)
	{
		if (!InEffectiveGridBounds.IsValid)
		{
			return false;
		}

		Seed = Context ? Context->GetSeed() : PCGValueConstants::DefaultSeed;

		if (Params.PointExtents.X <= 0 || Params.PointExtents.Y <= 0)
		{
			// PointExtents and Looseness are user overridable, if any of those values are 0 or negative, it's invalid, so we early out.
			PCGLog::LogWarningOnGraph(LOCTEXT("InvalidParametersExtents", "Skipped - Extents are negative or zero."), Context);
			return false;
		}

		if (!Params.bUseLegacyGridCreationMethod)
		{
			// Points per squared meter -> inverse is meters squared by point.
			const FVector::FReal SquaredUnitsPerPoint = Params.PointsPerSquaredMeter > 0 ? (100.0 * 100.0) / Params.PointsPerSquaredMeter : UE_DOUBLE_BIG_NUMBER;

			// Compute approximate cell size based on the point extents relative size.
			// Knowing that X * Y = SQM
			// if X = aY, then aY * Y = SQM -> aY^2 = SQM -> Y = sqrt(SQM / a)
			const FVector::FReal MinCellSize = 2.0 * FMath::Min(Params.PointExtents.X, Params.PointExtents.Y);
			const FVector::FReal MaxCellSize = 2.0 * FMath::Max(Params.PointExtents.X, Params.PointExtents.Y);

			const FVector::FReal BaseCellSize = FMath::Sqrt(SquaredUnitsPerPoint / (MaxCellSize / MinCellSize));

			CellSize = FVector(
				BaseCellSize * 2.0 * (Params.PointExtents.X / MinCellSize),
				BaseCellSize * 2.0 * (Params.PointExtents.Y / MinCellSize),
				2.0 * Params.PointExtents.Z);

			InterstitialDistance = Params.PointExtents * 2;
			
			FVector CellRemainder = CellSize - InterstitialDistance;

			// Enforce that the grid is at least the provided extents.
			if (CellRemainder.X < 0 || CellRemainder.Y < 0)
			{
				CellSize.X = FMath::Max(CellSize.X, InterstitialDistance.X);
				CellSize.Y = FMath::Max(CellSize.Y, InterstitialDistance.Y);
				CellRemainder = CellSize - InterstitialDistance;
				check(CellRemainder.X >= -UE_DOUBLE_SMALL_NUMBER && CellRemainder.Y >= -UE_DOUBLE_SMALL_NUMBER);
			}

			const FVector::FReal ClampedLooseness = FMath::Clamp(Params.Looseness, 0.0, 1.0);
			InnerCellSize = CellRemainder * ClampedLooseness;
			InnerCellOffset = CellRemainder * 0.5 * (1.0 - ClampedLooseness);
		}
		else
		{
			// Legacy grid creation - Conceptually, we will break down the surface bounds in a N x M grid, where the cells are extents * (1 + steepness).
			InterstitialDistance = Params.PointExtents * 2;
			InnerCellSize = InterstitialDistance * Params.Looseness;
			InnerCellOffset = FVector::ZeroVector;
			CellSize = InterstitialDistance + InnerCellSize;
		}

		if (CellSize.X <= 0 || CellSize.Y <= 0)
		{
			// PointExtents and Looseness are user overridable, if any of those values are 0 or negative, it's invalid, so we early out.
			PCGLog::LogWarningOnGraph(LOCTEXT("InvalidParameters", "Skipped - Extents and/or Looseness are negative or zero."), Context);
			return false;
		}

		// By using scaled indices in the world, we can easily make this process deterministic
		CellMinX = FMath::CeilToInt((InEffectiveGridBounds.Min.X) / CellSize.X);
		CellMaxX = FMath::FloorToInt((InEffectiveGridBounds.Max.X) / CellSize.X);
		CellMinY = FMath::CeilToInt((InEffectiveGridBounds.Min.Y) / CellSize.Y);
		CellMaxY = FMath::FloorToInt((InEffectiveGridBounds.Max.Y) / CellSize.Y);

		{
			const int64 CellCountX = 1 + CellMaxX - CellMinX;
			const int64 CellCountY = 1 + CellMaxY - CellMinY;
			if (CellCountX <= 0 || CellCountY <= 0)
			{
				if (Context)
				{
					PCGE_LOG_C(Verbose, LogOnly, Context, FText::Format(LOCTEXT("InvalidCellBounds", "Skipped - invalid cell bounds({0} x {1})"), CellCountX, CellCountY));
				}

				return false;
			}

			const int64 CellCount64 = CellCountX * CellCountY;
			if (CellCount64 <= 0 || CellCount64 >= MAX_int32)
			{
				PCGLog::LogErrorOnGraph(FText::Format((LOCTEXT("InvalidCellCount", "Skipped - tried to generate too many points ({0}).")), CellCount64), Context);
				return false;
			}

			CellCount = static_cast<int32>(CellCount64);
		}

		check(CellCount > 0);

		FVector::FReal TargetPointCount = 0;

		if (!Params.bUseLegacyGridCreationMethod)
		{
			TargetPointCount = (InEffectiveGridBounds.Max.X - InEffectiveGridBounds.Min.X) * (InEffectiveGridBounds.Max.Y - InEffectiveGridBounds.Min.Y);
			Ratio = 1.0;
		}
		else
		{
			constexpr FVector::FReal InvSquaredMeterUnits = 1.0 / (100.0 * 100.0);
			TargetPointCount = (InEffectiveGridBounds.Max.X - InEffectiveGridBounds.Min.X) * (InEffectiveGridBounds.Max.Y - InEffectiveGridBounds.Min.Y) * Params.PointsPerSquaredMeter * InvSquaredMeterUnits;
			Ratio = static_cast<float>(FMath::Clamp(TargetPointCount / (FVector::FReal)CellCount, 0.0, 1.0));
		}

		if (Ratio < UE_SMALL_NUMBER)
		{
			if (Context)
			{
				PCGE_LOG_C(Verbose, LogOnly, Context, LOCTEXT("NoPointsFromDensity", "Skipped - density yields no points"));
			}

			return false;
		}

		if (PCGFeatureSwitches::CVarCheckSamplerMemory.GetValueOnAnyThread()
			&& PCGFeatureSwitches::Helpers::GetAvailableMemoryForSamplers() < (sizeof(FPCGPoint) * CellCount))
		{
			PCGLog::LogErrorOnGraph(FText::Format((LOCTEXT("TooManyPoints", "Skipped - tried to generate too many points ({0}).\nAdjust 'pcg.SamplerMemoryThreshold' if needed.")), CellCount), Context);
			return false;
		}

		// Local transformation is only needed if we're rotating.
		bNeedsLocalTransformation = !InSurfaceTransform.Rotator().IsNearlyZero();
		if (bNeedsLocalTransformation)
		{
			// Build the semi-local transform matrix for transforming points to the pre-projection plane
			const FTransform TranslationTransform(InSurfaceTransform.GetTranslation());
			PreProjectionTransform = TranslationTransform.Inverse().ToMatrixNoScale();
			// Find the rotation between the world normal and the surface and apply to the matrix
			PreProjectionTransform *= FQuat::FindBetweenNormals(FVector::UpVector, InSurfaceTransform.GetRotation().GetUpVector().GetSafeNormal()).ToMatrix();
			PreProjectionTransform *= TranslationTransform.ToMatrixNoScale();
		}

		// Drop points slightly by an epsilon otherwise point can be culled. If the sampler has a volume connected as the Bounding Shape,
		// the volume will call through to PCGHelpers::IsInsideBounds() which is a one sided test and points at the top of the volume
		// will fail it. TODO perhaps the one-sided check can be isolated to component-bounds
		constexpr FVector::FReal DefaultHeightModifier = 1.0 - UE_DOUBLE_SMALL_NUMBER;
		// Try to use a multiplier instead of a simply offset to combat loss of precision in floats. However if MaxZ is very small,
		// then multiplier will not work, so just use an offset.
		PreProjectionDisplacement = (FMath::Abs(InEffectiveGridBounds.Max.Z) > UE_DOUBLE_SMALL_NUMBER) ? InEffectiveGridBounds.Max.Z * DefaultHeightModifier : -UE_DOUBLE_SMALL_NUMBER;
		// Make sure we're still in bounds though!
		PreProjectionDisplacement = FMath::Max(PreProjectionDisplacement, InEffectiveGridBounds.Min.Z);

		return true;
	}

	FIntVector2 FSurfaceSamplerData::ComputeCellIndices(int32 Index) const
	{
		check(Index >= 0 && Index < CellCount);
		const int32 CellCountX = 1 + CellMaxX - CellMinX;

		return FIntVector2(CellMinX + (Index % CellCountX), CellMinY + (Index / CellCountX));
	}

	UPCGBasePointData* SampleSurface(FPCGContext* Context, const FSurfaceSamplerParams& ExecutionParams, const UPCGSurfaceData* InSurface, const UPCGSpatialData* InBoundingShape, const FBox& EffectiveBounds, TSubclassOf<UPCGBasePointData> PointDataClass)
	{
		UPCGBasePointData* SampledData = PointDataClass != nullptr ? FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass) : FPCGContext::NewPointData_AnyThread(Context);
		SampledData->InitializeFromData(InSurface);

		FSurfaceSamplerData SamplerData;
		SamplerData.Params = ExecutionParams;
		SamplerData.Initialize(Context, EffectiveBounds);
		// We don't support time slicing here
		SampleSurface(Context, SamplerData, InSurface, InBoundingShape, SampledData, /*bTimeSlicingIsEnabled=*/false);

		return SampledData;
	}

	UPCGPointData* SampleSurface(FPCGContext* Context, const UPCGSurfaceData* InSurface, const UPCGSpatialData* InBoundingShape, const FBox& EffectiveBounds, const FSurfaceSamplerParams& ExecutionParams)
	{
		return CastChecked<UPCGPointData>(SampleSurface(Context, ExecutionParams, InSurface, InBoundingShape, EffectiveBounds, UPCGPointData::StaticClass()));
	}

	bool SampleSurface(FPCGContext* Context, const FSurfaceSamplerData& SamplerData, const UPCGSurfaceData* InSurface, const UPCGSpatialData* InBoundingShape, UPCGBasePointData* SampledData, const bool bTimeSlicingIsEnabled)
	{
		check(InSurface && SampledData);
		const FPCGProjectionParams ProjectionParams{};

		// Cache pointer ahead of time to avoid dereferencing object pointer which does access tracking and supports lazy loading, and can come with substantial
		// overhead (add trace marker to FObjectPtr::Get to see).
		UPCGMetadata* OutMetadata = SampledData->Metadata.Get();
				
		auto InitializeFunc = [&SamplerData, SampledData]()
		{
			SampledData->SetNumPoints(SamplerData.CellCount, /*bInitializeValues=*/false);
			SampledData->SetSteepness(SamplerData.Params.PointSteepness);

			FPCGPoint DefaultPoint;
			PCGPointHelpers::SetExtents(SamplerData.Params.PointExtents, DefaultPoint.BoundsMin, DefaultPoint.BoundsMax);

			SampledData->SetBoundsMin(DefaultPoint.BoundsMin);
			SampledData->SetBoundsMax(DefaultPoint.BoundsMax);

			EPCGPointNativeProperties PropertiesToAllocate = EPCGPointNativeProperties::Transform
				| EPCGPointNativeProperties::Density
				| EPCGPointNativeProperties::Seed
				| EPCGPointNativeProperties::MetadataEntry
				| EPCGPointNativeProperties::Color;

			SampledData->AllocateProperties(PropertiesToAllocate);
		};

		auto AsyncProcessRangeFunc = [&SamplerData, InBoundingShape, InSurface, &ProjectionParams, OutMetadata, SampledData](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			TPCGValueRange<FTransform> TransformRange = SampledData->GetTransformValueRange(/*bAllocate=*/false);
			TPCGValueRange<float> DensityRange = SampledData->GetDensityValueRange(/*bAllocate=*/false);
			TPCGValueRange<int32> SeedRange = SampledData->GetSeedValueRange(/*bAllocate=*/false);
			TPCGValueRange<int64> MetadataEntryRange = SampledData->GetMetadataEntryValueRange(/*bAllocate=*/false);
			TPCGValueRange<FVector4> ColorRange = SampledData->GetColorValueRange(/*bAllocate=*/false);

			int32 NumWritten = 0;

			for(int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex+Count; ++ReadIndex)
			{
				const int32 WriteIndex = StartWriteIndex + NumWritten;

				const FSurfaceSamplerParams& SamplerParams = SamplerData.Params;
				const FIntVector2 Indices = SamplerData.ComputeCellIndices(ReadIndex);

				const FVector::FReal CurrentX = Indices.X * SamplerData.CellSize.X;
				const FVector::FReal CurrentY = Indices.Y * SamplerData.CellSize.Y;
				const FVector InnerCellOffset = SamplerData.InnerCellOffset;
				const FVector InnerCellSize = SamplerData.InnerCellSize;

				FRandomStream RandomSource(PCGHelpers::ComputeSeed(SamplerData.Seed, Indices.X, Indices.Y));
				const float Chance = RandomSource.FRand();

				const float Ratio = SamplerData.Ratio;

				if (Chance >= Ratio)
				{
					continue;
				}

				const float RandX = RandomSource.FRand();
				const float RandY = RandomSource.FRand();

				FVector TentativeLocation = FVector(
					CurrentX + InnerCellOffset.X + RandX * InnerCellSize.X, 
					CurrentY + InnerCellOffset.Y + RandY * InnerCellSize.Y, 
					SamplerData.PreProjectionDisplacement);

				// If pre-projected points need a local transformation (ex. World Ray Hit Query) and not default to -Z
				if (SamplerData.bNeedsLocalTransformation)
				{
					// Transform the pre-projected sample point around the surface's origin from local to world space
					TentativeLocation = SamplerData.PreProjectionTransform.TransformPosition(TentativeLocation);
				}

				const FBox LocalBound(-SamplerParams.PointExtents, SamplerParams.PointExtents);

				// The output at this point is not initialized
				FPCGPoint OutPoint = FPCGPoint();

				// Firstly project onto elected generating shape to move to final position.
				if (!InSurface->ProjectPoint(FTransform(TentativeLocation), LocalBound, ProjectionParams, OutPoint, OutMetadata))
				{
					continue;
				}

				// Set physical properties that are needed for the bounding shape checks, etc.
				OutPoint.SetExtents(SamplerParams.PointExtents);
				OutPoint.Steepness = SamplerParams.PointSteepness;

				// Set default density
				DensityRange[WriteIndex] = 1.0f;

				// Now run gauntlet of shape network (if there is one) to accept or reject the point.
				if (InBoundingShape)
				{
					FPCGPoint BoundingShapeSample;
#if WITH_EDITOR
					if (!InBoundingShape->SamplePoint(OutPoint.Transform, OutPoint.GetLocalBounds(), BoundingShapeSample, nullptr) && !SamplerParams.bKeepZeroDensityPoints)
#else
					if (!InBoundingShape->SamplePoint(OutPoint.Transform, OutPoint.GetLocalBounds(), BoundingShapeSample, nullptr))
#endif
					{
						continue;
					}

					// Produce smooth density field
					DensityRange[WriteIndex] *= BoundingShapeSample.Density;
				}

				// Apply final parameters on the point
				if (SamplerParams.bApplyDensityToPoints)
				{
					DensityRange[WriteIndex] *= (SamplerParams.bApplyDensityToPoints ? ((Ratio - Chance) / Ratio) : 1.0f);
				}

				SeedRange[WriteIndex] = RandomSource.GetCurrentSeed();
				TransformRange[WriteIndex] = OutPoint.Transform;
				MetadataEntryRange[WriteIndex] = OutPoint.MetadataEntry;
				ColorRange[WriteIndex] = OutPoint.Color;
				++NumWritten;
			}

			return NumWritten;
		};

		auto MoveDataRangeFunc = [SampledData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
		{
			SampledData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		};

		auto FinishedFunc = [SampledData](int32 NumWritten)
		{
			SampledData->SetNumPoints(NumWritten);
		};

		FPCGAsyncState* AsyncState = Context ? &Context->AsyncState : nullptr;
		const bool bDone = FPCGAsync::AsyncProcessingRangeEx(
			AsyncState, 
			SamplerData.CellCount,
			InitializeFunc,
			AsyncProcessRangeFunc,
			MoveDataRangeFunc,
			FinishedFunc,
			/*bEnableTimeSlicing=*/Context && bTimeSlicingIsEnabled);

		return bDone;
	}

#if WITH_EDITOR
	static bool IsPinOnlyConnectedToInputNode(UPCGPin* DownstreamPin, UPCGNode* GraphInputNode)
	{
		if (DownstreamPin->Edges.Num() == 1)
		{
			const UPCGEdge* Edge = DownstreamPin->Edges[0];
			const UPCGNode* UpstreamNode = (Edge && Edge->InputPin) ? Edge->InputPin->Node : nullptr;
			const bool bConnectedToInputNode = UpstreamNode && (GraphInputNode == UpstreamNode);
			const bool bConnectedToInputPin = Edge && (Edge->InputPin->Properties.Label == FName(TEXT("In")) || Edge->InputPin->Properties.Label == FName(TEXT("Input")));
			return bConnectedToInputNode && bConnectedToInputPin;
		}

		return false;
	}
#endif
}

UPCGSurfaceSamplerSettings::UPCGSurfaceSamplerSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		PointSteepness = 1.0f;
	}
}

#if WITH_EDITOR
FText UPCGSurfaceSamplerSettings::GetNodeTooltipText() const
{
	return LOCTEXT("SurfaceSamplerNodeTooltip", "Generates points in two dimensional domain that sample the Surface input and lie within the Bounding Shape input.");
}
#endif

TArray<FPCGPinProperties> UPCGSurfaceSamplerSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	FPCGPinProperties& SurfacePinProperty = PinProperties.Emplace_GetRef(PCGSurfaceSamplerConstants::SurfaceLabel, EPCGDataType::Surface, /*bAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true, LOCTEXT("SurfaceSamplerSurfacePinTooltip",
		"The surface to sample with points. Points will be generated in the two dimensional footprint of the combined bounds of the Surface and the Bounding Shape (if any) "
		"and then projected onto this surface. If this input is omitted then the network of shapes connected to the Bounding Shape pin will be inspected for a surface "
		"shape to use to project the points onto."
	));
	SurfacePinProperty.SetRequiredPin();

	// Only one connection/data allowed. To avoid ambiguity, samplers should require users to union or intersect multiple shapes.
	PinProperties.Emplace(PCGSurfaceSamplerConstants::BoundingShapeLabel, EPCGDataType::Spatial, /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false, LOCTEXT("SurfaceSamplerBoundingShapePinTooltip",
		"All sampled points must be contained within this shape. If this input is omitted then bounds will be taken from the actor so that points are contained within actor bounds. "
		"The Unbounded property disables this and instead generates over the entire bounds of Surface."
	));

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSurfaceSamplerSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);

	return PinProperties;
}

void UPCGSurfaceSamplerSettings::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGChangedSurfaceSamplerDefaultGridCreationMode)
	{
		// Default value has changed for the point extents from FVector(100.0f) to FVector(50.0f)
		PointExtents = FVector(100.0f);

		// Prior to this version all settings were using what we now consider the "legacy" grid creation scheme
		bUseLegacyGridCreationMethod = true;
	}

	Super::Serialize(Ar);
}

void UPCGSurfaceSamplerSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (PointRadius_DEPRECATED != 0)
	{
		PointExtents = FVector(PointRadius_DEPRECATED);
		PointRadius_DEPRECATED = 0;
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
bool UPCGSurfaceSamplerSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	return !bUnbounded || InPin->Properties.Label != PCGSurfaceSamplerConstants::BoundingShapeLabel;
}
#endif

FPCGElementPtr UPCGSurfaceSamplerSettings::CreateElement() const
{
	return MakeShared<FPCGSurfaceSamplerElement>();
}

namespace PCGSurfaceSamplerHelpers
{
	using ContextType = FPCGSurfaceSamplerElement::ContextType;
	using ExecStateType = FPCGSurfaceSamplerElement::ExecStateType;

	EPCGTimeSliceInitResult InitializePerExecutionData(ContextType* Context, ExecStateType& OutState)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSurfaceSamplerElement::InitializePerExecutionData);

		check(Context);
		const UPCGSurfaceSamplerSettings* Settings = Context->GetInputSettings<UPCGSurfaceSamplerSettings>();
		check(Settings);

		const TArray<FPCGTaggedData> SurfaceInputs = Context->InputData.GetInputsByPin(PCGSurfaceSamplerConstants::SurfaceLabel);
		// If there are no surfaces to sample, early out
		if (SurfaceInputs.IsEmpty())
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Early out on invalid settings
		// TODO: we could compute an approximate radius based on the points per squared meters if that's useful
		const FVector& PointExtents = Settings->PointExtents;
		if (PointExtents.X <= 0 || PointExtents.Y <= 0)
		{
			PCGE_LOG_C(Warning, GraphAndLog, Context, LOCTEXT("SkippedInvalidPointExtents", "Skipped - Invalid point extents"));
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		// Grab the Bounding Shape input if there is one.
		if (!Settings->bUnbounded)
		{
			bool bUnionWasCreated;
			OutState.BoundingShape = PCGSettingsHelpers::ComputeBoundingShape(Context, PCGSurfaceSamplerConstants::BoundingShapeLabel, bUnionWasCreated);
			if (OutState.BoundingShape)
			{
				if (bUnionWasCreated)
				{
					Context->TrackObject(OutState.BoundingShape);
				}

				OutState.BoundingShapeBounds = OutState.BoundingShape->GetBounds();
			}

			if (!OutState.BoundingShapeBounds.IsValid)
			{
				// The bounding shape bounds is invalid, such as an empty intersection, so no operation will need to be performed.
				return EPCGTimeSliceInitResult::NoOperation;
			}
		}
		else if (Context->InputData.GetInputsByPin(PCGSurfaceSamplerConstants::BoundingShapeLabel).Num() > 0)
		{
			PCGE_LOG_C(Verbose, LogOnly, Context, LOCTEXT("BoundsIgnored", "The bounds of the Bounding Shape input pin will be ignored because the Unbounded option is enabled."));
		}

		TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
		// Find the generating shapes to sample
		TArray<const UPCGSurfaceData*>& GeneratingShapes = OutState.GeneratingShapes;
		GeneratingShapes.Reserve(SurfaceInputs.Num());

		// Construct a list of shapes to generate samples from. Get these directly from the first input pin.
		for (const FPCGTaggedData& TaggedData : SurfaceInputs)
		{
			if (const UPCGSurfaceData* SurfaceData = Cast<UPCGSurfaceData>(TaggedData.Data))
			{
				GeneratingShapes.Add(SurfaceData);
				Outputs.Add(TaggedData);
			}
		}

		// If there are no generating shapes, early out
		if (GeneratingShapes.IsEmpty())
		{
			PCGE_LOG_C(Warning, GraphAndLog, Context, LOCTEXT("NoSurfaceFound", "No surfaces found from which to generate"));
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		return EPCGTimeSliceInitResult::Success;
	}
}

bool FPCGSurfaceSamplerElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSurfaceSamplerElement::PrepareDataInternal);
	ContextType* Context = static_cast<ContextType*>(InContext);
	check(Context);

	const UPCGSurfaceSamplerSettings* Settings = Context->GetInputSettings<UPCGSurfaceSamplerSettings>();

	// Initialize the per-execution state data that won't change over the duration of the time slicing
	if (Context->InitializePerExecutionState(PCGSurfaceSamplerHelpers::InitializePerExecutionData) == EPCGTimeSliceInitResult::AbortExecution)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("CouldNotInitializeExecutionState", "Could not initialize per-execution timeslice state data"));
		return true;
	}

	TArray<const UPCGSurfaceData*>& GeneratingShapes = Context->GetPerExecutionState().GeneratingShapes;
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	TArray<FPCGTaskId> PrepareTasks;

	// Initialize the per-iteration data, using the generating shapes as the source of iteration
	Context->InitializePerIterationStates(GeneratingShapes.Num(),
		[&GeneratingShapes, &Outputs, &Settings, &Context, &PrepareTasks](IterStateType& OutState, const ExecStateType& ExecState, const uint32 IterationIndex)
		{
			// If we have generating shape inputs, use them
			const UPCGSurfaceData* GeneratingShape = GeneratingShapes[IterationIndex];
			check(GeneratingShape);

			OutState.OutputData = FPCGContext::NewPointData_AnyThread(Context);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OutState.OutputPoints = Cast<UPCGPointData>(OutState.OutputData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			OutState.OutputData->InitializeFromData(GeneratingShape);

			// This bounds will be used to generate the pre-projected grid
			FBox EffectiveGridBounds = FBox(EForceInit::ForceInit);
			const FBox& BoundingShapeBounds = ExecState.BoundingShapeBounds;

			// The shape's local bounds is most ideal for generating the grid
			EffectiveGridBounds = GeneratingShape->GetLocalBounds();

			// If local bounds exists, apply the transform without rotation to get into grid sampling space
			if (EffectiveGridBounds.IsValid)
			{
				// Transport the box into semi-local 2D space, where we can get the deterministic grid samples
				EffectiveGridBounds = PCGHelpers::OverlapBounds(EffectiveGridBounds.TransformBy(GeneratingShape->GetTransform()), BoundingShapeBounds);
			}
			else // If no local bounds, try to use the generating shape's bounds
			{
				if (GeneratingShape->IsBounded())
				{
					EffectiveGridBounds = GeneratingShape->GetBounds();

					// If we're using the generating shape's bounds, we can further optimize by overlapping the bounding shape if it was provided
					if (BoundingShapeBounds.IsValid)
					{
						EffectiveGridBounds = PCGHelpers::OverlapBounds(EffectiveGridBounds, BoundingShapeBounds);
					}
				}
				else // If no local or world bounds, then finally try to use the bounding shape
				{
					EffectiveGridBounds = BoundingShapeBounds;
				}
			}

			if (!OutState.SamplerData.Initialize(Settings, Context, EffectiveGridBounds, GeneratingShape->GetTransform()))
			{
				if (!GeneratingShape->IsBounded() && !ExecState.BoundingShapeBounds.IsValid)
				{
					// Some inputs are unable to provide bounds, like the WorldRayHit, in which case the user must provide bounds.
					PCGE_LOG_C(Warning, GraphAndLog, Context, LOCTEXT("CouldNotObtainInputBounds", "Input data is not bounded, so bounds must be provided for sampling. Consider providing a Bounding Shape input."));
				}
				else if (!EffectiveGridBounds.IsValid)
				{
					PCGE_LOG_C(Verbose, LogOnly, Context, LOCTEXT("InvalidSamplingBounds", "Final sampling bounds is invalid/zero-sized."));
				}

				return EPCGTimeSliceInitResult::NoOperation;
			}

			FBox PrepareSpatialQueryBounds = EffectiveGridBounds;
			if (OutState.SamplerData.bNeedsLocalTransformation)
			{
				// Transform the pre-projected bounds around the surface's origin from local to world space
				PrepareSpatialQueryBounds = EffectiveGridBounds.TransformBy(OutState.SamplerData.PreProjectionTransform);
			}

			PrepareTasks.Append(GeneratingShape->PrepareForSpatialQuery(Context, PrepareSpatialQueryBounds));

			// Assigning this here prevents the need to root
			Outputs[IterationIndex].Data = OutState.OutputData;
			return EPCGTimeSliceInitResult::Success;
		});

	if (!Context->DataIsPreparedForExecution())
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("CouldNotInitializeIterationState", "Could not initialize per-iteration timeslice state data"));
	}

	if (!PrepareTasks.IsEmpty())
	{
		Context->bIsPaused = true;
		Context->DynamicDependencies.Append(PrepareTasks);
		// ExecuteInternal will early out on Context->bIsPaused, we still want to advance to the Execute phase
	}

	return true;
}

bool FPCGSurfaceSamplerElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSurfaceSamplerElement::Execute);
	ContextType* TimeSlicedContext = static_cast<ContextType*>(InContext);
	check(TimeSlicedContext);

	// Paused waiting on PrepareForSpatialQuery tasks
	if (TimeSlicedContext->bIsPaused)
	{
		return false;
	}

	// Prepare data failed, no need to execute. Return an empty output
	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		TimeSlicedContext->OutputData.TaggedData.Empty();

		return true;
	}

	// The execution would have resulted in an empty set of points for all iterations
	if (TimeSlicedContext->GetExecutionStateResult() == EPCGTimeSliceInitResult::NoOperation)
	{
		for (FPCGTaggedData& Input : TimeSlicedContext->InputData.GetInputsByPin(PCGSurfaceSamplerConstants::SurfaceLabel))
		{
			FPCGTaggedData& Output = TimeSlicedContext->OutputData.TaggedData.Add_GetRef(Input);
			UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(InContext);
			PointData->InitializeFromData(Cast<UPCGSpatialData>(Input.Data));
			Output.Data = PointData;
		}

		return true;
	}

	// The context will iterate over per-iteration states and execute the lambda until it returns true
	return ExecuteSlice(TimeSlicedContext, [](ContextType* Context, const ExecStateType& ExecState, const IterStateType& IterState, const uint32 IterationIndex)->bool
	{
		const EPCGTimeSliceInitResult InitResult = Context->GetIterationStateResult(IterationIndex);

		// This iteration resulted in an early out for no sampling operation. Early out with empty point data.
		if (InitResult == EPCGTimeSliceInitResult::NoOperation)
		{
			Context->OutputData.TaggedData[IterationIndex].Data = FPCGContext::NewPointData_AnyThread(Context);

			return true;
		}

		// It should be guaranteed to be a success at this point
		check(InitResult == EPCGTimeSliceInitResult::Success);

		// Run the execution until the time slice is finished
		const bool bAsyncDone = PCGSurfaceSampler::SampleSurface(Context, IterState.SamplerData, ExecState.GeneratingShapes[IterationIndex], ExecState.BoundingShape, IterState.OutputData, Context->TimeSliceIsEnabled());

		if (bAsyncDone)
		{
			PCGE_LOG_C(Verbose, LogOnly, Context, FText::Format(LOCTEXT("GenerationInfo", "Generated {0} points in {1} cells"), IterState.OutputData->GetNumPoints(), IterState.SamplerData.CellCount));
		}

		return bAsyncDone;
	});
}

void FPCGSurfaceSamplerElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	if (const UPCGSurfaceSamplerSettings* Settings = Cast<UPCGSurfaceSamplerSettings>(InParams.Settings))
	{
		bool bUnbounded;
		PCGSettingsHelpers::GetOverrideValue(*InParams.InputData, Settings, GET_MEMBER_NAME_CHECKED(UPCGSurfaceSamplerSettings, bUnbounded), Settings->bUnbounded, bUnbounded);
		const bool bBoundsConnected = InParams.InputData->GetInputsByPin(PCGSurfaceSamplerConstants::BoundingShapeLabel).Num() > 0;

		// If we're operating in bounded mode and there is no bounding shape connected then we'll use actor bounds, and therefore take
		// dependency on actor data.
		if (!bUnbounded && !bBoundsConnected && InParams.ExecutionSource)
		{
			if (const UPCGData* Data = InParams.ExecutionSource->GetExecutionState().GetSelfData())
			{
				Crc.Combine(Data->GetOrComputeCrc(/*bFullDataCrc=*/false));
			}
		}
	}

	OutCrc = Crc;
}

#if WITH_EDITOR
void UPCGSurfaceSamplerSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	if (DataVersion < FPCGCustomVersion::SplitSamplerNodesInputs && ensure(InOutNode))
	{
		if (InputPins.Num() > 0 && InputPins[0])
		{
			// The node will function the same if we move all connections from "In" to "Bounding Shape". To make this happen, rename "In" to
			// "Bounding Shape" just prior to pin update and the edges will be moved over. In ApplyDeprecation we'll see if we can do better than
			// this baseline functional setup.
			InputPins[0]->Properties.Label = PCGSurfaceSamplerConstants::BoundingShapeLabel;
		}

		// A new params pin was added, migrate the first param connection there if any
		PCGSettingsHelpers::DeprecationBreakOutParamsToNewPin(InOutNode, InputPins, OutputPins);
	}

	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);
}

void UPCGSurfaceSamplerSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	if (DataVersion < FPCGCustomVersion::SplitSamplerNodesInputs && ensure(InOutNode && InOutNode->GetInputPins().Num() >= 2))
	{
		UE_LOGF(LogPCG, Log, "Surface Sampler node migrated from an older version. Review edges on the input pins and then save this graph to upgrade the data.");

		UPCGPin* SurfacePin = InOutNode->GetInputPin(FName(TEXT("Surface")));
		UPCGPin* BoundingShapePin = InOutNode->GetInputPin(FName(TEXT("Bounding Shape")));
		UPCGNode* GraphInputNode = InOutNode->GetGraph() ? InOutNode->GetGraph()->GetInputNode() : nullptr;

		if (SurfacePin && BoundingShapePin && GraphInputNode)
		{
			auto MoveEdgeOnInputNodeToLandscapePin = [InOutNode, GraphInputNode, SurfacePin](UPCGPin* DownstreamPin) {
				// Detect if we're connected to the Input node.
				if (PCGSurfaceSampler::IsPinOnlyConnectedToInputNode(DownstreamPin, GraphInputNode))
				{
					// If we are connected to the Input node, make just a connection from the Surface pin to the Landscape pin and rely on Unbounded setting to provide bounds.
					if (UPCGPin* LandscapePin = GraphInputNode->GetOutputPin(FName(TEXT("Landscape"))))
					{
						DownstreamPin->BreakAllEdges();

						LandscapePin->AddEdgeTo(SurfacePin);
					}
				}
			};

			// The input pin has been split into two. Detect if we have inputs on only one pin and are dealing with older data - if so there's a good chance we can rewire
			// in a better way.
			if (SurfacePin->Edges.Num() == 0 && BoundingShapePin->Edges.Num() > 0)
			{
				MoveEdgeOnInputNodeToLandscapePin(BoundingShapePin);
			}
			else if (SurfacePin->Edges.Num() > 0 && BoundingShapePin->Edges.Num() == 0)
			{
				MoveEdgeOnInputNodeToLandscapePin(SurfacePin);
			}
		}
	}

	Super::ApplyDeprecation(InOutNode);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
