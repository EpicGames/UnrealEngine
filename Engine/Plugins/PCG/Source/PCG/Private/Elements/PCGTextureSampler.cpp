// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGTextureSampler.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGCrc.h"
#include "PCGModule.h"
#include "PCGSubsystem.h"
#include "Compute/PCGComputeCommon.h"
#include "Data/PCGRenderTargetData.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGTextureSampler)

#define LOCTEXT_NAMESPACE "PCGTextureSamplerElement"

#if WITH_EDITOR
void UPCGTextureSamplerSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGTextureSamplerSettings, Texture)) || Texture.IsNull())
	{
		// Dynamic tracking or null settings
		return;
	}

	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(Texture.ToSoftObjectPath());

	OutKeysToSettings.FindOrAdd(Key).Emplace(this, /*bCulling=*/false);
}

EPCGChangeType UPCGTextureSamplerSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGTextureSamplerSettings, Texture))
	{
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	return ChangeType;
}

void UPCGTextureSamplerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) 
{
	if (PropertyChangedEvent.Property)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGTextureSamplerSettings, Texture))
		{
			UpdateDisplayTextureArrayIndex();
		} 
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPCGTextureSamplerSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DensityFunction != EPCGTextureDensityFunction::Multiply)
	{
		bUseDensitySourceChannel = false;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	UpdateDisplayTextureArrayIndex();
}
#endif

#if WITH_EDITOR
FText UPCGTextureSamplerSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Generates points by sampling the given texture.\n"
		"If the texture is CPU-accessible, the sampler will prefer the CPU version of the texture.\n"
		"Otherwise, the texture will be read back from the GPU if one is present.");
}

TOptional<FPCGNodeThumbnailProxy> UPCGTextureSamplerSettings::GetNodeThumbnail() const
{
	if (Texture.IsNull())
	{
		return {};
	}

	return FPCGNodeThumbnailProxy::FromAssetPath(Texture.ToSoftObjectPath());
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGTextureSamplerSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	FPCGPinProperties& Pin = Properties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::BaseTexture);
	Pin.bAllowMultipleData = false;

	return Properties;
}

FPCGElementPtr UPCGTextureSamplerSettings::CreateElement() const
{
	return MakeShared<FPCGTextureSamplerElement>();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EPCGTextureDensityFunction UPCGTextureSamplerSettings::GetDensityFunctionEquivalent() const
{
	return bUseDensitySourceChannel ? EPCGTextureDensityFunction::Multiply : EPCGTextureDensityFunction::Ignore;
}

void UPCGTextureSamplerSettings::SetDensityFunctionEquivalent(EPCGTextureDensityFunction InDensityFunction)
{
	bUseDensitySourceChannel = (InDensityFunction != EPCGTextureDensityFunction::Ignore);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
void UPCGTextureSamplerSettings::UpdateDisplayTextureArrayIndex()
{
	UTexture* NewTexture = Texture.LoadSynchronous();
	bDisplayTextureArrayIndex = NewTexture && NewTexture->IsA<UTexture2DArray>();
}
#endif

void FPCGTextureSamplerContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	if (TextureData)
	{
		Collector.AddReferencedObject(TextureData);
	}
}

bool FPCGTextureSamplerElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTextureSamplerElement::Execute);

	FPCGTextureSamplerContext* Context = static_cast<FPCGTextureSamplerContext*>(InContext);
	check(Context);

	const UPCGTextureSamplerSettings* Settings = Context->GetInputSettings<UPCGTextureSamplerSettings>();
	check(Settings);

	if (Settings->Texture.IsNull())
	{
		return true;
	}

	if (!Context->WasLoadRequested())
	{
		return Context->RequestResourceLoad(Context, { Settings->Texture.ToSoftObjectPath() }, !Settings->bSynchronousLoad);
	}

	return true;
}

bool FPCGTextureSamplerElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTextureSamplerElement::Execute);
	check(InContext);

	if (InContext->bIsPaused)
	{
		return false;
	}

	if (!InContext->ExecutionSource.IsValid())
	{
		return true;
	}

	FPCGTextureSamplerContext* Context = static_cast<FPCGTextureSamplerContext*>(InContext);

	const UPCGTextureSamplerSettings* Settings = Context->GetInputSettings<UPCGTextureSamplerSettings>();
	check(Settings);

	if (Settings->Texture.IsNull())
	{
		return true;
	}

	UTexture* Texture = Settings->Texture.Get();

	if (!Texture)
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("CouldNotResolveTexture", "Texture at path '{0}' could not be loaded"), FText::FromString(Settings->Texture.ToString())));
		return true;
	}

	UTexture2DArray* TextureArray = Cast<UTexture2DArray>(Texture);
	uint32 TextureArrayIndex = 0;
	const bool bIsTextureArray = TextureArray != nullptr;

	if (bIsTextureArray)
	{
#if WITH_EDITOR
		const int32 ArraySize = TextureArray->SourceTextures.Num();
#else
		const int32 ArraySize = TextureArray->GetArraySize();
#endif

		if (Settings->TextureArrayIndex >= 0 && Settings->TextureArrayIndex < ArraySize)
		{
			TextureArrayIndex = Settings->TextureArrayIndex;
		}
		else
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InvalidTextureIndex", "Array index {0} was out of bounds for TextureArray at path '{1}'."), Settings->TextureArrayIndex, FText::FromString(Settings->Texture.ToString())));
			return true;
		}
	}
	else if (!Texture->IsA<UTexture2D>() && !Texture->IsA<UTextureRenderTarget2D>())
	{
		PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InvalidTextureType", "Texture at path '{0}' is not a valid type. Must be one of the following types: UTexture2D, UTexture2DArray, UTextureRenderTarget2D."), FText::FromString(Settings->Texture.ToString())));
		return true;
	}

	if (!Context->bTransformInitialized)
	{
		if (Settings->bUseAbsoluteTransform)
		{
			Context->Transform = Settings->Transform;
		}
		else
		{
			const FTransform Transform = Context->ExecutionSource->GetExecutionState().GetTransform();
			Context->Transform = Settings->Transform * Transform;

			const FBox Bounds = Context->ExecutionSource->GetExecutionState().GetLocalSpaceBounds();
			Context->Transform.SetScale3D(Context->Transform.GetScale3D() * Bounds.GetExtent());
		}

		Context->bTransformInitialized = true;
	}

	// The new texture data to add. Will be assigned the correct texture data based on the referenced texture.
	UPCGTexture2DSingleBaseData* BaseTextureData = nullptr;

	if (UTextureRenderTarget2D* RenderTarget = Cast<UTextureRenderTarget2D>(Texture))
	{
		UPCGRenderTargetData* RenderTargetData = FPCGContext::NewObject_AnyThread<UPCGRenderTargetData>(Context);
		RenderTargetData->Initialize(RenderTarget, Context->Transform);
		BaseTextureData = RenderTargetData;
	}
	else
	{
		// Texture data can take some frames to prepare, so we poll it once per frame until it is done.
		// TODO - review other similar cases and consider adding helpers/abstractions to support this.
		UPCGTextureData* TextureData = nullptr;
		if (Context->TextureData)
		{
			TextureData = Context->TextureData.Get();
		}
		else
		{
			TextureData = FPCGContext::NewObject_AnyThread<UPCGTextureData>(Context);
			Context->TextureData = TextureData;
		}

		if (!ensure(TextureData))
		{
			PCGE_LOG_C(Error, LogOnly, Context, LOCTEXT("TextureDataInitFailed", "Failed to initialize texture data."));
			return true;
		}

		if (!TextureData->IsSuccessfullyInitialized())
		{
#if WITH_EDITOR
			const bool bForceEditorOnlyCPUSampling = Settings->bForceEditorOnlyCPUSampling;
#else
			const bool bForceEditorOnlyCPUSampling = false;
#endif

			if (!TextureData->Initialize(Texture, TextureArrayIndex, Context->Transform, bForceEditorOnlyCPUSampling))
			{
				// Initialization not complete. Could be waiting on async texture processing or for GPU readback. Sleep until next frame.
				Context->bIsPaused = true;
				FPCGModule::GetPCGModuleChecked().ExecuteNextTick([ContextHandle = Context->GetOrCreateHandle()]()
					{
						FPCGContext::FSharedContext<FPCGTextureSamplerContext> SharedContext(ContextHandle);
						if (FPCGTextureSamplerContext* ContextPtr = SharedContext.Get())
						{
							ContextPtr->bIsPaused = false;
						}
					});

				return false;
			}

			if (!TextureData->IsSuccessfullyInitialized())
			{
				PCGE_LOG(Warning, LogOnly, LOCTEXT("TextureInitFailed", "Data could not be retrieved for this texture, initialization failed."));
				Context->OutputData.TaggedData.Empty();
				return true;
			}
		}

		BaseTextureData = TextureData;
	}

	check(BaseTextureData);

	// Commit to adding texture data.
	FPCGTaggedData& OutTaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
	OutTaggedData.Data = BaseTextureData;

	// Tag the data with a label (based on the asset name) in order to reference this texture data downstream via name rather than data index.
	const FString DataLabel = bIsTextureArray ? FString::Format(TEXT("{0}_{1}"), { Settings->Texture.GetAssetName(), TextureArrayIndex }) : Settings->Texture.GetAssetName();
	OutTaggedData.Tags.Add(PCGComputeHelpers::GetPrefixedDataLabel(DataLabel));

	BaseTextureData->bKeepZeroDensityPoints = Settings->bKeepZeroDensityPoints;
	BaseTextureData->bUseDensitySourceChannel = Settings->bUseDensitySourceChannel;
	BaseTextureData->ColorChannel = Settings->ColorChannel;
	BaseTextureData->Filter = Settings->Filter;
	BaseTextureData->TexelSize = Settings->TexelSize;
	BaseTextureData->bUseAdvancedTiling = Settings->bUseAdvancedTiling;
	BaseTextureData->Tiling = Settings->Tiling;
	BaseTextureData->CenterOffset = Settings->CenterOffset;
	BaseTextureData->Rotation = Settings->Rotation;
	BaseTextureData->bUseTileBounds = Settings->bUseTileBounds;
	BaseTextureData->TileBounds = FBox2D(Settings->TileBoundsMin, Settings->TileBoundsMax);

#if WITH_EDITOR
	// If we have an override, register for dynamic tracking.
	if (Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGTextureSamplerSettings, Texture)))
	{
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, FPCGSelectionKey::CreateFromPath(Texture), /*bIsCulled=*/false);
	}
#endif // WITH_EDITOR

	return true;
}

FPCGContext* FPCGTextureSamplerElement::CreateContext()
{
	return new FPCGTextureSamplerContext();
}

void UPCGTextureSamplerSettings::SetTexture(TSoftObjectPtr<UTexture> InTexture)
{
	Texture = InTexture;

#if WITH_EDITOR
	UpdateDisplayTextureArrayIndex();
#endif // WITH_EDITOR
}

void FPCGTextureSamplerElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	if (const UPCGTextureSamplerSettings* Settings = Cast<UPCGTextureSamplerSettings>(InParams.Settings))
	{
		// If not using absolute transform, depend on actor transform and bounds, and therefore take dependency on actor data.
		bool bUseAbsoluteTransform;
		PCGSettingsHelpers::GetOverrideValue(*InParams.InputData, Settings, GET_MEMBER_NAME_CHECKED(UPCGTextureSamplerSettings, bUseAbsoluteTransform), Settings->bUseAbsoluteTransform, bUseAbsoluteTransform);
		if (!bUseAbsoluteTransform && InParams.ExecutionSource)
		{
			if (const UPCGData* Data = InParams.ExecutionSource->GetExecutionState().GetSelfData())
			{
				Crc.Combine(Data->GetOrComputeCrc(/*bFullDataCrc=*/false));
			}
		}
	}

	OutCrc = Crc;
}

#undef LOCTEXT_NAMESPACE
