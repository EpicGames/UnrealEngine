// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGTransformPoints.h"

#include "PCGContext.h"
#include "PCGPin.h"
#include "Compute/PCGKernelHelpers.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"

#if WITH_EDITOR
#include "Compute/PCGComputeKernel.h"
#include "Compute/Elements/PCGTransformPointsKernel.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#endif // WITH_EDITOR

#include "Math/RandomStream.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGTransformPoints)

#define LOCTEXT_NAMESPACE "PCGTransformPointsElement"

class UPCGTransformPointsKernel;

#if WITH_EDITOR
void UPCGTransformPointsSettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode,
                                                TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);
	PCGKernelHelpers::CreateKernel<UPCGTransformPointsKernel>(InOutContext, CreateParams, OutKernels, OutEdges);
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGTransformPointsSettings::CreateElement() const
{
	return MakeShared<FPCGTransformPointsElement>();
}

EPCGPointNativeProperties UPCGTransformPointsSettings::GetPropertiesToAllocate() const
{
	return bApplyToAttribute ? EPCGPointNativeProperties::MetadataEntry : (bRecomputeSeed ? (EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed) : EPCGPointNativeProperties::Transform);
}

bool FPCGTransformPointsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTransformPointsElement::Execute);

	const UPCGTransformPointsSettings* Settings = Context->GetInputSettings<UPCGTransformPointsSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const bool bApplyToAttribute = Settings->bApplyToAttribute;
	const FName AttributeName = Settings->AttributeName;
	const FVector& OffsetMin = Settings->OffsetMin;
	const FVector& OffsetMax = Settings->OffsetMax;
	const bool bAbsoluteOffset = Settings->bAbsoluteOffset;
	const FRotator& RotationMin = Settings->RotationMin;
	const FRotator& RotationMax = Settings->RotationMax;
	const bool bAbsoluteRotation = Settings->bAbsoluteRotation;
	const FVector& ScaleMin = Settings->ScaleMin;
	const FVector& ScaleMax = Settings->ScaleMax;
	const bool bAbsoluteScale = Settings->bAbsoluteScale;
	const bool bUniformScale = Settings->bUniformScale;
	const bool bRecomputeSeed = Settings->bRecomputeSeed;

	const int Seed = Context->GetSeed();

	// Use implicit capture, since we capture a lot
	//ProcessPoints(Context, Inputs, Outputs, [&](const FPCGPoint& InPoint, FPCGPoint& OutPoint)
	for (const FPCGTaggedData& Input : Inputs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTransformPointsElement::Execute::InputLoop);
		FPCGTaggedData& Output = Outputs.Add_GetRef(Input);

		const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Input.Data);

		if (!SpatialData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InputMissingSpatialData", "Unable to get Spatial data from input"));
			continue;
		}

		const UPCGBasePointData* PointData = SpatialData->ToBasePointData(Context);

		if (!PointData)
		{
			PCGE_LOG(Error, GraphAndLog, LOCTEXT("InputMissingPointData", "Unable to get Point data from input"));
			continue;
		}

		FName LocalAttributeName = AttributeName;
		const FPCGMetadataAttribute<FTransform>* SourceAttribute = nullptr;

		if (bApplyToAttribute)
		{
			const UPCGMetadata* PointMetadata = PointData->ConstMetadata();
			check(PointMetadata);

			if (LocalAttributeName == NAME_None)
			{
				LocalAttributeName = PointMetadata->GetLatestAttributeNameOrNone();
			}

			// Validate that the attribute has the proper type
			const FPCGMetadataAttributeBase* FoundAttribute = PointMetadata->GetConstAttribute(LocalAttributeName);

			if (!FoundAttribute || FoundAttribute->GetTypeId() != PCG::Private::MetadataTypes<FTransform>::Id)
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("AttributeMissing", "Attribute '{0}' does not exist or is not a transform"), FText::FromName(LocalAttributeName)));
				continue;
			}

			SourceAttribute = static_cast<const FPCGMetadataAttribute<FTransform>*>(FoundAttribute);
		}
				
		UPCGBasePointData* OutputData = FPCGContext::NewPointData_AnyThread(Context);
		
		OutputData->InitializeFromData(PointData);
		OutputData->SetNumPoints(PointData->GetNumPoints(), /*bInitializeValues=*/false);

		const EPCGPointNativeProperties PropertiesToAllocate = Settings->GetPropertiesToAllocate();

		if (OutputData->HasSpatialDataParent())
		{
			OutputData->AllocateProperties(PropertiesToAllocate);
		}
		else
		{
			OutputData->AllocateProperties(PointData->GetAllocatedProperties() | PropertiesToAllocate);
		}

		Output.Data = OutputData;

		FPCGMetadataAttribute<FTransform>* TargetAttribute = nullptr;
		TArray<TTuple<int64, int64>> AllMetadataEntries;

		if (bApplyToAttribute)
		{
			check(SourceAttribute && OutputData && OutputData->Metadata);
			TargetAttribute = OutputData->Metadata->GetMutableTypedAttribute<FTransform>(LocalAttributeName);
			AllMetadataEntries.SetNum(PointData->GetNumPoints());
		}

		auto ProcessRangeFunc = [&](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			int32 NumWritten = 0;
			
			// Copy all properties except the ones we are going to modify (if we are not inheriting)
			if (!OutputData->HasSpatialDataParent())
			{
				PointData->CopyPropertiesTo(OutputData, StartReadIndex, StartWriteIndex, Count, EPCGPointNativeProperties::All & ~PropertiesToAllocate);
			}

			const TConstPCGValueRange<FTransform> ReadTransformRange = PointData->GetConstTransformValueRange();
			const TConstPCGValueRange<int32> ReadSeedRange = PointData->GetConstSeedValueRange();
			const TConstPCGValueRange<int64> ReadMetadataEntryRange = PointData->GetConstMetadataEntryValueRange();

			TPCGValueRange<FTransform> WriteTransformRange = bApplyToAttribute ? TPCGValueRange<FTransform>() : OutputData->GetTransformValueRange(/*bAllocate=*/false);
			TPCGValueRange<int32> WriteSeedRange = bApplyToAttribute || !bRecomputeSeed ? TPCGValueRange<int32>() : OutputData->GetSeedValueRange(/*bAllocate=*/false);
			TPCGValueRange<int64> WriteMetadataEntryRange = bApplyToAttribute ? OutputData->GetMetadataEntryValueRange(/*bAllocate=*/false) : TPCGValueRange<int64>();

			for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
			{
				const int32 WriteIndex = StartWriteIndex + NumWritten;
				
				FRandomStream RandomSource(PCGHelpers::ComputeSeed(Seed, ReadSeedRange[ReadIndex]));

				const float OffsetX = RandomSource.FRandRange(OffsetMin.X, OffsetMax.X);
				const float OffsetY = RandomSource.FRandRange(OffsetMin.Y, OffsetMax.Y);
				const float OffsetZ = RandomSource.FRandRange(OffsetMin.Z, OffsetMax.Z);
				const FVector RandomOffset(OffsetX, OffsetY, OffsetZ);

				const float RotationX = RandomSource.FRandRange(RotationMin.Pitch, RotationMax.Pitch);
				const float RotationY = RandomSource.FRandRange(RotationMin.Yaw, RotationMax.Yaw);
				const float RotationZ = RandomSource.FRandRange(RotationMin.Roll, RotationMax.Roll);
				const FQuat RandomRotation(FRotator(RotationX, RotationY, RotationZ).Quaternion());

				FVector RandomScale;
				if (bUniformScale)
				{
					RandomScale = FVector(RandomSource.FRandRange(ScaleMin.X, ScaleMax.X));
				}
				else
				{
					RandomScale.X = RandomSource.FRandRange(ScaleMin.X, ScaleMax.X);
					RandomScale.Y = RandomSource.FRandRange(ScaleMin.Y, ScaleMax.Y);
					RandomScale.Z = RandomSource.FRandRange(ScaleMin.Z, ScaleMax.Z);
				}

				FTransform SourceTransform;

				if (!bApplyToAttribute)
				{
					SourceTransform = ReadTransformRange[ReadIndex];
				}
				else
				{
					SourceTransform = SourceAttribute->GetValueFromItemKey(ReadMetadataEntryRange[ReadIndex]);
				}

				FTransform FinalTransform = SourceTransform;

				if (bAbsoluteOffset)
				{
					FinalTransform.SetLocation(SourceTransform.GetLocation() + RandomOffset);
				}
				else
				{
					const FTransform RotatedTransform(SourceTransform.GetRotation());
					FinalTransform.SetLocation(SourceTransform.GetLocation() + RotatedTransform.TransformPosition(RandomOffset));
				}

				if (bAbsoluteRotation)
				{
					FinalTransform.SetRotation(RandomRotation);
				}
				else
				{
					FinalTransform.SetRotation(SourceTransform.GetRotation() * RandomRotation);
				}

				if (bAbsoluteScale)
				{
					FinalTransform.SetScale3D(RandomScale);
				}
				else
				{
					FinalTransform.SetScale3D(SourceTransform.GetScale3D() * RandomScale);
				}

				if (!bApplyToAttribute)
				{
					WriteTransformRange[WriteIndex] = FinalTransform;

					if (bRecomputeSeed)
					{
						const FVector& Position = FinalTransform.GetLocation();
						WriteSeedRange[WriteIndex] = PCGHelpers::ComputeSeed((int)Position.X, (int)Position.Y, (int)Position.Z);
					}
				}
				else
				{
					WriteMetadataEntryRange[WriteIndex] = OutputData->Metadata->AddEntryPlaceholder();
					AllMetadataEntries[ReadIndex] = MakeTuple(WriteMetadataEntryRange[WriteIndex], ReadMetadataEntryRange[ReadIndex]);
					TargetAttribute->SetValue(WriteMetadataEntryRange[WriteIndex], FinalTransform);
				}

				++NumWritten;
			}

			check(NumWritten == Count);
			return NumWritten;
		};
				
		FPCGAsync::AsyncProcessingOneToOneRangeEx(
			&Context->AsyncState,
			PointData->GetNumPoints(),
			/*InitializeFunc=*/[](){},
			ProcessRangeFunc,
			/*bTimeSliceEnabled=*/false);
				
		if (TargetAttribute)
		{
			OutputData->Metadata->AddDelayedEntries(AllMetadataEntries);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
