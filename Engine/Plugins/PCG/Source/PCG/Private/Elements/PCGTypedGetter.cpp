// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGTypedGetter.h"

#include "PCGComponent.h"
#include "Data/PCGLandscapeData.h"
#include "Helpers/PCGHelpers.h"

#include "Landscape.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "VT/RuntimeVirtualTextureVolume.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGTypedGetter)

#define LOCTEXT_NAMESPACE "PCGTypedGetterElements"

UPCGGetLandscapeSettings::UPCGGetLandscapeSettings()
{
	Mode = EPCGGetDataFromActorMode::ParseActorComponents;
	ActorSelector.bIncludeChildren = false;
	ActorSelector.bShowActorSelectionClass = false;
	ActorSelector.bSelectMultiple = true;
	ActorSelector.bShowSelectMultiple = false;

	// We want to apply different defaults to newly placed nodes. We detect new object if they are not a default object/archetype
	// and/or they do not need load.
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		// This setup replicates what was implemented on the Landscape input node pin
		ActorSelector.ActorFilter = EPCGActorFilter::AllWorldActors;
		ActorSelector.bMustOverlapSelf = true;
		ActorSelector.ActorSelection = EPCGActorSelection::ByClass;
		ActorSelector.ActorSelectionClass = ALandscapeProxy::StaticClass();

		SamplingProperties.bSampleVirtualTextures = false;
	}
}

void UPCGGetLandscapeSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (bGetHeightOnly_DEPRECATED)
	{
		SamplingProperties.bGetHeightOnly = bGetHeightOnly_DEPRECATED;
		bGetHeightOnly_DEPRECATED = false;
	}

	if (!bGetLayerWeights_DEPRECATED)
	{
		SamplingProperties.bGetLayerWeights = bGetLayerWeights_DEPRECATED;
		bGetLayerWeights_DEPRECATED = true;
	}
#endif
}

TSubclassOf<AActor> UPCGGetLandscapeSettings::GetDefaultActorSelectorClass() const
{
	return ALandscapeProxy::StaticClass();
}

TArray<FPCGPinProperties> UPCGGetLandscapeSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Landscape, /*bAllowMultipleConnections=*/true, /*bAllowMultipleData=*/false);

	return PinProperties;
}

FString UPCGGetLandscapeSettings::GetAdditionalTitleInformation() const
{
	// Do not use the version from data from actor otherwise we'll show the selected actor class, which serves no purpose
	return UPCGSettings::GetAdditionalTitleInformation();
}

#if WITH_EDITOR
FText UPCGGetLandscapeSettings::GetNodeTooltipText() const
{
	return LOCTEXT("GetLandscapeTooltip", "Builds a collection of landscapes from the selected actors.");
}
#endif

FPCGElementPtr UPCGGetLandscapeSettings::CreateElement() const
{
	return MakeShared<FPCGGetLandscapeDataElement>();
}

void FPCGGetLandscapeDataElement::ProcessActors(FPCGContext* Context, const UPCGDataFromActorSettings* InSettings, const TArray<AActor*>& FoundActors, TArray<FPCGTaskId>& OutDynamicDependencies) const
{
	check(Context);
	check(InSettings);

	const UPCGGetLandscapeSettings* Settings = CastChecked<UPCGGetLandscapeSettings>(InSettings);

	// In the base class (FPCGDataFromActorElement) we'd go through all actors, one by one and call
	// UPCGComponent::CreateActorPCGDataCollection, and push the results to the output.
	// However, in this case, we want to do what's done in UPCGComponent::GetLandscapeData,
	// which is to create a single UPCGLandscapeData keeping tabs on all selected landscapes.
	TArray<TWeakObjectPtr<ALandscapeProxy>> Landscapes;
	FBox LandscapeBounds(EForceInit::ForceInit);
	TSet<FString> LandscapeTags;

	for (AActor* FoundActor : FoundActors)
	{
		if (!IsValid(FoundActor))
		{
			continue;
		}

		ALandscapeProxy* Landscape = Cast<ALandscapeProxy>(FoundActor);
		if (Landscape)
		{
			Landscapes.Add(Landscape);
			LandscapeBounds += PCGHelpers::GetGridBounds(Landscape, nullptr);

			for (FName Tag : Landscape->Tags)
			{
				LandscapeTags.Add(Tag.ToString());
			}
		}
	}

	if (!Landscapes.IsEmpty())
	{
		UPCGLandscapeData* LandscapeData = FPCGContext::NewObject_AnyThread<UPCGLandscapeData>(Context);
		LandscapeData->Initialize(Landscapes, LandscapeBounds, Settings->SamplingProperties);
		
		// Prepare data for the PCG Bounds in Editor only as it can create cache entries which isn't thread safe
#if WITH_EDITOR
		const FBox PrepareBounds = Context->ExecutionSource.IsValid() && !Settings->bUnbounded ? Context->ExecutionSource->GetExecutionState().GetBounds() : LandscapeBounds;
		OutDynamicDependencies.Append(LandscapeData->PrepareForSpatialQuery(Context, PrepareBounds));
#endif

		FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = LandscapeData;
		TaggedData.Tags = LandscapeTags;
	}
}

void FPCGGetLandscapeDataElement::ProcessActors(FPCGContext* Context, const UPCGDataFromActorSettings* InSettings, const TArray<AActor*>& FoundActors) const
{
	checkf(false, TEXT("This should never be called directly"));
}

void FPCGGetLandscapeDataElement::ProcessActor(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, AActor* FoundActor) const
{
	checkf(false, TEXT("This should never be called directly"));
}

UPCGGetSplineSettings::UPCGGetSplineSettings()
{
	Mode = EPCGGetDataFromActorMode::ParseActorComponents;
}

TArray<FPCGPinProperties> UPCGGetSplineSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::PolyLine);

	return PinProperties;
}

#if WITH_EDITOR
FText UPCGGetSplineSettings::GetNodeTooltipText() const
{
	return LOCTEXT("GetSplineTooltip", "Builds a collection of splines from the selected actors.");
}
#endif

UPCGGetVolumeSettings::UPCGGetVolumeSettings()
{
	Mode = EPCGGetDataFromActorMode::ParseActorComponents;
}

TArray<FPCGPinProperties> UPCGGetVolumeSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Volume);

	return PinProperties;
}

#if WITH_EDITOR
FText UPCGGetVolumeSettings::GetNodeTooltipText() const
{
	return LOCTEXT("GetVolumeTooltip", "Builds a collection of volumes from the selected actors.\n"
		"AVolume or APCGPartitionActor produce volume data.\n"
		"Use GetPrimitiveData for primitive components (i.e like Box, Sphere or Static Mesh collisions).");
}
#endif

UPCGGetPrimitiveSettings::UPCGGetPrimitiveSettings()
{
	Mode = EPCGGetDataFromActorMode::ParseActorComponents;
}

TArray<FPCGPinProperties> UPCGGetPrimitiveSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Primitive);

	return PinProperties;
}

#if WITH_EDITOR
FText UPCGGetPrimitiveSettings::GetNodeTooltipText() const
{
	return LOCTEXT("GetPrimitiveTooltip", "Builds a collection of primitive data from primitive components on the selected actors.");
}
#endif

UPCGGetPCGComponentSettings::UPCGGetPCGComponentSettings()
{
	Mode = EPCGGetDataFromActorMode::GetDataFromPCGComponent;

	ActorSelector.bShowActorFilter = false;
	ActorSelector.ActorFilter = EPCGActorFilter::AllWorldActors;
}

TArray<FPCGPinProperties> UPCGGetPCGComponentSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties = Super::OutputPinProperties();
	check(!PinProperties.IsEmpty());
	PinProperties[0].AllowedTypes = EPCGDataType::Any;

	return PinProperties;
}

#if WITH_EDITOR
FText UPCGGetPCGComponentSettings::GetNodeTooltipText() const
{
	return LOCTEXT("GetPCGComponentTooltip", "Builds a collection of data from other PCG components on the selected actors.\n"
		"Automatically tags each output with the grid size it was collected from, prefixed by \"PCG_GridSize_\" (e.g.PCG_GridSize_12800).\n"
		"Note: a component cannot get component data from itself or other components in its execution context, as it could create a circular dependency.");
}
#endif

UPCGGetVirtualTextureSettings::UPCGGetVirtualTextureSettings()
{
	Mode = EPCGGetDataFromActorMode::ParseActorComponents;

	ActorSelector.ActorFilter = EPCGActorFilter::AllWorldActors;
	ActorSelector.bShowActorFilter = false;
	ActorSelector.ActorSelection = EPCGActorSelection::ByClass;
	ActorSelector.ActorSelectionClass = ARuntimeVirtualTextureVolume::StaticClass();
	ActorSelector.bShowActorSelection = false;
	ActorSelector.bShowActorSelectionClass = false;
	ActorSelector.bIncludeChildren = false;
	ActorSelector.bSelectMultiple = true;
	ActorSelector.bShowSelectMultiple = false;

	ComponentSelector.ComponentSelection = EPCGComponentSelection::ByClass;
	ComponentSelector.ComponentSelectionClass = URuntimeVirtualTextureComponent::StaticClass();
	ComponentSelector.bShowComponentSelectionClass = false;
}

#if WITH_EDITOR
FText UPCGGetVirtualTextureSettings::GetNodeTooltipText() const
{
	return LOCTEXT("GetVirtualTextureTooltip", "Builds a collection of virtual texture data from the selected actors.");
}
#endif

FString UPCGGetVirtualTextureSettings::GetAdditionalTitleInformation() const
{
	// Do not use the version from data from actor otherwise we'll show the selected actor class, which serves no purpose
	return UPCGSettings::GetAdditionalTitleInformation();
}

TArray<FPCGPinProperties> UPCGGetVirtualTextureSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& Pin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::VirtualTexture);
	Pin.bAllowMultipleData = false; // Must be set after pin properties constructed

	return PinProperties;
}

TSubclassOf<AActor> UPCGGetVirtualTextureSettings::GetDefaultActorSelectorClass() const
{
	return ARuntimeVirtualTextureVolume::StaticClass();
}

#undef LOCTEXT_NAMESPACE
