// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGUnionElement.h"
#include "Data/PCGSpatialData.h"
#include "PCGContext.h"
#include "PCGPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGUnionElement)

#define LOCTEXT_NAMESPACE "PCGUnionSettings"

TArray<FPCGPinProperties> UPCGUnionSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spatial);

	return PinProperties;
}

FPCGElementPtr UPCGUnionSettings::CreateElement() const
{
	return MakeShared<FPCGUnionElement>();
}

FName UPCGUnionSettings::GetDynamicInputPinsBaseLabel() const
{
	return PCGPinConstants::DefaultInputLabel;
}

TArray<FPCGPinProperties> UPCGUnionSettings::StaticInputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spatial);
	return Properties;
}

#if WITH_EDITOR
void UPCGUnionSettings::AddDefaultDynamicInputPin()
{
	FPCGPinProperties SecondaryPinProperties(
		FName(GetDynamicInputPinsBaseLabel().ToString() + FString::FromInt(DynamicInputPinProperties.Num() + 2)),
		EPCGDataType::Spatial,
		/*bInAllowMultipleConnections=*/false);
	SecondaryPinProperties.Tooltip = LOCTEXT("DynamicPinPropertyTooltip", "Dynamic pins, such as this one, will be unioned together in order.");
	AddDynamicInputPin(std::move(SecondaryPinProperties));
}
#endif // WITH_EDITOR

bool FPCGUnionElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGUnionElement::Execute);

	const UPCGUnionSettings* Settings = Context->GetInputSettings<UPCGUnionSettings>();
	check(Settings);

	const EPCGUnionType Type = Settings->Type;
	const EPCGUnionDensityFunction DensityFunction = Settings->DensityFunction;

	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const UPCGSpatialData* FirstSpatialData = nullptr;
	UPCGUnionData* UnionData = nullptr;
	int32 UnionTaggedDataIndex = -1;

	TArray<FPCGTaggedData, TInlineAllocator<8>> Sources;
	for (FName PinLabel : Settings->GetNodeDefinedPinLabels())
	{
		Sources.Append(Context->InputData.GetInputsByPin(PinLabel));
	}

	for (const FPCGTaggedData& Source : Sources)
	{
		const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(Source.Data);

		// Non-spatial data we're not going to touch
		if (!SpatialData)
		{
			Outputs.Add(Source);
			continue;
		}

		if (!FirstSpatialData)
		{
			FirstSpatialData = SpatialData;
			UnionTaggedDataIndex = Outputs.Num();
			Outputs.Add(Source);
			continue;
		}

		FPCGTaggedData& UnionTaggedData = Outputs[UnionTaggedDataIndex];

		// Create union or add to it
		if (!UnionData)
		{
			UnionData = FirstSpatialData->UnionWith(Context, SpatialData);
			UnionData->SetType(Type);
			UnionData->SetDensityFunction(DensityFunction);

			UnionTaggedData.Data = UnionData;
		}
		else
		{
			UnionData->AddData(SpatialData);
			UnionTaggedData.Tags.Append(Source.Tags);
		}
		
		UnionTaggedData.Data = UnionData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE