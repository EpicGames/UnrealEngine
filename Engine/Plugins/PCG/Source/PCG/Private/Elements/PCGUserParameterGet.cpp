// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGUserParameterGet.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "PCGSubgraph.h"
#include "Data/PCGUserParametersData.h"
#include "Elements/PCGCreateAttribute.h"
#include "Helpers/PCGConversion.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "Helpers/PCGGraphParametersHelpers.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "Algo/NoneOf.h"
#include "StructUtils/PropertyBag.h"
#include "StructUtils/StructView.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGUserParameterGet)

#define LOCTEXT_NAMESPACE "PCGUserParameterGetElement"

namespace PCGUserParameterGet
{
	namespace Settings
	{
		/**
		* Utility function to get the first valid instanced property bag
		* We define valid as if the ParameterOverrides from a GraphInstance and UserParameters from the graph owner of the node have the same property bag.
		* By construction, it should be always the case, but we want to prevent cases where graph instances depend on other graph instances, that has changed their graph
		* but didn't propagate the changes.
		* If the property bags aren't the same, we are traversing the graph instance hierarchy to find the first graph/graph instance that matches.
		*/
		TArray<FConstStructView, TInlineAllocator<16>> GetValidLayouts(FPCGContext& InContext)
		{
			TArray<FConstStructView, TInlineAllocator<16>> Layouts;
			const UPCGGraphInterface* GraphInterface = nullptr;

			// First we will read from the input if we find an input for this graph, set by the subgraph element, then use it.
			TArray<FPCGTaggedData> UserParameterData = InContext.InputData.GetTaggedTypedInputs<UPCGUserParametersData>(PCG::Private::UserParameterTagData);
			if (!UserParameterData.IsEmpty())
			{
	#if WITH_EDITOR
				// Safe guard to make sure we always have one and only one data of this type.
				ensure(UserParameterData.Num() == 1);
	#endif // WITH_EDITOR
				if (const UPCGUserParametersData* OverrideParametersData = CastChecked<UPCGUserParametersData>(UserParameterData[0].Data, ECastCheckedType::NullAllowed))
				{
					// Gathering all the user parameter data from upstream, starting with this current level data.
					const UPCGUserParametersData* UpstreamData = OverrideParametersData;
					while (UpstreamData)
					{
						// Make sure to check if the user parameters is valid, as we don't want to have invalid layouts
						if (UpstreamData->UserParameters.IsValid())
						{
							Layouts.Emplace(UpstreamData->UserParameters);
						}

						UpstreamData = UpstreamData->UpstreamData.Get();
					}

					return Layouts;
				}
				else
				{
					PCGE_LOG_C(Error, LogOnly, &InContext, LOCTEXT("InvalidUserParameterData", "Internal error, PCG User Parameters Data is null"));
					return {};
				}
			}

			// We gather the outer graph of this node, to make sure it matches our interface. It must happen on the game thread
			if (!ensure(IsInGameThread()))
			{
				PCGE_LOG_C(Error, LogOnly, &InContext, LOCTEXT("NoUserParameterData", "Internal error, PCG User Parameters Data is absent and we are not executing on the game thread"));
				return {};
			}

			const UPCGGraph* GraphFromNode = Cast<UPCGGraph>(InContext.Node->GetOuter());
			check(GraphFromNode);
			FConstStructView GraphFromNodeParameters = GraphFromNode->GetUserParametersStruct() ? GraphFromNode->GetUserParametersStruct()->GetValue() : FConstStructView{};

			// If we don't have a graph instance, we just use the user parameters from the node graph owner.
			const IPCGGraphExecutionSource* ExecutionSource = InContext.ExecutionSource.Get();
			const UPCGGraphInstance* GraphInstance = ExecutionSource ? ExecutionSource->GetExecutionState().GetGraphInstance() : nullptr;

			// If there is no user parameters, just use the one from the graph.
			// Go down the graph instance chain and take all layouts. Revert back to original graph node params if none were found
			bool bAddedNonTrivialLayout = false;

			while (GraphInstance)
			{
				FConstStructView GraphParameters = GraphInstance->GetUserParametersStruct() ? GraphInstance->GetUserParametersStruct()->GetValue() : FConstStructView{};
				if (GraphParameters.IsValid())
				{
					bAddedNonTrivialLayout = true;
					Layouts.Add(GraphParameters);
				}

				GraphInstance = Cast<UPCGGraphInstance>(GraphInstance->Graph);
			}

			if (!bAddedNonTrivialLayout && GraphFromNodeParameters.IsValid())
			{
				Layouts.Add(GraphFromNodeParameters);
			}

			return Layouts;
		}
	}

	namespace Constants::Conversion
	{
		constexpr int32 SpecificToGenericIndex = 0;
		constexpr int32 SpecificToConstantIndex = 1;
		constexpr int32 GenericToSpecificIndex = 0;
		constexpr int32 GenericToConstantIndex = 1;

		const FText SpecificToGenericTooltip = LOCTEXT("SpecificToGenericTooltip", "Convert to a Get User Parameter (Generic) node.");
		const FText GenericToSpecificTooltip = LOCTEXT("GenericToSpecificTooltip", "Convert to a specified Get User Parameter node.");
		const FText ToCreateConstantTooltip = LOCTEXT("ToConstantTooltip", "Convert to a Create Constant node.");

		const FText SpecificGetUserParameterNodeTitle = LOCTEXT("SpecificGetUserParameterNodeTitle", "Get Graph Parameter");
		const FText GenericGetUserParameterNodeTitle = LOCTEXT("GenericGetUserParameterNodeTitle", "Get Graph Parameter (Generic)");
	}

	namespace Helpers::Conversion
	{
		// Assigns a value based on a graph parameter to a MetadataTypesConstantStruct, which is normally only user defined.
		bool AssignValue(FPCGMetadataTypesConstantStruct& OutTypeStruct, const UPCGGraph* Graph, const FName GraphParameterName, FText& OutFailReason)
		{
			if (GraphParameterName == NAME_None || !Graph)
			{
				OutFailReason = LOCTEXT("ConversionAssignValueInvalidGraphOrParameter", "Invalid graph or parameter.");
				return false;
			}

			const FInstancedPropertyBag* UserParametersStruct = Graph->GetUserParametersStruct();
			check(UserParametersStruct);

			const FPropertyBagPropertyDesc* Desc = UserParametersStruct->FindPropertyDescByName(GraphParameterName);
			if (!Desc || !Desc->ContainerTypes.IsEmpty()) // Property doesn't exist, or it is a container, which doesn't work yet. Early out.
			{
				OutFailReason = LOCTEXT("ConversionAssignValueInvalidParamNameOrContainer", "Parameter not found or it is a container, which is not supported.");
				return false;
			}

			TUniquePtr<const IPCGAttributeAccessor> PropertyAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(Desc->CachedProperty);

			if (!PropertyAccessor.IsValid())
			{
				OutFailReason = LOCTEXT("ConversionAssignValueInvalidAccessor", "Could not access parameter, likely due to unsupported type.");
				return false;
			}

			// The type must be set before the dispatch.
			OutTypeStruct.Type = static_cast<EPCGMetadataTypes>(PropertyAccessor->GetUnderlyingType());
			if (OutTypeStruct.Type == EPCGMetadataTypes::Unknown)
			{
				OutFailReason = LOCTEXT("ConversionAssignValueUnknownType", "Parameter type is not supported by Create Constant.");
				return false;
			}

			auto DispatchFunc = [UserParametersStruct, &PropertyAccessor, &OutFailReason]<typename T>(const T& Value)
			{
				// Simulating initialization by the user. Const qualifier from 'Dispatcher' function signature only, so safe to cast away.
				T& Ref = const_cast<T&>(Value);

				const FPCGAttributeAccessorKeysSingleObjectPtr Keys(UserParametersStruct->GetValue().GetMemory());
				const bool bResult = PropertyAccessor->Get<T>(Ref, Keys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
				if (!bResult)
				{
					OutFailReason = LOCTEXT("ConversionAssignValueCouldNotReadAccessor", "Could not read value from user parameter accessor.");
				}

				return bResult;
			};

			return OutTypeStruct.Dispatcher(DispatchFunc);
		}
	}
} // namespace PCGUserParameterGet

TArray<FPCGPinProperties> UPCGUserParameterGetSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& Pin = PinProperties.Emplace_GetRef(PropertyName, EPCGDataType::Param);

	if (PropertyName != NAME_None)
	{
		// Find the property associated with this param
		const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
		const UPCGGraph* Graph = Node ? Node->GetGraph() : nullptr;
		const FInstancedPropertyBag* PropertyBag = Graph ? Graph->GetUserParametersStruct() : nullptr;
		const FPropertyBagPropertyDesc* PropertyDesc = PropertyBag ? PropertyBag->FindPropertyDescByName(PropertyName) : nullptr;
		if (PropertyDesc && PropertyDesc->CachedProperty)
		{
			// If the property is a container, check its inner
			const FProperty* Property = PropertyDesc->CachedProperty;
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				Property = ArrayProperty->Inner;
			}
			else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
			{
				Property = SetProperty->ElementProp;
			}

			if (Property)
			{
				Pin.AllowedTypes.CustomSubtype = static_cast<int32>(PCGAttributeAccessorHelpers::GetMetadataTypeForProperty(Property));
			}
		}
	}

	return PinProperties;
}

void UPCGUserParameterGetSettings::UpdatePropertyName(FName InNewName)
{
	if (PropertyName != InNewName)
	{
		Modify();
		PropertyName = InNewName;
	}
}

void UPCGUserParameterGetSettings::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITOR
	if (GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::OptionSanitizeOutputAttributeNamesPCG)
	{
		// For all previous nodes, we'll force this option to false. for retro-compatibility
		bSanitizeOutputAttributeName = false;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bForceObjectAndStructExtraction)
	{
		bForceObjectAndStructExtraction = false;
		StructExtractorBehavior = EPCGStructExtractorBehavior::Extract;
		ObjectExtractorBehavior = EPCGObjectExtractorBehavior::Extract;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

FPCGElementPtr UPCGUserParameterGetSettings::CreateElement() const
{
	return MakeShared<FPCGUserParameterGetElement>();
}

#if WITH_EDITOR
TArray<FPCGPreconfiguredInfo> UPCGUserParameterGetSettings::GetConversionInfo() const
{
	using namespace PCGUserParameterGet::Constants::Conversion;
	TArray<FPCGPreconfiguredInfo> ConversionInfo;

	ConversionInfo.Emplace(
		SpecificToGenericIndex,
		FText::Format(INVTEXT("{0} (Generic)"), PCGConversion::Helpers::GetDefaultNodeTitle(UPCGGenericUserParameterGetSettings::StaticClass())),
		SpecificToGenericTooltip);

	ConversionInfo.Emplace(
		SpecificToConstantIndex,
		PCGConversion::Helpers::GetDefaultNodeTitle(UPCGCreateAttributeSetSettings::StaticClass()),
		ToCreateConstantTooltip);

	return ConversionInfo;
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGUserParameterGetSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGMetadataTypes>(
		/*InValuesToSkip=*/{EPCGMetadataTypes::Count, EPCGMetadataTypes::Unknown},
		/*InOptionalFormat=*/FTextFormat(LOCTEXT("PreconfigureFormat", "New {0} Parameter")));
}
#endif // WITH_EDITOR

void UPCGUserParameterGetSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo)
{
	if (PreconfigureInfo.PreconfiguredIndex < 0 || PreconfigureInfo.PreconfiguredIndex >= static_cast<uint8>(EPCGMetadataTypes::Count))
	{
		return;
	}

	EPCGMetadataTypes NewType = EPCGMetadataTypes::Unknown;
	if (const UEnum* EnumPtr = StaticEnum<EPCGMetadataTypes>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfigureInfo.PreconfiguredIndex))
		{
			NewType = static_cast<EPCGMetadataTypes>(EnumPtr->GetValueByIndex(PreconfigureInfo.PreconfiguredIndex));
		}
		else
		{
			return;
		}
	}

	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	if (UPCGGraph* Graph = Node ? Node->GetGraph() : nullptr)
	{
		if (!PCGGraphParameter::Helpers::GenerateUniqueName(Graph, PropertyName))
		{
			UE_LOGF(LogPCG, Error, "Could not create a unique graph parameter with name '%ls'. Check if the graph is valid and the name has not reached its maximum.", *PropertyName.ToString());
			return;
		}
		check(FInstancedPropertyBag::IsPropertyNameValid(PropertyName));

		FPropertyBagPropertyDesc PropertyDesc = PCGPropertyHelpers::CreatePropertyBagDescWithMetadataType(PropertyName, NewType);

		const EPropertyBagAlterationResult Result = Graph->AddUserParameters({PropertyDesc});
		if (Result != EPropertyBagAlterationResult::Success)
		{
			PCGGraphParameter::Helpers::LogGraphParamNamingErrors(PropertyName, Result);
			return;
		}

		if (FInstancedPropertyBag* PropertyBag = Graph->GetMutableUserParametersStruct_Unsafe())
		{
			if (const FPropertyBagPropertyDesc* Desc = PropertyBag->FindPropertyDescByName(PropertyName))
			{
				PropertyGuid = Desc->ID;
				PropertyName = Desc->Name;
			}
			else
			{
				UE_LOGF(LogPCG, Error, "Could not create a unique graph parameter with name '%ls'.", *PropertyName.ToString());
				ensure(false);
			}
		}
	}
}

bool UPCGUserParameterGetSettings::ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo)
{
	using namespace PCGUserParameterGet::Constants::Conversion;
	UPCGNode* Node = CastChecked<UPCGNode>(GetOuter());

	if (ConversionInfo.PreconfiguredIndex == SpecificToGenericIndex)
	{
		FPCGSingleNodeConverter NodeConverter(Node, UPCGGenericUserParameterGetSettings::StaticClass());
		if (!NodeConverter.IsGraphInitialized() || !NodeConverter.IsSourceInitialized())
		{
			PCGLog::Settings::LogInvalidPreconfigurationWarning(ConversionInfo.PreconfiguredIndex, SpecificGetUserParameterNodeTitle);
			return false;
		}

		NodeConverter.PrepareData();

		UPCGGenericUserParameterGetSettings* Settings = Cast<UPCGGenericUserParameterGetSettings>(NodeConverter.GetGeneratedSettings());
		Settings->PropertyPath = PropertyName.ToString();
		Settings->OutputAttributeName = PropertyName;

		NodeConverter.ApplyStructural();
		NodeConverter.Finalize();

		return NodeConverter.IsComplete();
	}
	else if (ConversionInfo.PreconfiguredIndex == SpecificToConstantIndex)
	{
		FPCGSingleNodeConverter NodeConverter(Node, UPCGCreateAttributeSetSettings::StaticClass());
		if (!NodeConverter.IsGraphInitialized() || !NodeConverter.IsSourceInitialized())
		{
			PCGLog::Settings::LogInvalidPreconfigurationWarning(ConversionInfo.PreconfiguredIndex, SpecificGetUserParameterNodeTitle);
			return false;
		}

		NodeConverter.PrepareData();

		if (UPCGGraph* Graph = NodeConverter.GetGraph())
		{
			UPCGCreateAttributeSetSettings* Settings = CastChecked<UPCGCreateAttributeSetSettings>(NodeConverter.GetGeneratedSettings());
			Settings->OutputTarget.Update(PropertyName.ToString());

			FText ErrorMessage;
			if (PCGUserParameterGet::Helpers::Conversion::AssignValue(Settings->AttributeTypes, Graph, PropertyName, ErrorMessage))
			{
				NodeConverter.ApplyStructural();
				NodeConverter.Finalize();
			}
			else
			{
				PCGLog::Settings::LogInvalidConversionError(ConversionInfo.PreconfiguredIndex, SpecificGetUserParameterNodeTitle, ErrorMessage);
			}
		}

		return NodeConverter.IsComplete();
	}

	return false;
}

//////////////////////////////////////////////

TArray<FPCGPinProperties> UPCGGenericUserParameterGetSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Param);

	return PinProperties;
}

void UPCGGenericUserParameterGetSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::OptionSanitizeOutputAttributeNamesPCG)
	{
		// For all previous nodes, we'll force this option to false. for retro-compatibility
		bSanitizeOutputAttributeName = false;
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bForceObjectAndStructExtraction)
	{
		bForceObjectAndStructExtraction = false;
		StructExtractorBehavior = EPCGStructExtractorBehavior::Extract;
		ObjectExtractorBehavior = EPCGObjectExtractorBehavior::Extract;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

FString UPCGGenericUserParameterGetSettings::GetAdditionalTitleInformation() const
{
	return PropertyPath;
}

#if WITH_EDITOR
EPCGChangeType UPCGGenericUserParameterGetSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGenericUserParameterGetSettings, PropertyPath))
	{
		ChangeType |= EPCGChangeType::Cosmetic;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGGenericUserParameterGetSettings::CreateElement() const
{
	return MakeShared<FPCGUserParameterGetElement>();
}

#if WITH_EDITOR
TArray<FPCGPreconfiguredInfo> UPCGGenericUserParameterGetSettings::GetConversionInfo() const
{
	using namespace PCGUserParameterGet::Constants::Conversion;
	TArray<FPCGPreconfiguredInfo> ConversionInfo;

	ConversionInfo.Emplace(
		GenericToSpecificIndex,
		PCGConversion::Helpers::GetDefaultNodeTitle(UPCGUserParameterGetSettings::StaticClass()),
		GenericToSpecificTooltip);

	ConversionInfo.Emplace(
		GenericToConstantIndex,
		PCGConversion::Helpers::GetDefaultNodeTitle(UPCGCreateAttributeSetSettings::StaticClass()),
		ToCreateConstantTooltip);

	return ConversionInfo;
}
#endif // WITH_EDITOR

bool UPCGGenericUserParameterGetSettings::ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo)
{
	using namespace PCGUserParameterGet::Constants::Conversion;
	UPCGNode* Node = CastChecked<UPCGNode>(GetOuter());

	if (ConversionInfo.PreconfiguredIndex == GenericToSpecificIndex)
	{
		// If the property is overridden or upstream, abort.
		if (IsPropertyOverriddenByPin(
			GET_MEMBER_NAME_CHECKED(UPCGGenericUserParameterGetSettings, PropertyPath))
			|| Source != EPCGUserParameterSource::Current)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("GenericInvalidConversion", "Can't convert Get User Parameter with an overridden property or upstream source."));
			return false;
		}

		FPCGSingleNodeConverter NodeConverter(Node, UPCGUserParameterGetSettings::StaticClass());
		if (!NodeConverter.IsGraphInitialized() || !NodeConverter.IsSourceInitialized())
		{
			PCGLog::Settings::LogInvalidPreconfigurationWarning(ConversionInfo.PreconfiguredIndex, GenericGetUserParameterNodeTitle);
			return false;
		}

		NodeConverter.PrepareData();

		bool bParameterSuccess = false;
		if (UPCGUserParameterGetSettings* Settings = Cast<UPCGUserParameterGetSettings>(NodeConverter.GetGeneratedSettings()))
		{
			if (const UPCGGraph* Graph = NodeConverter.GetGraph())
			{
				if (const FInstancedPropertyBag* UserParametersStruct = Graph->GetUserParametersStruct())
				{
					// Use the base attribute and ignore extractors.
					FPCGAttributePropertySelector Selector;
					Selector.Update(PropertyPath);
					if (const FPropertyBagPropertyDesc* Desc = UserParametersStruct->FindPropertyDescByName(Selector.GetAttributeName()))
					{
						Settings->PropertyName = Desc->Name;
						Settings->PropertyGuid = Desc->ID;
						bParameterSuccess = true;
					}
				}
			}
		}

		if (bParameterSuccess)
		{
			NodeConverter.ApplyStructural();
			NodeConverter.Finalize();

			return NodeConverter.IsComplete();
		}
	}
	else if (ConversionInfo.PreconfiguredIndex == GenericToConstantIndex)
	{
		FPCGSingleNodeConverter NodeConverter(Node, UPCGCreateAttributeSetSettings::StaticClass());
		if (!NodeConverter.IsGraphInitialized() || !NodeConverter.IsSourceInitialized())
		{
			PCGLog::Settings::LogInvalidPreconfigurationWarning(ConversionInfo.PreconfiguredIndex, GenericGetUserParameterNodeTitle);
			return false;
		}

		NodeConverter.PrepareData();

		if (UPCGGraph* Graph = NodeConverter.GetGraph())
		{
			const FName PropertyName(PropertyPath);
			UPCGCreateAttributeSetSettings* Settings = CastChecked<UPCGCreateAttributeSetSettings>(NodeConverter.GetGeneratedSettings());
			Settings->OutputTarget.Update(OutputAttributeName.ToString());

			FText ErrorMessage;
			if (PCGUserParameterGet::Helpers::Conversion::AssignValue(Settings->AttributeTypes, Graph, PropertyName, ErrorMessage))
			{
				NodeConverter.ApplyStructural();
				NodeConverter.Finalize();
			}
			else
			{
				PCGLog::Settings::LogInvalidConversionError(ConversionInfo.PreconfiguredIndex, GenericGetUserParameterNodeTitle, ErrorMessage);
			}
		}

		return NodeConverter.IsComplete();
	}

	return false;
}

bool FPCGUserParameterGetElement::CanExecuteOnlyOnMainThread(FPCGContext* Context) const
{
	// If we have no user parameter data as input, we'll have to access the parameters on the component/graph directly, which is not thread safe
	// If there is no context, we can't check the inputs, so do the assumption it is possible to run on another thread, and we'll ask to come back to
	// the main thread on execute in the unlikely possibility that the parameters are not there. 
	return Context ? Algo::NoneOf(Context->InputData.TaggedData, [](const FPCGTaggedData& InData) { return InData.Data.IsA<UPCGUserParametersData>(); }) : false;
}

//////////////////////////////////////////////

bool FPCGUserParameterGetElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGUserParameterGetElement::Execute);

	check(Context);
	// If we have no user parameter data as input, we'll have to access the parameters on the component/graph directly, which is not thread safe
	if (CanExecuteOnlyOnMainThread(Context) && !IsInGameThread())
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("ForcedBackOnMainThread", "Get Graph Parameter node is forced back on the main thread because it doesn't have UPCGUserParametersData as (implicit) input.\n"
			"If the graph was scheduled manually through the Subsystem, make sure that you provide no PreTask element, or compute the UserParameterData in it."), Context);
		Context->RequestToRunOnMainThread();
		return false;
	}
	
	const UPCGUserParameterGetSettings* Settings = Context->GetInputSettings<UPCGUserParameterGetSettings>();
	const UPCGGenericUserParameterGetSettings* GenericSettings = Context->GetInputSettings<UPCGGenericUserParameterGetSettings>();
	check(Settings || GenericSettings);

	bool bForceQuiet = (GenericSettings && GenericSettings->bQuiet);

	TArray<FConstStructView, TInlineAllocator<16>> ValidLayouts = PCGUserParameterGet::Settings::GetValidLayouts(*Context);

	// Remove layouts we're not interested in, as per settings
	if (ValidLayouts.Num() > 1)
	{
		if (GenericSettings)
		{
			if (GenericSettings->Source == EPCGUserParameterSource::Current)
			{
				ValidLayouts.SetNum(1);
			}
			else if (GenericSettings->Source == EPCGUserParameterSource::Upstream)
			{
				ValidLayouts.RemoveAt(0);
			}
			else if (GenericSettings->Source == EPCGUserParameterSource::Root)
			{
				ValidLayouts.RemoveAtSwap(0);
				ValidLayouts.SetNum(1);
			}
		}
		else
		{
			// Not a generic get settings -> default to taking the current layout only
			ValidLayouts.SetNum(1);
		}
	}

	if (ValidLayouts.IsEmpty())
	{
		// In theory this should never happen, as the PCG graph should always have a user parameter struct, even if it is empty.
		PCGLog::LogErrorOnGraph(LOCTEXT("NoValidLayout", "Failed to find a valid user parameters to extract from."));
		return true;
	}

	for (int CurrentLayoutIndex = 0; CurrentLayoutIndex < ValidLayouts.Num(); ++CurrentLayoutIndex)
	{
		const bool bIsLastIteration = (CurrentLayoutIndex == ValidLayouts.Num() - 1);
		FConstStructView Parameters = ValidLayouts[CurrentLayoutIndex];
		const UScriptStruct* PropertyBag = Parameters.GetScriptStruct();

		PCGPropertyHelpers::FExtractorParameters ExtractorParameters;

		if (Settings)
		{
			ExtractorParameters = PCGPropertyHelpers::FExtractorParameters(Parameters.GetMemory(),
				PropertyBag,
				FPCGAttributePropertySelector::CreateAttributeSelector(Settings->PropertyName),
				Settings->PropertyName,
				/*bPropertyNeedsToBeVisible=*/false);
			
			ExtractorParameters.StructExtractorBehavior = Settings->StructExtractorBehavior;
			ExtractorParameters.ObjectExtractorBehavior = Settings->ObjectExtractorBehavior;
			ExtractorParameters.ContainerExtractorBehavior = Settings->ContainerExtractorBehavior;

			ExtractorParameters.bStrictSanitizeOutputAttributeNames = Settings->bSanitizeOutputAttributeName;
		}
		else if (GenericSettings)
		{
			ExtractorParameters = PCGPropertyHelpers::FExtractorParameters(Parameters.GetMemory(),
				PropertyBag,
				GenericSettings->PropertyPath,
				GenericSettings->OutputAttributeName,
				/*bPropertyNeedsToBeVisible=*/false);
			
			ExtractorParameters.StructExtractorBehavior = GenericSettings->StructExtractorBehavior;
			ExtractorParameters.ObjectExtractorBehavior = GenericSettings->ObjectExtractorBehavior;
			ExtractorParameters.ContainerExtractorBehavior = GenericSettings->ContainerExtractorBehavior;

			ExtractorParameters.bStrictSanitizeOutputAttributeNames = GenericSettings->bSanitizeOutputAttributeName;
		}

		// Don't care for object traversed in non-editor build, since it is only useful for tracking.
		TSet<FSoftObjectPath>* ObjectTraversedPtr = nullptr;
#if WITH_EDITOR
		TSet<FSoftObjectPath> ObjectTraversed;
		ObjectTraversedPtr = &ObjectTraversed;
#endif // WITH_EDITOR

		if (UPCGParamData* ParamData = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(ExtractorParameters, Context, ObjectTraversedPtr, /*bQuiet=*/bForceQuiet || !bIsLastIteration))
		{
			Context->OutputData.TaggedData.Emplace_GetRef().Data = ParamData;
		}
		else if (!bIsLastIteration)
		{
			continue;
		}
		else // Final iteration, should break & report errors
		{
			if (!ExtractorParameters.PropertySelectors.IsEmpty())
			{
				PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InvalidProperty", "Could not find the property '{0}' in the user parameters"), ExtractorParameters.PropertySelectors[0].GetDisplayText()));
			}
			else
			{
				PCGE_LOG(Error, GraphAndLog, LOCTEXT("InvalidNamelessProperty", "Could not find nameless property in the user parameters"));
			}
		}

		// Register dynamic tracking
#if WITH_EDITOR
		if (!ObjectTraversed.IsEmpty())
		{
			FPCGDynamicTrackingHelper DynamicTracking;
			DynamicTracking.EnableAndInitialize(Context, ObjectTraversed.Num());
			for (FSoftObjectPath& Path : ObjectTraversed)
			{
				DynamicTracking.AddToTracking(FPCGSelectionKey::CreateFromPath(std::move(Path)), /*bCulled=*/false);
			}

			DynamicTracking.Finalize(Context);
		}
#endif // WITH_EDITOR

		break;
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
