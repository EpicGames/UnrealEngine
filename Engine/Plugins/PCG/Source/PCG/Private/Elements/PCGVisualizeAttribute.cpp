// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGVisualizeAttribute.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGDebugDrawComponent.h"
#include "Data/PCGBasePointData.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGVisualizeAttribute)

#define LOCTEXT_NAMESPACE "PCGVisualizeAttributeElement"

FPCGElementPtr UPCGVisualizeAttributeSettings::CreateElement() const
{
	return MakeShared<FPCGVisualizeAttribute>();
}

bool FPCGVisualizeAttribute::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGVisualizeAttribute::ExecuteInternal);
	check(IsInGameThread());

	Context->OutputData = Context->InputData;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	const UPCGVisualizeAttributeSettings* Settings = Context->GetInputSettings<UPCGVisualizeAttributeSettings>();
	check(Settings);

	if (!Settings->bVisualizeEnabled)
	{
		return true;
	}

	UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get());
	if (!ensure(SourceComponent))
	{
		return true;
	}

	TArray<UPCGDebugDrawComponent*> DebugDrawComponents;

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	for (const FPCGTaggedData& InputData : Inputs)
	{
		const UPCGBasePointData* PointData = Cast<const UPCGBasePointData>(InputData.Data);
		if (!PointData)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidInputType", "Input data type is not supported, only supports Point Data."), Context);
			continue;
		}

		// No points, early out
		if (PointData->IsEmpty())
		{
			continue;
		}

		if (!PointData->Metadata)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidMetadata", "Input has no metadata."), Context);
			continue;
		}

		FPCGAttributePropertyInputSelector InputSource = Settings->AttributeSource.CopyAndFixLast(PointData);

		TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(PointData, InputSource);
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(PointData, InputSource);

		if (!Accessor.IsValid() || !Keys.IsValid())
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("AttributeDoesNotExist", "Input attribute/property '{0}' does not exist - skipped."), InputSource.GetDisplayText()), Context);
			continue;
		}

		TArray<FString> ValuesToString;
		const int32 NumValues = Keys->GetNum();
		if (NumValues == 0)
		{
			continue;
		}

		ValuesToString.SetNum(NumValues);

		// Get all the attribute values at once in range, for efficiency
		if (!Accessor->GetRange<FString>(ValuesToString, 0, *Keys, EPCGAttributeAccessorFlags::AllowBroadcast))
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidValues", "Could not get the range of attribute values."), Context);
			continue;
		}

		UPCGDebugDrawComponent* DebugDrawComponent = nullptr;

		AActor* TargetActor = Context->GetTypedExecutionTarget<AActor>();
		if (TargetActor)
		{
			// Grab it if it exists already
			DebugDrawComponent = TargetActor->GetComponentByClass<UPCGDebugDrawComponent>();
		}
		else
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("NoTargetActor", "Internal error: Could not find target actor."), Context);
			continue;
		}

		// If it doesn't exist already, create a new one
		if (!DebugDrawComponent)
		{
			DebugDrawComponent = NewObject<UPCGDebugDrawComponent>(TargetActor, FName(LOCTEXT("PCGDebugDrawComponent", "PCG Debug Draw Component").ToString()), RF_Transient);

			TargetActor->Modify(/*bAlwaysMarkDirty=*/false);
			DebugDrawComponent->RegisterComponent();
			TargetActor->AddInstanceComponent(DebugDrawComponent);
			UPCGManagedDebugDrawComponent* ManagedComponent = NewObject<UPCGManagedDebugDrawComponent>(SourceComponent);
			ManagedComponent->GeneratedComponent = DebugDrawComponent;
			SourceComponent->AddToManagedResources(ManagedComponent);
		}

		const TConstPCGValueRange<FTransform> TransformRanges = PointData->GetConstTransformValueRange();
		TArray<FDebugRenderSceneProxy::FText3d> DebugStrings;

		const int32 NumPoints = FMath::Min(PointData->GetNumPoints(), Settings->PointLimit);
		DebugStrings.Reserve(NumPoints);

		// TODO: Optimize which points will be drawn by distance, or other measures, rather than by Index
		// Debug messages are already culled by camera frustum, but a point limit is desired to prevent overloading
		for (int I = 0; I < NumPoints; ++I)
		{
			TStringBuilder<256> CompoundedString;

			CompoundedString += Settings->CustomPrefixString;

			if (Settings->bPrefixWithIndex)
			{
				CompoundedString += FString::Printf(TEXT("[%d]"), I);
			}

			if (Settings->bPrefixWithAttributeName)
			{
				CompoundedString += FString::Printf(TEXT("[%s]"), *InputSource.GetDisplayText().ToString());
			}

			CompoundedString = FString::Printf(TEXT("%s %s"), *CompoundedString, *ValuesToString[I % NumValues]);

			DebugStrings.Emplace(FString(CompoundedString), TransformRanges[I].GetLocation() + Settings->LocalOffset, Settings->Color);
		}

		check(DebugDrawComponent);
		DebugDrawComponent->AddDebugStrings(DebugStrings);
		DebugDrawComponents.AddUnique(DebugDrawComponent);
	}

	for (UPCGDebugDrawComponent* DebugDrawComponent : DebugDrawComponents)
	{
		check(DebugDrawComponent);
		DebugDrawComponent->StartTimer(Settings->Duration);
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING

	return true;
}

#undef LOCTEXT_NAMESPACE
