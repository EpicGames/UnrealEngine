// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGVolumeSampler.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGTimeSlicedElementBase.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "HAL/UnrealMemory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGVolumeSampler)

#define LOCTEXT_NAMESPACE "PCGVolumeSamplerElement"

namespace PCGVolumeSampler
{
	UPCGPointData* SampleVolume(FPCGContext* Context, const FVolumeSamplerParams& SamplerSettings, const UPCGSpatialData* Volume, const UPCGSpatialData* BoundingShape)
	{
		return Cast<UPCGPointData>(SampleVolume(Context, UPCGPointData::StaticClass(), SamplerSettings, Volume, BoundingShape));
	}

	UPCGBasePointData* SampleVolume(FPCGContext* Context, const UPCGSpatialData* Volume, const FVolumeSamplerParams& SamplerSettings, const UPCGSpatialData* BoundingShape)
	{
		return SampleVolume(Context, nullptr, SamplerSettings, Volume, BoundingShape);
	}

	UPCGBasePointData* SampleVolume(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass, const FVolumeSamplerParams& SamplerSettings, const UPCGSpatialData* Volume, const UPCGSpatialData* BoundingShape)
	{
		UPCGBasePointData* Data = PointDataClass != nullptr ? FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass) : FPCGContext::NewPointData_AnyThread(Context);

		FPCGInitializeFromDataParams InitializeFromDataParams(Volume);
		InitializeFromDataParams.bInheritSpatialData = false;
		Data->InitializeFromDataWithParams(InitializeFromDataParams);

		const bool bTimeSliceIsEnabled = Context ? Context->TimeSliceIsEnabled() : false;
		SampleVolume(Context, SamplerSettings, Volume, BoundingShape, Data, bTimeSliceIsEnabled);

		return Data;
	}

	bool SampleVolume(FPCGContext* Context, const FVolumeSamplerParams& SamplerSettings, const UPCGSpatialData* Volume, const UPCGSpatialData* BoundingShape, UPCGBasePointData* OutputData, const bool bTimeSlicingIsEnabled)
	{
		check(Volume && OutputData);

		FBox Bounds(SamplerSettings.Bounds);
		if (!Bounds.IsValid)
		{
			Bounds = Volume->GetBounds();
			// Early out
			if (!Bounds.IsValid)
			{
				return true;
			}
		}

		const FVector& VoxelSize = SamplerSettings.VoxelSize;

		const int32 MinX = FMath::CeilToInt(Bounds.Min.X / VoxelSize.X);
		const int32 MaxX = FMath::FloorToInt(Bounds.Max.X / VoxelSize.X);
		const int32 MinY = FMath::CeilToInt(Bounds.Min.Y / VoxelSize.Y);
		const int32 MaxY = FMath::FloorToInt(Bounds.Max.Y / VoxelSize.Y);
		const int32 MinZ = FMath::CeilToInt(Bounds.Min.Z / VoxelSize.Z);
		const int32 MaxZ = FMath::FloorToInt(Bounds.Max.Z / VoxelSize.Z);

		// Set uninitialized, then carefully initialize step by step with overflow checks
		int32 NumIterations = -1;

		{
			const int64 NumX = 1 + MaxX - MinX;
			const int64 NumY = 1 + MaxY - MinY;
			const int64 NumZ = 1 + MaxZ - MinZ;
			const int64 NumIterationsXY64 = NumX * NumY;
			const int64 NumIterations64 = NumIterationsXY64 * NumZ;

			if (NumX <= 0 || NumY <= 0 || NumZ <= 0)
			{
				if (Context)
				{
					PCGE_LOG_C(Verbose, LogOnly, Context, FText::Format(FText::FromString(TEXT("Skipped - invalid cell bounds ({0} x {1} x {2})")), NumX, NumY, NumZ));
				}

				return true;
			}

			if (NumIterationsXY64 > 0 &&
				NumIterationsXY64 < MAX_int32 &&
				NumIterations64 > 0 &&
				NumIterations64 < MAX_int32 &&
				(!PCGFeatureSwitches::CVarCheckSamplerMemory.GetValueOnAnyThread()
					|| PCGFeatureSwitches::Helpers::GetAvailableMemoryForSamplers() >= sizeof(FPCGPoint) * NumIterations64))
			{
				NumIterations = static_cast<int32>(NumIterations64);
			}
			else
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("TooManyPoints", "Skipped - tried to generate too many points ({0} x {1} x {2} = {3}).\nAdjust 'pcg.SamplerMemoryThreshold' if needed."), NumX, NumY, NumZ, NumIterations64), Context);
				return true;
			}
		}

		auto InitializeFunc = [NumIterations, OutputData]()
		{
			OutputData->SetNumPoints(NumIterations, /*bInitializeValues=*/false);
			// Volume->SamplePoint could write to all properties
			OutputData->AllocateProperties(EPCGPointNativeProperties::All);
		};

		auto MoveDataRangeFunc = [OutputData](int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
		{
			OutputData->MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		};

		auto FinishedFunc = [OutputData](int32 NumWritten)
		{
			OutputData->SetNumPoints(NumWritten);
		};

		auto ProcessRangeFunc = [OutputData, Volume, BoundingShape, PointSteepness = SamplerSettings.PointSteepness, VoxelSize, MinX, MaxX, MinY, MaxY, MinZ](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			int32 NumWritten = 0;

			FPCGPointValueRanges OutRanges(OutputData, /*bAllocate=*/false);

			for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
			{
				const int32 WriteIndex = StartWriteIndex + NumWritten;

				const int X = MinX + (ReadIndex % (1 + MaxX - MinX));
				const int Y = MinY + (ReadIndex / (1 + MaxX - MinX) % (1 + MaxY - MinY));
				const int Z = MinZ + (ReadIndex / ((1 + MaxX - MinX) * (1 + MaxY - MinY)));

				const FVector SampleLocation(X * VoxelSize.X, Y * VoxelSize.Y, Z * VoxelSize.Z);
				const FBox VoxelBox(VoxelSize * -0.5, VoxelSize * 0.5);

				const FTransform SampleTransform = FTransform(SampleLocation);

				FPCGPoint OutPoint;

				if (Volume->SamplePoint(SampleTransform, VoxelBox, OutPoint, nullptr))
				{
					if (BoundingShape)
					{
						FPCGPoint BoundingShapeSample;
						if (!BoundingShape->SamplePoint(SampleTransform, VoxelBox, BoundingShapeSample, nullptr))
						{
							continue;
						}
					}

					OutPoint.Seed = PCGHelpers::ComputeSeed(X, Y, Z);
					OutPoint.Steepness = PointSteepness;
					// Reset the bounds to the expected bounds, if it was modified by the Sample Point
					OutPoint.SetLocalBounds(VoxelBox);

					OutRanges.SetFromPoint(WriteIndex, OutPoint);
					++NumWritten;
				}
			}

			return NumWritten;
		};

		FPCGAsyncState* AsyncState = Context ? &Context->AsyncState : nullptr;
		
		return FPCGAsync::AsyncProcessingRangeEx(
			AsyncState,
			NumIterations,
			InitializeFunc,
			ProcessRangeFunc,
			MoveDataRangeFunc,
			FinishedFunc,
			/*bEnableTimeSlicing=*/Context && bTimeSlicingIsEnabled);
	}
}
UPCGVolumeSamplerSettings::UPCGVolumeSamplerSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		PointSteepness = 1.0f;
	}
}

#if WITH_EDITOR
FText UPCGVolumeSamplerSettings::GetNodeTooltipText() const
{
	return LOCTEXT("VolumeSamplerNodeTooltip", "Generates points in the three dimensional bounds of the Volume input and within the Bounding Shape input if provided.");
}
#endif

TArray<FPCGPinProperties> UPCGVolumeSamplerSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	// Spatial is ok - volume sampling just needs bounds.
	FPCGPinProperties& VolumePinProperty = PinProperties.Emplace_GetRef(PCGVolumeSamplerConstants::VolumeLabel, EPCGDataType::Spatial, /*bAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true, LOCTEXT("VolumeSamplerVolumePinTooltip",
		"The volume to sample with points. Can be any spatial data that can provide bounds."
	));
	VolumePinProperty.SetRequiredPin();

	// Only one connection/data allowed. To avoid ambiguity, samplers should require users to union or intersect multiple shapes.
	PinProperties.Emplace(PCGVolumeSamplerConstants::BoundingShapeLabel, EPCGDataType::Spatial, /*bInAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false, LOCTEXT("VolumeSamplerBoundingShapePinTooltip",
		"Optional. All sampled points must be contained within this shape."
	));

	return PinProperties;
}

FPCGElementPtr UPCGVolumeSamplerSettings::CreateElement() const
{
	return MakeShared<FPCGVolumeSamplerElement>();
}

namespace PCGVolumeSamplerHelpers
{
	using ContextType = FPCGVolumeSamplerElement::ContextType;
	using ExecStateType = FPCGVolumeSamplerElement::ExecStateType;

	EPCGTimeSliceInitResult InitializePerExecutionData(ContextType* Context, ExecStateType& OutState)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGVolumeSamplerElement::InitializePerExecutionData);

		check(Context);
		const UPCGVolumeSamplerSettings* Settings = Context->GetInputSettings<UPCGVolumeSamplerSettings>();
		check(Settings);

		const FVector& VoxelSize = Settings->VoxelSize;
		if (VoxelSize.X <= 0 || VoxelSize.Y <= 0 || VoxelSize.Z <= 0)
		{
			PCGE_LOG_C(Warning, GraphAndLog, Context, LOCTEXT("InvalidVoxelSize", "Skipped - Invalid voxel size"));
			return EPCGTimeSliceInitResult::AbortExecution;
		}

		const TArray<FPCGTaggedData> VolumeInputs = Context->InputData.GetInputsByPin(PCGVolumeSamplerConstants::VolumeLabel);
		// Grab the Bounding Shape input if there is one.
		TArray<FPCGTaggedData> BoundingShapeInputs = Context->InputData.GetInputsByPin(PCGVolumeSamplerConstants::BoundingShapeLabel);

		bool bUsedDefaultBoundingShape = false;
		if (!Settings->bUnbounded)
		{
			bool bUnionWasCreated;
			// Get a union of inputs and if successful, add it to the root. Will be removed and marked for GC in the state destructor
			OutState.BoundingShape = Context->InputData.GetSpatialUnionOfInputsByPin(Context, PCGVolumeSamplerConstants::BoundingShapeLabel, bUnionWasCreated);
			if (OutState.BoundingShape && bUnionWasCreated)
			{
				Context->TrackObject(OutState.BoundingShape);
			}

			if (!OutState.BoundingShape && Context->ExecutionSource.IsValid())
			{
				// Create a bounding shape from the actor data
				OutState.BoundingShape = Cast<UPCGSpatialData>(Context->ExecutionSource->GetExecutionState().GetSelfData());
				bUsedDefaultBoundingShape = true;
			}
		}
		else if (BoundingShapeInputs.Num() > 0)
		{
			PCGE_LOG_C(Verbose, LogOnly, Context, LOCTEXT("BoundsIgnored", "The bounds of the Bounding Shape input pin will be ignored because the Unbounded option is enabled"));
		}

		FBox& BoundingShapeBounds = OutState.BoundingShapeBounds;
		// Compute bounds of bounding shape input
		if (OutState.BoundingShape)
		{
			BoundingShapeBounds = OutState.BoundingShape->GetBounds();
		}

		TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
		// Construct a list of shapes to generate samples from. Prefer to get these directly from the first input pin.
		TArray<const UPCGSpatialData*>& GeneratingShapes = OutState.GeneratingShapes;
		GeneratingShapes.Reserve(VolumeInputs.Num());
		for (const FPCGTaggedData& TaggedData : VolumeInputs)
		{
			if (const UPCGSpatialData* SpatialData = Cast<UPCGSpatialData>(TaggedData.Data))
			{
				GeneratingShapes.Add(SpatialData);
				Outputs.Add(TaggedData);
			}
		}

		// Warn if something is connected but no spatial data could be obtained for sampling
		if (GeneratingShapes.IsEmpty() && (BoundingShapeInputs.Num() > 0 || VolumeInputs.Num() > 0))
		{
			PCGE_LOG_C(Warning, GraphAndLog, Context, LOCTEXT("NoShapeToSample", "No Spatial data shape was provided for sampling. No points will be sampled."));
			return EPCGTimeSliceInitResult::NoOperation;
		}

		return EPCGTimeSliceInitResult::Success;
	}
}

bool FPCGVolumeSamplerElement::PrepareDataInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGVolumeSamplerElement::PrepareDataInternal);
	ContextType* TimeSlicedContext = static_cast<ContextType*>(Context);
	check(TimeSlicedContext);

	const UPCGVolumeSamplerSettings* Settings = TimeSlicedContext->GetInputSettings<UPCGVolumeSamplerSettings>();
	check(Settings);

	if (TimeSlicedContext->InitializePerExecutionState(PCGVolumeSamplerHelpers::InitializePerExecutionData) == EPCGTimeSliceInitResult::AbortExecution)
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("CouldNotInitializeExecutionState", "Could not initialize per-execution timeslice state data"));
		return true;
	}

	// The generating shapes will be used for the time slicing iterations
	TArray<const UPCGSpatialData*>& GeneratingShapes = TimeSlicedContext->GetPerExecutionState().GeneratingShapes;
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	TimeSlicedContext->InitializePerIterationStates(GeneratingShapes.Num(),
		[&GeneratingShapes, &Settings, &Outputs, &Context](IterStateType& OutState, const ExecStateType& ExecState, const uint32 IterationIndex)
		{
			OutState.Settings.VoxelSize = Settings->VoxelSize;
			OutState.Settings.PointSteepness = Settings->PointSteepness;

			const UPCGSpatialData* GeneratingShape = GeneratingShapes[IterationIndex];
			check(GeneratingShape);

			OutState.Volume = GeneratingShape;
			OutState.OutputPointData = FPCGContext::NewPointData_AnyThread(Context);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OutState.OutputData = Cast<UPCGPointData>(OutState.OutputPointData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			FPCGInitializeFromDataParams InitializeFromDataParams(OutState.Volume);
			InitializeFromDataParams.bInheritSpatialData = false;
			OutState.OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);
			Outputs[IterationIndex].Data = OutState.OutputPointData;

			FBox& InputBounds = OutState.Settings.Bounds;

			// Get the bounding shape bounds from the execution state
			const FBox& BoundingShapeBounds = ExecState.BoundingShapeBounds;

			// Calculate the intersection of bounds of the provided inputs
			if (GeneratingShape->IsBounded())
			{
				InputBounds = GeneratingShape->GetBounds();

				if (BoundingShapeBounds.IsValid)
				{
					InputBounds = PCGHelpers::OverlapBounds(InputBounds, BoundingShapeBounds);
				}
			}
			else
			{
				InputBounds = BoundingShapeBounds;
			}

			if (!InputBounds.IsValid)
			{
				if (!GeneratingShape->IsBounded())
				{
					// Some inputs are unable to provide bounds, like the WorldVolumetricQuery, in which case the user must provide bounds.
					PCGE_LOG_C(Warning, GraphAndLog, Context, LOCTEXT("CouldNotObtainInputBounds", "Input data is not bounded, so bounds must be provided for sampling. Consider providing a Bounding Shape input."));
				}
				else
				{
					PCGE_LOG_C(Verbose, LogOnly, Context, LOCTEXT("InvalidSamplingBounds", "Final sampling bounds is invalid/zero-sized."));
				}

				return EPCGTimeSliceInitResult::NoOperation;
			}

			return EPCGTimeSliceInitResult::Success;
		});

	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		PCGE_LOG(Warning, GraphAndLog, LOCTEXT("CouldNotInitializeStateData", "Could not initialize timeslice state data"));
		return true;
	}

	return true;
}

bool FPCGVolumeSamplerElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGVolumeSamplerElement::Execute);
	ContextType* TimeSlicedContext = static_cast<ContextType*>(Context);
	check(TimeSlicedContext);

	// Abort execution was called at some point during the state initialization
	if (!TimeSlicedContext->DataIsPreparedForExecution())
	{
		TimeSlicedContext->OutputData.TaggedData.Empty();

		return true;
	}

	// The execution would have resulted in an empty set of points for all iterations
	if (TimeSlicedContext->GetExecutionStateResult() == EPCGTimeSliceInitResult::NoOperation)
	{
		for (const FPCGTaggedData& Input : TimeSlicedContext->InputData.GetInputsByPin(PCGVolumeSamplerConstants::VolumeLabel))
		{
			// TODO: Empty point data (to preserve previous behavior). Eventually, should be replaced with no output at all
			FPCGTaggedData& Output = TimeSlicedContext->OutputData.TaggedData.Add_GetRef(Input);
			UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(Context);
			
			FPCGInitializeFromDataParams InitializeFromDataParams(Cast<UPCGSpatialData>(Input.Data));
			InitializeFromDataParams.bInheritSpatialData = false;
			PointData->InitializeFromDataWithParams(InitializeFromDataParams);

			Output.Data = PointData;
		}

		return true;
	}

	return ExecuteSlice(TimeSlicedContext, [](ContextType* Context, const ExecStateType& ExecState, const IterStateType& IterState, const uint32 IterationIndex)->bool
	{
		const EPCGTimeSliceInitResult InitResult = Context->GetIterationStateResult(IterationIndex);

		if (InitResult == EPCGTimeSliceInitResult::NoOperation)
		{
			Context->OutputData.TaggedData[IterationIndex].Data = FPCGContext::NewPointData_AnyThread(Context);

			return true;
		}

		check(InitResult == EPCGTimeSliceInitResult::Success);

		const bool bAsyncDone = PCGVolumeSampler::SampleVolume(
				Context,
				IterState.Settings,
				ExecState.GeneratingShapes[IterationIndex],
				ExecState.BoundingShape,
				IterState.OutputPointData,
				Context->TimeSliceIsEnabled());

		PCGE_LOG_C(Verbose, LogOnly, Context, FText::Format(LOCTEXT("GenerationInfo", "Generated {0} points in volume"), IterState.OutputPointData->GetNumPoints()));

		return bAsyncDone;
	});
}

void FPCGVolumeSamplerElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	IPCGElement::GetDependenciesCrc(InParams, Crc);

	if (const UPCGVolumeSamplerSettings* Settings = Cast<UPCGVolumeSamplerSettings>(InParams.Settings))
	{
		bool bUnbounded;
		PCGSettingsHelpers::GetOverrideValue(*InParams.InputData, Settings, GET_MEMBER_NAME_CHECKED(UPCGVolumeSamplerSettings, bUnbounded), Settings->bUnbounded, bUnbounded);
		const bool bBoundsConnected = InParams.InputData->GetInputsByPin(PCGVolumeSamplerConstants::BoundingShapeLabel).Num() > 0;

		// If we're operating in bounded mode and there is no bounding shape connected then we'll use actor bounds, and therefore take
		// dependency on actor data.
		if (!bUnbounded && !bBoundsConnected && InParams.ExecutionSource)
		{
			if (const UPCGData* Data = InParams.ExecutionSource->GetExecutionState().GetSelfData())
			{
				Crc.Combine(Data->GetOrComputeCrc(/*bFullDataCrc=*/false));
			}
		}
	}

	OutCrc = Crc;
}

#if WITH_EDITOR
void UPCGVolumeSamplerSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	if (DataVersion < FPCGCustomVersion::SplitVolumeSamplerNodeInputs && ensure(InOutNode))
	{
		if (InputPins.Num() > 0 && InputPins[0])
		{
			// First pin renamed in this version. Rename here so that edges won't get culled in UpdatePins later.
			InputPins[0]->Properties.Label = PCGVolumeSamplerConstants::VolumeLabel;
		}
	}

	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
