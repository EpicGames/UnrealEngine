// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGWorldQuery.h"

#include "PCGContext.h"
#include "PCGSubsystem.h"
#include "Grid/PCGLandscapeCache.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGWorldQueryHelpers.h"

#include "GameFramework/Actor.h"
#include "LandscapeProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGWorldQuery)

#define LOCTEXT_NAMESPACE "PCGWorldQuery"

namespace PCGWorldQuery
{
	template <typename WorldQueryDataType>
	void ExtractObjectFiltersIfNeeded(const FPCGWorldCommonQueryParams& QueryParams, FPCGContext* Context, WorldQueryDataType* WorldQueryData)
	{
		check(WorldQueryData);
		
		if (QueryParams.ActorFilterFromInput != EPCGWorldQueryFilter::None)
		{
			const TArray<FPCGTaggedData> ActorFilterTaggedData = Context->InputData.GetInputsByPin(PCGWorldRayHitConstants::FilterActorPinLabel);
			if (ActorFilterTaggedData.Num() > 1)
			{
				PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGWorldRayHitConstants::FilterActorPinLabel, Context);
			}

			const UPCGData* ActorFilterData = ActorFilterTaggedData.IsEmpty() ? nullptr : ActorFilterTaggedData[0].Data;
			if (ActorFilterData && !QueryParams.ExtractObjectFiltersIfNeeded(ActorFilterData, WorldQueryData->GetMutableFilteredObjects(), Context))
			{
				PCGLog::LogWarningOnGraph(LOCTEXT("FailExtractActorFilters", "Failed to extract actor filters."));
			}
		}
	}
}

#if WITH_EDITOR
FText UPCGWorldQuerySettings::GetNodeTooltipText() const
{
	return LOCTEXT("WorldQueryTooltip", "Allows generic access (based on overlaps) to collisions in the world that behaves like a volume.");
}
#endif

TArray<FPCGPinProperties> UPCGWorldQuerySettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	QueryParams.AddFilterPinIfNeeded(PinProperties);
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGWorldQuerySettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Volume);

	return PinProperties;
}

FPCGElementPtr UPCGWorldQuerySettings::CreateElement() const
{
	return MakeShared<FPCGWorldVolumetricQueryElement>();
}

#if WITH_EDITOR
EPCGChangeType UPCGWorldQuerySettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	EPCGChangeType Result = Super::GetChangeTypeForProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGWorldQuerySettings, QueryParams) &&
		PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FPCGWorldCommonQueryParams, ActorFilterFromInput))
	{
		// This can add/remove a pin, so we need a structural change
		Result |= EPCGChangeType::Structural;
	}

	return Result;
}
#endif // WITH_EDITOR

bool FPCGWorldVolumetricQueryElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWorldVolumetricQueryElement::Execute);

	const UPCGWorldQuerySettings* Settings = Context->GetInputSettings<UPCGWorldQuerySettings>();
	check(Settings);

	FPCGWorldVolumetricQueryParams QueryParams = Settings->QueryParams;
	
	// TODO: Add params pin + Apply param data overrides

	check(Context->ExecutionSource.IsValid());

	if (!Context->ExecutionSource->GetExecutionState().GetWorld())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NoWorld_VolumetricQuery", "Execution source does not have a valid world."), Context);
		return true;
	}

	UPCGWorldVolumetricData* Data = FPCGContext::NewObject_AnyThread<UPCGWorldVolumetricData>(Context);
	Data->Initialize(Context->ExecutionSource.Get());
	Data->QueryParams = QueryParams;
	Data->QueryParams.Initialize();
	Data->SetOriginatingSource(Context->ExecutionSource.Get());

	PCGWorldQuery::ExtractObjectFiltersIfNeeded(Settings->QueryParams, Context, Data);

	FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
	Output.Data = Data;

	return true;
}

#if WITH_EDITOR
FText UPCGWorldRayHitSettings::GetNodeTooltipText() const
{
	return LOCTEXT("WorldRayHitTooltip", "Allows generic access (based on raycasts) to collisions in the world that behaves like a surface.");
}
#endif

TArray<FPCGPinProperties> UPCGWorldRayHitSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	QueryParams.AddFilterPinIfNeeded(PinProperties);
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGWorldRayHitSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Surface);

	return PinProperties;
}

FPCGElementPtr UPCGWorldRayHitSettings::CreateElement() const
{
	return MakeShared<FPCGWorldRayHitQueryElement>();
}

#if WITH_EDITOR
EPCGChangeType UPCGWorldRayHitSettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	EPCGChangeType Result = Super::GetChangeTypeForProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGWorldRayHitSettings, QueryParams) &&
		PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FPCGWorldCommonQueryParams, ActorFilterFromInput))
	{
		// This can add/remove a pin, so we need a structural change
		Result |= EPCGChangeType::Structural;
	}

	return Result;
}
#endif // WITH_EDITOR

bool FPCGWorldRayHitQueryElement::CanExecuteOnlyOnMainThread(FPCGContext* Context) const
{
	const UPCGWorldRayHitSettings* Settings = Context ? Context->GetInputSettings<UPCGWorldRayHitSettings>() : nullptr;

	// When getting landscape metadata we do a find actors that requires to be on the main thread.
	return !Settings || Settings->QueryParams.bApplyMetadataFromLandscape;
}

bool FPCGWorldRayHitQueryElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWorldRayHitQueryElement::Execute);

	const UPCGWorldRayHitSettings* Settings = Context->GetInputSettings<UPCGWorldRayHitSettings>();
	check(Settings);

	FPCGWorldRayHitQueryParams QueryParams = Settings->QueryParams;
	
	// TODO: Support params pin + Apply param data

	// TODO: Might want to revisit this for 3D partitioning. The reasoning here is that ray origin should be the same
	// if we are partitioned or non-partitioned. But with 3D partitioning we might want something different.
	// Since it is not yet widely used, we'll stick with same behavior for all cases.
	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	if (!ExecutionSource)
	{
		return true;
	}

	UWorld* World = ExecutionSource->GetExecutionState().GetWorld();
	if (!World)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NoWorld_RayHitQuery", "Execution source does not have a valid world."), Context);
		return true;
	}

	const FTransform Transform = Context->ExecutionSource->GetExecutionState().GetOriginalTransform();
	const FBox LocalBounds = Context->ExecutionSource->GetExecutionState().GetOriginalLocalSpaceBounds();

	if (!QueryParams.bOverrideDefaultParams)
	{
		// Compute default parameters based on original owner component - raycast down local Z axis
		const FVector RayOrigin = Transform.TransformPosition(FVector(0, 0, LocalBounds.Max.Z));
		const FVector RayEnd = Transform.TransformPosition(FVector(0, 0, LocalBounds.Min.Z));

		const FVector::FReal RayLength = (RayEnd - RayOrigin).Length();
		const FVector RayDirection = (RayLength > UE_SMALL_NUMBER ? (RayEnd - RayOrigin) / RayLength : FVector(0, 0, -1.0));

		QueryParams.RayOrigin = RayOrigin;
		QueryParams.RayDirection = RayDirection;
		QueryParams.RayLength = RayLength;
	}
	else // user provided ray parameters
	{
		const FVector::FReal RayDirectionLength = QueryParams.RayDirection.Length();
		if (RayDirectionLength > UE_SMALL_NUMBER)
		{
			QueryParams.RayDirection = QueryParams.RayDirection / RayDirectionLength;
			QueryParams.RayLength *= RayDirectionLength;
		}
		else
		{
			QueryParams.RayDirection = FVector(0, 0, -1.0);
		}
	}

	UPCGWorldRayHitData* Data = FPCGContext::NewObject_AnyThread<UPCGWorldRayHitData>(Context);
	Data->QueryParams = QueryParams;
	Data->QueryParams.Initialize();
	Data->CollisionShape = Settings->CollisionShape;
	Data->Initialize(Context->ExecutionSource.Get(), Transform, /*InBounds=*/FBox(EForceInit::ForceInit), LocalBounds);

	if (QueryParams.bApplyMetadataFromLandscape && Data->Metadata)
	{
		UPCGSubsystem* PCGSubsystem = Cast<UPCGSubsystem>(ExecutionSource->GetExecutionState().GetSubsystem());
		UPCGLandscapeCache* LandscapeCache = PCGSubsystem ? PCGSubsystem->GetLandscapeCache() : nullptr;

		if (LandscapeCache)
		{
			TFunction<bool(const AActor*)> BoundsCheck = [](const AActor*) -> bool { return true; };
			TFunction<bool(const AActor*)> SelfIgnoreCheck = [](const AActor*) -> bool { return true; };

			FPCGActorSelectorSettings ActorSelector;
			ActorSelector.ActorFilter = EPCGActorFilter::AllWorldActors;
			ActorSelector.ActorSelection = EPCGActorSelection::ByClass;
			ActorSelector.ActorSelectionClass = ALandscapeProxy::StaticClass();
			ActorSelector.bSelectMultiple = true;

			if (Data->Bounds.IsValid)
			{
				BoundsCheck = [Data, ExecutionSource](const AActor* OtherActor) -> bool
				{
					const FBox OtherActorBounds = OtherActor ? PCGHelpers::GetGridBounds(OtherActor, ExecutionSource) : FBox(EForceInit::ForceInit);
					return OtherActorBounds.Intersect(Data->Bounds);
				};
			}

			TArray<AActor*> LandscapeActors = PCGActorSelector::FindActors(ActorSelector, ExecutionSource, BoundsCheck, SelfIgnoreCheck);
			for (AActor* Landscape : LandscapeActors)
			{
				if (ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(Landscape))
				{
					Data->CachedLandscapeLayerNames.Append(LandscapeCache->GetLayerNames(LandscapeProxy));
				}
			}
		}
	}

	if (Data->CachedLandscapeLayerNames.IsEmpty())
	{
		Data->QueryParams.bApplyMetadataFromLandscape = false;
	}

	PCGWorldQuery::ExtractObjectFiltersIfNeeded(Settings->QueryParams, Context, Data);

	FPCGTaggedData& Output = Context->OutputData.TaggedData.Emplace_GetRef();
	Output.Data = Data;

	return true;
}

#undef LOCTEXT_NAMESPACE