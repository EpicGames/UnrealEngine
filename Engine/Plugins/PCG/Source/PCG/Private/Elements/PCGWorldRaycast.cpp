// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGWorldRaycast.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGPoint.h"
#include "PCGRayTrace.h"
#include "Compute/PCGKernelHelpers.h"
#include "Compute/BuiltInKernels/PCGPreRayTraceKernel.h"
#include "Compute/BuiltInKernels/PCGPostRayTraceKernel.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Helpers/PCGWorldQueryHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "Misc/Optional.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGWorldRaycast)

#define LOCTEXT_NAMESPACE "PCGWorldRaycastElement"

namespace PCGWorldRaycastElement::Constants
{
	const FText DefaultNodeTitle = LOCTEXT("NodeTitle", "World Raycast");

	namespace Labels
	{
		const FName OriginsInput = TEXT("Origins");
		const FName EndPointsInput = TEXT("End Points");
		const FName BoundingShape = TEXT("Bounding Shape");

		const FText LineTrace = LOCTEXT("LineTraceLabel", "Line Trace");
		const FText BoxSweep = LOCTEXT("BoxSweepLabel", "Box Sweep");
		const FText SphereSweep = LOCTEXT("SphereSweepLabel", "Sphere Sweep");
		const FText CapsuleSweep = LOCTEXT("CapsuleSweepLabel", "Capsule Sweep");
	}

	namespace Tooltips
	{
		const FText LineTrace = LOCTEXT("LineTraceTooltip", "Conduct a trace along a given ray.");
		const FText BoxSweep = LOCTEXT("BoxSweepTooltip", "Casts a box sweep along a given ray.");
		const FText SphereSweep = LOCTEXT("SphereSweepTooltip", "Casts a sphere sweep along a given ray.");
		const FText CapsuleSweep = LOCTEXT("CapsuleSweepTooltip", "Casts a capsule sweep along a given ray.");
	}

	namespace Preconfiguration
	{
		constexpr int32 LineTraceIndex = 0;
		constexpr int32 BoxSweepIndex = 1;
		constexpr int32 SphereSweepIndex = 2;
		constexpr int32 CapsuleSweepIndex = 3;
	}
}

UPCGWorldRaycastElementSettings::UPCGWorldRaycastElementSettings()
{
	OriginInputAttribute.SetPointProperty(EPCGPointProperties::Position);
	EndPointAttribute.SetPointProperty(EPCGPointProperties::Position);
	// Tracing rays along the normal is the common use case
	RayDirectionAttribute.Update("$Rotation.Up");
}

#if WITH_EDITOR
bool UPCGWorldRaycastElementSettings::IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo) const
{
	// Only the Line Trace preconfigured variant has a GPU-friendly backend currently. When no preconfig is provided
	// (e.g. the non-preconfigured palette entry) we report friendly, since the user may configure the instance as a line trace.
	if (!PreconfiguredInfo || PreconfiguredInfo->PreconfiguredIndex == INDEX_NONE)
	{
		return true;
	}

	return PreconfiguredInfo->PreconfiguredIndex == PCGWorldRaycastElement::Constants::Preconfiguration::LineTraceIndex;
}

FText UPCGWorldRaycastElementSettings::GetDefaultNodeTitle() const
{
	return PCGWorldRaycastElement::Constants::DefaultNodeTitle;
}

TArray<FText> UPCGWorldRaycastElementSettings::GetNodeTitleAliases() const
{
	return {LOCTEXT("WorldTraceAlias", "World Trace"), LOCTEXT("WorldSweepAlias", "World Sweep")};
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGWorldRaycastElementSettings::GetPreconfiguredInfo() const
{
	using namespace PCGWorldRaycastElement::Constants;

	return {
		{0, Labels::LineTrace, Tooltips::LineTrace},
		{1, Labels::BoxSweep, Tooltips::BoxSweep},
		{2, Labels::SphereSweep, Tooltips::SphereSweep},
		{3, Labels::CapsuleSweep, Tooltips::CapsuleSweep}
	};
}

EPCGChangeType UPCGWorldRaycastElementSettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	EPCGChangeType Result = Super::GetChangeTypeForProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPCGWorldRaycastElementSettings, WorldQueryParams)
		&& PropertyName == GET_MEMBER_NAME_CHECKED(FPCGWorldCommonQueryParams, ActorFilterFromInput))
	{
		// This can add/remove a pin, so we need a structural change
		Result |= EPCGChangeType::Structural;
	}

	if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPCGWorldRaycastElementSettings, CollisionShape)
		&& PropertyName == GET_MEMBER_NAME_CHECKED(FPCGCollisionShape, ShapeType))
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UPCGWorldRaycastElementSettings, RaycastMode))
	{
		// Changes the additional title info
		Result |= EPCGChangeType::Cosmetic;
	}

	return Result;
}

void UPCGWorldRaycastElementSettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	PCGKernelHelpers::FCreateKernelParams CreateParams(InObjectOuter, this, InNode);

	// Wiring is complex enough to just wire it out manually below (and node input label is not "In").
	CreateParams.NodeInputPinsToWire.Empty();
	CreateParams.NodeOutputPinsToWire.Empty();

	UPCGPreRayTraceKernel* PreKernel = PCGKernelHelpers::CreateKernel<UPCGPreRayTraceKernel>(InOutContext, CreateParams, OutKernels, OutEdges);
	UPCGPostRayTraceKernel* PostKernel = PCGKernelHelpers::CreateKernel<UPCGPostRayTraceKernel>(InOutContext, CreateParams, OutKernels, OutEdges);

	// Node input to pre-kernel which exports data collection buffer to raw buffer.
	OutEdges.Emplace(FPCGPinReference(PCGWorldRaycastElement::Constants::Labels::OriginsInput), FPCGPinReference(PreKernel, PCGPinConstants::DefaultInputLabel));

	if (RaycastMode == EPCGWorldRaycastMode::Segments)
	{
		OutEdges.Emplace(FPCGPinReference(PCGWorldRaycastElement::Constants::Labels::EndPointsInput), FPCGPinReference(PreKernel, PCGWorldRaycastElement::Constants::Labels::EndPointsInput));
		PreKernel->SetIncludeEndPointsInputPin(true);
	}

	// Node input to post-kernel so it can flow the data through (and modify positions based on RT results).
	OutEdges.Emplace(FPCGPinReference(PCGWorldRaycastElement::Constants::Labels::OriginsInput), FPCGPinReference(PostKernel, PCGPinConstants::DefaultInputLabel));

	// Ray trace results to post-kernel.
	OutEdges.Emplace(FPCGPinReference(FPCGPinReference(PreKernel, PCGPinConstants::DefaultOutputLabel)), FPCGPinReference(PostKernel, PCGPostRayTraceKernel::RayTraceResultsLabel));

	// Post kernel to node output.
	OutEdges.Emplace(FPCGPinReference(FPCGPinReference(PostKernel, PCGPinConstants::DefaultOutputLabel)), FPCGPinReference(PCGPinConstants::DefaultOutputLabel));
}
#endif // WITH_EDITOR

bool UPCGWorldRaycastElementSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	if (InPin && ShouldExecuteOnGPU() && InPin->Properties.Label == PCGWorldRaycastElement::Constants::Labels::BoundingShape)
	{
		return false;
	}

	return Super::IsPinUsedByNodeExecution(InPin);
}

FString UPCGWorldRaycastElementSettings::GetAdditionalTitleInformation() const
{
	using namespace PCGWorldRaycastElement::Constants;

	FText FinalLabel;

	if (IsPropertyOverriddenByPin({GET_MEMBER_NAME_CHECKED(UPCGWorldRaycastElementSettings, CollisionShape), GET_MEMBER_NAME_CHECKED(FPCGCollisionShape, ShapeType)}))
	{
		FinalLabel = LOCTEXT("ShapeTypeOverridden", "Overridden");
	}
	else
	{
		switch (CollisionShape.ShapeType)
		{
			case EPCGCollisionShapeType::Line:
				FinalLabel = Labels::LineTrace;
				break;
			case EPCGCollisionShapeType::Box:
				FinalLabel = Labels::BoxSweep;
				break;
			case EPCGCollisionShapeType::Sphere:
				FinalLabel = Labels::SphereSweep;
				break;
			case EPCGCollisionShapeType::Capsule:
				FinalLabel = Labels::CapsuleSweep;
				break;
			default:
				PCGLog::LogErrorOnGraph(LOCTEXT("InvalidCollisionShapeType", "World Raycast - Invalid Collision Shape Type"));
		}
	}

	if (!IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGWorldRaycastElementSettings, RaycastMode)))
	{
		if (const UEnum* EnumPtr = StaticEnum<EPCGWorldRaycastMode>())
		{
			FinalLabel = FText::Format(INVTEXT("{0} ({1})"), FinalLabel, EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(RaycastMode)));
		}
	}

	return FinalLabel.ToString();
}

void UPCGWorldRaycastElementSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo)
{
	Super::ApplyPreconfiguredSettings(PreconfigureInfo);

	using namespace PCGWorldRaycastElement::Constants;
	switch (PreconfigureInfo.PreconfiguredIndex)
	{
		case Preconfiguration::LineTraceIndex:
			CollisionShape.ShapeType = EPCGCollisionShapeType::Line;
			break;
		case Preconfiguration::BoxSweepIndex:
			CollisionShape.ShapeType = EPCGCollisionShapeType::Box;
			break;
		case Preconfiguration::SphereSweepIndex:
			CollisionShape.ShapeType = EPCGCollisionShapeType::Sphere;
			break;
		case Preconfiguration::CapsuleSweepIndex:
			CollisionShape.ShapeType = EPCGCollisionShapeType::Capsule;
			break;
		default:
			PCGLog::Settings::LogInvalidPreconfigurationWarning(PreconfigureInfo.PreconfiguredIndex, DefaultNodeTitle);
	}
}

TArray<FPCGPinProperties> UPCGWorldRaycastElementSettings::InputPinProperties() const
{
	// Note: If executing on the GPU, we need to prevent multiple connections on inputs, since it is not supported at this time.
	// Also note: Since the ShouldExecuteOnGPU() is already tied to structural changes, we don't need to implement any logic for this in GetChangeTypeForProperty()
	const bool bAllowMultipleConnections = !ShouldExecuteOnGPU();

	TArray<FPCGPinProperties> PinProperties;

	using namespace PCGWorldRaycastElement::Constants;
	FPCGPinProperties& OriginPointsInputPin = PinProperties.Emplace_GetRef(Labels::OriginsInput, EPCGDataType::PointOrParam, bAllowMultipleConnections);
	OriginPointsInputPin.SetRequiredPin();
#if WITH_EDITOR
	OriginPointsInputPin.Tooltip = LOCTEXT("OriginPointsInputPinTooltip", "Points acting as the point origin of each individual ray. Only point data from this pin may be forwarded.");
#endif // WITH_EDITOR

	if (RaycastMode == EPCGWorldRaycastMode::Segments)
	{
		FPCGPinProperties& EndPointsInputPin = PinProperties.Emplace_GetRef(Labels::EndPointsInput, EPCGDataType::PointOrParam, bAllowMultipleConnections);
#if WITH_EDITOR
		EndPointsInputPin.Tooltip = LOCTEXT("EndPointsInputPinTooltip", "Points acting as the end point of each individual ray.");
#endif // WITH_EDITOR
	}

	FPCGPinProperties& BoundingShapeInputPin = PinProperties.Emplace_GetRef(Labels::BoundingShape, EPCGDataType::Spatial);
#if WITH_EDITOR
	BoundingShapeInputPin.Tooltip = LOCTEXT("BoundingShapeInputPinTooltip", "All projected points must be contained within this shape. If this input is omitted then bounds will be taken from the actor so that points are contained within actor bounds.");
#endif // WITH_EDITOR

	WorldQueryParams.AddFilterPinIfNeeded(PinProperties);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGWorldRaycastElementSettings::OutputPinProperties() const
{
	return DefaultPointOutputPinProperties();
}

FPCGElementPtr UPCGWorldRaycastElementSettings::CreateElement() const
{
	return MakeShared<FPCGWorldRaycastElement>();
}

bool FPCGWorldRaycastElement::PrepareDataInternal(FPCGContext* InContext) const
{
	ContextType* Context = static_cast<ContextType*>(InContext);

	const UPCGWorldRaycastElementSettings* Settings = Context->GetInputSettings<UPCGWorldRaycastElementSettings>();
	check(Settings);

	using namespace PCGWorldRaycastElement::Constants;
	const TArray<FPCGTaggedData> OriginsInputData = Context->InputData.GetInputsByPin(Labels::OriginsInput);
	const TArray<FPCGTaggedData> EndPointsInputData = Context->InputData.GetInputsByPin(Labels::EndPointsInput);
	const TArray<FPCGTaggedData> FilterActorInputData = Context->InputData.GetInputsByPin(PCGWorldRayHitConstants::FilterActorPinLabel);

	const EPCGTimeSliceInitResult ExecutionInitResult = Context->InitializePerExecutionState([&OriginsInputData, &EndPointsInputData, &FilterActorInputData, Settings](ContextType* Context, ExecStateType& OutState)
	{
		// With no origins data, early out
		if (OriginsInputData.IsEmpty())
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (!Settings->bUnbounded)
		{
			bool bUnionWasCreated;
			const UPCGSpatialData* BoundingShape = PCGSettingsHelpers::ComputeBoundingShape(Context, Labels::BoundingShape, bUnionWasCreated);
			OutState.Bounds = BoundingShape ? BoundingShape->GetBounds() : FBox(EForceInit::ForceInit);
			if (!OutState.Bounds.IsValid)
			{
				// The bounding shape bounds is invalid, such as an empty intersection, so no operation will need to be performed.
				return EPCGTimeSliceInitResult::NoOperation;
			}
		}

		if (Settings->RaycastMode == EPCGWorldRaycastMode::Segments && EndPointsInputData.Num() != OriginsInputData.Num())
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidEndPointDataCount", "End point input data count must match the Origins input data count."), Context);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		if (FilterActorInputData.Num() > 1 && FilterActorInputData.Num() != OriginsInputData.Num())
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidFilterActorInputCount", "Filter Actor input data count must be 1 or match the Origins input data count."), Context);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		OutState.CollisionQueryParams = Settings->WorldQueryParams.ToCollisionQuery();
		OutState.CollisionObjectQueryParams = FCollisionObjectQueryParams(Settings->WorldQueryParams.CollisionChannel);

		return EPCGTimeSliceInitResult::Success;
	});

	if (ExecutionInitResult != EPCGTimeSliceInitResult::Success)
	{
		return true;
	}

	Context->InitializePerIterationStates(OriginsInputData.Num(), [Context, Settings, &OriginsInputData, &EndPointsInputData, &FilterActorInputData](IterStateType& OutState, const ExecStateType&, int32 Index)
	{
		const UPCGData* OriginsData = OriginsInputData[Index].Data;
		const UPCGData* FilterActorsData = !FilterActorInputData.IsEmpty() ? FilterActorInputData[Index % FilterActorInputData.Num()].Data : nullptr;

		// Accept only point or param data
		if (!OriginsData || (!OriginsData->IsA<UPCGBasePointData>() && !OriginsData->IsA<UPCGParamData>()))
		{
			PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::PointOrParam, Labels::OriginsInput, Context);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// Accept only point or param data
		if (FilterActorsData && !FilterActorsData->IsA<UPCGBasePointData>() && !FilterActorsData->IsA<UPCGParamData>())
		{
			PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::PointOrParam, PCGWorldRayHitConstants::FilterActorPinLabel, Context);
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// --- Gather filtered actors ---
		if (!Settings->WorldQueryParams.ExtractLoadedObjectFiltersIfNeeded(FilterActorsData, OutState.CachedFilterObjects, Context))
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// --- Find the ray origin ---
		if (!PCGAttributeAccessorHelpers::ExtractAllValues(OriginsData, Settings->OriginInputAttribute, OutState.CachedRayOrigins, Context) || OutState.CachedRayOrigins.IsEmpty())
		{
			return EPCGTimeSliceInitResult::NoOperation;
		}

		// --- Find the ray direction vectors ---
		// Calculate the vectors from origin to end points
		if (Settings->RaycastMode == EPCGWorldRaycastMode::Segments)
		{
			TObjectPtr<const UPCGData> EndPointsData = EndPointsInputData[Index].Data;
			if (!EndPointsData)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NoDataFoundOnEndPointsInput", "No valid data found on End Points input {0}"), Index), Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			TArray<FVector> EndPoints;
			if (!PCGAttributeAccessorHelpers::ExtractAllValues(EndPointsData, Settings->EndPointAttribute, EndPoints, Context))
			{
				return EPCGTimeSliceInitResult::NoOperation;
			}

			// Support N:N, 1:N, N:1
			if (OutState.CachedRayOrigins.Num() != 1 && EndPoints.Num() != 1 && EndPoints.Num() != OutState.CachedRayOrigins.Num())
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("InvalidEndPointCount", "The End Points input count must be 1 or directly match the Origins input count."), Context);
				return EPCGTimeSliceInitResult::NoOperation;
			}

			const int32 PointCount = FMath::Max(OutState.CachedRayOrigins.Num(), EndPoints.Num());

			OutState.CachedRayVectors.Empty();
			OutState.CachedRayVectors.Reserve(PointCount);

			// Find the ray vector from the origin to end points
			for (int PointIndex = 0; PointIndex < PointCount; ++PointIndex)
			{
				const FVector& Origin = OutState.CachedRayOrigins[OutState.CachedRayOrigins.Num() == 1 ? 0 : PointIndex];
				const FVector& EndPoint = EndPoints[EndPoints.Num() == 1 ? 0 : PointIndex];
				OutState.CachedRayVectors.Emplace((EndPoint - Origin));
			}
		}
		else if (Settings->bOverrideRayDirections) // Use a selector to determine the (non-normalized) directions
		{
			if (!PCGAttributeAccessorHelpers::ExtractAllValues(OriginsData, Settings->RayDirectionAttribute, OutState.CachedRayVectors, Context))
			{
				return EPCGTimeSliceInitResult::NoOperation;
			}
		}
		else // Use the single property
		{
			OutState.CachedRayVectors.Empty();
			OutState.CachedRayVectors.Emplace(Settings->RayDirection);
		}

		// --- Find the ray length ---
		// If the ray is infinite or user-selected length, we need to normalize the vector
		if (Settings->RaycastMode == EPCGWorldRaycastMode::Infinite || Settings->RaycastMode == EPCGWorldRaycastMode::NormalizedWithLength)
		{
			// Normalize the directions and store locally in the state
			for (FVector& Ray : OutState.CachedRayVectors)
			{
				Ray.Normalize();
			}

			// If the length is infinite, it will be a fixed value at execution
			if (Settings->RaycastMode == EPCGWorldRaycastMode::Infinite)
			{
				return EPCGTimeSliceInitResult::Success;
			}

			// Account for user length property. Use a selector to determine the (non-normalized) directions
			if (Settings->bOverrideRayLengths)
			{
				const FPCGAttributePropertyInputSelector LengthSelector = Settings->RayLengthAttribute.CopyAndFixLast(OriginsData);

				if (!PCGAttributeAccessorHelpers::ExtractAllValues(OriginsData, LengthSelector, OutState.CachedRayLengths, Context))
				{
					return EPCGTimeSliceInitResult::NoOperation;
				}
			}
			else
			{
				OutState.CachedRayLengths.Empty();
				OutState.CachedRayLengths.Emplace(Settings->RayLength);
			}
		}

		return EPCGTimeSliceInitResult::Success;
	});

	return true;
}

bool FPCGWorldRaycastElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWorldRaycastElement::Execute);

	ContextType* Context = static_cast<ContextType*>(InContext);

	if (!Context->DataIsPreparedForExecution())
	{
		return true;
	}

	UWorld* World = Context->ExecutionSource.IsValid() ? Context->ExecutionSource->GetExecutionState().GetWorld() : nullptr;
	if (!World)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NoWorldFound", "Execution source does not belong to a world."), Context);
		return true;
	}

	return ExecuteSlice(Context, [World](ContextType* Context, const ExecStateType& ExecState, IterStateType& IterState, const uint32 IterIndex)
	{
		if (Context->GetIterationStateResult(IterIndex) == EPCGTimeSliceInitResult::NoOperation)
		{
			return true;
		}

		const UPCGWorldRaycastElementSettings* Settings = Context->GetInputSettings<UPCGWorldRaycastElementSettings>();
		check(Settings);

		const TArray<FPCGTaggedData> OriginInputs = Context->InputData.GetInputsByPin(PCGWorldRaycastElement::Constants::Labels::OriginsInput);
		const FPCGTaggedData& OriginInputData = OriginInputs[IterIndex];
		const UPCGBasePointData* OriginInputPointData = Cast<UPCGBasePointData>(OriginInputData.Data);

		TArray<FPCGTaggedData>& OutputData = Context->OutputData.TaggedData;
		UPCGBasePointData* OutputPointData = FPCGContext::NewPointData_AnyThread(Context);

		// TODO: This should initialize from the End Points point data, if Origin point data doesn't exist
		FPCGInitializeFromDataParams InitializeFromDataParams(OriginInputPointData);
		InitializeFromDataParams.bInheritSpatialData = false;
		OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);

		FPCGTaggedData& Output = OutputData.Emplace_GetRef(OriginInputData);
		Output.Data = OutputPointData;

		UPCGMetadata* OutMetadata = OutputPointData->Metadata;

		FPCGWorldRaycastQueryParams WorldQueryParams = Settings->WorldQueryParams;
		WorldQueryParams.Initialize();

		if (!PCGWorldQueryHelpers::CreateRayHitAttributes(WorldQueryParams, OutMetadata))
		{
			PCGLog::LogWarningOnGraph(LOCTEXT("UnableToCreateAllAttributes", "One or more attributes were unable to be created."), Context);
		}

		bool bAttributeSuccess = true;

		const int32 PointCount = FMath::Max(IterState.CachedRayOrigins.Num(), IterState.CachedRayVectors.Num());
		
		TArray<TTuple<TOptional<FHitResult>, const FVector&, const FVector&>> HitResults;
		HitResults.Reserve(PointCount);
		
		TArray<FHitResult> Hits;

		for (int64 PointIndex = 0; PointIndex < PointCount; ++PointIndex)
		{
			const FVector& Origin = IterState.CachedRayOrigins[IterState.CachedRayOrigins.Num() == 1 ? 0 : PointIndex];
			const FVector& Direction = IterState.CachedRayVectors[IterState.CachedRayVectors.Num() == 1 ? 0 : PointIndex];
			double Magnitude = 1.0;

			if (Settings->RaycastMode == EPCGWorldRaycastMode::Infinite)
			{
				Magnitude = UE_LARGE_WORLD_MAX;
			}
			else if (Settings->RaycastMode == EPCGWorldRaycastMode::NormalizedWithLength)
			{
				Magnitude = IterState.CachedRayLengths[IterState.CachedRayLengths.Num() == 1 ? 0 : PointIndex];
			}

			FVector RayVector = Direction * Magnitude;
			// Discard point if the direction + magnitude vector is the ZeroVector
			if (RayVector.IsNearlyZero())
			{
				continue;
			}

			// The physics system will automatically follow the line trace path, if the collision shape is line or has no volume.
			Hits.Reset();
			World->SweepMultiByObjectType(
				Hits,
				Origin,
				Origin + RayVector,
				FQuat(Settings->CollisionShape.ShapeRotation),
				ExecState.CollisionObjectQueryParams,
				Settings->CollisionShape.ToCollisionShape(),
				ExecState.CollisionQueryParams);

			TOptional<FHitResult> HitResult = PCGWorldQueryHelpers::FilterRayHitResults(WorldQueryParams, Context->ExecutionSource.Get(), Hits, IterState.CachedFilterObjects);
			if (HitResult.IsSet() && (Settings->bUnbounded || PCGHelpers::IsInsideBounds(ExecState.Bounds, HitResult.GetValue().ImpactPoint)))
			{
				HitResults.Add({ MoveTemp(HitResult), Origin, Direction });
			}
			else if (Settings->bKeepOriginalPointOnMiss)
			{
				HitResults.Add({ TOptional<FHitResult>(), Origin, Direction });
			}
		}

		OutputPointData->SetNumPoints(HitResults.Num(), /*bInitializeValues=*/false);

		if (OriginInputPointData)
		{
			OutputPointData->AllocateProperties(OriginInputPointData->GetAllocatedProperties());
			OutputPointData->CopyUnallocatedPropertiesFrom(OriginInputPointData);
		}

		OutputPointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::MetadataEntry);
		if (Settings->bKeepOriginalPointOnMiss)
		{
			OutputPointData->AllocateProperties(EPCGPointNativeProperties::Density);
		}
		else
		{
			OutputPointData->SetDensity(1.f);
		}

		const FConstPCGPointValueRanges InRanges = OriginInputPointData ? FConstPCGPointValueRanges(OriginInputPointData) : FConstPCGPointValueRanges();
		FPCGPointValueRanges OutRanges(OutputPointData, /*bAllocate=*/false);

		for (int64 PointIndex = 0; PointIndex < HitResults.Num(); ++PointIndex)
		{
			const TOptional<FHitResult>& HitResult = HitResults[PointIndex].Get<0>();
			const FVector& Origin = HitResults[PointIndex].Get<1>();
			const FVector& Direction = HitResults[PointIndex].Get<2>();

			// If there's a hit, and it's within bounds, keep the point
			if (HitResult.IsSet())
			{
				const FHitResult& Hit = HitResult.GetValue();
				// Copy the input point, if it exists
				FPCGPoint OutPoint = OriginInputPointData ? InRanges.GetPoint(OriginInputPointData->GetNumPoints() == 1 ? 0 : PointIndex) : FPCGPoint();

				OutPoint.Transform = PCGWorldQueryHelpers::GetOrthonormalImpactTransform(Hit);
				OutPoint.Density = 1.f;
				OutPoint.Seed = PCGHelpers::ComputeSeedFromPosition(OutPoint.Transform.GetLocation());

				bAttributeSuccess &= PCGWorldQueryHelpers::ApplyRayHitMetadata(Hit, WorldQueryParams, Direction, OutPoint.Transform, OutPoint.MetadataEntry, OutMetadata, World);

				OutRanges.SetFromPoint(PointIndex, OutPoint);
			}
			else
			{
				// Copy the original point if possible
				if (OriginInputPointData)
				{
					OutRanges.SetFromPoint(PointIndex, InRanges.GetPoint(OriginInputPointData->GetNumPoints() == 1 ? 0 : PointIndex));	
				}
				else
				{
					OutRanges.TransformRange[PointIndex] = FTransform(Origin);
					OutRanges.SeedRange[PointIndex] = PCGHelpers::ComputeSeedFromPosition(Origin);
				}

				OutRanges.DensityRange[PointIndex] = 0.f;
				bAttributeSuccess &= PCGWorldQueryHelpers::ApplyRayMissMetadata(WorldQueryParams, OutRanges.MetadataEntryRange[PointIndex], OutMetadata);
			}
		}

		if (!bAttributeSuccess)
		{
			PCGLog::LogWarningOnGraph(LOCTEXT("UnableToApplyAllAttributes", "One or more attributes were unable to be applied."), Context);
		}

		return true;
	});
}

#undef LOCTEXT_NAMESPACE