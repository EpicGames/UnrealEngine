// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGWriteDataIndex.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGWriteDataIndex)

#define LOCTEXT_NAMESPACE "PCGWriteDataIndexElement"

UPCGWriteDataIndexSettings::UPCGWriteDataIndexSettings()
{
	IndexAttribute = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyOutputSelector>(TEXT("Index"), TEXT("Data"));
}

TArray<FPCGPinProperties> UPCGWriteDataIndexSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);

	return PinProperties;
}

bool FPCGWriteDataIndexElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGWriteDataIndexElement::Execute);

	check(Context);

	const UPCGWriteDataIndexSettings* Settings = Context->GetInputSettings<UPCGWriteDataIndexSettings>();
	check(Settings);

	TArray<FPCGTaggedData> InputData = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& OutputData = Context->OutputData.TaggedData;

	const FName IndexAttributeName = Settings->IndexAttribute.GetName();

	// Validate that the output tag is correct or passthrough otherwise
	if (Settings->bAddTag && Settings->IndexTag.IsEmpty())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("Invalid tag", "The index tag must not be empty."), Context);
		OutputData = InputData;
		return true;
	}

	for(int DataIndex = 0; DataIndex < InputData.Num(); ++DataIndex)
	{
		const FPCGTaggedData& Input = InputData[DataIndex];
		FPCGTaggedData& OutputTaggedData = OutputData.Emplace_GetRef(Input);

		const UPCGData* Data = Input.Data;

		if (Settings->bAddAttribute)
		{
			UPCGData* NewData = Data->DuplicateData(Context);
			FPCGMetadataDomain* IndexAttributeDomain = NewData ? NewData->MutableMetadata()->GetMetadataDomainFromSelector(Settings->IndexAttribute) : nullptr;

			// Ensure attribute doesn't exist because we need to change the default value
			if(IndexAttributeDomain && IndexAttributeDomain->HasAttribute(IndexAttributeName))
			{
				IndexAttributeDomain->DeleteAttribute(IndexAttributeName);
			}

			FPCGMetadataAttribute<int32>* IndexAttribute = IndexAttributeDomain ? IndexAttributeDomain->FindOrCreateAttribute<int32>(IndexAttributeName, DataIndex, /*bAllowInterpolation=*/false) : nullptr;

			if (!IndexAttribute)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FailedCreateIndexAttribute", "Failed to create the index attribute '{0}'."), FText::FromName(IndexAttributeName)), Context);
				continue;
			}

			OutputTaggedData.Data = NewData;
		}

		if (Settings->bAddTag)
		{
			OutputTaggedData.Tags.Add(FString::Format(TEXT("{0}:{1}"), { Settings->IndexTag, DataIndex }));
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE