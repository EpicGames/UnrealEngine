// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Polygon/PCGClipPaths.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGPolygon2DData.h"
#include "Elements/PCGSplitSplines.h"
#include "Elements/Polygon/PCGPolygon2DUtils.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGPointDataHelpers.h"
#include "Helpers/PCGPolygon2DProcessingHelpers.h"

#include "Curve/PolygonIntersectionUtils.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGClipPaths)

#define LOCTEXT_NAMESPACE "PCGClipPathsElement"

#if WITH_EDITOR
FText UPCGClipPathsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Clip Paths");
}

FText UPCGClipPathsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Clips paths against the polygons, to have the paths either inscribed in the polygons (intersection) or outside the polygons (difference).");
}

void UPCGClipPathsSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	if (DataVersion < FPCGCustomVersion::RemovedSpacesInPolygonPinLabels)
	{
		InOutNode->RenameInputPin(PCGPolygon2DUtils::Constants::Deprecated::OldClipPolysLabel, PCGPolygon2DUtils::Constants::ClipPolysLabel);
	}
}
#endif // WITH_EDITOR

FString UPCGClipPathsSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGClipPathOperation>())
	{
		return EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation)).ToString();
	}
	else
	{
		return FString();
	}
}

TArray<FPCGPinProperties> UPCGClipPathsSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	// @todo_pcg: add landscape spline support?
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point | EPCGDataType::Spline).SetRequiredPin();
	PinProperties.Emplace(PCGPolygon2DUtils::Constants::ClipPolysLabel, EPCGDataType::Polygon2D);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGClipPathsSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	// @todo_pcg: add landscape spline support?
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point | EPCGDataType::Spline);

	return PinProperties;
}

bool FPCGClipPathsElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCLipPathsElement::Execute);
	const UPCGClipPathsSettings* Settings = Context->GetInputSettings<UPCGClipPathsSettings>();
	check(Settings);

	TArray<FPCGTaggedData> InputPaths = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData> ClipInputs = Context->InputData.GetInputsByPin(PCGPolygon2DUtils::Constants::ClipPolysLabel);

	// Gather polygons for clipping. Note that we will not gather entry keys here as we will infer metadata only from the input paths.
	bool bHasTransform = false;
	FTransform PolyTransform;
	TArray<UE::Geometry::FGeneralPolygon2d> ClipPolys;
	ClipPolys.Reserve(ClipInputs.Num());

	for (const FPCGTaggedData& ClipInput : ClipInputs)
	{
		if (const UPCGPolygon2DData* ClipPolyData = Cast<UPCGPolygon2DData>(ClipInput.Data))
		{
			PCGPolygon2DProcessingHelpers::AddPolygon(ClipPolyData, ClipPolys, bHasTransform, PolyTransform);
		}
	}

	// Implementation note - there's an assumption that the input points/splines are on the same plane as the polygons.
	// However, that's not guaranteed in anyway, and if we don't have polygons to clip against, then we will return the inputs as is.
	if (!bHasTransform)
	{
		Context->OutputData.TaggedData = InputPaths;
		return true;
	}

	TArray<FVector2D> Positions2D;

	for (const FPCGTaggedData& Input : InputPaths)
	{
		if (!PCGPolygon2DProcessingHelpers::GetPath(Input.Data, PolyTransform, Settings->SplineMaxDiscretizationError, Positions2D))
		{
			// Unsuppported type
			continue;
		}
		
		TArray<TArrayView<FVector2D>> PathView;
		PathView.Add(MakeArrayView(Positions2D));

		TArray<TArray<FVector2D>> Result;

		bool bOperationSuccessful =
			(Settings->Operation == EPCGClipPathOperation::Intersection && UE::Geometry::PathsIntersection(PathView, ClipPolys, Result)) ||
			(Settings->Operation == EPCGClipPathOperation::Difference && UE::Geometry::PathsDifference(PathView, ClipPolys, Result));

		if (!bOperationSuccessful)
		{
			continue;
		}
		
		TArray<const UPCGSpatialData*> PathData;
		PathData.Reserve(Result.Num());

		for (const TArray<FVector2D>& ResultPath : Result)
		{
			if (ResultPath.Num() < 2)
			{
				continue;
			}

			const bool bTestForEquality = Result.Num() == 1 && ResultPath[0].Equals(Positions2D[0]) && ResultPath.Last().Equals(Positions2D.Last());
			auto TestPointEquality = [&ResultPath, &Positions2D]()
			{
				for (int PointIndex = 1; PointIndex < ResultPath.Num() - 1; ++PointIndex)
				{
					if (!ResultPath[PointIndex].Equals(Positions2D[PointIndex]))
					{
						return false;
					}
				}

				return true;
			};

			const bool bIsSameAsInput = bTestForEquality && TestPointEquality();

			// Special case: if the result is, in fact, the full path, return the data as-is. It'll be more efficient and will not cause issue with loop splines
			if(bIsSameAsInput)
			{
				PathData.Add(Cast<UPCGSpatialData>(Input.Data));
			}
			else if (const UPCGSplineData* InputSpline = Cast<UPCGSplineData>(Input.Data))
			{
				// Implementation note: while there's a small assumption that the spline is on the polygon plane,
				// in practice as long as it is parallel to the polygons it's fine as the minimum distance will be the right one.
				// Another detail here: while we could jiggle the start & end, considering we already know we have an actual cut happening
				// it will not yield a loop in the end. 
				// If we discover there are some edge cases to this, we can jiggle the path start (by doing result path 0 + epsilon result path 1) + same for end,
				// which will be sufficient. We could also then test if the "whole loop" is covered too.
				FVector PathStart = PolyTransform.TransformPosition(FVector(ResultPath[0], 0.0));
				FVector PathEnd = PolyTransform.TransformPosition(FVector(ResultPath.Last(), 0.0));

				double StartKey = InputSpline->SplineStruct.FindInputKeyClosestToWorldLocation(PathStart);
				double EndKey = InputSpline->SplineStruct.FindInputKeyClosestToWorldLocation(PathEnd);

				UPCGSplineData* SplitSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
				SplitSplineData->InitializeFromData(InputSpline);

				if (PCGSplitSpline::SplitSpline(InputSpline, SplitSplineData, StartKey, EndKey) == PCGSplitSpline::EStatus::Split)
				{
					PathData.Add(SplitSplineData);
				}
			}
			else if (const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(Input.Data))
			{
				// Implementation note: stronger assumption that the points are on the plane here.
				// If they aren't, then the sampling method will not work.
				// @todo_pcg : Improve this; we could find key-equivalent and split the points instead.
				TArray<FVector> PathPositions;
				PathPositions.Reserve(ResultPath.Num());

				for (const FVector2D& PathVertex : ResultPath)
				{
					PathPositions.Add(PolyTransform.TransformPosition(FVector(PathVertex, 0.0)));
				}

				UPCGBasePointData* SplitPointPath = FPCGContext::NewPointData_AnyThread(Context);
				FPCGInitializeFromDataParams InitializeDataParams(InputPointData);
				InitializeDataParams.bInheritMetadata = InitializeDataParams.bInheritAttributes = true;
				InitializeDataParams.bInheritSpatialData = false;

				SplitPointPath->InitializeFromDataWithParams(InitializeDataParams);

				SplitPointPath->SetNumPoints(ResultPath.Num());

				const bool bHasMetadata = !!(InputPointData->GetAllocatedProperties() & EPCGPointNativeProperties::MetadataEntry);
				SplitPointPath->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | InputPointData->GetAllocatedProperties());
				SplitPointPath->CopyUnallocatedPropertiesFrom(InputPointData);

				TConstPCGValueRange<FTransform> InputTransformRange = InputPointData->GetConstTransformValueRange();
				TPCGValueRange<FTransform> PathTransformRange = SplitPointPath->GetTransformValueRange(/*bAllocate=*/false);
				TPCGValueRange<int32> PathSeedRange = SplitPointPath->GetSeedValueRange(/*bAllocate=*/false);

				int32 SegmentStartIndex = 0;

				for (int PathVertexIndex = 0; PathVertexIndex < PathPositions.Num(); ++PathVertexIndex)
				{
					const FVector PathPosition = PathPositions[PathVertexIndex];
					const int32 SegmentEndIndex = SegmentStartIndex + 1;

					// By construction, the result path should be in the same order as the input path, so we should in theory never go out of bounds
					// But if we ever do, just fallback on the current point
					if (!ensure(InputTransformRange.IsValidIndex(SegmentEndIndex)))
					{
						InputPointData->CopyPointsTo(SplitPointPath, SegmentStartIndex, PathVertexIndex, 1);
						continue;
					}

					const FVector SegmentStart = InputTransformRange[SegmentStartIndex].GetLocation();
					const FVector SegmentEnd = InputTransformRange[SegmentEndIndex].GetLocation();

					const FVector StartToEndSegmentVector = SegmentEnd - SegmentStart;
					const double SegmentSquaredLength = StartToEndSegmentVector.Dot(StartToEndSegmentVector);

					if (FMath::IsNearlyZero(SegmentSquaredLength))
					{
						// Segment is of size 0, try again with next segment
						SegmentStartIndex++;
						PathVertexIndex--;
						continue;
					}

					const FVector StartToPathPositionVector = PathPositions[PathVertexIndex] - SegmentStart;
					const double Dot = StartToEndSegmentVector.Dot(StartToPathPositionVector);
					const double Alpha = Dot / SegmentSquaredLength;
					const FVector Cross = StartToEndSegmentVector.Cross(StartToPathPositionVector);

					if (Alpha < -UE_SMALL_NUMBER || Alpha > (1 + UE_SMALL_NUMBER) || !Cross.IsNearlyZero())
					{
						// If we're not within the segment (Alpha not in [0,1]), or not aligned (cross != 0):
						// Try again with next segment
						SegmentStartIndex++;
						PathVertexIndex--;
						continue;
					}

					if (FMath::IsNearlyZero(Alpha))
					{
						// First point, copy over and continue
						InputPointData->CopyPointsTo(SplitPointPath, SegmentStartIndex, PathVertexIndex, 1);
					}
					else if (FMath::IsNearlyEqual(Alpha, 1.0))
					{
						// Second point, copy over, and increase the segment start index
						InputPointData->CopyPointsTo(SplitPointPath, SegmentEndIndex, PathVertexIndex, 1);
						SegmentStartIndex++;
					}
					else
					{
						// In between
						// We need to do a last verification, that the result direction is the same as the input segment direction, because if the input path
						// cuts itself, we might have multiple points that satisfy the alignment condition
						// Also by construction, a point in between vertices is always the first or last point of the result path
						FVector ResultDir{};
						if (PathVertexIndex == 0)
						{
							ResultDir = PathPositions[PathVertexIndex + 1] - PathPosition;
						}
						else
						{
							ResultDir = PathPosition - PathPositions[PathVertexIndex - 1];
						}
						
						const FVector ResultCross = StartToEndSegmentVector.Cross(ResultDir);
						if (!ResultCross.IsNearlyZero())
						{
							SegmentStartIndex++;
							PathVertexIndex--;
							continue;
						}
						
						// We're in between, do an interpolation
						TStaticArray<TPair<int32, float>, 2> Coefficients;
						Coefficients[0] = {SegmentStartIndex, 1.0 - Alpha};
						Coefficients[1] = {SegmentEndIndex, Alpha};
						
						// Do the weighted average for all, but override the final position and the seed.
						PCGPointDataHelpers::WeightedAverage(InputPointData, Coefficients, SplitPointPath, PathVertexIndex, /*bApplyOnMetadata=*/bHasMetadata);
						PathTransformRange[PathVertexIndex].SetLocation(PathPosition);
						PathSeedRange[PathVertexIndex] = PCGHelpers::ComputeSeedFromPosition(PathPosition);
					}
				}

				PathData.Add(SplitPointPath);
			}
			else
			{
				// shouldn't happen
				check(0);
				continue;
			}
		}

		// Create output data for all new path data.
		for (const UPCGSpatialData* PathDatum : PathData)
		{
			Context->OutputData.TaggedData.Add_GetRef(Input).Data = PathDatum;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE