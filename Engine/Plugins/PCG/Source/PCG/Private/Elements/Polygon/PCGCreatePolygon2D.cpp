// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Polygon/PCGCreatePolygon2D.h"

#include "PCGContext.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGPolygon2DData.h"
#include "Utils/PCGLogErrors.h"
#include "Helpers/PCGPolygon2DProcessingHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "CompGeom/PolygonTriangulation.h"
#include "Curve/PolygonOffsetUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCreatePolygon2D)

#define LOCTEXT_NAMESPACE "PCGCreatePolygonElement"

UPCGCreatePolygon2DSettings::UPCGCreatePolygon2DSettings()
{
	HoleIndexAttribute.Update(StaticEnum<EPCGPolygon2DProperties>()->GetNameStringByValue(static_cast<int64>(EPCGPolygon2DProperties::HoleIndex)));
}

TArray<FPCGPinProperties> UPCGCreatePolygon2DSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	// @todo_pcg: add landscape spline support?
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point | EPCGDataType::Spline).SetRequiredPin();

	return PinProperties;
}

bool FPCGCreatePolygonElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSplitSplineElement::Execute);

	const UPCGCreatePolygon2DSettings* Settings = Context->GetInputSettings<UPCGCreatePolygon2DSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	TArray<FVector> Positions;
	TArray<int> HoleIndices;
	TArray<PCGMetadataEntryKey> EntryKeys;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGSpatialData* SpatialData = nullptr;
		FVector DataNormal = Settings->DefaultPolygonNormal.GetSafeNormal(UE_SMALL_NUMBER, FVector::ZAxisVector);

		bool bInputIsOpen = false;
		bool bInputFromClosedSpline = false;
		bool bBuildEntryKeysBySampling = false;
		bool bInputHasMetadata = false;

		Positions.Reset();
		HoleIndices.Reset();
		EntryKeys.Reset();

		if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(Input.Data))
		{
			SpatialData = SplineData;
			DataNormal = SplineData->GetTransform().GetUnitAxis(EAxis::Z);
			bInputFromClosedSpline = SplineData->IsClosed();
			bInputIsOpen = !bInputFromClosedSpline;

			// @todo_pcg: this method doesn't allow proper metadata inheritance, we will need to update it.
			const double ErrorTolerance = FMath::Max(Settings->SplineMaxDiscretizationError, 0.001);
			SplineData->SplineStruct.ConvertSplineToPolyLine(ESplineCoordinateSpace::World, ErrorTolerance, Positions);

			if (bInputFromClosedSpline && !Positions.IsEmpty())
			{
				Positions.Pop();
			}

			bInputHasMetadata = !SplineData->GetConstVerticesEntryKeys().IsEmpty();
			bBuildEntryKeysBySampling = bInputHasMetadata;
		}
		else if (const UPCGBasePointData* InputPointData = Cast<UPCGBasePointData>(Input.Data))
		{
			SpatialData = InputPointData;

			const TConstPCGValueRange<FTransform> TransformRange = InputPointData->GetConstTransformValueRange();

			if (TransformRange.Num() > 0)
			{
				Positions.Reserve(TransformRange.Num());
				for (const FTransform& Transform : TransformRange)
				{
					Positions.Emplace(Transform.GetLocation());
				}
			}

			const TConstPCGValueRange<PCGMetadataEntryKey> EntryKeysRange = InputPointData->GetConstMetadataEntryValueRange();

			if (EntryKeysRange.Num() > 0)
			{
				bInputHasMetadata = true;

				// There's no point in getting the entry keys if we're extruding the input path.
				if (Settings->InputType != EPCGCreatePolygonInputType::ForceOpen)
				{
					EntryKeys.Reserve(EntryKeysRange.Num());
					for (const PCGMetadataEntryKey& EK : EntryKeysRange)
					{
						EntryKeys.Add(EK);
					}
				}

				// Get hole index information if relevant
				if (Settings->bUseHoleAttribute &&
					!PCGAttributeAccessorHelpers::ExtractAllValues(Input.Data, Settings->HoleIndexAttribute, HoleIndices, Context))
				{
					continue;
				}
			}
		}
		else
		{
			// Unsupported type
			PCGLog::InputOutput::LogInvalidInputDataError(Context);
			continue;
		}

		double OpenPolygonWidth = Settings->OpenPolygonWidth;

		if (Settings->bUsePolygonWidthAttribute)
		{
			double PolyWidthFromData = 0.0;
			if (PCGAttributeAccessorHelpers::ExtractParamValue(Input.Data, Settings->PolygonWidthAttribute, PolyWidthFromData, Context))
			{
				OpenPolygonWidth = PolyWidthFromData;
			}
		}

		// Project the positions so they lie on a plane. Note that we'll assume at this point that the polygon is always closed, regardless of whether that's true or not, without loss of generality.
		FVector PlaneNormal = FVector::UpVector;
		FVector PlaneCentroid = FVector::ZeroVector;
		double PolygonArea = PolygonTriangulation::ComputePolygonPlane(Positions, PlaneNormal, PlaneCentroid);

		// Compute bounds for the input points so we have a comparison basis vs the polygon area
		FBox PolygonBounds(EForceInit::ForceInit);

		for (const FVector& Position : Positions)
		{
			PolygonBounds += Position;
		}

		const double LargestDimension = PolygonBounds.GetSize().GetMax();
		const bool bDegeneratePolygonCase = PolygonArea <= UE_SMALL_NUMBER * FMath::Square(LargestDimension);

		const bool bCreateOpenPolygon = (Settings->InputType == EPCGCreatePolygonInputType::ForceOpen || (bInputIsOpen && Settings->InputType == EPCGCreatePolygonInputType::Automatic));

		// If the polygon is a point/line or otherwise has a very small area, always use the provided default up axis.
		// Alternatively, use the default polygon normal based on the normal behavior and the polygon type.
		if (bDegeneratePolygonCase || 
			(Settings->DefaultPolygonNormalBehavior == EPCGCreatePolygonDefaultNormalBehavior::OnlyForOpenPolygons && bCreateOpenPolygon) ||
			Settings->DefaultPolygonNormalBehavior == EPCGCreatePolygonDefaultNormalBehavior::Always)
		{
			PlaneNormal = Settings->DefaultPolygonNormal.GetSafeNormal(UE_SMALL_NUMBER, FVector::ZAxisVector);
		}

		// Ensure normal is aligned to expectation otherwise it will break sampling, conversion to points, etc.
		if ((PlaneNormal | DataNormal) < 0)
		{
			PlaneNormal = -PlaneNormal;
		}

		FTransform PlaneTransform;
		PlaneTransform.SetRotation(FRotationMatrix::MakeFromZ(PlaneNormal).ToQuat());
		PlaneTransform.SetLocation(PlaneCentroid);

		TArray<FVector2D> Positions2D;
		Positions2D.Reserve(Positions.Num());
		for (const FVector& Position : Positions)
		{
			FVector LocalPosition = PlaneTransform.InverseTransformPosition(Position);
			Positions2D.Emplace(LocalPosition.X, LocalPosition.Y);
		}

		// Create starting polygon, which will be used as-is if this isn't an open polygon.
		// 1. Split positions into outer/holes...
		UE::Geometry::FGeneralPolygon2d SourcePolygon;

		if (HoleIndices.IsEmpty())
		{
			// Special case: if we had a closed polygon but forced this here to be considered as an opened polygon, we'll still respect the loop.
			if (bCreateOpenPolygon && bInputFromClosedSpline && !Positions2D.IsEmpty())
			{
				const FVector2D FirstPosition2D = Positions2D[0];
				Positions2D.Add(FirstPosition2D);
			}

			UE::Geometry::FPolygon2d Polygon(Positions2D);
			SourcePolygon.SetOuter(MoveTemp(Polygon));
		}
		else if (HoleIndices.Num() != Positions2D.Num())
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("PositionsAndHoleAttributeMismatch", "There needs to be the same number of hole indices as there are points in the data."), Context);
			continue;
		}
		else if(!Positions2D.IsEmpty())
		{
			// Some assumptions here:
			// Same hole vertices should be continuous.
			// Increasing hole values, assumed to be starting from -1.
			int CurrentIndex = 0;
			while (CurrentIndex != Positions2D.Num())
			{
				UE::Geometry::FPolygon2d Polygon;
				bool bIsOuter = (CurrentIndex == 0);

				const int StartIndex = CurrentIndex;
				while (CurrentIndex < Positions2D.Num() && HoleIndices[CurrentIndex] == HoleIndices[StartIndex])
				{
					Polygon.AppendVertex(Positions2D[CurrentIndex++]);
				}

				// Similarly to the closed spline case, when we're forcing open a representation that is supposed to be closed, we need to duplicate the start/end point
				if (bCreateOpenPolygon)
				{
					Polygon.AppendVertex(Positions2D[StartIndex]);
				}

				if (bIsOuter)
				{
					SourcePolygon.SetOuter(MoveTemp(Polygon));
				}
				else
				{
					SourcePolygon.AddHole(MoveTemp(Polygon));
				}
			}
		}

		// Cleanup
		if (SourcePolygon.OuterIsClockwise())
		{
			SourcePolygon.Reverse();
		}

		TArray<UE::Geometry::FGeneralPolygon2d> OutPolygons;

		if (bCreateOpenPolygon)
		{
			TConstArrayView<UE::Geometry::FGeneralPolygon2d> PolyView = MakeConstArrayView(&SourcePolygon, 1);
			TArray<UE::Geometry::FGeneralPolygon2d> OffsetPolygons;

			//Note: same values as FGeometryScriptOpenPathOffsetOptions, but could be parametrized in advanced settings
			const UE::Geometry::EPolygonOffsetEndType EndType = PCGPolygon2DUtils::GetEndType(Settings->EndType);
			const UE::Geometry::EPolygonOffsetJoinType JoinType = PCGPolygon2DUtils::GetJoinType(Settings->JoinType);
			const double MiterLimit = 2.0;
			const double MaxStepsPerRadian = 10.0;
			const double DefaultStepsPerRadianScale = 1.0;
			const double Offset = OpenPolygonWidth;

			bool bOperationSuccessful = UE::Geometry::PolygonsOffset(
				Offset,
				PolyView,
				OffsetPolygons,
				/*bCopyOnFailure=*/false,
				MiterLimit,
				JoinType,
				EndType,
				MaxStepsPerRadian,
				DefaultStepsPerRadianScale);

			if (!bOperationSuccessful)
			{
				// The offset failed, not something we can do a lot about at this point.
				PCGLog::LogWarningOnGraph(NSLOCTEXT("PCGCreatePolygon2DElement", "ErrorInPolygonOffset", "Unable to create extruded polygon from open polygon."), Context);
				continue;
			}
			else
			{
				OutPolygons = MoveTemp(OffsetPolygons);
			}
		}
		else
		{
			// Move source polygon into out polygons
			OutPolygons.Add(MoveTemp(SourcePolygon));
		}

		for (UE::Geometry::FGeneralPolygon2d& OutPoly : OutPolygons)
		{
			if (OutPoly.OuterIsClockwise())
			{
				OutPoly.Reverse();
			}

			UPCGPolygon2DData* Polygon = FPCGContext::NewObject_AnyThread<UPCGPolygon2DData>(Context);
			Polygon->InitializeFromData(SpatialData);
			Context->OutputData.TaggedData.Add_GetRef(Input).Data = Polygon;

			// Once we've created the polygon object we can assign new metadata entries as needed.
			if (bCreateOpenPolygon && bInputHasMetadata)
			{
				EntryKeys = PCGPolygon2DProcessingHelpers::ComputeEntryKeysByNearestSegmentSampling(SpatialData, OpenPolygonWidth, OutPoly, PlaneTransform, Polygon->MutableMetadata());
			}
			else if (bBuildEntryKeysBySampling)
			{
				EntryKeys = PCGPolygon2DProcessingHelpers::ComputeEntryKeysBySampling(SpatialData, Positions, Polygon->MutableMetadata());
			}

			TConstArrayView<PCGMetadataEntryKey> EntryKeysView(EntryKeys);
			Polygon->SetPolygon(MoveTemp(OutPoly), EntryKeysView.IsEmpty() ? nullptr : &EntryKeysView);
			Polygon->SetTransform(PlaneTransform);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE