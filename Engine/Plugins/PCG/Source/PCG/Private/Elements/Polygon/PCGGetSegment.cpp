// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Polygon/PCGGetSegment.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGPolygon2DData.h"
#include "Elements/PCGSplitSplines.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetSegment)

#define LOCTEXT_NAMESPACE "PCGGetSegmentElement"

namespace PCGGetSegment
{
	namespace Constants
	{
		const FLazyName SegmentIndexLabel = TEXT("SegmentIndex");
		const FLazyName HoleIndexLabel = TEXT("HoleIndex");
	}

	namespace Log
	{
		static const FText InvalidSegmentIndex = LOCTEXT("InvalidSegmentIndex", "Invalid segment index; got '{0}', expected a value between '{1}' and '{2}'.");
		static const FText InvalidHoleIndex = LOCTEXT("InvalidHoleIndex", "Invalid hole index; got '{0}', expected a value under '{1}'.");
	}
}

#if WITH_EDITOR
EPCGChangeType UPCGGetSegmentSettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGGetSegmentSettings, bUseInputSegmentData) ||
		PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGGetSegmentSettings, bUseInputHoleData) ||
		PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGGetSegmentSettings, bOutputSplineData))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif

TArray<FPCGPinProperties> UPCGGetSegmentSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point | EPCGDataType::Spline | EPCGDataType::Polygon2D).SetRequiredPin();

	if (bUseInputSegmentData)
	{
		PinProperties.Emplace_GetRef(PCGGetSegment::Constants::SegmentIndexLabel, EPCGDataType::Param).SetRequiredPin();
	}

	if (bUseInputHoleData)
	{
		PinProperties.Emplace_GetRef(PCGGetSegment::Constants::HoleIndexLabel, EPCGDataType::Param).SetRequiredPin();
	}

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGetSegmentSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, bOutputSplineData ? EPCGDataType::Spline : EPCGDataType::Point).SetRequiredPin();

	return PinProperties;
}

bool FPCGGetSegmentElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetSegmentElement::Execute);

	const UPCGGetSegmentSettings* Settings = Context->GetInputSettings<UPCGGetSegmentSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData> SegmentInputs;
	TArray<FPCGTaggedData> HoleInputs;

	if (Settings->bUseInputSegmentData)
	{
		SegmentInputs = Context->InputData.GetInputsByPin(PCGGetSegment::Constants::SegmentIndexLabel);

		// Validate cardinality
		if (SegmentInputs.Num() != 1 && SegmentInputs.Num() != Inputs.Num())
		{
			PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, PCGGetSegment::Constants::SegmentIndexLabel, Context);
			return true;
		}
	}

	if (Settings->bUseInputHoleData)
	{
		HoleInputs = Context->InputData.GetInputsByPin(PCGGetSegment::Constants::HoleIndexLabel);

		if (HoleInputs.Num() != 1 && HoleInputs.Num() != Inputs.Num())
		{
			PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, PCGGetSegment::Constants::HoleIndexLabel, Context);
			return true;
		}
	}

	const int32 ConstantSegmentIndex = Settings->SegmentIndex;
	const int32 ConstantHoleIndex = Settings->HoleIndex;

	for (int InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		const FPCGTaggedData& Input = Inputs[InputIndex];

		int32 SegmentIndex = ConstantSegmentIndex;
		int32 HoleIndex = ConstantHoleIndex;
		
		if (Settings->bUseInputSegmentData)
		{
			const UPCGParamData* ParamData = Cast<UPCGParamData>(SegmentInputs[InputIndex % SegmentInputs.Num()].Data);

			if (!ParamData)
			{
				PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::Param, PCGGetSegment::Constants::SegmentIndexLabel, Context);
				continue;
			}
			
			// Retrieve segment value
			int SegmentIndexFromParam = -1;
			if (PCGAttributeAccessorHelpers::ExtractParamValue(ParamData, Settings->SegmentIndexAttribute, SegmentIndexFromParam, Context))
			{
				SegmentIndex = SegmentIndexFromParam;
			}
		}

		if (Settings->bUseInputHoleData)
		{
			const UPCGParamData* ParamData = Cast<UPCGParamData>(HoleInputs[InputIndex % HoleInputs.Num()].Data);

			if (!ParamData)
			{
				PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::Param, PCGGetSegment::Constants::HoleIndexLabel, Context);
				continue;
			}

			// Retrieve hole value only the input supports holes, e.g. is a Polygon2D.
			int HoleIndexFromParam = -1;
			if (Cast<UPCGPolygon2DData>(Input.Data) && PCGAttributeAccessorHelpers::ExtractParamValue(ParamData, Settings->HoleIndexAttribute, HoleIndexFromParam, Context))
			{
				HoleIndex = HoleIndexFromParam;
			}
			else
			{
				HoleIndex = -1;
			}
		}

		const int OriginalSegmentIndex = SegmentIndex;

		UPCGData* OutputData = nullptr;

		auto UpdateAndValidateSegmentIndex = [](int& SegmentIndex, int SegmentCount) -> bool
		{
			// Support negative indices
			if (SegmentIndex < 0)
			{
				SegmentIndex += SegmentCount;
			}

			// If the index is still negative or over the segment count, it's a bad segment index
			return SegmentIndex >= 0 && SegmentIndex < SegmentCount;
		};

		FTransform StartTransform = FTransform::Identity;
		FTransform EndTransform = FTransform::Identity;
		bool bHasMetadata = false;
		PCGMetadataEntryKey StartEntryKey = PCGInvalidEntryKey;
		PCGMetadataEntryKey EndEntryKey = PCGInvalidEntryKey;

		auto CreatePointDataFromStartEnd = [Context, &StartTransform, &EndTransform, &bHasMetadata, &StartEntryKey, &EndEntryKey](const UPCGSpatialData* InData) -> UPCGData*
		{
			UPCGBasePointData* OutPointData = FPCGContext::NewPointData_AnyThread(Context);

			FPCGInitializeFromDataParams InitializeDataParams(InData);
			InitializeDataParams.bInheritMetadata = InitializeDataParams.bInheritAttributes = true;
			InitializeDataParams.bInheritSpatialData = false;
			OutPointData->InitializeFromDataWithParams(InitializeDataParams);
			OutPointData->SetNumPoints(2);
			OutPointData->AllocateProperties(EPCGPointNativeProperties::Transform | (bHasMetadata ? EPCGPointNativeProperties::MetadataEntry : EPCGPointNativeProperties::None));
			
			TPCGValueRange<FTransform> OutTransformRange = OutPointData->GetTransformValueRange(/*bAllocate=*/false);
			OutTransformRange[0] = StartTransform;
			OutTransformRange[1] = EndTransform;

			if (bHasMetadata)
			{
				TPCGValueRange<PCGMetadataEntryKey> OutMetadataEntryRange = OutPointData->GetMetadataEntryValueRange(/*bAllocate=*/false);
				OutMetadataEntryRange[0] = StartEntryKey;
				OutMetadataEntryRange[1] = EndEntryKey;
			}

			return OutPointData;
		};

		auto CreateSplineDataFromStartEnd = [Context, &StartTransform, &EndTransform, &bHasMetadata, &StartEntryKey, &EndEntryKey](const UPCGSpatialData* InData, const FTransform& InTransform) -> UPCGData*
		{
			UPCGSplineData* OutSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
			OutSplineData->InitializeFromData(InData);

			TArray<FSplinePoint> SplinePoints;
			SplinePoints.Reserve(2);

			FTransform LocalStartTransform = StartTransform.GetRelativeTransform(InTransform);
			FTransform LocalEndTransform = EndTransform.GetRelativeTransform(InTransform);

			SplinePoints.Emplace(0,
				LocalStartTransform.GetLocation(),
				FVector::ZeroVector,
				FVector::ZeroVector,
				LocalStartTransform.GetRotation().Rotator(),
				LocalStartTransform.GetScale3D(),
				ESplinePointType::Linear);

			SplinePoints.Emplace(1,
				LocalEndTransform.GetLocation(),
				FVector::ZeroVector,
				FVector::ZeroVector,
				LocalEndTransform.GetRotation().Rotator(),
				LocalEndTransform.GetScale3D(),
				ESplinePointType::Linear);

			TArray<PCGMetadataEntryKey> SplineEntryKeys;
			if (bHasMetadata)
			{
				SplineEntryKeys.Add(StartEntryKey);
				SplineEntryKeys.Add(EndEntryKey);
			}

			OutSplineData->Initialize(SplinePoints, /*bClosedLoop=*/false, InTransform, MoveTemp(SplineEntryKeys));
			return OutSplineData;
		};

		if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(Input.Data))
		{
			const int NumSegments = SplineData->GetNumSegments();
			if (!UpdateAndValidateSegmentIndex(SegmentIndex, NumSegments))
			{
				PCGLog::LogWarningOnGraph(FText::Format(PCGGetSegment::Log::InvalidSegmentIndex, FText::AsNumber(OriginalSegmentIndex), FText::AsNumber(-NumSegments), FText::AsNumber(NumSegments - 1)), Context);
				continue;
			}

			const double StartKey = SegmentIndex;
			const double EndKey = SegmentIndex + 1;

			if (Settings->bOutputSplineData)
			{
				UPCGSplineData* OutSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
				OutSplineData->InitializeFromData(SplineData);

				if (PCGSplitSpline::SplitSpline(SplineData, OutSplineData, StartKey, EndKey) == PCGSplitSpline::EStatus::Split)
				{
					OutputData = OutSplineData;
				}
			}
			else
			{
				StartTransform = SplineData->GetTransformAtAlpha(StartKey / static_cast<double>(NumSegments));
				EndTransform = SplineData->GetTransformAtAlpha(EndKey / static_cast<double>(NumSegments));

				TConstArrayView<PCGMetadataEntryKey> SplineMetadataEntryKeys = SplineData->GetConstVerticesEntryKeys();
				bHasMetadata = !SplineMetadataEntryKeys.IsEmpty();

				if (bHasMetadata)
				{
					StartEntryKey = SplineMetadataEntryKeys[SegmentIndex];
					EndEntryKey = SplineMetadataEntryKeys[(SegmentIndex + 1) % SplineMetadataEntryKeys.Num()];
				}

				OutputData = CreatePointDataFromStartEnd(SplineData);
			}
		}
		else if (const UPCGPolygon2DData* PolygonData = Cast<UPCGPolygon2DData>(Input.Data))
		{
			const UE::Geometry::FGeneralPolygon2d& GeneralPolygon = PolygonData->GetPolygon();
			if (HoleIndex != -1 && !GeneralPolygon.GetHoles().IsValidIndex(HoleIndex))
			{
				PCGLog::LogWarningOnGraph(FText::Format(PCGGetSegment::Log::InvalidHoleIndex, FText::AsNumber(HoleIndex), FText::AsNumber(GeneralPolygon.GetHoles().Num())), Context);
				continue;
			}

			const UE::Geometry::FPolygon2d& Polygon = (HoleIndex == -1 ? GeneralPolygon.GetOuter() : GeneralPolygon.GetHoles()[HoleIndex]);
			const int NumSegments = Polygon.VertexCount();

			if (!UpdateAndValidateSegmentIndex(SegmentIndex, NumSegments))
			{
				PCGLog::LogWarningOnGraph(FText::Format(PCGGetSegment::Log::InvalidSegmentIndex, FText::AsNumber(OriginalSegmentIndex), FText::AsNumber(-NumSegments), FText::AsNumber(NumSegments - 1)), Context);
				continue;
			}

			// We have a polygon-centric index, we want to get the polygon data-centric start/end indices
			int StartIndex = INDEX_NONE;
			int EndIndex = INDEX_NONE;
			PCGPolygon2DData::GetStartEndVertexIndices(PolygonData->GetPolygon(), SegmentIndex, HoleIndex, StartIndex, EndIndex);
			
			// Get transform & metadata entries
			StartTransform = PolygonData->GetTransformAtAlpha(StartIndex / static_cast<double>(PolygonData->GetNumSegments()));
			EndTransform = PolygonData->GetTransformAtAlpha(EndIndex / static_cast<double>(PolygonData->GetNumSegments()));

			TConstArrayView<PCGMetadataEntryKey> PolygonEntryKeys = PolygonData->GetConstVerticesEntryKeys();
			bHasMetadata = !PolygonEntryKeys.IsEmpty();

			if (bHasMetadata)
			{
				StartEntryKey = PolygonEntryKeys[StartIndex];
				EndEntryKey = PolygonEntryKeys[EndIndex];
			}
			
			if (Settings->bOutputSplineData)
			{
				OutputData = CreateSplineDataFromStartEnd(PolygonData, PolygonData->GetTransform());
			}
			else
			{
				OutputData = CreatePointDataFromStartEnd(PolygonData);
			}
		}
		else if(const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Input.Data))
		{
			const TConstPCGValueRange<FTransform> TransformRange = PointData->GetConstTransformValueRange();
			const TConstPCGValueRange<PCGMetadataEntryKey> EntryKeysRange = PointData->GetConstMetadataEntryValueRange();

			const int32 NumPoints = PointData->GetNumPoints();

			if (!UpdateAndValidateSegmentIndex(SegmentIndex, NumPoints))
			{
				PCGLog::LogWarningOnGraph(FText::Format(PCGGetSegment::Log::InvalidSegmentIndex, FText::AsNumber(OriginalSegmentIndex), FText::AsNumber(-NumPoints), FText::AsNumber(NumPoints - 1)), Context);
				continue;
			}

			int StartIndex = SegmentIndex;
			int EndIndex = ((SegmentIndex + 1) % NumPoints);

			StartTransform = TransformRange[StartIndex];
			EndTransform = TransformRange[EndIndex];

			bHasMetadata = !EntryKeysRange.IsEmpty();
			if (bHasMetadata)
			{
				StartEntryKey = EntryKeysRange[StartIndex];
				EndEntryKey = EntryKeysRange[EndIndex];
			}

			if (Settings->bOutputSplineData)
			{
				OutputData = CreateSplineDataFromStartEnd(PointData, FTransform::Identity);
			}
			else
			{
				OutputData = CreatePointDataFromStartEnd(PointData);
			}
		}

		if (OutputData)
		{
			Context->OutputData.TaggedData.Add_GetRef(Input).Data = OutputData;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE