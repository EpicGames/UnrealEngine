// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Polygon/PCGOffsetPolygon2D.h"

#include "PCGContext.h"
#include "Data/PCGPolygon2DData.h"
#include "Helpers/PCGPolygon2DProcessingHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Curve/PolygonOffsetUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGOffsetPolygon2D)

namespace PCGOffsetPolygon2D
{
	const FLazyName OffsetPinLabel = "Offset";
}

#if WITH_EDITOR
EPCGChangeType UPCGOffsetPolygon2DSettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGOffsetPolygon2DSettings, bUseOffsetFromInput))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGOffsetPolygon2DSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties = PCGPolygon2DUtils::DefaultPolygonInputPinProperties();

	if (bUseOffsetFromInput)
	{
		PinProperties.Emplace_GetRef(PCGOffsetPolygon2D::OffsetPinLabel, EPCGDataType::Param | EPCGDataType::Polygon2D).SetRequiredPin();
	}

	return PinProperties;
}

bool FPCGOffsetPolygonElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGOffsetPolygonElement::Execute);

	const UPCGOffsetPolygon2DSettings* Settings = Context->GetInputSettings<UPCGOffsetPolygon2DSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	TArray<FPCGTaggedData> OffsetInputs;
	if (Settings->bUseOffsetFromInput)
	{
		OffsetInputs = Context->InputData.GetInputsByPin(PCGOffsetPolygon2D::OffsetPinLabel);

		// Verify that cardinality matches
		if (OffsetInputs.Num() != 1 && OffsetInputs.Num() != Inputs.Num())
		{
			PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, PCGOffsetPolygon2D::OffsetPinLabel, Context);
			return true;
		}
	}

	for(int InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		const FPCGTaggedData& Input = Inputs[InputIndex];
		const UPCGPolygon2DData* PolygonData = Cast<UPCGPolygon2DData>(Input.Data);

		if (!PolygonData)
		{
			PCGLog::InputOutput::LogInvalidInputDataError(Context);
			continue;
		}

		const UE::Geometry::FGeneralPolygon2d& Polygon = PolygonData->GetPolygon();
		TConstArrayView<UE::Geometry::FGeneralPolygon2d> PolyView = MakeConstArrayView(&Polygon, 1);
		TArray<UE::Geometry::FGeneralPolygon2d> OutPolys;

		bool bOperationSuccess = false;

		double Offset = Settings->Offset;

		if (Settings->bUseOffsetFromInput)
		{
			const UPCGData* OffsetData = OffsetInputs[InputIndex % OffsetInputs.Num()].Data;
			double OffsetFromData = 0.0;
			if (PCGAttributeAccessorHelpers::ExtractParamValue(OffsetData, Settings->OffsetAttribute, OffsetFromData, Context))
			{
				Offset = OffsetFromData;
			}
		}

		const double MiterLimit = 2.0;
		const UE::Geometry::EPolygonOffsetJoinType JoinType = PCGPolygon2DUtils::GetJoinType(Settings->JoinType);
		const UE::Geometry::EPolygonOffsetEndType EndType = UE::Geometry::EPolygonOffsetEndType::Polygon;

		if (Settings->Operation == EPCGPolygonOffsetOperation::Offset)
		{
			bOperationSuccess = UE::Geometry::PolygonsOffset(
				Offset,
				PolyView,
				OutPolys,
				/*bCopyOnFailure=*/false,
				MiterLimit,
				JoinType,
				EndType);
		}
		else
		{
			const double FirstOffset = (Settings->Operation == EPCGPolygonOffsetOperation::Open) ? -Offset : +Offset;
			const double SecondOffset = -FirstOffset;

			bOperationSuccess = UE::Geometry::PolygonsOffsets(
				FirstOffset,
				SecondOffset,
				PolyView,
				OutPolys,
				/*bCopyOnFailure=*/false,
				MiterLimit,
				JoinType,
				EndType);
		}

		if (bOperationSuccess)
		{
			TArray<PCGPolygon2DProcessingHelpers::PolygonVertexMapping> VertexMapping;
			// Since the PolyToMetadataInfoMap refers to the polygon entry keys, this array must exist in the same (or larger) scope than the info map object.
			TArray<PCGMetadataEntryKey> PolygonEntryKeys;
			PCGPolygon2DProcessingHelpers::PolyToMetadataInfoMap PolyToMetadataInfoMap;

			if (Settings->bInheritMetadata)
			{
				PolygonEntryKeys = PolygonData->GetConstVerticesEntryKeys();
				PCGPolygon2DProcessingHelpers::AddToPolyMetadataInfoMap(PolyToMetadataInfoMap, Polygon, PolygonData->ConstMetadata(), PolygonEntryKeys);
				VertexMapping = PCGPolygon2DProcessingHelpers::BuildVertexMapping(OutPolys, PolyView, {});
			}

			for (int OutPolyIndex = 0; OutPolyIndex < OutPolys.Num(); ++OutPolyIndex)
			{
				UPCGPolygon2DData* OutPolyData = FPCGContext::NewObject_AnyThread<UPCGPolygon2DData>(Context);

				FPCGInitializeFromDataParams InitializeParams(PolygonData);
				InitializeParams.bInheritMetadata = InitializeParams.bInheritAttributes = Settings->bInheritMetadata;
				InitializeParams.bInheritSpatialData = true;

				OutPolyData->InitializeFromDataWithParams(InitializeParams);
				Context->OutputData.TaggedData.Add_GetRef(Input).Data = OutPolyData;

				TArray<PCGMetadataEntryKey> EntryKeys;
				if (Settings->bInheritMetadata)
				{
					EntryKeys = PCGPolygon2DProcessingHelpers::ComputeEntryKeysAndInitializeAttributes(OutPolyData->MutableMetadata(), VertexMapping[OutPolyIndex], PolyToMetadataInfoMap);
				}

				TConstArrayView<const PCGMetadataEntryKey> EntryKeysView = MakeConstArrayView(EntryKeys);

				OutPolyData->SetPolygon(OutPolys[OutPolyIndex], EntryKeysView.IsEmpty() ? nullptr : &EntryKeysView);
				OutPolyData->SetTransform(PolygonData->GetTransform());
			}
		}
		else
		{
			PCGLog::LogWarningOnGraph(NSLOCTEXT("PCGOffsetPolygon2D", "OperationFailed", "Polygon offset operation failed."), Context);
		}
	}

	return true;
}
