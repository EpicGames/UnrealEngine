// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Polygon/PCGPolygon2DOperation.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "Data/PCGPolygon2DData.h"
#include "Helpers/PCGPolygon2DProcessingHelpers.h"

#include "Curve/PolygonIntersectionUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPolygon2DOperation)

#define LOCTEXT_NAMESPACE "PCGPolygon2DOperation"

#if WITH_EDITOR
FText UPCGPolygon2DOperationSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Polygon Operation");
}

FText UPCGPolygon2DOperationSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Performs polygon operations between the inputs.");
}

TArray<FPCGPreConfiguredSettingsInfo> UPCGPolygon2DOperationSettings::GetPreconfiguredInfo() const
{
	return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EPCGPolygonOperation>({}, FTextFormat(LOCTEXT("PreconfigureInfo", "Polygon {0}")), FTextFormat(LOCTEXT("TooltipTextFormat", "{0}")));
}

EPCGChangeType UPCGPolygon2DOperationSettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGPolygon2DOperationSettings, Operation))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}

void UPCGPolygon2DOperationSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	if (DataVersion < FPCGCustomVersion::RemovedSpacesInPolygonPinLabels)
	{
		InOutNode->RenameInputPin(PCGPolygon2DUtils::Constants::Deprecated::OldClipPolysLabel, PCGPolygon2DUtils::Constants::ClipPolysLabel);
		InOutNode->RenameInputPin(PCGPolygon2DUtils::Constants::Deprecated::OldClipPathsLabel, PCGPolygon2DUtils::Constants::ClipPathsLabel);
	}
}
#endif // WITH_EDITOR

void UPCGPolygon2DOperationSettings::ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo)
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGPolygonOperation>())
	{
		if (EnumPtr->IsValidEnumValue(PreconfiguredInfo.PreconfiguredIndex))
		{
			Operation = EPCGPolygonOperation(PreconfiguredInfo.PreconfiguredIndex);
		}
	}
}

FString UPCGPolygon2DOperationSettings::GetAdditionalTitleInformation() const
{
	if (const UEnum* EnumPtr = StaticEnum<EPCGPolygonOperation>())
	{
		return EnumPtr->GetDisplayNameTextByValue(static_cast<int64>(Operation)).ToString();
	}
	else
	{
		return FString();
	}
}

TArray<FPCGPinProperties> UPCGPolygon2DOperationSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties = PCGPolygon2DUtils::DefaultPolygonInputPinProperties();

	if(Operation == EPCGPolygonOperation::Difference ||
		Operation == EPCGPolygonOperation::Intersection ||
		Operation == EPCGPolygonOperation::PairwiseIntersection ||
		Operation == EPCGPolygonOperation::ExclusiveOr)
	{
		PinProperties.Emplace(PCGPolygon2DUtils::Constants::ClipPolysLabel, EPCGDataType::Polygon2D);
	}
	else if (Operation == EPCGPolygonOperation::CutWithPaths)
	{
		PinProperties.Emplace(PCGPolygon2DUtils::Constants::ClipPathsLabel, EPCGDataType::Point | EPCGDataType::Spline);
	}

	return PinProperties;
}

EPCGElementExecutionLoopMode FPCGPolygon2DOperationElement::ExecutionLoopMode(const UPCGSettings* InSettings) const
{
	if (const UPCGPolygon2DOperationSettings* Settings = Cast<UPCGPolygon2DOperationSettings>(InSettings))
	{
		return (Settings->Operation == EPCGPolygonOperation::Union || Settings->Operation == EPCGPolygonOperation::InnerIntersection) ? EPCGElementExecutionLoopMode::NotALoop : EPCGElementExecutionLoopMode::SinglePrimaryPin;
	}
	else
	{
		return EPCGElementExecutionLoopMode::NotALoop;
	}
}

bool FPCGPolygon2DOperationElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPolygon2DOperationElement::Execute);
	using namespace PCGPolygon2DProcessingHelpers;

	const UPCGPolygon2DOperationSettings* Settings = Context->GetInputSettings<UPCGPolygon2DOperationSettings>();
	check(Settings);

	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData> ClipInputs;
	
	if (Settings->Operation == EPCGPolygonOperation::CutWithPaths)
	{
		ClipInputs = Context->InputData.GetInputsByPin(PCGPolygon2DUtils::Constants::ClipPathsLabel);
	}
	else
	{
		ClipInputs = Context->InputData.GetInputsByPin(PCGPolygon2DUtils::Constants::ClipPolysLabel);
	}

	bool bHasTransform = false;
	FTransform Transform = FTransform::Identity;
	TArray<UE::Geometry::FGeneralPolygon2d> Polygons;
	TArray<TArray<PCGMetadataEntryKey>> PolygonsVertexEntryKeys;
	TArray<const UPCGMetadata*> PolygonsMetadata;
	TArray<int> PolyIndexToInputIndex;
	TArray<UE::Geometry::FGeneralPolygon2d> ClipPolygons;
	TArray<TArray<PCGMetadataEntryKey>> ClipPolygonsVertexEntryKeys;
	TArray<const UPCGMetadata*> ClipPolygonsMetadata;

	for(int InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		if (const UPCGPolygon2DData* Polygon = Cast<UPCGPolygon2DData>(Inputs[InputIndex].Data))
		{
			PolyIndexToInputIndex.Add(InputIndex);
			PCGPolygon2DProcessingHelpers::AddPolygon(Polygon, Polygons, &PolygonsVertexEntryKeys, &PolygonsMetadata, bHasTransform, Transform);
		}
	}

	if (Settings->Operation == EPCGPolygonOperation::CutWithPaths)
	{
		UE::Geometry::FAxisAlignedBox2d PolyBounds;
		for (const UE::Geometry::FGeneralPolygon2d& Poly : Polygons)
		{
			PolyBounds.Contain(Poly.Bounds());
		}

		for (int ClipIndex = 0; ClipIndex < ClipInputs.Num(); ++ClipIndex)
		{
			UE::Geometry::FGeneralPolygon2d ClipPoly;
			if (PCGPolygon2DProcessingHelpers::GetEnclosingPolygonFromPath(ClipInputs[ClipIndex].Data, Transform, Settings->SplineMaxDiscretizationError, PolyBounds, ClipPoly))
			{
				ClipPolygons.Add(MoveTemp(ClipPoly));
				ClipPolygonsVertexEntryKeys.Emplace(); // no metadata
				ClipPolygonsMetadata.Add(nullptr); // no metadata
			}
		}
	}
	else
	{
		for (int ClipIndex = 0; ClipIndex < ClipInputs.Num(); ++ClipIndex)
		{
			if (const UPCGPolygon2DData* Polygon = Cast<UPCGPolygon2DData>(ClipInputs[ClipIndex].Data))
			{
				PCGPolygon2DProcessingHelpers::AddPolygon(Polygon, ClipPolygons, &ClipPolygonsVertexEntryKeys, &ClipPolygonsMetadata, bHasTransform, Transform);
			}
		}
	}

	const EPCGPolygonOperationMetadataMode MetadataMode = Settings->MetadataMode;

	// Build a map from polygon ptr to metadata for ease of access later
	PolyToMetadataInfoMap PolyToMetadataInfoMap;
	if (MetadataMode != EPCGPolygonOperationMetadataMode::None)
	{
		AddToPolyMetadataInfoMap(PolyToMetadataInfoMap, Polygons, PolygonsMetadata, PolygonsVertexEntryKeys);
	}

	if (MetadataMode == EPCGPolygonOperationMetadataMode::Full)
	{
		AddToPolyMetadataInfoMap(PolyToMetadataInfoMap, ClipPolygons, ClipPolygonsMetadata, ClipPolygonsVertexEntryKeys);
	}

	TArray<UE::Geometry::FGeneralPolygon2d> OutPolys;

	auto AddToOutput = [Context, &Transform, &PolyToMetadataInfoMap, MetadataMode](TArray<UE::Geometry::FGeneralPolygon2d>& Polys, const FPCGTaggedData* InputPtr, TArrayView<UE::Geometry::FGeneralPolygon2d> PrimaryPolygons, TArrayView<UE::Geometry::FGeneralPolygon2d> SecondaryPolygons)
	{
		TArray<PolygonVertexMapping> VertexMapping;

		if (MetadataMode != EPCGPolygonOperationMetadataMode::None)
		{
			TArrayView<UE::Geometry::FGeneralPolygon2d> FilteredSecondaryPolygons = ((MetadataMode == EPCGPolygonOperationMetadataMode::Full) ? SecondaryPolygons : TArrayView<UE::Geometry::FGeneralPolygon2d>());
			VertexMapping = BuildVertexMapping(Polys, PrimaryPolygons, FilteredSecondaryPolygons);
		}

		for (int PolyIndex = 0; PolyIndex < Polys.Num(); ++PolyIndex)
		{
			UPCGPolygon2DData* OutPolyData = FPCGContext::NewObject_AnyThread<UPCGPolygon2DData>(Context);

			if (InputPtr)
			{
				FPCGInitializeFromDataParams InitializeParams(Cast<UPCGSpatialData>(InputPtr->Data));
				InitializeParams.bInheritMetadata = InitializeParams.bInheritAttributes = (MetadataMode != EPCGPolygonOperationMetadataMode::None);
				InitializeParams.bInheritSpatialData = true;

				OutPolyData->InitializeFromDataWithParams(InitializeParams);
				Context->OutputData.TaggedData.Add_GetRef(*InputPtr).Data = OutPolyData;
			}
			else
			{
				Context->OutputData.TaggedData.Emplace_GetRef().Data = OutPolyData;
			}
			
			TArray<PCGMetadataEntryKey> EntryKeys;
			if (MetadataMode != EPCGPolygonOperationMetadataMode::None)
			{
				EntryKeys = ComputeEntryKeysAndInitializeAttributes(OutPolyData->MutableMetadata(), VertexMapping[PolyIndex], PolyToMetadataInfoMap);
			}

			TConstArrayView<PCGMetadataEntryKey> EntryKeysView = MakeConstArrayView(EntryKeys);

			OutPolyData->SetPolygon(MoveTemp(Polys[PolyIndex]), EntryKeysView.IsEmpty() ? nullptr : &EntryKeysView);
			OutPolyData->SetTransform(Transform);
		}

		Polys.Reset();
	};

	if (Settings->Operation == EPCGPolygonOperation::Union)
	{
		if (Polygons.Num() <= 1)
		{
			// Nothing to do.
			Context->OutputData.TaggedData.Append(Inputs);
		}
		else
		{
			if (UE::Geometry::PolygonsUnion(Polygons, OutPolys, /*bCopyInputOnFailure=*/false))
			{
				AddToOutput(OutPolys, nullptr, Polygons, {});
			}
			else if (!Settings->bQuiet)
			{
				PCGLog::LogWarningOnGraph(LOCTEXT("UnionFailed", "Union operation failed."), Context);
			}
		}
	}
	else if (Settings->Operation == EPCGPolygonOperation::InnerIntersection)
	{
		if (Polygons.Num() <= 1)
		{
			// Nothing to do
			Context->OutputData.TaggedData.Append(Inputs);
		}
		else
		{
			TArray<UE::Geometry::FGeneralPolygon2d> CurrentPolygons;
			CurrentPolygons.Add(Polygons[0]);

			for (int PolygonIndex = 1; PolygonIndex < Polygons.Num(); ++PolygonIndex)
			{
				TArrayView<UE::Geometry::FGeneralPolygon2d> ClipPolygon = MakeArrayView(&Polygons[PolygonIndex], 1);

				if (UE::Geometry::PolygonsIntersection(CurrentPolygons, ClipPolygon, OutPolys))
				{
					CurrentPolygons = MoveTemp(OutPolys);
					OutPolys.Reset();
				}
				else
				{
					if (!Settings->bQuiet)
					{
						PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("InnerIntersectionFailed", "Inner intersection operation failed at index '{0}'."), PolygonIndex), Context);
					}

					CurrentPolygons.Reset();
					break;
				}
			}

			OutPolys = MoveTemp(CurrentPolygons);
			AddToOutput(OutPolys, nullptr, Polygons, {});
		}
	}
	else if (ClipPolygons.IsEmpty())
	{
		// Nothing to do.
		Context->OutputData.TaggedData.Append(Inputs);
	}
	else
	{
		for(int PolygonIndex = 0; PolygonIndex < Polygons.Num(); ++PolygonIndex)
		{
			const FPCGTaggedData& CurrentInput = Inputs[PolyIndexToInputIndex[PolygonIndex]];

			UE::Geometry::FGeneralPolygon2d& Poly = Polygons[PolygonIndex];
			OutPolys.Reset();

			TArrayView<UE::Geometry::FGeneralPolygon2d> SubjPolygons = MakeArrayView(&Poly, 1);

			if (Settings->Operation == EPCGPolygonOperation::Difference ||
				Settings->Operation == EPCGPolygonOperation::Intersection ||
				Settings->Operation == EPCGPolygonOperation::ExclusiveOr)
			{
				if ((Settings->Operation == EPCGPolygonOperation::Difference && UE::Geometry::PolygonsDifference(SubjPolygons, ClipPolygons, OutPolys)) ||
					(Settings->Operation == EPCGPolygonOperation::Intersection && UE::Geometry::PolygonsIntersection(SubjPolygons, ClipPolygons, OutPolys)) ||
					(Settings->Operation == EPCGPolygonOperation::ExclusiveOr && UE::Geometry::PolygonsExclusiveOr(SubjPolygons, ClipPolygons, OutPolys)))
				{
					AddToOutput(OutPolys, &CurrentInput, SubjPolygons, ClipPolygons);
				}
				else if(!Settings->bQuiet)
				{
					PCGLog::LogWarningOnGraph(LOCTEXT("OperationFailed", "Polygon operation failed."), Context);
				}
			}
			else if (Settings->Operation == EPCGPolygonOperation::PairwiseIntersection)
			{
				for (int ClipIndex = 0; ClipIndex < ClipPolygons.Num(); ++ClipIndex)
				{
					OutPolys.Reset();

					TArrayView<UE::Geometry::FGeneralPolygon2d> ClipSubsetPolygons = MakeArrayView(&ClipPolygons[ClipIndex], 1);

					if (UE::Geometry::PolygonsIntersection(SubjPolygons, ClipSubsetPolygons, OutPolys))
					{
						AddToOutput(OutPolys, &CurrentInput, SubjPolygons, ClipPolygons);
					}
					else if (!Settings->bQuiet)
					{
						PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("PairwiseIntersectionFailed", "Pairwise polygon intersection failed at polygon '{0}' and clip polygon '{1}'."), PolygonIndex, ClipIndex), Context);
					}
				}
			}
			else if (Settings->Operation == EPCGPolygonOperation::CutWithPaths)
			{
				// Cutting with paths is a bit more involved, as we need to take the polygon, then intersect + difference it, and get 1-2 polygons out.
				// Then we need to cut these individually with the next path polygon and so on.
				TArray<UE::Geometry::FGeneralPolygon2d> OutPositivePolys;
				TArray<UE::Geometry::FGeneralPolygon2d> OutNegativePolys;
				TArray<UE::Geometry::FGeneralPolygon2d> CurrentGenerationPolygons;
				TArray<UE::Geometry::FGeneralPolygon2d> NextGenerationPolygons;

				CurrentGenerationPolygons.Add(Poly);
				bool bSuccess = true;

				for (int ClipIndex = 0; ClipIndex < ClipPolygons.Num(); ++ClipIndex)
				{
					TArrayView<UE::Geometry::FGeneralPolygon2d> ClipSubsetPolygons = MakeArrayView(&ClipPolygons[ClipIndex], 1);

					for (int GenerationPolyIndex = 0; GenerationPolyIndex < CurrentGenerationPolygons.Num(); ++GenerationPolyIndex)
					{
						TArrayView<UE::Geometry::FGeneralPolygon2d> CurrentSubjPolygons = MakeArrayView(&CurrentGenerationPolygons[GenerationPolyIndex], 1);
						if (UE::Geometry::PolygonsIntersection(CurrentSubjPolygons, ClipSubsetPolygons, OutPositivePolys) && UE::Geometry::PolygonsDifference(CurrentSubjPolygons, ClipSubsetPolygons, OutNegativePolys))
						{
							NextGenerationPolygons.Append(OutPositivePolys);
							NextGenerationPolygons.Append(OutNegativePolys);

							OutPositivePolys.Reset();
							OutNegativePolys.Reset();
						}
						else
						{
							bSuccess = false;
							break;
						}
					}

					if (bSuccess)
					{
						CurrentGenerationPolygons = MoveTemp(NextGenerationPolygons);
						NextGenerationPolygons.Reset();
					}
					else
					{
						break;
					}
				}

				if (bSuccess)
				{
					OutPolys = MoveTemp(CurrentGenerationPolygons);
					AddToOutput(OutPolys, &CurrentInput, SubjPolygons, ClipPolygons);
				}
				else if(!Settings->bQuiet)
				{
					PCGLog::LogWarningOnGraph(LOCTEXT("OperationCutWithPathsFailed", "Polygon operation (cut with paths) failed."), Context);
				}
			}
			else
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("InvalidOperation", "Invalid polygon operation."), Context);
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
