// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Polygon/PCGPolygon2DUtils.h"

namespace PCGPolygon2DUtils
{

UE::Geometry::EPolygonOffsetJoinType GetJoinType(EPCGPolygonJoinType JoinType)
{
	using namespace UE::Geometry;
	switch (JoinType)
	{
	case EPCGPolygonJoinType::Square:
		return EPolygonOffsetJoinType::Square;
	case EPCGPolygonJoinType::Round:
		return EPolygonOffsetJoinType::Round;
	case EPCGPolygonJoinType::Miter:
		return EPolygonOffsetJoinType::Miter;
	default:
		checkNoEntry();
	}
	return EPolygonOffsetJoinType::Square;
}

UE::Geometry::EPolygonOffsetEndType GetEndType(EPCGPolygonEndType EndType)
{
	using namespace UE::Geometry;
	switch (EndType)
	{
	case EPCGPolygonEndType::Butt:
		return EPolygonOffsetEndType::Butt;
	case EPCGPolygonEndType::Square:
		return EPolygonOffsetEndType::Square;
	case EPCGPolygonEndType::Round:
		return EPolygonOffsetEndType::Round;
	case EPCGPolygonEndType::Polygon:
		return EPolygonOffsetEndType::Polygon;
	default:
		checkNoEntry();
	}
	return EPolygonOffsetEndType::Square;
}

TArray<FPCGPinProperties> DefaultPolygonInputPinProperties()
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Polygon2D).SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> DefaultPolygonOutputPinProperties()
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Polygon2D);

	return PinProperties;
}

} // namespace PCGPolygon2DUtils
