// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Polygon/PCGSurfaceFromPolygon2D.h"

#include "PCGContext.h"
#include "Data/PCGPolygon2DData.h"
#include "Data/PCGPolygon2DInteriorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSurfaceFromPolygon2D)

TArray<FPCGPinProperties> UPCGCreateSurfaceFromPolygon2DSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Surface);
	return PinProperties;
}

bool FPCGCreateSurfaceFromPolygon2DElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreateSurfaceFromPolygon2DElement::Execute);

	const UPCGCreateSurfaceFromPolygon2DSettings* Settings = Context->GetInputSettings<UPCGCreateSurfaceFromPolygon2DSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGPolygon2DData* PolygonData = Cast<UPCGPolygon2DData>(Input.Data);

		if (!PolygonData)
		{
			continue;
		}

		UPCGPolygon2DInteriorSurfaceData* SurfaceData = FPCGContext::NewObject_AnyThread<UPCGPolygon2DInteriorSurfaceData>(Context);
		SurfaceData->Initialize(Context, PolygonData);

		Outputs.Add_GetRef(Input).Data = SurfaceData;
	}

	return true;
}
