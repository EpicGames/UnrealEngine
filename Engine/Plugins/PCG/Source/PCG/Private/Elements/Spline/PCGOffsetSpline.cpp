// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Spline/PCGOffsetSpline.h"

#include "Data/PCGSplineData.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

namespace PCGOffsetSpline
{
	const FLazyName OffsetMagnitudeLabel = TEXT("Magnitude");
}

TArray<FPCGPinProperties> UPCGOffsetSplineSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spline).SetRequiredPin();
	PinProperties.Emplace_GetRef(PCGOffsetSpline::OffsetMagnitudeLabel, EPCGDataType::Spline | EPCGDataType::Param).SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGOffsetSplineSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spline);

	return PinProperties;
}

bool FPCGOffsetSplineElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGOffsetSplineElement::Execute);

	check(Context);

	const UPCGOffsetSplineSettings* Settings = Context->GetInputSettings<UPCGOffsetSplineSettings>();
	check(Settings);

	// 1. Get inputs
	TArray<FPCGTaggedData> Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	TArray<FPCGTaggedData> MagnitudeInputs = Context->InputData.GetInputsByPin(PCGOffsetSpline::OffsetMagnitudeLabel);

	// 2. Validate cardinality
	if (MagnitudeInputs.Num() != 1 && MagnitudeInputs.Num() != Inputs.Num())
	{
		PCGLog::InputOutput::LogInvalidCardinalityError(PCGPinConstants::DefaultInputLabel, PCGOffsetSpline::OffsetMagnitudeLabel, Context);
		return true;
	}

	// 3. getting all the direction + magnitude data upfront (can be through extract all values)
	// 4. then move the control points, then compare the before/after distances and scale the tangents accordingly.
	for (int InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		const FPCGTaggedData& Input = Inputs[InputIndex];
		const UPCGSplineData* Spline = Cast<UPCGSplineData>(Input.Data);

		if (!Spline)
		{
			PCGLog::InputOutput::LogInvalidInputDataError(Context);
			continue;
		}

		TArray<FVector> Directions;
		if (!PCGAttributeAccessorHelpers::ExtractAllValues(Spline, Settings->DirectionAttribute, Directions, Context) || Directions.IsEmpty())
		{
			continue;
		}

		const FPCGTaggedData& MagnitudeInput = MagnitudeInputs[InputIndex % MagnitudeInputs.Num()];

		TArray<double> Magnitudes;
		if (!PCGAttributeAccessorHelpers::ExtractAllValues(MagnitudeInput.Data, Settings->MagnitudeAttribute, Magnitudes, Context) || Magnitudes.IsEmpty())
		{
			continue;
		}

		UPCGSplineData* OffsetSpline = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
		OffsetSpline->InitializeFromData(Spline);

		TArray<FSplinePoint> SplinePoints = Spline->GetSplinePoints();
		TArray<PCGMetadataEntryKey> SplinePointsEntryKeys;

		TConstArrayView<PCGMetadataEntryKey> ConstEntryKeys = Spline->GetConstVerticesEntryKeys();
		SplinePointsEntryKeys.Append(ConstEntryKeys);

		TArray<double> PreviousDistances;
		const int NumSegments = Spline->GetNumSegments();
		PreviousDistances.Reserve(SplinePoints.Num());
		for (int SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
		{
			PreviousDistances.Add((SplinePoints[SegmentIndex].Position - SplinePoints[(SegmentIndex + 1) % SplinePoints.Num()].Position).Length());
		}

		for (int PointIndex = 0; PointIndex < SplinePoints.Num(); ++PointIndex)
		{
			SplinePoints[PointIndex].Position += Spline->SplineStruct.Transform.InverseTransformVector(Directions[PointIndex % Directions.Num()]) * Magnitudes[PointIndex % Magnitudes.Num()];
		}

		for (int SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
		{
			const double NewDistance = (SplinePoints[SegmentIndex].Position - SplinePoints[(SegmentIndex + 1) % SplinePoints.Num()].Position).Length();

			const double Ratio = (PreviousDistances[SegmentIndex] > UE_DOUBLE_SMALL_NUMBER) ? NewDistance / PreviousDistances[SegmentIndex] : 1.0;
			SplinePoints[SegmentIndex].LeaveTangent *= Ratio;
			SplinePoints[(SegmentIndex + 1) % SplinePoints.Num()].ArriveTangent *= Ratio;
		}

		OffsetSpline->Initialize(MoveTemp(SplinePoints), Spline->IsClosed(), Spline->GetTransform(), MoveTemp(SplinePointsEntryKeys));

		Context->OutputData.TaggedData.Add_GetRef(Input).Data = OffsetSpline;
	}

	return true;
}