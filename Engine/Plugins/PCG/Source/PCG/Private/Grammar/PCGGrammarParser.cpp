// Copyright Epic Games, Inc. All Rights Reserved.

#include "Grammar/PCGGrammarParser.h"

#include "Algo/Transform.h"
#include "Internationalization/Internationalization.h"
#include "String/ParseTokens.h"

#define LOCTEXT_NAMESPACE "PCGGrammar"

void FPCGGrammarResult::AddLog(FText Message, ELogType Verbosity)
{
	Logs.AddUnique({ std::move(Message), Verbosity });
	if (Verbosity == ELogType::Error)
	{
		bSuccess = false;
	}
}

namespace PCGGrammar
{
	bool BuildModules(const FString& InGrammar, FPCGGrammarResult& Result)
	{
		static constexpr TCHAR Delimiter[] = TEXT(",");
		static constexpr TCHAR WeightDelimiter[] = TEXT(":");
		static constexpr TCHAR SequenceStart[] = TEXT("[");
		static constexpr TCHAR SequenceEnd[] = TEXT("]");
		static constexpr TCHAR PriorityStart[] = TEXT("<");
		static constexpr TCHAR PriorityEnd[] = TEXT(">");
		static constexpr TCHAR StochasticStart[] = TEXT("{");
		static constexpr TCHAR StochasticEnd[] = TEXT("}");
		static constexpr TCHAR InfiniteRepetitionStr[] = TEXT("*");
		static constexpr TCHAR AtLeastOneRepetitionStr[] = TEXT("+");

		// TODO: parse tokens in a more efficient way; this relies on token separation from what should essentially be token delimiters
		// This would also allow better information to the end user when the grammar is invalid (as we wouldn't need to modify the string).
		FString Grammar = InGrammar;
		Grammar.ReplaceInline(TEXT("\r"), TEXT(""));
		Grammar.ReplaceInline(TEXT("\n"), TEXT(""));
		Grammar.ReplaceInline(Delimiter, TEXT(" , "));
		Grammar.ReplaceInline(WeightDelimiter, TEXT(" : "));
		Grammar.ReplaceInline(SequenceStart, TEXT(" [ "));
		Grammar.ReplaceInline(SequenceEnd, TEXT(" ] "));
		Grammar.ReplaceInline(PriorityStart, TEXT(" < "));
		Grammar.ReplaceInline(PriorityEnd, TEXT(" > "));
		Grammar.ReplaceInline(StochasticStart, TEXT(" { "));
		Grammar.ReplaceInline(StochasticEnd, TEXT(" } "));
		Grammar.ReplaceInline(InfiniteRepetitionStr, TEXT(" * "));
		Grammar.ReplaceInline(AtLeastOneRepetitionStr, TEXT(" + "));

		TArray<FStringView> Tokens;
		UE::String::ParseTokens(Grammar, TEXT(" "), Tokens, UE::String::EParseTokensOptions::SkipEmpty);

		PCGGrammar::FModuleDescriptor* CurrentModule = &Result.Root;
		PCGGrammar::FModuleDescriptor* LastModule = nullptr;
		bool bParseOk = true;
		bool bHasWeightDelimiter = false;

		auto CreateSubmodule = [&CurrentModule, &LastModule](EModuleType Type)
		{
			PCGGrammar::FModuleDescriptor& Submodule = CurrentModule->Submodules.Emplace_GetRef(Type);
			Submodule.Parent = CurrentModule;
			CurrentModule = &Submodule;
			LastModule = nullptr;
		};

		auto CloseSubmodule = [&CurrentModule, &LastModule, &Result, &bParseOk](EModuleType Type)
		{
			if(CurrentModule && CurrentModule->Type == Type)
			{
				LastModule = CurrentModule;
				CurrentModule = CurrentModule->Parent;
			}
			else
			{
				Result.AddLog(LOCTEXT("InvalidCloseSubmodule", "Mismatched module end."), FPCGGrammarResult::ELogType::Error);
				bParseOk = false;
			}
		};

		for (const FStringView& Token : Tokens)
		{
			if (Token == SequenceStart)
			{
				CreateSubmodule(EModuleType::Sequence);
			}
			else if (Token == PriorityStart)
			{
				CreateSubmodule(EModuleType::Priority);
			}
			else if (Token == StochasticStart)
			{
				CreateSubmodule(EModuleType::Stochastic);
			}
			else if (Token == SequenceEnd)
			{
				CloseSubmodule(EModuleType::Sequence);
			}
			else if (Token == PriorityEnd)
			{
				CloseSubmodule(EModuleType::Priority);
			}
			else if (Token == StochasticEnd)
			{
				CloseSubmodule(EModuleType::Stochastic);
			}
			else if(Token == Delimiter)
			{
				LastModule = nullptr;
			}
			else if (Token == WeightDelimiter)
			{
				// TODO: verify module supports weights, add warning if so
				if(!LastModule || bHasWeightDelimiter)
				{
					Result.AddLog(LOCTEXT("InvalidWeightDelimiter", "Misplaced or mismatched weight (':') delimiter found."), FPCGGrammarResult::ELogType::Warning);
				}
				
				if (LastModule)
				{
					bHasWeightDelimiter = true;
				}
			}
			else
			{
				// Some Three possibilities here:
				// A literal (last module is null)
				if (LastModule == nullptr)
				{
					// Technically, we could assume this is part of the same token then (so we could have spaces in the tokens, but it's not clean)
					PCGGrammar::FModuleDescriptor& Literal = CurrentModule->Submodules.Emplace_GetRef(FName(Token));
					LastModule = &Literal;
				}
				// Otherwise : The repetition count or the weight
				else if(bHasWeightDelimiter)
				{
					FString TokenString(Token);
					if (TokenString.IsNumeric())
					{
						LastModule->Weight = FCString::Atoi(*TokenString);
					}
					else
					{
						FText WarningMessage = FText::Format(LOCTEXT("InvalidWeightValue", "Invalid weight value: {0}, will default to 1."), FText::FromStringView(Token));
						Result.AddLog(std::move(WarningMessage), FPCGGrammarResult::ELogType::Warning);
					}

					// Consumed the weight specifier
					bHasWeightDelimiter = false;
				}
				else
				{
					FString TokenString(Token);
					if (Token == InfiniteRepetitionStr)
					{
						LastModule->Repetitions = PCGGrammar::InfiniteRepetition;
					}
					else if (Token == AtLeastOneRepetitionStr)
					{
						LastModule->Repetitions = PCGGrammar::AtLeastOneRepetition;
					}
					else if (TokenString.IsNumeric())
					{
						LastModule->Repetitions = FCString::Atoi(*TokenString);
					}
					else
					{
						FText WarningMessage = FText::Format(LOCTEXT("InvalidRepetitionValue", "Invalid repetition value: {0}, will default to 1."), FText::FromStringView(Token));
						Result.AddLog(std::move(WarningMessage), FPCGGrammarResult::ELogType::Warning);
					}
				}
			}

			if (!bParseOk)
			{
				break;
			}
		}

		// Finally, validate that we're back on the root, otherwise we have an mismatched module (e.g. missing its end)
		if (CurrentModule != &Result.Root)
		{
			Result.AddLog(LOCTEXT("UnclosedSubmodule", "Missing module end."), FPCGGrammarResult::ELogType::Error);
			bParseOk = false;
		}

		return bParseOk;
	}

	PCG_API FPCGGrammarResult Parse(const FString& Grammar)
	{
		FPCGGrammarResult Result;

		if (Grammar.IsEmpty())
		{
			Result.AddLog(LOCTEXT("EmptyGrammar", "Grammar is empty."));
			return Result;
		}

		Result.bSuccess = BuildModules(Grammar, Result);

		return Result;
	}
}

#undef LOCTEXT_NAMESPACE