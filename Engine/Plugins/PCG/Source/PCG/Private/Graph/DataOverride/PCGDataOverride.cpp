// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/DataOverride/PCGDataOverride.h"

#include "Graph/DataOverride/PCGDataOverrideHelpers.h"

FPCGDeltaKey::FPCGDeltaKey(const FXxHash64 InHash, const FName InDeltaLabel)
#if WITH_EDITOR
	: DeltaLabel(InDeltaLabel)
#endif // WITH_EDITOR
{
	if (InDeltaLabel != NAME_None)
	{
		FXxHash64Builder Builder;
		const uint64 RawHash = InHash.Hash;
		Builder.Update(&RawHash, sizeof(RawHash));
		const FString LabelString = InDeltaLabel.ToString();
		Builder.Update(*LabelString, LabelString.Len() * sizeof(TCHAR));
		Hash = Builder.Finalize().Hash;
	}
	else
	{
		Hash = InHash.Hash;
	}
}

TInstancedStruct<FPCGDeltaBase>* FPCGDeltaCollection::Find(const FPCGDeltaKey& InKey)
{
	return Deltas.Find(InKey);
}

const TInstancedStruct<FPCGDeltaBase>* FPCGDeltaCollection::Find(const FPCGDeltaKey& InKey) const
{
	return Deltas.Find(InKey);
}

bool FPCGDeltaCollection::Contains(const FPCGDeltaKey& InKey) const
{
	return Find(InKey) != nullptr;
}

#if WITH_EDITOR
void FPCGDeltaCollection::Add(const FPCGDeltaKey& InKey, TInstancedStruct<FPCGDeltaBase>&& InDelta)
{
	(void)Add_GetRef(InKey, MoveTemp(InDelta));
}

TInstancedStruct<FPCGDeltaBase>& FPCGDeltaCollection::Add_GetRef(const FPCGDeltaKey& InKey, TInstancedStruct<FPCGDeltaBase>&& InDelta)
{
	TInstancedStruct<FPCGDeltaBase>& Struct = Deltas.Add(InKey, MoveTemp(InDelta));
	if (FPCGDeltaBase* Delta = Struct.GetMutablePtr<FPCGDeltaBase>())
	{
		Delta->ComputedKey = InKey;
	}

	return Struct;
}

bool FPCGDeltaCollection::Remove(const FPCGDeltaKey& InKey)
{
	return Deltas.Remove(InKey) > 0;
}

void FPCGDeltaCollection::Empty()
{
	Deltas.Empty();
}
#endif // WITH_EDITOR

int32 FPCGDeltaCollection::Num() const
{
	return Deltas.Num();
}

bool FPCGDeltaCollection::IsEmpty() const
{
	return Deltas.IsEmpty();
}

bool FPCGDeltaCollection::ForEachDelta(TFunctionRef<bool(const FPCGDeltaKey&, TInstancedStruct<FPCGDeltaBase>&)> Func)
{
	for (TPair<FPCGDeltaKey, TInstancedStruct<FPCGDeltaBase>>& Pair : Deltas)
	{
		if (!Func(Pair.Key, Pair.Value))
		{
			return false;
		}
	}
	return true;
}

bool FPCGDeltaCollection::ForEachDelta(TFunctionRef<bool(const FPCGDeltaKey&, const TInstancedStruct<FPCGDeltaBase>&)> Func) const
{
	for (const TPair<FPCGDeltaKey, TInstancedStruct<FPCGDeltaBase>>& Pair : Deltas)
	{
		if (!Func(Pair.Key, Pair.Value))
		{
			return false;
		}
	}
	return true;
}

FName FPCGDeltaBase::GetDeltaName() const
{
	return NAME_None;
}

PCGIndexing::FPCGIndexCollection FPCGDeltaBase::FilterCandidates(const UPCGData* InData) const
{
	return PCGIndexing::FPCGIndexCollection::Invalid();
}

int32 FPCGDeltaBase::Resolve(const UPCGData* InData, const PCGIndexing::FPCGIndexCollection& FilteredCandidates, const FPCGDeltaSettings& DeltaSettings) const
{
	return INDEX_NONE;
}

bool FPCGDeltaBase::Apply(UPCGData* InData, const int32 ResolvedIndex) const
{
	return false;
}

FPCGDeltaKey FPCGDeltaBase::ComputeKey(const FPCGDeltaSettings& DeltaSettings, const FTransform& InTransform, const FBox& InWorldBounds)
{
	using namespace PCG::DataOverride::Keys;
	switch (DeltaSettings.KeyTarget)
	{
	case EPCGSpatialKeyTarget::Position:
		return FPCGLocationDeltaKey(InTransform.GetLocation(), DeltaSettings.SpatialTolerance);
	case EPCGSpatialKeyTarget::AABB:
		return FPCGAABBDeltaKey(InWorldBounds, DeltaSettings.SpatialTolerance);
	case EPCGSpatialKeyTarget::Transform:
		return FPCGTransformDeltaKey(InTransform, DeltaSettings.SpatialTolerance);
	default:
		return {};
	}
}
