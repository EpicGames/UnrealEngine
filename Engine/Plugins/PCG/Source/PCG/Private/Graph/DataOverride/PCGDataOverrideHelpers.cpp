// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/DataOverride/PCGDataOverrideHelpers.h"

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "Data/PCGPointArrayData.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Graph/PCGStackContext.h"
#include "Subsystems/PCGSubsystem.h"

#include "Engine/World.h"

namespace PCG::DataOverride
{
	namespace Helpers
	{
		FXxHash64 ComputeDataHash(const FPCGStack& InStack)
		{
			FXxHash64Builder HashBuilder;

			for (const FPCGStackFrame& Frame : InStack.GetStackFrames())
			{
				if (Frame.IsLoopIndexFrame())
				{
					const int32 LoopIndex = Frame.LoopIndex;
					HashBuilder.Update(&LoopIndex, sizeof(LoopIndex));
				}
				else
				{
					// Skip the execution source, since it is not relevant to the local data.
					const UObject* FrameObject = Frame.GetObject_NoGuard();
					if (!FrameObject || FrameObject->Implements<UPCGGraphExecutionSource>())
					{
						continue;
					}

					const FString Path = Frame.Object.ToString();
					HashBuilder.Update(*Path, Path.Len() * sizeof(TCHAR));
				}
			}

			const FXxHash64 Result = HashBuilder.Finalize();
			return Result;
		}

		FXxHash64 ComputeNodeOverrideKey(const FPCGStack& InExecutionStack, const UPCGNode* InNode)
		{
			FPCGStack NodeStack = InExecutionStack;
			NodeStack.PushFrame(InNode);
			return ComputeDataHash(NodeStack);
		}

		FXxHash64 ComputePinOverrideKey(const FPCGStack& InExecutionStack, const UPCGNode* InNode, const FName PinLabel)
		{
			return ComputePinOverrideKey(ComputeNodeOverrideKey(InExecutionStack, InNode), PinLabel);
		}

		FXxHash64 ComputePinOverrideKey(const FXxHash64 NodeKey, const FName PinLabel)
		{
			if (PinLabel == NAME_None)
			{
				return NodeKey;
			}

			FXxHash64Builder HashBuilder;
			HashBuilder.Update(&NodeKey.Hash, sizeof(NodeKey.Hash));

			const FString PinString = PinLabel.ToString();
			HashBuilder.Update(*PinString, PinString.Len() * sizeof(TCHAR));

			return HashBuilder.Finalize();
		}

		const TArray<TObjectPtr<UPCGPin>>* GetOverridePins(const UPCGNode* InNode)
		{
			check(InNode);
			const UPCGSettings* Settings = InNode->GetSettings();
			const FPCGElementPtr Element = Settings ? Settings->GetElement() : nullptr;
			if (!Element)
			{
				return nullptr;
			}

			const bool bUsesInputPins = Element->GetDataOverridePhase() == EPCGDataOverridePhase::PrepareData;
			return bUsesInputPins ? &InNode->GetInputPins() : &InNode->GetOutputPins();
		}

#if WITH_EDITOR
		TArray<FPCGSourceDataStorageKey> CollectNodeStorageKeys(const UPCGNode* InNode, const IPCGGraphExecutionSource* InExecutionSource)
		{
			TArray<FPCGSourceDataStorageKey> Keys;
			if (!InNode || !InExecutionSource)
			{
				return Keys;
			}

			const TArray<TObjectPtr<UPCGPin>>* OverridePins = GetOverridePins(InNode);
			if (!OverridePins)
			{
				return Keys;
			}

			const FPCGGraphExecutionInspection& Inspection = InExecutionSource->GetExecutionState().GetInspection();

			const TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData> StackSet = Inspection.GetExecutedNodeStacks(InNode);
			if (StackSet.IsEmpty())
			{
				return Keys;
			}

			for (const FPCGGraphExecutionInspection::FNodeExecutedNotificationData& StackData : StackSet)
			{
				const FXxHash64 NodeKey = ComputeNodeOverrideKey(StackData.Stack, InNode);

				for (const UPCGPin* Pin : *OverridePins)
				{
					if (!Pin)
					{
						continue;
					}

					Keys.Emplace(Constants::DefaultOverrideLabel, ComputePinOverrideKey(NodeKey, Pin->Properties.Label).Hash);
				}

				Keys.Emplace(Constants::DefaultOverrideLabel, NodeKey.Hash);
			}

			return Keys;
		}
#endif

		const FPCGSourceDataContainer* GetSourceDataContainer(const IPCGGraphExecutionSource* InExecutionSource)
		{
			return InExecutionSource ? InExecutionSource->GetExecutionState().GetSourceDataContainer() : nullptr;
		}

		FPCGSourceDataContainer* GetMutableSourceDataContainer(IPCGGraphExecutionSource* InExecutionSource)
		{
			return InExecutionSource ? InExecutionSource->GetExecutionState().GetSourceDataContainer() : nullptr;
		}

		void RefreshExecutionSource(IPCGGraphExecutionSource* InExecutionSource)
		{
			if (!InExecutionSource)
			{
				return;
			}

			IPCGGraphExecutionState& State = InExecutionSource->GetExecutionState();
			if (State.IsManagedByRuntimeGenSystem())
			{
				if (UPCGSubsystem* Subsystem = State.GetWorld() ? State.GetWorld()->GetSubsystem<UPCGSubsystem>() : nullptr)
				{
					Subsystem->RefreshRuntimeGenExecutionSource(InExecutionSource);
				}
			}
			else
			{
				IPCGGraphExecutionState::FGenerateParams Params;
				Params.bEvenIfAlreadyGenerated = true;
				State.Generate(Params);
			}
		}
	} // namespace Helpers

	namespace Keys
	{
		FXxHash64 ComputePositionHash(const FVector& InLocation, const double Tolerance)
		{
			// Note: this assumes there's no padding in FVector that could contain uncompared data.
			// 0 Tolerance is an exact spatial hash
			if (FMath::IsNearlyZero(FMath::Max(Tolerance, 0.0)))
			{
				return FXxHash64::HashBuffer(&InLocation, sizeof(InLocation));
			}

			// IMPLEMENTATION NOTE: A significant change in location or bounds will orphan this override.
			const int64 GridX = FMath::RoundToInt(InLocation.X / Tolerance);
			const int64 GridY = FMath::RoundToInt(InLocation.Y / Tolerance);
			const int64 GridZ = FMath::RoundToInt(InLocation.Z / Tolerance);

			const int64 Grid[] = {GridX, GridY, GridZ};
			return FXxHash64::HashBuffer(&Grid, sizeof(Grid));
		}

		FXxHash64 ComputeTransformHash(const FTransform& InTransform, const double Tolerance)
		{
			if (FMath::IsNearlyZero(FMath::Max(Tolerance, 0.0)))
			{
				// FTransform uses vectorized register storage (double[4] per member)
				// Translation and Scale3D use uninitialized padding so decomposition is needed to hash only meaningful
				// bytes. This matches the pattern in GetTypeHash(TTransform).
				const FQuat Rotation = InTransform.GetRotation();
				const FVector Translation = InTransform.GetTranslation();
				const FVector Scale = InTransform.GetScale3D();

				FXxHash64Builder HashBuilder;
				HashBuilder.Update(&Rotation, sizeof(Rotation));
				HashBuilder.Update(&Translation, sizeof(Translation));
				HashBuilder.Update(&Scale, sizeof(Scale));

				return HashBuilder.Finalize();
			}

			const FVector Location = InTransform.GetLocation();
			const int64 GridX = FMath::RoundToInt(Location.X / Tolerance);
			const int64 GridY = FMath::RoundToInt(Location.Y / Tolerance);
			const int64 GridZ = FMath::RoundToInt(Location.Z / Tolerance);

			FXxHash64Builder HashBuilder;
			const int64 Grid[] = {GridX, GridY, GridZ};
			HashBuilder.Update(Grid, sizeof(Grid));

			// @todo_pcg: Scale doesn't follow the same spatial tolerance as location. Might experiment with a
			// separate tolerance or leave it directly hashed.
			const FQuat Rotation = InTransform.GetRotation();
			HashBuilder.Update(&Rotation, sizeof(Rotation));
			const FVector Scale = InTransform.GetScale3D();
			HashBuilder.Update(&Scale, sizeof(Scale));

			return HashBuilder.Finalize();
		}

		FXxHash64 ComputeAABBHash(const FBox& InBounds, const double Tolerance)
		{
			// Note: this assumes there's no padding in FBox that could contain uncompared data.
			if (FMath::IsNearlyZero(FMath::Max(Tolerance, 0.0)))
			{
				return FXxHash64::HashBuffer(&InBounds, sizeof(InBounds));
			}

			const int64 Grid[] = {
				FMath::RoundToInt(InBounds.Min.X / Tolerance),
				FMath::RoundToInt(InBounds.Min.Y / Tolerance),
				FMath::RoundToInt(InBounds.Min.Z / Tolerance),
				FMath::RoundToInt(InBounds.Max.X / Tolerance),
				FMath::RoundToInt(InBounds.Max.Y / Tolerance),
				FMath::RoundToInt(InBounds.Max.Z / Tolerance)
			};

			return FXxHash64::HashBuffer(&Grid, sizeof(Grid));
		}

		FXxHash64 ComputeCompositeHash(const FTransform& InTransform, const double Tolerance, const bool bIncludePosition, const bool bIncludeRotation, const bool bIncludeScale)
		{
			// Fall back to the full transform if no components are selected to avoid degenerate hashes.
			if (!bIncludePosition && !bIncludeRotation && !bIncludeScale)
			{
				return ComputeTransformHash(InTransform, Tolerance);
			}

			// Fast paths for common cases
			if (bIncludePosition && bIncludeRotation && bIncludeScale)
			{
				return ComputeTransformHash(InTransform, Tolerance);
			}

			if (bIncludePosition && !bIncludeRotation && !bIncludeScale)
			{
				return ComputePositionHash(InTransform.GetLocation(), Tolerance);
			}

			// Composite: hash selected components individually
			FXxHash64Builder HashBuilder;

			if (bIncludePosition)
			{
				const FXxHash64 PositionHash = ComputePositionHash(InTransform.GetLocation(), Tolerance);
				HashBuilder.Update(&PositionHash, sizeof(PositionHash));
			}

			if (bIncludeRotation)
			{
				const FQuat Rotation = InTransform.GetRotation();
				HashBuilder.Update(&Rotation, sizeof(Rotation));
			}

			if (bIncludeScale)
			{
				const FVector Scale = InTransform.GetScale3D();
				HashBuilder.Update(&Scale, sizeof(Scale));
			}

			return HashBuilder.Finalize();
		}
	} // namespace Keys

	bool HasNodeLevelDataOverrides(FPCGContext* Context)
	{
		const FPCGSourceDataContainer* DataContainer = Context ? Context->GetSourceDataContainer() : nullptr;
		const FPCGStack* Stack = DataContainer ? Context->GetStack() : nullptr;
		if (!DataContainer || !Stack || DataContainer->IsEmpty())
		{
			return false;
		}

		const FXxHash64 NodeKey = Helpers::ComputeNodeOverrideKey(*Stack, Context->Node);
		const FPCGSourceDataStorageKey NodeStorageKey(Constants::DefaultOverrideLabel, NodeKey.Hash);
		return DataContainer->Get<FPCGDeltaCollection>(NodeStorageKey).IsValid();
	}

	bool HasPinLevelDataOverrides(FPCGContext* Context, const FPCGTaggedData& InTaggedData)
	{
		const FPCGSourceDataContainer* DataContainer = Context ? Context->GetSourceDataContainer() : nullptr;
		const FPCGStack* Stack = DataContainer ? Context->GetStack() : nullptr;
		if (!DataContainer || !Stack || DataContainer->IsEmpty())
		{
			return false;
		}

		const FXxHash64 PinKey = Helpers::ComputePinOverrideKey(*Stack, Context->Node, InTaggedData.Pin);
		const FPCGSourceDataStorageKey PinStorageKey(Constants::DefaultOverrideLabel, PinKey.Hash);
		return DataContainer->Get<FPCGDeltaCollection>(PinStorageKey).IsValid();
	}

	bool HasAnyDataOverrides(FPCGContext* Context, const TArrayView<const FPCGTaggedData> InTaggedData)
	{
		if (HasNodeLevelDataOverrides(Context))
		{
			return true;
		}

		for (const FPCGTaggedData& TaggedData : InTaggedData)
		{
			if (HasPinLevelDataOverrides(Context, TaggedData))
			{
				return true;
			}
		}

		return false;
	}

	// @todo_pcg: Still requires collision handling. Will be done in future pass.
	void ApplyDataOverrides(FPCGContext* Context, const TArrayView<FPCGTaggedData> OutTaggedData)
	{
		const FPCGSourceDataContainer* DataContainer = Context ? Context->GetSourceDataContainer() : nullptr;
		if (!DataContainer)
		{
			return;
		}

		const FPCGStack* Stack = Context->GetStack();
		if (!Stack)
		{
			return;
		}

		const FXxHash64 NodeKey = Helpers::ComputeNodeOverrideKey(*Stack, Context->Node);

		// Accumulate across all TaggedData first, then submit once per storage key.
		TMap<FPCGSourceDataStorageKey, TArray<FPCGDeltaKey>> ResolvedDeltas;

		for (FPCGTaggedData& TaggedData : OutTaggedData)
		{
			// @todo_pcg: This should be part of a registry. For now, default to points, but should be element
			// We have data ownership at either phase and thus const_cast is okay.
			if (UPCGData* PCGData = const_cast<UPCGData*>(Cast<UPCGData>(TaggedData.Data)))
			{
				// Track the storage key which resolved to a delta collection to pair delta resolution.
				FPCGSourceDataStorageKey StorageKey(Constants::DefaultOverrideLabel, Helpers::ComputePinOverrideKey(NodeKey, TaggedData.Pin).Hash);

				// Check to see if the pin itself has overrides
				FConstSharedStruct SharedStruct = DataContainer->Get<FPCGDeltaCollection>(StorageKey);
				const FPCGDeltaCollection* DeltaCollection = SharedStruct.IsValid() ? SharedStruct.GetPtr<const FPCGDeltaCollection>() : nullptr;

				// If not, check the node level
				if (!DeltaCollection)
				{
					StorageKey = FPCGSourceDataStorageKey(Constants::DefaultOverrideLabel, NodeKey.Hash);
					SharedStruct = DataContainer->Get<FPCGDeltaCollection>(StorageKey);
					DeltaCollection = SharedStruct.IsValid() ? SharedStruct.GetPtr<const FPCGDeltaCollection>() : nullptr;
				}

				if (DeltaCollection && !DeltaCollection->IsEmpty())
				{
					// Resolve the deltas, but only for Editor inspection. Otherwise no-op.
					auto MarkResolved = [&ResolvedDeltas, &StorageKey](const FPCGDeltaKey& InKey)
					{
#if WITH_EDITOR
						ResolvedDeltas.FindOrAdd(StorageKey).AddUnique(InKey);
#endif
					};

					// @todo_pcg: Sort deltas (edits->deletions->insertions) to avoid index instability.
					// @todo_pcg: As a future optimization pass, aggregate candidates to avoid redundant key computation.
					DeltaCollection->ForEachDelta([DeltaCollection, PCGData, &TaggedData, &MarkResolved]
					(const FPCGDeltaKey& DeltaKey, const TInstancedStruct<FPCGDeltaBase>& DeltaStruct)
					{
						if (const FPCGDeltaBase* Delta = DeltaStruct.GetPtr<FPCGDeltaBase>())
						{
							const PCGIndexing::FPCGIndexCollection Candidates = Delta->FilterCandidates(PCGData);

							if (!Candidates.IsValid())
							{
								// Delta does not use candidate filtering.
								Delta->Apply(PCGData, INDEX_NONE);
								MarkResolved(DeltaKey);
							}
							else if (!Candidates.IsEmpty())
							{
								if (const int32 ResolvedIndex = Delta->Resolve(PCGData, Candidates, DeltaCollection->Settings); ResolvedIndex != INDEX_NONE)
								{
									// Full data override.
									if (Delta->UsesReplacementData())
									{
										if (UPCGData* ReplacementData = Delta->GetReplacementData())
										{
											TaggedData.Data = ReplacementData;
											MarkResolved(DeltaKey);
											// Stop iterating if replacement occurred.
											return false;
										}
										UE_LOGF(LogPCG, Error, "Delta marked as replacing but returned null replacement data");
										// @todo_pcg: Is this a resolution or should it be considered orphan?
									}
									else // Partial override.
									{
										Delta->Apply(PCGData, ResolvedIndex);
										MarkResolved(DeltaKey);
									}
								}
							}
						}

						return true;
					});
				}
			}
		}

#if WITH_EDITOR
		// Complete the data override inspection.
		if (Context->ExecutionSource.IsValid())
		{
			FPCGGraphExecutionInspection& Inspection = Context->ExecutionSource->GetExecutionState().GetInspection();
			for (TPair<FPCGSourceDataStorageKey, TArray<FPCGDeltaKey>>& Pair : ResolvedDeltas)
			{
				Inspection.NotifyDeltasResolved(Pair.Key, MoveTemp(Pair.Value));
			}
		}
#endif
	}
} // namespace PCG::DataOverride
