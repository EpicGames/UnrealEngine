// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/DataOverride/PCGDataOverridePoints.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGPointArrayData.h"
#include "Graph/DataOverride/PCGDataOverrideHelpers.h"

PCGIndexing::FPCGIndexCollection FPCGPointDeltaBase::FilterCandidates(const UPCGData* InData) const
{
	if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(InData))
	{
		PCGIndexing::FPCGIndexCollection Collection(PointData->GetNumPoints());

		TArray<int32> CandidateIndices;
		const PCGPointOctree::FPointOctree& Octree = PointData->GetPointOctree();

		Octree.FindElementsWithBoundsTest(
			Bounds,
			[&CandidateIndices](const PCGPointOctree::FPointRef& Ref)
			{
				CandidateIndices.Add(Ref.Index);
			});

		Collection += CandidateIndices;

		return Collection;
	}

	return PCGIndexing::FPCGIndexCollection::Invalid();
}

int32 FPCGPointDeltaBase::Resolve(const UPCGData* InData, const PCGIndexing::FPCGIndexCollection& FilteredCandidates, const FPCGDeltaSettings& DeltaSettings) const
{
	using namespace PCG::DataOverride;

	int32 ResolvedIndex = INDEX_NONE;
	const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(InData);
	if (!PointData)
	{
		return ResolvedIndex;
	}

	FPCGDeltaKey DeltaKey = ComputeKey(DeltaSettings, OriginalTransform, Bounds);

	FilteredCandidates.ForEach([&ResolvedIndex, &DeltaKey, &DeltaSettings, PointData](const PCGIndexing::FPCGIndexRange& IndexRange)
	{
		for (int32 Index = IndexRange.StartIndex; Index < IndexRange.EndIndex; ++Index)
		{
			FTransform Transform = PointData->GetTransform(Index);
			FBox LocalBounds = PointData->GetLocalBounds(Index);
			FBox WorldBounds = LocalBounds.TransformBy(Transform);

			if (FPCGDeltaKey CandidateKey = ComputeKey(DeltaSettings, Transform, WorldBounds); DeltaKey == CandidateKey)
			{
				ResolvedIndex = Index;
				return true;
			}
		}

		return false;
	});

	return ResolvedIndex;
}

FName FPCGPointTransformDelta::GetDeltaName() const
{
	return GetDeltaNameStatic();
}

bool FPCGPointTransformDelta::Apply(UPCGData* InData, const int32 ResolvedIndex) const
{
	if (UPCGPointArrayData* PointData = Cast<UPCGPointArrayData>(InData))
	{
		if (PointData->IsEmpty())
		{
			return false;
		}

		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();

		if (TransformRange.IsValidIndex(ResolvedIndex))
		{
			if (bOverridePosition)
			{
				TransformRange[ResolvedIndex].SetLocation(TransformOverride.GetLocation());
			}

			if (bOverrideRotation)
			{
				TransformRange[ResolvedIndex].SetRotation(TransformOverride.GetRotation());
			}

			if (bOverrideScale)
			{
				TransformRange[ResolvedIndex].SetScale3D(TransformOverride.GetScale3D());
			}

			return true;
		}
	}
	// @todo_pcg: Add the path to evaluate the other point data

	return false;
}

FName FPCGPointTransformOffsetDelta::GetDeltaName() const
{
	return GetDeltaNameStatic();
}

bool FPCGPointTransformOffsetDelta::Apply(UPCGData* InData, int32 ResolvedIndex) const
{
	if (UPCGPointArrayData* PointData = Cast<UPCGPointArrayData>(InData))
	{
		if (PointData->IsEmpty())
		{
			return false;
		}

		const TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		if (TransformRange.IsValidIndex(ResolvedIndex))
		{
			if (bOffsetPosition)
			{
				TransformRange[ResolvedIndex].AddToTranslation(TransformOffset.GetLocation());
			}

			if (bOffsetRotation)
			{
				TransformRange[ResolvedIndex].SetRotation(TransformOffset.GetRotation() * TransformRange[ResolvedIndex].GetRotation());
			}

			if (bOffsetScale)
			{
				const FVector CurrentScale = TransformRange[ResolvedIndex].GetScale3D();
				TransformRange[ResolvedIndex].SetScale3D(CurrentScale * TransformOffset.GetScale3D());
			}

			return true;
		}
	}
	// @todo_pcg: Add the path to evaluate the other point data

	return false;
}

FName FPCGPointDeletionDelta::GetDeltaName() const
{
	return GetDeltaNameStatic();
}

bool FPCGPointDeletionDelta::Apply(UPCGData* InData, const int32 ResolvedIndex) const
{
	if (UPCGPointArrayData* PointData = Cast<UPCGPointArrayData>(InData))
	{
		if (PointData->IsEmpty() || ResolvedIndex == INDEX_NONE)
		{
			return false;
		}

		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		if (TransformRange.IsValidIndex(ResolvedIndex))
		{
			// @todo_pcg: To be discussed and fixed in second pass. For now, we'll set the scale to 0,0,0. We should
			// delete the point, but deletion of a point introduces index instability, so it should happen last or it may thrash the octree.
			TransformRange[ResolvedIndex].SetScale3D(FVector::ZeroVector);
			return true;
		}
	}

	return false;
}

FName FPCGPointInsertionDelta::GetDeltaName() const
{
	return GetDeltaNameStatic();
}

PCGIndexing::FPCGIndexCollection FPCGPointInsertionDelta::FilterCandidates(const UPCGData* InData) const
{
	// Filtering not needed. Points are simply inserted.
	return PCGIndexing::FPCGIndexCollection::Invalid();
}

bool FPCGPointInsertionDelta::Apply(UPCGData* InData, const int32 ResolvedIndex) const
{
	if (UPCGPointArrayData* PointData = Cast<UPCGPointArrayData>(InData))
	{
		if (InsertedPoints.IsEmpty())
		{
			return false;
		}

		const int32 OriginalPointCount = PointData->GetNumPoints();
		if (!ensure(std::numeric_limits<int32>::max() - OriginalPointCount >= InsertedPoints.Num()))
		{
			// Too many point insertions
			return false;
		}

		PointData->SetNumPoints(OriginalPointCount + InsertedPoints.Num());

		FPCGPointValueRanges PointRanges(PointData, /*bAllocate=*/true);

		for (int32 I = 0; I < InsertedPoints.Num(); ++I)
		{
			PointRanges.SetFromPoint(OriginalPointCount + I, InsertedPoints[I]);
		}

		return true;
	}
	// @todo_pcg: Add the path to evaluate the other point data

	return false;
}
