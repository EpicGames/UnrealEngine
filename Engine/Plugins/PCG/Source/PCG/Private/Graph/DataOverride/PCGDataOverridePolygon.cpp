// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/DataOverride/PCGDataOverridePolygon.h"

#include "Graph/DataOverride/PCGDataOverrideHelpers.h"


namespace PCG::DataOverride::Polygon::Constants
{
	inline constexpr FLazyName PolygonOverrideDeltaName = "PolygonOverride";
}

FName FPCGPolygon2DDataDelta::GetDeltaNameStatic()
{
	return PCG::DataOverride::Polygon::Constants::PolygonOverrideDeltaName;
}

PCGIndexing::FPCGIndexCollection FPCGPolygon2DDataDelta::FilterCandidates(const UPCGData* InData) const
{
	if (!PolygonOverride)
	{
		return PCGIndexing::FPCGIndexCollection::Invalid();
	}

	if (const UPCGPolygon2DData* PolygonData = Cast<UPCGPolygon2DData>(InData))
	{
		PCGIndexing::FPCGIndexCollection IndexCollection(PolygonData->GetNumVertices());
		IndexCollection.AddRange(0, PolygonData->GetNumVertices());
		return IndexCollection;
	}

	return PCGIndexing::FPCGIndexCollection::Invalid();
}

int32 FPCGPolygon2DDataDelta::Resolve(const UPCGData* InData, const PCGIndexing::FPCGIndexCollection& FilteredCandidates,
	const FPCGDeltaSettings& DeltaSettings) const
{
	if (!PolygonOverride || FilteredCandidates.IsEmpty() || (DeltaSettings.KeyTarget == EPCGSpatialKeyTarget::AABB))
	{
		return INDEX_NONE;
	}

	if (const UPCGPolygon2DData* PolygonData = Cast<UPCGPolygon2DData>(InData); PolygonData && PolygonData->GetNumVertices() > 0)
	{
		const FPCGDeltaKey InputKey = ComputeKey(DeltaSettings, PolygonData->GetTransformAtDistance(0, 0.0, true), FBox{});
	
		const FPCGDeltaKey OverrideKey = ComputeKey(DeltaSettings, OriginalStartTransform, FBox{});
	
		// Index 0 means do replace the polygon data.
		return InputKey == OverrideKey ? 0 : INDEX_NONE;
	}

	return INDEX_NONE;
}

FName FPCGPolygon2DDataDelta::GetDeltaName() const
{
	return GetDeltaNameStatic();
}

bool FPCGPolygon2DDataDelta::Apply(UPCGData* InData, int32 ResolvedIndex) const
{
	// Should not call apply on this delta, it is a replacement delta.
	checkNoEntry();
	return false;
}

UPCGData* FPCGPolygon2DDataDelta::GetReplacementData() const
{
	return PolygonOverride;
}
