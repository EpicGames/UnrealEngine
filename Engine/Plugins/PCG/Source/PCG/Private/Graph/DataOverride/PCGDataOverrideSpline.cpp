// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/DataOverride/PCGDataOverrideSpline.h"

#include "Data/PCGSplineData.h"
#include "Graph/DataOverride/PCGDataOverrideHelpers.h"


namespace PCG::DataOverride::Spline::Constants
{
	inline constexpr FLazyName SplineOverrideDeltaName = "SplineOverride";
}

FName FPCGSplineDataDelta::GetDeltaNameStatic()
{
	return PCG::DataOverride::Spline::Constants::SplineOverrideDeltaName;
}

PCGIndexing::FPCGIndexCollection FPCGSplineDataDelta::FilterCandidates(const UPCGData* InData) const
{
	if (!SplineOverride)
	{
		return PCGIndexing::FPCGIndexCollection::Invalid();
	}

	if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(InData))
	{
		PCGIndexing::FPCGIndexCollection IndexCollection(SplineData->GetNumVertices());
		IndexCollection.AddRange(0, SplineData->GetNumVertices());
		return IndexCollection;
	}

	return PCGIndexing::FPCGIndexCollection::Invalid();
}

int32 FPCGSplineDataDelta::Resolve(const UPCGData* InData, const PCGIndexing::FPCGIndexCollection& FilteredCandidates,
	const FPCGDeltaSettings& DeltaSettings) const
{
	if (!SplineOverride || FilteredCandidates.IsEmpty() || (DeltaSettings.KeyTarget == EPCGSpatialKeyTarget::AABB))
	{
		return INDEX_NONE;
	}
	
	if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(InData); SplineData && SplineData->GetNumVertices() > 0)
	{
		FSplinePoint InputSplinePoint = SplineData->GetSplinePoints()[0];
		FTransform InputSplinePointTransform{InputSplinePoint.Rotation, InputSplinePoint.Position, InputSplinePoint.Scale};
		FPCGDeltaKey InputKey = ComputeKey(DeltaSettings, InputSplinePointTransform, FBox{});
		FPCGDeltaKey OverrideKey = ComputeKey(DeltaSettings, OriginalStartTransform, FBox{});
	
		// Index 0 means do replace the whole spline data.
		return InputKey == OverrideKey ? 0 : INDEX_NONE;
	}

	return INDEX_NONE;
}

FName FPCGSplineDataDelta::GetDeltaName() const
{
	return GetDeltaNameStatic();
}

bool FPCGSplineDataDelta::Apply(UPCGData* InData, int32 ResolvedIndex) const
{
	// Should not call apply on this delta, it is a replacement delta.
	checkNoEntry();
	return false;
}

UPCGData* FPCGSplineDataDelta::GetReplacementData() const
{
	return SplineOverride;
}

