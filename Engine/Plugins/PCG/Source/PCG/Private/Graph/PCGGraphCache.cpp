// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/PCGGraphCache.h"

#include "PCGComponent.h"
#include "PCGModule.h"
#include "Subsystems/PCGSubsystem.h"

#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopeRWLock.h"

static TAutoConsoleVariable<bool> CVarCacheEnabledEditor(
	TEXT("pcg.Cache.Editor.Enabled"),
	true,
	TEXT("Enables the cache system in editor worlds."));

static TAutoConsoleVariable<bool> CVarCacheEnabledRuntime(
	TEXT("pcg.Cache.Runtime.Enabled"),
	false,
	TEXT("Enables the cache system in runtime game worlds."));

static TAutoConsoleVariable<bool> CVarCacheDebugging(
	TEXT("pcg.Cache.EnableDebugging"),
	false,
	TEXT("Enable various features for debugging the graph cache system."));

static TAutoConsoleVariable<int32> CVarCacheMemoryBudgetEditorMB(
	TEXT("pcg.Cache.Editor.MemoryBudgetMB"),
	6144,
	TEXT("Memory budget for data in cache in editor worlds (MB)."));

static TAutoConsoleVariable<int32> CVarCacheMemoryBudgetRuntimeMB(
	TEXT("pcg.Cache.Runtime.MemoryBudgetMB"),
	128,
	TEXT("Memory budget for data in cache in game worlds (MB)."));

static TAutoConsoleVariable<float> CVarCacheMemoryCleanupRatio(
	TEXT("pcg.Cache.MemoryCleanupRatio"),
	0.5f,
	TEXT("Target cache size ratio after triggering a cleanup (between 0 and 1.)."));

static TAutoConsoleVariable<bool> CVarCacheMemoryBudgetEnabled(
	TEXT("pcg.Cache.EnableMemoryBudget"),
	true,
	TEXT("Whether memory budget is enforced (items purged from cache to respect pcg.Cache.MemoryBudgetMB."));

static TAutoConsoleVariable<bool> CVarValidateElementToCacheEntryKeys(
	TEXT("pcg.Cache.Debug.ValidateElementToCacheEntryKeys"),
	false,
	TEXT("Validate ElementToCacheEntryKeys acceleration table (debug)."));

static FAutoConsoleCommandWithWorld CResetCacheStatsCommand(
	TEXT("pcg.Cache.ResetStats"),
	TEXT("Resets the current cache stats"),
	FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld)
	{
		if(UPCGSubsystem* PCGSubsystem = UWorld::GetSubsystem<UPCGSubsystem>(InWorld))
		{
			PCGSubsystem->ResetCacheStats();
		}
	}));

static FAutoConsoleCommandWithWorld CLogCacheStatsCommand(
	TEXT("pcg.Cache.LogStats"),
	TEXT("Log the current cache stats"),
	FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld)
	{
		if (UPCGSubsystem* PCGSubsystem = UWorld::GetSubsystem<UPCGSubsystem>(InWorld))
		{
			PCGSubsystem->LogCacheStats();
		}
	}));

// Aliases for old deprecated settings.
static TAutoConsoleVariable<bool> CVarCacheEnabled_DEPRECATED(
	TEXT("pcg.Cache.Enabled"),
	true,
	TEXT("DEPRECATED (5.7): use pcg.Cache.Editor.Enabled and pcg.Cache.Runtime.Enabled instead."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		// Called when user sets cvar from console, or when this cvar is set when launching standalone build. Pushes value to new cvars.
		UE_LOGF(LogPCG, Warning, "pcg.Cache.Enabled is deprecated in 5.7. use pcg.Cache.Editor.Enabled and pcg.Cache.Runtime.Enabled instead.");
		check(InVariable);
		bool bNewValue = true;
		InVariable->GetValue(bNewValue);
		CVarCacheEnabledRuntime->Set(bNewValue, ECVF_SetByCode);
		CVarCacheEnabledEditor->Set(bNewValue, ECVF_SetByCode);
	}));
static TAutoConsoleVariable<int32> CVarCacheMemoryBudgetMB_DEPRECATED(
	TEXT("pcg.Cache.MemoryBudgetMB"),
	6144,
	TEXT("DEPRECATED (5.7): use pcg.Cache.Editor.MemoryBudgetMB and pcg.Cache.Runtime.MemoryBudgetMB instead."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		// Called when user sets cvar from console, or when this cvar is set when launching standalone build. Pushes value to new cvars.
		UE_LOGF(LogPCG, Warning, "pcg.Cache.MemoryBudgetMB is deprecated in 5.7. use pcg.Cache.Editor.MemoryBudgetMB and pcg.Cache.Runtime.MemoryBudgetMB instead.");
		check(InVariable);
		int32 NewValue = 6144;
		InVariable->GetValue(NewValue);
		CVarCacheMemoryBudgetEditorMB->Set(NewValue, ECVF_SetByCode);
		CVarCacheMemoryBudgetRuntimeMB->Set(NewValue, ECVF_SetByCode);
	}));

// Initial max number of entries graph cache
static const int32 GPCGGraphCacheInitialCapacity = 65536;

namespace PCGGraphCacheHelpers
{
	thread_local bool bIsGatheringCacheSize = false;

	TAutoConsoleVariable<bool> CVarEnableTLSGatherCacheSize(
		TEXT("pcg.Cache.EnableTLSGatherCacheSize"),
		true,
		TEXT("Whether we give a hint to UPCGData subclass that we are calling GetResourceSizeEx for cache purposes."));

	TAutoConsoleVariable<int> CVarCacheSizeDetailLevel(
		TEXT("pcg.Cache.CacheSizeDetailLevel"),
		1,
		TEXT("Controls effort put into gather memory size stats for the cache & profiling. 0 = full, 1 = approximate, 2 = light-weight."));
}

bool FPCGGraphCache::IsGatheringCacheSize()
{
	return PCGGraphCacheHelpers::bIsGatheringCacheSize;
}

int FPCGGraphCache::CacheSizeDetailLevel()
{
	// Cache size detail level only applies when in the proper scope, otherwise if it's driven by the system, we'll do a full gathering
	return IsGatheringCacheSize() ? PCGGraphCacheHelpers::CVarCacheSizeDetailLevel.GetValueOnAnyThread() : 0;
}

FPCGGraphCache::FPCGGraphCache(bool bGameWorld)
	: bIsGameWorld(bGameWorld)
{
	const int32 InitialMaxCapacity = IsEnabled() ? GPCGGraphCacheInitialCapacity : 0;
	CacheData.Empty(InitialMaxCapacity);
}

FPCGGraphCache::~FPCGGraphCache()
{
	ClearCache();
}

void FPCGGraphCache::ResetStats()
{
	Stats = FPCGCacheStats();
}

void FPCGGraphCache::LogStats()
{
	PCG::TScopeLock ScopedLock(CacheLock);
	const float HitRate = Stats.GetCount > 0 ? ((float)Stats.HitCount / Stats.GetCount) * 100.f : 0.f;

	UE_LOGF(LogPCG, Log, "Cache Stats : %lld Stores, %lld Hits / %lld Gets (%.2f %%), %lld Grow count / %lld Remove count / %d Entry count / %.2f Memory(MB)", 
		Stats.StoreCount, 
		Stats.HitCount, 
		Stats.GetCount, 
		HitRate, 
		Stats.GrowCount,
		Stats.RemoveCount,
		CacheData.Num(),
		TotalMemoryUsed / 1024.f / 1024.f);

	struct FPerClassStats
	{
		int32 Instances = 0;
		uint64 TotalMemoryUsed = 0;
	};

	TMap<UClass*, FPerClassStats> PerClassStats;
	FPerClassStats UnknownStats;

	for (const auto& [UID, MemoryRecord] : MemoryRecords)
	{
		if (UObject* DataPtr = MemoryRecord.Data.ResolveObjectPtr())
		{
			FPerClassStats& ClassStats = PerClassStats.FindOrAdd(DataPtr->GetClass());
			++ClassStats.Instances;
			ClassStats.TotalMemoryUsed += MemoryRecord.MemoryPerInstance;
		}
		else
		{
			// Unknown object
			++UnknownStats.Instances;
			UnknownStats.TotalMemoryUsed += MemoryRecord.MemoryPerInstance;
		}
	}

	for (const auto& [ClassPtr, ClassStats] : PerClassStats)
	{
		UE_LOGF(LogPCG, Log, "Cache Stats: %ls : %d Instances : %.2f Memory(MB)", *ClassPtr->GetPathName(), ClassStats.Instances, ClassStats.TotalMemoryUsed / 1024.f / 1024.f);
	}

	if (UnknownStats.Instances > 0)
	{
		UE_LOGF(LogPCG, Log, "Cache Stats: <Unknown type> : %d Instances : %.2f Memory(MB)", UnknownStats.Instances, UnknownStats.TotalMemoryUsed / 1024.f / 1024.f);
	}
}

bool FPCGGraphCache::IsEnabled() const
{
	return bIsGameWorld ? CVarCacheEnabledRuntime.GetValueOnAnyThread() : CVarCacheEnabledEditor.GetValueOnAnyThread();
}

int32 FPCGGraphCache::GetMemoryBudgetMB() const
{
	return bIsGameWorld ? CVarCacheMemoryBudgetRuntimeMB.GetValueOnAnyThread() : CVarCacheMemoryBudgetEditorMB.GetValueOnAnyThread();
}

bool FPCGGraphCache::GetFromCache(const FPCGGetFromCacheParams& Params, FPCGDataCollection& OutOutput) const
{
	if (!IsEnabled())
	{
		return false;
	}

	const UPCGNode* InNode = Params.Node;
	const IPCGElement* InElement = Params.Element;
	const IPCGGraphExecutionSource* InExecutionSource = Params.ExecutionSource;
	const FPCGCrc& InDependenciesCrc = Params.Crc;

	if(!InDependenciesCrc.IsValid())
	{
		UE_LOGF(LogPCG, Warning, "Invalid dependencies passed to FPCGGraphCache::GetFromCache(), lookup aborted.");
		return false;
	}

	const bool bDebuggingEnabled = IsDebuggingEnabled() && InExecutionSource && InNode;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCache::GetFromCache);
		PCG::TScopeLock ScopedLock(CacheLock);
		++Stats.GetCount;

		FPCGCacheEntryKey CacheKey(InElement, InDependenciesCrc);
		if (const FPCGDataCollection* Value = const_cast<FPCGGraphCache*>(this)->CacheData.FindAndTouch(CacheKey))
		{
			if (bDebuggingEnabled)
			{
				// Leading spaces to align log content with warnings below - helps readability a lot.
				UE_LOGF(LogPCG, Log, "         [%ls] %ls\t\tCACHE HIT %u", *InExecutionSource->GetExecutionState().GetDebugName(), *InNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString(), InDependenciesCrc.GetValue());
			}

			OutOutput = *Value;
			++Stats.HitCount;

			return true;
		}
		else
		{
			if (bDebuggingEnabled)
			{
				UE_LOGF(LogPCG, Warning, "[%ls] %ls\t\tCACHE MISS %u", *InExecutionSource->GetExecutionState().GetDebugName(), *InNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString(), InDependenciesCrc.GetValue());
			}

			return false;
		}
	}
}

void FPCGGraphCache::StoreInCache(const FPCGStoreInCacheParams& Params, const FPCGDataCollection& InOutput)
{
	if (!IsEnabled())
	{
		return;
	}

	const IPCGElement* InElement = Params.Element;
	const FPCGCrc& InDependenciesCrc = Params.Crc;

	if (!ensure(InDependenciesCrc.IsValid()))
	{
		return;
	}

	// Proxies should never go into the graph cache. These can hold onto large chunks of video memory.
	for (const FPCGTaggedData& Data : InOutput.TaggedData)
	{
		ensure(!Data.Data || Data.Data->IsCacheable());
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCache::StoreInCache);
		PCG::TScopeLock ScopedLock(CacheLock);

		++Stats.StoreCount;
		if (CacheData.Num() == CacheData.Max())
		{
			++Stats.GrowCount;
			GrowCache_Unsafe();
		}

		const FPCGCacheEntryKey CacheKey(InElement, InDependenciesCrc);
		AddToCacheInternal(CacheKey, InOutput, /*bAddToMemory=*/true);
	}
}

void FPCGGraphCache::ClearCache()
{
	PCG::TScopeLock ScopedLock(CacheLock);

	// Remove all entries
	ClearCacheInternal(CacheData.Max(), /*bClearMemory=*/true);
}

bool FPCGGraphCache::EnforceMemoryBudget()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCache::FPCGGraphCache::EnforceMemoryBudget);
	if (!IsEnabled())
	{
		return false;
	}

	if (!CVarCacheMemoryBudgetEnabled.GetValueOnAnyThread())
	{
		return false;
	}

	const uint64 MemoryBudget = static_cast<uint64>(GetMemoryBudgetMB()) * 1024 * 1024;
	if (TotalMemoryUsed <= MemoryBudget)
	{
		return false;
	}

	{
		PCG::TScopeLock ScopedLock(CacheLock);
		const float MemoryCleanupRatio = FMath::Clamp(CVarCacheMemoryCleanupRatio.GetValueOnAnyThread(), 0.0f, 1.0f);
		const uint64 TargetCacheMemoryUsage = static_cast<uint64>(MemoryCleanupRatio * MemoryBudget);

		while (TotalMemoryUsed > TargetCacheMemoryUsage && CacheData.Num() > 0)
		{
			RemoveFromCacheInternal(CacheData.GetLeastRecentKey());
		}
		ValidateElementToCacheEntryKeys();
	}

	return true;
}

#if WITH_EDITOR
void FPCGGraphCache::CleanFromCache(const IPCGElement* InElement, const UPCGSettings* InSettings/*= nullptr*/)
{
	if (!InElement)
	{
		return;
	}

	if (IsDebuggingEnabled())
	{
		UE_LOGF(LogPCG, Warning, "[] \t\tCACHE: PURGED [%ls]", InSettings ? *InSettings->GetDefaultNodeTitle().ToString() : TEXT("AnonymousElement"));
	}

	{
		PCG::TScopeLock ScopedLock(CacheLock);

		ValidateElementToCacheEntryKeys();

		TSet<FPCGCacheEntryKey> ElementCacheEntryKeys;
		ElementToCacheEntryKeys.RemoveAndCopyValue(InElement, ElementCacheEntryKeys);

		for (const FPCGCacheEntryKey& Key : ElementCacheEntryKeys)
		{
			RemoveFromCacheInternal(Key);
		}

		ValidateElementToCacheEntryKeys();
	}
}

uint32 FPCGGraphCache::GetGraphCacheEntryCount(IPCGElement* InElement) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCache::GetFromCache);
	PCG::TScopeLock ScopedLock(CacheLock);

	if (const TSet<FPCGCacheEntryKey>* ElementCacheEntryKeys = ElementToCacheEntryKeys.Find(InElement))
	{
		return ElementCacheEntryKeys->Num();
	}

	return 0;
}
#endif // WITH_EDITOR

void FPCGGraphCache::AddReferencedObjects(FReferenceCollector& Collector)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCache::AddReferencedObjects);
	PCG::TScopeLock ScopedLock(CacheLock);

	for (FPCGDataCollection& CacheEntry : CacheData)
	{
		CacheEntry.AddReferences(Collector);
	}
}

void FPCGGraphCache::ValidateElementToCacheEntryKeys() const
{
	if (CVarValidateElementToCacheEntryKeys.GetValueOnAnyThread())
	{
		int32 CacheKeyCount = 0;
		for (const auto& Kvp : ElementToCacheEntryKeys)
		{
			CacheKeyCount += Kvp.Value.Num();
		}

		check(CacheKeyCount == CacheData.Num());
	}
}

void FPCGGraphCache::ClearCacheInternal(int32 InMaxEntries, bool bClearMemory)
{
	if (bClearMemory)
	{
		MemoryRecords.Empty();
		TotalMemoryUsed = 0;
	}

	CacheData.Empty(InMaxEntries);
	ElementToCacheEntryKeys.Empty();
}

void FPCGGraphCache::AddToCacheInternal(const FPCGCacheEntryKey& InKey, const FPCGDataCollection& InCollection, bool bAddToMemory)
{
	// We currently grow the cache before calling add so this shouldn't be needed but if 
	// the rules change we need to make sure we keep ElementToCacheEntryKeys in sync
	if (CacheData.Num() == CacheData.Max())
	{
		RemoveFromCacheInternal(CacheData.GetLeastRecentKey());
	}

	CacheData.Add(InKey, InCollection);
	ElementToCacheEntryKeys.FindOrAdd(InKey.GetElement()).Add(InKey);

	if (bAddToMemory)
	{
		AddDataToAccountedMemory(InCollection);
	}

	ValidateElementToCacheEntryKeys();
}

void FPCGGraphCache::RemoveFromCacheInternal(const FPCGCacheEntryKey& InKey)
{
	++Stats.RemoveCount;

	if (TSet<FPCGCacheEntryKey>* ElementCacheEntryKeys = ElementToCacheEntryKeys.Find(InKey.GetElement()))
	{
		ElementCacheEntryKeys->Remove(InKey);
		if (ElementCacheEntryKeys->IsEmpty())
		{
			ElementToCacheEntryKeys.Remove(InKey.GetElement());
		}
	}

	if (const FPCGDataCollection* RemovedData = CacheData.Find(InKey))
	{
		RemoveFromMemoryTotal(*RemovedData);
		CacheData.Remove(InKey);
	}
}

void FPCGGraphCache::GrowCache_Unsafe()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCache::GrowCache_Unsafe);

	const int32 NewSize = (CacheData.Num() > 0) ? (2 * CacheData.Num()) : GPCGGraphCacheInitialCapacity;
	CacheData.SetMaxNumElements(NewSize);
}

void FPCGGraphCache::AddDataToAccountedMemory(const FPCGDataCollection& InCollection)
{
	for (const FPCGTaggedData& Data : InCollection.TaggedData)
	{
		if (Data.Data)
		{
			Data.Data->VisitDataNetwork([this](const UPCGData* Data)
			{
				if (Data)
				{
					// Find or add record
					if (FCachedMemoryRecord* ExistingRecord = MemoryRecords.Find(Data->UID))
					{
						ExistingRecord->InstanceCount++;
					}
					else
					{
						PCGGraphCacheHelpers::bIsGatheringCacheSize = PCGGraphCacheHelpers::CVarEnableTLSGatherCacheSize.GetValueOnAnyThread();
						ON_SCOPE_EXIT
						{
							PCGGraphCacheHelpers::bIsGatheringCacheSize = false;
						};

						// @todo_pcg: revisit usage of EResourceSizeMode::Exclusive instead of EResourceSizeMode::EstimatedTotals
						FResourceSizeEx ResSize = FResourceSizeEx(EResourceSizeMode::Exclusive);
						// Calculate data size. Function is non-const but is const-like, especially when
						// resource mode is Exclusive. The other mode calls a function to find all outer'd
						// objects which is non-const.
						const_cast<UPCGData*>(Data)->GetResourceSizeEx(ResSize);
						const SIZE_T DataSize = ResSize.GetDedicatedSystemMemoryBytes();

						FCachedMemoryRecord& NewRecord = MemoryRecords.Add(Data->UID);
						NewRecord.Data = Data;
						NewRecord.MemoryPerInstance = DataSize;
						NewRecord.InstanceCount = 1;
						TotalMemoryUsed += DataSize;
					}
				}
			});
		}
	}
}

void FPCGGraphCache::RemoveFromMemoryTotal(const FPCGDataCollection& InCollection)
{
	for (const FPCGTaggedData& Data : InCollection.TaggedData)
	{
		if (Data.Data)
		{
			Data.Data->VisitDataNetwork([this](const UPCGData* Data)
			{
				FCachedMemoryRecord* Record = Data ? MemoryRecords.Find(Data->UID) : nullptr;
				if (ensure(Record))
				{
					// Update instance count
					if (ensure(Record->InstanceCount > 0))
					{
						--Record->InstanceCount;
					}

					if (Record->InstanceCount == 0)
					{
						// Last instance removed, update accordingly
						if (TotalMemoryUsed >= Record->MemoryPerInstance)
						{
							TotalMemoryUsed -= Record->MemoryPerInstance;
						}
						else
						{
							// Should not normally reach here but it seems to happen in rare cases. Clamp to 0.
							TotalMemoryUsed = 0;
						}

						MemoryRecords.Remove(Data->UID);
					}
				}
			});
		}
	}
}

bool FPCGGraphCache::IsDebuggingEnabled() const
{
	return CVarCacheDebugging.GetValueOnAnyThread();
}

#if WITH_EDITOR
// Cvar deprecation. One-shot migration: runs after ini loading / user input via a cvar sink. Only required with editor, in standalone
// it appears the pcg.Cache.Enabled change delegate fires which handles driving the new cvars.
static void MigrateCVarsIfNeeded()
{
	static bool bDone = false;
	if (bDone)
	{
		return;
	}
	bDone = true;

	auto SetFromIni = [](uint32 InFlags, uint32 InSetBy)
	{
		// Treat any of these as "user set in ini" (covers typical places users put CVars)
		return InSetBy == ECVF_SetBySystemSettingsIni || // [SystemSettings] sections in *Engine.ini
			InSetBy == ECVF_SetByConsoleVariablesIni || // ConsoleVariables.ini
			InSetBy == ECVF_SetByDeviceProfile || // DeviceProfiles.ini
			(InFlags & ECVF_CreatedFromIni) != 0; // Created by ini before registration
	};

	const uint32 CacheEnabledFlags = CVarCacheEnabled_DEPRECATED->GetFlags();
	const uint32 CacheEnabledSetBy = (CacheEnabledFlags & ECVF_SetByMask);

	if (SetFromIni(CacheEnabledFlags, CacheEnabledSetBy))
	{
		const bool bValue = CVarCacheEnabled_DEPRECATED->GetBool();

		// Copy with the same SetBy priority so behavior matches the user's source of truth.
		CVarCacheEnabledRuntime->Set(bValue, (EConsoleVariableFlags)CacheEnabledSetBy);
		CVarCacheEnabledEditor->Set(bValue, (EConsoleVariableFlags)CacheEnabledSetBy);

		UE_LOGF(LogPCG, Warning, "Cvar 'pcg.Cache.Enabled' was set from ini but is deprecated in 5.7. Migrated value to pcg.Cache.Runtime.Enabled=%d, pcg.Cache.Editor.Enabled=%d (SetBy=0x%x).", bValue, bValue, CacheEnabledSetBy);
	}

	const uint32 CacheBudgetFlags = CVarCacheMemoryBudgetMB_DEPRECATED->GetFlags();
	const uint32 CacheBudgetSetBy = (CacheBudgetFlags & ECVF_SetByMask);

	if (SetFromIni(CacheBudgetFlags, CacheBudgetSetBy))
	{
		const int32 Value = CVarCacheMemoryBudgetMB_DEPRECATED->GetInt();

		// Copy with the same SetBy priority so behavior matches the user's source of truth.
		CVarCacheMemoryBudgetRuntimeMB->Set(Value, (EConsoleVariableFlags)CacheBudgetSetBy);
		CVarCacheMemoryBudgetEditorMB->Set(Value, (EConsoleVariableFlags)CacheBudgetSetBy);

		UE_LOGF(LogPCG, Warning, "Cvar 'pcg.Cache.MemoryBudgetMB' was set from ini but is deprecated in 5.7. Migrated value to pcg.Cache.Runtime.MemoryBudgetMB=%d, pcg.Cache.Editor.MemoryBudgetMB=%d (SetBy=0x%x).", Value, Value, CacheBudgetSetBy);
	}
}

// Register a sink so this runs at the right time (after ini load / on first cvar change tick).
static FAutoConsoleVariableSink GCVarMigrationSink(FConsoleCommandDelegate::CreateStatic(&MigrateCVarsIfNeeded));
#endif // WITH_EDITOR
