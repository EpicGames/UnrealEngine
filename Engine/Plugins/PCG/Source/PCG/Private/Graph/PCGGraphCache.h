// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCrc.h"
#include "PCGData.h"
#include "Graph/IPCGGraphCache.h"

#include "Containers/LruCache.h"
#include "Misc/SpinLock.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"

class IPCGElement;
class UPCGNode;
class UPCGSettings;

/** A record kept each data object instance to count how many instances are tracked. */
struct FCachedMemoryRecord
{
	FCachedMemoryRecord() = default;

	/** Number of instances that are contributing to used memory. */
	uint32 InstanceCount = 0;

	/** Memory usage in bytes of one instance. Cached here to save recomputing later. */
	SIZE_T MemoryPerInstance = 0;

	/** Allows retrieval of data ptr for logging purposes. */
	FObjectKey Data;
};

struct FPCGCacheEntryKey
{
	FPCGCacheEntryKey(const IPCGElement* InElement, const FPCGCrc& InCrc)
		: Element(InElement)
		, CrcHash(InCrc.GetValue())
	{
		check(Element);
		ensure(InCrc.IsValid());

		Hash = GetTypeHash(Element);
		Hash = HashCombine(Hash, CrcHash);
	}

	bool operator==(const FPCGCacheEntryKey& Other) const
	{
		return Element == Other.Element && CrcHash == Other.CrcHash;
	}

	friend uint32 GetTypeHash(const FPCGCacheEntryKey& Key)
	{
		return Key.Hash;
	}

	const IPCGElement* GetElement() const { return Element; }

private:
	const IPCGElement* Element = nullptr;
	uint32 CrcHash = 0;
	uint32 Hash = 0;
};

namespace PCGGraphCacheHelpers
{
	extern thread_local bool bIsGatheringCacheSize;
	extern TAutoConsoleVariable<bool> CVarEnableTLSGatherCacheSize;
}

/** Core idea is to store cache entries per node, but that will be less efficient
* In cases where we have some subgraph reuse. Under that premise, we can then
* instead store by element, as we will never recreate elements (except for arbitrary tasks)
*/
class FPCGGraphCache : public FGCObject, public IPCGGraphCache
{
public:
	FPCGGraphCache(bool bGameWorld);
	~FPCGGraphCache();

	/** Returns whether a GetResourceSizeEx call is done for cache sizing (or pcg memory profiling) scope vs. the system-level resource gathering. */
	static bool IsGatheringCacheSize();

	/** Memory details required in the GetResourceSizeEx calls. Will always be 0 (e.g. full) when not gathering for the cache. */
	static int CacheSizeDetailLevel();

	virtual bool IsEnabled() const override;

	int32 GetMemoryBudgetMB() const;

	/** Returns true if data was found from the cache, in which case the outputs are written in OutOutput. InNode is optional and for logging only. */
	virtual bool GetFromCache(const FPCGGetFromCacheParams& Params, FPCGDataCollection& OutCollection) const override;

	/** Stores data in the cache for later use */
	virtual void StoreInCache(const FPCGStoreInCacheParams& Params, const FPCGDataCollection& InCollection) override;

	/** Removes all entries from the cache, unroots data, etc. */
	void ClearCache();

	/** While memory usage is more than budget, remove cache entries for elements, LRU policy. Returns true if something removed. */
	bool EnforceMemoryBudget();

	/** True if debugging features enabled. Exposes a CVar so it can be called from editor module. */
	bool IsDebuggingEnabled() const;

	/** Reset cache stats. */
	void ResetStats();

	/** Log cache stats. */
	void LogStats();

#if WITH_EDITOR
	/** Clears any cache entry for the given data. InSettings is optional and for logging only. */
	void CleanFromCache(const IPCGElement* InElement, const UPCGSettings* InSettings = nullptr);

	/** Returns number of copies of data cached for InElement. */
	uint32 GetGraphCacheEntryCount(IPCGElement* InElement) const;
#endif

	//~Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FPCGGraphCache"); }
	//~End FGCObject interface

private:
	// Grow cache entry capacity, preserving current entries.
	void GrowCache_Unsafe();

	/** For each data element in collection, keep records of memory usage and instance count. */
	void AddDataToAccountedMemory(const FPCGDataCollection& InCollection);

	/** For each data element in collection, update records to reflect removal and update total memory figure if all instances removed. */
	void RemoveFromMemoryTotal(const FPCGDataCollection& InCollection);

	void ClearCacheInternal(int32 InMaxEntries, bool bClearMemory);
	void AddToCacheInternal(const FPCGCacheEntryKey& InKey, const FPCGDataCollection& InCollection, bool bAddToMemory);
	void RemoveFromCacheInternal(const FPCGCacheEntryKey& InKey);
	void ValidateElementToCacheEntryKeys() const;

	// Note: we are not going to serialize this as-is, since the pointers will change
	// We will have to serialize on a node id basis most likely
	TLruCache<FPCGCacheEntryKey, FPCGDataCollection> CacheData;
	TMap<const IPCGElement*, TSet<FPCGCacheEntryKey>> ElementToCacheEntryKeys;

	/** Map from data UIDs to records. Provides ref counting and caches memory size. */
	TMap<uint64, FCachedMemoryRecord> MemoryRecords;

	/** Total memory usage by all data objects in cache. */
	uint64 TotalMemoryUsed = 0;

	bool bIsGameWorld = true;

	mutable PCG::FLock CacheLock;

	// Store Get/Store stats
	struct FPCGCacheStats
	{
		int64 GetCount = 0;
		int64 StoreCount = 0;
		int64 HitCount = 0;
		int64 GrowCount = 0;
		int64 RemoveCount = 0;
	};
	
	mutable FPCGCacheStats Stats;
};
