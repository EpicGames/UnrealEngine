// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/PCGGraphTask.h"

#include "PCGGraphCompilationData.generated.h"

class UPCGComputeGraph;
struct FPCGStackContext;

/** Helper struct to allow serialization of GridSize to Compiled Graph Tasks map. */
USTRUCT()
struct FPCGGraphTasks
{
	GENERATED_BODY();

	FPCGGraphTasks() = default;

	explicit FPCGGraphTasks(TArray<FPCGGraphTask> InGraphTasks)
		: GraphTasks(std::move(InGraphTasks))
	{}

	UPROPERTY()
	TArray<FPCGGraphTask> GraphTasks;
};

/** Helper struct to allow serialization of GridSize to ComputeGraphs map. */
USTRUCT()
struct FPCGComputeGraphs
{
	GENERATED_BODY();

	FPCGComputeGraphs() = default;

	explicit FPCGComputeGraphs(TArray<TObjectPtr<UPCGComputeGraph>> InComputeGraphs)
		: ComputeGraphs(std::move(InComputeGraphs))
	{}

	UPROPERTY()
	TArray<TObjectPtr<UPCGComputeGraph>> ComputeGraphs;
};

/** Holds compiled data for serialization during cook. */
UCLASS()
class UPCGGraphCompilationData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TMap</*HiGen Grid Size*/uint32, FPCGGraphTasks> Tasks;

	UPROPERTY()
	TMap</*HiGen Grid Size*/uint32, FPCGStackContext> StackContexts;

	UPROPERTY()
	TMap</*HiGen Grid Size*/uint32, FPCGComputeGraphs> ComputeGraphs;
};
