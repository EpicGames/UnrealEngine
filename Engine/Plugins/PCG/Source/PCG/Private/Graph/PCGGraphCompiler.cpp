// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/PCGGraphCompiler.h"

#include "PCGComponent.h"
#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGInputOutputSettings.h"
#include "PCGModule.h"
#include "PCGPin.h"
#include "PCGSubgraph.h"
#include "Elements/PCGGather.h"
#include "Elements/PCGHiGenGridSize.h"
#include "Elements/PCGReroute.h"
#include "Graph/PCGGraphCompilationData.h"
#include "Graph/PCGGraphCompilerGPU.h"
#include "Graph/PCGGraphExecutor.h"
#include "Graph/PCGPinDependencyExpression.h"

#include "HAL/IConsoleManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/ScopeRWLock.h"

namespace PCGGraphCompiler
{
	static TAutoConsoleVariable<bool> CVarEnableTaskStaticCulling(
		TEXT("pcg.GraphExecution.TaskStaticCulling"),
		true,
		TEXT("Enable static culling of tasks which considers static branches, generation grid size, trivial nodes and more."));

	static TAutoConsoleVariable<bool> CVarEnableGPUExecution(
		TEXT("pcg.GraphExecution.GPU.Enable"),
		true,
		TEXT("Whether to emit compatible nodes as compute graphs to execute on the GPU."));

	/**
	 * Utility structure to store "FPCGGraphTaskInput" in a hashable set
	 * and query it easily using only the task id and its upstream pin, to keep track of inputs used multiple times.
	 * TaskInput is the same as another if they have the same task id and the same upstream pin (or they are both null).
	 */
	struct FTaskInputOrigin
	{
		FTaskInputOrigin(FPCGGraphTaskInput* InTaskInput)
		{
			// Only set it if the upstream properties is set
			TaskInput = (InTaskInput && InTaskInput->UpstreamPin.IsSet()) ? InTaskInput : nullptr;
		}
		
		friend uint32 GetTypeHash(const FTaskInputOrigin& Value)
		{
			check(!Value.TaskInput || Value.TaskInput->UpstreamPin.IsSet());
			return Value.TaskInput ? HashCombine(Value.TaskInput->TaskId, GetTypeHash(Value.TaskInput->UpstreamPin.GetValue())) : PointerHash(nullptr);
		}

		bool operator==(const FTaskInputOrigin& Other) const
		{
			if (!TaskInput && !Other.TaskInput)
			{
				return true;
			}
			else if (!TaskInput || !Other.TaskInput || TaskInput->TaskId != Other.TaskInput->TaskId)
			{
				return false;
			}
			else
			{
				// By construction those are set.
				const FPCGPinProperties& Properties = TaskInput->UpstreamPin.GetValue();
				const FPCGPinProperties& OtherProperties = Other.TaskInput->UpstreamPin.GetValue();

				return Properties == OtherProperties;
			}
		}
		
		FPCGGraphTaskInput* TaskInput = nullptr;
	};
}

UPCGComputeGraph* FPCGGraphCompilerCache::GetCompiledComputeGraph(const UPCGGraph* InGraph, uint32 GridSize, uint32 ComputeGraphIndex)
{
#if WITH_EDITOR
	{
		PCG::TSharedScopeLock ReadLock(GraphToTaskMapLock);

		if (TMap<uint32, TArray<TObjectPtr<UPCGComputeGraph>>>* GridSizeToComputeGraphs = TopGraphToComputeGraphMap.Find(InGraph))
		{
			if (TArray<TObjectPtr<UPCGComputeGraph>>* ComputeGraphs = GridSizeToComputeGraphs->Find(GridSize))
			{
				if (ComputeGraphs->IsValidIndex(ComputeGraphIndex))
				{
					return (*ComputeGraphs)[ComputeGraphIndex];
				}
			}
		}
	}
#else
	// Cannot compile compute graphs outside of editor, so we always look for cooked compute graphs.
	const UPCGGraphCompilationData* CookedData = InGraph ? InGraph->GetCookedCompilationData().Get() : nullptr;

	if (CookedData)
	{
		const FPCGComputeGraphs* CookedComputeGraphs = CookedData->ComputeGraphs.Find(GridSize);

		if (CookedComputeGraphs && CookedComputeGraphs->ComputeGraphs.IsValidIndex(ComputeGraphIndex))
		{
			return CookedComputeGraphs->ComputeGraphs[ComputeGraphIndex];
		}
	}
#endif

	return nullptr;
}

#if WITH_EDITOR
void FPCGGraphCompilerCache::RemoveFromCache(UPCGGraph* InGraph)
{
	UE_LOGF(LogPCG, Verbose, "FPCGGraphCompilerCache::RemoveFromCache '%ls'", *InGraph->GetName());

	check(InGraph);
	RemoveFromCacheRecursive(InGraph);
}

void FPCGGraphCompilerCache::RemoveFromCacheRecursive(UPCGGraph* InGraph)
{
	{
		PCG::TUniqueScopeLock WriteLock(GraphToTaskMapLock);
		GraphToTaskMap.Remove(InGraph);
		GraphToStackContextMap.Remove(InGraph);
		TopGraphToTaskMap.Remove(InGraph);
		TopGraphToStackContextMap.Remove(InGraph);
		TopGraphToComputeGraphMap.Remove(InGraph);
	}

	TArray<UPCGGraph*> ParentGraphs;

	{
		PCG::TScopeLock Lock(GraphDependenciesLock);
		GraphDependencies.MultiFind(InGraph, ParentGraphs);
		GraphDependencies.Remove(InGraph);
	}

	for (UPCGGraph* Graph : ParentGraphs)
	{
		RemoveFromCacheRecursive(Graph);
	}
}
#endif // WITH_EDITOR

TArray<FPCGGraphTask> FPCGGraphCompiler::CompileGraph(UPCGGraph* InGraph, FPCGTaskId& NextId, FPCGStackContext& InOutStackContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompiler::CompileGraph);
	if (!InGraph)
	{
		return TArray<FPCGGraphTask>();
	}

	InOutStackContext.PushFrame(InGraph);

	TArray<FPCGGraphTask> CompiledTasks;
	TMap<const UPCGNode*, FPCGTaskId> IdMapping;
	TArray<const UPCGNode*> NodeQueue;

	// Prime the node queue with all nodes that have no inbound edges
	for (const UPCGNode* Node : InGraph->GetNodes())
	{
		if(!Node->HasInboundEdges())
		{
			NodeQueue.Add(Node);
		}
	}

	// By definition, the input node has no inbound edge.
	// Put it last in the queue so it gets picked up first - order is important for hooking up the fetch input element
	NodeQueue.Add(InGraph->GetInputNode());

	while (NodeQueue.Num() > 0)
	{
		const UPCGNode* Node = NodeQueue.Pop();

		const UPCGBaseSubgraphNode* SubgraphNode = Cast<const UPCGBaseSubgraphNode>(Node);
		UPCGGraph* Subgraph = SubgraphNode ? SubgraphNode->GetSubgraph().Get() : nullptr;
		const UPCGBaseSubgraphSettings* SubgraphSettings = SubgraphNode ? Cast<const UPCGBaseSubgraphSettings>(SubgraphNode->GetSettings()) : nullptr;

		// Note that recursive graphs must be dynamic by definition, as we aren't able to 'finish' emitting compiled tasks during compilation otherwise.
		// Implementation note: it is very important that the predicate here is symmetrical with the one in the subgraph node otherwise some tasks could be missing.
		const bool bIsRecursiveGraph = Subgraph && (InGraph == Subgraph || Subgraph->Contains(InGraph));
		bool bIsDynamic = (!SubgraphSettings || SubgraphSettings->IsDynamicGraph());
		const bool bIsNotDisabled = SubgraphSettings && SubgraphSettings->bEnabled;

#if WITH_EDITOR
		bIsDynamic |= PCGSystemSwitches::CVarForceDynamicGraphDispatch.GetValueOnAnyThread();
#endif

		if(Subgraph && !bIsDynamic && bIsNotDisabled && !bIsRecursiveGraph)
		{
			const FPCGTaskId PreId = NextId++;

			// 1. Compile the subgraph making sure we don't reuse the same ids
			// Note that we will not consume the pre or post-execute tasks, ergo bIsTopGraph=false
			FPCGStackContext SubgraphStackContext;
			// Passed uninitialized grid size to get all tasks
			TArray<FPCGGraphTask> Subtasks = GetCompiledTasks(Subgraph, PCGHiGenGrid::UninitializedGridSize(), SubgraphStackContext, /*bIsTopGraph=*/false);

#if WITH_EDITOR
			Cache.GraphDependenciesLock.Lock();
			Cache.GraphDependencies.AddUnique(Subgraph, InGraph);
			Cache.GraphDependenciesLock.Unlock();
#endif // WITH_EDITOR

			// Append all the stack frames from inside the subgraph to my current graph stack
			InOutStackContext.PushFrame(Node);
			const int32 StackOffset = InOutStackContext.GetNumStacks();
			for (FPCGGraphTask& Subtask : Subtasks)
			{
				Subtask.StackIndex += StackOffset;
			}
			InOutStackContext.AppendStacks(SubgraphStackContext);
			InOutStackContext.PopFrame();

			OffsetNodeIds(Subtasks, NextId, PreId);
			NextId += Subtasks.Num();

			const UPCGNode* SubgraphInputNode = Subgraph->GetInputNode();
			const UPCGNode* SubgraphOutputNode = Subgraph->GetOutputNode();

			// 2. Update the "input" and "output" node tasks so we can add the proper dependencies
			FPCGGraphTask* InputNodeTask = Subtasks.FindByPredicate([SubgraphInputNode](const FPCGGraphTask& Subtask) {
				return Subtask.Node == SubgraphInputNode;
				});

			FPCGGraphTask* OutputNodeTask = Subtasks.FindByPredicate([SubgraphOutputNode](const FPCGGraphTask& Subtask) {
				return Subtask.Node == SubgraphOutputNode;
				});

			// Build pre-task
			FPCGGraphTask& PreTask = CompiledTasks.Emplace_GetRef();
			PreTask.Node = Node;
			PreTask.NodeId = PreId;
			PreTask.StackIndex = InOutStackContext.GetCurrentStackIndex();

			for (const UPCGPin* InputPin : Node->InputPins)
			{
				check(InputPin);
				// Params have to be funneled in the subgraph element (so we can pass down user parameters)
				const bool bIsOverrideOrUserParamPin = InputPin->Properties.IsOverrideOrUserParamPin();
				const bool bIsDatalessPin = InputPin->Properties.IsDatalessPin();

				for (const UPCGEdge* InboundEdge : InputPin->Edges)
				{
					if (InboundEdge->IsValid())
					{
						// Implementation note: conceptually, the non-param inputs need to be connected only to the input node task.
						// However, because of the static/dynamic culling which happen on the subgraph node, we need to have the connections on the pretask too (e.g. the subgraph node) even if they don't provide data.
						if (bIsOverrideOrUserParamPin || !InputNodeTask)
						{
							PreTask.Inputs.Emplace(IdMapping[InboundEdge->InputPin->Node], InboundEdge->InputPin->Properties, InboundEdge->OutputPin->Properties, /*bInProvideData=*/true);
						}
						else
						{
							PreTask.Inputs.Emplace(IdMapping[InboundEdge->InputPin->Node], InboundEdge->InputPin->Properties, InboundEdge->OutputPin->Properties, /*bInProvideData=*/false);
							InputNodeTask->Inputs.Emplace(IdMapping[InboundEdge->InputPin->Node], InboundEdge->InputPin->Properties, InboundEdge->OutputPin->Properties, /*bInProvideData=*/!bIsDatalessPin);
						}
					}
					else
					{
						UE_LOGFMT(LogPCG, Warning, "Invalid inbound edge on subgraph");
					}
				}
			}

			// Add pre-task as input to subgraph input node task, without data dependency
			if (InputNodeTask)
			{
				InputNodeTask->Inputs.Emplace(PreId,/*InUpstreamPin=*/FPCGGraphTaskInput::NoPin, /*InDownstreamPin=*/FPCGGraphTaskInput::NoPin, /*bInProvideData=*/false);
			}

			// Hook nodes to the PreTask if they require so.
			// Only do it for nodes that are directly under the subgraph, not in subsequent subgraphs.
			for (FPCGGraphTask& Subtask : Subtasks)
			{
				if (!Subtask.Node || Subtask.Node->GetOuter() != Subgraph)
				{
					continue;
				}

				// We hook to pretask if the element requires data from the pretask, or otherwise if there are no inputs, to ensure
				// that the element executes with the other subgraph tasks (and can be culled if the subgraph node is culled).
				const UPCGSettings* Settings = Subtask.Node->GetSettings();
				const bool bRequiresDataFromPreTask = Settings && Settings->RequiresDataFromPreTask();
				if (bRequiresDataFromPreTask || Subtask.Inputs.IsEmpty())
				{
					Subtask.Inputs.Emplace(PreId, /*InUpstreamPin=*/FPCGGraphTaskInput::NoPin, /*InDownstreamPin=*/FPCGGraphTaskInput::NoPin, bRequiresDataFromPreTask);
					Subtask.bWasHookedToPreTask = true;
				}
			}

			// Merge subgraph tasks into current tasks.
			CompiledTasks.Append(Subtasks);

			// Build post-task
			const FPCGTaskId PostId = NextId++;
			FPCGGraphTask& PostTask = CompiledTasks.Emplace_GetRef();
			PostTask.Node = Node;
			PostTask.NodeId = PostId;
			PostTask.StackIndex = InOutStackContext.GetCurrentStackIndex();
			// Implementation note: since we`ve already executed the node once, we normally don`t need to execute it a second time
			// especially since we cannot distinguish between the pre and post during execution so any data filtering related to pins is bound to fail.
			PostTask.ElementSource = EPCGElementSource::Trivial;

			// Add execution-only dependency on pre-task, without this post task can be scheduled concurrently with pre-task, and concurrently
			// with something that might become inactive and would then fail to dynamically cull this already-scheduled task.
			// Additional implementation note: this first depedencency is critical in our ability to do static culling (see CalculateStaticallyActiveRecursive)
			// and should not be changed here without changing the other.
			PostTask.Inputs.Emplace(PreId, /*InUpstreamPin=*/FPCGGraphTaskInput::NoPin, /*InDownstreamPin=*/FPCGGraphTaskInput::NoPin, /*bInProvideData=*/false);

			// Add subgraph output node task as input to the post-task
			if (OutputNodeTask)
			{
				PostTask.Inputs.Emplace(OutputNodeTask->NodeId);
			}

			check(!IdMapping.Contains(Node));
			IdMapping.Add(Node, PostId);
		}
		else
		{
			const FPCGTaskId NodeId = NextId++;
			FPCGGraphTask& Task = CompiledTasks.Emplace_GetRef();
			Task.Node = Node;
			Task.NodeId = NodeId;
			Task.StackIndex = InOutStackContext.GetCurrentStackIndex();

			for (const UPCGPin* InputPin : Node->InputPins)
			{
				// Cull dependency only data if the pin is marked as such.
				const bool bIsDatalessPin = InputPin->Properties.IsDatalessPin();
				for (const UPCGEdge* InboundEdge : InputPin->Edges)
				{
					if (!InboundEdge->IsValid())
					{
						UE_LOGFMT(LogPCG, Warning, "Unbound inbound edge");
						continue;
					}

					check(InboundEdge->OutputPin == InputPin);
					if (FPCGTaskId* InboundId = IdMapping.Find(InboundEdge->InputPin->Node))
					{
						Task.Inputs.Emplace(*InboundId, InboundEdge->InputPin->Properties, InputPin->Properties, !bIsDatalessPin);
					}
					else
					{
						UE_LOGFMT(LogPCG, Error, "Inconsistent node linkage on node '{Node}'", FSoftObjectPath(Node));
						return TArray<FPCGGraphTask>();
					}
				}
			}

			check(!IdMapping.Contains(Node));
			IdMapping.Add(Node, NodeId);
		}

		// Push next ready nodes on the queue
		for (const UPCGPin* OutPin : Node->OutputPins)
		{
			for (const UPCGEdge* OutboundEdge : OutPin->Edges)
			{
				if (!OutboundEdge->IsValid())
				{
					UE_LOGFMT(LogPCG, Warning, "Unbound outbound edge");
					continue;
				}

				const UPCGNode* OutboundNode = OutboundEdge->OutputPin->Node;
				check(OutboundNode);

				if (NodeQueue.Contains(OutboundNode))
				{
					continue;
				}

				bool bAllPrerequisitesMet = true;

				for (const UPCGPin* OutboundNodeInputPin : OutboundNode->InputPins)
				{
					for (const UPCGEdge* OutboundNodeInboundEdge : OutboundNodeInputPin->Edges)
					{
						if (OutboundNodeInboundEdge->IsValid())
						{
							bAllPrerequisitesMet &= IdMapping.Contains(OutboundNodeInboundEdge->InputPin->Node);
						}
					}
				}

				if (bAllPrerequisitesMet)
				{
					NodeQueue.Add(OutboundNode);
				}
			}
		}
	}

	return CompiledTasks;
}

void FPCGGraphCompiler::Compile(UPCGGraph* InGraph)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompiler::Compile);

	bool bAlreadyCached = false;
	{
		PCG::TSharedScopeLock ReadLock(Cache.GraphToTaskMapLock);
		bAlreadyCached = Cache.GraphToTaskMap.Contains(InGraph);
	}

#if !WITH_EDITOR
	// Don't need to compile if the tasks are already cooked.
	const UPCGGraphCompilationData* CookedData = InGraph ? InGraph->GetCookedCompilationData().Get() : nullptr;

	if (CookedData)
	{
		bAlreadyCached |= !CookedData->Tasks.IsEmpty();
	}
#endif

	if (bAlreadyCached)
	{
		return;
	}

	// Otherwise, do the compilation; note that we always start at zero since
	// the caller will offset the ids as needed
	FPCGTaskId FirstId = 0;
	FPCGStackContext StackContext;
	TArray<FPCGGraphTask> CompiledTasks = CompileGraph(InGraph, FirstId, StackContext);

	// TODO: optimize no-ops, etc.

	// Store back the results in the cache if it's valid
	if (!CompiledTasks.IsEmpty())
	{
		PCG::TUniqueScopeLock WriteLock(Cache.GraphToTaskMapLock);
		if (!Cache.GraphToTaskMap.Contains(InGraph))
		{
			Cache.GraphToTaskMap.Add(InGraph, MoveTemp(CompiledTasks));
			Cache.GraphToStackContextMap.Add(InGraph, StackContext);
		}
	}
}

TArray<FPCGGraphTask> FPCGGraphCompiler::GetPrecompiledTasks(const UPCGGraph* InGraph, uint32 GenerationGridSize, FPCGStackContext& OutStackContext, bool bIsTopGraph) const
{
	// Get compiled tasks in a threadsafe way
	PCG::TSharedScopeLock ReadLock(Cache.GraphToTaskMapLock);

	const TArray<FPCGGraphTask>* ExistingTasks = nullptr;
	if (bIsTopGraph)
	{
		// Top graphs are optimized per grid size.
		const TMap<uint32, TArray<FPCGGraphTask>>* GridSizeToCompiledGraph = Cache.TopGraphToTaskMap.Find(InGraph);
		ExistingTasks = GridSizeToCompiledGraph ? GridSizeToCompiledGraph->Find(GenerationGridSize) : nullptr;

		const TMap<uint32, FPCGStackContext>* GridSizeToStackContext = Cache.TopGraphToStackContextMap.Find(InGraph);
		const FPCGStackContext* ExistingStackContext = GridSizeToStackContext ? GridSizeToStackContext->Find(GenerationGridSize) : nullptr;
		OutStackContext = ExistingStackContext ? *ExistingStackContext : FPCGStackContext();

		// Should either find both, or find neither.
		ensure(!ExistingTasks == !ExistingStackContext);
	}
	else
	{
		ExistingTasks = Cache.GraphToTaskMap.Find(InGraph);
	}

	return ExistingTasks ? *ExistingTasks : TArray<FPCGGraphTask>();
}

UPCGComputeGraph* FPCGGraphCompiler::GetComputeGraph(const UPCGGraph* InGraph, uint32 GridSize, uint32 ComputeGraphIndex)
{
	return Cache.GetCompiledComputeGraph(InGraph, GridSize, ComputeGraphIndex);
}

void FPCGGraphCompiler::ResolveGridSizes(
	uint32 GenerationGrid,
	const TArray<FPCGGraphTask>& CompiledTasks,
	const FPCGStackContext& StackContext,
	uint32 GenerationDefaultGrid,
	TArray<uint32>& InOutTaskGenerationGrid)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompiler::ResolveGridSizes);

	if (CompiledTasks.IsEmpty())
	{
		return;
	}
	check(!InOutTaskGenerationGrid.IsEmpty());

	// Special case - input node must always be present. Execute it on the current grid.
	InOutTaskGenerationGrid[0] = GenerationGrid;

	// Calculate execution grid values for subsequent tasks.
	for (int32 i = 1; i < CompiledTasks.Num(); ++i)
	{
		CalculateGridRecursive(CompiledTasks[i].NodeId, GenerationDefaultGrid, StackContext, CompiledTasks, InOutTaskGenerationGrid);
	}
}

void FPCGGraphCompiler::CreateGridLinkages(
	UPCGGraph* InGraph,
	uint32 InGenerationGrid,
	TArray<uint32>& InOutTaskGenerationGrid,
	TArray<FPCGGraphTask>& InOutCompiledTasks,
	const FPCGStackContext& InStackContext,
	bool bIsCooking)
{
	// Now add link tasks - if a Grid256 task depends on data from a Grid512 task, inject a link
	// task that looks up the Grid512 component, schedules its execution if it does not have data, and
	// then uses its output data.
	// 
	// The stack is used to form the ResourceKey - a string that provides a path to the data from top graph down to specific pin.
	// This will be used by link tasks as store/retrieve keys to marshal data for edges that cross grid size boundaries.
	const FPCGStack* CurrentStack = InStackContext.GetStack(InStackContext.GetCurrentStackIndex());
	if (InOutCompiledTasks.IsEmpty() || !ensure(CurrentStack))
	{
		return;
	}

	const int32 NumCompiledTasksBefore = InOutCompiledTasks.Num();
	for (FPCGTaskId TaskId = 0; TaskId < NumCompiledTasksBefore; ++TaskId)
	{
		const uint32 GraphGenerationGrid = InOutTaskGenerationGrid[TaskId];

		for (int32 InputIndex = 0; InputIndex < InOutCompiledTasks[TaskId].Inputs.Num(); ++InputIndex)
		{
			const FPCGGraphTaskInput TaskInput = InOutCompiledTasks[TaskId].Inputs[InputIndex];

			const UPCGPin* UpstreamPin = nullptr;
			if (TaskInput.UpstreamPin.IsSet())
			{
				if (const UPCGNode* Node = InOutCompiledTasks[TaskInput.TaskId].Node)
				{
					UpstreamPin = Node->GetOutputPin(TaskInput.UpstreamPin.GetValue().Label);
				}
			}

			if (!UpstreamPin)
			{
				// Don't link if we don't have a upstream pin to retrieve data from
				continue;
			}

			const uint32 InputGraphGenerationGrid = InOutTaskGenerationGrid[TaskInput.TaskId];
			// Register linkage task if grid sizes don't match - either way! This allows us to generate an execution-time error if going from
			// small grid to large grid.
			if (InputGraphGenerationGrid != PCGHiGenGrid::UninitializedGridSize() && InputGraphGenerationGrid > GraphGenerationGrid)
			{
				// Build a string identifier for the data
				FString ResourceKey;
				if (!ensure(CurrentStack->CreateStackFramePath(ResourceKey, UpstreamPin->Node, UpstreamPin)))
				{
					continue;
				}

				// Build task & element to hold the operation to perform
				FPCGGraphTask& LinkTask = InOutCompiledTasks.Emplace_GetRef();
				LinkTask.NodeId = InOutCompiledTasks.Num() - 1;
				LinkTask.StackIndex = InOutCompiledTasks[TaskId].StackIndex;

				FPCGPinProperties UpstreamPinProps = UpstreamPin->Properties;

				// Make sure we have the correct type in the case of dynamically typed pins.
				if (const UPCGSettings* UpstreamSettings = InOutCompiledTasks[TaskInput.TaskId].Node ? InOutCompiledTasks[TaskInput.TaskId].Node->GetSettings() : nullptr)
				{
					UpstreamPinProps.AllowedTypes = UpstreamSettings->GetCurrentPinTypesID(UpstreamPin);
				}

				// Create pin properties for the input of the LinkTask to convey type information. The grid linkage use the upstream output pin label on both input and
				// output, so effectively the upstream output pin label is forwarded all the way through to the final downstream edge (on the lower grid).
				FPCGPinProperties DownstreamPinProps = UpstreamPinProps;

				LinkTask.Inputs.Emplace(TaskInput.TaskId, UpstreamPinProps, DownstreamPinProps);

				const uint32 FromGrid = InOutTaskGenerationGrid[TaskInput.TaskId];
				const uint32 ToGrid = InOutTaskGenerationGrid[TaskId];

#if WITH_EDITOR
				if (bIsCooking)
				{
					TObjectPtr<UPCGGridLinkageSettings> Settings = NewObject<UPCGGridLinkageSettings>(InGraph);
					Settings->FromGrid = FromGrid;
					Settings->ToGrid = ToGrid;
					Settings->GenerationGrid = InGenerationGrid;
					Settings->ResourceKey = ResourceKey;
					Settings->UpstreamPin = UpstreamPin;

					LinkTask.ElementSource = EPCGElementSource::FromCookedSettings;
					LinkTask.CookedSettings = Settings;
				}
				else
#endif
				{
					LinkTask.Element = MakeShared<PCGGraphExecutor::FPCGGridLinkageElement>(FromGrid, ToGrid, InGenerationGrid, ResourceKey, UpstreamPin);
				}

				// Now splice in the new task - redirect the downstream task to grab its input from the link task.
				InOutCompiledTasks[TaskId].Inputs[InputIndex].TaskId = LinkTask.NodeId;

				// Set the task input's UpstreamPin so that it has the correct type.
				InOutCompiledTasks[TaskId].Inputs[InputIndex].UpstreamPin = UpstreamPinProps;

				// The link executes at both FROM grid size (store) and TO grid size (retrieve). Just store from grid size here - the task culling lambda checks for
				// grind links and checks both grid sizes.
				InOutTaskGenerationGrid.Add(FromGrid);
			}
		}
	}
}

uint32 FPCGGraphCompiler::CalculateGridRecursive(
	FPCGTaskId InTaskId,
	uint32 GenerationDefaultGrid,
	const FPCGStackContext& InStackContext,
	const TArray<FPCGGraphTask>& InCompiledTasks,
	TArray<uint32>& InOutTaskGenerationGrid)
{
	if (InOutTaskGenerationGrid[InTaskId] != PCGHiGenGrid::UninitializedGridSize())
	{
		return InOutTaskGenerationGrid[InTaskId];
	}

	const FPCGStack* Stack = InStackContext.GetStack(InCompiledTasks[InTaskId].StackIndex);
	check(Stack);
	const bool bTopLevelGraph = InCompiledTasks[InTaskId].ParentId == InvalidPCGTaskId;

	const UPCGNode* Node = InCompiledTasks[InTaskId].Node;
	const UPCGSettings* Settings = Node ? Node->GetSettings() : nullptr;
	const UPCGHiGenGridSizeSettings* GridSizeSettings = Cast<UPCGHiGenGridSizeSettings>(Settings);

	uint32 GridSize = PCGHiGenGrid::UninitializedGridSize();

	// Grid Size nodes in the top graph set the execution grid level.
	if (GridSizeSettings && GridSizeSettings->bEnabled && bTopLevelGraph)
	{
		GridSize = FMath::Min(GenerationDefaultGrid, GridSizeSettings->GetScaledGridSize());
	}
	else
	{
		if (InCompiledTasks[InTaskId].Inputs.IsEmpty())
		{
			if (bTopLevelGraph)
			{
				// Tasks with no inputs in top graph get prescribed the default generation grid.
				GridSize = GenerationDefaultGrid;
			}
			else
			{
				// Tasks with no inputs in a subgraph should execute on the same grid as the subgraph node task.
				check(InCompiledTasks[InTaskId].ParentId != InvalidPCGTaskId);
				GridSize = CalculateGridRecursive(InCompiledTasks[InTaskId].ParentId, GenerationDefaultGrid, InStackContext, InCompiledTasks, InOutTaskGenerationGrid);
			}
		}
		else
		{
			// This task has inputs. Grid of this task is minimum of all input grids. We can link in data from a larger grid, but
			// not from a finer grid (this goes against hierarchy).
			GridSize = PCGHiGenGrid::UnboundedGridSize();
			for (FPCGGraphTaskInput InputTask : InCompiledTasks[InTaskId].Inputs)
			{
				const uint32 InputGridSize = CalculateGridRecursive(InputTask.TaskId, GenerationDefaultGrid, InStackContext, InCompiledTasks, InOutTaskGenerationGrid);
				if (PCGHiGenGrid::IsValidGridSize(InputGridSize))
				{
					GridSize = FMath::Min(InputGridSize, GridSize);
				}
			}
		}
	}

	ensure(GridSize != PCGHiGenGrid::UninitializedGridSize());

	InOutTaskGenerationGrid[InTaskId] = GridSize;

	return GridSize;
}

bool FPCGGraphCompiler::CalculateStaticallyActiveRecursive(FPCGTaskId InTaskId, const TArray<FPCGGraphTask>& InCompiledTasks, TMap<FPCGTaskId, bool>& InOutTaskIdToActiveFlag)
{
	if (const bool* bEntry = InOutTaskIdToActiveFlag.Find(InTaskId))
	{
		return *bEntry;
	}

	const UPCGNode* Node = InCompiledTasks[InTaskId].Node;

	// Nodes within subgraphs - if the subgraph node is inactive then all tasks within the subgraph are inactive.
	if (InCompiledTasks[InTaskId].ParentId != InvalidPCGTaskId)
	{
		const bool bParentActive = CalculateStaticallyActiveRecursive(InCompiledTasks[InTaskId].ParentId, InCompiledTasks, InOutTaskIdToActiveFlag);
		// With respect to the input node in a static subgraph, it has to use the same value as its parent since there is no direct edge between the subgraph node and this input node.
		if (!bParentActive || (Node && Node->GetSettings() && Node->GetSettings()->IsA<UPCGGraphInputOutputSettings>()))
		{
			InOutTaskIdToActiveFlag.Add(InTaskId, bParentActive);
			return bParentActive;
		}
	}

	if (!Node)
	{
		InOutTaskIdToActiveFlag.Add(InTaskId, true);
		return true;
	}

	// For static subgraphs, the second time this node is seen, it has a trivial task and no proper edge, which will trip the static validation below
	// This should basically forward the same information as the original node - which is conveniently the first input on that task.
	if (UPCGSubgraphSettings* SubgraphSettings = Cast<UPCGSubgraphSettings>(Node->GetSettings()))
	{
		FPCGTaskId TentativeSubgraphInputId = InCompiledTasks[InTaskId].Inputs.IsEmpty() ? InvalidPCGTaskId : InCompiledTasks[InTaskId].Inputs[0].TaskId;

		if (!SubgraphSettings->IsDynamicGraph() && TentativeSubgraphInputId != InvalidPCGTaskId && InCompiledTasks[TentativeSubgraphInputId].Node == Node)
		{
			const bool bSubgraphInputActive = CalculateStaticallyActiveRecursive(TentativeSubgraphInputId, InCompiledTasks, InOutTaskIdToActiveFlag);

			InOutTaskIdToActiveFlag.Add(InTaskId, bSubgraphInputActive);
			return bSubgraphInputActive;
		}
	}

	TArray<FName, TInlineAllocator<8>> PinsRequiringActiveConnection;

	// Three relevant types of input pins - required, non-advanced and advanced. This tracks if we encounter the second category.
	bool bHasAnyNonAdvancedPins = false;

	for (UPCGPin* InputPin : Node->GetInputPins())
	{
		if (!InputPin)
		{
			continue;
		}

		if (!InputPin->Properties.IsAdvancedPin())
		{
			bHasAnyNonAdvancedPins = true;

			if (Node->IsInputPinRequiredByExecution(InputPin))
			{
				PinsRequiringActiveConnection.AddUnique(InputPin->Properties.Label);
			}
		}
	}
	
	bool bHasAnyNonAdvancedInputs = false;
	bool bHasAnyActiveNonAdvancedInput = false;

	for (const FPCGGraphTaskInput& Input : InCompiledTasks[InTaskId].Inputs)
	{
		// Only non-advanced input pins play a part in determining active/inactive state.
		if (Input.DownstreamPin.IsSet() && Input.DownstreamPin.GetValue().IsAdvancedPin())
		{
			continue;
		}

		bHasAnyNonAdvancedInputs = true;

		// Default to tasks being active unless proved otherwise.
		bool bInputActive = true;

		// If we are connected to an upstream node, evaluate if the output pin is active.
		if (Input.UpstreamPin.IsSet())
		{
			const UPCGNode* UpstreamNode = InCompiledTasks[Input.TaskId].Node;
			if (const UPCGSettings* UpstreamSettings = UpstreamNode ? UpstreamNode->GetSettings() : nullptr)
			{
				bInputActive &= UpstreamSettings->IsPinStaticallyActive(Input.UpstreamPin.GetValue().Label);
			}
		}

		if (bInputActive)
		{
			bInputActive &= CalculateStaticallyActiveRecursive(Input.TaskId, InCompiledTasks, InOutTaskIdToActiveFlag);
		}

		if (bInputActive)
		{
			bHasAnyActiveNonAdvancedInput = true;

			if (Input.DownstreamPin.IsSet())
			{
				// Register received input on this pin.
				PinsRequiringActiveConnection.Remove(Input.DownstreamPin.GetValue().Label);
			}
		}
	}

	const UPCGSettings* Settings = Node->GetSettings();
	const bool bCanCullIfUnwired = Settings && Settings->CanCullTaskIfUnwired();

	bool bActive = true;

	if (!PinsRequiringActiveConnection.IsEmpty())
	{
		// If PinsRequiringActiveConnection is not empty then we did not find an input for each required pin.
		bActive = false;
	}
	else if (bCanCullIfUnwired)
	{
		// Cull if we have non-advanced pins but we don't have any active non-advanced inputs.
		bActive = !(bHasAnyNonAdvancedPins && !bHasAnyActiveNonAdvancedInput);
	}
	else if (bHasAnyNonAdvancedInputs)
	{
		// This task is allowed to be unwired, so we have non-advanced inputs and they're all inactive - basically
		// all upstream inputs are inactive which forces this task to be inactive.
		bActive = bHasAnyActiveNonAdvancedInput || InCompiledTasks[InTaskId].Inputs.IsEmpty();
	}

	InOutTaskIdToActiveFlag.Add(InTaskId, bActive);

	return bActive;
}

void FPCGGraphCompiler::CullTasksStaticInactive(TArray<FPCGGraphTask>& InOutCompiledTasks)
{
	if (InOutCompiledTasks.IsEmpty())
	{
		return;
	}

	TMap<FPCGTaskId, bool> NodeIdToActiveFlag;
	// First task is input node task which is active
	NodeIdToActiveFlag.Add(InOutCompiledTasks[0].NodeId, true);

	for (int i = 1; i < InOutCompiledTasks.Num(); ++i)
	{
		// Results of each call memoized via NodeIdToActiveFlag.
		CalculateStaticallyActiveRecursive(InOutCompiledTasks[i].NodeId, InOutCompiledTasks, NodeIdToActiveFlag);
	}

	auto ShouldCull = [&NodeIdToActiveFlag](const FPCGGraphTask& InTask)
	{
		const bool* bActive = NodeIdToActiveFlag.Find(InTask.NodeId);
		return bActive && !*bActive;
	};
	CullTasks(InOutCompiledTasks, /*bAddPassthroughWires=*/false, ShouldCull);
}

void FPCGGraphCompiler::CullTasks(TArray<FPCGGraphTask>& InOutCompiledTasks, bool bAddPassthroughWires, TFunctionRef<bool(const FPCGGraphTask&)> CullTask, TArray<FPCGTaskId>* OutOptionalOldIdToNewIdMap)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompiler::CullTasks);

	// Check that we have more than just the input task to work with.
	if (InOutCompiledTasks.Num() < 2)
	{
		return;
	}

	TArray<FPCGTaskId> OldIdToNewId;
	OldIdToNewId.SetNumUninitialized(InOutCompiledTasks.Num());

	// Mark culled tasks by remapping to INDEX_NONE. First task is input task and is never culled.
	OldIdToNewId[0] = 0;
	for (int TaskIndex = 1; TaskIndex < InOutCompiledTasks.Num(); ++TaskIndex)
	{
		OldIdToNewId[TaskIndex] = CullTask(InOutCompiledTasks[TaskIndex]) ? InvalidPCGTaskId : 0;
	}

	// Optionally add wires that bypass culled nodes.
	if (bAddPassthroughWires)
	{
		for (int TaskIndex = 1; TaskIndex < InOutCompiledTasks.Num(); ++TaskIndex)
		{
			FPCGGraphTask& Task = InOutCompiledTasks[TaskIndex];
			if (!Task.Node || Task.Node->GetInputPins().IsEmpty())
			{
				continue;
			}

			int InputIndex = 0;
			while (InputIndex < Task.Inputs.Num())
			{
				const FPCGTaskId InputTaskId = Task.Inputs[InputIndex].TaskId;

				// Is node culled?
				if (OldIdToNewId[InputTaskId] == InvalidPCGTaskId)
				{
					const FPCGGraphTask& CulledInputTask = InOutCompiledTasks[InputTaskId];
					if (CulledInputTask.Node && CulledInputTask.Node->GetInputPins().Num() > 1)
					{
						ensureMsgf(false, TEXT("Task culling currently only supports nodes with a single input pin which are trivial to unwire."));
						continue;
					}

					// Upstream node was culled. To preserve order, insert new wires just after the wire to be culled--which will be removed later.
					Task.Inputs.Insert(CulledInputTask.Inputs, InputIndex + 1);

					const int NewWireCount = CulledInputTask.Inputs.Num();
					for (int I = 0; I < NewWireCount; ++I)
					{
						Task.Inputs[InputIndex + 1 + I].DownstreamPin = Task.Inputs[InputIndex].DownstreamPin;
					}

					// Skip evaluating the newly wired inputs and increment
					InputIndex += NewWireCount;
				}

				++InputIndex;
			}
		}
	}

	// Remove all culled tasks by compacting the task array. Never cull first (Input) task.
	int WriteIndex = 1;
	int ReadIndex = 1;
	while (ReadIndex < InOutCompiledTasks.Num())
	{
		// If not culled, then move the task to it's final remapped position in the task array.
		if (OldIdToNewId[ReadIndex] != InvalidPCGTaskId)
		{
			if (WriteIndex != ReadIndex)
			{
				InOutCompiledTasks[WriteIndex] = MoveTemp(InOutCompiledTasks[ReadIndex]);
				InOutCompiledTasks[WriteIndex].NodeId = WriteIndex;
			}

			OldIdToNewId[ReadIndex] = WriteIndex;
			++WriteIndex;
		}

		++ReadIndex;
	}

	InOutCompiledTasks.SetNum(WriteIndex);

	for (FPCGGraphTask& Task : InOutCompiledTasks)
	{
		// Remap input task IDs, and remove edges that connect to culled nodes.
		for (int InputIndex = Task.Inputs.Num() - 1; InputIndex >= 0; --InputIndex)
		{
			const FPCGTaskId InputTaskId = Task.Inputs[InputIndex].TaskId;
			const FPCGTaskId Remap = OldIdToNewId[InputTaskId];
			if (Remap != InvalidPCGTaskId)
			{
				Task.Inputs[InputIndex].TaskId = Remap;
			}
			else
			{
				Task.Inputs.RemoveAt(InputIndex);
			}
		}

		// Remap parent ID if there tasks is in a child scope.
		if (Task.ParentId != InvalidPCGTaskId)
		{
			const FPCGTaskId RemappedParentId = OldIdToNewId[Task.ParentId];
			// Parent task should not have been culled.
			ensure(RemappedParentId != InvalidPCGTaskId);

			// Write the remapped ID - even if it's invalid/INDEX_NONE. Hanging parent IDs can cause issues elsewhere.
			Task.ParentId = RemappedParentId;
		}
	}

	// Return the mapping if caller requested it.
	if (OutOptionalOldIdToNewIdMap)
	{
		*OutOptionalOldIdToNewIdMap = MoveTemp(OldIdToNewId);
	}
}

void FPCGGraphCompiler::AddReferencedObjects(FReferenceCollector& Collector)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCPGGraphCompiler::AddReferencedObjects);

	PCG::TSharedScopeLock ReadLock(Cache.GraphToTaskMapLock);

	for (TPair<UPCGGraph*, TMap<uint32, TArray<TObjectPtr<UPCGComputeGraph>>>>& GraphToGridSizes : Cache.TopGraphToComputeGraphMap)
	{
		for (TPair<uint32, TArray<TObjectPtr<UPCGComputeGraph>>>& GridSizeToComputeGraphs : GraphToGridSizes.Value)
		{
			for (TObjectPtr<UPCGComputeGraph>& ComputeGraph : GridSizeToComputeGraphs.Value)
			{
				Collector.AddReferencedObject(ComputeGraph);
			}
		}
	}
}

bool FPCGGraphCompiler::VisitTasksInExecutionOrder(const TArray<FPCGGraphTask>& InTasks, const TMap<FPCGTaskId, TArray<FPCGTaskId>>& InTaskToTaskSuccessors, const TFunction<bool(FPCGTaskId)>& InVisitor)
{
	// Populate initial sets of tasks that are ready to consume vs ones that currently blocked.
	TArray<FPCGTaskId> ReadyTaskIds;
	TSet<FPCGTaskId> RemainingTaskIds;
	ReadyTaskIds.Reserve(InTasks.Num());
	RemainingTaskIds.Reserve(InTasks.Num());

	// Sort tasks into those that don't have inputs (ready tasks) vs those that cannot execute yet (remaining).
	for (FPCGTaskId TaskId = 0; TaskId < InTasks.Num(); ++TaskId)
	{
		if (InTasks[TaskId].Inputs.IsEmpty())
		{
			ReadyTaskIds.Add(TaskId);
		}
		else
		{
			RemainingTaskIds.Add(TaskId);
		}
	}

	// Loop until we have consumed all tasks.
	while (!ReadyTaskIds.IsEmpty() || !RemainingTaskIds.IsEmpty())
	{
		FPCGTaskId ReadyTaskId = ReadyTaskIds.Pop();

		if (!InVisitor(ReadyTaskId))
		{
			return false;
		}

		// Queue up any successors that are ready to go.
		const TArray<FPCGTaskId>* Successors = InTaskToTaskSuccessors.Find(ReadyTaskId);
		if (!Successors)
		{
			continue;
		}

		for (FPCGTaskId Successor : *Successors)
		{
			const bool bSuccessorQueued = ReadyTaskIds.Contains(Successor);

			// All successors should either be already queued, or waiting to be queued.
			check(bSuccessorQueued || RemainingTaskIds.Contains(Successor));

			if (!bSuccessorQueued)
			{
				bool bSuccessorReady = true;
				for (const FPCGGraphTaskInput& Input : InTasks[Successor].Inputs)
				{
					if (ReadyTaskIds.Contains(Input.TaskId) || RemainingTaskIds.Contains(Input.TaskId))
					{
						bSuccessorReady = false;
						break;
					}
				}

				if (bSuccessorReady)
				{
					ReadyTaskIds.Add(Successor);
					RemainingTaskIds.Remove(Successor);
				}
			}
		}
	}

	return true;
}

void FPCGGraphCompiler::PostCullStackCleanup(TArray<FPCGGraphTask>& InCompiledTasks, FPCGStackContext& InOutStackContext)
{
	// Build set of stack IDs used by tasks. Using array as set is likely small.
	TArray<int32> ActiveStackIDs;
	for (FPCGGraphTask& Task : InCompiledTasks)
	{
		ActiveStackIDs.AddUnique(Task.StackIndex);
	}

	if (ActiveStackIDs.Num() < InOutStackContext.GetNumStacks())
	{
		// Prepare cumulative counts for stack ID remapping.
		TArray<int> RemovedCounts;
		RemovedCounts.SetNumZeroed(InOutStackContext.GetNumStacks());

		// Remove any stacks that are inactive.
		TArray<FPCGStack>& Stacks = InOutStackContext.GetStacksMutable();
		for (int StackIndex = Stacks.Num() - 1; StackIndex >= 0; --StackIndex)
		{
			if (!ActiveStackIDs.Contains(StackIndex))
			{
				Stacks.RemoveAt(StackIndex);
				RemovedCounts[StackIndex] = 1;
			}
		}

		// Accumulate removed counts.
		for (int Index = 1; Index < RemovedCounts.Num(); ++Index)
		{
			RemovedCounts[Index] += RemovedCounts[Index - 1];
		}

		// Remap stack IDs.
		for (FPCGGraphTask& Task : InCompiledTasks)
		{
			if (ensure(RemovedCounts.IsValidIndex(Task.StackIndex)))
			{
				Task.StackIndex -= RemovedCounts[Task.StackIndex];
			}
		}
	}
}

void FPCGGraphCompiler::CalculateDynamicActivePinDependencies(FPCGTaskId InTaskId, TArray<FPCGGraphTask>& InOutCompiledTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompiler::CalculateDynamicActivePinDependencies);

	if (!ensure(InOutCompiledTasks.IsValidIndex(InTaskId)))
	{
		return;
	}

	auto LogTooManyOutputPinsWarning = [](const UPCGNode* InUpstreamNode)
	{
		UE_LOGFMT(LogPCG, Warning, "Number of output pins ({OutputPinCount}) on node {NodeName} exceeded maximum limit ({Limit}) for dynamic node culling. "
			"Dynamic culling will not function for this node's output pins.",
			InUpstreamNode->GetOutputPins().Num(), InUpstreamNode->GetName(), PCGPinIdHelpers::MaxOutputPins);
	};

	const UPCGNode* Node = InOutCompiledTasks[InTaskId].Node;

	// First if there are any required pins, build the pin dependencies from these. We have an array of inputs, each one can
	// correspond to a pin. Use a map to compile an expression for each input pin label.
	TMap<FName, FPCGPinDependencyExpression> InputPinLabelToPinDependency;

	for (const FPCGGraphTaskInput& Input : InOutCompiledTasks[InTaskId].Inputs)
	{
		if (!Input.DownstreamPin.IsSet())
		{
			continue;
		}

		if (Node)
		{
			const UPCGPin* InputPin = Node->GetInputPin(Input.DownstreamPin.GetValue().Label);

			// Consider only primary input pins in this pass.
			if (!InputPin || !Node->IsInputPinRequiredByExecution(InputPin))
			{
				continue;
			}
		}
		else
		{
			// For ComputeGraph tasks (Node=nullptr), the node's settings are unavailable so we cannot call
			// IsInputPinRequiredByExecution. Fall back to the downstream pin's own PinStatus: pins explicitly
			// marked Required (e.g. exec dep pins with bExecutionDependencyRequired=true) are treated as
			// required by execution. Normal/Advanced pins fall through to the disjunction pass below.
			if (!Input.DownstreamPin.GetValue().IsRequiredPin())
			{
				continue;
			}
		}

		const UPCGNode* UpstreamNode = InOutCompiledTasks[Input.TaskId].Node;
		if (!UpstreamNode)
		{
			continue;
		}

		if (UpstreamNode->GetOutputPins().Num() > PCGPinIdHelpers::MaxOutputPins)
		{
			LogTooManyOutputPinsWarning(UpstreamNode);
			continue;
		}

		const int PinIndex = UpstreamNode->GetOutputPins().IndexOfByPredicate([&Input](const UPCGPin* InPin)
		{
			return InPin->Properties == Input.UpstreamPin;
		});

		if (PinIndex != INDEX_NONE)
		{
			check(PinIndex < PCGPinIdHelpers::MaxOutputPins);

			FPCGPinDependencyExpression& PinDependency = InputPinLabelToPinDependency.FindOrAdd(Input.DownstreamPin.GetValue().Label);
			PinDependency.AddPinDependency(PCGPinIdHelpers::NodeIdAndPinIndexToPinId(Input.TaskId, PinIndex));
		}
	}

	// Result expression. Conjunction of disjunctions of pin IDs that are required to be active for this task to be active.
	// Example - keep task if: UpstreamPin0Active && (UpstreamPin1Active || UpstreamPin2Active)
	FPCGPinDependencyExpression PinDependency;

	if (!InputPinLabelToPinDependency.IsEmpty())
	{
		// If we have registered pin dependencies from the first pass (required pins) then we have can compose these for our pin
		// dependency expression. Build conjunction from the per-pin disjunctions.
		for (TPair<FName, FPCGPinDependencyExpression>& PinExpression : InputPinLabelToPinDependency)
		{
			PinDependency.AppendUsingConjunction(PinExpression.Value);
		}
	}
	else
	{
		// In the case of output nodes, advanced pins shouldn't be ignored as it can and will prevent culling in some instances
		const bool bTreatAdvancedPinsAsNormal = (Node && Cast<UPCGGraphInputOutputSettings>(Node->GetSettings()) && !Cast<UPCGGraphInputOutputSettings>(Node->GetSettings())->IsInput());

		// If we don't have any dependent pins logged by now then there are no required pins (note that a node that does not have task
		// inputs for required pins will be statically culled in an earlier compilation step). In which case we'll be active if *any* input
		// is active. We build a disjunction that expresses this.
		for (const FPCGGraphTaskInput& Input : InOutCompiledTasks[InTaskId].Inputs)
		{
			if (Input.DownstreamPin.IsSet() && Input.DownstreamPin.GetValue().IsAdvancedPin() && !bTreatAdvancedPinsAsNormal)
			{
				// Advanced input pins never participate in keeping node active.
				continue;
			}
		
			if (Input.UpstreamPin.IsSet())
			{
				// Input connection is via node pins.
				const UPCGNode* UpstreamNode = InOutCompiledTasks[Input.TaskId].Node;
				if (!UpstreamNode)
				{
					// Upstream is a ComputeGraph task (Node=nullptr). We cannot resolve a pin index from its virtual output pins, so fall back
					// to a pin-less dependency. This ensures that if the ComputeGraph task is dynamically culled, the culling propagates to
					// this task as well (CullInactiveDownstreamNodes adds NodeIdToPinId for culled tasks).
					PinDependency.AddPinDependency(PCGPinIdHelpers::NodeIdToPinId(Input.TaskId));
				}
				else
				{
					if (UpstreamNode->GetOutputPins().Num() > PCGPinIdHelpers::MaxOutputPins)
					{
						LogTooManyOutputPinsWarning(UpstreamNode);
						continue;
					}

					const int PinIndex = UpstreamNode->GetOutputPins().IndexOfByPredicate([&Input](const UPCGPin* InPin)
					{
						return InPin->Properties.Label == Input.UpstreamPin.GetValue().Label;
					});

					if (PinIndex != INDEX_NONE)
					{
						check(PinIndex < PCGPinIdHelpers::MaxOutputPins);

						PinDependency.AddPinDependency(PCGPinIdHelpers::NodeIdAndPinIndexToPinId(Input.TaskId, PinIndex));
					}
				}
			}
			else
			{
				// No associated pin, use a special pin ID for a pin-less dependency.
				PinDependency.AddPinDependency(PCGPinIdHelpers::NodeIdToPinId(Input.TaskId));
			}
		}
	}

	InOutCompiledTasks[InTaskId].PinDependency = MoveTemp(PinDependency);
}

TArray<FPCGGraphTask> FPCGGraphCompiler::GetCompiledTasks(UPCGGraph* InGraph, uint32 GenerationGridSize, FPCGStackContext& OutStackContext, bool bIsTopGraph)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompiler::GetCompiledTasks);
	UE_LOG_CONTEXT("PCGGraph", FSoftObjectPath(InGraph));
	
#if !WITH_EDITOR
	// In standalone builds, try using cooked tasks first.
	if (InGraph)
	{
		if (UPCGGraphCompilationData* CookedData = InGraph->GetCookedCompilationData())
		{
			FPCGGraphTasks* CookedTasks = CookedData->Tasks.Find(GenerationGridSize);
			const FPCGStackContext* CookedStackContext = CookedData->StackContexts.Find(GenerationGridSize);

			if (CookedTasks && CookedStackContext)
			{
				for (FPCGGraphTask& GraphTask : CookedTasks->GraphTasks)
				{
					GraphTask.LoadCookedData();
				}

				OutStackContext = *CookedStackContext;
				return CookedTasks->GraphTasks;
			}
		}
	}
#endif

	TArray<FPCGGraphTask> CompiledTasks;

	if (bIsTopGraph)
	{
		// Always try to compile
		CompileTopGraph(InGraph, GenerationGridSize);

		// Get compiled tasks in a threadsafe way
		PCG::TSharedScopeLock ReadLock(Cache.GraphToTaskMapLock);
		const TMap<uint32, TArray<FPCGGraphTask>>* GridSizeToCompiledGraph = Cache.TopGraphToTaskMap.Find(InGraph);
		const TArray<FPCGGraphTask>* Tasks = GridSizeToCompiledGraph ? GridSizeToCompiledGraph->Find(GenerationGridSize) : nullptr;

		const TMap<uint32, FPCGStackContext>* GridSizeToStackContext = Cache.TopGraphToStackContextMap.Find(InGraph);
		const FPCGStackContext* StackContext = GridSizeToStackContext ? GridSizeToStackContext->Find(GenerationGridSize) : nullptr;

		// Should have either found both or neither.
		ensure(!Tasks == !StackContext);

		if (Tasks && StackContext)
		{
			CompiledTasks = *Tasks;
			OutStackContext = *StackContext;
		}
	}
	else
	{
		// Always try to compile
		Compile(InGraph);

		// Get compiled tasks in a threadsafe way
		PCG::TSharedScopeLock ReadLock(Cache.GraphToTaskMapLock);
		if (TArray<FPCGGraphTask>* Tasks = Cache.GraphToTaskMap.Find(InGraph))
		{
			CompiledTasks = *Tasks;
			OutStackContext = Cache.GraphToStackContextMap[InGraph];
		}
	}

	return CompiledTasks;
}

void FPCGGraphCompiler::OffsetNodeIds(TArray<FPCGGraphTask>& Tasks, FPCGTaskId Offset, FPCGTaskId ParentId)
{
	for (FPCGGraphTask& Task : Tasks)
	{
		Task.NodeId += Offset;

		if (Task.ParentId == InvalidPCGTaskId)
		{
			Task.ParentId = ParentId;
		}
		else
		{
			Task.ParentId += Offset;
		}

		for (FPCGGraphTaskInput& Input : Task.Inputs)
		{
			Input.TaskId += Offset;
		}

		Task.PinDependency.OffsetNodeIds(Offset);
	}
}

void FPCGGraphCompiler::CompileTopGraph(UPCGGraph* InGraph, uint32 GenerationGridSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompiler::CompileTopGraph);
	bool bAlreadyCached = false;

	{
		PCG::TSharedScopeLock ReadLock(Cache.GraphToTaskMapLock);
		const TMap<uint32, TArray<FPCGGraphTask>>* GridSizeToCompiledGraph = Cache.TopGraphToTaskMap.Find(InGraph);
		bAlreadyCached = GridSizeToCompiledGraph && GridSizeToCompiledGraph->Contains(GenerationGridSize);
	}

	if (bAlreadyCached)
	{
		return;
	}

	UE_LOGF(LogPCG, Verbose, "FPCGGraphCompiler::CompileTopGraph '%ls' grid: %u", *InGraph->GetName(), GenerationGridSize);

	// Build from non-top tasks
	FPCGStackContext StackContext;
	TArray<FPCGGraphTask> CompiledTasks = GetCompiledTasks(InGraph, GenerationGridSize, StackContext, /*bIsTopGraph=*/false);

	// Check that the compilation was valid
	if (CompiledTasks.Num() == 0)
	{
		return;
	}

	if (PCGGraphCompiler::CVarEnableTaskStaticCulling.GetValueOnAnyThread())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TaskStaticCulling);

		// Remove reroute nodes before execution grid setup, as grid linkages need final nodes to connect from/to.
		// Note that reroutes nodes won't be culled if they have no inbound edges or are marked for debug.
		auto RerouteCullTasks = [](const FPCGGraphTask& InTask) -> bool
		{
			if (const UPCGRerouteSettings* Settings = InTask.Node ? Cast<UPCGRerouteSettings>(InTask.Node->GetSettings()) : nullptr)
			{
				return InTask.Node->HasInboundEdges() && !Settings->bDebug;
			}
			else
			{
				return false;
			}
		};

		CullTasks(CompiledTasks, /*bAddPassthroughWires=*/true, RerouteCullTasks);

		// Cull inactive branches downstream of branch nodes with static selection values.
		CullTasksStaticInactive(CompiledTasks);

		// For hierarchical generation resolve the execution grid for each task and cull any tasks that won't execute.
		// TODO - we could add an else branch for higen disabled that culls the grid size nodes.
		if (InGraph->IsHierarchicalGenerationEnabled() && GenerationGridSize != PCGHiGenGrid::UninitializedGridSize())
		{
			const uint32 DefaultGrid = InGraph->GetDefaultGridSize();

			// Propagate grid size nodes through the graph to determine which grid size each task should execute on.
			TArray<uint32> TaskGenerationGrid;
			TaskGenerationGrid.SetNumZeroed(CompiledTasks.Num());
			ResolveGridSizes(GenerationGridSize, CompiledTasks, StackContext, DefaultGrid, TaskGenerationGrid);

			// Create linkage tasks for edges that cross from large grid to small grid tasks.
			CreateGridLinkages(InGraph, GenerationGridSize, TaskGenerationGrid, CompiledTasks, StackContext, bIsCooking);

			// Cull any task that should not execute on the current grid.
			CullTasks(CompiledTasks, /*bAddPassthroughWires=*/false, [GenerationGridSize, &TaskGenerationGrid](const FPCGGraphTask& InTask)
			{
				const uint32 TaskGridSize = TaskGenerationGrid[InTask.NodeId];
				if (TaskGridSize == PCGHiGenGrid::UninitializedGridSize())
				{
					// Don't cull if no grid size set.
					return false;
				}

				if (InTask.Element && InTask.Element->IsGridLinkage()) // Non-cooking case
				{
					PCGGraphExecutor::FPCGGridLinkageElement* LinkElement = static_cast<PCGGraphExecutor::FPCGGridLinkageElement*>(InTask.Element.Get());
					return LinkElement->GetFromGridSize() != GenerationGridSize && LinkElement->GetToGridSize() != GenerationGridSize;
				}
				else if (const UPCGGridLinkageSettings* GridLinkSettings = Cast<UPCGGridLinkageSettings>(InTask.CookedSettings)) // Cooking case
				{
					return GridLinkSettings->FromGrid != GenerationGridSize && GridLinkSettings->ToGrid != GenerationGridSize;
				}
				else
				{
					return TaskGridSize != GenerationGridSize;
				}
			});
		}
	}

	// TODO: GPU execution is editor only for now. Need to cook kernels for standalone.
#if WITH_EDITOR
	if (PCGGraphCompiler::CVarEnableGPUExecution.GetValueOnAnyThread())
	{
		FPCGGraphCompilerGPU::CreateGPUNodes(*this, InGraph, GenerationGridSize, CompiledTasks);
	}
	else
#endif
	{
		// GPU not supported. Cull all the GPU compatible nodes without adding passthrough wires. This is destructive!
		// TODO: In the future when we have nodes that can target both CPU and GPU, this could just force CPU (and throw
		// errors if nodes strictly require GPU).
		int CountBefore = CompiledTasks.Num();

		CullTasks(CompiledTasks, /*bAddPassthroughWires=*/false, [](const FPCGGraphTask& InTask)
		{
			const UPCGSettings* Settings = InTask.Node ? InTask.Node->GetSettings() : nullptr;
			return Settings && Settings->ShouldExecuteOnGPU();
		});

		if (CountBefore > CompiledTasks.Num())
		{
			UE_LOGFMT(LogPCG, Warning, "One or more GPU nodes were culled from the graph!");
		}
	}

	// Post culling - remove any stacks that are no longer part of execution. Besides being tidy this also helps
	// debug tools discern which stacks were executed or not.
	PostCullStackCleanup(CompiledTasks, StackContext);

	// To feed dynamic culling at execution time, build list of upstream pins that we depend on (if all of these pins
	// are determined to be inactive at execution time, then the task will be deactivated). An empty list means the
	// task will never be deactivated and will always execute.
	// TODO: Expand pin dependencies to pure branches that aren't directly downstream - nodes that have no side effects
	// and feed into a branch can also be culled if the branch is culled.
	// TODO: Pin dependencies should be transitive across nodes. If a node is dependent on a single upstream node, it could
	// likely take the pin dependencies from the upstream node, which should save iterations in the dynamic culling code.
	for (int TaskIndex = 0; TaskIndex < CompiledTasks.Num(); ++TaskIndex)
	{
		// Result is written directly to tasks.
		CalculateDynamicActivePinDependencies(CompiledTasks[TaskIndex].NodeId, CompiledTasks);
	}

	// Also we keep track of all the inputs of each task. If the input is in SeenInputs it means it is used multiple times, so mark it as is.
	TSet<PCGGraphCompiler::FTaskInputOrigin> SeenInputs;
	for (int TaskIndex = 0; TaskIndex < CompiledTasks.Num(); ++TaskIndex)
	{
		for (FPCGGraphTaskInput& TaskInput : CompiledTasks[TaskIndex].Inputs)
		{
			PCGGraphCompiler::FTaskInputOrigin InputOrigin(&TaskInput);
			if (!InputOrigin.TaskInput || !InputOrigin.TaskInput->bProvideData)
			{
				continue;
			}

			if (PCGGraphCompiler::FTaskInputOrigin* It = SeenInputs.Find(InputOrigin))
			{
				check(It->TaskInput);
				It->TaskInput->bIsUsedMultipleTimes = true;
				TaskInput.bIsUsedMultipleTimes = true;
			}
			else
			{
				// @todo_pcg: This should not have that big of a cost, as most InputTasks do not already have an element when compiling.
				// The virtual call will then not be often called. However, it would be cleaner to have a boolean state on the TaskInput 
				// that is set to true when we know we cross boundaries.
				// By definition, if we cross a boundary, then an input will be used multiple times.
				const FPCGGraphTask* InputTask = CompiledTasks.IsValidIndex(TaskInput.TaskId) ? &CompiledTasks[TaskInput.TaskId] : nullptr;
				const bool bCrossBoundaries = InputTask && InputTask->Element && InputTask->Element->IsGridLinkage();
				TaskInput.bIsUsedMultipleTimes = bCrossBoundaries;
				SeenInputs.Emplace(std::move(InputOrigin));
			}
		}
	}

	const int TaskNum = CompiledTasks.Num();
	const FPCGTaskId PreExecuteTaskId = FPCGTaskId(TaskNum);
	const FPCGTaskId PostExecuteTaskId = PreExecuteTaskId + 1;

	FPCGGraphTask& PreExecuteTask = CompiledTasks.Emplace_GetRef();
	PreExecuteTask.ElementSource = EPCGElementSource::Trivial;
	PreExecuteTask.NodeId = PreExecuteTaskId;

	for (int TaskIndex = 0; TaskIndex < TaskNum; ++TaskIndex)
	{
		FPCGGraphTask& Task = CompiledTasks[TaskIndex];
		const UPCGSettings* Settings = Task.Node ? Task.Node->GetSettings() : nullptr;
		const bool bRequiresDataFromPreTask = Settings && Settings->RequiresDataFromPreTask();

		// Tasks could have already been hooked to the pre task of the subgraph, so don't re-hook it if that was the case.
		if (!Task.bWasHookedToPreTask && (Task.Inputs.IsEmpty() || bRequiresDataFromPreTask))
		{
			Task.Inputs.Emplace(PreExecuteTaskId, /*InUpstreamPin=*/FPCGGraphTaskInput::NoPin, /*InDownstreamPin=*/FPCGGraphTaskInput::NoPin, bRequiresDataFromPreTask);
			Task.bWasHookedToPreTask = true;
		}

		Task.CompiledTaskId = Task.NodeId;
	}

	FPCGGraphTask& PostExecuteTask = CompiledTasks.Emplace_GetRef();
	PostExecuteTask.ElementSource = EPCGElementSource::TrivialPostGraph;
	PostExecuteTask.NodeId = PostExecuteTaskId;

	// Find end nodes, e.g. all nodes that have no successors.
	// In our representation we don't have this, so find it out by going backwards
	// Note: this works because there is a weak ordering on the tasks such that
	// a successor task is always after its predecessors
	TSet<FPCGTaskId> TasksWithSuccessors;
	const UPCGNode* GraphOutputNode = InGraph->GetOutputNode();

	for (int TaskIndex = TaskNum - 1; TaskIndex >= 0; --TaskIndex)
	{
		const FPCGGraphTask& Task = CompiledTasks[TaskIndex];
		if (!TasksWithSuccessors.Contains(Task.NodeId))
		{
			// For the post task, only the output node will provide data. 
			// It is necessary for any post generation task to get the content of the output node
			// and only this content.
			const bool bProvideData = Task.Node == GraphOutputNode;
			FPCGGraphTaskInput& TaskInput = PostExecuteTask.Inputs.Emplace_GetRef(Task.NodeId, /*InUpstreamPin=*/FPCGGraphTaskInput::NoPin, /*InDownstreamPin=*/FPCGGraphTaskInput::NoPin, bProvideData);
			TaskInput.bIsUsedMultipleTimes = false;
		}

		for (const FPCGGraphTaskInput& Input : Task.Inputs)
		{
			TasksWithSuccessors.Add(Input.TaskId);
		}
	}

	// Store back the results in the cache
	PCG::TUniqueScopeLock WriteLock(Cache.GraphToTaskMapLock);
	TMap<uint32, TArray<FPCGGraphTask>>& TasksPerGenerationGrid = Cache.TopGraphToTaskMap.FindOrAdd(InGraph);
	if (!TasksPerGenerationGrid.Contains(GenerationGridSize))
	{
		TasksPerGenerationGrid.Add(GenerationGridSize, MoveTemp(CompiledTasks));
	}

	TMap<uint32, FPCGStackContext>& StackContextPerGenerationGrid = Cache.TopGraphToStackContextMap.FindOrAdd(InGraph);
	if (!StackContextPerGenerationGrid.Contains(GenerationGridSize))
	{
		StackContextPerGenerationGrid.Add(GenerationGridSize, MoveTemp(StackContext));
	}
}

FPCGElementPtr FPCGGraphCompiler::GetSharedTrivialElement()
{
	{
		PCG::TSharedScopeLock ReadLock(SharedTrivialElementLock);

		if (SharedTrivialElement)
		{
			return SharedTrivialElement;
		}
	}

	PCG::TUniqueScopeLock WriteLock(SharedTrivialElementLock);

	if (!SharedTrivialElement)
	{
		SharedTrivialElement = MakeShared<FPCGTrivialElement>();
	}

	return SharedTrivialElement;
}

FPCGElementPtr FPCGGraphCompiler::GetSharedGatherElement()
{
	{
		PCG::TSharedScopeLock ReadLock(SharedGatherElementLock);

		if (SharedGatherElement)
		{
			return SharedGatherElement;
		}
	}

	PCG::TUniqueScopeLock WriteLock(SharedGatherElementLock);

	if (!SharedGatherElement)
	{
		SharedGatherElement = MakeShared<FPCGGatherElement>();
	}

	return SharedGatherElement;
}

FPCGElementPtr FPCGGraphCompiler::GetSharedTrivialPostGraphElement()
{
	{
		PCG::TSharedScopeLock ReadLock(SharedTrivialPostGraphElementLock);
		
		if (SharedTrivialPostGraphElement)
		{
			return SharedTrivialPostGraphElement;
		}
	}

	PCG::TUniqueScopeLock WriteLock(SharedTrivialPostGraphElementLock);
	
	if (!SharedTrivialPostGraphElement)
	{
		SharedTrivialPostGraphElement = MakeShared<FPCGTrivialElement>();
	}

	return SharedTrivialPostGraphElement;
}

void FPCGGraphCompiler::ClearCache()
{
	PCG::TUniqueScopeLock WriteLock(Cache.GraphToTaskMapLock);
	Cache.GraphToTaskMap.Reset();
	Cache.GraphToStackContextMap.Reset();
	Cache.TopGraphToTaskMap.Reset();
	Cache.TopGraphToStackContextMap.Reset();
	Cache.TopGraphToComputeGraphMap.Reset();
}

#if WITH_EDITOR
void FPCGGraphCompiler::NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType)
{
	if (InGraph && (ChangeType != EPCGChangeType::Cosmetic))
	{
		Cache.RemoveFromCache(InGraph);
	}
}

bool FPCGGraphCompiler::Recompile(UPCGGraph* InGraph, uint32 GenerationGridSize, bool bIsTopGraph)
{
	FPCGStackContext StackContextBefore;
	const TArray<FPCGGraphTask> TasksBefore = GetPrecompiledTasks(InGraph, GenerationGridSize, StackContextBefore, bIsTopGraph);

	// Need to manually purge as the graph compiler will not have gotten the change notification yet. Editor only.
	Cache.RemoveFromCache(InGraph);

	FPCGStackContext StackContextAfter;
	const TArray<FPCGGraphTask> TasksAfter = GetCompiledTasks(InGraph, GenerationGridSize, StackContextAfter, bIsTopGraph);

	bool bAllTasksEqual = TasksBefore.Num() == TasksAfter.Num();
	if (bAllTasksEqual)
	{
		for (int I = 0; I < TasksBefore.Num(); ++I)
		{
			bAllTasksEqual = bAllTasksEqual && TasksAfter[I].IsApproximatelyEqual(TasksBefore[I]);
		}
	}

	// Compiled result is compiled tasks + associated stacks. Compare both and return true if compiled result changes.
	return !bAllTasksEqual || (StackContextBefore != StackContextAfter);
}
#endif // WITH_EDITOR
