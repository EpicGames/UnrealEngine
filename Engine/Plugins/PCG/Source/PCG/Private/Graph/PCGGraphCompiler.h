// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Graph/PCGGraphTask.h"
#include "Graph/PCGStackContext.h"

class IPCGElement;
class UPCGComputeGraph;
class UPCGGraph;

/** Holds compilation results from FPCGGraphCompiler. */
struct FPCGGraphCompilerCache
{
	friend class FPCGGraphCompiler;
	friend class UPCGGraph;
#if WITH_EDITOR
	friend class FPCGGraphCompilerGPU;
#endif

public:
	UPCGComputeGraph* GetCompiledComputeGraph(const UPCGGraph* InGraph, uint32 GridSize, uint32 ComputeGraphIndex);

#if WITH_EDITOR
	void RemoveFromCache(UPCGGraph* InGraph);
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	void RemoveFromCacheRecursive(UPCGGraph* InGraph);
#endif // WITH_EDITOR

	mutable PCG::FSharedLock GraphToTaskMapLock;
	TMap<UPCGGraph*, TArray<FPCGGraphTask>> GraphToTaskMap;
	TMap<UPCGGraph*, FPCGStackContext> GraphToStackContextMap;

	// Top graphs are optimized for execution grid and store one set of compiled tasks per grid size.
	TMap<UPCGGraph*, TMap<uint32, TArray<FPCGGraphTask>>> TopGraphToTaskMap;
	TMap<UPCGGraph*, TMap<uint32, FPCGStackContext>> TopGraphToStackContextMap;

	// A single graph can have multiple compute graphs at each grid size, so we store multiple and refer to them by index when querying them.
	TMap<UPCGGraph*, TMap<uint32, TArray<TObjectPtr<UPCGComputeGraph>>>> TopGraphToComputeGraphMap;

#if WITH_EDITOR
	PCG::FLock GraphDependenciesLock;
	TMultiMap<UPCGGraph*, UPCGGraph*> GraphDependencies;
#endif // WITH_EDITOR
};

/** 
* This class compiles a graph into tasks and populates an FPCGGraphCompilerCache.
*/
class FPCGGraphCompiler
{
public:
	explicit FPCGGraphCompiler(bool bInIsCooking = false)
		: bIsCooking(bInIsCooking)
	{}

	void Compile(UPCGGraph* InGraph);
	TArray<FPCGGraphTask> GetCompiledTasks(UPCGGraph* InGraph, uint32 GenerationGridSize, FPCGStackContext& OutStackContext, bool bIsTopGraph = true);
	TArray<FPCGGraphTask> GetPrecompiledTasks(const UPCGGraph* InGraph, uint32 GenerationGridSize, FPCGStackContext& OutStackContext, bool bIsTopGraph = true) const;

	FPCGGraphCompilerCache& GetCache() { return Cache; }
	const FPCGGraphCompilerCache& GetCache() const { return Cache; }

	UPCGComputeGraph* GetComputeGraph(const UPCGGraph* InGraph, uint32 GridSize, uint32 ComputeGraphIndex);

	bool IsCooking() const { return bIsCooking; }

	static void OffsetNodeIds(TArray<FPCGGraphTask>& Tasks, FPCGTaskId Offset, FPCGTaskId ParentId);

#if WITH_EDITOR
	/** Checks whether the compiled result changes when recompiling InGraph. Note that this incurs the cost of a graph compilation each time it is called.
	* The cache will be updated with the latest compiled result. Returns true if the compiled result changes;
	*/
	bool Recompile(UPCGGraph* InGraph, uint32 GenerationGridSize, bool bIsTopGraph = true);
	void NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType);
#endif

	/** Flush all cached compiled graphs. */
	void ClearCache();

	/** Returns the trivial element object shared by all tasks that need it. */
	FPCGElementPtr GetSharedTrivialElement();

	/** Returns the post graph element, which is used to determine results caching behavior */
	FPCGElementPtr GetSharedTrivialPostGraphElement();

	/** Returns the gather element object shared by all tasks that need it. */
	FPCGElementPtr GetSharedGatherElement();

	/** Culls tasks based on a given lambda. Never culls the first (input) task in the array. */
	static void CullTasks(TArray<FPCGGraphTask>& InOutCompiledTasks, bool bAddPassthroughWires, TFunctionRef<bool(const FPCGGraphTask&)> CullTask, TArray<FPCGTaskId>* OutOptionalOldIdToNewIdMap = nullptr);

	void AddReferencedObjects(FReferenceCollector& Collector);

	/** Executes lambda over provided tasks in execution order (task is not visited until all upstream dependencies are visited). Aborts if visitor lambda returns false.
	* Returns true if all tasks visited. */
	static bool VisitTasksInExecutionOrder(const TArray<FPCGGraphTask>& InTasks, const TMap<FPCGTaskId, TArray<FPCGTaskId>>& InTaskToTaskSuccessors, const TFunction<bool(FPCGTaskId)>& InVisitor);

private:
	TArray<FPCGGraphTask> CompileGraph(UPCGGraph* InGraph, FPCGTaskId& NextId, FPCGStackContext& InOutStackContext);

	/** Compiles the top graph and applies culling optimizations if a non-uninitialized grid size is provided. */
	void CompileTopGraph(UPCGGraph* InGraph, uint32 GenerationGridSize);

	/** Propagates grid sizes through a graph's compiled tasks. */
	static void ResolveGridSizes(
		uint32 GenerationGridSize,
		const TArray<FPCGGraphTask>& CompiledTasks,
		const FPCGStackContext& StackContext,
		uint32 ScaledDefaultGridSize,
		TArray<uint32>& InOutTaskGenerationGrid);

	/** Returns the execution grid for the given task. */
	static uint32 CalculateGridRecursive(
		FPCGTaskId InTaskId,
		uint32 InScaledDefaultGridSize,
		const FPCGStackContext& InStackContext,
		const TArray<FPCGGraphTask>& InCompiledTasks,
		TArray<uint32>& InOutTaskGenerationGridSize);

	/** Create linkage tasks for edges that cross from large grid to small grid tasks. */
	static void CreateGridLinkages(
		UPCGGraph* InGraph,
		uint32 InGenerationGrid,
		TArray<uint32>& TaskGenerationGrid,
		TArray<FPCGGraphTask>& InOutCompiledTasks,
		const FPCGStackContext& InStackContext,
		bool bIsCooking);

	/** Discovers whether task is on a statically active branch (and needs to be passed to graph executor). */
	static bool CalculateStaticallyActiveRecursive(FPCGTaskId InTaskId, const TArray<FPCGGraphTask>& InCompiledTasks, TMap<FPCGTaskId, bool>& InTaskIdToActiveFlag);

	/** Culls nodes that are inactive for e.g. missing required inputs or on a statically inactive branch. */
	static void CullTasksStaticInactive(TArray<FPCGGraphTask>& InOutCompiledTasks);

	/** Remove any stack frames that are not used by any task. */
	static void PostCullStackCleanup(TArray<FPCGGraphTask>& InCompiledTasks, FPCGStackContext& InOutStackContext);

	/** Task will be active if *any* of the upstream pin IDs are active, or if the pin ID list is empty. */
	static void CalculateDynamicActivePinDependencies(FPCGTaskId InTaskId, TArray<FPCGGraphTask>& InOutCompiledTasks);

private:
	FPCGGraphCompilerCache Cache;
	bool bIsCooking = false;

	FPCGElementPtr SharedTrivialElement;
	PCG::FSharedLock SharedTrivialElementLock;

	FPCGElementPtr SharedTrivialPostGraphElement;
	PCG::FSharedLock SharedTrivialPostGraphElementLock;

	FPCGElementPtr SharedGatherElement;
	PCG::FSharedLock SharedGatherElementLock;
};
