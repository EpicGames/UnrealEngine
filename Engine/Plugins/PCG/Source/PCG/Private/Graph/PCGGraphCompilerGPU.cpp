// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/PCGGraphCompilerGPU.h"

#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGPin.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGComputeGraph.h"
#include "Compute/PCGComputeKernel.h"
#include "Compute/PCGComputeKernelSource.h"
#include "Compute/PCGKernelOverridableParamTypeInfo.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Compute/DataInterfaces/PCGComputeDataInterface.h"
#include "Compute/DataInterfaces/PCGCustomKernelDataInterface.h"
#include "Compute/DataInterfaces/PCGDataCollectionDataInterface.h"
#include "Compute/DataInterfaces/PCGDataLabelResolverDataInterface.h"
#include "Compute/DataInterfaces/PCGDebugDataInterface.h"
#include "Compute/Elements/PCGComputeGraphElement.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Compute/Packing/PCGDataCollectionPacking.h"
#include "Elements/PCGStaticMeshSpawnerKernel.h"
#include "Graph/PCGGPUGraphCompilationContext.h"
#include "Graph/PCGGraphCompiler.h"
#include "Graph/PCGGraphExecutor.h"

#include "ComputeFramework/ComputeFramework.h"
#include "ComputeFramework/ComputeKernel.h"
#include "HAL/IConsoleManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Shader/ShaderTypes.h"
#include "UObject/Package.h"

#if WITH_EDITOR
namespace PCGGraphCompilerGPU
{
	static TAutoConsoleVariable<bool> CVarEnableDebugging(
		TEXT("pcg.GraphExecution.GPU.EnableDebugging"),
		false,
		TEXT("Enable verbose logging of GPU compilation and execution."));

	static void ValidateNodeKernels(const UPCGSettings* InNodeSettings, const TArray<UPCGComputeKernel*>& InNodeKernels, const TArray<FPCGKernelEdge>& InNodeKernelEdges)
	{
		UE_CLOGF(InNodeKernels.IsEmpty(), LogPCG, Error, "PCG GPU compiler: Settings '%ls' did not emit any kernels, check implementation of UPCGSettings::CreateKernels().", *InNodeSettings->GetName());
		UE_CLOGF(InNodeKernels.Contains(nullptr), LogPCG, Error, "PCG GPU compiler: Settings '%ls' emitted one or more null kernels, check implementation of UPCGSettings::CreateKernels().", *InNodeSettings->GetName());

		auto ValidateKernelPin = [InNodeSettings](const FPCGPinReference& Pin, bool bIsInputPin)
		{
			if (Pin.Kernel)
			{
				bool bPinFound = false;

				if (bIsInputPin)
				{
					TArray<FPCGPinProperties> Pins;
					Pin.Kernel->GetInputPins(Pins);
					bPinFound = Pins.ContainsByPredicate([InNodeSettings, &Pin](const FPCGPinProperties& Other) { return Other.Label == Pin.Label; });
				}
				else
				{
					TArray<FPCGPinPropertiesGPU> Pins;
					Pin.Kernel->GetOutputPins(Pins);
					bPinFound = Pins.ContainsByPredicate([InNodeSettings, &Pin](const FPCGPinPropertiesGPU& Other) { return Other.Label == Pin.Label; });
				}

				if (!bPinFound)
				{
					UE_LOGF(LogPCG, Error, "PCG GPU compiler: Settings '%ls' emitted a kernel edge connecting to %s pin '%ls' of kernel '%ls' which does not exist, check the kernel is returning this pin from Get%sPins().",
						*InNodeSettings->GetName(),
						bIsInputPin ? "input" : "output",
						*Pin.Label.ToString(),
						*Pin.Kernel->GetName(),
						bIsInputPin ? "Input" : "Output");
				}
			}
			else
			{
				TArray<FPCGPinProperties> Pins = bIsInputPin ? InNodeSettings->InputPinProperties() : InNodeSettings->OutputPinProperties();

				if (!Pins.ContainsByPredicate([InNodeSettings, &Pin](const FPCGPinProperties& Other) { return Other.Label == Pin.Label; }))
				{
					UE_LOGF(LogPCG, Error, "PCG GPU compiler: Settings '%ls' emitted a kernel edge connecting to external %s pin '%ls' of the node which does not exist, check the node really has this pin.",
						*InNodeSettings->GetName(),
						bIsInputPin ? "input" : "output",
						*Pin.Label.ToString());
				}
			}
		};

		for (const FPCGKernelEdge& Edge : InNodeKernelEdges)
		{
			ValidateKernelPin(Edge.UpstreamPin, /*bIsInputPin=*/!Edge.GetUpstreamKernel()); // Either wired to an output pin of an upstream kernel, or a node input pin.
			ValidateKernelPin(Edge.DownstreamPin, /*bIsInputPin=*/!!Edge.GetDownstreamKernel()); // Either wired to an input pin of a downstream kernel, or a node output pin.
		}
	}
}

#define PCG_DATA_DESCRIPTION_LOGGING 0

void FPCGGraphCompilerGPU::LabelConnectedGPUNodeIslands(
	const TArray<FPCGGraphTask>& InCompiledTasks,
	const TSet<FPCGTaskId>& InGPUCompatibleTaskIds,
	const FTaskToSuccessors& InTaskSuccessors,
	const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
	TArray<uint32>& OutIslandIDs)
{
	OutIslandIDs.SetNumZeroed(InCompiledTasks.Num());

	// Traverses task inputs and successors and assigns the given island ID to each one. Memoized via output OutIslandIDs.
	auto FloodFillIslandID = [&InCompiledTasks, &InTaskSuccessors, &InGPUCompatibleTaskIds, &OutIslandIDs, &InTaskIdToKernel](FPCGTaskId InTaskId, int InIslandID, FPCGTaskId InTraversedFromTaskId, auto&& RecursiveCall) -> void
	{
		check(InTaskId != InTraversedFromTaskId);

		OutIslandIDs[InTaskId] = InIslandID;

		for (const FPCGGraphTaskInput& Input : InCompiledTasks[InTaskId].Inputs)
		{
			if (Input.TaskId != InTraversedFromTaskId && OutIslandIDs[Input.TaskId] == 0 && InGPUCompatibleTaskIds.Contains(Input.TaskId))
			{
				RecursiveCall(Input.TaskId, InIslandID, InTaskId, RecursiveCall);
			}
		}

		if (const TArray<FPCGTaskId>* Successors = InTaskSuccessors.Find(InTaskId))
		{
			for (FPCGTaskId Successor : *Successors)
			{
				if (Successor != InTraversedFromTaskId && OutIslandIDs[Successor] == 0 && InGPUCompatibleTaskIds.Contains(Successor))
				{
					RecursiveCall(Successor, InIslandID, InTaskId, RecursiveCall);
				}
			}
		}
	};

	for (FPCGTaskId GPUTaskId : InGPUCompatibleTaskIds)
	{
		if (OutIslandIDs[GPUTaskId] == 0)
		{
			// Really doesn't matter what the island IDs are so just use ID of first task encountered in island.
			const uint32 IslandID = static_cast<uint32>(GPUTaskId);
			FloodFillIslandID(GPUTaskId, IslandID, InvalidPCGTaskId, FloodFillIslandID);
		}
	}
}

void FPCGGraphCompilerGPU::CollectGPUNodeSubsets(
	const TArray<FPCGGraphTask>& InCompiledTasks,
	const FTaskToSuccessors& InTaskSuccessors,
	const TSet<FPCGTaskId>& InGPUCompatibleTaskIds,
	const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
	TArray<TSet<FPCGTaskId>>& OutNodeSubsetsToConvertToCFGraph)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompilerGPU::CollectGPUNodeSubsets);

	// Identifies connected sets of GPU nodes, giving each a non-zero ID value.
	TArray<uint32> ConnectedGPUNodeIslandIDs;
	LabelConnectedGPUNodeIslands(InCompiledTasks, InGPUCompatibleTaskIds, InTaskSuccessors, InTaskIdToKernel, ConnectedGPUNodeIslandIDs);

	// Any new island IDs will be created from the task count which will be larger than any island IDs presently set.
	uint32 NextIslandId = InCompiledTasks.Num();

	// Cache dependencies. Since our islands are only ever split (rather than replaced or removed), the dependency on each island ID is invariant.
	TMap<TPair<FPCGTaskId, /*IslandId*/uint32>, /*bIsDependent*/bool> CPUNodeIsDependentOnIslandCached;

	// Visit tasks in execution order so that splits happen as upstream to minimize island splits.
	FPCGGraphCompiler::VisitTasksInExecutionOrder(
		InCompiledTasks,
		InTaskSuccessors,
		[&InCompiledTasks, &InTaskSuccessors, &InGPUCompatibleTaskIds, &ConnectedGPUNodeIslandIDs, &InTaskIdToKernel, &NextIslandId, &CPUNodeIsDependentOnIslandCached](FPCGTaskId InTaskId) -> bool
		{
			auto PropagateIslandIDDownstream = [&InCompiledTasks, &InTaskSuccessors, &InGPUCompatibleTaskIds, &ConnectedGPUNodeIslandIDs](FPCGTaskId InTaskId, uint32 InOldIslandID, uint32 InNewIslandID, auto&& RecursiveCall) -> void
			{
				ConnectedGPUNodeIslandIDs[InTaskId] = InNewIslandID;

				if (const TArray<FPCGTaskId>* Successors = InTaskSuccessors.Find(InTaskId))
				{
					for (FPCGTaskId Successor : *Successors)
					{
						if (ConnectedGPUNodeIslandIDs[Successor] == InOldIslandID && InGPUCompatibleTaskIds.Contains(Successor))
						{
							RecursiveCall(Successor, InOldIslandID, InNewIslandID, RecursiveCall);
						}
					}
				}
			};

			// GPU task. Split at edges if the kernel requests it.
			if (InGPUCompatibleTaskIds.Contains(InTaskId))
			{
				UPCGComputeKernel* const* FoundKernel = InTaskIdToKernel.Find(InTaskId);
				UPCGComputeKernel* ProducerKernel = FoundKernel ? *FoundKernel : nullptr;

				if (ProducerKernel)
				{
					for (const FPCGGraphTaskInput& TaskInput : InCompiledTasks[InTaskId].Inputs)
					{
						if (!TaskInput.DownstreamPin.IsSet())
						{
							continue;
						}

						if (ProducerKernel->SplitGraphAtInput(TaskInput.DownstreamPin->Label))
						{
							// Don't need to split if the upstream task is already on a different GPU island.
							if (InGPUCompatibleTaskIds.Contains(TaskInput.TaskId))
							{
								const uint32 UpstreamIslandId = ConnectedGPUNodeIslandIDs[TaskInput.TaskId];

								if (ConnectedGPUNodeIslandIDs[InTaskId] == UpstreamIslandId)
								{
									// Propagate a new island ID to all downstream GPU tasks within the island.
									PropagateIslandIDDownstream(InTaskId, ConnectedGPUNodeIslandIDs[InTaskId], NextIslandId++, PropagateIslandIDDownstream);
								}
							}
						}
					}

					if (ProducerKernel->SplitGraphAtOutput())
					{
						if (const TArray<FPCGTaskId>* Successors = InTaskSuccessors.Find(InTaskId))
						{
							for (FPCGTaskId SuccessorTaskId : *Successors)
							{
								if (InGPUCompatibleTaskIds.Contains(SuccessorTaskId))
								{
									const uint32 SuccessorIsland = ConnectedGPUNodeIslandIDs[SuccessorTaskId];

									if (ConnectedGPUNodeIslandIDs[InTaskId] == SuccessorIsland)
									{
										// Propagate a new island ID to all downstream GPU tasks within the island.
										PropagateIslandIDDownstream(SuccessorTaskId, SuccessorIsland, NextIslandId++, PropagateIslandIDDownstream);
									}
								}
							}
						}
					}
				}
			}
			// Otherwise CPU task. For every CPU node that has one or more downstream GPU node connected, check for a GPU -> CPU -> GPU pattern where data flows
			// from a GPU node island to the CPU and then back to the same island. For such cases, we traverse the entire tree of GPU nodes that
			// are in the island and downstream of the CPU node and bump their island ID - splitting the island into a portion that is independent
			// of the CPU node and a portion that is dependent on it, so that we can read the data back to CPU, execute the CPU portion, then
			// re-upload to GPU.
			else if (const TArray<FPCGTaskId>* Successors = InTaskSuccessors.Find(InTaskId))
			{
				for (FPCGTaskId SuccessorTaskId : *Successors)
				{
					if (!InGPUCompatibleTaskIds.Contains(SuccessorTaskId))
					{
						continue;
					}

					const uint32 SuccessorIsland = ConnectedGPUNodeIslandIDs[SuccessorTaskId];

					// Recursively check entire node tree upstream of this CPU node to see if it can be fed by any node in the GPU island.
					auto CPUNodeIsDependentOnIsland = [&InCompiledTasks, &InTaskSuccessors, &InGPUCompatibleTaskIds, &ConnectedGPUNodeIslandIDs, &CPUNodeIsDependentOnIslandCached](FPCGTaskId InTaskId, uint32 InIslandID, auto&& RecursiveCall) -> bool
					{
						if (bool* CachedValue = CPUNodeIsDependentOnIslandCached.Find({ InTaskId, InIslandID }))
						{
							return *CachedValue;
						}

						// Is this task is part of the specified island.
						bool bIsDependent = ConnectedGPUNodeIslandIDs[InTaskId] == InIslandID;

						if (!bIsDependent)
						{
							// Check upstream tasks recursively.
							for (const FPCGGraphTaskInput& Input : InCompiledTasks[InTaskId].Inputs)
							{
								if (RecursiveCall(Input.TaskId, InIslandID, RecursiveCall))
								{
									bIsDependent = true;
									break;
								}
							}
						}

						CPUNodeIsDependentOnIslandCached.Add({ InTaskId, InIslandID }, bIsDependent);

						return bIsDependent;
					};

					if (CPUNodeIsDependentOnIsland(InTaskId, SuccessorIsland, CPUNodeIsDependentOnIsland))
					{
						// Propagate a new island ID to all downstream GPU tasks within the island.
						PropagateIslandIDDownstream(SuccessorTaskId, SuccessorIsland, NextIslandId++, PropagateIslandIDDownstream);
					}
				}
			}

			return true;
		});

	// Island IDs now correctly identify subsets of nodes that will be assembled into compute graphs for GPU execution.
	for (FPCGTaskId TaskId = 0; TaskId < InCompiledTasks.Num(); ++TaskId)
	{
		if (ConnectedGPUNodeIslandIDs[TaskId] != 0)
		{
			TSet<FPCGTaskId> GPUNodeSubset;

			const uint32 IslandId = ConnectedGPUNodeIslandIDs[TaskId];

			for (FPCGTaskId OtherTaskId = TaskId; OtherTaskId < InCompiledTasks.Num(); ++OtherTaskId)
			{
				if (ConnectedGPUNodeIslandIDs[OtherTaskId] == IslandId)
				{
					GPUNodeSubset.Add(OtherTaskId);

					ConnectedGPUNodeIslandIDs[OtherTaskId] = 0;
				}
			}

			OutNodeSubsetsToConvertToCFGraph.Add(MoveTemp(GPUNodeSubset));
		}
	}
}

void FPCGGraphCompilerGPU::ExpandGPUNodeKernelsToTasks(
	FPCGGPUCompilationContext& InOutContext,
	TSet<FPCGTaskId>& InOutGPUCompatibleTaskIds,
	TArray<FPCGGraphTask>& InOutCompiledTasks,
	TMap<FPCGTaskId, UPCGComputeKernel*>& OutTaskIdToKernel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompilerGPU::ExpandGPUNodeKernelsToTasks);
	const int NumTasksBefore = InOutCompiledTasks.Num();

	// Build successors map, only for nodes that are downstream from one or more of the relevant GPU tasks.
	FTaskToSuccessors TaskSuccessors;
	TaskSuccessors.Reserve(InOutCompiledTasks.Num());
	for (FPCGTaskId TaskId = 0; TaskId < InOutCompiledTasks.Num(); ++TaskId)
	{
		for (const FPCGGraphTaskInput& TaskInput : InOutCompiledTasks[TaskId].Inputs)
		{
			if (InOutGPUCompatibleTaskIds.Contains(TaskInput.TaskId))
			{
				TaskSuccessors.FindOrAdd(TaskInput.TaskId).AddUnique(TaskId);
			}
		}
	}

	// Local to loop body, but hoisted for performance.
	TArray<UPCGComputeKernel*> NodeKernels;
	TArray<FPCGKernelEdge> NodeKernelEdges;
	TMap<const UPCGComputeKernel*, FPCGTaskId> KernelToTaskId;
	TArray<FPCGPinProperties> KernelInputs;
	TArray<FPCGPinPropertiesGPU> UpstreamKernelOutputs;

	KernelToTaskId.Reserve(InOutGPUCompatibleTaskIds.Num());
	OutTaskIdToKernel.Reserve(InOutGPUCompatibleTaskIds.Num());

	// Process one GPU task at a time.
	for (FPCGTaskId TaskId : InOutGPUCompatibleTaskIds)
	{
		const UPCGSettings* Settings = InOutCompiledTasks[TaskId].Node ? InOutCompiledTasks[TaskId].Node->GetSettings() : nullptr;
		if (!ensure(Settings))
		{
			continue;
		}

		NodeKernels.Reset();
		NodeKernelEdges.Reset();
		Settings->CreateKernels(InOutContext, GetTransientPackage(), InOutCompiledTasks[TaskId].Node, NodeKernels, NodeKernelEdges);

		PCGGraphCompilerGPU::ValidateNodeKernels(Settings, NodeKernels, NodeKernelEdges);

		NodeKernels.Remove(nullptr);

		if (NodeKernels.IsEmpty())
		{
			continue;
		}

#ifdef PCG_GPU_KERNEL_PROFILING
		if (NodeKernels.IsValidIndex(Settings->ProfileKernelIndex))
		{
			NodeKernels[Settings->ProfileKernelIndex]->EnableRepeatDispatch();
		}
#endif // PCG_GPU_KERNEL_PROFILING

		// Wire up overridden parameters for any kernels that require them.
		for (UPCGComputeKernel* NodeKernel : NodeKernels)
		{
			for (const FPCGKernelOverridableParam& OverridableParam : NodeKernel->GetCachedOverridableParams())
			{
				if (OverridableParam.bIsPropertyOverriddenByPin)
				{
					NodeKernelEdges.Emplace(FPCGPinReference(OverridableParam.Label), FPCGPinReference(NodeKernel, OverridableParam.Label));
				}
			}
		}

		if (NodeKernelEdges.IsEmpty())
		{
			continue;
		}

		for (FPCGKernelEdge& KernelEdge : NodeKernelEdges)
		{
			if (!KernelEdge.IsConnectedToNodeInput() && !KernelEdge.IsConnectedToNodeOutput())
			{
				if (UPCGComputeKernel* UpstreamKernel = KernelEdge.GetUpstreamKernel())
				{
					UpstreamKernel->AddInternalPin(KernelEdge.UpstreamPin.Label);
				}

				if (UPCGComputeKernel* DownstreamKernel = KernelEdge.GetDownstreamKernel())
				{
					DownstreamKernel->AddInternalPin(KernelEdge.DownstreamPin.Label);
				}
			}
		}

		KernelToTaskId.Reset();

		// Create a new task for each kernel emitted by the node.
		while (!NodeKernels.IsEmpty())
		{
			// Find a kernel that is ready to process (does not depend on another kernel in this node).
			// Number of kernels is likely to be small so brute force loop over them.
			int ReadyKernelIndex = 0;

			for (int Index = 0; Index < NodeKernels.Num(); ++Index)
			{
				// Look for an edge connected to an upstream kernel that is yet to be processed.
				const FPCGKernelEdge* BlockingEdge = NodeKernelEdges.FindByPredicate([Index, &NodeKernels, &KernelToTaskId](const FPCGKernelEdge& InEdge)
				{
					const UPCGComputeKernel* UpstreamKernel = InEdge.UpstreamPin.Kernel;
					return InEdge.DownstreamPin.Kernel == NodeKernels[Index] && UpstreamKernel && !KernelToTaskId.Contains(UpstreamKernel);
				});

				if (!BlockingEdge)
				{
					// No pending kernels, ready to execute this one.
					ReadyKernelIndex = Index;
					break;
				}
			}

			if (ReadyKernelIndex == INDEX_NONE)
			{
				ensureMsgf(false, TEXT("Compilation did not make progress, %d kernels were not processed."), NodeKernels.Num());
				break;
			}

			UPCGComputeKernel* Kernel = NodeKernels[ReadyKernelIndex];

			// Create a new task for this kernel.
			FPCGTaskId NewKernelTaskId = InOutCompiledTasks.Num();

			FPCGGraphTask TaskCopy = InOutCompiledTasks[TaskId];
			TaskCopy.NodeId = NewKernelTaskId;
			TaskCopy.Inputs.Reset();

			FPCGGraphTask& NewKernelTask = InOutCompiledTasks.Add_GetRef(MoveTemp(TaskCopy));

			// Update maps.
			KernelToTaskId.Add(Kernel, NewKernelTaskId);
			OutTaskIdToKernel.Add(NewKernelTaskId, Kernel);

			// Get kernel inputs in preparation for wiring.
			KernelInputs.Reset();
			Kernel->GetInputPinsAndOverridablePins(KernelInputs);

			if (Settings->bTriggerRenderCapture)
			{
				Kernel->KernelFlags |= (int32)EComputeKernelFlags::TriggerRenderCapture;
			}

			// Wire inputs
			for (const FPCGPinProperties& KernelInputProps : KernelInputs)
			{
				const FPCGKernelEdge* Edge = NodeKernelEdges.FindByPredicate([Kernel, &KernelInputProps](const FPCGKernelEdge& Edge)
				{
					return Edge.DownstreamPin.Kernel == Kernel && Edge.DownstreamPin.Label == KernelInputProps.Label;
				});

				if (!Edge)
				{
					continue;
				}

				if (const UPCGComputeKernel* UpstreamKernel = Edge->UpstreamPin.Kernel)
				{
					checkf(KernelToTaskId.Contains(UpstreamKernel), TEXT("Missing kernel '%s', was kernel emitted from CreateKernels()?"), *UpstreamKernel->GetName());

					// Connection from an upstream kernel from this node.
					const FPCGTaskId UpstreamTaskId = KernelToTaskId[UpstreamKernel];

					FPCGGraphTaskInput& TaskInput = NewKernelTask.Inputs.AddDefaulted_GetRef();
					TaskInput.TaskId = UpstreamTaskId;
					TaskInput.bProvideData = true;
					TaskInput.DownstreamPin = KernelInputProps;

					// Look for output pin of upstream kernel in order to set the input upstream pin properties.
					UpstreamKernelOutputs.Reset();
					UpstreamKernel->GetOutputPins(UpstreamKernelOutputs);

					const FPCGPinPropertiesGPU* UpstreamOutput = UpstreamKernelOutputs.FindByPredicate([Edge](const FPCGPinPropertiesGPU& InUpstreamOutput)
					{
						return InUpstreamOutput.Label == Edge->UpstreamPin.Label;
					});

					if (ensure(UpstreamOutput))
					{
						TaskInput.UpstreamPin = *UpstreamOutput;
					}
				}
				// Edge from an upstream node/task. Correlates to an input for the original node task.
				else
				{
					check(Edge->IsConnectedToNodeInput());
					const FName NodePinLabel = Edge->UpstreamPin.Label;

					const FPCGGraphTaskInput* OriginalTaskInput = InOutCompiledTasks[TaskId].Inputs.FindByPredicate([NodePinLabel, &KernelInputProps](const FPCGGraphTaskInput& TaskInput)
					{
						return TaskInput.DownstreamPin && TaskInput.DownstreamPin->Label == NodePinLabel;
					});

					if (OriginalTaskInput)
					{
						// Match - add a new input wire to the new task.
						FPCGGraphTaskInput& NewTaskInput = NewKernelTask.Inputs.Add_GetRef(*OriginalTaskInput);

						// Resolve dynamic pin type here to simplify later logic.
						// todo_pcg: Possibly the graph compiler should resolve dynamic pins during compilation.
						if (NewTaskInput.UpstreamPin)
						{
							const UPCGSettings* ProducerSettings = InOutCompiledTasks[NewTaskInput.TaskId].Node ? InOutCompiledTasks[NewTaskInput.TaskId].Node->GetSettings() : nullptr;
							const UPCGPin* ProducerPin = InOutCompiledTasks[NewTaskInput.TaskId].Node
								? InOutCompiledTasks[NewTaskInput.TaskId].Node->GetOutputPin(NewTaskInput.UpstreamPin->Label)
								: nullptr;

							if (ProducerSettings && ProducerPin)
							{
								NewTaskInput.UpstreamPin->AllowedTypes = ProducerSettings->GetCurrentPinTypesID(ProducerPin);
							}
						}

						// Update the pin properties with those specified by the kernel.
						NewTaskInput.DownstreamPin = KernelInputProps;
					}
				}
			}

			// Preserve dependency-only inputs (e.g. execution dependency pins) from the original task. These are PCG control-flow inputs
			// (bProvideData=false) that have no kernel equivalent but are required for dynamic culling to work correctly at execution time.
			for (const FPCGGraphTaskInput& OriginalInput : InOutCompiledTasks[TaskId].Inputs)
			{
				if (!OriginalInput.bProvideData && OriginalInput.DownstreamPin.IsSet())
				{
					NewKernelTask.Inputs.Add(OriginalInput);
				}
			}

			NodeKernels.RemoveAt(ReadyKernelIndex);
		}

		// Now rewire all tasks downstream of the original task to the appropriate kernel outputs.
		if (const TArray<FPCGTaskId>* Successors = TaskSuccessors.Find(TaskId))
		{
			for (FPCGTaskId SuccessorTaskId : *Successors)
			{
				for (FPCGGraphTaskInput& Input : InOutCompiledTasks[SuccessorTaskId].Inputs)
				{
					if (Input.TaskId == TaskId && Input.UpstreamPin)
					{
						// Outgoing edge. Rewire to appropriate kernel.
						for (FPCGKernelEdge& Edge : NodeKernelEdges)
						{
							if (Edge.IsConnectedToNodeOutput() && Edge.DownstreamPin.Label == Input.UpstreamPin->Label && ensure(Edge.UpstreamPin.Kernel))
							{
								Input.TaskId = KernelToTaskId[Edge.UpstreamPin.Kernel];
							}
						}
					}
				}
			}
		}
	}

	const int NumTasksAfter = InOutCompiledTasks.Num();

	TArray<FPCGTaskId> OldIdToNewId;

	// All the original GPU tasks have now been replaced with a task for each kernel. Remove them without rewiring.
	FPCGGraphCompiler::CullTasks(
		InOutCompiledTasks,
		/*bAddPassthroughWires=*/false,
		[&InOutGPUCompatibleTaskIds](const FPCGGraphTask& InTask)
		{
			return InOutGPUCompatibleTaskIds.Contains(InTask.NodeId);
		},
		&OldIdToNewId);

	check(OldIdToNewId.Num() == NumTasksAfter);

	// Refresh GPU compatible task IDs to reflect shifted indices after culling.
	{
		InOutGPUCompatibleTaskIds.Reset();

		for (FPCGTaskId OldGPUTaskId = NumTasksBefore; OldGPUTaskId < NumTasksAfter; ++OldGPUTaskId)
		{
			const FPCGTaskId RemappedId = OldIdToNewId[OldGPUTaskId];
			check(RemappedId != INDEX_NONE);

			InOutGPUCompatibleTaskIds.Add(RemappedId);
		}
	}

	// Remap task IDs of the task->kernel map to reflect shifted indices after culling.
	{
		TMap<FPCGTaskId, UPCGComputeKernel*> TaskIdToKernelBeforeCull = MoveTemp(OutTaskIdToKernel);
		OutTaskIdToKernel.Reset();

		for (TPair<FPCGTaskId, UPCGComputeKernel*>& Entry : TaskIdToKernelBeforeCull)
		{
			const FPCGTaskId RemappedId = OldIdToNewId[Entry.Key];
			check(RemappedId != INDEX_NONE);

			OutTaskIdToKernel.Add(RemappedId, Entry.Value);
		}
	}
}

void FPCGGraphCompilerGPU::CreateGatherTasksAtGPUInputs(UPCGGraph* InGraph, const TSet<FPCGTaskId>& InGPUCompatibleTaskIds, TArray<FPCGGraphTask>& InOutCompiledTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompilerGPU::CreateGatherTasksAtGPUInputs);

	using FOriginalInputPinKey = TPair<FPCGTaskId /* Original GPU task */, FName /* Input pin label */>;

	// These are local to loop below but hoisted here for efficiency.
	TSet<FOriginalInputPinKey> EncounteredInputPins; // TODO this is heavyweight, could use simple array.
	TMap<FOriginalInputPinKey, FPCGTaskId /* Gather task */> InputPinToGatherTask;

	// Add all compute graph task inputs and outputs.
	for (FPCGTaskId GPUTaskId : InGPUCompatibleTaskIds)
	{
		EncounteredInputPins.Empty(EncounteredInputPins.Num());
		InputPinToGatherTask.Empty(InputPinToGatherTask.Num());

		// First pass - create gather tasks for all original input pins which have more than one incident edge.
		// This is so we can gather on the CPU (much more efficient than going it on the GPU).
		for (int InputIndex = 0; InputIndex < InOutCompiledTasks[GPUTaskId].Inputs.Num(); ++InputIndex)
		{
			// Helper to get current input. We avoid simply taking a local reference as InOutCompiledTasks can be modified below.
			auto CurrentInput = [&InOutCompiledTasks, InputIndex, GPUTaskId]() -> FPCGGraphTaskInput&
			{
				return InOutCompiledTasks[GPUTaskId].Inputs[InputIndex];
			};

			if (!CurrentInput().DownstreamPin.IsSet())
			{
				continue;
			}

			const FOriginalInputPinKey PinKey = { GPUTaskId, CurrentInput().DownstreamPin->Label };

			// If already created a gather task, then nothing more to do for this pin.
			if (const FPCGTaskId* GatherTaskId = InputPinToGatherTask.Find(PinKey))
			{
				continue;
			}

			// If we're encountering pin for first time, register it.
			if (!EncounteredInputPins.Contains(PinKey))
			{
				EncounteredInputPins.Add(PinKey);
				continue;
			}

			// Second time we've encountered this input pin - create a gather element because we need one edge connected to
			// each virtual input pin, so that we can obtain the data items from the input data collection using the unique
			// virtual pin label at execution time.
			const FPCGTaskId GatherTaskId = InOutCompiledTasks.Num();
			FPCGGraphTask& GatherTask = InOutCompiledTasks.Emplace_GetRef();
			GatherTask.NodeId = GatherTaskId;
			GatherTask.ParentId = InOutCompiledTasks[GPUTaskId].ParentId;
			GatherTask.ElementSource = EPCGElementSource::Gather;

			InputPinToGatherTask.Add(PinKey, GatherTaskId);
		}

		EncounteredInputPins.Empty(EncounteredInputPins.Num());

		// Second pass - wire up the newly added gather tasks once we have the full picture of which edges are affected.
		for (int InputIndex = 0; InputIndex < InOutCompiledTasks[GPUTaskId].Inputs.Num(); ++InputIndex)
		{
			// Helper to get input rather than reference because we modify the inputs array below.
			auto CurrentGPUTaskInput = [&InOutCompiledTasks, InputIndex, GPUTaskId]() -> FPCGGraphTaskInput&
			{
				return InOutCompiledTasks[GPUTaskId].Inputs[InputIndex];
			};

			if (!CurrentGPUTaskInput().DownstreamPin)
			{
				continue;
			}

			const FOriginalInputPinKey PinKey = { GPUTaskId, CurrentGPUTaskInput().DownstreamPin->Label };

			if (const FPCGTaskId* GatherTaskId = InputPinToGatherTask.Find(PinKey))
			{
				// Wire the upstream output pin to the gather task input.
				{
					FPCGGraphTaskInput& GatherTaskInput = InOutCompiledTasks[*GatherTaskId].Inputs.Add_GetRef(CurrentGPUTaskInput());
					if (GatherTaskInput.DownstreamPin)
					{
						GatherTaskInput.DownstreamPin->Label = PCGPinConstants::DefaultInputLabel;
					}
				}

				if (!EncounteredInputPins.Contains(PinKey))
				{
					// First time we're encountering this input pin, wire it to the gather task output.
					EncounteredInputPins.Add(PinKey);

					CurrentGPUTaskInput().TaskId = *GatherTaskId;

					// Wire the gather task output pin to the downstream GPU task input pin.
					if (CurrentGPUTaskInput().UpstreamPin)
					{
						CurrentGPUTaskInput().UpstreamPin->Label = PCGPinConstants::DefaultOutputLabel;
					}
				}
				else
				{
					// Input pin already encountered, already wired to gather task. Remove this input.
					InOutCompiledTasks[GPUTaskId].Inputs.RemoveAt(InputIndex);
					--InputIndex;
				}
			}
		}
	}
}

void FPCGGraphCompilerGPU::SetupVirtualPins(
	const TSet<FPCGTaskId>& InCollapsedTasks,
	const TArray<FPCGGraphTask>& InCompiledTasks,
	const FTaskToSuccessors& InTaskSuccessors,
	const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
	FOriginalToVirtualPin& OutOriginalToVirtualPin)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompilerGPU::SetupVirtualPins);

	// Used to construct unique input/output labels, ultimately consumed in graph executor in BuildTaskInput and PostExecute for input/output respectively.
	int InputCount = 0;
	int OutputCount = 0;

	// Add all compute graph task inputs and outputs.
	for (FPCGTaskId GPUTaskId : InCollapsedTasks)
	{
		// First input edges to the compute graph.
		for (const FPCGGraphTaskInput& Input : InCompiledTasks[GPUTaskId].Inputs)
		{
			if (InCollapsedTasks.Contains(Input.TaskId))
			{
				continue;
			}

			// Only graph edges associated with pins are considered.
			if (Input.DownstreamPin.IsSet())
			{
				const FName VirtualLabel = *FString::Format(TEXT("{0}-VirtualIn{1}"), { Input.DownstreamPin->Label.ToString(), InputCount++ });
				const bool bIsInputPin = true;
				OutOriginalToVirtualPin.Add({ GPUTaskId, Input.DownstreamPin->Label, bIsInputPin }, VirtualLabel);
			}
		}

		// Create virtual pin labels for all output pins of GPU nodes. We could create them only for output pins that internal->external edges, however
		// it is helpful for special cases like inspection-only to always have virtual labels for all external output pins.
		if (const UPCGComputeKernel* const* FoundKernel = InTaskIdToKernel.Find(GPUTaskId); ensure(FoundKernel && *FoundKernel))
		{
			TArray<FPCGPinPropertiesGPU> OutputPins;
			(*FoundKernel)->GetOutputPins(OutputPins);

			for (const FPCGPinPropertiesGPU& PinProps : OutputPins)
			{
				const FNodePin PinKey = { GPUTaskId, PinProps.Label, /*Pin is input*/false };
				if (!OutOriginalToVirtualPin.Contains(PinKey))
				{
					const FName VirtualLabel = *FString::Format(TEXT("{0}-VirtualOut{1}"), { PinProps.Label.ToString(), OutputCount++ });
					OutOriginalToVirtualPin.Add(PinKey, VirtualLabel);
				}
			}
		}
	}
}

void FPCGGraphCompilerGPU::WireComputeGraphTask(
	FPCGTaskId InGPUGraphTaskId,
	const TSet<FPCGTaskId>& InCollapsedTasks,
	TArray<FPCGGraphTask>& InOutCompiledTasks,
	const FTaskToSuccessors& InTaskSuccessors,
	const FOriginalToVirtualPin& InOriginalToVirtualPin)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompilerGPU::WireComputeGraphTask);

	FPCGGraphTask& GPUGraphTask = InOutCompiledTasks[InGPUGraphTaskId];

	// Add all compute graph task inputs and outputs.
	for (FPCGTaskId GPUTaskId : InCollapsedTasks)
	{
		// First find CPU to GPU edges and wire in the GPU graph node inputs.
		for (const FPCGGraphTaskInput& Input : InOutCompiledTasks[GPUTaskId].Inputs)
		{
			if (InCollapsedTasks.Contains(Input.TaskId))
			{
				continue;
			}

			FPCGGraphTaskInput& AddedInput = GPUGraphTask.Inputs.Add_GetRef(Input);

			// TODO is pinless fine with skipping?
			if (AddedInput.DownstreamPin.IsSet())
			{
				// Get the compute graph virtual pin corresponding to this pin, if one was created.
				const FName* VirtualPinLabel = InOriginalToVirtualPin.Find({ GPUTaskId, AddedInput.DownstreamPin->Label, /*bIsInputPin=*/true });

				if (ensure(VirtualPinLabel))
				{
					AddedInput.DownstreamPin->Label = *VirtualPinLabel;
				}
			}
		}

		if (!InTaskSuccessors.Contains(GPUTaskId))
		{
			continue;
		}

		// Next consider GPU to CPU edges to wire in the GPU graph node outputs.
		for (FPCGTaskId Successor : InTaskSuccessors[GPUTaskId])
		{
			if (InCollapsedTasks.Contains(Successor))
			{
				continue;
			}

			// Rewire inputs of this downstream CPU node to the outputs of the compute graph task.
			FPCGGraphTask& DownstreamCPUNode = InOutCompiledTasks[Successor];

			// Build a new inputs array that retains the existing kernel-task entry (so CullTasks can still find and remove it later) and injects the
			// corresponding compute-graph-task edge immediately after it. Injecting immediately after each match preserves the relative position of
			// every entry, so the final order after CullTasks matches edge-addition order even when multiple separate GPU islands all feed the same downstream CPU node.
			TArray<FPCGGraphTaskInput> NewInputs;
			for (FPCGGraphTaskInput& ExistingInput : DownstreamCPUNode.Inputs)
			{
				NewInputs.Add(MoveTemp(ExistingInput));

				// Move on if this input is not associated with the current task.
				if (ExistingInput.TaskId != GPUTaskId)
				{
					continue;
				}

				// Inject new compute-graph-task edge immediately after the original kernel-task edge. Start by duplicating the input.
				FPCGGraphTaskInput NewInput = NewInputs.Last();
				NewInput.TaskId = InGPUGraphTaskId;

				if (NewInput.UpstreamPin.IsSet())
				{
					// Get the compute graph virtual pin corresponding to this pin, if one was created.
					const FName* VirtualPinLabel = InOriginalToVirtualPin.Find({ GPUTaskId, NewInput.UpstreamPin->Label, /*Pin is input*/false });

					if (ensure(VirtualPinLabel)) // Ensure should never fire - compute graph outputs are currently created for all kernel output pins.
					{
						NewInput.UpstreamPin->Label = *VirtualPinLabel;
					}
				}

				NewInputs.Add(MoveTemp(NewInput));
			}

			DownstreamCPUNode.Inputs = MoveTemp(NewInputs);
		}
	}
}

void FPCGGraphCompilerGPU::BuildComputeGraphTask(
	FPCGGPUCompilationContext& InOutContext,
	FPCGTaskId InGPUGraphTaskId,
	UPCGGraph* InGraph,
	uint32 InGridSize,
	uint32 InComputeGraphIndex,
	const TSet<FPCGTaskId>& InCollapsedTasks,
	const TArray<FPCGTaskId>& InComputeElementTasks,
	const FTaskToSuccessors& InTaskSuccessors,
	TArray<FPCGGraphTask>& InOutCompiledTasks,
	const FOriginalToVirtualPin& InOriginalToVirtualPin,
	TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompilerGPU::BuildComputeGraphTask);

	check(InGraph);

	FPCGGraphCompiler& GraphCompiler = InOutContext.GetGraphCompiler();
	UPCGComputeGraph* ComputeGraph = GraphCompiler.GetComputeGraph(InGraph, InGridSize, InComputeGraphIndex);

	// If the graph does not exist already, create it.
	// todo_pcg: If the graph does exist already, then we throw the kernels we created away, which is pretty wasteful, even if only happens
	// on first graph compilation. We could probably defer creation of these objects until they're really needed by storing "createinfo" data.
	// Or possibly the caching of compute graphs could be done per pcg-graph, rather than per-compute-graph, and we could avoid doing any GPU
	// compilation work if already cached.
	if (!ComputeGraph)
	{
		// TODO: There is a race condition here where two threads can try to find this graph at the same time and both fail, so both create the graph.
		// This is fine, as they'll both create the same graph and place it at the same compute graph index. However it would be ideal perf-wise to
		// avoid this, as creating the compute graph can be expensive.
		
		// Create a new compute graph.
		UObject* ComputeGraphOuter = InOutContext.GetGraphCompiler().IsCooking() ? InGraph : Cast<UObject>(GetTransientPackage());
		const FName GraphName = MakeUniqueObjectName(ComputeGraphOuter, UPCGComputeGraph::StaticClass(), InGraph->GetFName());
		ComputeGraph = InOutContext.NewObject_AnyThread<UPCGComputeGraph>(ComputeGraphOuter, GraphName);
		InOutContext.SetStaticAttributeTable(&ComputeGraph->GetStaticAttributeTable());

		// Not incredibly useful for us - DG adds GetComponentSource()->GetComponentClass() object which allows it to bind at execution time by class.
		// But execution code requires it currently.
		ComputeGraph->Bindings.Add(UPCGDataBinding::StaticClass());

		BuildComputeGraphStaticData(InOutCompiledTasks, InCollapsedTasks, InTaskIdToKernel, ComputeGraph);

		FPinToDataInterface OutputPinDataInterfaces;

		CreateDataInterfaces(InOutContext, InOutCompiledTasks, InCollapsedTasks, InComputeElementTasks, InTaskSuccessors, InOriginalToVirtualPin, InTaskIdToKernel, ComputeGraph, OutputPinDataInterfaces);

		CompileComputeGraph(InOutContext, InOutCompiledTasks, InCollapsedTasks, InComputeElementTasks, InTaskSuccessors, InOriginalToVirtualPin, OutputPinDataInterfaces, InTaskIdToKernel, ComputeGraph);

		// Remove empty strings (at execution time empty string is placed in table slot 0).
		// NOTE: This can scramble order but order is not important at this stage (key values created at execution time in data binding).
		ComputeGraph->StringTable.RemoveAllSwap([](const FString& InString) { return InString.IsEmpty(); });

		for (FPCGTaskId TaskId : InCollapsedTasks)
		{
			// Re-outer kernels to compute graph.
			InTaskIdToKernel[TaskId]->Rename(nullptr, ComputeGraph);
		}

		// Clear out the current static attribute table.
		InOutContext.SetStaticAttributeTable(nullptr);
	}

	const int32 ComputeGraphIndex = InOutContext.AddCompiledComputeGraph(ComputeGraph);
	ensure(ComputeGraphIndex == InComputeGraphIndex);

	if (GraphCompiler.IsCooking())
	{
		TObjectPtr<UPCGComputeGraphSettings> Settings = NewObject<UPCGComputeGraphSettings>(InGraph);
		Settings->ComputeGraphIndex = ComputeGraphIndex;
		InOutCompiledTasks[InGPUGraphTaskId].ElementSource = EPCGElementSource::FromCookedSettings;
		InOutCompiledTasks[InGPUGraphTaskId].CookedSettings = Settings;
	}
	else
	{
		InOutCompiledTasks[InGPUGraphTaskId].Element = MakeShared<FPCGComputeGraphElement>(ComputeGraphIndex);
	}
}

void FPCGGraphCompilerGPU::BuildComputeGraphStaticData(
	const TArray<FPCGGraphTask>& InCompiledTasks,
	const TSet<FPCGTaskId>& InCollapsedTasks,
	const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
	UPCGComputeGraph* InOutComputeGraph)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompilerGPU::BuildComputeGraphStaticData);

	for (FPCGTaskId TaskId : InCollapsedTasks)
	{
		if (const UPCGComputeKernel* Kernel = InTaskIdToKernel[TaskId]; ensure(Kernel))
		{
			// Record attributes.
			TArray<FPCGKernelAttributeKey> AttributeKeys;
			Kernel->GetKernelAttributeKeys(AttributeKeys);

			for (const FPCGKernelAttributeKey& Key : AttributeKeys)
			{
				if (InOutComputeGraph->StaticAttributeTable.AddAttribute(Key) == INDEX_NONE)
				{
					UE_LOGFMT(LogPCG, Error, "FPCGGraphCompilerGPU: Static attribute table exceeded maximum size ({Maximum}), use the 'Dump Data Descriptions' setting on the GPU node(s) to list attributes that are present.",
						PCGDataCollectionPackingConstants::MAX_NUM_CUSTOM_ATTRS);
					break;
				}
			}

			// Record strings.
			Kernel->AddStaticCreatedStrings(InOutComputeGraph->StringTable);
		}
	}

	ensure(InOutComputeGraph->StaticAttributeTable.Num() <= PCGDataCollectionPackingConstants::MAX_NUM_CUSTOM_ATTRS);
}

void FPCGGraphCompilerGPU::CreateDataInterfaces(
	FPCGGPUCompilationContext& InOutContext,
	const TArray<FPCGGraphTask>& InCompiledTasks,
	const TSet<FPCGTaskId>& InCollapsedTasks,
	const TArray<FPCGTaskId>& InComputeElementTasks,
	const FTaskToSuccessors& InTaskSuccessors,
	const FOriginalToVirtualPin& InOriginalToVirtualPin,
	const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
	UPCGComputeGraph* InOutComputeGraph,
	FPinToDataInterface& InOutPinToDataInterface)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompilerGPU::CreateDataInterfaces);

	// Create data interfaces which allow kernels to read or write data. Each data interface is associated with a node output pin.
	// For CPU->GPU edges, an upload data interface is created. For GPU->CPU edges, a readback data interface is created.
	auto CreateOutputDataInterface = [&InOutContext, &InCollapsedTasks, &InCompiledTasks, &InTaskIdToKernel, InOutComputeGraph](FPCGTaskId InProducerTaskId, bool bRequiresExport, const FPCGPinProperties& InOutputPinProperties) -> UPCGComputeDataInterface*
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateDataInterface);

		// The kernel that produced this data, if any (otherwise could be CPU node or a node from a different compute graph).
		const UPCGComputeKernel* ProducerKernel = nullptr;
		if (UPCGComputeKernel* const* FoundKernel = InTaskIdToKernel.Find(InProducerTaskId))
		{
			ProducerKernel = *FoundKernel;
		}
		
		const UPCGSettings* ProducerSettings = nullptr;

		if (InCompiledTasks[InProducerTaskId].Node)
		{
			ProducerSettings = InCompiledTasks[InProducerTaskId].Node->GetSettings();
		}
		else if (InCompiledTasks[InProducerTaskId].Element && InCompiledTasks[InProducerTaskId].Element->IsGridLinkage())
		{
			// Special case - edges that cross grids without an explicit grid size node have a special grid linkage element. In this case
			// look up the producer settings from the link. This enables the data upload indicator to be displayed on the producer.
			const PCGGraphExecutor::FPCGGridLinkageElement& LinkElement = static_cast<const PCGGraphExecutor::FPCGGridLinkageElement&>(*InCompiledTasks[InProducerTaskId].Element);
			const UPCGPin* UpstreamPin = LinkElement.GetUpstreamPin();
			const UPCGNode* Node = UpstreamPin ? UpstreamPin->Node.Get() : nullptr;
			ProducerSettings = Node ? Node->GetSettings() : nullptr;
		}

		FPCGPinProperties PinProperties = InOutputPinProperties;

		PCGComputeHelpers::FCreateDataInterfaceParams Params;
		Params.Context = &InOutContext;
		Params.PinProperties = &PinProperties;
		Params.ProducerKernel = ProducerKernel;
		Params.ObjectOuter = InOutComputeGraph;
		Params.bProducedByCPU = !InCollapsedTasks.Contains(InProducerTaskId);
		Params.bRequiresExport = bRequiresExport;
		Params.NodeForDebug = InCompiledTasks[InProducerTaskId].Node;

		UPCGComputeDataInterface* DataInterface = nullptr;

		if (ProducerKernel)
		{
			// Allow kernel to decide which DI should be created.
			DataInterface = ProducerKernel->CreateOutputPinDataInterface(Params);
		}
		else
		{
			// Use default logic for DI creation.
			DataInterface = PCGComputeHelpers::CreateOutputPinDataInterface(Params);
		}

		if (DataInterface)
		{
			DataInterface->SetProducerKernel(ProducerKernel);
			DataInterface->SetProducerSettings(ProducerSettings);
			DataInterface->SetOutputPin(PinProperties.Label);
			DataInterface->SetProducedByCPU(Params.bProducedByCPU);
		}

		return DataInterface;
	};

	// Create DIs for output pins (either output pins within this compute graph or outside).
	for (FPCGTaskId TaskId : InCollapsedTasks)
	{
		TArray<FPCGPinProperties> OutputPinsProperties;

		// Create DIs for all output pins regardless of outbound connections, because the kernels currently need their outputs to be bound to valid resources.

		const UPCGComputeKernel* TaskKernel = nullptr;
		if (UPCGComputeKernel* const* FoundKernel = InTaskIdToKernel.Find(TaskId))
		{
			TaskKernel = *FoundKernel;
		}

		if (TaskKernel)
		{
			// Get output pins from kernel.
			TArray<FPCGPinPropertiesGPU> OutputPins;
			TaskKernel->GetOutputPins(OutputPins);

			for (const FPCGPinPropertiesGPU& Pin : OutputPins)
			{
				OutputPinsProperties.Add(Pin);
			}
		}
		else if (const UPCGSettings* Settings = InCompiledTasks[TaskId].Node ? InCompiledTasks[TaskId].Node->GetSettings() : nullptr)
		{
			// Get output pins from settings.
			OutputPinsProperties = Settings->AllOutputPinProperties();
		}

		// Create all the output data interfaces.
		for (const FPCGPinProperties& OutputPinProperties : OutputPinsProperties) 
		{
			if (OutputPinProperties.Label == NAME_None)
			{
				continue;
			}

			if (InOutPinToDataInterface.Contains({ TaskId, OutputPinProperties.Label }))
			{
				continue;
			}

			// Request buffer export from compute graph if any downstream task is outside of this compute graph.
			bool bRequiresExport = false;
			if (const TArray<FPCGTaskId>* Successors = InTaskSuccessors.Find(TaskId))
			{
				for (FPCGTaskId Successor : *Successors)
				{
					for (const FPCGGraphTaskInput& Input : InCompiledTasks[Successor].Inputs)
					{
						if (Input.TaskId == TaskId && Input.UpstreamPin && Input.UpstreamPin->Label == OutputPinProperties.Label && !InCollapsedTasks.Contains(Successor))
						{
							bRequiresExport = true;
							break;
						}
					}
				}
			}

			if (UPCGComputeDataInterface* OutputDI = CreateOutputDataInterface(TaskId, bRequiresExport, OutputPinProperties))
			{
				// Get the compute graph virtual pin corresponding to this pin, if one was created.
				const FName* VirtualPinLabel = InOriginalToVirtualPin.Find({ TaskId, OutputPinProperties.Label, /*bIsInputPin*/false });
				OutputDI->SetOutputPin(OutputPinProperties.Label, VirtualPinLabel);

				InOutComputeGraph->DataInterfaces.Add(OutputDI);

				InOutPinToDataInterface.Add({ TaskId, OutputPinProperties.Label }, OutputDI);

				// Iterate over downstream connections and register each downstream pin.
				if (const TArray<FPCGTaskId>* Successors = InTaskSuccessors.Find(TaskId))
				{
					for (FPCGTaskId Successor : *Successors)
					{
						for (const FPCGGraphTaskInput& Input : InCompiledTasks[Successor].Inputs)
						{
							if (Input.TaskId == TaskId && Input.UpstreamPin && Input.DownstreamPin)
							{
								// Get the compute graph virtual pin corresponding to this pin, if one was created.
								const FName* VirtualInputPinLabel = InOriginalToVirtualPin.Find({ TaskId, Input.DownstreamPin->Label, /*bIsInputPin=*/true });
								OutputDI->AddDownstreamInputPin(Input.DownstreamPin->Label, VirtualInputPinLabel);
							}
						}
					}
				}
			}
		}

		// Create any DIs for upstream nodes outside of this compute graph.
		for (const FPCGGraphTaskInput& Input : InCompiledTasks[TaskId].Inputs)
		{
			// Never wire directly to the GPU-compatible tasks, these are collapsed into compute graphs (and will be culled in a next step).
			if (InCollapsedTasks.Contains(Input.TaskId) || InComputeElementTasks.Contains(Input.TaskId))
			{
				continue;
			}

			if (!Input.DownstreamPin.IsSet())
			{
				continue;
			}

			if (!Input.UpstreamPin.IsSet())
			{
				continue;
			}

			if (TaskKernel && !TaskKernel->UploadInputPinDataToGPU(Input.DownstreamPin->Label))
			{
				UE_LOGF(LogPCG, Verbose, "Kernel '%ls' input pin '%ls' does not require data on GPU, skipped creating data interface.", *TaskKernel->GetName(), *Input.DownstreamPin->Label.ToString());
				continue;
			}

			UPCGComputeDataInterface* OutputDI = nullptr;

			if (UPCGComputeDataInterface** FoundOutputDI = InOutPinToDataInterface.Find({ Input.TaskId, Input.UpstreamPin->Label }))
			{
				OutputDI = *FoundOutputDI;
			}
			else if (UPCGComputeDataInterface* NewOutputDI = CreateOutputDataInterface(Input.TaskId, /*bRequiresExport=*/false, *Input.UpstreamPin))
			{
				OutputDI = NewOutputDI;

				// Get the compute graph virtual pin corresponding to this pin, if one was created.
				const FName* VirtualPinLabel = InOriginalToVirtualPin.Find({ Input.TaskId, Input.UpstreamPin->Label, /*bIsInputPin*/false });
				OutputDI->SetOutputPin(Input.UpstreamPin->Label, VirtualPinLabel);

				InOutComputeGraph->DataInterfaces.Add(OutputDI);

				InOutPinToDataInterface.Add({ Input.TaskId, Input.UpstreamPin->Label }, OutputDI);
			}

			if (OutputDI)
			{
				// Get the compute graph virtual pin corresponding to this pin, if one was created.
				const FName* VirtualPinLabel = InOriginalToVirtualPin.Find({ TaskId, Input.DownstreamPin->Label, /*bIsInputPin=*/true });
				OutputDI->AddDownstreamInputPin(Input.DownstreamPin->Label, VirtualPinLabel);
			}
		}
	}

	// Resolve bLegacyAllowSettersOnInputs on each UPCGDataCollectionDataInterface: a DI is legacy iff ANY of its consumer kernels is a UPCGCustomHLSLKernel
	// with bUseLegacyDataCollectionAPI=true. Only consumer-side legacy matters because the flag controls what GetSupportedInputs exposes - i.e. what functions
	// consumers see on their input pins.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ResolveDataInterfaceLegacy);

		auto IsKernelLegacy = [&InTaskIdToKernel](FPCGTaskId TaskId) -> bool
		{
			const UPCGComputeKernel* const* FoundKernel = InTaskIdToKernel.Find(TaskId);
			if (!FoundKernel || !*FoundKernel)
			{
				return false;
			}
			const UPCGCustomHLSLSettings* CustomHLSLSettings = Cast<UPCGCustomHLSLSettings>((*FoundKernel)->GetSettings());
			return CustomHLSLSettings && CustomHLSLSettings->GetUseLegacyDataCollectionAPI();
		};

		for (const TPair<TPair<FPCGTaskId, FName>, UPCGComputeDataInterface*>& PinAndDataInterface : InOutPinToDataInterface)
		{
			UPCGDataCollectionDataInterface* DataCollectionDI = Cast<UPCGDataCollectionDataInterface>(PinAndDataInterface.Value);
			if (!DataCollectionDI)
			{
				continue;
			}

			const FPCGTaskId ProducerTaskId = PinAndDataInterface.Key.Key;
			const FName ProducerPinLabel = PinAndDataInterface.Key.Value;

			bool bAnyConsumerLegacy = false;

			// Walk all consumers (any task with an input edge from this producer pin).
			for (FPCGTaskId TaskId = 0; !bAnyConsumerLegacy && TaskId < (FPCGTaskId)InCompiledTasks.Num(); ++TaskId)
			{
				for (const FPCGGraphTaskInput& TaskInput : InCompiledTasks[TaskId].Inputs)
				{
					if (TaskInput.TaskId == ProducerTaskId
						&& TaskInput.UpstreamPin.IsSet()
						&& TaskInput.UpstreamPin.GetValue().Label == ProducerPinLabel
						&& IsKernelLegacy(TaskId))
					{
						bAnyConsumerLegacy = true;
						break;
					}
				}
			}

			DataCollectionDI->bLegacyAllowSettersOnInputs = bAnyConsumerLegacy;
		}
	}
}

void FPCGGraphCompilerGPU::CompileComputeGraph(
	FPCGGPUCompilationContext& InOutContext,
	const TArray<FPCGGraphTask>& InCompiledTasks,
	const TSet<FPCGTaskId>& InCollapsedTasks,
	const TArray<FPCGTaskId>& InComputeElementTasks,
	const FTaskToSuccessors& InTaskSuccessors,
	const FOriginalToVirtualPin& InOriginalToVirtualPin,
	const FPinToDataInterface& InOutputPinToDataInterface,
	const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
	UPCGComputeGraph* InOutComputeGraph)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompilerGPU::CompileComputeGraph);

	TMap<const UComputeDataInterface*, TArray<FShaderFunctionDefinition>> DataInterfaceToInputs;
	TMap<const UComputeDataInterface*, TArray<FShaderFunctionDefinition>> DataInterfaceToOutputs;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CacheDataInterfaceInputOutputs);
		for (const TPair<TPair</*Node task*/FPCGTaskId, /*Pin label*/FName>, UPCGComputeDataInterface*>& PinAndDataInterface : InOutputPinToDataInterface)
		{
			// FindOrAdd is safe because the data interfaces in InOutputPinToDataInterface will be unique by definition, because every output pin maps to its own DI.
			TArray<FShaderFunctionDefinition>& InputDefinitions = DataInterfaceToInputs.FindOrAdd(PinAndDataInterface.Value);
			InputDefinitions.Reserve(128);
			PinAndDataInterface.Value->GetSupportedInputs(InputDefinitions);

			TArray<FShaderFunctionDefinition>& OutputDefinitions = DataInterfaceToOutputs.FindOrAdd(PinAndDataInterface.Value);
			OutputDefinitions.Reserve(128);
			PinAndDataInterface.Value->GetSupportedOutputs(OutputDefinitions);
		}
	}

	// Build compute graph by traversing the set of collapsed tasks in execution order (order of queued kernel invocations matters).
	TSet<FPCGTaskId> RemainingTasks;
	TArray<FPCGTaskId> ReadyTasks;
	RemainingTasks.Reserve(InCollapsedTasks.Num());
	ReadyTasks.Reserve(InCollapsedTasks.Num());

	for (FPCGTaskId TaskId : InCollapsedTasks)
	{
		bool bTaskReady = true;
		for (const FPCGGraphTaskInput& Input : InCompiledTasks[TaskId].Inputs)
		{
			if (InCollapsedTasks.Contains(Input.TaskId))
			{
				bTaskReady = false;
				break;
			}
		}

		if (bTaskReady)
		{
			ReadyTasks.Add(TaskId);
		}
		else
		{
			RemainingTasks.Add(TaskId);
		}
	}

	while (!ReadyTasks.IsEmpty())
	{
		FPCGTaskId TaskId = ReadyTasks.Pop();

		// Queue up any successors that are ready to go (do not depend on any remaining kernel).
		if (const TArray<FPCGTaskId>* Successors = InTaskSuccessors.Find(TaskId))
		{
			for (const FPCGTaskId Successor : *Successors)
			{
				if (!RemainingTasks.Contains(Successor))
				{
					continue;
				}

				bool bTaskReady = true;
				for (const FPCGGraphTaskInput& Input : InCompiledTasks[Successor].Inputs)
				{
					if (RemainingTasks.Contains(Input.TaskId) || ReadyTasks.Contains(Input.TaskId))
					{
						bTaskReady = false;
						break;
					}
				}

				if (bTaskReady)
				{
					ReadyTasks.Add(Successor);
					RemainingTasks.Remove(Successor);
				}
			}
		}

		const UPCGNode* Node = InCompiledTasks[TaskId].Node;

		const UPCGSettings* Settings = Node ? Node->GetSettings() : nullptr;
		check(Settings && Settings->bEnabled && Settings->ShouldExecuteOnGPU());

		struct FInterfaceBinding
		{
			const UComputeDataInterface* DataInterface = nullptr;
			int32 DataInterfaceBindingIndex = INDEX_NONE;
			FName Prefix = NAME_None;
			FName PinLabel = NAME_None;
			FString BindingFunctionName;
		};

		struct FKernelWithDataBindings
		{
			UPCGComputeKernel* Kernel = nullptr;
			TArray<FInterfaceBinding> InputDataBindings;
			TArray<FInterfaceBinding> OutputDataBindings;
		};

		FKernelWithDataBindings KernelWithBindings;

		const int32 KernelIndex = InOutComputeGraph->KernelInvocations.Num();
		KernelWithBindings.Kernel = InTaskIdToKernel[TaskId];
		check(KernelWithBindings.Kernel);
		KernelWithBindings.Kernel->SetKernelIndex(KernelIndex);

		InOutComputeGraph->KernelInvocations.Add(KernelWithBindings.Kernel);

		TArray<FPCGPinPropertiesGPU> KernelOutputPinProperties;
		KernelWithBindings.Kernel->GetOutputPins(KernelOutputPinProperties);

		if (const UPCGStaticMeshSpawnerKernel* SMSpawnerKernel = Cast<UPCGStaticMeshSpawnerKernel>(KernelWithBindings.Kernel))
		{
			InOutComputeGraph->StaticMeshSpawners.Add(SMSpawnerKernel);
		}

		// Populate static data labels. Cannot happen during BuildComputeGraphStaticData() because this relies on kernel index, which is not initialized at that time.
		{
			auto GetDataLabels = [Kernel=KernelWithBindings.Kernel, InOutComputeGraph](const auto& PinProperties)
			{
				for (const FPCGPinProperties& PinProps : PinProperties)
				{
					TArray<FString> DataLabels;
					Kernel->GetDataLabels(PinProps.Label, DataLabels);

					if (!DataLabels.IsEmpty())
					{
						FPCGPinDataLabels& PinDataLabels = InOutComputeGraph->StaticDataLabelsTable.FindOrAdd(Kernel->GetKernelIndex());
						FPCGDataLabels& Labels = PinDataLabels.PinToDataLabels.Add(PinProps.Label);
						Labels.Labels = MoveTemp(DataLabels);
					}
				}
			};

			TArray<FPCGPinProperties> InputPinProperties;
			KernelWithBindings.Kernel->GetInputPins(InputPinProperties);
			GetDataLabels(InputPinProperties);

			GetDataLabels(KernelOutputPinProperties);
		}

		struct FDataInterfaceInfo
		{
			FDataInterfaceInfo(int InIndex, FName InPrefix, FName InPinLabel)
				: Index(InIndex)
				, Prefix(InPrefix)
				, PinLabel(InPinLabel)
			{}

			int Index = INDEX_NONE;
			FName Prefix = NAME_None;
			FName PinLabel = NAME_None;
		};

		TArray<FDataInterfaceInfo> InputDataInterfaces;
		TArray<FDataInterfaceInfo> OutputDataInterfaces;
		InputDataInterfaces.Reserve(8);
		OutputDataInterfaces.Reserve(8);

		TMap</*OriginalPinLabel*/FName, /*VirtualPinLabel*/FName, TInlineSetAllocator<8>> OriginalToVirtualInputPin;

		auto CreateDataResolverDataInterface = [&InOutContext, Kernel = KernelWithBindings.Kernel, InOutComputeGraph](TArray<FDataInterfaceInfo>& InOutDataInterfaces, FName PinLabel, bool bIsInput)
		{
			// Add data label resolver DI if this pin uses any data labels.
			const FPCGPinDataLabels* PinDataLabels = InOutComputeGraph->GetStaticDataLabelsTable().Find(Kernel->GetKernelIndex());
			const FPCGDataLabels* DataLabels = PinDataLabels ? PinDataLabels->PinToDataLabels.Find(PinLabel) : nullptr;

			if (DataLabels && !DataLabels->Labels.IsEmpty())
			{
				UPCGDataLabelResolverDataInterface* Resolver = InOutContext.NewObject_AnyThread<UPCGDataLabelResolverDataInterface>(InOutComputeGraph);
				Resolver->Kernel = Kernel;
				Resolver->PinLabel = PinLabel;
				Resolver->bIsInput = bIsInput;

				const int ResolverIndex = InOutComputeGraph->DataInterfaces.Add(Resolver);
				InOutDataInterfaces.Emplace(ResolverIndex, FName(*PCGComputeHelpers::GetDataLabelResolverName(PinLabel)), /*PinLabel=*/NAME_None);
			}
		};

		// Setup input pin DIs.
		for (const FPCGGraphTaskInput& Input : InCompiledTasks[TaskId].Inputs)
		{
			// Currently the new compute graph tasks are wired into the graph in parallel to each GPU node task. The GPU node tasks
			// will be culled at the end, leaving only the compute graphs. Only create DIs for tasks within this compute graph (in InCollapsedTasks),
			// or for tasks that will not be culled (not in InAllGPUCompatibleTasks).
			const bool bValidInput = InCollapsedTasks.Contains(Input.TaskId) || !InComputeElementTasks.Contains(Input.TaskId);
			if (!bValidInput)
			{
				continue;
			}

			if (!Input.UpstreamPin || !Input.DownstreamPin)
			{
				// Execution-only dependencies not supported currently. Unclear if this should ever be supported for GPU graphs.
				// Writes followed by reads will be protected via barriers added by RDG.
				continue;
			}

			// Setup maps related to virtual pins.
			if (const FName* VirtualPin = InOriginalToVirtualPin.Find(FNodePin(TaskId, Input.DownstreamPin->Label, /*IsInput*/true)))
			{
				InOutComputeGraph->KernelPinToVirtualPinLabel.Add(FPCGKernelPin(KernelIndex, Input.DownstreamPin->Label, /*bInIsInput=*/true), *VirtualPin);
				InOutComputeGraph->VirtualInputPinToType.FindOrAdd(*VirtualPin) = Input.DownstreamPin->AllowedTypes;
				OriginalToVirtualInputPin.Add(Input.DownstreamPin->Label, *VirtualPin);
			}

			UPCGComputeDataInterface* const* FoundDI = InOutputPinToDataInterface.Find({ Input.TaskId, Input.UpstreamPin->Label });
			if (!FoundDI || !*FoundDI)
			{
				UE_LOGFMT(LogPCG, Verbose, "No data interface associated with pin '{Label}', expected if the pin opted out of creating one.", Input.UpstreamPin->Label);
				continue;
			}

			const int Index = InOutComputeGraph->DataInterfaces.Find(*FoundDI);
			if (Index == INDEX_NONE)
			{
				ensure(false);
				continue;
			}

			InputDataInterfaces.Emplace(Index, /*Prefix=*/Input.DownstreamPin->Label, /*PinLabel=*/Input.DownstreamPin->Label);

			CreateDataResolverDataInterface(InputDataInterfaces, Input.DownstreamPin->Label, /*bIsInput=*/true);
		}

		// Setup output pin DIs. Always bind a DI to every output pin, so kernel always has something to write to.
		for (const FPCGPinPropertiesGPU& OutputPinProperties : KernelOutputPinProperties)
		{
			if (OutputPinProperties.Label == NAME_None)
			{
				continue;
			}

			UPCGComputeDataInterface* const* FoundDI = InOutputPinToDataInterface.Find({ TaskId, OutputPinProperties.Label });
			if (!FoundDI || !*FoundDI)
			{
				UE_LOGFMT(LogPCG, Verbose, "No data interface associated with pin '{Label}', expected if the pin opted out of creating one.", OutputPinProperties.Label);
				continue;
			}

			const int Index = InOutComputeGraph->DataInterfaces.Find(*FoundDI);
			if (Index == INDEX_NONE)
			{
				ensure(false);
				continue;
			}

			OutputDataInterfaces.Emplace(Index, /*Prefix=*/OutputPinProperties.Label, /*PinLabel=*/OutputPinProperties.Label);
			CreateDataResolverDataInterface(OutputDataInterfaces, OutputPinProperties.Label, /*bIsInput=*/false);
		}

		// Kernel data interface.
		{
			UPCGCustomKernelDataInterface* KernelDI = InOutContext.NewObject_AnyThread<UPCGCustomKernelDataInterface>(InOutComputeGraph);
			KernelDI->SetSettings(Settings);
			KernelDI->Kernel = KernelWithBindings.Kernel;

			const int32 KernelDIIndex = InOutComputeGraph->DataInterfaces.Num();
			InOutComputeGraph->DataInterfaces.Add(KernelDI);

			// @todo_pcg: CustomKernel DI should probably have a prefix, e.g. "Kernel_"
			InputDataInterfaces.Emplace(KernelDIIndex, /*Prefix=*/NAME_None, /*PinLabel=*/NAME_None);
		}

		// Additional DIs created by kernels.
		{
			TArray<TObjectPtr<UComputeDataInterface>> AdditionalInputDIs, AdditionalOutputDIs;
			KernelWithBindings.Kernel->CreateAdditionalInputDataInterfaces(InOutContext, InOutComputeGraph, AdditionalInputDIs);
			KernelWithBindings.Kernel->CreateKernelParamsDataInterface(InOutContext, InOutComputeGraph, AdditionalInputDIs);
			KernelWithBindings.Kernel->CreateAdditionalOutputDataInterfaces(InOutContext, InOutComputeGraph, AdditionalOutputDIs);

			auto AddAdditionalDataInterface = [InOutComputeGraph](TObjectPtr<UComputeDataInterface> DataInterface) -> int32
			{
				if (ensure(DataInterface))
				{
					const int32 DataInterfaceIndex = InOutComputeGraph->DataInterfaces.Num();
					InOutComputeGraph->DataInterfaces.Add(DataInterface);

					return DataInterfaceIndex;
				}

				return static_cast<int32>(INDEX_NONE);
			};

			for (TObjectPtr<UComputeDataInterface> DataInterface : AdditionalInputDIs)
			{
				// TODO: This could also produce a prefix?
				const int32 DataInterfaceIndex = AddAdditionalDataInterface(DataInterface);

				if (DataInterfaceIndex != INDEX_NONE)
				{
					InputDataInterfaces.Emplace(DataInterfaceIndex, /*Prefix=*/NAME_None, /*PinLabel=*/NAME_None);
				}
			}

			for (TObjectPtr<UComputeDataInterface> DataInterface : AdditionalOutputDIs)
			{
				// TODO: This could also produce a prefix?
				const int32 DataInterfaceIndex = AddAdditionalDataInterface(DataInterface);

				if (DataInterfaceIndex != INDEX_NONE)
				{
					OutputDataInterfaces.Emplace(DataInterfaceIndex, /*Prefix=*/NAME_None, /*PinLabel=*/NAME_None);
				}
			}
		}

		// Debug data interface.
		if (Settings->bPrintShaderDebugValues)
		{
			UPCGDebugDataInterface* DebugDI = InOutContext.NewObject_AnyThread<UPCGDebugDataInterface>(InOutComputeGraph);
			DebugDI->SetDebugBufferSize(Settings->DebugBufferSize);
			DebugDI->SetProducerSettings(Settings);
			DebugDI->SetProducerKernel(KernelWithBindings.Kernel);

			const int32 DebugDIIndex = InOutComputeGraph->DataInterfaces.Num();
			InOutComputeGraph->DataInterfaces.Add(DebugDI);

			// TODO: Maybe debug DI should be prefixed with 'Debug', e.g. 'Debug_WriteValue()'.
			OutputDataInterfaces.Emplace(DebugDIIndex, /*Prefix=*/NAME_None, /*PinLabel=*/NAME_None);
		}

		InOutComputeGraph->bLogDataDescriptions |= Settings->bDumpDataDescriptions;
		InOutComputeGraph->bBreakDebugger |= Settings->bBreakDebugger;

		// Now that all data interfaces added, create the (trivial) binding mapping. All map to primary binding, index 0.
		InOutComputeGraph->DataInterfaceToBinding.SetNumZeroed(InOutComputeGraph->DataInterfaces.Num());

		InOutComputeGraph->KernelToNode.Add(Node);

		auto SetupAllInputBindings = [&KernelWithBindings, InOutComputeGraph, &DataInterfaceToInputs](const FDataInterfaceInfo& InDataInterfaceInfo)
		{
			const UComputeDataInterface* DataInterface = InOutComputeGraph->DataInterfaces[InDataInterfaceInfo.Index];
			check(DataInterface);

			TArray<FShaderFunctionDefinition>* FunctionsFromCache = DataInterfaceToInputs.Find(DataInterface);
			TArray<FShaderFunctionDefinition> RetrievedFunctions; // Used as local storage if we need to retrieve the functions here.
			if (!FunctionsFromCache)
			{
				RetrievedFunctions.Reserve(128);
				DataInterface->GetSupportedInputs(RetrievedFunctions);
			}

			TArray<FShaderFunctionDefinition>& Functions = FunctionsFromCache ? *FunctionsFromCache : RetrievedFunctions;

			for (int FuncIndex = 0; FuncIndex < Functions.Num(); ++FuncIndex)
			{
				FInterfaceBinding& Binding = KernelWithBindings.InputDataBindings.Emplace_GetRef();
				Binding.DataInterface = DataInterface;
				Binding.Prefix = InDataInterfaceInfo.Prefix;
				Binding.PinLabel = InDataInterfaceInfo.PinLabel;
				Binding.BindingFunctionName = Functions[FuncIndex].Name;
				Binding.DataInterfaceBindingIndex = FuncIndex;
			}

			if (!FunctionsFromCache)
			{
				// Augment the DI function cache with any additional data interfaces created for the current task.
				DataInterfaceToInputs.Add(DataInterface, MoveTemp(RetrievedFunctions));
			}
		};

		auto SetupAllOutputBindings = [&KernelWithBindings, InOutComputeGraph, &DataInterfaceToOutputs](const FDataInterfaceInfo& InDataInterfaceInfo)
		{
			const UComputeDataInterface* DataInterface = InOutComputeGraph->DataInterfaces[InDataInterfaceInfo.Index];
			check(DataInterface);

			TArray<FShaderFunctionDefinition>* FunctionsFromCache = DataInterfaceToOutputs.Find(DataInterface);
			TArray<FShaderFunctionDefinition> RetrievedFunctions; // Used as local storage if we need to retrieve the functions here.
			if (!FunctionsFromCache)
			{
				RetrievedFunctions.Reserve(128);
				DataInterface->GetSupportedOutputs(RetrievedFunctions);
			}

			TArray<FShaderFunctionDefinition>& Functions = FunctionsFromCache ? *FunctionsFromCache : RetrievedFunctions;

			for (int FuncIndex = 0; FuncIndex < Functions.Num(); ++FuncIndex)
			{
				FInterfaceBinding& Binding = KernelWithBindings.OutputDataBindings.Emplace_GetRef();
				Binding.DataInterface = DataInterface;
				Binding.Prefix = InDataInterfaceInfo.Prefix;
				Binding.PinLabel = InDataInterfaceInfo.PinLabel;
				Binding.BindingFunctionName = Functions[FuncIndex].Name;
				Binding.DataInterfaceBindingIndex = FuncIndex;
			}

			if (!FunctionsFromCache)
			{
				// Augment the DI function cache with any additional data interfaces created for the current task.
				DataInterfaceToOutputs.Add(DataInterface, MoveTemp(RetrievedFunctions));
			}
		};

		// Bind data interfaces.
		for (const FDataInterfaceInfo& DataInterfaceInfo : InputDataInterfaces)
		{
			SetupAllInputBindings(DataInterfaceInfo);
		}

		for (const FDataInterfaceInfo& DataInterfaceInfo : OutputDataInterfaces)
		{
			SetupAllOutputBindings(DataInterfaceInfo);
		}

		{
			UPCGComputeKernelSource* KernelSource = InOutContext.NewObject_AnyThread<UPCGComputeKernelSource>(KernelWithBindings.Kernel);
			KernelWithBindings.Kernel->KernelSource = KernelSource;

			// These could be exposed through PCGSettings API later when the need arises (and/or when GPU feature matures).
			KernelSource->EntryPoint = KernelWithBindings.Kernel->GetEntryPoint();
			KernelSource->GroupSize = FIntVector(PCGComputeConstants::THREAD_GROUP_SIZE, 1, 1);

			// All kernels require ComputeShaderUtils.ush, so inject that before anything else.
			FString Source = TEXT("#include \"/Engine/Private/ComputeShaderUtils.ush\"\n\n");

			// Generate parameter override functions.
			for (const FPCGSettingsOverridableParam& OverridableParam : Settings->OverridableParams())
			{
				if (OverridableParam.bSupportsGPU)
				{
					const FPCGKernelOverridableParamTypeInfo& TypeInfo = FPCGKernelOverridableParamTypeInfo::Get(OverridableParam.UnderlyingType);
					const FString ParamLabel = OverridableParam.Label.ToString();

					if (!OverridableParam.bRequiresGPUReadback && Settings->IsPropertyOverriddenByPin(OverridableParam.PropertiesNames))
					{
						Source += FString::Format(TEXT(
							"{1} {0}_GetOverridableValue()\n"
							"{\n"
							"	return {0}_GetFirstAttributeAs{2}();\n"
							"}\n"
							"\n"), { ParamLabel, TypeInfo.HLSLType, TypeInfo.AccessorSuffix });
					}
					else
					{
						Source += FString::Format(TEXT(
							"{1} {0}_GetOverridableValue()\n"
							"{\n"
							"	return Get{0}Internal();\n"
							"}\n"
							"\n"), { ParamLabel, TypeInfo.HLSLType });
					}
				}
			}
 
			Source += KernelWithBindings.Kernel->GetCookedSource(InOutContext);
			KernelSource->SetSource(MoveTemp(Source));
			KernelWithBindings.Kernel->GatherAdditionalSources(KernelSource->AdditionalSources);

			if (Settings->bDumpCookedHLSL)
			{
				UE_LOGF(LogPCG, Log, "Cooked HLSL (%ls - %ls):\n%ls\n", *Settings->GetName(), *KernelWithBindings.Kernel->GetName(), *KernelSource->GetSource());
			}

#if PCG_KERNEL_LOGGING_ENABLED
			if (PCGGraphCompilerGPU::CVarEnableDebugging.GetValueOnAnyThread())
			{
				UE_LOGF(LogPCG, Display, "\n### STATIC METADATA ATTRIBUTE TABLE ###");
				InOutComputeGraph->StaticAttributeTable.DebugLog();
			}
#endif

			// Add functions for external inputs/outputs which must be fulfilled by DIs
			for (FInterfaceBinding& Binding : KernelWithBindings.InputDataBindings)
			{
				TArray<FShaderFunctionDefinition>& Functions = DataInterfaceToInputs.FindChecked(Binding.DataInterface);
				check(Functions.IsValidIndex(Binding.DataInterfaceBindingIndex));

				FShaderFunctionDefinition FuncDef = Functions[Binding.DataInterfaceBindingIndex];
				for (FShaderParamTypeDefinition& ParamType : FuncDef.ParamTypes)
				{
					// Making sure parameter has type declaration generated
					ParamType.ResetTypeDeclaration();
				}

				KernelSource->ExternalInputs.Emplace(FuncDef);
			}

			for (FInterfaceBinding& Binding : KernelWithBindings.OutputDataBindings)
			{
				TArray<FShaderFunctionDefinition>& Functions = DataInterfaceToOutputs.FindChecked(Binding.DataInterface);
				check(Functions.IsValidIndex(Binding.DataInterfaceBindingIndex));

				FShaderFunctionDefinition FuncDef = Functions[Binding.DataInterfaceBindingIndex];
				for (FShaderParamTypeDefinition& ParamType : FuncDef.ParamTypes)
				{
					// Making sure parameter has type declaration generated
					ParamType.ResetTypeDeclaration();
				}

				KernelSource->ExternalOutputs.Emplace(FuncDef);
			}
		}

		auto AddAllEdgesForKernel = [&KernelWithBindings, InOutComputeGraph, &OriginalToVirtualInputPin, &DataInterfaceToInputs, &DataInterfaceToOutputs](int32 InKernelIndex, bool bInEdgesAreInputs)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AddAllEdgesForKernel);
			TArray<FInterfaceBinding>& Bindings = bInEdgesAreInputs ? KernelWithBindings.InputDataBindings : KernelWithBindings.OutputDataBindings;
			InOutComputeGraph->GraphEdges.Reserve(InOutComputeGraph->GraphEdges.Num() + Bindings.Num());

			// Add all graph edges for bindings, which means include all functions that the data interfaces expose.
			for (int KernelBindingIndex = 0; KernelBindingIndex < Bindings.Num(); ++KernelBindingIndex)
			{
				FInterfaceBinding& Binding = Bindings[KernelBindingIndex];

				// Edge in compute graph is more correlated with a binding (it more describes a kernel input or output rather than a connection
				// between two kernels) so we use "binding" terminology.
				const int32 GraphBindingIndex = InOutComputeGraph->GraphEdges.Num();
				FComputeGraphEdge& Edge = InOutComputeGraph->GraphEdges.AddDefaulted_GetRef();

				Edge.KernelIndex = InKernelIndex;
				Edge.KernelBindingIndex = KernelBindingIndex;
				Edge.DataInterfaceIndex = InOutComputeGraph->DataInterfaces.IndexOfByPredicate([&Binding](const UComputeDataInterface* In) { return Binding.DataInterface == In; });
				check(Edge.DataInterfaceIndex != INDEX_NONE);
				Edge.DataInterfaceBindingIndex = Binding.DataInterfaceBindingIndex;
				Edge.bKernelInput = bInEdgesAreInputs;

				UComputeDataInterface* DataInterface = InOutComputeGraph->DataInterfaces[Edge.DataInterfaceIndex];
				check(DataInterface);

				if (Binding.Prefix != NAME_None)
				{
					TArray<FShaderFunctionDefinition>& DataInterfaceFunctions = bInEdgesAreInputs ? DataInterfaceToInputs.FindChecked(DataInterface) : DataInterfaceToOutputs.FindChecked(DataInterface);

					// Strip the "_PCG_SRV" / "_PCG_UAV" suffix from the function name so the user-facing wrap name is direction-agnostic (e.g. In_Load / Out_Load, regardless of which underlying body is bound).
					// The full suffixed name is still used by the shim to resolve the underlying body in the .ush.
					FString PublicFuncName = DataInterfaceFunctions[Edge.DataInterfaceBindingIndex].Name;
					PCGComputeHelpers::StripSRVUAVSuffix(PublicFuncName);

					Edge.BindingFunctionNameOverride = FString::Format(
						TEXT("{0}_{1}"),
						{ PCGComputeHelpers::SanitizePinLabelForHLSL(Binding.Prefix), PublicFuncName }
					);
				}

				if (Binding.PinLabel != NAME_None)
				{
					// A binding corresponds to a single API within a data interface, like GetNumData() for example. There are multiple bindings
					// per PCG graph edge and we only need to create our mappings for the first binding.
					if (Edge.DataInterfaceBindingIndex == 0)
					{
						const FPCGKernelPin KernelPin(InKernelIndex, Binding.PinLabel, bInEdgesAreInputs);

						InOutComputeGraph->KernelBindingToPinLabel.Add(GraphBindingIndex, Binding.PinLabel);

						if (!InOutComputeGraph->KernelPinToFirstBinding.Contains(KernelPin))
						{
							InOutComputeGraph->KernelPinToFirstBinding.Add(KernelPin, GraphBindingIndex);
						}

						if (const FName* VirtualPin = bInEdgesAreInputs ? OriginalToVirtualInputPin.Find(Binding.PinLabel) : nullptr)
						{
							InOutComputeGraph->CPUDataBindingToVirtualPinLabel.Add(GraphBindingIndex, *VirtualPin);
						}

						UPCGComputeDataInterface* PCGDataInterface = Cast<UPCGComputeDataInterface>(DataInterface);
						if (PCGDataInterface && PCGDataInterface->GetGraphBindingIndex() == INDEX_NONE)
						{
							PCGDataInterface->SetGraphBindingIndex(GraphBindingIndex);
						}
					}
				}
			}
		};

		AddAllEdgesForKernel(KernelIndex, /*bInEdgesAreInputs=*/true);
		AddAllEdgesForKernel(KernelIndex, /*bInEdgesAreInputs=*/false);
	}

	ensureMsgf(RemainingTasks.IsEmpty(), TEXT("PCG GPU graph compiler did not consume all tasks, %d/%d remaining."), RemainingTasks.Num(), InCollapsedTasks.Num());

	// Setup DownstreamToUpstreamBinding to assist in traversing up kernel->kernel connections.
	// TODO: optimize quadratic loop.
	for (int32 DownstreamEdgeIndex = 0; DownstreamEdgeIndex < InOutComputeGraph->GraphEdges.Num(); ++DownstreamEdgeIndex)
	{
		const FComputeGraphEdge& DownstreamEdge = InOutComputeGraph->GraphEdges[DownstreamEdgeIndex];
		const bool bFirstEdgeOfDI = DownstreamEdge.DataInterfaceBindingIndex == 0;

		// Need only one direction (only need to follow edges upstream).
		if (DownstreamEdge.bKernelInput && bFirstEdgeOfDI)
		{
			// Find corresponding matching edge - different kernel, same data interface, and bound as output.
			for (int32 UpstreamEdgeIndex = 0; UpstreamEdgeIndex < InOutComputeGraph->GraphEdges.Num(); ++UpstreamEdgeIndex)
			{
				const FComputeGraphEdge& UpstreamEdge = InOutComputeGraph->GraphEdges[UpstreamEdgeIndex];
				const bool bFirstEdgeOfUpstreamDI = UpstreamEdge.DataInterfaceBindingIndex == 0;

				if (bFirstEdgeOfUpstreamDI && UpstreamEdge.KernelIndex != DownstreamEdge.KernelIndex && UpstreamEdge.DataInterfaceIndex == DownstreamEdge.DataInterfaceIndex && !UpstreamEdge.bKernelInput)
				{
#if PCG_DATA_DESCRIPTION_LOGGING
					UE_LOGF(LogPCG, Warning, "Connecting edge %d %ls (KBD: %d) to edge %d %ls (KBD: %d)",
						DownstreamEdge.KernelIndex,
						*DownstreamEdge.BindingFunctionNameOverride,
						DownstreamEdgeIndex,
						UpstreamEdge.KernelIndex,
						*UpstreamEdge.BindingFunctionNameOverride,
						UpstreamEdge.KernelBindingIndex);

					UE_LOGF(LogPCG, Warning, "\tUpstream is kernel %d, DI '%ls', edge index %d ('%ls')",
						UpstreamEdge.KernelIndex,
						*InOutComputeGraph->DataInterfaces[UpstreamEdge.DataInterfaceIndex]->GetName(),
						UpstreamEdgeIndex,
						*InOutComputeGraph->GraphEdges[UpstreamEdge.KernelBindingIndex].BindingFunctionNameOverride);
					UE_LOGF(LogPCG, Warning, "\tDownstream is kernel %d, DI '%ls', edge index %d ('%ls')",
						DownstreamEdge.KernelIndex,
						*InOutComputeGraph->DataInterfaces[DownstreamEdge.DataInterfaceIndex]->GetName(),
						DownstreamEdgeIndex,
						*InOutComputeGraph->GraphEdges[DownstreamEdge.KernelBindingIndex].BindingFunctionNameOverride);
#endif

					InOutComputeGraph->DownstreamToUpstreamBinding.Add(DownstreamEdgeIndex, UpstreamEdgeIndex);
				}
			}
		}
	}
}

void FPCGGraphCompilerGPU::CreateGPUNodes(FPCGGraphCompiler& InOutCompiler, UPCGGraph* InGraph, uint32 InGridSize, TArray<FPCGGraphTask>& InOutCompiledTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphCompilerGPU::CreateGPUNodes);

	if (!InGraph)
	{
		ensure(false);
		return;
	}

	FPCGGPUCompilationContext Context(InOutCompiler);

	TSet<FPCGTaskId> GPUCompatibleTaskIds;
	GPUCompatibleTaskIds.Reserve(InOutCompiledTasks.Num());
	for (FPCGTaskId TaskId = 0; TaskId < InOutCompiledTasks.Num(); ++TaskId)
	{
		const UPCGNode* Node = InOutCompiledTasks[TaskId].Node;
		const UPCGSettings* Settings = Node ? Node->GetSettings() : nullptr;
		if (Settings && Settings->ShouldExecuteOnGPU() && Settings->bEnabled)
		{
			GPUCompatibleTaskIds.Add(TaskId);
		}
	}

	if (GPUCompatibleTaskIds.IsEmpty())
	{
		// Nothing to do for this graph.
		return;
	}

	// For input pins at CPU -> GPU boundary, inject gather elements to pre-combine data on CPU side
	// before passing to GPU.
	CreateGatherTasksAtGPUInputs(InGraph, GPUCompatibleTaskIds, InOutCompiledTasks);

	TMap<FPCGTaskId, UPCGComputeKernel*> TaskIdToKernel;

	// Create one or more kernel objects for each GPU node. A graph task will be created for each.
	ExpandGPUNodeKernelsToTasks(Context, GPUCompatibleTaskIds, InOutCompiledTasks, TaskIdToKernel);

	FTaskToSuccessors TaskSuccessors;
	TaskSuccessors.Reserve(InOutCompiledTasks.Num());
	for (FPCGTaskId TaskId = 0; TaskId < InOutCompiledTasks.Num(); ++TaskId)
	{
		for (int InputIndex = 0; InputIndex < InOutCompiledTasks[TaskId].Inputs.Num(); ++InputIndex)
		{
			// Only add unique task successors to avoid storing the data from each output pin
			// multiple times when multiple pins connect to the same downstream node.
			TaskSuccessors.FindOrAdd(InOutCompiledTasks[TaskId].Inputs[InputIndex].TaskId).AddUnique(TaskId);
		}
	}

	TArray<TSet<FPCGTaskId>> NodeSubsetsToConvertToCFGraph;
	CollectGPUNodeSubsets(InOutCompiledTasks, TaskSuccessors, GPUCompatibleTaskIds, TaskIdToKernel, NodeSubsetsToConvertToCFGraph);

	const uint32 NumComputeGraphs = NodeSubsetsToConvertToCFGraph.Num();

	// Mapping from task ID & pin label to a virtual pin label. Compute graphs are executed within a generated element,
	// and the input and output pins of this element must have unique virtual pin labels so that we can parse the data that
	// PCG provides through the input data collection correctly, and route the output data to the downstream pins correctly.
	TArray<FOriginalToVirtualPin> OriginalToVirtualPin;
	OriginalToVirtualPin.SetNum(NumComputeGraphs);

	// Setup mappings from existing pins to compute graph element virtual pins as a prestep before wiring in the compute graph tasks.
	for (uint32 ComputeGraphIndex = 0; ComputeGraphIndex < NumComputeGraphs; ++ComputeGraphIndex)
	{
		SetupVirtualPins(NodeSubsetsToConvertToCFGraph[ComputeGraphIndex], InOutCompiledTasks, TaskSuccessors, TaskIdToKernel, OriginalToVirtualPin[ComputeGraphIndex]);
	}

	TArray<FPCGTaskId> ComputeElementTasks;

	// Build each compute graph.
	for (uint32 ComputeGraphIndex = 0; ComputeGraphIndex < NumComputeGraphs; ++ComputeGraphIndex)
	{
		TSet<FPCGTaskId>& NodeSubsetToConvertToCFGraph = NodeSubsetsToConvertToCFGraph[ComputeGraphIndex];

		if (NodeSubsetToConvertToCFGraph.IsEmpty())
		{
			ensure(false);
			continue;
		}

		// Add a new compute graph task. The original GPU tasks will then be culled later below.
		const FPCGTaskId ComputeGraphTaskId = InOutCompiledTasks.Num();
		ComputeElementTasks.Add(ComputeGraphTaskId);
		FPCGGraphTask& ComputeGraphTask = InOutCompiledTasks.Emplace_GetRef();
		ComputeGraphTask.NodeId = ComputeGraphTaskId;

		// All nodes in subset will be from same stack/parent, so assign from any.
		if (auto It = NodeSubsetToConvertToCFGraph.CreateConstIterator(); It)
		{
			FPCGTaskId GPUTaskId = *It;
			ComputeGraphTask.ParentId = InOutCompiledTasks[GPUTaskId].ParentId;
			ComputeGraphTask.StackIndex = InOutCompiledTasks[GPUTaskId].StackIndex;
		}

		// Wire in the compute graph task, side by side with the individual GPU tasks, which will be culled below.
		WireComputeGraphTask(
			ComputeGraphTaskId,
			NodeSubsetToConvertToCFGraph,
			InOutCompiledTasks,
			TaskSuccessors,
			OriginalToVirtualPin[ComputeGraphIndex]);

		// Generate a compute graph from all of the individual GPU tasks.
		BuildComputeGraphTask(
			Context,
			ComputeGraphTaskId,
			InGraph,
			InGridSize,
			ComputeGraphIndex,
			NodeSubsetToConvertToCFGraph,
			ComputeElementTasks,
			TaskSuccessors,
			InOutCompiledTasks,
			OriginalToVirtualPin[ComputeGraphIndex],
			TaskIdToKernel);
	}

	{
		PCG::TUniqueScopeLock WriteLock(InOutCompiler.GetCache().GraphToTaskMapLock);
		TMap<uint32, TArray<TObjectPtr<UPCGComputeGraph>>>& GridSizeToComputeGraphs = InOutCompiler.GetCache().TopGraphToComputeGraphMap.FindOrAdd(InGraph);
		TArray<TObjectPtr<UPCGComputeGraph>>& ComputeGraphs = GridSizeToComputeGraphs.FindOrAdd(InGridSize);

		// Replace any existing compute graphs with the newly compiled ones. It's okay if multiple threads do this, because
		// compute graph index order should be deterministic, so different threads will produce the same results.
		ComputeGraphs = MoveTemp(Context.GetCompiledComputeGraphs());
		ensure(ComputeGraphs.Num() == NumComputeGraphs);
	}

	// Now cull all the GPU compatible nodes. The compute graph task are already wired in so we're fine to just delete.
	FPCGGraphCompiler::CullTasks(InOutCompiledTasks, /*bAddPassthroughWires=*/false, [&NodeSubsetsToConvertToCFGraph](const FPCGGraphTask& InTask)
	{
		for (TSet<FPCGTaskId>& NodeSubsetToConvertToCFGraph : NodeSubsetsToConvertToCFGraph)
		{
			if (NodeSubsetToConvertToCFGraph.Contains(InTask.NodeId))
			{
				return true;
			}
		}

		return false;
	});
}

#undef PCG_DATA_DESCRIPTION_LOGGING 

#endif // WITH_EDITOR
