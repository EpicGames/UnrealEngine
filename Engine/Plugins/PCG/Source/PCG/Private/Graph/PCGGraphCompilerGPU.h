// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

class FPCGGraphCompiler;
class UPCGComputeDataInterface;
class UPCGComputeGraph;
class UPCGComputeKernel;
class UPCGGraph;
struct FPCGGraphTask;
struct FPCGGPUCompilationContext;

#if WITH_EDITOR
class FPCGGraphCompilerGPU
{
public:
	/** Finds connected subgraphs of GPU - enabled nodes that can be dispatched together and replaces each one with a compute graph. */
	static void CreateGPUNodes(FPCGGraphCompiler& InOutCompiler, UPCGGraph* InGraph, uint32 InGridSize, TArray<FPCGGraphTask>& InOutCompiledTasks);

private:
	/** Used to track new unique virtual pins created on generated compute graph elements. */
	using FNodePin = TTuple<FPCGTaskId, /*Pin label*/FName, /*Pin is input*/bool>;
	using FOriginalToVirtualPin = TMap<FNodePin, /*Virtual pin label*/FName>;
	using FTaskToSuccessors = TMap<FPCGTaskId, TArray<FPCGTaskId>>;
	using FPinToDataInterface = TMap<TPair</*Node task*/FPCGTaskId, /*Pin label*/FName>, UPCGComputeDataInterface*>;

	/** Identifies connected sets of GPU nodes, giving each a non - zero ID value. */
	static void LabelConnectedGPUNodeIslands(
		const TArray<FPCGGraphTask>& InCompiledTasks,
		const TSet<FPCGTaskId>& InGPUCompatibleTaskIds,
		const FTaskToSuccessors& InTaskSuccessors,
		const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
		TArray<uint32>& OutIslandIDs);

	/** Outputs sets of task IDs, where each set is GPU nodes that can be compiled into a compute graph and dispatched together. */
	static void CollectGPUNodeSubsets(
		const TArray<FPCGGraphTask>& InCompiledTasks,
		const FTaskToSuccessors& InTaskSuccessors,
		const TSet<FPCGTaskId>& InGPUCompatibleTaskIds,
		const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
		TArray<TSet<FPCGTaskId>>& OutNodeSubsetsToConvertToCFGraph);
	
	/** Create a separate task for each GPU kernel emitted by each GPU node. */
	static void ExpandGPUNodeKernelsToTasks(
		FPCGGPUCompilationContext& InOutContext,
		TSet<FPCGTaskId>& InOutGPUCompatibleTaskIds,
		TArray<FPCGGraphTask>& InOutCompiledTasks,
		TMap<FPCGTaskId, UPCGComputeKernel*>& OutTaskIdToKernel);

	/** For GPU node inputs that have multiple incident edges, bundle them into a single edge. This is to avoid an inefficient
	* gather operation on the GPU, and allows data interfaces to pick their data from the compute graph element input data collection
	* using unique virtual input pin labels. */
	static void CreateGatherTasksAtGPUInputs(UPCGGraph* InGraph, const TSet<FPCGTaskId>& InGPUCompatibleTaskIds, TArray<FPCGGraphTask>& InOutCompiledTasks);
	
	/** The compute graph element does not have a node or pins, so create maps of unique "virtual pins" that will allow us to marshal
	 * data properly from an input data collection at execution time. */
	static void SetupVirtualPins(
		const TSet<FPCGTaskId>& InCollapsedTasks,
		const TArray<FPCGGraphTask>& InCompiledTasks,
		const TMap<FPCGTaskId, TArray<FPCGTaskId>>& InTaskSuccessors,
		const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
		FOriginalToVirtualPin& OutOriginalToVirtualPin);

	/** Wires in a compute graph task alongside each set of GPU compatible nodes. The tasks for each node will be culled later. */
	static void WireComputeGraphTask(
		FPCGTaskId InGPUGraphTaskId,
		const TSet<FPCGTaskId>& InCollapsedTasks,
		TArray<FPCGGraphTask>& InOutCompiledTasks,
		const TMap<FPCGTaskId, TArray<FPCGTaskId>>& InTaskSuccessors,
		const FOriginalToVirtualPin& InOriginalToVirtualPin);
	
	/** Creates compute graph with all resources and data interfaces and sets up the compute graph task. */
	static void BuildComputeGraphTask(
		FPCGGPUCompilationContext& InOutContext,
		FPCGTaskId InGPUGraphTaskId,
		UPCGGraph* InGraph,
		uint32 InGridSize,
		uint32 InComputeGraphIndex,
		const TSet<FPCGTaskId>& InCollapsedTasks,
		const TArray<FPCGTaskId>& InComputeElementTasks,
		const FTaskToSuccessors& InTaskSuccessors,
		TArray<FPCGGraphTask>& InOutCompiledTasks,
		const FOriginalToVirtualPin& InOriginalToVirtualPin,
		TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel);

	/** Collates data for the compute graph which is statically known. */
	static void BuildComputeGraphStaticData(
		const TArray<FPCGGraphTask>& InCompiledTasks,
		const TSet<FPCGTaskId>& InCollapsedTasks,
		const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
		UPCGComputeGraph* InOutComputeGraph);

	/** Creates and configures all data interfaces for the graph. */
	static void CreateDataInterfaces(
		FPCGGPUCompilationContext& InOutContext,
		const TArray<FPCGGraphTask>& InCompiledTasks,
		const TSet<FPCGTaskId>& InCollapsedTasks,
		const TArray<FPCGTaskId>& InComputeElementTasks,
		const FTaskToSuccessors& InTaskSuccessors,
		const FOriginalToVirtualPin& InOriginalToVirtualPin,
		const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
		UPCGComputeGraph* InOutComputeGraph,
		FPinToDataInterface& InOutPinToDataInterface);

	/** Builds the compute graph from the input tasks and data interfaces. */
	static void CompileComputeGraph(
		FPCGGPUCompilationContext& InOutContext,
		const TArray<FPCGGraphTask>& InCompiledTasks,
		const TSet<FPCGTaskId>& InCollapsedTasks,
		const TArray<FPCGTaskId>& InComputeElementTasks,
		const FTaskToSuccessors& InTaskSuccessors,
		const FOriginalToVirtualPin& InOriginalToVirtualPin,
		const FPinToDataInterface& InOutputPinToDataInterface,
		const TMap<FPCGTaskId, UPCGComputeKernel*>& InTaskIdToKernel,
		UPCGComputeGraph* InOutComputeGraph);
};
#endif // WITH_EDITOR
