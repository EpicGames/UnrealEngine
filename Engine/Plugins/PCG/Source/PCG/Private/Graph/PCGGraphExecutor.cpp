// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGGraphExecutor.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGCrc.h"
#include "PCGGraph.h"
#include "PCGInputOutputSettings.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "PCGProfilingLog.h"
#include "PCGSubsystem.h"
#include "PCGWorldActor.h"
#include "Data/PCGUserParametersData.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Subsystems/PCGEngineSubsystem.h"
#include "Utils/PCGGraphExecutionLogging.h"

#include "Algo/AnyOf.h"
#include "Algo/ForEach.h"
#include "Engine/Engine.h"
#include "GameFramework/Actor.h"
#include "Misc/Crc.h"
#include "Misc/ScopeExit.h"
#include "Tasks/Task.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/IPCGEditorModule.h"
#include "Modules/ModuleManager.h"
#else
#include "GameFramework/Actor.h"
#endif

// World partition support for in-editor workflows needs these includes
#if WITH_EDITOR
#include "FileHelpers.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGraphExecutor)

namespace PCGGraphExecutor
{
	TAutoConsoleVariable<int32> CVarRuntimeMaxNumProcessingTasks(
		TEXT("pcg.Runtime.MaxProcessingTasks"),
		4096,
		TEXT("Runtime: Maximum number of concurrent tasks for PCG Async processing"));

	TAutoConsoleVariable<float> CVarRuntimeMaxPercentageOfThreadsToUse(
		TEXT("pcg.Runtime.MaxPercentageOfThreadsToUse"),
		0.9f,
		TEXT("Runtime: Maximum percentage of number of threads for all concurrent PCG processing/execution"));

	TAutoConsoleVariable<float> CVarRuntimeMaxPercentageOfExecutingThreads(
		TEXT("pcg.Runtime.MaxPercentageOfExecutingThreads"),
		0.3f,
		TEXT("Runtime: Maximum percentage of number of threads for processing scheduled pcg elements (percentage of pcg.Runtime.MaxPercentageOfThreadsToUse)"));

	TAutoConsoleVariable<int32> CVarRuntimeMaxExecutingThreads(
		TEXT("pcg.Runtime.MaxExecutingThreads"),
		8,
		TEXT("Runtime: Hard cap on number of threads for processing scheduled pcg elements calculated from pcg.Runtime.MaxPercentageOfExecutingThreads"));

	// Disabled by default because on lower end platforms it is a performance choice (adding a dedicated thread considering lower number of cores might cause less scheduling contention which this consumer thread tries to alleviate)
	TAutoConsoleVariable<bool> CVarRuntimeEnablePostExecuteConsumerThread(
		TEXT("pcg.Runtime.EnablePostExecuteConsumerThread"),
		false,
		TEXT("Runtime: When enabled, a single thread will process all post executes and scheduling, instead of doing it directly in the async the executing threads"));

	TAutoConsoleVariable<float> CVarRuntimePostExecuteConsumerThreadTime(
		TEXT("pcg.Runtime.PostExecuteConsumerThreadTime"),
		5.0f,
		TEXT("Runtime: Allocated time the consumer thread will use before yielding (when pcg.Runtime.EnablePostExecuteConsumerThread is enabled)"));

	TAutoConsoleVariable<float> CVarTimePerFrame(
		TEXT("pcg.FrameTime"),
		5.0f,
		TEXT("Allocated time in ms per frame"));

	TAutoConsoleVariable<bool> CVarGraphMultithreading(
		TEXT("pcg.GraphMultithreading"),
		true,
		TEXT("Controls whether the graph can dispatch multiple tasks at the same time"));

#if WITH_EDITOR
	TAutoConsoleVariable<int32> CVarEditorMaxNumProcessingTasks(
		TEXT("pcg.Editor.MaxProcessingTasks"),
		4096,
		TEXT("Editor: Maximum number of concurrent tasks for PCG Async processing"));

	TAutoConsoleVariable<float> CVarEditorMaxPercentageOfThreadsToUse(
		TEXT("pcg.Editor.MaxPercentageOfThreadsToUse"),
		0.9f,
		TEXT("Editor: Maximum percentage of number of threads for all concurrent PCG processing/execution"));

	TAutoConsoleVariable<float> CVarEditorMaxPercentageOfExecutingThreads(
		TEXT("pcg.Editor.MaxPercentageOfExecutingThreads"),
		0.3f,
		TEXT("Editor: Maximum percentage of number of threads for processing scheduled pcg elements (percentage of pcg.Editor.MaxPercentageOfThreadsToUse)"));

	TAutoConsoleVariable<int32> CVarEditorMaxExecutingThreads(
		TEXT("pcg.Editor.MaxExecutingThreads"),
		8,
		TEXT("Editor: Hard cap on number of threads for processing scheduled pcg elements calculated from pcg.Editor.MaxPercentageOfExecutingThreads"));

	TAutoConsoleVariable<bool> CVarEditorEnablePostExecuteConsumerThread(
		TEXT("pcg.Editor.EnablePostExecuteConsumerThread"),
		false,
		TEXT("Editor: When enabled, a single thread will process all post executes and scheduling, instead of doing it directly in the async the executing threads"));

	TAutoConsoleVariable<float> CVarEditorPostExecuteConsumerThreadTime(
		TEXT("pcg.Editor.PostExecuteConsumerThreadTime"),
		15.0f,
		TEXT("Editor: Allocated time the consumer thread will use before yielding (when pcg.Editor.EnablePostExecuteConsumerThread is enabled)"));

	TAutoConsoleVariable<float> CVarEditorTimePerFrame(
		TEXT("pcg.EditorFrameTime"),
		15.0f,
		TEXT("Allocated time in ms per frame when running in editor (non pie)"));

	TAutoConsoleVariable<float> CVarEditorNotificationDelayInSeconds(
		TEXT("pcg.EditorNotificationDelayInSeconds"),
		3.0f,
		TEXT("Time in seconds to wait before showing a notification to avoid spamming for short tasks"));

	TAutoConsoleVariable<bool> CVarPausePCGExecutionWhileTransactionActive(
		TEXT("pcg.PauseExecutionWhileTransactionActive"),
		true,
		TEXT("Pauses all execution of PCG while user transactions are active"));
#endif

	TAutoConsoleVariable<int32> CVarMaxNumTasks_DEPRECATED(
		TEXT("pcg.MaxConcurrentTasks"),
		4096,
		TEXT("DEPRECATED(5.8): use pcg.Editor.MaxProcessingTasks or pcg.Runtime.MaxProcessingTasks instead."),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			// Called when user sets cvar from console, or when this cvar is set when launching standalone build. Pushes value to new cvars.
			UE_LOGF(LogPCG, Warning, "pcg.MaxConcurrentTasks is deprecated in 5.8. use pcg.Editor.MaxProcessingTasks or pcg.Runtime.MaxProcessingTasks instead.");
			check(InVariable);
			int32 NewValue = 0;
			InVariable->GetValue(NewValue);

			CVarRuntimeMaxNumProcessingTasks->Set(NewValue, ECVF_SetByCode);
#if WITH_EDITOR
			CVarEditorMaxNumProcessingTasks->Set(NewValue, ECVF_SetByCode);
#endif
		}));

	TAutoConsoleVariable<float> CVarMaxPercentageOfThreadsToUse_DEPRECATED(
		TEXT("pcg.MaxPercentageOfThreadsToUse"),
		0.9f,
		TEXT("DEPRECATED(5.8) : use pcg.Editor.MaxPercentageOfThreadsToUse or pcg.Runtime.MaxPercentageOfThreadsToUse instead."),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			// Called when user sets cvar from console, or when this cvar is set when launching standalone build. Pushes value to new cvars.
			UE_LOGF(LogPCG, Warning, "pcg.MaxPercentageOfThreadsToUse is deprecated in 5.8. use pcg.Editor.MaxPercentageOfThreadsToUse or pcg.Runtime.MaxPercentageOfThreadsToUse instead.");
			check(InVariable);
			float NewValue = 0.f;
			InVariable->GetValue(NewValue);

			CVarRuntimeMaxPercentageOfThreadsToUse->Set(NewValue, ECVF_SetByCode);
#if WITH_EDITOR
			CVarEditorMaxPercentageOfThreadsToUse->Set(NewValue, ECVF_SetByCode);
#endif
			}));

	TAutoConsoleVariable<float> CVarMaxPercentageOfExecutingThreads_DEPRECATED(
		TEXT("pcg.MaxPercentageOfExecutingThreads"),
		0.3f,
		TEXT("DEPRECATED(5.8) : use pcg.Editor.MaxPercentageOfExecutingThreads or pcg.Runtime.MaxPercentageOfExecutingThreads instead."),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			// Called when user sets cvar from console, or when this cvar is set when launching standalone build. Pushes value to new cvars.
			UE_LOGF(LogPCG, Warning, "pcg.MaxPercentageOfExecutingThreads is deprecated in 5.8. use pcg.Editor.MaxPercentageOfExecutingThreads or pcg.Runtime.MaxPercentageOfExecutingThreads instead.");
			check(InVariable);
			float NewValue = 0.f;
			InVariable->GetValue(NewValue);

			CVarRuntimeMaxPercentageOfExecutingThreads->Set(NewValue, ECVF_SetByCode);
#if WITH_EDITOR
			CVarEditorMaxPercentageOfExecutingThreads->Set(NewValue, ECVF_SetByCode);
#endif
		}));

#if WITH_EDITOR
	// Cvar deprecation. One-shot migration: runs after ini loading / user input via a cvar sink. Only required with editor, in standalone
	// it appears the pcg.Cache.Enabled change delegate fires which handles driving the new cvars.
	static void MigrateCVarsIfNeeded()
	{
		static bool bDone = false;
		if (bDone)
		{
			return;
		}
		bDone = true;

		auto SetFromIni = [](uint32 InFlags, uint32 InSetBy)
		{
			// Treat any of these as "user set in ini" (covers typical places users put CVars)
			return InSetBy == ECVF_SetBySystemSettingsIni || // [SystemSettings] sections in *Engine.ini
				InSetBy == ECVF_SetByConsoleVariablesIni || // ConsoleVariables.ini
				InSetBy == ECVF_SetByDeviceProfile || // DeviceProfiles.ini
				(InFlags & ECVF_CreatedFromIni) != 0; // Created by ini before registration
		};

		{
			const uint32 MaxNumTasks_Flags = CVarMaxNumTasks_DEPRECATED->GetFlags();
			const uint32 MaxNumTasks_SetBy = (MaxNumTasks_Flags & ECVF_SetByMask);

			if (SetFromIni(MaxNumTasks_Flags, MaxNumTasks_SetBy))
			{
				const int32 Value = CVarMaxNumTasks_DEPRECATED->GetInt();

				// Copy with the same SetBy priority so behavior matches the user's source of truth.
				CVarRuntimeMaxNumProcessingTasks->Set(Value, (EConsoleVariableFlags)MaxNumTasks_SetBy);
				CVarEditorMaxNumProcessingTasks->Set(Value, (EConsoleVariableFlags)MaxNumTasks_SetBy);

				UE_LOGF(LogPCG, Warning, "Cvar 'pcg.MaxConcurrentTasks' was set from ini but is deprecated in 5.8. Migrated value to pcg.Runtime.MaxProcessingTasks=%d, pcg.Editor.MaxProcessingTasks=%d (SetBy=0x%x).", Value, Value, MaxNumTasks_SetBy);
			}
		}

		{
			const uint32 MaxPercentage_Flags = CVarMaxPercentageOfThreadsToUse_DEPRECATED->GetFlags();
			const uint32 MaxPercentage_SetBy = (MaxPercentage_Flags & ECVF_SetByMask);

			if (SetFromIni(MaxPercentage_Flags, MaxPercentage_SetBy))
			{
				const float Value = CVarMaxPercentageOfThreadsToUse_DEPRECATED->GetFloat();

				// Copy with the same SetBy priority so behavior matches the user's source of truth.
				CVarRuntimeMaxPercentageOfThreadsToUse->Set(Value, (EConsoleVariableFlags)MaxPercentage_SetBy);
				CVarEditorMaxPercentageOfThreadsToUse->Set(Value, (EConsoleVariableFlags)MaxPercentage_SetBy);

				UE_LOGF(LogPCG, Warning, "Cvar 'pcg.MaxPercentageOfThreadsToUse' was set from ini but is deprecated in 5.8. Migrated value to pcg.Runtime.MaxPercentageOfThreadsToUse=%f, pcg.Editor.MaxPercentageOfThreadsToUse=%f (SetBy=0x%x).", Value, Value, MaxPercentage_SetBy);
			}
		}

		{
			const uint32 MaxExecutePercentage_Flags = CVarMaxPercentageOfExecutingThreads_DEPRECATED->GetFlags();
			const uint32 MaxExecutePercentage_SetBy = (MaxExecutePercentage_Flags & ECVF_SetByMask);

			if (SetFromIni(MaxExecutePercentage_Flags, MaxExecutePercentage_SetBy))
			{
				const float Value = CVarMaxPercentageOfExecutingThreads_DEPRECATED->GetFloat();

				// Copy with the same SetBy priority so behavior matches the user's source of truth.
				CVarRuntimeMaxPercentageOfExecutingThreads->Set(Value, (EConsoleVariableFlags)MaxExecutePercentage_SetBy);
				CVarEditorMaxPercentageOfExecutingThreads->Set(Value, (EConsoleVariableFlags)MaxExecutePercentage_SetBy);

				UE_LOGF(LogPCG, Warning, "Cvar 'pcg.MaxPercentageOfExecutingThreads' was set from ini but is deprecated in 5.8. Migrated value to pcg.Runtime.MaxPercentageOfExecutingThreads=%f, pcg.Editor.MaxPercentageOfExecutingThreads=%f (SetBy=0x%x).", Value, Value, MaxExecutePercentage_SetBy);
			}
		}
	}

	// Register a sink so this runs at the right time (after ini load / on first cvar change tick).
	static FAutoConsoleVariableSink GCVarMigrationSink(FConsoleCommandDelegate::CreateStatic(&MigrateCVarsIfNeeded));
#endif // WITH_EDITOR

	TAutoConsoleVariable<bool> CVarDynamicTaskCulling(
		TEXT("pcg.Graph.DynamicTaskCulling"),
		true,
		TEXT("Controls whether tasks are culled at execution time, for example in response to an deactivated dynamic branch pin"));

#if WITH_EDITOR
	TAutoConsoleVariable<int> CVarCacheEditorCleanupBehavior(
		TEXT("pcg.Cache.EditorCleanupLevel"),
		2,
		TEXT("Level for the cache cleanup, when the memory budget is enforced.\n")
		TEXT(" 0: Release objects only;\n")
		TEXT(" 1: Release objects and force garbage collection (iterative);\n")
		TEXT(" 2: Release objects and force garbage collection (complete)\n"));
#endif

	TAutoConsoleVariable<int> CVarCacheCleanupBehavior(
		TEXT("pcg.Cache.CleanupLevel"),
		0,
		TEXT("Level for the cache cleanup, when the memory budget is enforced.\n")
		TEXT(" 0: Release objects only;\n")
		TEXT(" 1: Release objects and force garbage collection (iterative);\n")
		TEXT(" 2: Release objects and force garbage collection (complete)\n"));

	void ClearAsyncFlags(TSet<TObjectPtr<UObject>>& AsyncObjects)
	{
		// Remove Async Flags from Objects created in Async PCG Tasks so that they can get tracked by the GarbageCollector properly
		// The reason to do this here and not in the FPCGContext destructor is that FPCGContext might get copied in Blueprint node execution causing multiple instances of the same AsyncObjects list
		// The FPCGGraphActiveTask is the true owner of the original context (throughg TUniquePtr) so we let it do the cleanup
		for (TObjectPtr<UObject>& AsyncObject : AsyncObjects)
		{
			if (ensure(AsyncObject->HasAnyInternalFlags(EInternalObjectFlags::Async)))
			{
				AsyncObject->ClearInternalFlags(EInternalObjectFlags::Async);
				ForEachObjectWithOuter(AsyncObject, [&AsyncObjects](UObject* SubObject)
				{
					if (AsyncObjects.Contains(SubObject))
					{
						return;
					}
					
					SubObject->ClearInternalFlags(EInternalObjectFlags::Async);
				}, EGetObjectsFlags::IncludeNestedObjects);
			}
		}
	}

	// Needs to be called by owner of LiveTasksLock
	void InsertToActiveTaskArrayNoLock(int32 Index, TArray<TSharedPtr<FPCGGraphActiveTask>>& ActiveTaskArray, TSharedPtr<FPCGGraphActiveTask> ActiveTask)
	{
		ActiveTaskArray.Insert(ActiveTask, Index);
		for (int32 i = Index; i < ActiveTaskArray.Num(); ++i)
		{
			ActiveTaskArray[i]->TaskIndex = i;
		}
	}

	// Needs to be called by owner of LiveTasksLock
	void AddToActiveTaskArrayNoLock(TArray<TSharedPtr<FPCGGraphActiveTask>>& ActiveTaskArray, TSharedPtr<FPCGGraphActiveTask> ActiveTask)
	{
		ActiveTaskArray.Add(ActiveTask);
		ActiveTask->TaskIndex = ActiveTaskArray.Num() - 1;
	}

	// Needs to be called by owner of LiveTasksLock
	void RemoveAtFromActiveTaskArrayNoLock(TArray<TSharedPtr<FPCGGraphActiveTask>>& ActiveTaskArray, int32 Index)
	{
		if (ActiveTaskArray.IsValidIndex(Index))
		{
			ActiveTaskArray.RemoveAtSwap(Index, EAllowShrinking::No);

			if (ActiveTaskArray.IsValidIndex(Index))
			{
				ActiveTaskArray[Index]->TaskIndex = Index;
			}
		}
	}

	template<class T>
	T GetCVarValueForWorld(const UWorld* World, const TAutoConsoleVariable<T>& RuntimeValue, const TAutoConsoleVariable<T>& EditorValue)
	{
#if WITH_EDITOR
		if (!World || !World->IsGameWorld())
		{
			return EditorValue.GetValueOnAnyThread();
		}
#endif

		return RuntimeValue.GetValueOnAnyThread();
	}
}

FPCGGraphExecutor::FPCGGraphExecutor()
	: FPCGGraphExecutor(nullptr)
{
}

FPCGGraphExecutor::FPCGGraphExecutor(UWorld* InWorld)
	: GraphCache(InWorld && InWorld->IsGameWorld())
	, World(InWorld)
{
	GameThreadHandler = MakeShared<FGameThreadHandler>(this);
}

FPCGGraphExecutor::~FPCGGraphExecutor()
{
	GameThreadHandler = nullptr;
	
	// We don't really need to do this here (it would be done in the destructor of these both)
	// but this is to clarify/ensure the order in which this happens
	GraphCache.ClearCache();

#if WITH_EDITOR
	// Cleanup + clear notification
	ClearAllTasks();
	ReleaseGenerationNotification();
#endif

	// Shutdown thread
	PostTaskExecuteConsumerThread.Reset();
}

void FPCGGraphExecutor::Compile(UPCGGraph* Graph)
{
	GraphCompiler.Compile(Graph);
}

FPCGTaskId FPCGGraphExecutor::Schedule(IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& ExternalDependencies, const FPCGStack* InFromStack)
{
	check(ExecutionSource);
	UPCGGraph* Graph = ExecutionSource->GetExecutionState().GetGraph();

	return Schedule(Graph, ExecutionSource, GetPreGraphElement(), GetFetchInputElement(), ExternalDependencies, InFromStack, /*bAllowHierarchicalGeneration=*/true);
}

FPCGTaskId FPCGGraphExecutor::Schedule(
	UPCGGraph* Graph,
	IPCGGraphExecutionSource* ExecutionSource,
	FPCGElementPtr PreGraphElement,
	FPCGElementPtr InputElement,
	const TArray<FPCGTaskId>& ExternalDependencies,
	const FPCGStack* InFromStack,
	bool bAllowHierarchicalGeneration)
{
	return ScheduleGraph(FPCGScheduleGraphParams(Graph, ExecutionSource, PreGraphElement, InputElement, ExternalDependencies, InFromStack, bAllowHierarchicalGeneration));
}

FPCGTaskId FPCGGraphExecutor::ScheduleGraph(const FPCGScheduleGraphParams& InParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::ScheduleGraph);
	check(InParams.ExecutionSource);

	PCGGraphExecutionLogging::LogGraphSchedule(InParams.ExecutionSource, InParams.Graph);
	
	FPCGTaskId ScheduledId = InvalidPCGTaskId;

	uint32 GenerationGridSize = PCGHiGenGrid::UninitializedGridSize();
	if (InParams.bAllowHierarchicalGeneration && InParams.Graph->IsHierarchicalGenerationEnabled())
	{
		const IPCGGraphExecutionState& ExecutionState = InParams.ExecutionSource->GetExecutionState();

		if (ExecutionState.IsLocalSource() || ExecutionState.IsPartitioned())
		{
			GenerationGridSize = ExecutionState.GetGenerationGridSize();
		}
	}

	// Get compiled tasks from compiler
	TSharedPtr<FPCGStackContext> StackContextPtr = MakeShared<FPCGStackContext>();
	TArray<FPCGGraphTask> CompiledTasks = GraphCompiler.GetCompiledTasks(InParams.Graph, GenerationGridSize, *StackContextPtr);

	// Create the final stack context by including the current stack frames
	if (InParams.FromStack)
	{
		StackContextPtr->PrependParentStack(InParams.FromStack);
	}
	else
	{
		FPCGStack ComponentStack;
		ComponentStack.PushFrame(Cast<UObject>(InParams.ExecutionSource));
		StackContextPtr->PrependParentStack(&ComponentStack);		
	}

#if WITH_EDITOR
	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		PCGEditorModule->OnScheduleGraph(*StackContextPtr);
	}

	if (IPCGBaseSubsystem* Subsystem = InParams.ExecutionSource->GetExecutionState().GetSubsystem())
	{
		Subsystem->OnScheduleGraph(*StackContextPtr);
	}
#endif

	// Assign this component to the tasks
	for (FPCGGraphTask& Task : CompiledTasks)
	{
		Task.ExecutionSource = InParams.ExecutionSource;
		Task.StackContext = StackContextPtr;
	}

	// Prepare scheduled task that will be promoted in the next Execute call.
	if (CompiledTasks.Num() > 0)
	{ 
		check(CompiledTasks[0].Node == InParams.Graph->GetInputNode());

#if WITH_EDITOR
		bAnyNonRuntimeGenComponentScheduled = bAnyNonRuntimeGenComponentScheduled || !PCGHelpers::IsRuntimeGeneration(InParams.ExecutionSource);
#endif

		// Setup fetch task on input node
		CompiledTasks[0].Element = InParams.InputElement;

		PCG::TScopeLock ScopeLock(ScheduleLock);

		FPCGGraphScheduleTask& ScheduledTask = ScheduledTasks.Emplace_GetRef();
		ScheduledTask.Tasks = MoveTemp(CompiledTasks);
		ScheduledTask.ExecutionSource = InParams.ExecutionSource;

		// Offset task node ids
		FPCGGraphCompiler::OffsetNodeIds(ScheduledTask.Tasks, NextTaskId, InvalidPCGTaskId);
		NextTaskId += ScheduledTask.Tasks.Num();
		ScheduledId = NextTaskId - 1; // This is true because the last task is from the output node or is the post-execute task

		const FPCGTaskId GraphExecutionTaskId = InParams.FromStack ? InParams.FromStack->GetGraphExecutionTaskId() : ScheduledId;
		StackContextPtr->SetGraphExecutionTaskId(GraphExecutionTaskId);

		// Push task (not data) dependencies on the pre-execute task
		// Note must be done after the offset ids, otherwise we'll break the dependencies
		check(ScheduledTask.Tasks.Num() >= 2 && ScheduledTask.Tasks[ScheduledTask.Tasks.Num() - 2].Node == nullptr);
		FPCGGraphTask& PreGraphTask = ScheduledTask.Tasks[ScheduledTask.Tasks.Num() - 2];

		if (InParams.PreGraphElement.IsValid())
		{
			PreGraphTask.Element = InParams.PreGraphElement;
		}
		else if (InParams.ExecutionSource->GetExecutionState().GetGraph() != InParams.Graph)
		{
			const UPCGGraphInterface* OverrideGraph = InParams.ExecutionSource->GetExecutionState().GetGraphInstance();
			if (!OverrideGraph)
			{
				OverrideGraph = InParams.ExecutionSource->GetExecutionState().GetGraph();
			}

			if (OverrideGraph)
			{
				PreGraphTask.Element = MakeShared<FPCGPreGraphElement>(OverrideGraph);
			}
		}

		if (!PreGraphTask.Element)
		{
			PreGraphTask.Element = GetPreGraphElement();
		}

		for (FPCGTaskId ExternalDependency : InParams.ExternalDependencies)
		{
			// For the pre-task, we don't consume any input
			PreGraphTask.Inputs.Emplace(ExternalDependency, /*InUpstreamPin=*/FPCGGraphTaskInput::NoPin, /*InDownstreamPin=*/FPCGGraphTaskInput::NoPin, /*bInProvideData=*/false);
		}

		ScheduledTask.FirstTaskIndex = ScheduledTask.Tasks.Num() - 2;
		ScheduledTask.LastTaskIndex = ScheduledTask.Tasks.Num() - 1;
	}

	return ScheduledId;
}

TArray<IPCGGraphExecutionSource*> FPCGGraphExecutor::Cancel(IPCGGraphExecutionSource* InExecutionSource)
{
	auto CancelIfSameExecutionSource = [InExecutionSource](TWeakInterfacePtr<IPCGGraphExecutionSource> ExecutionSource)
	{
		return ExecutionSource.IsValid() && InExecutionSource == ExecutionSource.Get();
	};

	return Cancel(CancelIfSameExecutionSource).Array();
}

TArray<IPCGGraphExecutionSource*> FPCGGraphExecutor::Cancel(UPCGGraph* InGraph)
{
	auto CancelIfComponentUsesGraph = [InGraph](TWeakInterfacePtr<IPCGGraphExecutionSource> ExecutionSource)
	{
		return (ExecutionSource.IsValid() && ExecutionSource->GetExecutionState().GetGraph() == InGraph);
	};

	return Cancel(CancelIfComponentUsesGraph).Array();
}

TArray<IPCGGraphExecutionSource*> FPCGGraphExecutor::CancelAll()
{
	auto CancelAllGeneration = [](TWeakInterfacePtr<IPCGGraphExecutionSource> ExecutionSource)
	{
		return ExecutionSource.IsValid();
	};

	return Cancel(CancelAllGeneration).Array();
}

TSet<IPCGGraphExecutionSource*> FPCGGraphExecutor::Cancel(TFunctionRef<bool(TWeakInterfacePtr<IPCGGraphExecutionSource>)> CancelFilter)
{
	check(IsInGameThread());
	TSet<IPCGGraphExecutionSource*> CancelledExecutionSources;
	
	// Prevent any scheduling while cancelling
	SchedulingDisabled = true;
	ON_SCOPE_EXIT
	{
		SchedulingDisabled = false;
	};

	// Visit scheduled tasks
	{
		PCG::TScopeLock ScopeLock(ScheduleLock);
		for (const FPCGGraphScheduleTask& ScheduledTask : ScheduledTasks)
		{
			if (CancelFilter(ScheduledTask.ExecutionSource))
			{
				CancelledExecutionSources.Add(ScheduledTask.ExecutionSource.Get());
			}
		}
	}

	{
		// Visit ready tasks
		PCG::TScopeLock ScopeLock(LiveTasksLock);
		for (const FPCGGraphTask& Task : ReadyTasks)
		{
			if (CancelFilter(Task.ExecutionSource))
			{
				CancelledExecutionSources.Add(Task.ExecutionSource.Get());
			}
		}

		auto GatherCancelledExecutionSourcesForActiveTasks = [&CancelFilter, &CancelledExecutionSources](const TArray<TSharedPtr<FPCGGraphActiveTask>>& InActiveTasks)
		{
			for (const TSharedPtr<FPCGGraphActiveTask>& Task : InActiveTasks)
			{
				if (Task->Context && CancelFilter(Task->Context->ExecutionSource))
				{
					CancelledExecutionSources.Add(Task->Context->ExecutionSource.Get());
				}
			}
		};

		GatherCancelledExecutionSourcesForActiveTasks(ActiveTasks);
		GatherCancelledExecutionSourcesForActiveTasks(ActiveTasksGameThreadOnly);
		GatherCancelledExecutionSourcesForActiveTasks(PausedTasks);
	}

	// In one instance this function was observed to return nullptr in the CancelledComponents set.
	// All the cancel filter lambdas check the ptr is valid, so it's not clear why. It seems perhaps
	// the SourceComponent weak ptr became nullptr between calling CancelFilter and adding the component
	// to the set.
	ensure(CancelledExecutionSources.Remove(nullptr) == 0);

	// Early out - nothing to cancel
	if (CancelledExecutionSources.IsEmpty())
	{
		return CancelledExecutionSources;
	}

	PCGGraphExecutionLogging::LogExecutionSourceCancellation(CancelledExecutionSources);

	auto TryAbortScheduledTasks = [](FPCGGraphScheduleTask& ScheduledTask)
	{
		if (ScheduledTask.bHasAbortCallbacks)
		{
			for (FPCGGraphTask& InternalTask : ScheduledTask.Tasks)
			{
				if (InternalTask.Element)
				{
					InternalTask.Element->Abort(InternalTask.Context);
				}
			}
		}
	};

	TArray<FPCGTaskId> CancelledScheduledTasks;

	bool bStableCancellationSet = false;
	while (!bStableCancellationSet)
	{
		bStableCancellationSet = true;

		// Remove from scheduled tasks
		{
			PCG::TScopeLock ScopeLock(ScheduleLock);
			for (int32 ScheduledTaskIndex = ScheduledTasks.Num() - 1; ScheduledTaskIndex >= 0; --ScheduledTaskIndex)
			{
				FPCGGraphScheduleTask& ScheduledTask = ScheduledTasks[ScheduledTaskIndex];
				if (CancelledExecutionSources.Contains(ScheduledTask.ExecutionSource.Get()))
				{
					CancelledScheduledTasks.Add(ScheduledTask.Tasks[ScheduledTask.LastTaskIndex].NodeId);

					TryAbortScheduledTasks(ScheduledTask);
					ScheduledTasks.RemoveAtSwap(ScheduledTaskIndex);
				}
			}

			// WARNING: variable upper bound
			for (int32 CancelledTaskIdIndex = 0; CancelledTaskIdIndex < CancelledScheduledTasks.Num(); ++CancelledTaskIdIndex)
			{
				const FPCGTaskId EndTaskId = CancelledScheduledTasks[CancelledTaskIdIndex];
				for (int32 ScheduledTaskIndex = ScheduledTasks.Num() - 1; ScheduledTaskIndex >= 0; --ScheduledTaskIndex)
				{
					FPCGGraphScheduleTask& ScheduledTask = ScheduledTasks[ScheduledTaskIndex];
					const bool bContainsDependency = Algo::AnyOf(ScheduledTask.Tasks[ScheduledTask.FirstTaskIndex].Inputs, [EndTaskId](const FPCGGraphTaskInput& Input) { return Input.TaskId == EndTaskId; });

					if (bContainsDependency)
					{
						IPCGGraphExecutionSource* TaskExecutionSource = ScheduledTask.ExecutionSource.Get();
						if (TaskExecutionSource && !CancelledExecutionSources.Contains(TaskExecutionSource))
						{
							CancelledExecutionSources.Add(TaskExecutionSource);
							bStableCancellationSet = false;
						}

						CancelledScheduledTasks.Add(ScheduledTask.Tasks[ScheduledTask.LastTaskIndex].NodeId);

						TryAbortScheduledTasks(ScheduledTask);
						ScheduledTasks.RemoveAtSwap(ScheduledTaskIndex);
					}
				}
			}

			CancelledScheduledTasks.Reset();
		}
				
		// Mark as cancelled in the active tasks - needed to make sure we're not breaking the current execution (if any)
		{
			TArray<TTuple<TSharedPtr<FPCGGraphActiveTask>, UE::Tasks::TTask<bool>>> CancelledActiveTasks;
			{
				auto CancelActiveTasks = [&CancelledExecutionSources, &CancelledActiveTasks](TArray<TSharedPtr<FPCGGraphActiveTask>>& InActiveTasks)
				{
					for (int32 ActiveTaskIndex = InActiveTasks.Num() - 1; ActiveTaskIndex >= 0; --ActiveTaskIndex)
					{
						TSharedPtr<FPCGGraphActiveTask> ActiveTask = InActiveTasks[ActiveTaskIndex];
						if (ActiveTask->Context && CancelledExecutionSources.Contains(ActiveTask->Context->ExecutionSource.Get()))
						{
							// While we have lock Task can't complete, but we can't wait on this task with the lock neither so we capture it here 
							// and are going to wait on it outside of the LiveTaskLock
							UE::Tasks::TTask<bool> TaskHandle = ActiveTask->ExecutingTask;
							ActiveTask->bWasCancelled = true;

							CancelledActiveTasks.Add({ ActiveTask, MoveTemp(TaskHandle) });
							
							check(ActiveTask->TaskIndex == ActiveTaskIndex);
							PCGGraphExecutor::RemoveAtFromActiveTaskArrayNoLock(InActiveTasks, ActiveTaskIndex);
							ActiveTask->TaskIndex = INDEX_NONE;
						}
					}
				};
				
				PCG::TScopeLock ScopeLock(LiveTasksLock);
				CancelActiveTasks(ActiveTasks);
				CancelActiveTasks(ActiveTasksGameThreadOnly);
			}

			for (TTuple<TSharedPtr<FPCGGraphActiveTask>, UE::Tasks::TTask<bool>>& CancelledActiveTask : CancelledActiveTasks)
			{
				// Wait for Task (if any) to complete
				UE::Tasks::TTask<bool> TaskHandle = CancelledActiveTask.Get<UE::Tasks::TTask<bool>>();
				TaskHandle.Wait();

				TSharedPtr<FPCGGraphActiveTask> ActiveTask = CancelledActiveTask.Get<TSharedPtr<FPCGGraphActiveTask>>();
				check(ActiveTask->Element);

				// Even if we've set SchedulingDisabled = true, it is possible some tasks slipped into the queue and will get PostTaskExecute called through consumer thread
				// if this is the case the bIsExecutingTask would be true and we can wait for the consumer thread to process.
				while (PostTaskExecuteConsumerThread.IsValid() && ActiveTask->bIsExecutingTask)
				{
					// Wait for consumer thread to process...
					FPlatformProcess::YieldThread();
				}

				ActiveTask->Element->Abort(ActiveTask->Context.Get());

				bStableCancellationSet &= !CancelNextTasks(ActiveTask->NodeId, CancelledExecutionSources);
			}
		}

		// Remove from ready tasks
		{
			TArray<FPCGGraphTask> CancelledReadyTasks;
			{
				PCG::TScopeLock ScopeLock(LiveTasksLock);
				for (int32 ReadyTaskIndex = ReadyTasks.Num() - 1; ReadyTaskIndex >= 0; --ReadyTaskIndex)
				{
					FPCGGraphTask& Task = ReadyTasks[ReadyTaskIndex];

					if (CancelledExecutionSources.Contains(Task.ExecutionSource.Get()))
					{
						CancelledReadyTasks.Add(MoveTemp(Task));
						ReadyTasks.RemoveAtSwap(ReadyTaskIndex);
					}
				}
			}

			for (FPCGGraphTask& CancelledReadyTask : CancelledReadyTasks)
			{
				if (CancelledReadyTask.Element)
				{
					CancelledReadyTask.Element->Abort(CancelledReadyTask.Context);
				}

				FPCGTaskId CancelledTaskId = CancelledReadyTask.NodeId;
				RemoveTaskFromInputSuccessors(CancelledTaskId, CancelledReadyTask.Inputs);

				FPCGContext::Release(CancelledReadyTask.Context);
				bStableCancellationSet &= !CancelNextTasks(CancelledTaskId, CancelledExecutionSources);
			}
		}

		// Remove from paused tasks
		{
			TArray<TSharedPtr<FPCGGraphActiveTask>> CancelledPausedTasks;
			{
				PCG::TScopeLock ScopeLock(LiveTasksLock);
				PCG::TScopeLock PauseScopeLock(PausedTaskSuccessorsLock);
				for (int32 PausedTaskIndex = PausedTasks.Num() - 1; PausedTaskIndex >= 0; --PausedTaskIndex)
				{
					TSharedPtr<FPCGGraphActiveTask> Task = PausedTasks[PausedTaskIndex];
					if (Task->Context && CancelledExecutionSources.Contains(Task->Context->ExecutionSource.Get()))
					{
						check(Task->TaskIndex == PausedTaskIndex);
						CancelledPausedTasks.Add(Task);
						PCGGraphExecutor::RemoveAtFromActiveTaskArrayNoLock(PausedTasks, PausedTaskIndex);
						Task->TaskIndex = INDEX_NONE;

						if (!Task->Context->DynamicDependencies.IsEmpty())
						{
							// Remove this task from the PausedTaskSuccessors
							for (FPCGTaskId DynamicDependency : Task->Context->DynamicDependencies)
							{
								if (TArray<TSharedPtr<FPCGGraphActiveTask>>* Successors = PausedTaskSuccessors.Find(DynamicDependency))
								{
									Successors->Remove(Task);

									// If there are no more successors, remove the entry all together
									if (Successors->IsEmpty())
									{
										PausedTaskSuccessors.Remove(DynamicDependency);
									}
								}
							}
							Task->Context->OutputData.bCancelExecution = true;
							Task->Context->DynamicDependencies.Empty();
						}
					}
				}
			}

			for (TSharedPtr<FPCGGraphActiveTask>& CancelledPausedTask : CancelledPausedTasks)
			{
				check(CancelledPausedTask->Element);
				CancelledPausedTask->Element->Abort(CancelledPausedTask->Context.Get());

				FPCGTaskId CancelledTaskId = CancelledPausedTask->NodeId;
				RemoveTaskFromInputSuccessors(CancelledTaskId, CancelledPausedTask->Inputs);
								
				bStableCancellationSet &= !CancelNextTasks(CancelledTaskId, CancelledExecutionSources);
			}
		}
	}

	// Finally, update the notification so it shows the new information
#if WITH_EDITOR
	UpdateGenerationNotification();
#endif

	return CancelledExecutionSources;
}

bool FPCGGraphExecutor::IsGraphCurrentlyExecuting(const UPCGGraph* InGraph)
{
	bool bAnyPresent = false;

	// This makes use of the Cancel function which runs over all tasks, but it always returns false so no tasks are cancelled.
	auto CheckIfAnyGraphTasksPresent = [InGraph, &bAnyPresent](TWeakInterfacePtr<IPCGGraphExecutionSource> ExecutionSource)
	{
		bAnyPresent |= ExecutionSource.IsValid() && ExecutionSource->GetExecutionState().GetGraph() == InGraph;
		return false;
	};

	Cancel(CheckIfAnyGraphTasksPresent);

	return bAnyPresent;
}

bool FPCGGraphExecutor::IsAnyGraphCurrentlyExecuting() const
{
	if (GetNonScheduledRemainingTaskCount() > 0)
	{
		return true;
	}

	return ScheduledTasks.Num() > 0;
}

int32 FPCGGraphExecutor::GetNonScheduledRemainingTaskCount() const
{
	return NewReadyTaskCount + Tasks.Num() + ReadyTasks.Num() + ActiveTasks.Num() + ActiveTasksGameThreadOnly.Num() + PausedTasks.Num();
}

IPCGBaseSubsystem* FPCGGraphExecutor::GetSubsystem() const
{
	if (World)
	{
		return UPCGSubsystem::GetInstance(World);
	}
	else
	{
		return UPCGEngineSubsystem::Get();
	}
}

FPCGTaskId FPCGGraphExecutor::ScheduleGeneric(const FPCGScheduleGenericParams& Params)
{
	return ScheduleSingleElement(MakeShared<FPCGGenericElement>(Params.Operation, Params.AbortOperation, Params.bSupportBasePointDataInput, Params.bCanExecuteOnlyOnMainThread),
		!!Params.AbortOperation, Params.ExecutionSource, Params.ExecutionDependencies, Params.DataDependencies);
}

FPCGTaskId FPCGGraphExecutor::ScheduleGeneric(TFunction<bool()> InOperation, IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies)
{
	return ScheduleGeneric(
		InOperation,
		TFunction<void()>(),
		InExecutionSource,
		TaskExecutionDependencies);
}

FPCGTaskId FPCGGraphExecutor::ScheduleGeneric(TFunction<bool()> InOperation, TFunction<void()> InAbortOperation, IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies)
{
	// Since we have no context, the generic task will consume no input (no data dependencies).
	return ScheduleGenericWithContext(
		[Operation = MoveTemp(InOperation)](FPCGContext*) -> bool
		{
			return Operation && Operation();
		}, 
		[AbortOperation = MoveTemp(InAbortOperation)](FPCGContext*)
		{
			if(AbortOperation)
			{
				AbortOperation();
			}
		},
		InExecutionSource,
		TaskExecutionDependencies,
		/*TaskDataDependencies=*/{});
}

FPCGTaskId FPCGGraphExecutor::ScheduleGenericWithContext(TFunction<bool(FPCGContext*)> InOperation, IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies, const TArray<FPCGTaskId>& TaskDataDependencies, bool bSupportBasePointDataInput)
{
	return ScheduleGenericWithContext(
		InOperation,
		TFunction<void(FPCGContext*)>(),
		InExecutionSource,
		TaskExecutionDependencies,
		TaskDataDependencies,
		bSupportBasePointDataInput);
}

FPCGTaskId FPCGGraphExecutor::ScheduleGenericWithContext(TFunction<bool(FPCGContext*)> InOperation, TFunction<void(FPCGContext*)> InAbortOperation, IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies, const TArray<FPCGTaskId>& TaskDataDependencies, bool bSupportBasePointDataInput)
{
	return ScheduleSingleElement(MakeShared<FPCGGenericElement>(InOperation, InAbortOperation, bSupportBasePointDataInput),
		!!InAbortOperation, InExecutionSource, TaskExecutionDependencies, TaskDataDependencies);
}

FPCGTaskId FPCGGraphExecutor::ScheduleSingleElement(TSharedRef<IPCGElement> Element, bool bHasAbortCallbacks, IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies, const TArray<FPCGTaskId>& TaskDataDependencies)
{
#if WITH_EDITOR
	bAnyNonRuntimeGenComponentScheduled = bAnyNonRuntimeGenComponentScheduled || !PCGHelpers::IsRuntimeGeneration(InExecutionSource);
#endif

	// Build task & element to hold the operation to perform
	FPCGGraphTask Task;

	for (FPCGTaskId TaskDependency : TaskExecutionDependencies)
	{
		ensure(TaskDependency != InvalidPCGTaskId);
		Task.Inputs.Emplace(TaskDependency, /*InUpstreamPin=*/FPCGGraphTaskInput::NoPin, /*InDownstreamPin=*/FPCGGraphTaskInput::NoPin, /*bInProvideData=*/false);
	}

	for (FPCGTaskId TaskDependency : TaskDataDependencies)
	{
		ensure(TaskDependency != InvalidPCGTaskId);
		// By default the input of a generic element is not used multiple times.
		FPCGGraphTaskInput& TaskInput = Task.Inputs.Emplace_GetRef(TaskDependency);
		TaskInput.bIsUsedMultipleTimes = false;
	}

	Task.ExecutionSource = InExecutionSource;
	Task.Element = Element;

	{
		PCG::TScopeLock ScopeLock(ScheduleLock);

		// Assign task id
		Task.NodeId = NextTaskId++;

		FPCGGraphScheduleTask& ScheduledTask = ScheduledTasks.Emplace_GetRef();
		ScheduledTask.Tasks.Add(Task);
		ScheduledTask.ExecutionSource = InExecutionSource;
		ScheduledTask.bHasAbortCallbacks = bHasAbortCallbacks;
	}
	return Task.NodeId;
}

void FPCGGraphExecutor::MarkInputResults(TArrayView<const FPCGTaskId> InInputResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::MarkInputResults);
#if WITH_EDITOR
	if (bDebugKeepIntermediateResults)
	{
		return;
	}
#endif

	if (InInputResults.Num() > 0)
	{
		PCG::TUniqueScopeLock WriteLock(TaskOutputsLock);
		for (FPCGTaskId TaskId : InInputResults)
		{
			FOutputDataInfo& OutputDataInfo = TaskOutputs[TaskId];
			if (!OutputDataInfo.bNeedsManualClear && --OutputDataInfo.RemainingSuccessorCount == 0)
			{
				OutputDataInfo.DataCollection.ClearUsage(EPCGDataUsage::GraphExecutorTaskOutput);

				TaskOutputs.Remove(TaskId);
			}
		}
	}
}

bool FPCGGraphExecutor::GetOutputData(FPCGTaskId TaskId, FPCGDataCollection& OutData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::GetOutputData);
	PCG::TSharedScopeLock ReadLock(TaskOutputsLock);
	if (FOutputDataInfo* OutputDataInfo = TaskOutputs.Find(TaskId))
	{
		ensure(!OutputDataInfo->bCulled);
		OutData = OutputDataInfo->DataCollection;
		return true;
	}
	else
	{
		return false;
	}
}

void FPCGGraphExecutor::ClearOutputData(FPCGTaskId TaskId)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::ClearOutputData);
	PCG::TUniqueScopeLock WriteLock(TaskOutputsLock);
	if (FOutputDataInfo* OutputDataInfo = TaskOutputs.Find(TaskId); OutputDataInfo && ensure(OutputDataInfo->bNeedsManualClear))
	{
		TaskOutputs.Remove(TaskId);
	}
}

void FPCGGraphExecutor::CheckState(bool bIsInGameThread)
{
	// Only check state on game thread
	if (!bIsInGameThread)
	{
		return;
	}

	// ScheduledTasks needs to be empty, check without a lock to early out if that is not the case
	if (!ScheduledTasks.IsEmpty())
	{
		return;
	}

	// Check every second to avoid locking too often for an error that shouldn't happen
	const double TimeInSeconds = FPlatformTime::Seconds();
	const double ElapsedTimeSinceLastCheck = TimeInSeconds - LastSchedulingErrorCheck;
	if (ElapsedTimeSinceLastCheck < 1.0)
	{
		return;
	}

	// Since this is for an error case avoid locking if lock is already taken, we can comeback later but if we haven't checked in more than 3 seconds actually lock
	PCG::TScopeLock ScheduleScopeLock(ScheduleLock, /*bTryLock=*/ ElapsedTimeSinceLastCheck < 3.0);
	if (!ScheduleScopeLock.IsLocked())
	{
		return;
	}
	
	LastSchedulingErrorCheck = TimeInSeconds;

	// Check again now with lock
	if (!ScheduledTasks.IsEmpty())
	{
		return;
	}
		
	PCG::TScopeLock ScopeLock(TasksLock);
	PCG::TScopeLock ChildScopeLock(LiveTasksLock);
	PCG::TScopeLock PausedLock(PausedTaskSuccessorsLock);
	PCG::TScopeLock CachingResultsScopeLock(CachingResultsLock);

	// This is a safeguard to check if we're in a stuck state
	if (CachingResultsForGC.Num() == 0 && ReadyTasks.Num() == 0 && ActiveTasks.Num() == 0 && ActiveTasksGameThreadOnly.Num() == 0 && PausedTasks.Num() == 0 && Tasks.Num() > 0)
	{
		UE_LOGF(LogPCG, Error, "PCG Graph executor error: tasks are in a deadlocked state. Will drop all tasks.");
		ClearAllTasks();
	}
}

bool FPCGGraphExecutor::ProcessScheduledTasks(double EndTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::ProcessScheduledTasks);
	bool bProcessedNewTasks = false;
	const bool bIsInGameThread = IsInGameThread();
	FCachedResults CachedResults;

	{
		// Process any newly scheduled graphs to execute
		PCG::TScopeLock ScopeLock(ScheduleLock, !bIsInGameThread);
		if (ScopeLock.IsLocked())
		{
			if (ScheduledTasks.Num() > 0)
			{
				int32 ScheduledTaskIndex = ScheduledTasks.Num() - 1;
				while (ScheduledTaskIndex >= 0)
				{
					TArray<FPCGGraphTask, TInlineAllocator<16>> NewReadyTasks;
					{
						PCG::TScopeLock ChildScopeLock(TasksLock);
						PCG::TSharedScopeLock ReadLock(TaskOutputsLock);

						// Every X ScheduledTasks, we want to check the frame budget and early out if we consumed it.
						const int32 MaxScheduledTasksPerBudgetCheck = 32;
						const int32 FirstScheduledTaskIndex = ScheduledTaskIndex;
						while (ScheduledTaskIndex >= 0 && (FirstScheduledTaskIndex - ScheduledTaskIndex) < MaxScheduledTasksPerBudgetCheck)
						{
							FPCGGraphScheduleTask& ScheduledTask = ScheduledTasks[ScheduledTaskIndex];

							check(ScheduledTask.Tasks.Num() > 0);
							// Push tasks to the primary task list & build successors map
							for (FPCGGraphTask& Task : ScheduledTask.Tasks)
							{
								const FPCGTaskId TaskId = Task.NodeId;

								bool bPushToReady = true;
								{
									for (const FPCGGraphTaskInput& Input : Task.Inputs)
									{
										if (!TaskOutputs.Contains(Input.TaskId))
										{
											TaskSuccessors.FindOrAdd(Input.TaskId).Add(TaskId);
											bPushToReady = false;
										}
									}
								}

								// Automatically push inputless/already satisfied tasks to the ready queue
								if (bPushToReady)
								{
									NewReadyTasks.Add(MoveTemp(Task));
									++NewReadyTaskCount;
								}
								else
								{
									Tasks.Add(TaskId, MoveTemp(Task));
								}
								bProcessedNewTasks = true;
							}

							--ScheduledTaskIndex;
						}
					}

					// Process ready tasks
					if (NewReadyTasks.Num() > 0)
					{
						OnTaskInputsReady(NewReadyTasks, CachedResults, bIsInGameThread);
						NewReadyTaskCount -= NewReadyTasks.Num();
					}

					if (FPlatformTime::Seconds() > EndTime)
					{
						break;
					}
				}

				if (PCGGraphExecutionLogging::CullingLogEnabled())
				{
					PCG::TScopeLock ChildScopeLock(TasksLock);
					PCGGraphExecutionLogging::LogGraphPostSchedule(Tasks, TaskSuccessors);
				}

				if (ScheduledTaskIndex >= 0)
				{
					// [0, ScheduledTaskIndex] were not processed
					ScheduledTasks.SetNum(ScheduledTaskIndex + 1);
				}
				else
				{
					ScheduledTasks.Reset();
				}
			}
		}
	}

	ProcessCachedResults(CachedResults);

	CheckState(bIsInGameThread);

	return bProcessedNewTasks;
}

void FPCGGraphExecutor::ExecuteTasksEnded()
{
	if (!bNeedToExecuteTasksEnded)
	{
		return;
	}
	bNeedToExecuteTasksEnded = false;

	// Nothing left to do; we'll release everything here.
	// TODO: this is fine and will make sure any intermediate data is properly
	// garbage collected, however, this goes a bit against our goals if we want to
	// keep a cache of intermediate results.
	if (GetNonScheduledRemainingTaskCount() == 0)
	{
		// Make sure no scheduling can happen here (and check condition again)
		PCG::TScopeLock ScopeLock(ScheduleLock);
		if (GetNonScheduledRemainingTaskCount() == 0)
		{
			{
				PCG::TScopeLock ScopeChildLock(TasksLock);
				if (!ensure(TaskSuccessors.IsEmpty()))
				{
					TaskSuccessors.Reset();
				}
			}

			ClearResults();

#if WITH_EDITOR
			// Call the notification update here to prevent it from sticking around - needed because we early out before this
			UpdateGenerationNotification();
#endif

			PCGGraphExecutionLogging::LogGraphExecuteFrameFinished();

#if WITH_EDITOR
			// Reset when we go idle. Will be set again next time any non-runtime-gen component is scheduled.
			bAnyNonRuntimeGenComponentScheduled = false;
#endif
		}
	}

	// Purge things from cache if memory usage is too high
	if (GraphCache.EnforceMemoryBudget() && GEngine)
	{
#if WITH_EDITOR
		const int CleanupMode = (World && World->IsGameWorld()) ? PCGGraphExecutor::CVarCacheCleanupBehavior.GetValueOnAnyThread() : PCGGraphExecutor::CVarCacheEditorCleanupBehavior.GetValueOnAnyThread();
#else
		const int CleanupMode = PCGGraphExecutor::CVarCacheCleanupBehavior.GetValueOnAnyThread();
#endif
		if (CleanupMode <= 0)
		{
			// Do nothing
		}
		else if (CleanupMode == 1)
		{
			GEngine->ForceGarbageCollection();
		}
		else // CleanupMode >= 2
		{
			GEngine->ForceGarbageCollection(/*bFullPurge=*/true);
		}
	}
}

void FPCGGraphExecutor::PostTaskExecute(TSharedPtr<FPCGGraphActiveTask> ActiveTaskPtr, bool bIsDone)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::PostTaskExecute);
	check(ActiveTaskPtr);
	FPCGGraphActiveTask& ActiveTask = *ActiveTaskPtr;
	check(ActiveTask.Context);
	check(ActiveTask.bIsExecutingTask || ActiveTask.bWasCancelled);

	// If input data collection was modified (for example replacing proxies with full data), add any new references.
	if (ActiveTask.Context->bInputDataModified)
	{
		PCG::TScopeLock ScopeLock(LiveTasksLock);

		for (const FPCGTaggedData& Data : ActiveTask.Context->InputData.TaggedData)
		{
			if (Data.Data)
			{
				ActiveTask.ExecutingReferences.AddUnique(Data.Data);
			}
		}

		ActiveTask.Context->bInputDataModified = false;
	}

	if (!bIsDone || !ActiveTask.Context->DynamicDependencies.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::PostTaskExecute::NotDone);
		PCG::TScopeLock ScopeLock(LiveTasksLock);
		
		// Once we get the lock check for cancellation again in case task was cancelled while executing
		if (!ActiveTaskPtr->bWasCancelled)
		{
			ActiveTask.StopExecuting();

			TArray<TSharedPtr<FPCGGraphActiveTask>>& ActiveTaskArrayToRemove = ActiveTask.bIsGameThreadOnly ? ActiveTasksGameThreadOnly : ActiveTasks;
			const int32 TaskRemoveIndex = ActiveTask.TaskIndex;
			check(TaskRemoveIndex != INDEX_NONE && ActiveTaskArrayToRemove[TaskRemoveIndex] == ActiveTaskPtr);

			const bool bHasPausedDependencies = !ActiveTask.Context->DynamicDependencies.IsEmpty();

			// We have dynamic dependencies bIsPaused flag might not have been set by the element
			// It is safe to set bIsPaused here because tasks with DynamicDependencies can't be unpaused by another task (race-condition)
			if (bHasPausedDependencies)
			{
				ActiveTask.Context->bIsPaused = true;
			}

			// Still paused (because of multithreading it is possible that we were unpaused before processing the paused state)
			if (ActiveTask.Context->bIsPaused)
			{
				bool bMoveFromActiveToPaused = !bHasPausedDependencies;

				// When using DynamicDependencies the task will be un-paused  (bIsPaused will toggle back to false) after completion of those dependency tasks.
				// It is not possible for a task to have DynamicDependencies and some other external mechanism to unpause (like polling).
				// This could be supported but for now we have no use for this added complexity.
				if (bHasPausedDependencies)
				{
					PCG::TScopeLock PauseLock(PausedTaskSuccessorsLock);
					PCG::TSharedScopeLock ReadLock(TaskOutputsLock);
					for (TSet<FPCGTaskId>::TIterator It = ActiveTask.Context->DynamicDependencies.CreateIterator(); It; ++It)
					{
						FPCGTaskId DependencyId = *It;

						if (!TaskOutputs.Contains(DependencyId))
						{
							PausedTaskSuccessors.FindOrAdd(DependencyId).Add(ActiveTaskPtr);
							bMoveFromActiveToPaused = true;
						}
						else
						{
							// Make sure to remove DynamicDependency since it has already completed
							It.RemoveCurrent();
						}
					}
				}

				if (bMoveFromActiveToPaused)
				{
					PCGGraphExecutor::AddToActiveTaskArrayNoLock(PausedTasks, ActiveTaskPtr);
					PCGGraphExecutor::RemoveAtFromActiveTaskArrayNoLock(ActiveTaskArrayToRemove, TaskRemoveIndex);
				}
				else
				{
					// All the paused dependencies have completed
					ActiveTask.Context->bIsPaused = false;
				}
			}
			else if (ActiveTask.bIsGameThreadOnly != ActiveTask.CanExecuteOnlyOnMainThread())
			{
				ActiveTask.bIsGameThreadOnly = !ActiveTask.bIsGameThreadOnly;
				TArray<TSharedPtr<FPCGGraphActiveTask>>& ActiveTaskArrayToAdd = ActiveTask.bIsGameThreadOnly ? ActiveTasksGameThreadOnly : ActiveTasks;

				PCGGraphExecutor::AddToActiveTaskArrayNoLock(ActiveTaskArrayToAdd, ActiveTaskPtr);
				PCGGraphExecutor::RemoveAtFromActiveTaskArrayNoLock(ActiveTaskArrayToRemove, TaskRemoveIndex);
			}

			// Task might have been moved to PausedTasks or changed from ActiveTasks to ActiveTasksGameThreadOnly (or vice-versa) 
			// but it isn't done so we return
			return;
		}
	}

	const bool bTaskFullyExecuted = !ActiveTask.bWasCancelled && ActiveTask.Context->CurrentPhase == EPCGExecutionPhase::Done && ActiveTask.Context->DynamicDependencies.IsEmpty();

#if WITH_EDITOR
	const bool bTaskWasBypassed = ActiveTask.bIsBypassed;
#else
	const bool bTaskWasBypassed = false;
#endif

	bNeedToExecuteTasksEnded = true;

	const UPCGSettingsInterface* ActiveTaskSettingsInterface = ActiveTask.Context->GetInputSettingsInterface();
	const uint64 InactivePinMask = ActiveTask.Context->OutputData.InactiveOutputPinBitmask;

	if (InactivePinMask != 0 && ActiveTaskSettingsInterface)
	{
		const UPCGSettings* ActiveTaskSettings = ActiveTaskSettingsInterface ? ActiveTaskSettingsInterface->GetSettings() : nullptr;

		// If output pins may have been deactivated then perform culling and update information for editor visualization.
		if (ActiveTaskSettings && ActiveTaskSettings->OutputPinsCanBeDeactivated())
		{
			CullInactiveDownstreamNodes(ActiveTask.NodeId, InactivePinMask);

#if WITH_EDITOR
			SendInactivePinNotification(ActiveTask.Context->Node, ActiveTask.StackContext->GetStack(ActiveTask.StackIndex), InactivePinMask);
#endif
		}
	}

	if (bTaskWasBypassed || bTaskFullyExecuted)
	{
#if WITH_EDITOR
		// Execute debug display.
		if (ActiveTaskSettingsInterface && ActiveTaskSettingsInterface->bDebug)
		{
			// Debug display can span multiple frames so hold onto reference to task.
			// TODO: Investigate scheduling an additional task to perform the debug display rather than manually holding onto references here.
			{
				PCG::TScopeLock ScopeLock(CollectGCReferenceTasksLock);
				CollectGCReferenceTasks.Add(ActiveTaskPtr);
			}

			PCGHelpers::ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakHandler = GameThreadHandler->AsWeak(), ActiveTaskPtr]()
			{
				LLM_SCOPE_BYTAG(PCG);

				DebugDisplayTask(WeakHandler, ActiveTaskPtr);
			});
		}
#endif // WITH_EDITOR
		
#if PCG_PROFILING_ENABLED
		// Outside of editor, only collect this information if profiling is enabled. In editor inspection data is always collected.
#if !WITH_EDITOR
		if (PCGProfilingLog::IsEnabled())
#endif
		{
			if (IPCGGraphExecutionSource* ExecutionSource = ActiveTask.Context->ExecutionSource.Get())
			{
				if (ActiveTask.StackIndex != INDEX_NONE)
				{
					const FPCGStack* Stack = ActiveTask.StackContext->GetStack(ActiveTask.StackIndex);
					ExecutionSource->GetExecutionState().GetInspection().StoreInspectionData(Stack, ActiveTask.Context->Node, &ActiveTask.Context->Timer, ActiveTask.Context->InputData, ActiveTask.Context->OutputData, /*bUsedCache=*/false);
				}
			}
	}
#endif // PCG_PROFILING_ENABLED

		// If the task is a post execute, then we can safely clear the data after getting it from the results.
		const bool bNeedsManualClear = (ActiveTask.Element == GraphCompiler.GetSharedTrivialPostGraphElement());

		// Store output in data map.
		FTaskResult NewResult{ ActiveTask.NodeId, &ActiveTask.Context->OutputData, bNeedsManualClear };
		StoreResults(MakeArrayView(&NewResult, 1));

		// Book-keeping
		QueueNextTasks(MakeArrayView(&ActiveTask.NodeId, 1));
	}

	// Erase from ActiveTasks
	{
		PCG::TScopeLock ScopeLock(LiveTasksLock);
		TArray<TSharedPtr<FPCGGraphActiveTask>>& ActiveTaskArray = ActiveTask.bIsGameThreadOnly ? ActiveTasksGameThreadOnly : ActiveTasks;
		const int32 TaskIndex = ActiveTask.TaskIndex >= 0 ? ActiveTask.TaskIndex : ActiveTaskArray.Find(ActiveTaskPtr);
		if (ActiveTaskArray.IsValidIndex(TaskIndex))
		{
			check(ActiveTaskArray[TaskIndex] == ActiveTaskPtr);
			// Remove current active task from list
			AddReleasedContextForGC(ActiveTaskArray[TaskIndex]->Context.Get());
			PCGGraphExecutor::RemoveAtFromActiveTaskArrayNoLock(ActiveTaskArray, TaskIndex);
		}

		// Make sure we set this to false inside lock
		ActiveTask.StopExecuting();

		// Next Scheduling call needs to check if this task completion unblocked some paused task(s)
		bNeedToCheckPausedTasks = true;
	}
}

void FPCGGraphExecutor::OnTaskInputsReady(TArrayView<FPCGGraphTask> InNewReadyTasks, FCachedResults& OutCachedResults, bool bIsInGameThread)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::OnTaskInputsReady);
	FTaskIds ResultsToMarkAsRead;
	TArray<FPCGGraphTask, TInlineAllocator<16>> UncachedReadyTasks;

	for (FPCGGraphTask& Task : InNewReadyTasks)
	{
		// LocalResultsToMarkAsRead does not contain duplicates
		FTaskIds LocalResultsToMarkAsRead;
		if (SetupTask(Task, LocalResultsToMarkAsRead))
		{
			check(Task.Element);

			FCachedResult* CachedResult = nullptr;

			// Query Cache if we can here, if not the Task will be pushed to ReadyTasks and PrepareForExecute will be called on MainThread if needed
			if (bIsInGameThread || !Task.CanExecuteOnlyOnMainThread())
			{
				PrepareForExecute(Task, CachedResult, /*bLiveTasksLockAlreadyLocked=*/false);
			}

			if (CachedResult)
			{
				check(CachedResult->TaskId != InvalidPCGTaskId);
				OutCachedResults.Add(CachedResult);
			}
			else
			{
				UncachedReadyTasks.Emplace(MoveTemp(Task));
			}
		}

		// ResultsToMarkAsRead can contain duplicates as we want to mark the same input as read multiple times if multiple tasks depend on it
		ResultsToMarkAsRead.Append(LocalResultsToMarkAsRead);
	}

	if (UncachedReadyTasks.Num() > 0)
	{
		PCG::TScopeLock ChildScopeLock(LiveTasksLock);
		ReadyTasks.Append(MoveTemp(UncachedReadyTasks));
	}

	if (ResultsToMarkAsRead.Num() > 0)
	{
		MarkInputResults(ResultsToMarkAsRead);
	}
}

bool FPCGGraphExecutor::SetupTask(FPCGGraphTask& Task, FTaskIds& ResultsToMarkAsRead)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::SetupTask);
	if (Task.bHasDoneSetup)
	{
		return true;
	}

	Task.bHasDoneSetup = true;

	// Build input
	BuildTaskInput(Task, ResultsToMarkAsRead);

	// Initialize the element if needed (required to know whether it will run on the main thread or not)
	if (!Task.Element)
	{
		// TODO: It may be more efficient to do this in UPCGGraph::PostLoad(), since these are only related to cooking (with the exception of EPCGElementSource::FromNode).
		if (Task.ElementSource == EPCGElementSource::Trivial)
		{
			Task.Element = GraphCompiler.GetSharedTrivialElement();
		}
		else if (Task.ElementSource == EPCGElementSource::TrivialPostGraph)
		{
			Task.Element = GraphCompiler.GetSharedTrivialPostGraphElement();
		}
		else if (Task.ElementSource == EPCGElementSource::Gather)
		{
			Task.Element = GraphCompiler.GetSharedGatherElement();
		}
		else if (Task.ElementSource == EPCGElementSource::FromNode)
		{
			// Get appropriate settings
			check(Task.Node);
			const UPCGSettings* Settings = Task.TaskInput.GetSettings(Task.Node->GetSettings());

			if (Settings)
			{
				Task.Element = Settings->GetElement();
			}
		}
		else if (Task.ElementSource == EPCGElementSource::FromCookedSettings)
		{
			check(Task.CookedSettings);
			Task.Element = Task.CookedSettings->GetElement();
		}
		else
		{
			checkNoEntry();
		}
	}

	// At this point, if the task doesn't have an element, we will never be able to execute it, so we can drop it.
	if (!Task.Element)
	{
		check(!Task.Context);
		return false;
	}

	return true;
}

void FPCGGraphExecutor::PrepareForExecute(FPCGGraphTask& Task, FCachedResult*& OutCachedResult, bool bLiveTasksLockAlreadyLocked)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::PrepareForExecute);
	check(Task.bHasDoneSetup && !Task.bHasDonePrepareForExecute);
	
	// Set this only when exiting this method so that Context is created and is referencing the inputs
	ON_SCOPE_EXIT
	{
		Task.bHasDonePrepareForExecute = true;
	};

	PCGGraphExecutionLogging::LogTaskExecute(Task);

	// If a task is cacheable and has been cached, then we don't need to create an active task for it unless
	// there is an execution mode that would prevent us from doing so.
	const UPCGSettingsInterface* TaskSettingsInterface = Task.TaskInput.GetSettingsInterface(Task.Node ? Task.Node->GetSettingsInterface() : nullptr);
	const UPCGSettings* TaskSettings = TaskSettingsInterface ? TaskSettingsInterface->GetSettings() : nullptr;

	// Calculate Crc of dependencies (input data Crcs, settings) and use this as the key in the cache lookup
	FPCGGetFromCacheParams CacheParam;
	EPCGCachingStatus CacheStatus = EPCGCachingStatus::NotInCache;
	bool bResultAlreadyInCache = false;

	IPCGGraphExecutionSource* ExecutionSource = Task.ExecutionSource.Get();

	// This section requires the lock on the caching results to prevent interaction between different tasks and the GC so we avoid locking if cache is disabled.
	if(GraphCache.IsEnabled() && (!ExecutionSource || ExecutionSource->GetExecutionState().IsCacheEnabled()))
	{
		// Build cache param outside of lock
		if (Task.Element->BuildCacheParams(&GraphCache, Task.Node, Task.TaskInput, ExecutionSource, CacheParam))
		{
			// Check that we have a valid cache param to avoid locking for nothing
			if (CacheParam.Crc.IsValid())
			{
				PCG::TScopeLock ScopeLock(CachingResultsLock);
				FCachedResult LocalCachedResult;
				CacheStatus = Task.Element->RetrieveResultsFromCache(&GraphCache, CacheParam, LocalCachedResult.Output);

				bResultAlreadyInCache = (CacheStatus == EPCGCachingStatus::Cached);
				if (bResultAlreadyInCache)
				{
					check(!CachingResultsForGC.Contains(Task.NodeId));
					TUniquePtr<FCachedResult>& CachingResultPtr = CachingResultsForGC.Add(Task.NodeId, MakeUnique<FCachedResult>());
					OutCachedResult = CachingResultPtr.Get();
					OutCachedResult->TaskId = Task.NodeId;
					OutCachedResult->Output = MoveTemp(LocalCachedResult.Output);
				}
			}
		}
	}

	if (CacheStatus == EPCGCachingStatus::NotCacheable)
	{
		PCGGraphExecutionLogging::LogTaskExecuteCachingDisabled(Task);
	}

#if WITH_EDITOR
	const bool bNeedsToCreateActiveTask = !bResultAlreadyInCache || TaskSettingsInterface->bDebug;
#else
	const bool bNeedsToCreateActiveTask = !bResultAlreadyInCache;
#endif

	if (!bNeedsToCreateActiveTask)
	{
		check(OutCachedResult);
#if PCG_PROFILING_ENABLED
		// Doing this now since we're about to modify ReadyTasks potentially reallocating while Task is a reference. 
#if !WITH_EDITOR
		// Outside of editor, only collect this information if profiling is enabled. In editor inspection data is always collected.
		if (PCGProfilingLog::IsEnabled())
#endif
		{
			if (ExecutionSource)
			{
				if (Task.StackIndex != INDEX_NONE)
				{
					const FPCGStack* Stack = Task.GetStack();
					ExecutionSource->GetExecutionState().GetInspection().StoreInspectionData(Stack, Task.Node, nullptr, Task.TaskInput, OutCachedResult->Output, /*bUsedCache=*/true);
				}
			}
		}
#endif // PCG_PROFILING_ENABLED

		if (PCGGraphExecutor::CVarDynamicTaskCulling.GetValueOnAnyThread() && TaskSettings && TaskSettings->OutputPinsCanBeDeactivated() && OutCachedResult->Output.InactiveOutputPinBitmask != 0)
		{
			OutCachedResult->bDoDynamicTaskCulling = true;
			OutCachedResult->Stack = Task.GetStackHandle();
			OutCachedResult->Node = Task.Node;
		}

		// If the task is a post execute, then we can safely clear the data after getting it from the results.
		OutCachedResult->bIsPostGraphTask = (Task.Element == GraphCompiler.GetSharedTrivialPostGraphElement());

		bNeedToExecuteTasksEnded = true;

		return;
	}
		
	// Allocate context if not previously done
	check(!Task.Context);
	
	{
		Task.Context = Task.Element->Initialize(FPCGInitializeElementParams(&Task.TaskInput, ExecutionSource, Task.Node));
		Task.Context->GraphExecutor = this->AsWeak();

		// Skip PostLoad only when not on GameThread
		// The reason is that some nodes like SpawnActor / CreateTargetActor have TemplateActor subobjects that need PostLoad calls to function properly
		// If a node supports being prepared outside of the GameThread it needs to make sure that it and its sub-object do not depend on PostLoad being called.
		// This is true for most nodes as they only run some deprecation code in their PostLoad which isn't needed after duplication
		const bool bSkipPostLoad = !IsInGameThread();
		Task.Context->InitializeSettings(bSkipPostLoad);
		
		Task.Context->TaskId = Task.NodeId;
		Task.Context->CompiledTaskId = Task.CompiledTaskId;
		Task.Context->DependenciesCrc = CacheParam.Crc;
		Task.Context->StackHandle = Task.GetStackHandle();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Task.Context->Stack = Task.GetStack();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Safe to remove async flags on CombineParams here
		PCGGraphExecutor::ClearAsyncFlags(Task.CombineParamsAsyncObjects);
	}

#if WITH_EDITOR
	if (bResultAlreadyInCache)
	{
		check(OutCachedResult->TaskId != InvalidPCGTaskId);
		Task.bIsBypassed = true;

		// Directly transfer Cache result for debugged tasks to Context output
		PCG::TScopeLock ScopeLock(CachingResultsLock);
		Task.Context->OutputData = MoveTemp(OutCachedResult->Output);
		OutCachedResult = nullptr;
		CachingResultsForGC.Remove(Task.NodeId);
	}
#endif
}

#if WITH_EDITOR
void FPCGGraphExecutor::DebugDisplayTask(TWeakPtr<FPCGGraphExecutor::FGameThreadHandler> WeakHandler, TSharedPtr<FPCGGraphActiveTask> ActiveTaskPtr)
{
	TSharedPtr<FPCGGraphExecutor::FGameThreadHandler> Handler = WeakHandler.Pin();
	if (!Handler)
	{
		return;
	}

	if (!ActiveTaskPtr->Element->DebugDisplay(ActiveTaskPtr->Context.Get()))
	{
		// Not done, run on next game thread next tick instead. This is safe to call even if running inside a begin tick action.
		FPCGModule::GetPCGModuleChecked().ExecuteNextTick([WeakHandler, ActiveTaskPtr]()
		{
			DebugDisplayTask(WeakHandler, ActiveTaskPtr);
		});

		return;
	}

	// If we fall through to here, debug display completed. Cleanup.
	PCG::TScopeLock ScopeLock(Handler->GetExecutor()->CollectGCReferenceTasksLock);
	Handler->GetExecutor()->CollectGCReferenceTasks.Remove(ActiveTaskPtr);
}
#endif // WITH_EDITOR

double FPCGGraphExecutor::GetTickBudgetInSeconds() const
{
#if WITH_EDITOR
	return PCGGraphExecutor::GetCVarValueForWorld(World, PCGGraphExecutor::CVarTimePerFrame, PCGGraphExecutor::CVarEditorTimePerFrame) / 1000.0;
#else
	return PCGGraphExecutor::CVarTimePerFrame.GetValueOnAnyThread() / 1000.0;
#endif
}

double FPCGGraphExecutor::GetPostExecuteConsumerThreadBudgetInSeconds() const
{
#if WITH_EDITOR
	return PCGGraphExecutor::GetCVarValueForWorld(World, PCGGraphExecutor::CVarRuntimePostExecuteConsumerThreadTime, PCGGraphExecutor::CVarEditorPostExecuteConsumerThreadTime) / 1000.0;
#else
	return PCGGraphExecutor::CVarRuntimePostExecuteConsumerThreadTime.GetValueOnAnyThread() / 1000.0;
#endif
}

int32 FPCGGraphExecutor::GetMaxNumProcessingTasks() const
{
#if WITH_EDITOR
	return PCGGraphExecutor::GetCVarValueForWorld(World, PCGGraphExecutor::CVarRuntimeMaxNumProcessingTasks, PCGGraphExecutor::CVarEditorMaxNumProcessingTasks);
#else
	return PCGGraphExecutor::CVarRuntimeMaxNumProcessingTasks.GetValueOnAnyThread();
#endif
}

float FPCGGraphExecutor::GetMaxPercentageOfThreadsToUse() const
{
#if WITH_EDITOR
	return PCGGraphExecutor::GetCVarValueForWorld(World, PCGGraphExecutor::CVarRuntimeMaxPercentageOfThreadsToUse, PCGGraphExecutor::CVarEditorMaxPercentageOfThreadsToUse);
#else
	return PCGGraphExecutor::CVarRuntimeMaxPercentageOfThreadsToUse.GetValueOnAnyThread();
#endif
}

float FPCGGraphExecutor::GetMaxPercentageOfExecutingThreads() const
{
#if WITH_EDITOR
	return PCGGraphExecutor::GetCVarValueForWorld(World, PCGGraphExecutor::CVarRuntimeMaxPercentageOfExecutingThreads, PCGGraphExecutor::CVarEditorMaxPercentageOfExecutingThreads);
#else
	return PCGGraphExecutor::CVarRuntimeMaxPercentageOfExecutingThreads.GetValueOnAnyThread();
#endif
}

int32 FPCGGraphExecutor::GetMaxExecutingThreads() const
{
#if WITH_EDITOR
	return PCGGraphExecutor::GetCVarValueForWorld(World, PCGGraphExecutor::CVarRuntimeMaxExecutingThreads, PCGGraphExecutor::CVarEditorMaxExecutingThreads);
#else
	return PCGGraphExecutor::CVarRuntimeMaxExecutingThreads.GetValueOnAnyThread();
#endif
}

bool FPCGGraphExecutor::IsPostExecuteConsumerThreadEnabled() const
{
	if (!FApp::ShouldUseThreadingForPerformance())
	{
		return false;
	}

#if WITH_EDITOR
	return PCGGraphExecutor::GetCVarValueForWorld(World, PCGGraphExecutor::CVarRuntimeEnablePostExecuteConsumerThread, PCGGraphExecutor::CVarEditorEnablePostExecuteConsumerThread);
#else
	return PCGGraphExecutor::CVarRuntimeEnablePostExecuteConsumerThread.GetValueOnAnyThread();
#endif
}

void FPCGGraphExecutor::ProcessAsyncExecuteResult(TSharedPtr<FPCGGraphActiveTask> Task, bool bIsDone, bool bIsPaused, double EndTime)
{
	PostTaskExecute(Task, bIsDone);

	if (bIsDone || bIsPaused)
	{
		// If this task is paused it is possible it scheduled some new tasks, process those new tasks before calling ExecuteScheduling (so we are not one frame behind)
		if (bIsPaused)
		{
			ProcessScheduledTasks(EndTime);
		}

		if (FPlatformTime::Seconds() < EndTime)
		{
			// Budget is provided by CVarAsyncOutOfTickBudgetInMilliseconds
			ExecuteScheduling(EndTime, nullptr,  /*bForceCheckPausedTasks=*/bIsPaused);
		}
	}
}

uint32 FPCGGraphExecutor::FPostTaskExecuteConsumerThread::Run()
{
	check(Owner);
	while (Event->Wait() && !bQuit)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::FPostTaskExecuteConsumerThread::Run);
		const double EndTime = FPlatformTime::Seconds() + Owner->GetPostExecuteConsumerThreadBudgetInSeconds();

		FPostTaskExecuteItem Item;
		while (Queue.Dequeue(Item))
		{
			Owner->ProcessAsyncExecuteResult(Item.Task, Item.bIsDone, Item.bIsPaused, EndTime);

			// Yield everytime we've exhausted our budget
			if (FPlatformTime::Seconds() > EndTime)
			{
				FPlatformProcess::YieldThread();
			}
		}
	}

	return 0;
}

FPCGGraphExecutor::FPostTaskExecuteConsumerThread::FPostTaskExecuteConsumerThread(FPCGGraphExecutor* InOwner)
	: Owner(InOwner)
{
	Thread = FRunnableThread::Create(this, TEXT("FPostTaskExecuteConsumerThread"));
}

FPCGGraphExecutor::FPostTaskExecuteConsumerThread::~FPostTaskExecuteConsumerThread()
{
	bQuit = true;
	Event->Trigger();
	Thread->Kill(true);
	delete Thread;
	Thread = nullptr;
}

void FPCGGraphExecutor::StartPostExecuteConsumerThread()
{
	if (PostTaskExecuteConsumerThread.IsValid() || !IsPostExecuteConsumerThreadEnabled())
	{
		return;
	}

	PostTaskExecuteConsumerThread = MakeUnique<FPostTaskExecuteConsumerThread>(this);
}

bool FPCGGraphExecutor::ExecuteScheduling(double EndTime, TSharedPtr<FPCGGraphActiveTask>* OutMainThreadTask, bool bForceCheckPausedTasks)
{
	// Can currently be disabled by cancellation
	if (SchedulingDisabled)
	{
		return false;
	}

	const float OutOfTickBudgetInSeconds = FMath::Max(1.0f, FPCGAsync::ConsoleVar::CVarAsyncOutOfTickBudgetInMilliseconds.GetValueOnAnyThread()) / 1000.f;
	check(EndTime > 0.0);

	bool bStateChanged = false;
	const bool bIsInGameThread = IsInGameThread();
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::ExecuteScheduling);
	
	const float MaxPercentageOfThreadsToUse = FMath::Clamp(GetMaxPercentageOfThreadsToUse(), 0.0f, 1.0f);

	// Clamp to 0.8 to never allow starving of PCGAsync Tasks
	const float MaxPercentageOfExecutingThreads = FMath::Clamp(GetMaxPercentageOfExecutingThreads(), 0.0f, 0.8f);

	// Number of threads to use across PCG
	const int32 MaxNumThreads = FApp::ShouldUseThreadingForPerformance() ? FMath::Max(0, (int32)(LowLevelTasks::FScheduler::Get().GetNumWorkers() * MaxPercentageOfThreadsToUse)) : 0;

	// Number of threads to use for PCG Element execution (outside of main thread tick)
	const int32 MaxExecutingThreadsPriorHardCap = PCGGraphExecutor::CVarGraphMultithreading.GetValueOnAnyThread() ? FMath::Max(0, (int32)(MaxNumThreads * MaxPercentageOfExecutingThreads)) : 0;

	// Final number of threads to use for PCG Element execution (outside of main thread tick)
	const int32 MaxExecutingThreads = FMath::Min(MaxExecutingThreadsPriorHardCap, FMath::Max(0, GetMaxExecutingThreads()));

	// Number of threads to use for PCG Async operations 
	const int32 MaxPCGAsyncThreads = FMath::Min(MaxNumThreads - MaxExecutingThreads, GetMaxNumProcessingTasks());

	FCachedResults CachedResults;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::ExecuteScheduling::ReadyTasks);
		const bool bTryLock = (OutMainThreadTask == nullptr);
		PCG::TScopeLock ScopeLock(LiveTasksLock, bTryLock);
		if (ScopeLock.IsLocked())
		{
			if (bNeedToCheckPausedTasks || bForceCheckPausedTasks)
			{
				bNeedToCheckPausedTasks = false;

				// First, wake up any paused tasks that can be reactivated
				for (int32 PausedTaskIndex = PausedTasks.Num() - 1; PausedTaskIndex >= 0; --PausedTaskIndex)
				{
					TSharedPtr<FPCGGraphActiveTask>& PausedTask = PausedTasks[PausedTaskIndex];

					// It would be surprising that just looping through paused tasks would go over budget so no need to check time here
					if (PausedTask->Context->bIsPaused)
					{
						continue;
					}

					const bool bIsGameThreadOnly = PausedTask->CanExecuteOnlyOnMainThread();

					TArray<TSharedPtr<FPCGGraphActiveTask>>& ActiveTaskArray = bIsGameThreadOnly ? ActiveTasksGameThreadOnly : ActiveTasks;
					PausedTask->bIsGameThreadOnly = bIsGameThreadOnly;

					PCGGraphExecutor::InsertToActiveTaskArrayNoLock(0, ActiveTaskArray, PausedTask);
					PCGGraphExecutor::RemoveAtFromActiveTaskArrayNoLock(PausedTasks, PausedTaskIndex);
					bStateChanged = true;

					if (FPlatformTime::Seconds() > EndTime)
					{
						// Allow at least one loop of ReadyTasks
						break;
					}
				}
			}

			for (int32 ReadyTaskIndex = ReadyTasks.Num() - 1; ReadyTaskIndex >= 0; --ReadyTaskIndex)
			{
				FPCGGraphTask& Task = ReadyTasks[ReadyTaskIndex];
				check(Task.bHasDoneSetup);

				if (!Task.bHasDonePrepareForExecute)
				{
					if (!bIsInGameThread && Task.CanExecuteOnlyOnMainThread())
					{
						continue;
					}

					FCachedResult* CachedResult = nullptr;
					PrepareForExecute(Task, CachedResult, /*bLiveTasksLockAlreadyLocked=*/true);
					check(Task.bHasDonePrepareForExecute);

					if (CachedResult)
					{
#if WITH_EDITOR
						check(!Task.bIsBypassed);
#endif
						check(CachedResult->TaskId != InvalidPCGTaskId);
						CachedResults.Add(MoveTemp(CachedResult));
						ReadyTasks.RemoveAtSwap(ReadyTaskIndex, EAllowShrinking::No);
						bStateChanged = true;

						if (FPlatformTime::Seconds() > EndTime)
						{
							// Make sure CachedResults are processed and that we process one loop of active tasks
							break;
						}
						continue;
					}

					check(Task.Context);
				}

				// Validate that we can start this task now
				const bool bIsGameThreadOnly = Task.CanExecuteOnlyOnMainThread();
				TArray<TSharedPtr<FPCGGraphActiveTask>>& ActiveTaskArray = bIsGameThreadOnly ? ActiveTasksGameThreadOnly : ActiveTasks;

				FPCGGraphActiveTask& ActiveTask = *ActiveTaskArray.Emplace_GetRef(MakeShared<FPCGGraphActiveTask>());
				ActiveTask.TaskIndex = ActiveTaskArray.Num() - 1;
				ActiveTask.bIsGameThreadOnly = bIsGameThreadOnly;
				ActiveTask.Inputs = MoveTemp(Task.Inputs);
				ActiveTask.Element = Task.Element;
				ActiveTask.NodeId = Task.NodeId;
				ActiveTask.Context = TUniquePtr<FPCGContext>(Task.Context);
				ActiveTask.StackIndex = Task.StackIndex;
				ActiveTask.StackContext = Task.StackContext;
#if WITH_EDITOR
				ActiveTask.bIsBypassed = Task.bIsBypassed;
#endif
				// Move the task up front if it needs to run on the main thread
				ReadyTasks.RemoveAtSwap(ReadyTaskIndex, EAllowShrinking::No);
				bStateChanged = true;

				if (FPlatformTime::Seconds() > EndTime)
				{
					// Make sure CachedResults are processed and that we process one loop of active tasks
					break;
				}
			}
		}
	}

	ProcessCachedResults(CachedResults);

	// Dispatch Tasks
	{
		PCG::TScopeLock ScopeLock(LiveTasksLock, !bIsInGameThread);
		if (ScopeLock.IsLocked())
		{
			if (bIsInGameThread && OutMainThreadTask)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::Execute::SelectMainThreadTask);
				// Execute main thread task
				if (!ActiveTasksGameThreadOnly.IsEmpty() || !ActiveTasks.IsEmpty())
				{
					auto FindTaskToExecute = [](const TSharedPtr<FPCGGraphActiveTask>& InActiveTask)
					{
						return !InActiveTask->bIsExecutingTask && !InActiveTask->Context->bIsPaused;
					};

					TSharedPtr<FPCGGraphActiveTask>* FoundActiveTask = ActiveTasksGameThreadOnly.FindByPredicate(FindTaskToExecute);
					if (!FoundActiveTask)
					{
						FoundActiveTask = ActiveTasks.FindByPredicate(FindTaskToExecute);
					}

					if (FoundActiveTask)
					{
						// Sort should give us best target to execute
						TSharedPtr<FPCGGraphActiveTask>& ActiveTask = *FoundActiveTask;

						// Skip already running task
						check (!ActiveTask->bIsExecutingTask && !ActiveTask->Context->bIsPaused)
						
						ActiveTask->StartExecuting();
						ActiveTask->Context->AsyncState.NumAvailableTasks = FMath::Max(1, MaxPCGAsyncThreads);
						ActiveTask->Context->AsyncState.bIsRunningOnMainThread = true;
						ActiveTask->Context->AsyncState.bIsRunningOutOfTick = false;
						ActiveTask->Context->AsyncState.EndTime = EndTime;
						*OutMainThreadTask = ActiveTask;
						bStateChanged = true;
					}
				}
			}

			// Consider the Main Thread task (+1)
			if (MaxExecutingThreads > 0 && (MaxExecutingThreads+1) > FPCGGraphActiveTask::NumExecuting)
			{
				const double SchedulingBudgetInSeconds = GetTickBudgetInSeconds();

				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::ExecuteScheduling::LaunchTasks);
				for (int32 ExecutionIndex = 0; ExecutionIndex < ActiveTasks.Num(); ++ExecutionIndex)
				{
					TSharedPtr<FPCGGraphActiveTask>& ActiveTask = ActiveTasks[ExecutionIndex];

					if ((MaxExecutingThreads+1) <= FPCGGraphActiveTask::NumExecuting)
					{
						break;
					}

					// Running task
					if (ActiveTask->bIsExecutingTask || ActiveTask->Context->bIsPaused)
					{
						continue;
					}

					// Main thread only
					if (ActiveTask->CanExecuteOnlyOnMainThread())
					{
						continue;
					}
#if WITH_EDITOR
					if (!ActiveTask->bIsBypassed)
#endif
					{
						check(ActiveTask->Context->DynamicDependencies.IsEmpty());
						check(ActiveTask->Context->CurrentPhase <= EPCGExecutionPhase::Done);
						check(MaxNumThreads > 0);
						ActiveTask->Context->AsyncState.NumAvailableTasks = FMath::Max(1, MaxPCGAsyncThreads);
						ActiveTask->Context->AsyncState.EndTime = EndTime;
						ActiveTask->Context->AsyncState.bIsRunningOnMainThread = false;

						// Capture copy of SharedPtr so task stays valid while Async task is alive
						ActiveTask->StartExecuting();
						ActiveTask->ExecutingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, ActiveTask, SchedulingBudgetInSeconds]() -> bool
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::ExecuteAsyncTask);
							LLM_SCOPE_BYTAG(PCG);

							// Set Out Of tick budget time (minimum 1ms)
							const float OutOfTickBudgetInSeconds = FMath::Max(1.0f, FPCGAsync::ConsoleVar::CVarAsyncOutOfTickBudgetInMilliseconds.GetValueOnAnyThread()) / 1000.f;
							ActiveTask->Context->AsyncState.EndTime = FPlatformTime::Seconds() + OutOfTickBudgetInSeconds;

							const bool bIsDone = ActiveTask->bWasCancelled || ActiveTask->Element->Execute(ActiveTask->Context.Get());
							const bool bIsPaused = ActiveTask->Context->bIsPaused;

							// If Scheduling is disabled it means we are cancelling tasks so in that case avoid using the consumer thread so that the Wait calls in Cancel
							// can rely on the task having been fully executed (PostTaskExecute included)
							if (!SchedulingDisabled && PostTaskExecuteConsumerThread.IsValid())
							{
								PostTaskExecuteConsumerThread->Queue.Enqueue(ActiveTask, bIsDone, bIsPaused);
								PostTaskExecuteConsumerThread->Event->Trigger();
							}
							else
							{
								const double EndTime = FPlatformTime::Seconds() + SchedulingBudgetInSeconds;
								ProcessAsyncExecuteResult(ActiveTask, bIsDone, bIsPaused, EndTime);
							}
							return bIsDone;
						});
						bStateChanged = true;

						if (FPlatformTime::Seconds() > EndTime)
						{
							// Make sure ProcessScheduledTasks runs
							break;
						}
					}
				}
			}
		}
	}

	bStateChanged |= ProcessScheduledTasks(EndTime);

	return bStateChanged;
}

void FPCGGraphExecutor::Execute()
{
	double EndTime = FPlatformTime::Seconds() + FPCGGraphExecutor::GetTickBudgetInSeconds();
	Execute(EndTime);
}

struct FPCGExcludeScopeFromTickBudget
{
	FPCGExcludeScopeFromTickBudget(double& InEndTime)
		: EndTime(InEndTime)
	{
		StartTime = FPlatformTime::Seconds();
	}

	~FPCGExcludeScopeFromTickBudget()
	{
		EndTime += FPlatformTime::Seconds() - StartTime;
	}

private:
	double& EndTime;
	double StartTime;
};

/**
 * Graph Execution Pipeline:
 * 
 * - New Tasks first enter the execution pipeline through [ScheduledTasks]
 * - Each [ScheduledTasks] is a list of Task with Input dependencies on other Tasks
 * - [ScheduleTasks] are processed through [FPCGraphExecutor::ProcessScheduledTasks] which can happen on GameThread/WorkerThread 
 * - On WorkerThreads [FPCGraphExecutor::ProcessScheduledTasks] will try to acquire the Lock and if it can't will just skip through, on GameThread it will wait until it can grab the lock (ScheduleLock)
*	 The reason why WorkerThreads will only Try to lock is because [FPCGGraphExecutor::ExecuteScheduling] runs in the same UE::Task as the Node execution and we don't want to make those spin for too long. (Let's call it opportunistic scheduling)
* 
 * - [FPCGGraphExecutor::ProcessScheduledTasks] will move [ScheduledTasks] into one of [Tasks] if task still waiting on other task inputs or [ReadyTasks] if tasks is ready to execute
 * 
 * - When moving a [ScheduledTasks] into [ReadyTasks] we call [FPCGGraphExecutor::SetupTask] which will build the task inputs and its FPCGGraphElement used for execution.
 *	 It will also try and call [FPCGGraphExecutor::PrepareForExecute] on the task if it is on the GameThread or if it is on a WorkerThread and the FPCGGraphElement allows it [!FPCGGraphElement::CanExecuteOnlyOnMainThread(nullptr)]
 * - [FPCGGraphExecutor::PrepareForExecute] will query the Cache and might return a valid Output or it will create the FPCGContext for the task so that it can be executed
 *
 * - [ReadyTasks] are processed through [FPCGGraphExecutor::ExecuteScheduling] which can happen on GameThread/WorkerThread
 * - On WorkerThreads [FPCGGraphExecutor::ExecuteScheduling] will try to acquire the Lock and if it can't will just skip through, on GameThread it will wait until it can grab the lock. (LiveTasksLock) (same reason as above)
 *	 
 * - [ReadyTasks] will be moved to [ActiveTasks/ActiveTasksGameThreadOnly] through [FPCGGraphExecutor::ExecuteScheduling]
 * - This will only happen if Task.bHasDonePrepareForExecute is true or if we can run [FPCGGraphExecutor::PrepareForExecute] on it. (this can always happen on GameThread but not always on WorkerThreads as explained before)
 * 
 * - [ActiveTasks/ActiveTasksGameThreadOnly] are also processed in [FPCGGraphExecutor::ExecuteScheduling]
 * - The GameThread will always try to execute the first [ActiveTasksGameThreadOnly] or [ActiveTasks] if no tasks need to be executed on GameThread, it will do this [FPCGGraphExecutor::ExecuteScheduling] while it exhausts its GameThread budget
 * - Both GameThread/WorkerThread will also spawn new UE::Tasks for the remaining [ActiveTasks] that are allowed to run outside of the GameThread still happens in [FPCGGraphExecutor::ExecuteScheduling]
 * 
 * - When an [ActiveTasks/ActiveTasksGameThreadOnly] Execute call returns and it returns true, it means that the task is done. In this case we call [FPCGGraphExecutor::StoreResults] followed by [FPCGGraphExecutor::QueueNextTasks] this will potentially move some [Tasks] to [ReadyTasks]
 *   so that new tasks can get executed now that their inputs have been filled
 * 
 * - After a call to Execute, if a [ActiveTasks/ActiveTasksGameThreadOnly] bPaused flag is true, it gets moved to [PausedTasks]
 * - [PausedTasks] gets processed at least once a frame to see if some tasks are no longer paused and need to be moved back to [ActiveTasks/ActiveTasksGameThreadOnly]
 * - It will also be processed again if some other [ActiveTasks/ActiveTasksGameThreadOnly] completes execution
 * 
 * - Cache information: When we call [FPCGGraphExecutor::PrepareForExecute] is called on a task, if a Cache result is found for this task then we can directly call [FPCGGraphExecutor::StoreResults] and [FPCGGraphExecutor::QueueNextTasks] on it skipping execution [FPCGGraphExecutor::ProcessCachedResults]
 * - An exception to this is when the FPCGElement is marked for debugging then we move the task through the normal execution pipeline but set a flag on it so that the actual Execute call can be skipped
 * - [CachingResults] exists only to keep GC references to PCG Data that has been read from the Cache but hasn't yet been moved to the [TaskOutputs]
 * 
 * - [CVars]
 *   pcg.GraphMultithreading : Enables multithreading on this execution pipeline, without that flag most multithreading happens only in PCGAsync.h/.cpp
 *	 pcg.Graph.MaxWorkerTasks : Limits the number of UE::Tasks that [FPCGGraphExecutor::ExecuteScheduling] can launch
 *   pcg.DisableAsyncTimeSlicing : Disables timeslicing in UE::Tasks launched by [FPCGGraphExecutor::ExecuteScheduling]
 *   pcg.Async.OutOfTickBudgetInMilliseconds : Timeslicing budget for UE::Tasks launched by [FPCGGraphExecutor::ExecuteScheduling]
 *   pcg.DisableAsyncTimeSlicingOnGameThread : Disables timeslicing on GameThread execution
 */
void FPCGGraphExecutor::Execute(double& InOutEndTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::Execute);

	// Start Thread if enabled
	StartPostExecuteConsumerThread();

#if WITH_EDITOR

	{
		FPCGExcludeScopeFromTickBudget Exclusion(InOutEndTime);

		const bool bIsEditorWorld = !World || !World->IsGameWorld();

		if (PCGSystemSwitches::CVarPausePCGExecution.GetValueOnAnyThread() || (bIsEditorWorld && GEditor && GEditor->IsTransactionActive() && PCGGraphExecutor::CVarPausePCGExecutionWhileTransactionActive.GetValueOnAnyThread()))
		{
			UpdateGenerationNotification();
			return;
		}
	}
#endif // WITH_EDITOR

	const double StartTime = FPlatformTime::Seconds();
	const double Budget = FMath::Max(0.0, InOutEndTime - StartTime);

	const double ProcessScheduledTasksBudgetPercentage = 0.3;
	const double ProcessScheduledTasksEndTime = StartTime + Budget * ProcessScheduledTasksBudgetPercentage;

	ProcessScheduledTasks(ProcessScheduledTasksEndTime);

	const double ProcessScheduledTasksTime = FPlatformTime::Seconds() - StartTime;
	
#if WITH_EDITOR
	{
		FPCGExcludeScopeFromTickBudget Exclusion(InOutEndTime);

		UpdateGenerationNotification();
	}
#endif // WITH_EDITOR
				
	bool bContinueExecute = GetNonScheduledRemainingTaskCount() > 0;
	bool bFirstLoop = true;

#if WITH_EDITOR
	const bool bOnWorldGenerationProcessBroadcasted = bContinueExecute && World && !World->IsGameWorld();
	if (bOnWorldGenerationProcessBroadcasted)
	{
		FEditorDelegates::OnPreWorldGenerationProcess.Broadcast(World);
	}
#endif // WITH_EDITOR

	while ((bFirstLoop || (FPlatformTime::Seconds() < InOutEndTime)) && bContinueExecute)
	{
		TSharedPtr<FPCGGraphActiveTask> MainThreadTask;
		bContinueExecute = ExecuteScheduling(InOutEndTime, &MainThreadTask, /*bForceCheckPausedTasks=*/bFirstLoop);
		bFirstLoop = false;

		if (MainThreadTask)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::Execute::ExecuteMainThreadTask);
#if WITH_EDITOR
			const bool bIsDone = (MainThreadTask->bIsBypassed || MainThreadTask->bWasCancelled || MainThreadTask->Element->Execute(MainThreadTask->Context.Get()));
#else
			const bool bIsDone = (MainThreadTask->bWasCancelled || MainThreadTask->Element->Execute(MainThreadTask->Context.Get()));
#endif
			PostTaskExecute(MainThreadTask, bIsDone);
		}
	}

	ExecuteTasksEnded();

#if WITH_EDITOR
	if (bOnWorldGenerationProcessBroadcasted)
	{
		FEditorDelegates::OnPostWorldGenerationProcess.Broadcast(World);
	}
#endif // WITH_EDITOR
}

void FPCGGraphExecutor::ClearAllTasks()
{
	Tasks.Reset();
	TaskSuccessors.Reset();
	
	// Make sure we don't leak preallocated contexts
	for (FPCGGraphTask& ReadyTask : ReadyTasks)
	{
		FPCGContext::Release(ReadyTask.Context);
	}

	ReadyTasks.Reset();
	ActiveTasks.Reset();
	ActiveTasksGameThreadOnly.Reset();
	PausedTasks.Reset();
	PausedTaskSuccessors.Reset();
}

void FPCGGraphExecutor::QueueNextTasksInternal(TConstArrayView<FPCGTaskId> FinishedTasks, FCachedResults& OutCachedResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::QueueNextTasks);
	const bool bIsInGameThread = IsInGameThread();

	TSet<FPCGTaskId> Successors;
	TArray<FPCGGraphTask, TInlineAllocator<16>> NewReadyTasks;
	{
		PCG::TScopeLock ScopeLock(TasksLock);
		PCG::TSharedScopeLock ReadLock(TaskOutputsLock);
		for (FPCGTaskId FinishedTask : FinishedTasks)
		{
			if (TaskSuccessors.RemoveAndCopyValue(FinishedTask, Successors))
			{
				for (FPCGTaskId Successor : Successors)
				{
					bool bAllPrerequisitesMet = true;

					// When running multithreaded it is possible for the Successor to already have been moved to ReadyTask after the current task stored its results
					if (FPCGGraphTask* SuccessorTaskPtr = Tasks.Find(Successor))
					{
						FPCGGraphTask& SuccessorTask = *SuccessorTaskPtr;

						if (SuccessorTask.Inputs.Num() > 0)
						{
							for (const FPCGGraphTaskInput& Input : SuccessorTask.Inputs)
							{
								bAllPrerequisitesMet &= TaskOutputs.Contains(Input.TaskId);
							}
						}

						if (bAllPrerequisitesMet)
						{
							NewReadyTasks.Add(MoveTemp(SuccessorTask));
							++NewReadyTaskCount;
							Tasks.Remove(Successor);
						}
					}
				}
			}
		}
	}

	if (NewReadyTasks.Num() > 0)
	{
		OnTaskInputsReady(NewReadyTasks, OutCachedResults, bIsInGameThread);
		NewReadyTaskCount -= NewReadyTasks.Num();
	}

	{
		PCG::TScopeLock PauseScopeLock(PausedTaskSuccessorsLock);
		for (FPCGTaskId FinishedTask : FinishedTasks)
		{
			TArray<TSharedPtr<FPCGGraphActiveTask>> ActiveTaskSuccessors;

			if (PausedTaskSuccessors.RemoveAndCopyValue(FinishedTask, ActiveTaskSuccessors))
			{
				for (const TSharedPtr<FPCGGraphActiveTask>& ActiveTask : ActiveTaskSuccessors)
				{
					ActiveTask->Context->DynamicDependencies.Remove(FinishedTask);
					if (ActiveTask->Context->DynamicDependencies.IsEmpty())
					{
						ActiveTask->Context->bIsPaused = false;
					}
				}
			}
		}
	}
}

void FPCGGraphExecutor::QueueNextTasks(TConstArrayView<FPCGTaskId> FinishedTasks)
{
	FCachedResults CachedResults;
	QueueNextTasksInternal(FinishedTasks, CachedResults);

	if (CachedResults.Num() > 0)
	{
		ProcessCachedResults(CachedResults);
	}
}

void FPCGGraphExecutor::ProcessCachedResults(TConstArrayView<FCachedResult*> CachedResults)
{
	if (CachedResults.IsEmpty())
	{
		return;
	}

	FTaskIds NextTasks;
	ProcessCachedResultsInternal(CachedResults, NextTasks);
	
	FCachedResults NewCachedResults;
	QueueNextTasksInternal(NextTasks, NewCachedResults);

	while (NewCachedResults.Num() > 0)
	{
		NextTasks.Reset();
		ProcessCachedResultsInternal(NewCachedResults, NextTasks);
		
		NewCachedResults.Reset();
		QueueNextTasksInternal(NextTasks, NewCachedResults);
	}
}

void FPCGGraphExecutor::ProcessCachedResultsInternal(TConstArrayView<FCachedResult*> CachedResults, FTaskIds& OutNextTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::ProcessCachedResults);
	
	TArray<FTaskResult, TInlineAllocator<16>> NewResults;

	for (FCachedResult* CachedResult : CachedResults)
	{
		check(CachedResult);
		if (CachedResult->bDoDynamicTaskCulling)
		{
			CullInactiveDownstreamNodes(CachedResult->TaskId, CachedResult->Output.InactiveOutputPinBitmask);
#if WITH_EDITOR
			SendInactivePinNotification(CachedResult->Node, CachedResult->Stack.GetStack(), CachedResult->Output.InactiveOutputPinBitmask);
#endif
		}

		
		NewResults.Add({ CachedResult->TaskId, &CachedResult->Output, CachedResult->bIsPostGraphTask });
		OutNextTasks.Add(CachedResult->TaskId);
	}

	StoreResults(NewResults);

	if (CachedResults.Num() > 0)
	{
		PCG::TScopeLock ScopeLock(CachingResultsLock);
		for (const FCachedResult* CachedResult : CachedResults)
		{
			check(CachingResultsForGC.Contains(CachedResult->TaskId));
			CachingResultsForGC.Remove(CachedResult->TaskId);
		}

		// Next Scheduling call needs to check if this task completion unblocked some paused task(s)
		bNeedToCheckPausedTasks = true;
	}
}

bool FPCGGraphExecutor::CancelNextTasks(FPCGTaskId CancelledTask, TSet<IPCGGraphExecutionSource*>& OutCancelledExecutionSources)
{
	check(IsInGameThread());

	bool bNewCancellations = false;

	TSet<FPCGTaskId> LocalSuccessors;
	{
		PCG::TScopeLock ScopeLock(TasksLock);
		if (TSet<FPCGTaskId>* Successors = TaskSuccessors.Find(CancelledTask))
		{
			LocalSuccessors = MoveTemp(*Successors);
			TaskSuccessors.Remove(CancelledTask);

			for (FPCGTaskId Successor : LocalSuccessors)
			{
				if (FPCGGraphTask* Task = Tasks.Find(Successor))
				{
					if (!OutCancelledExecutionSources.Contains(Task->ExecutionSource.Get()))
					{
						OutCancelledExecutionSources.Add(Task->ExecutionSource.Get());
						bNewCancellations = true;
					}

					if (Task->Element)
					{
						Task->Element->Abort(Task->Context);
					}

					RemoveTaskFromInputSuccessorsNoLock(Task->NodeId, Task->Inputs);
					Tasks.Remove(Successor);
				}
			}
		}
	}

	for (FPCGTaskId Successor : LocalSuccessors)
	{
		bNewCancellations |= CancelNextTasks(Successor, OutCancelledExecutionSources);
	}

	{
		PCG::TScopeLock PauseScopeLock(PausedTaskSuccessorsLock);

		TArray<TSharedPtr<FPCGGraphActiveTask>> ActiveTaskSuccessors;

		if (PausedTaskSuccessors.RemoveAndCopyValue(CancelledTask, ActiveTaskSuccessors))
		{
			for (const TSharedPtr<FPCGGraphActiveTask>& ActiveTask : ActiveTaskSuccessors)
			{
				if (!OutCancelledExecutionSources.Contains(ActiveTask->Context->ExecutionSource.Get()))
				{
					OutCancelledExecutionSources.Add(ActiveTask->Context->ExecutionSource.Get());
					bNewCancellations = true;
				}

				// Do not call abort as this is an ActiveTask and will be processed by the Cancellation of it's component
			}
		}
	}

	// Tasks cancelled might have an impact on scheduled-but-not-processed tasks
	{
		PCG::TScopeLock ScopeLock(ScheduleLock);
		for (FPCGGraphScheduleTask& ScheduledTask : ScheduledTasks)
		{
			if (!OutCancelledExecutionSources.Contains(ScheduledTask.ExecutionSource.Get()) &&
				Algo::AnyOf(ScheduledTask.Tasks[ScheduledTask.FirstTaskIndex].Inputs, [CancelledTask](const FPCGGraphTaskInput& Input) { return Input.TaskId == CancelledTask; }))
			{
				OutCancelledExecutionSources.Add(ScheduledTask.ExecutionSource.Get());
				bNewCancellations = true;
			}
		}
	}
	return bNewCancellations;
}

void FPCGGraphExecutor::RemoveTaskFromInputSuccessors(FPCGTaskId CancelledTask, const TArray<FPCGGraphTaskInput>& CancelledTaskInputs)
{
	PCG::TScopeLock ScopeLock(TasksLock);
	RemoveTaskFromInputSuccessorsNoLock(CancelledTask, CancelledTaskInputs);
}

void FPCGGraphExecutor::RemoveTaskFromInputSuccessorsNoLock(FPCGTaskId CancelledTask, const TArray<FPCGGraphTaskInput>& CancelledTaskInputs)
{
	for (const FPCGGraphTaskInput& Input : CancelledTaskInputs)
	{
		if (TSet<FPCGTaskId>* Successors = TaskSuccessors.Find(Input.TaskId))
		{
			Successors->Remove(CancelledTask);
			if (Successors->IsEmpty())
			{
				TaskSuccessors.Remove(Input.TaskId);
			}
		}
	}
}

void FPCGGraphExecutor::BuildTaskInput(FPCGGraphTask& Task, FTaskIds& ResultsToMarkAsRead)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::BuildTaskInput);

	auto LogDiscardedData = [&Task](const TOptional<FPCGPinProperties>& DownstreamPin)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		// Turn off eventual errors/warnings when the node is disabled, as this is irrelevant.
		const bool bNodeIsDisabled = (Task.Context && Task.Context->GetOriginalSettings<UPCGSettings>() && !Task.Context->GetOriginalSettings<UPCGSettings>()->bEnabled);
		if (bNodeIsDisabled)
		{
			return;
		}

		const FString Message = FString::Printf(
			TEXT("[%s] %s - BuildTaskInput - too many data items arriving on single data pin '%s', only first data item will be used"),
			Task.ExecutionSource.Get() ? *Task.ExecutionSource->GetExecutionState().GetDebugName() : TEXT("Missing Execution Source"),
			Task.Node ? *Task.Node->GetNodeTitle(EPCGNodeTitleType::ListView).ToString() : TEXT("MissingNode"),
			DownstreamPin.IsSet() ? *DownstreamPin.GetValue().Label.ToString() : TEXT("NoPin"));

#if WITH_EDITOR
		Task.LogVisual(ELogVerbosity::Warning, FText::FromString(Message));
#endif // WITH_EDITOR

		UE_LOGF(LogPCG, Warning, "%ls", *Message);
#endif
	};

	// Initialize a Crc onto which each input Crc will be combined (using random prime number).
	FPCGCrc Crc(1000033);

	// Random prime numbers to use as placeholders in the CRC computation when there are no defined in/out pins.
	// Note that they aren't strictly needed, but will make sure we don't introduce issues if we rework this bit of code.
	constexpr uint32 DefaultHashForNoInputPin = 955333;
	constexpr uint32 DefaultHashForNoOutputPin = 999983;

	// Hoisted out of loop for performance reasons.
	TArray<FPCGTaggedData, TInlineAllocator<16>> InputDataOnPin;
	TArray<FPCGCrc, TInlineAllocator<16>> InputDataCrcsOnPin;
	
	if(Task.Inputs.Num() > 0)
	{
		PCG::TSharedScopeLock ReadLock(TaskOutputsLock);
		for (const FPCGGraphTaskInput& Input : Task.Inputs)
		{
			const FOutputDataInfo& TaskOutput = TaskOutputs.FindChecked(Input.TaskId);

			ResultsToMarkAsRead.AddUnique(Input.TaskId);

			// Input was culled before this task managed to get scheduled (moved from ScheduleTasks to Tasks)
			if (TaskOutput.bCulled)
			{
				continue;
			}

			// If the input does not provide any data, don't add it to the task input.
			if (!Input.bProvideData || (Input.DownstreamPin.IsSet() && Input.DownstreamPin.GetValue().IsDatalessPin()))
			{
				continue;
			}

			const bool bAllowMultipleData = Input.DownstreamPin.IsSet() ? Input.DownstreamPin.GetValue().bAllowMultipleData : true;
			const uint32 InputPinLabelCrc = Input.DownstreamPin.IsSet() ? FCrc::StrCrc32(*Input.DownstreamPin.GetValue().Label.ToString()) : DefaultHashForNoOutputPin;

			// Enforce single data - if already have input for this pin, don't add more. Early check before other side effects below.
			if (Input.DownstreamPin.IsSet() && !bAllowMultipleData && Task.TaskInput.GetInputCountByPin(Input.DownstreamPin.GetValue().Label) > 0)
			{
				LogDiscardedData(Input.DownstreamPin);
				continue;
			}

			const FPCGDataCollection& InputCollection = TaskOutput.DataCollection;

			Task.TaskInput.bCancelExecution |= InputCollection.bCancelExecution;

			const int32 TaggedDataOffset = Task.TaskInput.TaggedData.Num();

			// Get input data at the given pin (or everything). This will add the data and include the input pin Crc to uniquely identify
			// inputs per-pin, or use a placeholder for symmetry.
			// Note: The input data CRC will already contain the output pin (calculated in element post execute).
			if (Input.UpstreamPin.IsSet())
			{
				InputDataOnPin.Reset();
				InputDataCrcsOnPin.Reset();
				InputCollection.GetInputsAndCrcsByPin(Input.UpstreamPin.GetValue().Label, InputDataOnPin, InputDataCrcsOnPin);

				if (!InputDataOnPin.IsEmpty())
				{
					// Proceed carefully when adding data items - if pin is single-data, only add first item.
					check(InputDataCrcsOnPin.IsEmpty() || InputDataOnPin.Num() == InputDataCrcsOnPin.Num());

					const int NumberDataItemsToTake = bAllowMultipleData ? InputDataOnPin.Num() : 1;

					Task.TaskInput.AddDataForPin(
						MakeArrayView(InputDataOnPin.GetData(), NumberDataItemsToTake),
						MakeArrayView(InputDataCrcsOnPin.GetData(), FMath::Min(NumberDataItemsToTake, InputDataCrcsOnPin.Num())),
						InputPinLabelCrc);

					if (NumberDataItemsToTake < InputDataOnPin.Num())
					{
						LogDiscardedData(Input.DownstreamPin);
					}
				}
			}
			else
			{
				Task.TaskInput.AddData(InputCollection.TaggedData, InputCollection.DataCrcs);
			}

			// Apply labelling on data; technically, we should ensure that we do this only for pass-through nodes,
			// Otherwise we could also null out the label on the input...
			// Also marking the data used multiple times if the given input is used multiple times.
			if (Input.DownstreamPin.IsSet() || Input.bIsUsedMultipleTimes)
			{
				for (int32 TaggedDataIndex = TaggedDataOffset; TaggedDataIndex < Task.TaskInput.TaggedData.Num(); ++TaggedDataIndex)
				{
					FPCGTaggedData& TaggedData = Task.TaskInput.TaggedData[TaggedDataIndex];
					if (Input.DownstreamPin.IsSet())
					{
						TaggedData.Pin = Input.DownstreamPin.GetValue().Label;
					}

					TaggedData.bIsUsedMultipleTimes |= Input.bIsUsedMultipleTimes;
				}
			}
		}
	}

	// Then combine params if needed
	CombineParams(Task);
}

void FPCGGraphExecutor::CombineParams(FPCGGraphTask& Task)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::CombineParams);
	
	TArray<FPCGTaggedData> AllParamsData = Task.TaskInput.GetParamsByPin(PCGPinConstants::DefaultParamsLabel);
	if (AllParamsData.Num() > 1)
	{
		UPCGParamData* CombinedParamData = nullptr;
		bool bSuccess = true;

		for (const FPCGTaggedData& TaggedDatum : AllParamsData)
		{
			if (const UPCGParamData* ParamData = Cast<UPCGParamData>(TaggedDatum.Data))
			{
				if (!CombinedParamData)
				{
					// We always need a Context to allow capture of Async objects if this runs outside of the game thread
					FPCGContext TempContext;
					TempContext.AsyncState.bIsRunningOnMainThread = IsInGameThread();
				
					CombinedParamData = ParamData->DuplicateData(&TempContext);

					if (!TempContext.AsyncState.bIsRunningOnMainThread)
					{
						Task.CombineParamsAsyncObjects = std::move(TempContext.AsyncObjects);
					}
				}
				else
				{
					PCGMetadataHelpers::FPCGCopyAllAttributesParams Params
					{
						.SourceData = ParamData,
						.TargetData = CombinedParamData
					};
					
					bSuccess &= PCGMetadataHelpers::CopyAllAttributes(Params);
				}
			}
		}

		if (!bSuccess)
		{
			return;
		}

		const int32 NewNumberOfInputs = Task.TaskInput.TaggedData.Num() - AllParamsData.Num() + (CombinedParamData ? 1 : 0);
		check(NewNumberOfInputs >= 0);
		
		TArray<FPCGTaggedData> TempTaggedData{};
		TArray<FPCGCrc> TempDataCrcs{};

		TempTaggedData.Reserve(NewNumberOfInputs);

		bool bForwardCrcs = true;

		if (!ensure(Task.TaskInput.TaggedData.Num() == Task.TaskInput.DataCrcs.Num()))
		{
			bForwardCrcs = false;
			TempDataCrcs.SetNumZeroed(NewNumberOfInputs);
		}
		else
		{
			TempDataCrcs.Reserve(NewNumberOfInputs);
		}

		for (int32 i = 0; i < Task.TaskInput.TaggedData.Num(); ++i)
		{
			FPCGTaggedData& TaggedData = Task.TaskInput.TaggedData[i];

			if (TaggedData.Pin != PCGPinConstants::DefaultParamsLabel)
			{
				TempTaggedData.Add(MoveTemp(TaggedData));

				if (bForwardCrcs)
				{
					TempDataCrcs.Add(MoveTemp(Task.TaskInput.DataCrcs[i]));
				}
			}
		}

		if (CombinedParamData)
		{
			FPCGTaggedData CombineParams{};
			CombineParams.Data = CombinedParamData;
			CombineParams.Pin = PCGPinConstants::DefaultParamsLabel;
			TempTaggedData.Add(CombineParams);
			if (bForwardCrcs)
			{
				TempDataCrcs.Add(CombineParams.ComputeCrc(/*bFullDataCrc=*/true));
			}
		}

		Task.TaskInput.TaggedData = MoveTemp(TempTaggedData);
		Task.TaskInput.DataCrcs = MoveTemp(TempDataCrcs);
	}
}

void FPCGGraphExecutor::StoreResults(TConstArrayView<FTaskResult> InResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::StoreResults);

	TArray<TPair<FPCGTaskId, FOutputDataInfo>, TInlineAllocator<16>> NewOutputs;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::StoreResults::Successors);
		PCG::TScopeLock ScopeLock(TasksLock);
		for (const FTaskResult& Result : InResults)
		{
			FOutputDataInfo OutputDataInfo;
			check(Result.TaskOutput);
			OutputDataInfo.DataCollection = *Result.TaskOutput;
			OutputDataInfo.bNeedsManualClear = Result.bNeedsManualClear;
			OutputDataInfo.DataCollection.MarkUsage(EPCGDataUsage::GraphExecutorTaskOutput);

			if (TSet<FPCGTaskId>* Successors = TaskSuccessors.Find(Result.TaskId))
			{
				OutputDataInfo.RemainingSuccessorCount = Successors->Num();
			}

			NewOutputs.Add({ Result.TaskId, MoveTemp(OutputDataInfo) });
		}
	}

	// Store output in map
	PCG::TUniqueScopeLock WriteLock(TaskOutputsLock);

	for (auto& Pair : NewOutputs)
	{
		TaskOutputs.Add(Pair.Key, MoveTemp(Pair.Value));
	}
}

void FPCGGraphExecutor::ClearResults()
{
#if WITH_EDITOR
	if (bDisableClearResults)
	{
		return;
	}
#endif

	{
		// Only reset if we have no more scheduled tasks, to avoid breaking dependencies
		if (ScheduledTasks.IsEmpty())
		{
			NextTaskId = 0;

			PerExecutionCache.Clear();
		}
	}

	PCG::TUniqueScopeLock WriteLock(TaskOutputsLock);

	for (const TPair<FPCGTaskId, FOutputDataInfo>& TaskOutput : TaskOutputs)
	{
		TaskOutput.Value.DataCollection.ClearUsage(EPCGDataUsage::GraphExecutorTaskOutput);
	}

	TaskOutputs.Reset();
}

void FPCGGraphExecutor::GetPinIdsToDeactivate(FPCGTaskId TaskId, uint64 InactiveOutputPinBitmask, TArray<FPCGPinId>& InOutPinIds)
{
	InOutPinIds.Reserve(InOutPinIds.Num() + FMath::CountBits64(InactiveOutputPinBitmask));

	int OutputPinIndex = 0;

	while (InactiveOutputPinBitmask != 0)
	{
		if (InactiveOutputPinBitmask & 1)
		{
			InOutPinIds.AddUnique(PCGPinIdHelpers::NodeIdAndPinIndexToPinId(TaskId, OutputPinIndex));
		}

		InactiveOutputPinBitmask >>= 1;
		++OutputPinIndex;
	}
}

void FPCGGraphExecutor::CullInactiveDownstreamNodes(FPCGTaskId InCompletedTaskId, uint64 InInactiveOutputPinBitmask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::CullInactiveDownstreamNodes);

	TArray<FPCGPinId> PinIdsToDeactivate;
	GetPinIdsToDeactivate(InCompletedTaskId, InInactiveOutputPinBitmask, PinIdsToDeactivate);
	check(!PinIdsToDeactivate.IsEmpty());

	PCGGraphExecutionLogging::LogTaskCullingBegin(InCompletedTaskId, InInactiveOutputPinBitmask, PinIdsToDeactivate);

	TSet<FPCGTaskId> AllRemovedTasks;

	// Hoisted out of loop for performance reasons.
	FTaskIds TasksToRemove;

	if (PinIdsToDeactivate.Num() > 0)
	{
		PCG::TScopeLock ScopeLock(TasksLock);
		while (!PinIdsToDeactivate.IsEmpty())
		{
			const FPCGPinId PinId = PinIdsToDeactivate.Pop(EAllowShrinking::No);
			const FPCGTaskId PinTaskId = PCGPinIdHelpers::GetNodeIdFromPinId(PinId);

			PCGGraphExecutionLogging::LogTaskCullingBeginLoop(PinTaskId, PCGPinIdHelpers::GetPinIndexFromPinId(PinId), PinIdsToDeactivate);
			LogTaskStateNoLock();
						
			const TSet<FPCGTaskId>* Successors = TaskSuccessors.Find(PinTaskId);
			if (!Successors)
			{
				continue;
			}

			TasksToRemove.SetNum(0, EAllowShrinking::No);

			// Build set of tasks that are candidates for culling when PinId is deactivated.
			for (const FPCGTaskId SuccessorTaskId : *Successors)
			{
				// Successors are updated at the end of this function, which means it may
				// contain task IDs that have been removed.
				if (FPCGGraphTask* FoundTask = Tasks.Find(SuccessorTaskId))
				{
					bool bDependencyExpressionBecameFalse;
					FoundTask->PinDependency.DeactivatePin(PinId, bDependencyExpressionBecameFalse);

					if (bDependencyExpressionBecameFalse)
					{
						TasksToRemove.AddUnique(SuccessorTaskId);
					}

					PCGGraphExecutionLogging::LogTaskCullingUpdatedPinDeps(SuccessorTaskId, FoundTask->PinDependency, bDependencyExpressionBecameFalse);
				}
			}

			// Now remove the tasks.
			for (const FPCGTaskId RemovedTaskId : TasksToRemove)
			{
				// Scope in which RemovedTask reference is valid.
				{
					FPCGGraphTask& RemovedTask = Tasks[RemovedTaskId];

					const UPCGNode* Node = RemovedTask.Node;
					const int PinCount = Node ? Node->GetOutputPins().Num() : 0;

					// Dynamically culled grid linkage tasks must do special processing to prevent downstream tasks from never finishing execution
					if (RemovedTask.Element && RemovedTask.Element->IsGridLinkage())
					{
						static_cast<PCGGraphExecutor::FPCGGridLinkageElement*>(RemovedTask.Element.Get())->OnUpstreamLinkCulled(RemovedTask.ExecutionSource.Get());
					}

					if (PinCount > 0)
					{
						// Deactivate all output pins.
						const uint64 InactiveOutputPinBitmask = (1ULL << PinCount) - 1;

						// Deactivate its pins - add to set of pins to deactivate.
						GetPinIdsToDeactivate(RemovedTaskId, InactiveOutputPinBitmask, PinIdsToDeactivate);

#if WITH_EDITOR
						SendInactivePinNotification(RemovedTask.Node, RemovedTask.GetStack(), InactiveOutputPinBitmask);
#endif
					}

					// Also register a special pin-less pin ID for this node, for task dependencies that do not have a specific pin.
					PinIdsToDeactivate.AddUnique(PCGPinIdHelpers::NodeIdToPinId(RemovedTaskId));

					// Remove task as successor of upstream node.
					RemoveTaskFromInputSuccessorsNoLock(RemovedTaskId, RemovedTask.Inputs);

					// Remove the deleted tasks from the inputs of downstream tasks.
					if (const TSet<FPCGTaskId>* SuccessorsOfRemovedTask = TaskSuccessors.Find(RemovedTaskId))
					{
						for (const FPCGTaskId SuccessorTaskId : *SuccessorsOfRemovedTask)
						{
							if (FPCGGraphTask* SuccessorTask = Tasks.Find(SuccessorTaskId))
							{
								for (int InputIndex = SuccessorTask->Inputs.Num() - 1; InputIndex >= 0; --InputIndex)
								{
									if (SuccessorTask->Inputs[InputIndex].TaskId == RemovedTaskId)
									{
										SuccessorTask->Inputs.RemoveAtSwap(InputIndex, EAllowShrinking::No);
									}
								}
							}
						}
					}
				}

				// Remove from tasks. After this step all traces of RemovedTaskId should be erased from tasks, task inputs. Task successors will be
				// updated below when queuing next tasks.
				Tasks.Remove(RemovedTaskId);
			}

			AllRemovedTasks.Append(TasksToRemove);
		}
	}

	if (AllRemovedTasks.Num() > 0)
	{
		{
			// Store culled output
			PCG::TUniqueScopeLock WriteLock(TaskOutputsLock);
			for (const FPCGTaskId TaskId : AllRemovedTasks)
			{
				FOutputDataInfo CullDataInfo;
				CullDataInfo.RemainingSuccessorCount = 0;
				CullDataInfo.bCulled = true;
				TaskOutputs.Add(TaskId, MoveTemp(CullDataInfo));
			}
		}

		// Queue downstream tasks in a similar manner to when a task draws from the cache and is skipped.
		// Some downstream tasks will have been culled which we don't care about (hence the ignore flag),
		// but some may not be queued and may be ready for queuing.
		QueueNextTasks(AllRemovedTasks.Array());
	}
}

#if WITH_EDITOR
void FPCGGraphExecutor::SendInactivePinNotification(const UPCGNode* InNode, const FPCGStack* InStack, uint64 InactiveOutputPinBitmask)
{
	const IPCGGraphExecutionSource* ExecutionSource = InStack ? InStack->GetRootSource() : nullptr;
	if (ExecutionSource && InNode)
	{
		ExecutionSource->GetExecutionState().GetInspection().NotifyNodeDynamicInactivePins(InNode, InStack, InactiveOutputPinBitmask);
	}
}
#endif

void FPCGGraphExecutor::AddReleasedContextForGC(FPCGContext* InContext)
{
	PCG::TScopeLock ScopeLock(ReleasedContextLock);
	ReleasedContextsForGC.Add(InContext);
}

void FPCGGraphExecutor::RemoveReleasedContextForGC(FPCGContext* InContext)
{
	PCG::TScopeLock ScopeLock(ReleasedContextLock);
	ensure(ReleasedContextsForGC.Remove(InContext) > 0);
}

void FPCGGraphExecutor::AddReferencedObjects(FReferenceCollector& Collector)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGraphExecutor::AddReferencedObjects);

	// Go through all data in the cached output map
	{
		PCG::TSharedScopeLock ReadLock(TaskOutputsLock);
		for (auto& OutputDataEntry : TaskOutputs)
		{
			OutputDataEntry.Value.DataCollection.AddReferences(Collector);
		}
	}

	{
		PCG::TScopeLock ScopeLock(LiveTasksLock);
		Algo::ForEach(ReadyTasks, [&Collector](auto& Task)
		{
			if (!Task.bHasDonePrepareForExecute)
			{
				check(!Task.Context);
				Task.TaskInput.AddReferences(Collector);
			}
			else if (Task.Context)
			{
				Task.Context->AddStructReferencedObjects(Collector);
			}
		});

		auto AddReferencesActiveTask = [&Collector](TSharedPtr<FPCGGraphActiveTask>& InActiveTask)
		{
			if (InActiveTask->bIsExecutingTask)
			{
				Collector.AddReferencedObjects(InActiveTask->ExecutingReferences);
				// @todo_pcg this is to allow referencing extra objects from the Context sub classes, should probably be part of the visiting pattern also
				if (InActiveTask->Context)
				{
					InActiveTask->Context->AddExtraStructReferencedObjects(Collector);
				}
			}
			else if (InActiveTask->Context)
			{
				InActiveTask->Context->AddStructReferencedObjects(Collector);
			}
		};

		Algo::ForEach(ActiveTasks, AddReferencesActiveTask);
		Algo::ForEach(ActiveTasksGameThreadOnly, AddReferencesActiveTask);
		Algo::ForEach(PausedTasks, AddReferencesActiveTask);
	}

	{
		PCG::TScopeLock ScopeLock(CachingResultsLock);
		Algo::ForEach(CachingResultsForGC, [&Collector](auto& CachingResult)
		{
			CachingResult.Value->Output.AddReferences(Collector);
		});
	}

	// Some task might have been delayed to the game thread and we are no longer referencing them through (Ready/Paused/Active)
	// but we still want their UObject references to stay valid
	{
		PCG::TScopeLock ScopeLock(CollectGCReferenceTasksLock);
		Algo::ForEach(CollectGCReferenceTasks, [&Collector](auto& Task)
		{
			if (Task->Context)
			{
				Task->Context->AddStructReferencedObjects(Collector);
			}
		});
	}

	PerExecutionCache.AddStructReferencedObjects(Collector);

	{
		PCG::TScopeLock ScopeLock(ReleasedContextLock);
		Algo::ForEach(ReleasedContextsForGC, [&Collector](FPCGContext* InContext)
		{
			if(ensure(InContext))
			{
				InContext->AddStructReferencedObjects(Collector);
			}
		});
	}

	GraphCompiler.AddReferencedObjects(Collector);
}

FPCGElementPtr FPCGGraphExecutor::GetFetchInputElement()
{
	PCG::TScopeLock Lock(DefaultElementsLock);
	
	if (!FetchInputElementPtr)
	{
		FetchInputElementPtr = MakeShared<FPCGFetchInputElement>();
	}

	return FetchInputElementPtr;
}

FPCGElementPtr FPCGGraphExecutor::GetPreGraphElement()
{
	PCG::TScopeLock Lock(DefaultElementsLock);
	
	if (!PreGraphElementPtr)
	{
		PreGraphElementPtr = MakeShared<FPCGPreGraphElement>();
	}

	return PreGraphElementPtr;
}

void FPCGGraphExecutor::LogTaskStateNoLock() const
{
#if WITH_EDITOR
	if (PCGGraphExecutionLogging::CullingLogEnabled())
	{
		UE_LOGF(LogPCG, Log, "\tDORMANT (FPCGGraphExecutor::Tasks):");
		PCGGraphExecutionLogging::LogGraphTasks(Tasks, &TaskSuccessors);
	}
#endif
}

#if WITH_EDITOR

FPCGTaskId FPCGGraphExecutor::ScheduleDebugWithTaskCallback(UPCGComponent* InComponent, TFunction<void(FPCGTaskId/* TaskId*/, const UPCGNode*/* Node*/, const FPCGDataCollection&/* TaskOutput*/)> TaskCompleteCallback)
{
	ensure(bDebugKeepIntermediateResults); // If this triggers, call SetKeepIntermediateResults first, as otherwise there are no guarantees the results will be kept, and this might trigger other ensures.

	check(InComponent);
	FPCGTaskId FinalTaskID = Schedule(InComponent, {});
	InComponent->CurrentGenerationTask = FinalTaskID;

	const bool bNonPartitionedComponent = !InComponent->IsLocalComponent() && !InComponent->IsPartitioned();
	const uint32 GenerationGridSize = bNonPartitionedComponent ? PCGHiGenGrid::UninitializedGridSize() : InComponent->GetGenerationGridSize();

	FPCGStackContext DummyStackContext;
	TArray<FPCGGraphTask> CompiledTasks = GraphCompiler.GetCompiledTasks(InComponent->GetGraph(), GenerationGridSize, DummyStackContext, /*bIsTopGraph=*/true);
	CompiledTasks.Pop(); // Remove the final task

	// Set up all final dependencies for the entire execution
	TArray<FPCGTaskId> FinalDependencies;
	FinalDependencies.Reserve(CompiledTasks.Num() + 1);
	FinalDependencies.Add(FinalTaskID);

	for (const FPCGGraphTask& CompiledTask : CompiledTasks)
	{
		// Schedule the output capture hooks
		FPCGTaskId CaptureTaskId = ScheduleGeneric([this, TaskCompleteCallback, CompiledTask]
		{
			// If the task was dynamically culled it produced no output — skip the callback.
			{
				PCG::TSharedScopeLock ReadLock(TaskOutputsLock);
				const FOutputDataInfo* OutputDataInfo = TaskOutputs.Find(CompiledTask.NodeId);
				if (OutputDataInfo && OutputDataInfo->bCulled)
				{
					return true;
				}
			}

			FPCGDataCollection TaskOutputData;
			if (CompiledTask.Node && GetOutputData(CompiledTask.NodeId, TaskOutputData))
			{
				TaskCompleteCallback(CompiledTask.NodeId, CompiledTask.Node, TaskOutputData);
			}

			return true;
		}, InComponent, {CompiledTask.NodeId});

		// Add these tasks to the final dependencies
		FinalDependencies.Add(CaptureTaskId);
	}

	// Finally, add a task to wait on the graph itself plus the capture tasks
	return ScheduleGeneric([] { return true; }, InComponent, FinalDependencies);
}

void FPCGGraphExecutor::NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType)
{
	GraphCompiler.NotifyGraphChanged(InGraph, ChangeType);
}

void FPCGGraphExecutor::UpdateGenerationNotification()
{
	// Avoid notifications for commandlets
	if (IsRunningCommandlet())
	{
		return;
	}

	// Avoid notifications for non world / non simulating game world executors unless they are standalone executors
	if (!bIsStandaloneGraphExecutor && (!World || (World->IsGameWorld() && GEditor && !GEditor->IsSimulateInEditorInProgress())))
	{
		return;
	}

	const int32 RemainingTaskNum = GetNonScheduledRemainingTaskCount();
	const bool bPaused = PCGSystemSwitches::CVarPausePCGExecution.GetValueOnAnyThread();
	if (RemainingTaskNum == 0 || bPaused)
	{
		// If we had tasks on the last frame, start a timer to avoid releasing the existing notification too soon
		if (GenerationProgressLastTaskNum != 0)
		{
			// Start Release timer
			GenerationProgressNotificationStartTime = FPlatformTime::Seconds();
		}

		if (bPaused || (FPlatformTime::Seconds() - GenerationProgressNotificationStartTime) > PCGGraphExecutor::CVarEditorNotificationDelayInSeconds.GetValueOnAnyThread())
		{
			ReleaseGenerationNotification();
		}
		else if(GenerationProgressNotification.IsValid())
		{
			GenerationProgressNotification.Pin()->Update(0);
		}

		GenerationProgressLastTaskNum = 0;
		return;
	}

	// Start timer for notification, if any non-runtime-gen component is generating.
	if (!GenerationProgressNotification.IsValid() && bAnyNonRuntimeGenComponentScheduled)
	{
		// If we didn't have tasks on the prior frame, start a timer to create the new one to cause a delay and avoid spamming on and off for short tasks
		if (GenerationProgressLastTaskNum == 0)
		{
			// Start Create timer
			GenerationProgressNotificationStartTime = FPlatformTime::Seconds();
		}

		if ((FPlatformTime::Seconds() - GenerationProgressNotificationStartTime) > PCGGraphExecutor::CVarEditorNotificationDelayInSeconds.GetValueOnAnyThread())
		{
			GenerationProgressNotificationStartTime = 0.0;

			IPCGEditorModule* EditorModule = IPCGEditorModule::Get();
			if (!EditorModule)
			{
				return;
			}

			GenerationProgressNotification = EditorModule->CreateProgressNotification(GetNotificationTextFormat(), /*bCanCancel=*/true);
			if (GenerationProgressNotification.IsValid())
			{
				GenerationProgressNotification.Pin()->OnCancelTasks().AddRaw(this, &FPCGGraphExecutor::OnNotificationCancel);
			}
		}
	}

	if (GenerationProgressNotification.IsValid())
	{
		GenerationProgressNotification.Pin()->Update(RemainingTaskNum);
	}

	GenerationProgressLastTaskNum = RemainingTaskNum;
}

void FPCGGraphExecutor::ReleaseGenerationNotification()
{
	if (GenerationProgressNotification.IsValid())
	{
		GenerationProgressNotification.Pin()->OnCancelTasks().RemoveAll(this);
		if (IPCGEditorModule* EditorModule = IPCGEditorModule::Get())
		{
			EditorModule->ReleaseProgressNotification(GenerationProgressNotification);
		}
		GenerationProgressNotification = nullptr;
	}
	GenerationProgressNotificationStartTime = 0.0;
}

void FPCGGraphExecutor::OnNotificationCancel()
{
	ReleaseGenerationNotification();

	if (IPCGBaseSubsystem* PCGSubsystem = GetSubsystem())
	{
		PCGSubsystem->CancelAllGeneration();
	}
}

FTextFormat FPCGGraphExecutor::GetNotificationTextFormat()
{
	return NSLOCTEXT("PCG", "PCGGenerationNotificationFormat", "Executing PCG {0}|plural(one=Task,other=Tasks)");
}

#endif // WITH_EDITOR

bool FPCGPreGraphElement::ExecuteInternal(FPCGContext* Context) const
{
	check(Context);

	check(IsInGameThread());
	const bool bIsValid = Context->ExecutionSource.IsValid();
	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();

	// Early out if the execution source has been deleted/is invalid
	if (!ExecutionSource)
	{
		// If the execution source should exist but it doesn't (which is all the time here, previously we checked for it), then this should be cancelled
		Context->OutputData.bCancelExecution = !bIsValid;
		return true;
	}

	{
		PCG_EXECUTION_CACHE_VALIDATION_CREATE_SCOPE(ExecutionSource);
		ExecutionSource->GetExecutionState().ExecutePreGraph(Context);
	}
	
	// Also emplace in the output a copy of the user parameters. Check in priority: Graph override, graph instance on the execution source and graph on the execution source.
	const UPCGGraphInterface* Graph = GraphOverride.Get();

	if (!Graph)
	{
		Graph = ExecutionSource->GetExecutionState().GetGraphInstance();
	}
	
	if (!Graph)
	{
		Graph = ExecutionSource->GetExecutionState().GetGraph();
	}
	
	if (const UPCGUserParametersData* UserParametersData = Graph ? Graph->ConstructUserParametersData(Context) : nullptr)
	{
		FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = UserParametersData;
		TaggedData.Tags.Add(PCG::Private::UserParameterTagData);
		// Mark this data pinless, since it is internal data, not meant to be shown in the graph editor.
		TaggedData.bPinlessData = true;
	}

	return true;
}

bool FPCGFetchInputElement::ExecuteInternal(FPCGContext* Context) const
{
	check(Context);
	// First: any input can be passed through to the output trivially
	Context->OutputData = Context->InputData;

	// Second: fetch the inputs provided by the component
	const bool bIsValid = Context->ExecutionSource.IsValid();
	UPCGComponent* Component = Cast<UPCGComponent>(Context->ExecutionSource.Get());
	
	// Early out if the component has been deleted/is invalid
	if (!Component)
	{
		// If the component should exist but it doesn't (which is all the time here, previously we checked for it), then this should be cancelled
		Context->OutputData.bCancelExecution = !bIsValid;
		return true;
	}

#if WITH_EDITOR
	check(Component->IsGenerationInProgress());
#endif

	check(Context->Node);

	// Skip retrieval if the graph is marked as not providing default inputs
	if (const UPCGGraph* Graph = Context->Node->GetGraph())
	{
		if (!Graph->HasDefaultConstructedInputs())
		{
			return true;
		}
	}

	if (Context->Node->IsOutputPinConnected(PCGPinConstants::DefaultInputLabel))
	{
		if (UPCGData* PCGData = Component->GetPCGData())
		{
			FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
			TaggedData.Data = PCGData;
			TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		}
	}

	if (Context->Node->IsOutputPinConnected(PCGInputOutputConstants::DefaultInputLabel))
	{
		if (UPCGData* InputPCGData = Component->GetInputPCGData())
		{
			FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
			TaggedData.Data = InputPCGData;
			TaggedData.Pin = PCGInputOutputConstants::DefaultInputLabel;
		}
	}

	if (Context->Node->IsOutputPinConnected(PCGInputOutputConstants::DefaultActorLabel))
	{
		if (UPCGData* ActorPCGData = Component->GetActorPCGData())
		{
			FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
			TaggedData.Data = ActorPCGData;
			TaggedData.Pin = PCGInputOutputConstants::DefaultActorLabel;
		}
	}

	if (Context->Node->IsOutputPinConnected(PCGInputOutputConstants::DefaultLandscapeLabel))
	{
		if (UPCGData* LandscapePCGData = Component->GetLandscapePCGData())
		{
			FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
			TaggedData.Data = LandscapePCGData;
			TaggedData.Pin = PCGInputOutputConstants::DefaultLandscapeLabel;
		}
	}

	if (Context->Node->IsOutputPinConnected(PCGInputOutputConstants::DefaultLandscapeHeightLabel))
	{
		if (UPCGData* LandscapeHeightPCGData = Component->GetLandscapeHeightPCGData())
		{
			FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
			TaggedData.Data = LandscapeHeightPCGData;
			TaggedData.Pin = PCGInputOutputConstants::DefaultLandscapeHeightLabel;
		}
	}

	if (Context->Node->IsOutputPinConnected(PCGInputOutputConstants::DefaultOriginalActorLabel))
	{
		if (UPCGData* OriginalActorPCGData = Component->GetOriginalActorPCGData())
		{
			FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
			TaggedData.Data = OriginalActorPCGData;
			TaggedData.Pin = PCGInputOutputConstants::DefaultOriginalActorLabel;
		}
	}

	return true;
}

FPCGPreGraphElement::FPCGPreGraphElement(const UPCGGraphInterface* InGraphOverride)
	: GraphOverride(InGraphOverride)
{
}

FPCGGenericElement::FPCGGenericElement(TFunction<bool(FPCGContext*)> InOperation, const FContextAllocator& InContextAllocator)
	: Operation(InOperation)
	, ContextAllocator(InContextAllocator)
{
}

FPCGGenericElement::FPCGGenericElement(TFunction<bool(FPCGContext*)> InOperation, TFunction<void(FPCGContext*)> InAbortOperation, const FContextAllocator& InContextAllocator)
	: Operation(InOperation)
	, AbortOperation(InAbortOperation)
	, ContextAllocator(InContextAllocator)
{
}

FPCGGenericElement::FPCGGenericElement(TFunction<bool(FPCGContext*)> InOperation, TFunction<void(FPCGContext*)> InAbortOperation, bool bInSupportBasePointDataInput, const FContextAllocator& InContextAllocator)
	: Operation(InOperation)
	, AbortOperation(InAbortOperation)
	, bSupportBasePointDataInput(bInSupportBasePointDataInput)
	, ContextAllocator(InContextAllocator)
{
}

FPCGGenericElement::FPCGGenericElement(TFunction<bool(FPCGContext*)> InOperation, TFunction<void(FPCGContext*)> InAbortOperation, bool bInSupportBasePointDataInput, bool bInCanExecuteOnlyOnMainThread, const FContextAllocator& InContextAllocator)
	: Operation(InOperation)
	, AbortOperation(InAbortOperation)
	, bSupportBasePointDataInput(bInSupportBasePointDataInput)
	, bCanExecuteOnlyOnMainThread(bInCanExecuteOnlyOnMainThread)
	, ContextAllocator(InContextAllocator)
{
}

FPCGContext* FPCGGenericElement::Initialize(const FPCGInitializeElementParams& InParams)
{
	check(ContextAllocator);
	FPCGContext* Context = ContextAllocator(InParams);
	Context->InitFromParams(InParams);

	return Context;
}

bool FPCGGenericElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGenericElement::Execute);
	return Operation && Operation(Context);
}

void FPCGGenericElement::AbortInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGenericElement::Abort);
	if(AbortOperation)
	{
		AbortOperation(Context);
	}
}

namespace PCGGraphExecutor
{
#if WITH_EDITOR
	bool FPCGGridLinkageElement::operator==(const FPCGGridLinkageElement& Other) const
	{
		return FromGrid == Other.FromGrid && ToGrid == Other.ToGrid && ResourceKey == Other.ResourceKey;
	}
#endif

	bool FPCGGridLinkageElement::SupportsGPUResidentData(FPCGContext* InContext) const
	{
		return PCGSystemSwitches::CVarPassGPUDataThroughGridLinks.GetValueOnAnyThread();
	}

	bool FPCGGridLinkageElement::ExecuteInternal(FPCGContext* InContext) const
	{
		check(InContext);
		FPCGGridLinkageContext* Context = static_cast<FPCGGridLinkageContext*>(InContext);

		return ExecuteGridLinkage(
			GenerationGrid,
			FromGrid,
			ToGrid,
			ResourceKey,
			UpstreamPinLabel,
			Context);
	}

	bool ExecuteGridLinkage(
		uint32 InGenerationGrid,
		uint32 InFromGrid,
		uint32 InToGrid,
		const FString& InResourceKey,
		FName InUpstreamPinLabel,
		FPCGGridLinkageContext* InContext)
	{
		check(InContext);
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGGraphExecutor::ExecuteGridLinkage);

		const IPCGGraphExecutionSource* ExecutionSource = InContext->ExecutionSource.Get();
		if (!ExecutionSource)
		{
			// Source no longer exists, nothing to be done.
			return true;
		}

		const IPCGGraphExecutionState& ExecutionState = InContext->ExecutionSource->GetExecutionState();

		// Non-hierarchical generation - no linkage required - data should just pass through.
		if (!ExecutionState.GetGraph()->IsHierarchicalGenerationEnabled()
			|| !ensure(PCGHiGenGrid::IsValidGridSize(InFromGrid) || InFromGrid == PCGHiGenGrid::UnboundedGridSize()))
		{
			InContext->OutputData = InContext->InputData;
			return true;
		}

		// Never allow a large grid to read data from small grid - this violates hierarchy.
		if (InFromGrid < InToGrid)
		{
#if WITH_EDITOR
			if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
			{
				// Using the low level logging call because we have only a node pointer for the downstream node. Note that InContext
				// is the context for the linkage element/task, which is not represented on the graph and cannot receive graph warnings/errors.
				if (InToGrid == PCGHiGenGrid::UnboundedGridSize())
				{
					PCGEditorModule->GetNodeVisualLogsMutable().Log(
						*InContext->GetStack(),
						ELogVerbosity::Error,
						FText::Format(
							NSLOCTEXT("PCGGraphCompiler", "InvalidLinkageToUnbounded", "Could not read data across grid levels - cannot read from grid size {0} to Unbounded domain."),
							InFromGrid,
							InToGrid));
				}
				else
				{
					PCGEditorModule->GetNodeVisualLogsMutable().Log(
						*InContext->GetStack(),
						ELogVerbosity::Error,
						FText::Format(
							NSLOCTEXT("PCGGraphCompiler", "InvalidLinkageInvalidGridSizes", "Could not read data across grid levels - origin grid size {0} must be greater than destination grid size {1}. Graph default grid size may need increasing."),
							InFromGrid,
							InToGrid));
				}
			}
#endif

			return true;
		}

		if (InFromGrid == InGenerationGrid && InFromGrid != InToGrid)
		{
			FPCGDataCollection Data;

			// Grid linkages have a single virtual input pin with the label matching the upstream pin. A grid linkage is created for each edge.
			Data.TaggedData = InContext->InputData.GetInputsByPin(InUpstreamPinLabel);

			PCGGraphExecutionLogging::LogGridLinkageTaskExecuteStore(InContext, InGenerationGrid, InFromGrid, InToGrid, InResourceKey, Data.TaggedData.Num());
			
			ExecutionState.StoreOutputDataForPin(InResourceKey, Data);
		}
		else if (InToGrid == InGenerationGrid && InFromGrid != InToGrid)
		{
			PCGGraphExecutionLogging::LogGridLinkageTaskExecuteRetrieve(InContext, InGenerationGrid, InFromGrid, InToGrid, InResourceKey);

			IPCGGraphExecutionSource* ExecSourceWithData = nullptr;

			if (InFromGrid == PCGHiGenGrid::UnboundedGridSize())
			{
				ExecSourceWithData = ExecutionState.GetOriginalSource();
			}
			else
			{
				const IPCGGraphExecutionSource* OriginalSource = ExecutionState.GetOriginalSource();
				if (!ensure(OriginalSource))
				{
					return false;
				}

				const FPCGGridDescriptor GridDescriptor = OriginalSource->GetExecutionState().GetGridDescriptor(InFromGrid);

				// Get grid coords using the parent grid (InFromGrid).
				const FIntVector CellCoords = UPCGActorHelpers::GetCellCoord(ExecutionState.GetTransform().GetLocation(), GridDescriptor.GetGridSize(), GridDescriptor.Is2DGrid());

				ExecSourceWithData = ExecutionState.GetLocalSource(GridDescriptor, CellCoords);
			}

			if (!ExecSourceWithData)
			{
				// Nothing we can do currently if execution source is not present. One idea is to schedule an artifact-less execution but that
				// comes with complications - artifacts/side effects are an integral part of execution. Most likely we'll do a cleanup
				// pass of any unwanted artifacts/local-execution sources later.
				PCGGraphExecutionLogging::LogGridLinkageTaskExecuteRetrieveNoLocalSource(InContext, InResourceKey);
				return true;
			}

			// Once we've found our execution source, try to retrieve the data.
			if (const FPCGDataCollection* Data = ExecSourceWithData->GetExecutionState().RetrieveOutputDataForPin(InResourceKey))
			{
				PCGGraphExecutionLogging::LogGridLinkageTaskExecuteRetrieveSuccess(InContext, ExecSourceWithData, InResourceKey, Data->TaggedData.Num());
				InContext->OutputData = *Data;

				// Grid Linkages have a single virtual output pin which forwards the same label as the upstream output pin. Apply the label now.
				// The normal path in IPCGElement::CleanupAndValidateOutput() to fix up the output label does not run because Grid Linkages
				// have no Settings.
				for (FPCGTaggedData& TaggedData : InContext->OutputData.TaggedData)
				{
					TaggedData.Pin = InUpstreamPinLabel;
				}

				return true;
			}

			// At this point we could not get to the data, so we'll try executing the graph if we did not do that already.
			// If we need data from a higher execution source but it's still generating, then we'll wait for it.
			if (ExecSourceWithData->GetExecutionState().IsGenerating())
			{
				PCGGraphExecutionLogging::LogGridLinkageTaskExecuteRetrieveWaitOnScheduledGraph(InContext, ExecSourceWithData, InResourceKey);

				// The execution source was already generating, but we were not paused. Not really clear what's happening here,
				// but in any case go to paused and unpause when it's done.
				InContext->bIsPaused = true;

				// Wake up this task after graph has generated.
				FPCGTaskId GenerationTask = ExecSourceWithData->GetExecutionState().GetGenerationTaskId();
				if (ensure(GenerationTask != InvalidPCGTaskId))
				{
					InContext->DynamicDependencies.Add(GenerationTask);
				}

				return false;
			}

			// Graph is not currently generating. If we have not already tried generating, try it once now.
			if (!InContext->bScheduledGraph)
			{
				PCGGraphExecutionLogging::LogGridLinkageTaskExecuteRetrieveScheduleGraph(InContext, ExecSourceWithData, InResourceKey);

				// Wake up this task after graph has generated.
				IPCGGraphExecutionState::FGenerateParams Params;
				Params.bEvenIfAlreadyGenerated = true;
				const FPCGTaskId GraphTaskId = ExecSourceWithData->GetExecutionState().Generate(Params);

				// Update state and go to paused.
				InContext->DynamicDependencies.Add(GraphTaskId);
				InContext->bScheduledGraph = true;
				InContext->bIsPaused = true;
				return false;
			}
			else
			{
				// We tried generating but no luck, at this point give up.
				PCGGraphExecutionLogging::LogGridLinkageTaskExecuteRetrieveNoData(InContext, ExecSourceWithData, InResourceKey);
				return true;
			}
		}
		else
		{
			// Graceful no op - no linkage required.
			InContext->OutputData = InContext->InputData;
		}

		return true;
	}

	void FPCGGridLinkageElement::OnUpstreamLinkCulled(IPCGGraphExecutionSource* InExecutionSource) const
	{
		if (!InExecutionSource)
		{
			return;
		}

		const IPCGGraphExecutionState& ExecutionState = InExecutionSource->GetExecutionState();

		// Non-hierarchical generation - no linkage required - data should just pass through.
		if (!ExecutionState.GetGraph()->IsHierarchicalGenerationEnabled()
			|| !ensure(PCGHiGenGrid::IsValidGridSize(FromGrid) || FromGrid == PCGHiGenGrid::UnboundedGridSize()))
		{
			return;
		}

		if (FromGrid == GenerationGrid && FromGrid != ToGrid)
		{
			FPCGDataCollection EmptyData;
			PCGGraphExecutionLogging::LogGridLinkageTaskStoreNullDataOnCulled(InExecutionSource, GenerationGrid, FromGrid, ToGrid, ResourceKey);

			ExecutionState.StoreOutputDataForPin(ResourceKey, EmptyData);
		}
	}
}

UPCGGridLinkageSettings::UPCGGridLinkageSettings()
{
#if WITH_EDITOR
	bExposeToLibrary = false;
#endif // WITH_EDITOR
}

FPCGElementPtr UPCGGridLinkageSettings::CreateElement() const
{
	const UPCGPin* ResolvedUpstreamPin = nullptr;
	{
		FGCScopeGuard Guard;
		ResolvedUpstreamPin = UpstreamPin.Get();
	}

	return MakeShared<PCGGraphExecutor::FPCGGridLinkageElement>(FromGrid, ToGrid, GenerationGrid, ResourceKey, ResolvedUpstreamPin);
}
