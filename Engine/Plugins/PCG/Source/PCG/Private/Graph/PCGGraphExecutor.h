// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "PCGSubsystem.h"
#include "Graph/PCGGraphCache.h"
#include "Graph/PCGGraphCompiler.h"
#include "Graph/PCGGraphPerExecutionCache.h"
#include "Graph/PCGGraphTask.h"
#include "Graph/PCGStackContext.h"

#include "Containers/MpscQueue.h"
#include "HAL/Runnable.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"
#include "UObject/GCObject.h"

#if WITH_EDITOR
#include "Editor/IPCGEditorProgressNotification.h"
#endif

#include "PCGGraphExecutor.generated.h"

class UPCGComponent;
class UPCGGraph;
class UPCGNode;
struct FPCGStack;
class FTextFormat;

namespace PCGGraphExecutor
{
	extern PCG_API TAutoConsoleVariable<float> CVarTimePerFrame;
	extern PCG_API TAutoConsoleVariable<bool> CVarGraphMultithreading;

#if WITH_EDITOR
	extern PCG_API TAutoConsoleVariable<float> CVarEditorTimePerFrame;
#endif

	void ClearAsyncFlags(TSet<TObjectPtr<UObject>>& AsyncObjects);
}

class FPCGGraphExecutor : public FGCObject, public TSharedFromThis<FPCGGraphExecutor>
{
public:
	// Default constructor used by unittests
	FPCGGraphExecutor();
	FPCGGraphExecutor(UWorld* InWorld);
	~FPCGGraphExecutor();

	/** Compile (and cache) a graph for later use. This call is threadsafe */
	void Compile(UPCGGraph* InGraph);

	/** Schedules the execution of a given graph with specified inputs. This call is threadsafe */
	FPCGTaskId Schedule(IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGTaskId>& TaskDependency = TArray<FPCGTaskId>(), const FPCGStack* InFromStack = nullptr);

	/** Schedules the execution of a given graph with specified inputs. This call is threadsafe */
	FPCGTaskId ScheduleGraph(const FPCGScheduleGraphParams& InParams);

	FPCGTaskId Schedule(
		UPCGGraph* Graph,
		IPCGGraphExecutionSource* InExecutionSource,
		FPCGElementPtr PreGraphElement,
		FPCGElementPtr InputElement,
		const TArray<FPCGTaskId>& TaskDependency,
		const FPCGStack* InFromStack,
		bool bAllowHierarchicalGeneration);

	/** Cancels all tasks originating from the given component */
	TArray<IPCGGraphExecutionSource*> Cancel(IPCGGraphExecutionSource* InExecutionSource);
	
	/** Cancels all tasks running a given graph */
	TArray<IPCGGraphExecutionSource*> Cancel(UPCGGraph* InGraph);

	/** Cancels all tasks */
	TArray<IPCGGraphExecutionSource*> CancelAll();

	/** Returns true if any task is scheduled or executing for the given graph. */
	bool IsGraphCurrentlyExecuting(const UPCGGraph* InGraph);

	/** Returns true if any task is scheduled or executing for any graph */
	bool IsAnyGraphCurrentlyExecuting() const;
	
	// General job scheduling
	FPCGTaskId ScheduleGeneric(const FPCGScheduleGenericParams& Params);

	// Back compatibility function. Use ScheduleGenericWithContext or ScheduleGeneric taking a FPCGScheduleGenericParams
	FPCGTaskId ScheduleGeneric(TFunction<bool()> InOperation, IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies);
	FPCGTaskId ScheduleGeneric(TFunction<bool()> InOperation, TFunction<void()> InAbortOperation, IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies);

	/** General job scheduling
	*  @param InOperation:                Callback that takes a Context as argument and returns true if the task is done, false otherwise
	*  @param InExecutionSource:          Execution source associated with this task. Can be null.
	*  @param TaskExecutionDependencies:  Task will wait on these tasks to execute and won't take their output data as input.
	*  @param TaskDataDependencies:       Task will wait on these tasks to execute and will take their output data as input.
	*  @param bSupportBasePointDataInput: When true, generic element will not convert input to UPCGPointData, this is false by default to preserver backward compatibility.
	* 
	*/
	FPCGTaskId ScheduleGenericWithContext(TFunction<bool(FPCGContext*)> InOperation, IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies, const TArray<FPCGTaskId>& TaskDataDependencies, bool bSupportBasePointDataInput = false);

	/** General job scheduling
	*  @param InOperation:                Callback that takes a Context as argument and returns true if the task is done, false otherwise
	*  @param InAbortOperation:           Callback that is called if the task is aborted (cancelled) before fully executed.
	*  @param InExecutionSource:          Execution source associated with this task. Can be null.
	*  @param TaskExecutionDependencies:  Task will wait on these tasks to execute and won't take their output data as input.
	*  @param TaskDataDependencies:       Task will wait on these tasks to execute and will take their output data as input.
	*  @param bSupportBasePointDataInput: When true, generic element will not convert input to UPCGPointData, this is false by default to preserver backward compatibility.
	*/
	FPCGTaskId ScheduleGenericWithContext(TFunction<bool(FPCGContext*)> InOperation, TFunction<void(FPCGContext*)> InAbortOperation, IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies, const TArray<FPCGTaskId>& TaskDataDependencies, bool bSupportBasePointDataInput = false);

	/** Gets data in the output results. Returns false if data is not ready. */
	bool GetOutputData(FPCGTaskId InTaskId, FPCGDataCollection& OutData);

	/** Clear output data */
	void ClearOutputData(FPCGTaskId InTaskId);

	/** Accessor so PCG tools (e.g. profiler) can easily decode graph task ids **/
	FPCGGraphCompiler* GetCompiler() { return &GraphCompiler; }

	/** Accessor so PCG tools (e.g. profiler) can easily decode graph task ids **/
	const FPCGGraphCompiler* GetCompiler() const { return &GraphCompiler; }

#if WITH_EDITOR
	/** Experimental & Internal - Visits all intermediate results for all tasks. Requires SetKeepIntermediateResults to be called before hand. */
	FPCGTaskId ScheduleDebugWithTaskCallback(UPCGComponent* InComponent, TFunction<void(FPCGTaskId, const UPCGNode*, const FPCGDataCollection&)> TaskCompleteCallback);

	/** Notifies the executor to keep all intermediate results. Is needed prior to the ScheduleDebugWithTaskCallback call. @todo_pcg: Replace this & SetDisableClearResults with an enum. */
	void SetKeepIntermediateResults(bool bInKeepIntermediateResults) { bDebugKeepIntermediateResults = bInKeepIntermediateResults; }

	/** Notify compiler that graph has changed so it'll be removed from the cache */
	void NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType);

	/** Returns the number of entries currently in the cache for InElement. */
	uint32 GetGraphCacheEntryCount(IPCGElement* InElement) const { return GraphCache.GetGraphCacheEntryCount(InElement); }
#endif

	/** "Tick" of the graph executor. This call is NOT THREADSAFE */
	void Execute();
	void Execute(double& InOutEndTime);

	/** Expose cache so it can be dirtied */
	FPCGGraphCache& GetCache() { return GraphCache; }

	/** True if graph cache debugging is enabled. */
	bool IsGraphCacheDebuggingEnabled() const { return GraphCache.IsDebuggingEnabled(); }

	//~Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FPCGGraphExecutor"); }
	//~End FGCObject interface

	void AddReleasedContextForGC(FPCGContext* InContext);
	void RemoveReleasedContextForGC(FPCGContext* InContext);

private:
	friend class UPCGSubsystem;
	friend class UPCGEngineSubsystem;
	friend class IPCGBaseSubsystem;

	IPCGBaseSubsystem* GetSubsystem() const;

	FPCGPerExecutionCache PerExecutionCache;

	double GetTickBudgetInSeconds() const;
	int32 GetMaxNumProcessingTasks() const;
	float GetMaxPercentageOfThreadsToUse() const;
	float GetMaxPercentageOfExecutingThreads() const;
	int32 GetMaxExecutingThreads() const;
	bool IsPostExecuteConsumerThreadEnabled() const;
	double GetPostExecuteConsumerThreadBudgetInSeconds() const;

	struct FCachedResult
	{
		FPCGTaskId TaskId = InvalidPCGTaskId;
		FPCGDataCollection Output;
		FPCGStackHandle Stack;
		const UPCGNode* Node = nullptr;
		bool bDoDynamicTaskCulling = false;
		bool bIsPostGraphTask = false;		
	};

	// Handler that we can use as a Weak ptr to determine if the Executor is still valid
	class FGameThreadHandler : public TSharedFromThis<FGameThreadHandler>
	{
	public:
		FGameThreadHandler(FPCGGraphExecutor* InExecutor)
			: Executor(InExecutor) { }

		FPCGGraphExecutor* GetExecutor() { return Executor; }

	private:
		FPCGGraphExecutor* Executor = nullptr;
	};

	FPCGTaskId ScheduleSingleElement(TSharedRef<IPCGElement> Element, bool bHasAbortCallbacks, IPCGGraphExecutionSource* InExecutionSource,
		const TArray<FPCGTaskId>& TaskExecutionDependencies, const TArray<FPCGTaskId>& TaskDataDependencies);

	void ProcessAsyncExecuteResult(TSharedPtr<FPCGGraphActiveTask> Task, bool bIsDone, bool bIsPaused, double EndTime);
	void StartPostExecuteConsumerThread();
	void PostTaskExecute(TSharedPtr<FPCGGraphActiveTask> ActiveTask, bool bIsDone);
	bool ProcessScheduledTasks(double EndTime);
	void ExecuteTasksEnded();
	bool ExecuteScheduling(double EndTime, TSharedPtr<FPCGGraphActiveTask>* OutMainThreadTask = nullptr, bool bForceCheckPausedTasks = false);
	void CheckState(bool bIsInGameThread);

	TSet<IPCGGraphExecutionSource*> Cancel(TFunctionRef<bool(TWeakInterfacePtr<IPCGGraphExecutionSource>)> CancelFilter);
	void ClearAllTasks();
	
	using FCachedResults = TArray<FCachedResult*, TInlineAllocator<16>>;
	using FTaskIds = TArray<FPCGTaskId, TInlineAllocator<32>>;
	
	void QueueNextTasks(TConstArrayView<FPCGTaskId> FinishedTasks);
	void QueueNextTasksInternal(TConstArrayView<FPCGTaskId> FinishedTasks, FCachedResults& OutCachedResults);

	bool CancelNextTasks(FPCGTaskId CancelledTask, TSet<IPCGGraphExecutionSource*>& OutCancelledExecutionSources);
	void RemoveTaskFromInputSuccessors(FPCGTaskId CancelledTask, const TArray<FPCGGraphTaskInput>& CancelledTaskInputs);
	void RemoveTaskFromInputSuccessorsNoLock(FPCGTaskId CancelledTask, const TArray<FPCGGraphTaskInput>& CancelledTaskInputs);
	
	/** Called from QueueNextTasks/ProcessScheduledTasks will try and setup/prepare task for execution */
	void OnTaskInputsReady(TArrayView<FPCGGraphTask> InNewReadyTasks, FCachedResults& OutCachedResults, bool bIsInGameThread);

	/** SetupTask will call BuildTaskInput and assign a IPCGElement to the FPCGGraphTask */
	bool SetupTask(FPCGGraphTask& Task, FTaskIds& ResultsToMarkAsRead);
	void BuildTaskInput(FPCGGraphTask& Task, FTaskIds& ResultsToMarkAsRead);
	/** Will check the cache for existing result or create and initialize the FPCGContext to the task */
	void PrepareForExecute(FPCGGraphTask& Task, FCachedResult*& OutCachedResult, bool bLiveTasksLockAlreadyLocked);

#if WITH_EDITOR
	static void DebugDisplayTask(TWeakPtr<FGameThreadHandler> GameThreadHandler, TSharedPtr<FPCGGraphActiveTask> ActiveTaskPtr);
#endif

	/** Store cache results and Queue next tasks */
	void ProcessCachedResults(TConstArrayView<FCachedResult*> CachedResults);
	void ProcessCachedResultsInternal(TConstArrayView<FCachedResult*> CachedResults, FTaskIds& OutNextTasks);

	/** Combine all param data on the Overrides pin into a single output, if needed.*/
	void CombineParams(FPCGGraphTask& Task);

	struct FTaskResult
	{
		FPCGTaskId TaskId;
		const FPCGDataCollection* TaskOutput = nullptr;
		bool bNeedsManualClear = false;
	};

	void StoreResults(TConstArrayView<FTaskResult> Results);
	void ClearResults();
	void MarkInputResults(TArrayView<const FPCGTaskId> InInputResults);

	/** If the completed task has one or more deactivated pins, delete any downstream tasks that are inactive as a result. */
	void CullInactiveDownstreamNodes(FPCGTaskId CompletedTaskId, uint64 InInactiveOutputPinBitmask);

	/** Builds an array of all deactivated unique pin IDs. */
	static void GetPinIdsToDeactivate(FPCGTaskId TaskId, uint64 InactiveOutputPinBitmask, TArray<FPCGPinId>& InOutPinIds);

	FPCGElementPtr GetFetchInputElement();
	FPCGElementPtr GetPreGraphElement();

	void LogTaskStateNoLock() const;

	int32 GetNonScheduledRemainingTaskCount() const;

#if WITH_EDITOR
	/** Notify the component that the given pins were deactivated during execution. */
	void SendInactivePinNotification(const UPCGNode* InNode, const FPCGStack* InStack, uint64 InactiveOutputPinBitmask);

	void UpdateGenerationNotification();
	void ReleaseGenerationNotification();
	void OnNotificationCancel();
	static FTextFormat GetNotificationTextFormat();

	void SetDisableClearResults(bool bInDisableClearResults) { bDisableClearResults = bInDisableClearResults; }
#endif

	/** Graph compiler that turns a graph into tasks */
	FPCGGraphCompiler GraphCompiler;

	/** Graph results cache */
	FPCGGraphCache GraphCache;

	/** Input fetch element, stored here so we have only one */
	FPCGElementPtr FetchInputElementPtr;

	/** PreGraph element, stored here so we have only one */
	FPCGElementPtr PreGraphElementPtr;

	/** 
	 * Define a Lock level for future reference. Rule is when we have a lock, we can't lock a lower or equal level lock to prevent deadlocks.
	 * Example: When locking LiveTasksLock we can't lock ScheduleLock or CollectGCReferenceTasksLock 
	 */

	/** Lock level - 1 (top most lock) */
	PCG::FLock ScheduleLock;
	TArray<FPCGGraphScheduleTask> ScheduledTasks;
	std::atomic<bool> SchedulingDisabled = false;

	/** Lock level 2 */
	mutable PCG::FLock TasksLock;
	TMap<FPCGTaskId, FPCGGraphTask> Tasks;
	TMap<FPCGTaskId, TSet<FPCGTaskId>> TaskSuccessors;

	/** Lock level 3 */
	PCG::FLock LiveTasksLock;
	TArray<FPCGGraphTask> ReadyTasks;
	TArray<TSharedPtr<FPCGGraphActiveTask>> ActiveTasks;
	TArray<TSharedPtr<FPCGGraphActiveTask>> ActiveTasksGameThreadOnly;
	TArray<TSharedPtr<FPCGGraphActiveTask>> PausedTasks;
	bool bNeedToCheckPausedTasks = false;

	/** Lock level leaf */
	PCG::FLock CollectGCReferenceTasksLock;
	TSet<TSharedPtr<FPCGGraphActiveTask>> CollectGCReferenceTasks;
			
	/** Map of node instances to their output, could be cleared once execution is done */
	/** Note: this should at some point unload based on loaded/unloaded proxies, otherwise memory cost will be unbounded */
	struct FOutputDataInfo
	{
		FPCGDataCollection DataCollection;
		// Controls whether the results will be expunged from the OutputData as soon as the successor count reaches 0 or not.
		bool bNeedsManualClear = false;
		// Successor count, updated after a successor is done executing (MarkInputResults).
		int32 RemainingSuccessorCount = 0;
		// Culled
		bool bCulled = false;
	};

	/** Lock level 4 */
	PCG::FLock PausedTaskSuccessorsLock;
	TMap<FPCGTaskId, TArray<TSharedPtr<FPCGGraphActiveTask>>> PausedTaskSuccessors;
	
	/** Lock level leaf */
	PCG::FLock CachingResultsLock;
	// Used to keep GC references to in flight caching results (not yet stored to output and might not be in cache anymore)
	TMap<FPCGTaskId, TUniquePtr<FCachedResult>> CachingResultsForGC;

	/** Lock level leaf */
	PCG::FLock ReleasedContextLock;
	TSet<FPCGContext*> ReleasedContextsForGC;

	/** Lock level leaf */
	PCG::FSharedLock TaskOutputsLock;
	TMap<FPCGTaskId, FOutputDataInfo> TaskOutputs;
	
	/** Monotonically increasing id. Should be reset once all tasks are executed, should be protected by the ScheduleLock */
	FPCGTaskId NextTaskId = 0;

	/** To protect element creation. */
	PCG::FLock DefaultElementsLock;

	std::atomic<bool> bNeedToExecuteTasksEnded = false;
	std::atomic<int32> NewReadyTaskCount = 0;

	struct FPostTaskExecuteItem
	{
		TSharedPtr<FPCGGraphActiveTask> Task;
		bool bIsDone = false;
		bool bIsPaused = false;
	};

	struct FPostTaskExecuteConsumerThread : public FRunnable
	{
		FPCGGraphExecutor* Owner = nullptr;

		FRunnableThread* Thread = nullptr;
		TMpscQueue<FPostTaskExecuteItem> Queue;
		FEventRef Event{ EEventMode::AutoReset };
		bool bQuit = false;

		FPostTaskExecuteConsumerThread(FPCGGraphExecutor* InOwner);
		virtual ~FPostTaskExecuteConsumerThread();

		virtual uint32 Run() override;
	};

	TUniquePtr<FPostTaskExecuteConsumerThread> PostTaskExecuteConsumerThread;

#if WITH_EDITOR
	TWeakPtr<IPCGEditorProgressNotification> GenerationProgressNotification;
	double GenerationProgressNotificationStartTime = 0.0;
	int32 GenerationProgressLastTaskNum = 0;
	bool bIsStandaloneGraphExecutor = false;

	/** Set true when a non-runtime-gen component is scheduled and reset to false when all tasks are done executing. */
	std::atomic<bool> bAnyNonRuntimeGenComponentScheduled = false;

	bool bDebugKeepIntermediateResults = false;
	bool bDisableClearResults = false;
#endif

	TObjectPtr<UWorld> World = nullptr;

	TSharedPtr<FGameThreadHandler> GameThreadHandler;

	double LastSchedulingErrorCheck = 0.0;
};

class FPCGFetchInputElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool IsPassthrough(const UPCGSettings* InSettings) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};

/**
 * Potential first task executed in a graph that will do some verifications to make sure the graph can be executed.
 * Will also execute some pre task logic provided by the ExecutionSource (ExecutePreGraph) and will freeze the user parameters
 * of the graph for them to be accessed in multi-thread through the graph execution.
 * Note that by default, the user parameters are pulled from the ExecutionSource graph, but in a case where we schedule a different graph
 * than the one on the ExecutionSource, the PreGraph element can be setup with a graph override.
 */
class FPCGPreGraphElement : public IPCGElement
{
public:
	FPCGPreGraphElement(const UPCGGraphInterface* InGraphOverride = nullptr);
	
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool IsPassthrough(const UPCGSettings* InSettings) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }

private:
	TWeakObjectPtr<const UPCGGraphInterface> GraphOverride;
};

class FPCGGenericElement : public IPCGElement
{
public:
	using FContextAllocator = TFunction<FPCGContext*(const FPCGInitializeElementParams&)>;

	FPCGGenericElement(
		TFunction<bool(FPCGContext*)> InOperation,
		const FContextAllocator& InContextAllocator = (FContextAllocator)[](const FPCGInitializeElementParams&)
	{
		return new FPCGContext();
	});

	FPCGGenericElement(
		TFunction<bool(FPCGContext*)> InOperation,
		TFunction<void(FPCGContext*)> InAbortOperation,
		const FContextAllocator& InContextAllocator = (FContextAllocator)[](const FPCGInitializeElementParams&)
	{
		return new FPCGContext();
	});

	FPCGGenericElement(
		TFunction<bool(FPCGContext*)> InOperation,
		TFunction<void(FPCGContext*)> InAbortOperation,
		bool bInSupportBasePointDataInput,
		const FContextAllocator& InContextAllocator = (FContextAllocator)[](const FPCGInitializeElementParams&)
	{
		return new FPCGContext();
	});

	FPCGGenericElement(
		TFunction<bool(FPCGContext*)> InOperation,
		TFunction<void(FPCGContext*)> InAbortOperation,
		bool bInSupportBasePointDataInput,
		bool bInCanExecuteOnlyOnMainThread,
		const FContextAllocator& InContextAllocator = (FContextAllocator)[](const FPCGInitializeElementParams&)
	{
		return new FPCGContext();
	});
	
	virtual FPCGContext* Initialize(const FPCGInitializeElementParams& InParams) override;

	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return bCanExecuteOnlyOnMainThread; }

protected:
	// Important note: generic elements must always be run on the main thread
	// as most of these will impact the editor in some way (loading, unloading, saving)
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual void AbortInternal(FPCGContext* Context) const override;
	virtual bool IsCancellable() const override { return false; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return bSupportBasePointDataInput; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }

#if WITH_EDITOR
	virtual bool ShouldLog() const override { return false; }
#endif

private:
	TFunction<bool(FPCGContext*)> Operation;
	TFunction<void(FPCGContext*)> AbortOperation;
	bool bSupportBasePointDataInput = false;
	bool bCanExecuteOnlyOnMainThread = true;

	/** Creates a context object for this element. */
	FContextAllocator ContextAllocator;
};

/** Context for linkage element which marshalls data across hierarchical generation grids. */
struct FPCGGridLinkageContext : public FPCGContext
{
	/** If we require data from a component that is not generated, we schedule it once to see if we can get the data later. */
	bool bScheduledGraph = false;
};

namespace PCGGraphExecutor
{
	/** Marshals data across grid sizes at execution time. */
	class FPCGGridLinkageElement : public IPCGElementWithCustomContext<FPCGGridLinkageContext>
	{
	public:
		FPCGGridLinkageElement(uint32 InFromGrid, uint32 InToGrid, uint32 InGenerationGrid, const FString& InResourceKey, const UPCGPin* InUpstreamPin)
			: FromGrid(InFromGrid)
			, ToGrid(InToGrid)
			, GenerationGrid(InGenerationGrid)
			, ResourceKey(InResourceKey)
		{
			if (IsValid(InUpstreamPin))
			{
				UpstreamPin = TWeakObjectPtr(InUpstreamPin);
				UpstreamPinLabel = InUpstreamPin->Properties.Label;
			}
		}

		//~Begin IPCGElement interface
	public:
		virtual bool IsGridLinkage() const override { return true; }
		virtual bool ExecuteInternal(FPCGContext* Context) const override;

	protected:
		virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override;
		virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
		//~End IPCGElement interface

	public:
		void OnUpstreamLinkCulled(IPCGGraphExecutionSource* InSource) const;

		const UPCGPin* GetUpstreamPin() const { return UpstreamPin.Get(); }
		uint32 GetFromGridSize() const { return FromGrid; }
		uint32 GetToGridSize() const { return ToGrid; }

#if WITH_EDITOR
		/** Return true if the grid sizes & path match. */
		bool operator==(const FPCGGridLinkageElement& Other) const;
#endif

	private:
		uint32 FromGrid = PCGHiGenGrid::UninitializedGridSize();
		uint32 ToGrid = PCGHiGenGrid::UninitializedGridSize();

		// This tells us which side of the From/To relationship this grid linkage is on.
		uint32 GenerationGrid = PCGHiGenGrid::UninitializedGridSize();

		FString ResourceKey;
		FName UpstreamPinLabel = NAME_None;
		TWeakObjectPtr<const UPCGPin> UpstreamPin = nullptr;
	};

	/** Compares InFromGrid and InToGrid and performs data storage/retrieval as necessary to marshal data across execution grids. */
	bool ExecuteGridLinkage(
		uint32 InGenerationGridSize,
		uint32 InFromGridSize,
		uint32 InToGridSize,
		const FString& InResourceKey,
		FName InUpstreamPinLabel,
		FPCGGridLinkageContext* InContext);
}

UCLASS(ClassGroup = (Procedural))
class UPCGGridLinkageSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGGridLinkageSettings();

protected:
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY()
	uint32 FromGrid = PCGHiGenGrid::UninitializedGridSize();

	UPROPERTY()
	uint32 ToGrid = PCGHiGenGrid::UninitializedGridSize();

	UPROPERTY()
	uint32 GenerationGrid = PCGHiGenGrid::UninitializedGridSize();

	UPROPERTY()
	FString ResourceKey;

	UPROPERTY()
	TSoftObjectPtr<const UPCGPin> UpstreamPin = nullptr;
};
