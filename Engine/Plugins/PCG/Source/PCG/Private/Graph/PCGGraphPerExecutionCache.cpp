// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/PCGGraphPerExecutionCache.h"

#include "PCGCommon.h"
#include "PCGModule.h"
#include "Subsystems/IPCGBaseSubsystem.h"

void FPCGPerExecutionCache::Clear()
{
	PCG::TUniqueScopeLock WriteLock(Lock);
	Entries.Empty();
}

void FPCGPerExecutionCache::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	PCG::TSharedScopeLock ReadLock(Lock, !Lock.IsLockedByCurrentThread());
	for (TMap<FPCGTaskId, FPCGPerExecutionCacheEntry>::TIterator It = Entries.CreateIterator(); It; ++It)
	{
		It.Value().AddStructReferencedObjects(Collector);
	}
}

void FPCGPerExecutionCacheEntry::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& [Id, StructData] : Data)
	{
		if (StructData.IsValid())
		{
			StructData.GetMutable().AddStructReferencedObjects(Collector);
		}
	}
}

void FPCGPerExecutionCachePCGData::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Data);
}
