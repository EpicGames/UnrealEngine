// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGGraphTask.h"

#include "PCGComponent.h"
#include "PCGModule.h"
#include "PCGSubsystem.h"
#include "Compute/Elements/PCGComputeGraphElement.h"
#include "Editor/IPCGEditorModule.h"
#include "Graph/PCGGraphExecutor.h"
#include "Helpers/PCGHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGraphTask)

int32 FPCGGraphActiveTask::NumExecuting = 0;

const FPCGStack* FPCGGraphTask::GetStack() const
{
	return (StackContext && StackIndex != INDEX_NONE) ? StackContext->GetStack(StackIndex) : nullptr;
}

FPCGStackHandle FPCGGraphTask::GetStackHandle() const
{
	return FPCGStackHandle(StackContext, StackIndex);
}

FPCGTaskId FPCGGraphTask::GetGraphExecutionTaskId() const
{
	return StackContext ? StackContext->GetGraphExecutionTaskId() : InvalidPCGTaskId;
}

bool FPCGGraphTask::CanExecuteOnlyOnMainThread() const
{
	return (Element && Element->CanExecuteOnlyOnMainThread(Context)) || (Context && Context->CanExecuteOnlyOnMainThread());
}

#if WITH_EDITOR
void FPCGGraphTask::LogVisual(ELogVerbosity::Type InVerbosity, const FText& InMessage) const
{
	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		const FPCGStack* TaskStack = GetStack();
		FPCGStack StackWithNode = TaskStack ? FPCGStack(*TaskStack) : FPCGStack();
		StackWithNode.PushFrame(Node);
		PCGEditorModule->GetNodeVisualLogsMutable().Log(StackWithNode, InVerbosity, InMessage);
	}
}

bool FPCGGraphTaskInput::operator==(const FPCGGraphTaskInput& Other) const
{
	return (TaskId == Other.TaskId)
		&& (UpstreamPin == Other.UpstreamPin)
		&& (DownstreamPin == Other.DownstreamPin)
		&& (bProvideData == Other.bProvideData);
}

bool FPCGGraphTask::IsApproximatelyEqual(const FPCGGraphTask& Other) const
{
	// Do trivial pointer comparisons first, then run == operator to determine equivalence.
	bool bElementsMatch = (Element == Other.Element);
	if (!bElementsMatch && Element && Other.Element)
	{
		if (Element->IsGridLinkage() && Other.Element->IsGridLinkage())
		{
			const PCGGraphExecutor::FPCGGridLinkageElement& LinkageElement = static_cast<const PCGGraphExecutor::FPCGGridLinkageElement&>(*Element);
			const PCGGraphExecutor::FPCGGridLinkageElement& OtherLinkageElement = static_cast<const PCGGraphExecutor::FPCGGridLinkageElement&>(*Other.Element);
			bElementsMatch = (LinkageElement == OtherLinkageElement);
		}
		else if (Element->IsComputeGraphElement() && Other.Element->IsComputeGraphElement())
		{
			const FPCGComputeGraphElement& ComputeGraphElement = static_cast<const FPCGComputeGraphElement&>(*Element);
			const FPCGComputeGraphElement& OtherComputeGraphElement = static_cast<const FPCGComputeGraphElement&>(*Other.Element);
			bElementsMatch = (ComputeGraphElement == OtherComputeGraphElement);
		}
		else
		{
			ensureMsgf(false, TEXT("Graph compilation emitted an element type that is not a trivial element or a grid linkage element. Element comparison will fail. ")
				TEXT("Equivalence operator needs to be implemented for this new element."));
		}
	}

	return (Inputs == Other.Inputs)
		&& (Node == Other.Node)
		&& (ExecutionSource == Other.ExecutionSource)
		&& bElementsMatch
		&& (Context == Other.Context)
		&& (NodeId == Other.NodeId)
		&& (CompiledTaskId == Other.CompiledTaskId)
		&& (ParentId == Other.ParentId)
		&& (PinDependency == Other.PinDependency)
		&& (StackIndex == Other.StackIndex)
		&& (StackContext == Other.StackContext);
}

void FPCGGraphTask::PrepareForCook()
{
	if (ElementSource == EPCGElementSource::FromNode)
	{
		check(Node);
		NodePtr = Node;
	}
}
#else
void FPCGGraphTask::LoadCookedData()
{
	if (ElementSource == EPCGElementSource::FromNode)
	{
		FGCScopeGuard Guard;
		check(NodePtr.Get());
		Node = NodePtr.Get();
	}
}
#endif // WITH_EDITOR

FPCGGraphActiveTask::~FPCGGraphActiveTask()
{
	if (Context)
	{
		PCGHelpers::ExecuteOnGameThread(UE_SOURCE_LOCATION, [ContextPtr = Context.Release()]()
		{
			LLM_SCOPE_BYTAG(PCG);

			PCGGraphExecutor::ClearAsyncFlags(ContextPtr->AsyncObjects);

			FPCGContext::Release(ContextPtr);
		});	
	}
}

void FPCGGraphActiveTask::StartExecuting()
{
	check(!bIsExecutingTask);
	++NumExecuting;

	auto GatherReferences = [this](const FPCGDataCollection& Collection)
	{
		for (const FPCGTaggedData& TaggedInputData : Collection.TaggedData)
		{
			if (TaggedInputData.Data)
			{
				ExecutingReferences.Add(TaggedInputData.Data);
			}
		}
	};

	// @todo_pcg this should be merged through some kind of visitor pattern with AddStructReferencedObjects
	// We need to keep a immutable list of references prior to execution mainly so that TaggedData contents don't change while we are running GC
	{
		ExecutingReferences.Reserve(Context->InputData.TaggedData.Num() + Context->OutputData.TaggedData.Num() + 1);
		GatherReferences(Context->InputData);
		GatherReferences(Context->OutputData);

		// Additional cached per-data
		for (TPair<FPCGDataCollection, FPCGDataCollection>& CachedInputToOutput : Context->CachedInputToOutputInternalResults)
		{
			GatherReferences(CachedInputToOutput.Key);
			GatherReferences(CachedInputToOutput.Value);
		}

		if (Context->SettingsWithOverride)
		{
			ExecutingReferences.Add(Context->SettingsWithOverride);
		}
	}

	bIsExecutingTask = true;
}

void FPCGGraphActiveTask::StopExecuting()
{
	if (bIsExecutingTask)
	{
		bIsExecutingTask = false;
		ExecutingTask = {};
		--NumExecuting;

		ExecutingReferences.Empty();
	}
}

bool FPCGGraphActiveTask::CanExecuteOnlyOnMainThread() const
{
	return (Element && Element->CanExecuteOnlyOnMainThread(Context.Get())) || (Context && Context->CanExecuteOnlyOnMainThread());
}
