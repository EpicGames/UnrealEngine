// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "PCGElement.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGPin.h"
#include "Graph/PCGPinDependencyExpression.h"
#include "Graph/PCGStackContext.h"

#include "Tasks/Task.h"

#include "PCGGraphTask.generated.h"

class UPCGComputeKernel;

/** Dictates how to recreate an element if it does not exist. */
UENUM()
enum class EPCGElementSource
{
	/** Element is a shared Trivial element. */
	Trivial,

	/** Element is a shared Trivial Post Graph element. */
	TrivialPostGraph,

	/** Element is a shared Gather element. */
	Gather,

	/** Element should be recreated from the node settings. */
	FromNode,

	/** Element does not have a node and should be recreated directly from cooked settings. */
	FromCookedSettings,
};

USTRUCT()
struct FPCGGraphTaskInput
{
	GENERATED_BODY()

	FPCGGraphTaskInput() = default;

	explicit FPCGGraphTaskInput(FPCGTaskId InTaskId, const TOptional<FPCGPinProperties>& InUpstreamPin = NoPin, const TOptional<FPCGPinProperties>& InDownstreamPin = NoPin, bool bInProvideData = true)
		: TaskId(InTaskId)
		, UpstreamPin(InUpstreamPin)
		, DownstreamPin(InDownstreamPin)
		, bProvideData(bInProvideData)
	{
	}

#if WITH_EDITOR
	bool operator==(const FPCGGraphTaskInput& Other) const;
#endif

	UPROPERTY()
	uint64 TaskId = InvalidPCGTaskId;

	/** The upstream output pin from which the input data comes. */
	UPROPERTY()
	TOptional<FPCGPinProperties> UpstreamPin;

	/** The input pin on the task element. */
	UPROPERTY()
	TOptional<FPCGPinProperties> DownstreamPin;

	/** Whether the input provides any data. For the post execute task, only the output node will provide data. */
	UPROPERTY()
	bool bProvideData = false;
	
	/** Whether the input is used multiple times (previous task has multiple successors). Useful to know if the data can be stolen or not. True by default.*/
	UPROPERTY()
	bool bIsUsedMultipleTimes = true;

	static inline const TOptional<FPCGPinProperties> NoPin = TOptional<FPCGPinProperties>();
};

USTRUCT()
struct FPCGGraphTask
{
	GENERATED_BODY()

#if WITH_EDITOR
	/** Approximate equivalence. Does not deeply check node settings, nor does it do a deep comparison of the element. */
	bool IsApproximatelyEqual(const FPCGGraphTask& Other) const;

	/** Because we might not already have a context, but still want to attach some logs to the node, use this utility function */
	void LogVisual(ELogVerbosity::Type InVerbosity, const FText& InMessage) const;

	void PrepareForCook();
#else
	void LoadCookedData();
#endif

	const FPCGStack* GetStack() const;
	FPCGStackHandle GetStackHandle() const;
	FPCGTaskId GetGraphExecutionTaskId() const;
	bool CanExecuteOnlyOnMainThread() const;

	UPROPERTY()
	TArray<FPCGGraphTaskInput> Inputs;

	/** Dictates how to recreate an element if it does not exist. */
	UPROPERTY()
	EPCGElementSource ElementSource = EPCGElementSource::FromNode;

	/** Serialized version of Node (see below) to avoid hard object references. Only used for cooking and standalone. */
	UPROPERTY()
	TSoftObjectPtr<const UPCGNode> NodePtr = nullptr;

	/**
	* Useful to recreate elements that don't have a node, such as ComputeGraph elements and GridLinkage elements.
	* Note: Because this is a hard pointer owned by the Graph Task, CookedSettings should ALWAYS be outered ONLY to the TopGraph, otherwise it will break.
	*/
	UPROPERTY()
	TObjectPtr<const UPCGSettings> CookedSettings = nullptr;

	UPROPERTY()
	uint64 NodeId = InvalidPCGTaskId;

	/** The task id as it exists when compiled. */
	UPROPERTY()
	uint64 CompiledTaskId = InvalidPCGTaskId;

	/** Represents the parent sub object graph task, if we were called from one. */
	UPROPERTY()
	uint64 ParentId = InvalidPCGTaskId;

	/** Conjunction of disjunctions of pin IDs that are required to be active for this task to be active.
	* Example - keep task if: UpstreamPin0Active && (UpstreamPin1Active || UpstreamPin2Active)
	*/
	UPROPERTY()
	FPCGPinDependencyExpression PinDependency;

	UPROPERTY()
	int32 StackIndex = INDEX_NONE;

	FPCGElementPtr Element; // Added to have tasks that aren't node-bound

	TWeakInterfacePtr<IPCGGraphExecutionSource> ExecutionSource = nullptr;

	const UPCGNode* Node = nullptr;
	FPCGContext* Context = nullptr;
	TSharedPtr<const FPCGStackContext> StackContext;

	// Whether SetupTask has been called on this task
	bool bHasDoneSetup = false;
	// BuildTaskInput will initialize this Collection which will later be used by PrepareForExecute
	FPCGDataCollection TaskInput;
	// CombineParams call might have created AsyncObjects
	TSet<TObjectPtr<UObject>> CombineParamsAsyncObjects;

	// Whether PrepareForExecute as been called on this task
	bool bHasDonePrepareForExecute = false;
	
	// Keep track if this task input has already been hooked to the pre task, like to a pre-task of a subgraph, to avoid hooking it twice, like to the graph pre-task.
	bool bWasHookedToPreTask = false;

#if WITH_EDITOR
	// Can be true when we want to have debug display on a task but have taken the results from the cache
	bool bIsBypassed = false;
#endif
};

struct FPCGGraphScheduleTask
{
	TArray<FPCGGraphTask> Tasks;

	TWeakInterfacePtr<IPCGGraphExecutionSource> ExecutionSource = nullptr;
	int32 FirstTaskIndex = 0;
	int32 LastTaskIndex = 0;
	bool bHasAbortCallbacks = false;
};

struct FPCGGraphActiveTask : TSharedFromThis<FPCGGraphActiveTask>
{
	FPCGGraphActiveTask() = default;
	virtual ~FPCGGraphActiveTask();

	FPCGGraphActiveTask(const FPCGGraphActiveTask&) = delete;
	FPCGGraphActiveTask& operator=(const FPCGGraphActiveTask&) = delete;

	FPCGGraphActiveTask(FPCGGraphActiveTask&&) = delete;
	FPCGGraphActiveTask& operator=(FPCGGraphActiveTask&&) = delete;

	void StartExecuting();
	void StopExecuting();
	bool CanExecuteOnlyOnMainThread() const;

	TArray<FPCGGraphTaskInput> Inputs;
	FPCGElementPtr Element;
	TUniquePtr<FPCGContext> Context;
	FPCGTaskId NodeId = InvalidPCGTaskId;
	std::atomic<bool> bWasCancelled = false;
#if WITH_EDITOR
	bool bIsBypassed = false;
#endif
	int32 StackIndex = INDEX_NONE;
	TSharedPtr<const FPCGStackContext> StackContext;
		
	// Those members need to be modified under the FPCGGraphExecutor::LiveTasksLock (unless we are running the old executor path)
	UE::Tasks::TTask<bool> ExecutingTask;
	bool bIsExecutingTask = false;

	// Used to know if task should be in ActiveTasks or ActiveTasksGameThreadOnly
	bool bIsGameThreadOnly = false;
	// TaskIndex inside ActiveTasks/ActiveTasksGameThreadOnly/PausedTasks for fast removal
	int32 TaskIndex = INDEX_NONE;

	static int32 NumExecuting;
	TArray<TObjectPtr<const UObject>> ExecutingReferences;
};
