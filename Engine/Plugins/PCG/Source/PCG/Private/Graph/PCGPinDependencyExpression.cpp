// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/PCGPinDependencyExpression.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPinDependencyExpression)

void FPCGPinDependencyExpression::AddPinDependency(FPCGPinId PinId)
{
	check(PinId != ConjunctionMarker && PinId != RemovedTermMarker);
	Expression.Add(PinId);
}

void FPCGPinDependencyExpression::AddConjunction()
{
	Expression.Add(ConjunctionMarker);
}

void FPCGPinDependencyExpression::AppendUsingConjunction(const FPCGPinDependencyExpression& Other)
{
	if (Other.Expression.IsEmpty())
	{
		return;
	}

	// If this expression is not empty, add a conjunction to join the two expressions.
	if (!Expression.IsEmpty())
	{
		AddConjunction();
	}

	Expression.Append(Other.Expression);
}

void FPCGPinDependencyExpression::DeactivatePin(FPCGPinId PinId, bool& bOutExpressionBecameFalse)
{
	bOutExpressionBecameFalse = false;

	for (int I = 0; I < Expression.Num(); ++I)
	{
		if (Expression[I] == PinId)
		{
			// Remove pin by replacing with conjunction marker. Could have removed the element but we don't
			// expect to see long expressions on average so this uses a simple approach.
			Expression[I] = RemovedTermMarker;

			// Check if this is was the last term in a disjunction (if we're surrounded by conjunction markers or removed pins).
			bool bNoTermsOnLeft = true;
			for (int J = I - 1; J >= 0; --J)
			{
				// End of disjunction
				if (Expression[J] == ConjunctionMarker)
				{
					break;
				}

				if (Expression[J] != RemovedTermMarker)
				{
					bNoTermsOnLeft = false;
					break;
				}
			}

			bool bNoTermsOnRight = true;
			for (int J = I + 1; J < Expression.Num(); ++J)
			{
				// End of disjunction
				if (Expression[J] == ConjunctionMarker)
				{
					break;
				}

				if (Expression[J] != RemovedTermMarker)
				{
					bNoTermsOnRight = false;
					break;
				}
			}

			if (bNoTermsOnLeft && bNoTermsOnRight)
			{
				// Removing this term made the disjunction become false, and hence the whole expression.
				// Don't early out however - make sure to update all disjunctions.
				bOutExpressionBecameFalse = true;
			}
		}
	}
}

void FPCGPinDependencyExpression::OffsetNodeIds(uint64 NodeIdOffset)
{
	for (FPCGPinId& PinId : Expression)
	{
		if (PinId != ConjunctionMarker)
		{
			PinId = PCGPinIdHelpers::OffsetNodeIdInPinId(PinId, NodeIdOffset);
		}
	}
}

#if WITH_EDITOR

FString FPCGPinDependencyExpression::ToString() const
{
	FString ExpressionString;

	bool InDisjunction = false;

	for (int I = 0; I < Expression.Num(); ++I)
	{
		if (Expression[I] == ConjunctionMarker)
		{
			ExpressionString += TEXT(" && ");

			if (InDisjunction)
			{
				ExpressionString += TEXT(")");
				InDisjunction = false;
			}
		}
		else
		{
			if (InDisjunction)
			{
				ExpressionString += TEXT(" || ");
			}
			else
			{
				ExpressionString += TEXT("(");
			}

			if (Expression[I] == RemovedTermMarker)
			{
				ExpressionString += TEXT(".");
			}
			else
			{
				// Decompose pin ID into task ID and pin index which is friendly to read.
				const FPCGTaskId NodeId = PCGPinIdHelpers::GetNodeIdFromPinId(Expression[I]);
				const uint64 PinIndex = PCGPinIdHelpers::GetPinIndexFromPinId(Expression[I]);
				ExpressionString += FString::Printf(TEXT("%" UINT64_FMT "_%" UINT64_FMT), NodeId, PinIndex);
			}

			InDisjunction = true;
		}
	}

	if (InDisjunction)
	{
		ExpressionString += TEXT(")");
	}

	return ExpressionString;
}

#endif // WITH_EDITOR
