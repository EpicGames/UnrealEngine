// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#include "PCGPinDependencyExpression.generated.h"

/** Conjunction of disjunctions of pin IDs that are required to be active for this task to be active.
* 
* Example:
*     Keep task if: UpstreamPin0Active && (UpstreamPin1Active || UpstreamPin2Active)
*     This will be stored in an internal array as: [ PinID0, ConjunctionMarker, PinID1, PinID2 ]
*     If Pin2 is deactivated during execution: [ PinID0, ConjunctionMarker, PinID1, RemovedMarker ]
*     If Pin1 is deactivated during execution: [ PinID0, ConjunctionMarker, RemovedMarker, RemovedMarker ]
*     Expression is now false and the task using this expression can be culled.
*/
USTRUCT()
struct FPCGPinDependencyExpression
{
	GENERATED_BODY()

	/** Expression building. Appends a pin ID to the current disjunction. */
	void AddPinDependency(FPCGPinId PinId);

	/** Expression building. Add a boolean && to the expression. */
	void AddConjunction();

	/** Expression building. Append the given expression using a conjunction, result will be Expression && Other.Expression. */
	void AppendUsingConjunction(const FPCGPinDependencyExpression& Other);

	/** Expression evaluation. Deactivating a pin is done by removing it from the expression. If any disjunction collapses
	* then the expression becomes false and the task can be culled.
	*/
	void DeactivatePin(FPCGPinId PinId, bool& bOutExpressionBecameFalse);

	/** Applies given node ID offset to all pin IDs in expression (pin ID is combination of node ID and pin index). */
	void OffsetNodeIds(uint64 NodeIdOffset);

	bool operator==(const FPCGPinDependencyExpression& Other) const { return (Expression == Other.Expression); }

#if WITH_EDITOR
	FString ToString() const;
#endif

private:
	/* These are used to demarcate conjunctions '&&' in the boolean expression. */
	inline static const FPCGPinId ConjunctionMarker = std::numeric_limits<FPCGPinId>::max();

	/* Signifies a pin removed from the expression, used to set terms false (pin inactive). */
	inline static const FPCGPinId RemovedTermMarker = std::numeric_limits<FPCGPinId>::max() - 1;

	UPROPERTY()
	TArray<uint64> Expression;
};
