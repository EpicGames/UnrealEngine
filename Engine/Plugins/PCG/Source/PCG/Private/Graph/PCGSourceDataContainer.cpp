// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/PCGSourceDataContainer.h"

#include "Serialization/Archive.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSourceDataContainer)

#define LOCTEXT_NAMESPACE "PCGSourceDataContainer"

FPCGSourceDataStorageKey::FPCGSourceDataStorageKey(const FName InLabel, const uint64 InHash) : Label(InLabel), Hash(InHash) {}

FPCGSourceDataContainer::FPCGSourceDataContainer(const FPCGSourceDataContainer& Other)
{
	*this = Other;
}

FPCGSourceDataContainer::FPCGSourceDataContainer(FPCGSourceDataContainer&& Other)
{
	*this = MoveTemp(Other);
}

FPCGSourceDataContainer& FPCGSourceDataContainer::operator=(FPCGSourceDataContainer&& Other)
{
	if (this != &Other)
	{
		DataStorage = MoveTemp(Other.DataStorage);
		DirtyGeneration = Other.DirtyGeneration;
		bAutoDirty.Store(Other.bAutoDirty.Load());
		Other.DirtyGeneration = 0;
		Other.bAutoDirty.Store(false);
	}
	return *this;
}

bool FPCGSourceDataContainer::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Ar << DirtyGeneration;

	int32 NumEntries = DataStorage.Num();
	Ar << NumEntries;

	// Load container
	if (Ar.IsLoading())
	{
		if (!ensure(NumEntries >= 0))
		{
			return false;
		}

		DataStorage.Empty(NumEntries);

		for (int32 Index = 0; Index < NumEntries; ++Index)
		{
			FPCGSourceDataStorageKey Key;

			Ar << Key.Label;
			Ar << Key.Hash;

			// @todo_pcg: Write a custom helper 'SerializeSharedStruct' in a future pass to avoid the copy overhead.
			FInstancedStruct InstancedValue;
			InstancedValue.Serialize(Ar);

			if (Ar.IsError())
			{
				DataStorage.Empty();
				return false;
			}

			if (InstancedValue.IsValid())
			{
				FPCGSourceDataStorageValue Value;
				Value.Data = FSharedStruct::Make(InstancedValue.GetScriptStruct(), InstancedValue.GetMemory());
				DataStorage.Add(Key, MoveTemp(Value));
			}
		}
	}
	// Save container
	else if (Ar.IsSaving())
	{
		for (auto& [Key, Value] : DataStorage)
		{
			Ar << Key.Label;
			Ar << Key.Hash;

			// @todo_pcg: Much like loading above, could be replaced with a custom serializer for the FSharedStruct.
			FInstancedStruct InstancedValue;
			InstancedValue.InitializeAs(Value.Data.GetScriptStruct(), Value.Data.GetMemory());
			InstancedValue.Serialize(Ar);
		}
	}

	return true;
}

void FPCGSourceDataContainer::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	// Add references to the heap allocated objects, to keep them alive for transactional lifetime.
	for (const auto& [Key, Value] : DataStorage)
	{
		Value.Data.AddStructReferencedObjects(Collector);
	}
}

FPCGSourceDataContainer& FPCGSourceDataContainer::operator=(const FPCGSourceDataContainer& Other)
{
	if (this != &Other)
	{
		bAutoDirty.Store(Other.bAutoDirty.Load());
		DirtyGeneration = Other.DirtyGeneration;

		// Deep copy required for transactional memory.
		DataStorage.Empty(Other.DataStorage.Num());
		for (const auto& [Key, Value] : Other.DataStorage)
		{
			FPCGSourceDataStorageValue NewValue;
			if (Value.Data.IsValid())
			{
				NewValue.Data = FSharedStruct::Make(Value.Data.GetScriptStruct(), Value.Data.GetMemory());
			}

			DataStorage.Add(Key, MoveTemp(NewValue));
		}
	}

	return *this;
}

#if WITH_EDITOR
bool FPCGSourceDataContainer::Remove(const FPCGSourceDataStorageKey& DataKey)
{
	PCG::TUniqueScopeLock Lock(DataStorageLock);

	const int32 NumRemovedItems = DataStorage.Remove(DataKey);
	const bool bWasRemoved = NumRemovedItems > 0;

	if (bWasRemoved && bAutoDirty)
	{
		++DirtyGeneration;
	}

	return bWasRemoved;
}

void FPCGSourceDataContainer::MarkDirty()
{
	PCG::TUniqueScopeLock Lock(DataStorageLock);
	++DirtyGeneration;
}

void FPCGSourceDataContainer::SetShouldAutoDirty(const bool bShouldAutoDirty)
{
	PCG::TUniqueScopeLock Lock(DataStorageLock);
	bAutoDirty = bShouldAutoDirty;
}

void FPCGSourceDataContainer::Empty()
{
	DataStorage.Empty();
	DirtyGeneration = 0;
}
#endif // WITH_EDITOR

uint32 FPCGSourceDataContainer::GetDirtyGeneration() const
{
#if WITH_EDITOR
	PCG::TSharedScopeLock Lock(DataStorageLock);
#endif // WITH_EDITOR
	return DirtyGeneration;
}

int32 FPCGSourceDataContainer::Num() const
{
	return DataStorage.Num();
}

bool FPCGSourceDataContainer::IsEmpty() const
{
	return DataStorage.IsEmpty();
}

uint32 GetTypeHash(const FPCGSourceDataStorageKey& Key)
{
	return HashCombine(GetTypeHash(Key.Label), GetTypeHash(Key.Hash));
}

#undef LOCTEXT_NAMESPACE