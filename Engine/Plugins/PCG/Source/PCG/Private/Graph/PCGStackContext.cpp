// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/PCGStackContext.h"

#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSubgraph.h"

#include "Algo/Find.h"
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"
#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGStackContext)

void FPCGStackFrame::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		ComputeHash();
	}
}

void FPCGStackFrame::ComputeHash()
{
	if (Object.IsNull())
	{
		Hash = GetTypeHash(LoopIndex);
	}
	else
	{
		Hash = GetTypeHash(Object.ToString());
	}
}

const FPCGStack* FPCGStackContext::GetStack(int32 InStackIndex) const
{
	if (ensure(Stacks.IsValidIndex(InStackIndex)))
	{
		return &Stacks[InStackIndex];
	}
	else
	{
		return nullptr;
	}
}

void FPCGStack::PopFrame()
{
	if (ensure(!StackFrames.IsEmpty()))
	{
		StackFrames.Pop();
	}
}

bool FPCGStack::CreateStackFramePath(FString& OutString, const UPCGNode* InNode, const UPCGPin* InPin) const
{
	// Give a healthy amount of scratch space on the stack, if it overflows it will use heap.
	TStringBuilderWithBuffer<TCHAR, 2048> StringBuilder;

	auto AddPathSeparator = [&StringBuilder]()
	{
		if (StringBuilder.Len())
		{
			StringBuilder << TEXT("/");
		}
	};

	if (StackFrames.Num() > 0)
	{
		FGCScopeGuard Guard;
		for (const FPCGStackFrame& Frame : StackFrames)
		{
			if (const UObject* Object = Frame.GetObject_NoGuard())
			{
				AddPathSeparator();

				if (!Object)
				{
					// If any object does not resolve, cannot build the string
					return false;
				}

				if (Object->Implements<UPCGGraphExecutionSource>())
				{
					StringBuilder << TEXT("EXECUTION SOURCE:") << Object->GetFullName();
				}
				else if (Object->IsA<UPCGGraph>())
				{
					StringBuilder << TEXT("GRAPH:") << Object->GetFullName();
				}
				else if (Object->IsA<UPCGNode>())
				{
					StringBuilder << TEXT("NODE:") << Object->GetFName();
				}
				else
				{
					// Unrecognized type, should not happen
					ensure(false);
					StringBuilder << TEXT("UNRECOGNIZED:") << Object->GetFullName();
				}
			}
			else if (Frame.LoopIndex != -1)
			{
				AddPathSeparator();
				StringBuilder << TEXT("LOOP:") << FString::FromInt(Frame.LoopIndex);
			}
		}
	}

	if (InNode)
	{
		AddPathSeparator();
		StringBuilder << TEXT("NODE:") << InNode->GetFName().ToString();

		if (InPin)
		{
			AddPathSeparator();
			StringBuilder << TEXT("PIN:") << InPin->GetFName().ToString();
		}
	}

	OutString = StringBuilder;
	return true;
}

uint32 FPCGStack::GetNumGraphLevels() const
{
	uint32 GraphCount = 0;
	if (StackFrames.Num() > 0)
	{
		FGCScopeGuard Guard;
		for (const FPCGStackFrame& Frame : StackFrames)
		{
			if (const UObject* FrameObject = Frame.GetObject_NoGuard<UPCGGraph>())
			{
				++GraphCount;
			}
		}
	}

	return GraphCount;
}

bool FPCGStack::BeginsWith(const FPCGStack& Other) const
{
	if (Other.GetStackFrames().Num() > GetStackFrames().Num())
	{
		return false;
	}

	for (int32 StackIndex = 0; StackIndex < Other.GetStackFrames().Num(); ++StackIndex)
	{
		// Test hash first which is a lot faster (hash is cached) than testing for equality
		if (GetTypeHash(Other.GetStackFrames()[StackIndex]) != GetTypeHash(GetStackFrames()[StackIndex]))
		{
			return false;
		}

		if (Other.GetStackFrames()[StackIndex] != GetStackFrames()[StackIndex])
		{
			return false;
		}
	}

	return true;
}

const UPCGComponent* FPCGStack::GetRootComponent() const
{
	return StackFrames.IsEmpty() ? nullptr : StackFrames[0].GetObject_AnyThread<UPCGComponent>();
}

const IPCGGraphExecutionSource* FPCGStack::GetRootSource() const
{
	return StackFrames.IsEmpty() ? nullptr : StackFrames[0].GetObject_AnyThread<IPCGGraphExecutionSource>();
}

const UPCGGraph* FPCGStack::GetRootGraph(int32* OutRootFrameIndex) const
{
	if (GetStackFrames().Num() > 0)
	{
		FGCScopeGuard Guard;
		for (int StackIndex = 0; StackIndex < GetStackFrames().Num(); ++StackIndex)
		{
			if (const UPCGGraph* Graph = StackFrames[StackIndex].GetObject_NoGuard<UPCGGraph>())
			{
				if (OutRootFrameIndex)
				{
					*OutRootFrameIndex = StackIndex;
				}

				return Graph;
			}
		}
	}

	return nullptr;
}

const UPCGGraph* FPCGStack::GetGraphForCurrentFrame() const
{
	if (GetStackFrames().Num() > 0)
	{
		FGCScopeGuard Guard;
		for (int StackIndex = GetStackFrames().Num() - 1; StackIndex >= 0; --StackIndex)
		{
			if (const UPCGGraph* Graph = StackFrames[StackIndex].GetObject_AnyThread<UPCGGraph>())
			{
				return Graph;
			}
		}
	}

	return nullptr;
}

const UPCGGraph* FPCGStack::GetNearestDynamicSubgraphForCurrentFrame() const
{
	// Dynamic subgraphs and looped subgraphs always have 3 stacks: SubgraphNode/LoopIndex/Subgraph
	// Look for a loop index frame and then return the corresponding graph.
	for (int StackIndex = GetStackFrames().Num() - 1; StackIndex >= 2; --StackIndex)
	{
		if (StackFrames[StackIndex - 1].IsLoopIndexFrame())
		{
			return StackFrames[StackIndex].GetObject_AnyThread<UPCGGraph>();
		}
	}

	return nullptr;
}

const UPCGGraph* FPCGStack::GetNearestNonInlinedGraphForCurrentFrame() const
{
	const UPCGGraph* NearestDynamicSubgraph = GetNearestDynamicSubgraphForCurrentFrame();
	return NearestDynamicSubgraph ? NearestDynamicSubgraph : GetRootGraph();
}

const UPCGNode* FPCGStack::GetCurrentFrameNode() const
{
	return StackFrames.IsEmpty() ? nullptr : StackFrames.Last().GetObject_AnyThread<UPCGNode>();
}

const UPCGNode* FPCGStack::GetNodeForCurrentFrame() const
{
	if (GetStackFrames().Num() > 0)
	{
		FGCScopeGuard Guard;
		for (int StackIndex = GetStackFrames().Num() - 1; StackIndex >= 0; --StackIndex)
		{
			if (const UPCGNode* Node = StackFrames[StackIndex].GetObject_AnyThread<UPCGNode>())
			{
				return Node;
			}
		}
	}

	return nullptr;
}

bool FPCGStack::HasObject(const UObject* InObject) const
{
	if (GetStackFrames().IsEmpty())
	{
		return false;
	}

	FGCScopeGuard Guard;
	return !!Algo::FindByPredicate(StackFrames, [InObject](const FPCGStackFrame& Frame)
	{
		return Frame.GetObject_NoGuard() == InObject;
	});
}

bool FPCGStack::operator==(const FPCGStack& Other) const
{
	// Stacks are the same if all stack frames are the same
	if (StackFrames.Num() != Other.StackFrames.Num())
	{
		return false;
	}

	for (int32 i = 0; i < StackFrames.Num(); i++)
	{
		if (StackFrames[i] != Other.StackFrames[i])
		{
			return false;
		}
	}

	return true;
}

FPCGCrc FPCGStack::GetCrc() const
{
	FArchiveCrc32 Ar;
		
	for (const FPCGStackFrame& StackFrame : StackFrames)
	{
		if (!StackFrame.Object.IsNull())
		{
			TSoftObjectPtr<const UObject> SoftObjectPtr = StackFrame.Object;
			Ar << SoftObjectPtr;
		}
		else
		{
			int32 LoopIndex = StackFrame.LoopIndex;
			Ar << LoopIndex;
		}
	}

	return FPCGCrc(Ar.GetCrc());
}

FPCGStackContext::FPCGStackContext(const FPCGStackContext& InStackContext, const FPCGStack& InCommonParentStack)
{
	const TArray<FPCGStackFrame>& ParentFrames = InCommonParentStack.StackFrames;
	const int ParentStackFrameCount = ParentFrames.Num();

	const TArray<FPCGStack>& InStacks = InStackContext.Stacks;

	Stacks.SetNum(InStacks.Num());
	for(int StackIndex = 0; StackIndex < Stacks.Num(); ++StackIndex)
	{
		FPCGStack& Stack = Stacks[StackIndex];
		const FPCGStack& InStack = InStacks[StackIndex];

		Stack.StackFrames.Reserve(InStack.StackFrames.Num() + ParentStackFrameCount);
		Stack.StackFrames.Append(ParentFrames);
		Stack.StackFrames.Append(InStack.StackFrames);
	}
}

int32 FPCGStackContext::PushFrame(const UObject* InFrameObject)
{
	if (CurrentStackIndex == INDEX_NONE)
	{
		// Create first stack using the given frame.
		FPCGStack& Stack = Stacks.Emplace_GetRef();
		Stack.PushFrame(InFrameObject);
		CurrentStackIndex = 0;
	}
	else
	{
		if (!ensure(Stacks.IsValidIndex(CurrentStackIndex)))
		{
			return INDEX_NONE;
		}

		// Append given frame object to current stack. Newly encountered stacks should generally
		// be unique, so we just commit to creating it immediately rather than searching to see
		// if it already exists first.
		FPCGStack CurrentStack = Stacks[CurrentStackIndex];
		CurrentStack.PushFrame(FPCGStackFrame(InFrameObject));
		CurrentStackIndex = Stacks.AddUnique(MoveTemp(CurrentStack));
	}

	return CurrentStackIndex;
}

int32 FPCGStackContext::PopFrame()
{
	if (!ensure(Stacks.IsValidIndex(CurrentStackIndex)))
	{
		return INDEX_NONE;
	}

	// Find the 'parent' callstack (current stack minus latest frame). Can be anywhere in the list of stacks so do a search.
	CurrentStackIndex = Stacks.IndexOfByPredicate([&CurrentStack = Stacks[CurrentStackIndex]](const FPCGStack& OtherStack)
	{
		const int32 RequiredFrameCount = CurrentStack.StackFrames.Num() - 1;
		if (OtherStack.StackFrames.Num() != RequiredFrameCount)
		{
			return false;
		}
		
		for (int32 i = 0; i < RequiredFrameCount; ++i)
		{
			if (OtherStack.StackFrames[i] != CurrentStack.StackFrames[i])
			{
				return false;
			}
		}

		return true;
	});
	ensure(CurrentStackIndex != INDEX_NONE);

	return CurrentStackIndex;
}

void FPCGStackContext::AppendStacks(const FPCGStackContext& InStacks)
{
	if (!ensure(Stacks.IsValidIndex(CurrentStackIndex)))
	{
		return;
	}

	for (const FPCGStack& SubgraphStack : InStacks.Stacks)
	{
		FPCGStack& NewStack = Stacks.Emplace_GetRef();
		NewStack.GraphExecutionTaskId = GraphExecutionTaskId;
		NewStack.StackFrames.Reserve(Stacks[CurrentStackIndex].StackFrames.Num() + SubgraphStack.StackFrames.Num());
		
		NewStack.StackFrames.Append(Stacks[CurrentStackIndex].StackFrames);
		NewStack.StackFrames.Append(SubgraphStack.StackFrames);
	}
}

void FPCGStackContext::PrependParentStack(const FPCGStack* InParentStack)
{
	if (!InParentStack || InParentStack->StackFrames.IsEmpty())
	{
		return;
	}

	for (FPCGStack& Stack : Stacks)
	{
		Stack.StackFrames.Insert(InParentStack->StackFrames, 0);
	}
}

bool FPCGStackContext::operator==(const FPCGStackContext& Other) const
{
	return (CurrentStackIndex == Other.CurrentStackIndex) && (Stacks == Other.Stacks);
}

void FPCGStackContext::SetGraphExecutionTaskId(FPCGTaskId InGraphExecutionTaskId)
{
	check(GraphExecutionTaskId == InvalidPCGTaskId);
	GraphExecutionTaskId = InGraphExecutionTaskId;

	for (FPCGStack& Stack : Stacks)
	{
		Stack.GraphExecutionTaskId = GraphExecutionTaskId;
	}
}
