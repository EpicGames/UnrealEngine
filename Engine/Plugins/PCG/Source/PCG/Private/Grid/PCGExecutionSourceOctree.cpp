// Copyright Epic Games, Inc. All Rights Reserved.

#include "Grid/PCGExecutionSourceOctree.h"
#include "PCGGraphExecutionStateInterface.h"

FPCGExecutionSourceRef::FPCGExecutionSourceRef(IPCGGraphExecutionSource* InExecutionSource, const FPCGExecutionSourceOctreeIDSharedRef& InIdShared)
	: IdShared(InIdShared)
{
	check(InExecutionSource);

	ExecutionSource = InExecutionSource;

	UpdateBounds();
}

void FPCGExecutionSourceRef::UpdateBounds()
{
	check(ExecutionSource);

	Bounds = ExecutionSource->GetExecutionState().GetBounds();
}

FPCGExecutionSourceOctreeAndMap::FPCGExecutionSourceOctreeAndMap(const FVector& InOrigin, FVector::FReal InExtent)
	: Octree(InOrigin, InExtent)
{
}

void FPCGExecutionSourceOctreeAndMap::Reset(const FVector& InOrigin, FVector::FReal InExtent)
{
	UE::TWriteScopeLock WriteLock(Lock);
	Octree = FPCGExecutionSourceOctree(InOrigin, InExtent);
	ExecutionSourceToIdMap.Empty();
}

bool FPCGExecutionSourceOctreeAndMap::Contains(const IPCGGraphExecutionSource* InExecutionSource) const
{
	UE::TReadScopeLock ReadLock(Lock);

	return ExecutionSourceToIdMap.Find(InExecutionSource) != nullptr;
}

FBox FPCGExecutionSourceOctreeAndMap::GetBounds(const IPCGGraphExecutionSource* InExecutionSource) const
{
	FBox Bounds(EForceInit::ForceInit);

	UE::TReadScopeLock ReadLock(Lock);
	if (const FPCGExecutionSourceOctreeIDSharedRef* ElementIdPtr = ExecutionSourceToIdMap.Find(InExecutionSource))
	{
		const FPCGExecutionSourceRef& ExecutionSourceRef = Octree.GetElementById((*ElementIdPtr)->Id);
		Bounds = ExecutionSourceRef.Bounds.GetBox();
	}

	return Bounds;
}

void FPCGExecutionSourceOctreeAndMap::AddOrUpdateExecutionSource(IPCGGraphExecutionSource* InExecutionSource, FBox& OutBounds, bool& bOutExecutionSourceHasChanged, bool& bOutExecutionSourceWasAdded)
{
	bOutExecutionSourceHasChanged = false;
	bOutExecutionSourceWasAdded = false;

	UE::TWriteScopeLock WriteLock(Lock);

	FPCGExecutionSourceOctreeIDSharedRef* ElementIdPtr = ExecutionSourceToIdMap.Find(InExecutionSource);

	if (!ElementIdPtr)
	{
		// Does not exist yet, add it.
		FPCGExecutionSourceOctreeIDSharedRef IdShared = MakeShared<FPCGExecutionSourceOctreeID>();
		FPCGExecutionSourceRef ExecutionSourceRef(InExecutionSource, IdShared);
		OutBounds = ExecutionSourceRef.Bounds.GetBox();
		check(OutBounds.IsValid);

		// If the ExecutionSource is already generated, it probably mean we are in loading. The ExecutionSource bounds and last
		// generated bounds should be the same.
		// If the bounds depends on other ExecutionSources on the owner however, it might not be the same, because of the registration order.
		// In this case, override the bounds by the last generated ones.
		FBox LastGeneratedBounds = InExecutionSource->GetExecutionState().GetGeneratedBounds();
		if (InExecutionSource->GetExecutionState().IsGenerated() && !LastGeneratedBounds.Equals(OutBounds))
		{
			OutBounds = LastGeneratedBounds;
			ExecutionSourceRef.Bounds = OutBounds;
		}

		Octree.AddElement(ExecutionSourceRef);

		bOutExecutionSourceHasChanged = true;
		bOutExecutionSourceWasAdded = true;

		// Store the shared ptr, because if we add/remove ExecutionSources in the octree, the id might change.
		// We need to make sure that we always have the latest id for the given ExecutionSource.
		ExecutionSourceToIdMap.Add(InExecutionSource, MoveTemp(IdShared));
	}
	else
	{
		// It already exists, update it if the bounds changed.

		// Do a copy here.
		FPCGExecutionSourceRef ExecutionSourceRef = Octree.GetElementById((*ElementIdPtr)->Id);
		FBox PreviousBounds = ExecutionSourceRef.Bounds.GetBox();
		ExecutionSourceRef.UpdateBounds();
		OutBounds = ExecutionSourceRef.Bounds.GetBox();
		check(OutBounds.IsValid);

		// If bounds changed, remove and re-add to the octree
		if (!PreviousBounds.Equals(OutBounds))
		{
			Octree.RemoveElement((*ElementIdPtr)->Id);
			Octree.AddElement(ExecutionSourceRef);
			bOutExecutionSourceHasChanged = true;
		}
	}
}

bool FPCGExecutionSourceOctreeAndMap::RemapExecutionSource(const IPCGGraphExecutionSource* InOldExecutionSource, IPCGGraphExecutionSource* InNewExecutionSource, bool& bOutBoundsHasChanged)
{
	bOutBoundsHasChanged = false;

	// First verification we have the old ExecutionSource registered
	if (!Contains(InOldExecutionSource))
	{
		return false;
	}

	// If so, lock again in write and recheck if it has not been remapped already.
	{
		UE::TWriteScopeLock WriteLock(Lock);

		FPCGExecutionSourceOctreeIDSharedRef* ElementIdPtr = ExecutionSourceToIdMap.Find(InOldExecutionSource);

		if (!ElementIdPtr)
		{
			// Nothing done
			return false;
		}

		FPCGExecutionSourceOctreeIDSharedRef ElementId = *ElementIdPtr;

		FPCGExecutionSourceRef ExecutionSourceRef = Octree.GetElementById(ElementId->Id);
		FBox PreviousBounds = ExecutionSourceRef.Bounds.GetBox();
		ExecutionSourceRef.ExecutionSource = InNewExecutionSource;
		ExecutionSourceRef.UpdateBounds();
		FBox Bounds = ExecutionSourceRef.Bounds.GetBox();
		check(Bounds.IsValid);

		// If bounds changed, we need to update the mapping
		bOutBoundsHasChanged = !Bounds.Equals(PreviousBounds);

		Octree.RemoveElement((*ElementIdPtr)->Id);
		Octree.AddElement(ExecutionSourceRef);
		ExecutionSourceToIdMap.Remove(InOldExecutionSource);
		ExecutionSourceToIdMap.Add(InNewExecutionSource, MoveTemp(ElementId));
	}

	return true;
}

bool FPCGExecutionSourceOctreeAndMap::RemoveExecutionSource(IPCGGraphExecutionSource* InExecutionSource)
{
	UE::TWriteScopeLock WriteLock(Lock);

	FPCGExecutionSourceOctreeIDSharedRef* ElementIdPtr = ExecutionSourceToIdMap.Find(InExecutionSource);

	if (!ElementIdPtr)
	{
		return false;
	}

	Octree.RemoveElement((*ElementIdPtr)->Id);
	ExecutionSourceToIdMap.Remove(InExecutionSource);

	return true;
}

TSet<IPCGGraphExecutionSource*> FPCGExecutionSourceOctreeAndMap::GetAllExecutionSources() const
{
	UE::TReadScopeLock ReadLock(Lock);

	TSet<IPCGGraphExecutionSource*> ExecutionSources;
	ExecutionSourceToIdMap.GetKeys(ExecutionSources);

	return ExecutionSources;
}
