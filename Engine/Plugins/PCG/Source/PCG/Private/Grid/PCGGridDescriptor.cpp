// Copyright Epic Games, Inc. All Rights Reserved.

#include "Grid/PCGGridDescriptor.h"

#include "Algo/Transform.h"
#include "Misc/StringBuilder.h"
#include "Serialization/ArchiveCrc32.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGridDescriptor)

namespace PCGGridDescriptor
{
	void Serialize(FArchive& Ar, const TArray<TSoftObjectPtr<UDataLayerAsset>>& InDataLayerAssets, TSoftObjectPtr<UHLODLayer> InHLODLayer)
	{
		for (const TSoftObjectPtr<UDataLayerAsset>& DataLayerAsset : InDataLayerAssets)
		{
			FString DataLayerAssetString = DataLayerAsset.ToString();
			Ar << DataLayerAssetString;
		}

		if (!InHLODLayer.IsNull())
		{
			FString HLODAssetString = InHLODLayer.ToString();
			Ar << HLODAssetString;
		}
	}
}

bool FPCGGridDescriptor::operator==(const FPCGGridDescriptor& Other) const
{
	// We can always compare RuntimeHash (it will be 0 in non game worlds)
	if (GridSize == Other.GridSize && bIs2DGrid == Other.bIs2DGrid && bIsRuntime == Other.bIsRuntime && RuntimeHash == Other.RuntimeHash)
	{
#if WITH_EDITORONLY_DATA
		return DataLayerAssets == Other.DataLayerAssets && HLODLayer == Other.HLODLayer;
#else
		return true;
#endif
	}

	return false;
}

uint32 FPCGGridDescriptor::ComputeHash() const
{
	// Initialize Archive with RuntimeHash, in Editor worlds this value will be 0 and the editor only properties will be hashed instead
	FArchiveCrc32 Ar(RuntimeHash);
	FPCGGridDescriptor* NonConstThis = const_cast<FPCGGridDescriptor*>(this);

	Ar << NonConstThis->GridSize << NonConstThis->bIs2DGrid << NonConstThis->bIsRuntime;

#if WITH_EDITORONLY_DATA	
	// Hashing of Editor only properties
	PCGGridDescriptor::Serialize(Ar, DataLayerAssets, HLODLayer);
#endif

	return Ar.GetCrc();
}

#if WITH_EDITOR
uint32 FPCGGridDescriptor::GetRuntimeHash() const
{
	if (DataLayerAssets.IsEmpty())
	{
		return 0;
	}

	FArchiveCrc32 Ar;
	PCGGridDescriptor::Serialize(Ar, DataLayerAssets, HLODLayer);
	return Ar.GetCrc();
}

FPCGGridDescriptor& FPCGGridDescriptor::SetDataLayerAssets(const TArray<TSoftObjectPtr<UDataLayerAsset>>& InDataLayerAssets)
{
	check(!Hash.IsSet());
	DataLayerAssets = InDataLayerAssets;
	DataLayerAssets.Sort([](const TSoftObjectPtr<UDataLayerAsset>& A, const TSoftObjectPtr<UDataLayerAsset>& B) { return A.ToString() < B.ToString(); });
	return *this;
}

FPCGGridDescriptor& FPCGGridDescriptor::SetDataLayerAssets(const TArray<const UDataLayerAsset*>& InDataLayerAssets)
{
	TArray<TSoftObjectPtr<UDataLayerAsset>> DataLayerAssetPtrs;
	Algo::Transform(InDataLayerAssets, DataLayerAssetPtrs, [](const UDataLayerAsset* DataLayerAsset) { return TSoftObjectPtr<UDataLayerAsset>(const_cast<UDataLayerAsset*>(DataLayerAsset)); });
	return SetDataLayerAssets(DataLayerAssetPtrs);
}

void FPCGGridDescriptor::GetDataLayerAssets(TArray<TSoftObjectPtr<UDataLayerAsset>>& OutDataLayerAssets, const UExternalDataLayerAsset*& OutExternalDataLayerAsset) const
{
	OutDataLayerAssets.Empty();
	OutExternalDataLayerAsset = nullptr;

	for (const TSoftObjectPtr<UDataLayerAsset>& DataLayerAssetPtr : DataLayerAssets)
	{
		if (UDataLayerAsset* ResolvedAsset = DataLayerAssetPtr.Get())
		{
			if (UExternalDataLayerAsset* ExternalDataLayerAsset = Cast<UExternalDataLayerAsset>(ResolvedAsset))
			{
				check(!OutExternalDataLayerAsset);
				OutExternalDataLayerAsset = ExternalDataLayerAsset;
			}
			else
			{
				OutDataLayerAssets.Add(DataLayerAssetPtr);
			}
		}
	}
}
#endif // WITH_EDITOR

FString FPCGGridDescriptor::GetPartitionActorName(const FIntVector& GridCoords) const
{
	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> ActorNameBuilder;

	if (IsRuntime())
	{
		ActorNameBuilder += TEXT("PCGRuntimePartitionGridActor_");
	}
	else
	{
		ActorNameBuilder += TEXT("PCGPartitionGridActor_");
	}

	ActorNameBuilder += FString::Printf(TEXT("%d_"), GetGridSize());

	if (Is2DGrid())
	{
		ActorNameBuilder += FString::Printf(TEXT("%d_%d"), GridCoords.X, GridCoords.Y);
	}
	else
	{
		ActorNameBuilder += FString::Printf(TEXT("%d_%d_%d"), GridCoords.X, GridCoords.Y, GridCoords.Z);
	}

#if WITH_EDITORONLY_DATA
	// For Data Layer Instances, build a hash to add as a suffix
	{
		FArchiveCrc32 Ar;
		PCGGridDescriptor::Serialize(Ar, DataLayerAssets, HLODLayer);
		
		uint32 NameHash = Ar.GetCrc();
		if (NameHash != 0)
		{
			ActorNameBuilder += FString::Printf(TEXT("_%X"), NameHash);
		}
	}
#endif

	return ActorNameBuilder.ToString();
}

bool FPCGGridCellDescriptor::operator==(const FPCGGridCellDescriptor& InOther) const
{
	return Descriptor == InOther.Descriptor && GridCoords == InOther.GridCoords;
}

uint32 GetTypeHash(const FPCGGridCellDescriptor& In)
{
	return HashCombine(GetTypeHash(In.Descriptor), GetTypeHash(In.GridCoords));
}
