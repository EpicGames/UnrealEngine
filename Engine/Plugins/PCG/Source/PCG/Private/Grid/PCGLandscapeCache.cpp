// Copyright Epic Games, Inc. All Rights Reserved.

#include "Grid/PCGLandscapeCache.h"

#include "PCGCustomVersion.h"
#include "PCGPoint.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Utils/PCGLogErrors.h"

#include "LandscapeComponent.h"
#include "LandscapeDataAccess.h"
#include "LandscapeInfoMap.h"
#include "LandscapeProxy.h"
#include "LandscapeSubsystem.h"

#include "Async/ParallelFor.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Kismet/GameplayStatics.h"
#include "Serialization/BufferWriter.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGLandscapeCache)

static TAutoConsoleVariable<float> CVarLandscapeCacheGCFrequency(
	TEXT("pcg.LandscapeCacheGCFrequency"),
	10,
	TEXT("Rate at which to release landscape cache data, in seconds"));

static TAutoConsoleVariable<int32> CVarLandscapeCacheSizeThreshold(
	TEXT("pcg.LandscapeCacheSizeThreshold"),
	64,
	TEXT("Memory Threhold at which we start cleaning up the landscape cache"));

namespace PCGLandscapeCache
{
	FSafeIndices CalcSafeIndices(FVector2D LocalPoint, int32 Stride)
	{
		check(Stride != 0);

		const FVector2D ClampedLocalPoint = LocalPoint.ClampAxes(0.0, FVector2D::FReal(Stride-1));

		FSafeIndices Result;
		const int32 CellX0 = FMath::FloorToInt(ClampedLocalPoint.X);
		const int32 CellY0 = FMath::FloorToInt(ClampedLocalPoint.Y);
		const int32 CellX1 = FMath::Min(CellX0+1, Stride-1);
		const int32 CellY1 = FMath::Min(CellY0+1, Stride-1);

		Result.X0Y0 = CellX0 + CellY0 * Stride;
		Result.X1Y0 = CellX1 + CellY0 * Stride;
		Result.X0Y1 = CellX0 + CellY1 * Stride;
		Result.X1Y1 = CellX1 + CellY1 * Stride;

		Result.XFraction = FMath::Fractional(ClampedLocalPoint.X);
		Result.YFraction = FMath::Fractional(ClampedLocalPoint.Y);

		return Result;
	}

	FIntPoint GetCoordinates(const ULandscapeComponent* LandscapeComponent)
	{
		check(LandscapeComponent && LandscapeComponent->ComponentSizeQuads != 0);
		return FIntPoint(LandscapeComponent->SectionBaseX / LandscapeComponent->ComponentSizeQuads, LandscapeComponent->SectionBaseY / LandscapeComponent->ComponentSizeQuads);
	}
}

#if WITH_EDITOR
TSharedPtr<FPCGLandscapeCacheEntry> FPCGLandscapeCacheEntry::CreateCacheEntry(ULandscapeInfo* LandscapeInfo, ULandscapeComponent* InComponent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGLandscapeCacheEntry::CreateCacheEntry);

	if (!IsInGameThread() && !IsInParallelGameThread())
	{
		return nullptr;
	}

	FLandscapeComponentDataInterface CDI(InComponent, 0, /*bWorkInEditingLayer=*/false);

	// CDI.GetRawHeightData() may be nullptr if InComponent has no texture data.
	if (CDI.GetRawHeightData() == nullptr)
	{
		return nullptr;
	}

	// The component has an extra vertex on the edge, for interpolation purposes
	const int32 Stride = InComponent->ComponentSizeQuads + 1;

	if (Stride <= 0)
	{
		return nullptr;
	}

	TSharedPtr<FPCGLandscapeCacheEntry> Result = MakeShared<FPCGLandscapeCacheEntry>();

	Result->PointHalfSize = InComponent->GetComponentTransform().GetScale3D() * 0.5;
	Result->Stride = Stride;

	// Get landscape default layer (heightmap/tangents/normal)
	{
		FVector WorldPos;
		FVector WorldTangentX;
		FVector WorldTangentY;
		FVector WorldTangentZ;

		const int32 NumVertices = FMath::Square(Stride);

		Result->PositionsAndNormals.Reserve(2 * NumVertices);

		TRACE_CPUPROFILER_EVENT_SCOPE(GetWorldPositionTangents);
		for (int32 Index = 0; Index < NumVertices; ++Index)
		{
			CDI.GetWorldPositionTangents(Index, WorldPos, WorldTangentX, WorldTangentY, WorldTangentZ);
			Result->PositionsAndNormals.Add(WorldPos);
			Result->PositionsAndNormals.Add(WorldTangentZ);
		}
	}

	// Get other layers, push data into metadata attributes
	{
		TArray<uint8> LayerCache;
		for (const FLandscapeInfoLayerSettings& Layer : LandscapeInfo->Layers)
		{
			ULandscapeLayerInfoObject* LayerInfo = Layer.LayerInfoObj;
			if (!LayerInfo)
			{
				continue;
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(GetWeightmapTextureData);
			if (CDI.GetWeightmapTextureData(LayerInfo, LayerCache, /*bUseEditingLayer=*/false, /*bRemoveSubsectionDuplicates=*/true))
			{
				Result->LayerDataNames.Add(Layer.LayerName);
				Result->LayerData.Emplace(MoveTemp(LayerCache));
			}

			LayerCache.Reset();
		}
	}
	
	Result->bDataLoaded = true;

	return Result;
}
#endif // WITH_EDITOR

void FPCGLandscapeCacheEntry::GetPoint(int32 PointIndex, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(FPCGLandscapeCacheEntry::GetPoint);
	check(bDataLoaded && PointIndex >= 0 && 2 * PointIndex < PositionsAndNormals.Num());	

	const FVector& Position = PositionsAndNormals[2 * PointIndex];
	const FVector& Normal = PositionsAndNormals[2 * PointIndex + 1];
	
	FVector TangentX;
	FVector TangentY;
	TangentX = FVector(Normal.Z, 0.f, -Normal.X);
	TangentY = Normal ^ TangentX;

	const float Density = 1;
	const int32 Seed = 1 + PointIndex;

	new(&OutPoint) FPCGPoint(FTransform(TangentX.GetSafeNormal(), TangentY.GetSafeNormal(), Normal.GetSafeNormal(), Position), Density, Seed);
	OutPoint.BoundsMin = -PointHalfSize;
	OutPoint.BoundsMax = PointHalfSize;
	
	if (OutMetadata && !LayerData.IsEmpty())
	{
		check(LayerData.Num() == LayerDataNames.Num());
		OutPoint.MetadataEntry = OutMetadata->AddEntry();

		for(int32 LayerIndex = 0; LayerIndex < LayerData.Num(); ++LayerIndex)
		{
			const FName& CurrentLayerName = LayerDataNames[LayerIndex];
			const TArray<uint8>& CurrentLayerData = LayerData[LayerIndex];

			if (FPCGMetadataAttributeBase* Attribute = OutMetadata->GetMutableAttribute(CurrentLayerName))
			{
				check(Attribute->GetTypeId() == PCG::Private::MetadataTypes<float>::Id);
				static_cast<FPCGMetadataAttribute<float>*>(Attribute)->SetValue(OutPoint.MetadataEntry, (float)CurrentLayerData[PointIndex] / 255.0f);
			}
		}
	}
}

void FPCGLandscapeCacheEntry::GetPointHeightOnly(int32 PointIndex, FPCGPoint& OutPoint) const
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(FPCGLandscapeCacheEntry::GetPointHeightOnly);
	check(bDataLoaded && PointIndex >= 0 && 2 * PointIndex < PositionsAndNormals.Num());

	const FVector& Position = PositionsAndNormals[2 * PointIndex];
	
	const float Density = 1;
	const int32 Seed = 1 + PointIndex;

	new(&OutPoint) FPCGPoint(FTransform(Position), Density, Seed);
	OutPoint.BoundsMin = -PointHalfSize;
	OutPoint.BoundsMax = PointHalfSize;
}

void FPCGLandscapeCacheEntry::GetInterpolatedLayerWeights(const FVector2D& LocalPoint, TArray<FPCGLandscapeLayerWeight>& OutLayerWeights) const
{
	OutLayerWeights.SetNum(LayerData.Num());
	
	if (!bDataLoaded)
	{
		return;
	}

	const PCGLandscapeCache::FSafeIndices Indices = PCGLandscapeCache::CalcSafeIndices(LocalPoint, Stride);

	if (!LayerData.IsEmpty())
	{
		check(LayerData.Num() == LayerDataNames.Num());

		for(int32 LayerIndex = 0; LayerIndex < LayerData.Num(); ++LayerIndex)
		{
			FPCGLandscapeLayerWeight& OutLayer = OutLayerWeights[LayerIndex];
			OutLayer.Name = LayerDataNames[LayerIndex];

			const TArray<uint8>& CurrentLayerData = LayerData[LayerIndex];

			const float Y0Data = FMath::Lerp((float)CurrentLayerData[Indices.X0Y0] / 255.0f, (float)CurrentLayerData[Indices.X1Y0] / 255.0f, Indices.XFraction);
			const float Y1Data = FMath::Lerp((float)CurrentLayerData[Indices.X0Y1] / 255.0f, (float)CurrentLayerData[Indices.X1Y1] / 255.0f, Indices.XFraction);

			OutLayer.Weight = FMath::Lerp(Y0Data, Y1Data, Indices.YFraction);
		}
	}
}

void FPCGLandscapeCacheEntry::GetInterpolatedPointInternal(const PCGLandscapeCache::FSafeIndices& Indices, FPCGPoint& OutPoint, bool bHeightOnly) const
{
	check(bDataLoaded);
	check(2 * Indices.X1Y1 < PositionsAndNormals.Num());

	const FVector& PositionX0Y0 = PositionsAndNormals[2 * Indices.X0Y0];
	const FVector& PositionX1Y0 = PositionsAndNormals[2 * Indices.X1Y0];
	const FVector& PositionX0Y1 = PositionsAndNormals[2 * Indices.X0Y1];
	const FVector& PositionX1Y1 = PositionsAndNormals[2 * Indices.X1Y1];

	const FVector LerpPositionY0 = FMath::Lerp(PositionX0Y0, PositionX1Y0, Indices.XFraction);
	const FVector LerpPositionY1 = FMath::Lerp(PositionX0Y1, PositionX1Y1, Indices.XFraction);
	const FVector Position = FMath::Lerp(LerpPositionY0, LerpPositionY1, Indices.YFraction);

	const int32 Seed = UPCGBlueprintHelpers::ComputeSeedFromPosition(Position);
	const float Density = 1;

	if (bHeightOnly)
	{
		new(&OutPoint) FPCGPoint(FTransform(Position), Density, Seed);
	}
	else
	{
		const FVector& NormalX0Y0 = PositionsAndNormals[2 * Indices.X0Y0 + 1];
		const FVector& NormalX1Y0 = PositionsAndNormals[2 * Indices.X1Y0 + 1];
		const FVector& NormalX0Y1 = PositionsAndNormals[2 * Indices.X0Y1 + 1];
		const FVector& NormalX1Y1 = PositionsAndNormals[2 * Indices.X1Y1 + 1];

		const FVector LerpNormalY0 = FMath::Lerp(NormalX0Y0.GetSafeNormal(), NormalX1Y0.GetSafeNormal(), Indices.XFraction).GetSafeNormal();
		const FVector LerpNormalY1 = FMath::Lerp(NormalX0Y1.GetSafeNormal(), NormalX1Y1.GetSafeNormal(), Indices.XFraction).GetSafeNormal();
		const FVector Normal = FMath::Lerp(LerpNormalY0, LerpNormalY1, Indices.YFraction);

		FVector TangentX;
		FVector TangentY;
		TangentX = FVector(Normal.Z, 0.f, -Normal.X);
		TangentY = Normal ^ TangentX;

		new(&OutPoint) FPCGPoint(FTransform(TangentX.GetSafeNormal(), TangentY.GetSafeNormal(), Normal.GetSafeNormal(), Position), Density, Seed);
	}

	OutPoint.BoundsMin = -PointHalfSize;
	OutPoint.BoundsMax = PointHalfSize;
}

void FPCGLandscapeCacheEntry::GetInterpolatedPointMetadataInternal(const PCGLandscapeCache::FSafeIndices& Indices, int64& OutMetadataEntry, UPCGMetadata* OutMetadata) const
{
	if (bDataLoaded && OutMetadata && !LayerData.IsEmpty())
	{
		OutMetadata->InitializeOnSet(OutMetadataEntry);

		check(LayerData.Num() == LayerDataNames.Num());

		for (int32 LayerIndex = 0; LayerIndex < LayerData.Num(); ++LayerIndex)
		{
			const FName& CurrentLayerName = LayerDataNames[LayerIndex];
			const TArray<uint8>& CurrentLayerData = LayerData[LayerIndex];
			check(Indices.X1Y1 < CurrentLayerData.Num());

			if (FPCGMetadataAttribute<float>* Attribute = OutMetadata->FindOrCreateAttribute<float>(CurrentLayerName))
			{
				const float Y0Data = FMath::Lerp((float)CurrentLayerData[Indices.X0Y0] / 255.0f, (float)CurrentLayerData[Indices.X1Y0] / 255.0f, Indices.XFraction);
				const float Y1Data = FMath::Lerp((float)CurrentLayerData[Indices.X0Y1] / 255.0f, (float)CurrentLayerData[Indices.X1Y1] / 255.0f, Indices.XFraction);
				const float Data = FMath::Lerp(Y0Data, Y1Data, Indices.YFraction);

				Attribute->SetValue(OutMetadataEntry, Data);
			}
		}
	}
}

void FPCGLandscapeCacheEntry::GetInterpolatedPoint(const FVector2D& LocalPoint, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	check (bDataLoaded);
	const PCGLandscapeCache::FSafeIndices Indices = PCGLandscapeCache::CalcSafeIndices(LocalPoint, Stride);
	GetInterpolatedPointInternal(Indices, OutPoint);
	GetInterpolatedPointMetadataInternal(Indices, OutPoint.MetadataEntry, OutMetadata);
}

void FPCGLandscapeCacheEntry::GetInterpolatedPointMetadataOnly(const FVector2D& LocalPoint, int64& OutMetadataEntry, UPCGMetadata* OutMetadata) const
{
	check(bDataLoaded);
	const PCGLandscapeCache::FSafeIndices Indices = PCGLandscapeCache::CalcSafeIndices(LocalPoint, Stride);
	GetInterpolatedPointMetadataInternal(Indices, OutMetadataEntry, OutMetadata);
}

void FPCGLandscapeCacheEntry::GetInterpolatedPointHeightOnly(const FVector2D& LocalPoint, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	check(bDataLoaded);
	const PCGLandscapeCache::FSafeIndices Indices = PCGLandscapeCache::CalcSafeIndices(LocalPoint, Stride);
	GetInterpolatedPointInternal(Indices, OutPoint, /*bHeightOnly=*/true);
	GetInterpolatedPointMetadataInternal(Indices, OutPoint.MetadataEntry, OutMetadata);
}

bool FPCGLandscapeCacheEntry::TouchAndLoad(int32 InTouch) const
{
	Touch = InTouch; // technically, this could be an atomic, but we don't need this to be very precise

	if (!bDataLoaded)
	{
		PCG::TScopeLock ScopeDataLock(DataLock);
		if (!bDataLoaded)
		{
			return SerializeFromBulkData();
		}
	}

	return false;
}

void FPCGLandscapeCacheEntry::Unload()
{
	check(bDataLoaded);
	PositionsAndNormals.Reset();
	PositionsAndNormals.Shrink();
	LayerData.Reset();
	LayerData.Shrink();
	bDataLoaded = false;
	Touch = 0;
}

int32 FPCGLandscapeCacheEntry::GetMemorySize() const
{
	int32 MemSize = PositionsAndNormals.GetAllocatedSize();
	for (const TArray<uint8>& Layer : LayerData)
	{
		MemSize += Layer.GetAllocatedSize();
	}

	return MemSize;
}

void FPCGLandscapeCacheEntry::SerializeToBulkData(EPCGLandscapeCacheSerializationContents SerializationContents)
{
	const bool bSerializeEverything = SerializationContents == EPCGLandscapeCacheSerializationContents::SerializeAll;
	const bool bShouldSerializePositionAndNormals = (bSerializeEverything || SerializationContents == EPCGLandscapeCacheSerializationContents::SerializeOnlyPositionsAndNormals);
	const bool bShouldSerializeLayerData = (bSerializeEverything || SerializationContents == EPCGLandscapeCacheSerializationContents::SerializeOnlyLayerData);

	// Use empty buffers if the contents should not be serialized
	TArray<FVector> PositionsAndNormalsTemp;
	TArray<TArray<uint8>> LayerDataTemp;
	TArray<FVector>& SerializePositionsAndNormals = bShouldSerializePositionAndNormals ? PositionsAndNormals : PositionsAndNormalsTemp;
	TArray<TArray<uint8>>& SerializeLayerData = bShouldSerializeLayerData ? LayerData : LayerDataTemp;


	// Move data from local arrays to the bulk data
	BulkData.Lock(LOCK_READ_WRITE);

	int32 NumBytes = 0;
	// Number of entries in the array + size of the array
	NumBytes += sizeof(int32) + SerializePositionsAndNormals.Num() * SerializePositionsAndNormals.GetTypeSize();

	// Number of layers
	NumBytes += sizeof(int32); 
	for (TArray<uint8>& CurrentLayerData : SerializeLayerData)
	{
		// Number of entries in the layer data array + size of the array
		NumBytes += sizeof(int32) + CurrentLayerData.Num() * CurrentLayerData.GetTypeSize();
	}

	uint8* Dest = (uint8*)BulkData.Realloc(NumBytes);
	FBufferWriter Ar(Dest, NumBytes);
	Ar.SetIsPersistent(true);

	Ar << SerializePositionsAndNormals;
	
	int32 LayerDataCount = SerializeLayerData.Num();
	Ar << LayerDataCount;

	for (TArray<uint8>& CurrentLayerData : SerializeLayerData)
	{
		Ar << CurrentLayerData;
	}

	BulkData.Unlock();
}

bool FPCGLandscapeCacheEntry::SerializeFromBulkData() const
{
	check(!bDataLoaded);

	// If owner object has been unloaded, we can't serialize the bulk data.
	if (!OwningCache.IsValid())
	{
		return false;
	}

	LLM_SCOPE_BYNAME(TEXT("PCGLandscape"));

	// Note: this call is not threadsafe by itself, it is meant to be called from a locked region
	uint8* Data = nullptr;
	BulkData.GetCopy((void**)&Data);
	int32 DataSize = BulkData.GetBulkDataSize();

	if (!Data)
	{
		UE_LOGF(LogPCG, Error, "Unable to load Landscape Cache Entry bulk data");
		return false;
	}

	FBufferReader Ar(Data, DataSize, /*bInFreeOnClose=*/true, /*bIsPersistent=*/true);

	Ar << PositionsAndNormals;

	int32 LayerCount = 0;
	Ar << LayerCount;

	LayerData.SetNum(LayerCount);

	for (int32 LayerIndex = 0; LayerIndex < LayerCount; ++LayerIndex)
	{
		Ar << LayerData[LayerIndex];
	}

	bDataLoaded = true;
	return true;
}

void FPCGLandscapeCacheEntry::Serialize(FArchive& Archive, UObject* Owner, int32 Index, EPCGLandscapeCacheSerializationContents SerializeContents)
{
	LLM_SCOPE_BYTAG(PCG);

	// If the current bulk output data does not match the desired save data, make sure to load from the bulk data and update the bulked save data
	if(Archive.IsCooking() && SerializeContents != EPCGLandscapeCacheSerializationContents::SerializeAll && !bDataLoaded)
	{
		SerializeFromBulkData();
	}
	
	// Important implementation note:
	// If the serialization here or in the cache entries change, we still need to load data from previous versions.
	// While that's not really needed in non-editor builds, it is very much important when loading from the editor,
	// at least in the "AlwaysSerialize" case.
	if (bDataLoaded && Archive.IsSaving())
	{
		SerializeToBulkData(SerializeContents);
	}
	
	Archive << PointHalfSize;
	Archive << Stride;
	Archive << LayerDataNames;

	// We force it not inline that means bulk data won't automatically be loaded when we deserialize
	// later but only when we explicitly take action to load it
	BulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
	BulkData.Serialize(Archive, Owner, Index);
}

void UPCGLandscapeCache::BeginDestroy()
{
	ClearCache();
#if WITH_EDITOR
	TeardownLandscapeCallbacks();
#endif

	Super::BeginDestroy();
}

#if WITH_EDITOR
void UPCGLandscapeCache::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGLandscapeCache, SerializationMode))
	{
		// Reset error notification if the serialization mode changes.
		bLoggedNoCacheError = false;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

void UPCGLandscapeCache::Serialize(FArchive& Archive)
{
	LLM_SCOPE_BYTAG(PCG);
	Archive.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Archive.IsLoading() && Archive.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGLandscapeCacheDefaultSerializationChanged)
	{
		// Default value has changed from "serialize at cook" to "never serialize", so put back original default.
		SerializationMode = EPCGLandscapeCacheSerializationMode::SerializeOnlyAtCook;
	}

	Super::Serialize(Archive);

	const bool bShouldSerializeEntries = (Archive.IsSaving() &&
		(SerializationMode == EPCGLandscapeCacheSerializationMode::AlwaysSerialize || 
			(SerializationMode == EPCGLandscapeCacheSerializationMode::SerializeOnlyAtCook && Archive.IsCooking())));
	EPCGLandscapeCacheSerializationContents SerializedContents = Archive.IsCooking() ? CookedSerializedContents : EPCGLandscapeCacheSerializationContents::SerializeAll;
	// Important implementation note:
	// If the serialization here or in the cache entries change, we still need to load data from previous versions.
	// While that's not really needed in non-editor builds, it is very much important when loading from the editor,
	// at least in the "AlwaysSerialize" case.
	Archive.UsingCustomVersion(FPCGCustomVersion::GUID);

	int32 DataVersion = FPCGCustomVersion::LatestVersion;
	if (Archive.IsLoading())
	{
		DataVersion = Archive.CustomVer(FPCGCustomVersion::GUID);
	}

	// Serialize cache entries
	int32 NumEntries = ((Archive.IsLoading() || !bShouldSerializeEntries) ? 0 : CachedData.Num());
	Archive << NumEntries;

	if (Archive.IsLoading())
	{
		CachedData.Reserve(NumEntries);

		for (int32 EntryIndex = 0; EntryIndex < NumEntries; ++EntryIndex)
		{
			TPair<FGuid, FIntPoint> Key;
			Archive << Key;

			TSharedPtr<FPCGLandscapeCacheEntry> Entry = MakeShared<FPCGLandscapeCacheEntry>();
			Entry->OwningCache = this;
			Entry->Serialize(Archive, this, EntryIndex, SerializedContents);

			CacheMapKey MapKey(Key.Key, Key.Value, nullptr);
			CachedData.Add(MapKey, MoveTemp(Entry));
		}

#if WITH_EDITOR
		CacheEntryCount = CachedData.Num();
#endif
	}
	else if(bShouldSerializeEntries)
	{
		int32 EntryIndex = 0;
		for (auto& CacheEntry : CachedData)
		{
			TPair<FGuid, FIntPoint> Key(CacheEntry.Key.LandscapeGuid, CacheEntry.Key.Coordinate);
			Archive << Key;
			CacheEntry.Value->Serialize(Archive, this, EntryIndex++, SerializedContents);
		}
	}
}

void UPCGLandscapeCache::Initialize()
{
	if (!bInitialized && GetWorld())
	{
#if WITH_EDITOR
		SetupLandscapeCallbacks();
		CacheLayerNames();
#endif
		UpdateCacheWorldKeys();
		bInitialized = true;
	}

}

void UPCGLandscapeCache::UpdateCacheWorldKeys()
{
	check(IsInGameThread());

	AActor* HintActor = Cast<AActor>(GetOuter());
	if (!HintActor)
	{
		return;
	}

	TMap<CacheMapKey, TSharedPtr<FPCGLandscapeCacheEntry>> UpdatedCachedData;
	UpdatedCachedData.Reserve(CachedData.Num());

	for (const TPair<CacheMapKey, TSharedPtr<FPCGLandscapeCacheEntry>>& CacheEntry : CachedData)
	{
		CacheMapKey Key = CacheEntry.Key;
		if (Key.WorldKey == FObjectKey())
		{
			Key = CacheMapKey(Key.LandscapeGuid, Key.Coordinate, HintActor);
		}

		UpdatedCachedData.Add(Key, CacheEntry.Value);
	}

	CachedData = MoveTemp(UpdatedCachedData);
}

void UPCGLandscapeCache::Tick(float DeltaSeconds)
{
	Initialize();

#if !WITH_EDITOR
	// Important implementation note:
	// If the threshold is too low, it could lead to some issues - namely the check(bDataLoaded) in the cache entries
	// as we currently do not have a state in the landscape cache to know whether something is still under use.
	// It is possible to do so however, by adding a scoped variable + a counter, so we could safely remove these from
	// the cleanup process done below, but at this point in time it doesn't seem to be needed or likely,
	// since we do not keep (wrt time-slicing) the landscape cache entry pointers at any point in the PCG landscape data implementation
	TimeSinceLastCleanupInSeconds += DeltaSeconds;

	if (TimeSinceLastCleanupInSeconds >= CVarLandscapeCacheGCFrequency.GetValueOnAnyThread())
	{
		TimeSinceLastCleanupInSeconds = 0;

		const int32 MemoryThresholdInBytes = CVarLandscapeCacheSizeThreshold.GetValueOnAnyThread() * 1024 * 1024;

		if (CacheMemorySize > MemoryThresholdInBytes)
		{
			TArray<TSharedPtr<FPCGLandscapeCacheEntry>> LoadedEntries;
			for(TPair<CacheMapKey, TSharedPtr<FPCGLandscapeCacheEntry>>& CacheEntry : CachedData)
			{
				if (CacheEntry.Value->bDataLoaded)
				{
					LoadedEntries.Add(CacheEntry.Value);
				}
			}

			Algo::Sort(LoadedEntries, [](const TSharedPtr<FPCGLandscapeCacheEntry>& A, const TSharedPtr<FPCGLandscapeCacheEntry>& B)
			{
				return A->Touch < B->Touch;
			});

			// Go through the oldest data and free them until we're under the threshold
			int32 EntryIndex = 0;
			int32 HighestTouchUnloaded = 0;
			while(EntryIndex < LoadedEntries.Num() && CacheMemorySize > MemoryThresholdInBytes)
			{
				CacheMemorySize -= LoadedEntries[EntryIndex]->GetMemorySize();
				HighestTouchUnloaded = LoadedEntries[EntryIndex]->Touch;
				LoadedEntries[EntryIndex]->Unload();
				check(LoadedEntries[EntryIndex]->GetMemorySize()==0);
				++EntryIndex;
			}

			// Adjust touch on the other loaded entries so we can keep Touch bounded
			while (EntryIndex < LoadedEntries.Num())
			{
				LoadedEntries[EntryIndex++]->Touch -= HighestTouchUnloaded;
			}

			CacheTouch -= HighestTouchUnloaded;
		}
	}
#endif // !WITH_EDITOR
}

void UPCGLandscapeCache::PrimeCache()
{
#if WITH_EDITOR
	UWorld* World = GetWorld();

	if (!World)
	{
		return;
	}

	// First, gather all potential cached data to create, emplace a nullptr at these locations
	TArray<TTuple<ULandscapeInfo*, ULandscapeComponent*, CacheMapKey>> CacheEntriesToBuild;

	for (auto It = ULandscapeInfoMap::GetLandscapeInfoMap(World).Map.CreateIterator(); It; ++It)
	{
		ULandscapeInfo* LandscapeInfo = It.Value();
		if (IsValid(LandscapeInfo))
		{
			// Build per-component information
			LandscapeInfo->ForEachLandscapeProxy([this, LandscapeInfo, &CacheEntriesToBuild](ALandscapeProxy* LandscapeProxy)
			{
				check(LandscapeProxy);
				Landscapes.Add(LandscapeProxy);
				const FGuid LandscapeGuid = LandscapeProxy->GetOriginalLandscapeGuid();

				for (ULandscapeComponent* LandscapeComponent : LandscapeProxy->LandscapeComponents)
				{
					if (!LandscapeComponent)
					{
						continue;
					}

					CacheMapKey ComponentKey(LandscapeGuid, PCGLandscapeCache::GetCoordinates(LandscapeComponent), Cast<AActor>(GetOuter()));
					if (!CachedData.Contains(ComponentKey))
					{
						CacheEntriesToBuild.Emplace(LandscapeInfo, LandscapeComponent, ComponentKey);
					}
				}
				return true;
			});
		}
	}

	// Then create the landscape cache entries
	TArray<TSharedPtr<FPCGLandscapeCacheEntry>> NewEntries;
	NewEntries.SetNum(CacheEntriesToBuild.Num());

	ParallelFor(CacheEntriesToBuild.Num(), [this, &CacheEntriesToBuild, &NewEntries](int32 EntryIndex)
	{
		LLM_SCOPE_BYTAG(PCG);
		FTaskTagScope Scope(ETaskTag::EParallelGameThread);
		const TTuple<ULandscapeInfo*, ULandscapeComponent*, CacheMapKey>& CacheEntryInfo = CacheEntriesToBuild[EntryIndex];
		NewEntries[EntryIndex] = FPCGLandscapeCacheEntry::CreateCacheEntry(CacheEntryInfo.Get<0>(), CacheEntryInfo.Get<1>());
		if (NewEntries[EntryIndex])
		{
			NewEntries[EntryIndex]->OwningCache = this;
		}
	});

	if (SerializationMode != EPCGLandscapeCacheSerializationMode::NeverSerialize)
	{
		Modify();
	}

	// Finally, write them back to the cache
	for (int32 EntryIndex = 0; EntryIndex < CacheEntriesToBuild.Num(); ++EntryIndex)
	{
		if (NewEntries[EntryIndex])
		{
			CachedData.Add(CacheEntriesToBuild[EntryIndex].Get<2>(), MoveTemp(NewEntries[EntryIndex]));
		}
	}

	CacheEntryCount = CachedData.Num();
	CacheLayerNames();
#endif
}

void UPCGLandscapeCache::ClearCache()
{
	if (SerializationMode != EPCGLandscapeCacheSerializationMode::NeverSerialize)
	{
		Modify();
	}

	CachedData.Reset();
	CachedLayerNames.Reset();
#if WITH_EDITOR
	CacheEntryCount = 0;
#endif
}

void UPCGLandscapeCache::TakeOwnership(UPCGLandscapeCache* InLandscapeCache)
{
	if (SerializationMode != EPCGLandscapeCacheSerializationMode::NeverSerialize)
	{
		Modify(/*bAlwaysMarkDirty=*/false);
	}

	InLandscapeCache->UpdateCacheWorldKeys();
	bool bShouldDirty = false;

	for (TPair<CacheMapKey, TSharedPtr<FPCGLandscapeCacheEntry>>& CacheEntryPair : InLandscapeCache->CachedData)
	{
		if (CacheEntryPair.Value && !CachedData.Contains(CacheEntryPair.Key))
		{
			CacheEntryPair.Value->OwningCache = this;
			CachedData.Emplace(CacheEntryPair.Key, MoveTemp(CacheEntryPair.Value));
			CacheEntryPair.Value = TSharedPtr<FPCGLandscapeCacheEntry>{};
			bShouldDirty = true;
		}
	}

	for (FName LayerName : InLandscapeCache->CachedLayerNames)
	{
		bool bLayerWasAlreadyPresent = false;
		CachedLayerNames.Add(LayerName, &bLayerWasAlreadyPresent);
		bShouldDirty |= !bLayerWasAlreadyPresent;
	}

	InLandscapeCache->ClearCache();

#if WITH_EDITOR
	CacheEntryCount = CachedData.Num();
#endif

	if (bShouldDirty && SerializationMode != EPCGLandscapeCacheSerializationMode::NeverSerialize)
	{
		MarkPackageDirty();
	}
}

bool UPCGLandscapeCache::AreCacheEntriesReady(const ULandscapeInfo* InLandscapeInfo, const FIntPoint& InMinComponentKey, const FIntPoint& InMaxComponentKey)
{
	ALandscapeProxy* LandscapeProxy = InLandscapeInfo->GetLandscapeProxy();
	for (int32 ComponentKeyX = InMinComponentKey.X; ComponentKeyX <= InMaxComponentKey.X; ++ComponentKeyX)
	{
		for (int32 ComponentKeyY = InMinComponentKey.Y; ComponentKeyY <= InMaxComponentKey.Y; ++ComponentKeyY)
		{
			const FIntPoint ComponentKey(ComponentKeyX, ComponentKeyY);

			bool bOutIsValidEntry = false;
			const TSharedPtr<const FPCGLandscapeCacheEntry> CacheEntry = GetCacheEntryInternal(InLandscapeInfo, ComponentKey, LandscapeProxy, /*bAllowLoadOrCreate=*/false, bOutIsValidEntry);

			if (!bOutIsValidEntry)
			{
				continue;
			}

			// Entry not created yet or not loaded
			if (!CacheEntry || !CacheEntry->bDataLoaded)
			{
				return false;
			}
		}
	}

	return true;
}

void UPCGLandscapeCache::PrimeCache(const ULandscapeInfo* InLandscapeInfo, const FIntPoint& InMinComponentKey, const FIntPoint& InMaxComponentKey)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGLandscapeData::PrimeCache);
	for (int32 ComponentKeyX = InMinComponentKey.X; ComponentKeyX <= InMaxComponentKey.X; ++ComponentKeyX)
	{
		for (int32 ComponentKeyY = InMinComponentKey.Y; ComponentKeyY <= InMaxComponentKey.Y; ++ComponentKeyY)
		{
			GetCacheEntry(InLandscapeInfo, FIntPoint(ComponentKeyX, ComponentKeyY));
		}
	}
}

TSharedPtr<const FPCGLandscapeCacheEntry> UPCGLandscapeCache::GetCacheEntry(const ULandscapeInfo* InLandscapeInfo, const FIntPoint& InComponentKey, ALandscapeProxy* InLandscapeProxy, bool bInAllowLoadOrCreate)
{
	bool bOutIsValidEntry = false;

	if (!InLandscapeInfo)
	{
		return nullptr;
	}

	return GetCacheEntryInternal(InLandscapeInfo, InComponentKey, InLandscapeProxy, bInAllowLoadOrCreate, bOutIsValidEntry);
}

TSharedPtr<const FPCGLandscapeCacheEntry> UPCGLandscapeCache::GetCacheEntryInternal(const ULandscapeInfo* InLandscapeInfo, const FIntPoint& InComponentKey, ALandscapeProxy* InLandscapeProxy, bool bInAllowLoadOrCreate, bool& bOutIsValidEntry)
{
	// If entry can't exist this will be false (at runtime it means we don't have the data, in editor it means component key is invalid)
	bOutIsValidEntry = false;

#if WITH_EDITOR
	// ComponentKey exists so this entry should be valid but it might not exist yet
	return GetOrCreateCacheEntryInternal(InLandscapeInfo, InComponentKey, bInAllowLoadOrCreate, bOutIsValidEntry);
#else
	ALandscapeProxy* LandscapeProxy = InLandscapeProxy ? InLandscapeProxy : InLandscapeInfo->GetLandscapeProxy();
	const FGuid LandscapeGuid = LandscapeProxy ? LandscapeProxy->GetOriginalLandscapeGuid() : FGuid();
	TSharedPtr<const FPCGLandscapeCacheEntry> CacheEntry = GetCacheEntryInternal(LandscapeProxy, LandscapeGuid, InComponentKey, bInAllowLoadOrCreate);

	// Entry is valid if it exists (it might not be loaded yet)
	bOutIsValidEntry = CacheEntry.IsValid();

	return CacheEntry;
#endif
}


#if WITH_EDITOR
TSharedPtr<const FPCGLandscapeCacheEntry> UPCGLandscapeCache::GetOrCreateCacheEntryInternal(const ULandscapeInfo* InLandscapeInfo, const FIntPoint& InComponentKey, bool bInAllowLoadOrCreate, bool& bOutIsValidEntry)
{
	bOutIsValidEntry = false;
	
	ULandscapeComponent* LandscapeComponent = InLandscapeInfo->XYtoComponentMap.FindRef(InComponentKey);
	if (!LandscapeComponent)
	{
		return nullptr;
	}

	if (SerializationMode == EPCGLandscapeCacheSerializationMode::NeverSerialize && PCGHelpers::IsRuntimeOrPIE())
	{
		if (!bLoggedNoCacheError)
		{
			PCGLog::Landscape::LogLandscapeCacheNotAvailableError();
			bLoggedNoCacheError = true;
		}

		return nullptr;
	}

	// Entry is valid, it might not be created/loaded yet
	bOutIsValidEntry = true;

	const FGuid LandscapeGuid = (LandscapeComponent->GetLandscapeProxy() ? LandscapeComponent->GetLandscapeProxy()->GetOriginalLandscapeGuid() : FGuid());
	TSharedPtr<const FPCGLandscapeCacheEntry> CacheEntry = GetCacheEntryInternal(LandscapeComponent->GetLandscapeProxy(), LandscapeGuid, InComponentKey, bInAllowLoadOrCreate);
	if (CacheEntry || !bInAllowLoadOrCreate)
	{
		return CacheEntry;
	}
		
	PCG::TUniqueScopeLock WriteLock(CacheLock);
	CacheMapKey ComponentKey(LandscapeGuid, InComponentKey, LandscapeComponent->GetOwner());
	if (const TSharedPtr<FPCGLandscapeCacheEntry>* FoundEntry = CachedData.Find(ComponentKey))
	{
		CacheEntry = *FoundEntry;
	}
	else
	{
		check(LandscapeComponent->SectionBaseX / LandscapeComponent->ComponentSizeQuads == ComponentKey.Coordinate.X && LandscapeComponent->SectionBaseY / LandscapeComponent->ComponentSizeQuads == ComponentKey.Coordinate.Y);
		if (TSharedPtr<FPCGLandscapeCacheEntry> NewEntry = FPCGLandscapeCacheEntry::CreateCacheEntry(LandscapeComponent->GetLandscapeInfo(), LandscapeComponent))
		{
			NewEntry->OwningCache = this;
			CacheEntry = NewEntry;
			CachedData.Add(ComponentKey, NewEntry);
			++CacheEntryCount;
		}
	}

	if (CacheEntry)
	{
		if (CacheEntry->TouchAndLoad(CacheTouch++))
		{
			CacheMemorySize += CacheEntry->GetMemorySize();
		}
		else if(!CacheEntry->bDataLoaded)
		{
			CacheEntry = nullptr;
		}
	}

	return CacheEntry;
}
#endif

TSharedPtr<const FPCGLandscapeCacheEntry> UPCGLandscapeCache::GetCacheEntryInternal(AActor* HintActor, const FGuid& LandscapeGuid, const FIntPoint& ComponentCoordinate, bool bInAllowLoad)
{
	TSharedPtr<const FPCGLandscapeCacheEntry> CacheEntry = nullptr;
	CacheMapKey ComponentKey(LandscapeGuid, ComponentCoordinate, HintActor);

	{
#if WITH_EDITOR
		PCG::TSharedScopeLock ReadLock(CacheLock);
#else
		if (!bLoggedNoCacheError && CachedData.IsEmpty())
		{
			UE_LOGF(LogPCG, Error, "PCG Landscape cache is empty; it probably was never serialized. Check the landscape cache object on the PCG World Actor.");
			bLoggedNoCacheError = true;
		}
#endif
		if (const TSharedPtr<FPCGLandscapeCacheEntry>* FoundEntry = CachedData.Find(ComponentKey))
		{
			CacheEntry = *FoundEntry;
		}
	}

	if (CacheEntry && bInAllowLoad)
	{
		if (CacheEntry->TouchAndLoad(CacheTouch++))
		{
			CacheMemorySize += CacheEntry->GetMemorySize();
		}
		else if(!CacheEntry->bDataLoaded)
		{
			CacheEntry = nullptr;
		}
	}

	return CacheEntry;
}

TArray<FName> UPCGLandscapeCache::GetLayerNames(ALandscapeProxy* Landscape)
{
#if WITH_EDITOR
	PCG::TSharedScopeLock ReadLock(CacheLock);
#endif
	return CachedLayerNames.Array();
}

void UPCGLandscapeCache::SampleMetadataOnPoint(ALandscapeProxy* Landscape, const FTransform& InTransform, int64& OutMetadataEntry, UPCGMetadata* OutMetadata)
{
	if (!Landscape || !Landscape->GetWorld() || !OutMetadata || CachedLayerNames.IsEmpty())
	{
		return;
	}

	ULandscapeInfo* LandscapeInfo = Landscape->GetLandscapeInfo();
	if (!LandscapeInfo)
	{
		UE_LOGF(LogPCG, Warning, "Unable to get landscape layer weights because the landscape info is not available (landscape not registered yet?");
		return;
	}

	const FTransform LandscapeTransform = Landscape->LandscapeActorToWorld();
	const FVector LocalPoint = LandscapeTransform.InverseTransformPosition(InTransform.GetLocation());
	const FIntPoint ComponentMapKey(FMath::FloorToInt(LocalPoint.X / LandscapeInfo->ComponentSizeQuads), FMath::FloorToInt(LocalPoint.Y / LandscapeInfo->ComponentSizeQuads));

	TSharedPtr<const FPCGLandscapeCacheEntry> CacheEntry = GetCacheEntry(LandscapeInfo, ComponentMapKey, Landscape);

	if (!CacheEntry || CacheEntry->LayerData.IsEmpty())
	{
		return;
	}

	const FVector2D ComponentLocalPoint(LocalPoint.X - ComponentMapKey.X * LandscapeInfo->ComponentSizeQuads, LocalPoint.Y - ComponentMapKey.Y * LandscapeInfo->ComponentSizeQuads);
	CacheEntry->GetInterpolatedPointMetadataOnly(ComponentLocalPoint, OutMetadataEntry, OutMetadata);
}

#if WITH_EDITOR
void UPCGLandscapeCache::SetupLandscapeCallbacks()
{
	// Remove previous callbacks, if any
	TeardownLandscapeCallbacks();

	UWorld* World = GetWorld();
	if (!World)
	{
		return;
	}

	// Gather landspace actors
	TArray<AActor*> FoundLandscapes;
	UGameplayStatics::GetAllActorsOfClass(World, ALandscapeProxy::StaticClass(), FoundLandscapes);

	for (AActor* FoundLandscape : FoundLandscapes)
	{
		ALandscapeProxy* Landscape = CastChecked<ALandscapeProxy>(FoundLandscape);

		Landscapes.Add(Landscape);
	}

	if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
	{
		LandscapeSubsystem->OnLandscapeProxyComponentDataChanged().AddUObject(this, &UPCGLandscapeCache::OnLandscapeChanged);
	}

	// Also track when the landscape is moved, added, or deleted
	if (GEngine)
	{
		GEngine->OnActorMoved().AddUObject(this, &UPCGLandscapeCache::OnLandscapeMoved);
		GEngine->OnLevelActorAdded().AddUObject(this, &UPCGLandscapeCache::OnLandscapeAdded);
		GEngine->OnLevelActorDeleted().AddUObject(this, &UPCGLandscapeCache::OnLandscapeDeleted);
	}

	// In editor, loading from the persistent level should add it to the cache. Note that we don't need to track unloaded landscapes
	if (!World->IsPlayInEditor() && World->IsPartitionedWorld() && World->PersistentLevel)
	{
		World->PersistentLevel->OnLoadedActorAddedToLevelEvent.AddUObject(this, &UPCGLandscapeCache::OnLandscapeLoaded);
	}
}

void UPCGLandscapeCache::TeardownLandscapeCallbacks()
{
	if (GEngine)
	{
		GEngine->OnActorMoved().RemoveAll(this);
		GEngine->OnLevelActorAdded().RemoveAll(this);
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}

	if (UWorld* World = GetWorld())
	{
	    if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
	    {
		    LandscapeSubsystem->OnLandscapeProxyComponentDataChanged().RemoveAll(this);
	    }

		if (!World->IsPlayInEditor() && World->IsPartitionedWorld() && World->PersistentLevel)
		{
			World->PersistentLevel->OnLoadedActorAddedToLevelEvent.RemoveAll(this);
		}
	}
}

void UPCGLandscapeCache::OnLandscapeChanged(ALandscapeProxy* InLandscape, const FLandscapeProxyComponentDataChangedParams& InChangeParams)
{
	if (!Landscapes.Contains(InLandscape))
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(CacheLock);

	// Just remove these from the cache, they'll be added back on demand
	InChangeParams.ForEachComponent([this, InLandscape](const ULandscapeComponent* LandscapeComponent)
	{
		if (LandscapeComponent)
		{
			const CacheMapKey ComponentKey(InLandscape->GetOriginalLandscapeGuid(), PCGLandscapeCache::GetCoordinates(LandscapeComponent), InLandscape);

			if (CachedData.Remove(ComponentKey) > 0)
			{
				--CacheEntryCount;
			}
		}
	});

	CacheLayerNames(InLandscape);
}

void UPCGLandscapeCache::OnLandscapeMoved(AActor* InActor)
{
	ALandscapeProxy* Landscape = Cast<ALandscapeProxy>(InActor);
	if (!Landscape || !Landscapes.Contains(Landscape))
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(CacheLock);

	// Just remove these from the cache, they'll be added back on demand
	RemoveComponentFromCache_Unsafe(Landscape);

	CacheLayerNames(Landscape);
}

void UPCGLandscapeCache::OnLandscapeDeleted(AActor* Actor)
{
	ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(Actor);
	if (!LandscapeProxy || !Landscapes.Contains(LandscapeProxy))
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(CacheLock);

	RemoveComponentFromCache_Unsafe(LandscapeProxy);

	Landscapes.Remove(LandscapeProxy);
}

void UPCGLandscapeCache::OnLandscapeLoaded(AActor& Actor)
{
	OnLandscapeAdded(&Actor);
}

void UPCGLandscapeCache::OnLandscapeAdded(AActor* Actor)
{
	if (ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(Actor))
	{
		Landscapes.Add(LandscapeProxy);

		// Note: Landscape Proxies have no components at this stage, so they will need to be added on demand
	}
}

void UPCGLandscapeCache::CacheLayerNames()
{
	CachedLayerNames.Reset();

	for (TWeakObjectPtr<ALandscapeProxy> Landscape : Landscapes)
	{
		if (Landscape.Get())
		{
			CacheLayerNames(Landscape.Get());
		}
	}
}

void UPCGLandscapeCache::RemoveComponentFromCache_Unsafe(const ALandscapeProxy* LandscapeProxy)
{
	check(LandscapeProxy);
	LandscapeProxy->ForEachComponent<ULandscapeComponent>(/*bIncludeFromChildActors=*/false, [this, LandscapeProxy](const ULandscapeComponent* LandscapeComponent)
	{
		if (LandscapeComponent)
		{
			const CacheMapKey ComponentKey(LandscapeProxy->GetOriginalLandscapeGuid(), PCGLandscapeCache::GetCoordinates(LandscapeComponent), LandscapeProxy);
			if (CachedData.Remove(ComponentKey) > 0)
			{
				--CacheEntryCount;
			}
		}
	});
}

void UPCGLandscapeCache::CacheLayerNames(ALandscapeProxy* InLandscape)
{
	check(InLandscape);

	if (ULandscapeInfo* LandscapeInfo = InLandscape->GetLandscapeInfo())
	{
		for (const FLandscapeInfoLayerSettings& Layer : LandscapeInfo->Layers)
		{
			ULandscapeLayerInfoObject* LayerInfo = Layer.LayerInfoObj;
			if (!LayerInfo)
			{
				continue;
			}

			CachedLayerNames.Add(Layer.LayerName);
		}
	}
}

#endif // WITH_EDITOR