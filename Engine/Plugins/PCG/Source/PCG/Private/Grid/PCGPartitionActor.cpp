// Copyright Epic Games, Inc. All Rights Reserved.

#include "Grid/PCGPartitionActor.h"

#include "PCGComponent.h"
#include "PCGModule.h"
#include "PCGSubsystem.h"
#include "PCGWorldActor.h"
#include "Editor/IPCGEditorModule.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"

#include "Components/BoxComponent.h"
#include "Engine/World.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#if WITH_EDITOR
#include "Engine/Level.h"
#include "Grid/PCGPartitionActorDesc.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPartitionActor)

constexpr uint32 InvalidPCGGridSizeValue = 0u;

APCGPartitionActor::APCGPartitionActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PCGGridSize = InvalidPCGGridSizeValue;

#if WITH_EDITORONLY_DATA
	bActorLabelEditable = false;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	// Setup bounds component
	BoundsComponent = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UBoxComponent>(this, TEXT("BoundsComponent"));
	if (BoundsComponent)
	{
		BoundsComponent->SetCollisionObjectType(ECC_WorldStatic);
		BoundsComponent->SetCollisionResponseToAllChannels(ECR_Ignore);
		BoundsComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		BoundsComponent->SetGenerateOverlapEvents(false);
		BoundsComponent->SetupAttachment(GetRootComponent());
		BoundsComponent->bDrawOnlyIfSelected = true;
	}
#endif // WITH_EDITOR
}

void APCGPartitionActor::PostLoad()
{
	Super::PostLoad();

	// If the grid size is not set, set it to the default value.
	if (PCGGridSize == InvalidPCGGridSizeValue)
	{
		PCGGridSize = APCGWorldActor::DefaultPartitionGridSize;
	}

#if WITH_EDITOR
	if (GetGridSize() != PCGGridSize)
	{
		SetGridSize(PCGGridSize);
	}
#endif

#if WITH_EDITORONLY_DATA
	// Prior to this version bUse2DGrid was dependant of the PCGWorldActor so make sure we update it one last time upon registration
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGGridDescriptor)
	{
		bRequiresUse2DGridFixup = true;
	}
#endif

	// Safe guard if we ever load a local that was deleted but not removed (like if the user deleted themselves the component)
	// We can have multiple nullptr entries (if multiple components were removed), hence the while.
	while (LocalToOriginal.Contains(nullptr))
	{
		LocalToOriginal.Remove(nullptr);
	}

	// Mark all our local components as local
	for (UPCGComponent* LocalComponent : GetAllLocalPCGComponents())
	{
		LocalComponent->MarkAsLocalComponent();
		LocalComponent->SetGenerationGridSize(PCGGridSize);
		LocalComponent->ConditionalPostLoad();
	}

	bWasPostCreatedLoaded = true;
}

void APCGPartitionActor::BeginDestroy()
{
	UnregisterPCG();

	Super::BeginDestroy();
}

void APCGPartitionActor::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYTAG(PCG);

#if WITH_EDITOR
	TMap<TObjectPtr<UPCGComponent>, TSoftObjectPtr<UPCGComponent>> LocalToOriginalCopy;
	TMap<TObjectPtr<UPCGComponent>, TSoftObjectPtr<UPCGComponent>> Transients;
	TSet<TObjectPtr<UPCGComponent>> AddedLoadedPreviewKeys;

	if (Ar.IsSaving())
	{
		// Split out the map into persistent and transient components. Keep the persistent ones in LocalToOriginal
		// for serialization purposes (saving, linker fixup of SOPs).
		LocalToOriginalCopy = MoveTemp(LocalToOriginal);

		LocalToOriginal.Reset();
		for (const auto& It : LocalToOriginalCopy)
		{
			if (!ensure(It.Key))
			{
				continue;
			}

			if (!It.Key->HasAnyFlags(RF_Transient))
			{
				LocalToOriginal.Add(It);
			}
			else
			{
				Transients.Add(It);
			}
		}

		// For components that were cleared but marked as load-as-preview
		// we need to keep those around only if the original is still in its original state,
		// i.e. the serialization mode is load on preview and the current mode is preview.
		for (const auto& It : LoadedPreviewComponents)
		{
			if (UPCGComponent* OriginalComponent = It.Value.Get())
			{
				if (OriginalComponent->GetSerializedEditingMode() == EPCGEditorDirtyMode::LoadAsPreview && OriginalComponent->GetEditingMode() == EPCGEditorDirtyMode::Preview)
				{
					if (ensure(!LocalToOriginal.Contains(It.Key)))
					{
						LocalToOriginal.Add(It);
						AddedLoadedPreviewKeys.Add(It.Key);
					}
				}
			}
		}
	}
#endif

	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.IsSaving())
	{
		// Remove added loaded preview components.
		for (const TObjectPtr<UPCGComponent>& Key : AddedLoadedPreviewKeys)
		{
			LocalToOriginal.Remove(Key);
		}

		// Restore removed transient components.
		for (const auto& It : Transients)
		{
			LocalToOriginal.Add(It);
		}
	}
#endif
}

void APCGPartitionActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

#if WITH_EDITOR
	UpdateBoundsComponentExtents();
#endif // WITH_EDITOR

	RebuildOriginalToLocal();

	// Make the Partition actor register itself to the PCG Subsystem. RuntimeGen components should only register in the PostCreation path.
	if (PCGGridSize != InvalidPCGGridSizeValue && !IsRuntimeGenerated())
	{
		RegisterPCG();
	}
}

void APCGPartitionActor::RebuildOriginalToLocal()
{
	// Reset the OriginalToLocal and build it from the local map
	OriginalToLocal.Reset();
	OriginalToLocal.Reserve(LocalToOriginal.Num());
	for (const auto& It : LocalToOriginal)
	{
		if (UPCGComponent* OriginalComponent = It.Value.Get())
		{
			OriginalToLocal.Add(OriginalComponent, It.Key);
		}
	}
}

void APCGPartitionActor::PostUnregisterAllComponents()
{
	UnregisterPCG();

	Super::PostUnregisterAllComponents();
}

void APCGPartitionActor::BeginPlay()
{
	// bIsRuntimeGenerated is not set yet, so we also need to check if the PA is transient.
	if (!IsRuntimeGenerated() && !HasAnyFlags(RF_Transient))
	{
		// Pass through all the pcg components, to verify if we need to generate them.
		for (auto& It : OriginalToLocal)
		{
			UPCGComponent* OriginalComponent = It.Key.ResolveObjectPtr();
			UPCGComponent* LocalComponent = It.Value;
			if (OriginalComponent && LocalComponent)
			{
				// If OriginalComponent was already generated when loading then our LocalComponent should be in the same situation, we want to avoid generating and instead warn.
				if (OriginalComponent->GenerationTrigger == EPCGComponentGenerationTrigger::GenerateOnDemand && OriginalComponent->IsGeneratedOffline())
				{
					LocalComponent->GenerationTrigger = OriginalComponent->GenerationTrigger;
					UE_CLOGF(!LocalComponent->bGenerated, LogPCG, Warning, "PartitionActor '%ls' has a Local component '%ls' which should have been generated offline", *GetName(), *LocalComponent->GetName());
				}
				// If we have an original component that is generated (or generating), this one is automatically generated => GenerateOnLoad.
				// But if its runtime generated then it's handled by the runtime generation scheduler.
				else if ((OriginalComponent->bGenerated || OriginalComponent->IsGenerating()) && ensure(!OriginalComponent->IsManagedByRuntimeGenSystem())) 
				{
					LocalComponent->GenerationTrigger = EPCGComponentGenerationTrigger::GenerateOnLoad;
				}
				// Otherwise, make them match
				else
				{
					LocalComponent->GenerationTrigger = OriginalComponent->GenerationTrigger;
				}
			}
		}
	}

	Super::BeginPlay();
}

void APCGPartitionActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	// It is possible for a PA to get called EndPlay before its original volume actor so make sure to cleanup properly here
	if (IsRuntimeGenerated())
	{
		TMap<TObjectKey<UPCGComponent>, TObjectPtr<UPCGComponent>> OriginalToLocalCopy(OriginalToLocal);
		for (const TTuple<TObjectKey<UPCGComponent>, TObjectPtr<UPCGComponent>>& OriginalToLocalItem : OriginalToLocalCopy)
		{
			RemoveGraphInstance(OriginalToLocalItem.Key.ResolveObjectPtrEvenIfGarbage());
		}
	}

	UnregisterPCG();

	Super::EndPlay(EndPlayReason);
}

#if WITH_EDITOR
uint32 APCGPartitionActor::GetDefaultGridSize(UWorld* InWorld) const
{
	if (APCGWorldActor* PCGActor = PCGHelpers::GetPCGWorldActor(InWorld))
	{
		return PCGActor->PartitionGridSize;
	}

	UE_LOGF(LogPCG, Error, "[APCGPartitionActor::InternalGetDefaultGridSize] PCG World Actor was null. Returning default value");
	return APCGWorldActor::DefaultPartitionGridSize;
}

TUniquePtr<class FWorldPartitionActorDesc> APCGPartitionActor::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FPCGPartitionActorDesc());
}

bool APCGPartitionActor::IsUserManaged() const
{
	// Allows actor to be deleted
	if (IsInvalidForPCG())
	{
		return true;
	}

	return Super::IsUserManaged();
}

void APCGPartitionActor::UpdateUse2DGridIfNeeded(bool bInUse2DGrid)
{
	if (bRequiresUse2DGridFixup)
	{
		bUse2DGrid = bInUse2DGrid;
		bRequiresUse2DGridFixup = false;
	}
}

void APCGPartitionActor::SetInvalidForPCG()
{
	if (!bIsInvalidForPCG)
	{
		bIsInvalidForPCG = true;
		SetActorLabel(TEXT("TO_DELETE_") + GetActorLabel());
	}
}

TSoftObjectPtr<UPCGComponent> APCGPartitionActor::GetOriginalComponentSoftObjectPtr(UPCGComponent* LocalComponent) const
{
	const TSoftObjectPtr<UPCGComponent>* OriginalComponent = LocalToOriginal.Find(LocalComponent);
	return OriginalComponent ? *OriginalComponent : TSoftObjectPtr<UPCGComponent>();
}
#endif

FBox APCGPartitionActor::GetFixedBounds() const
{
	const FVector Center = GetActorLocation();
	const FVector::FReal HalfGridSize = PCGGridSize / 2.0;

	FVector Extent(HalfGridSize, HalfGridSize, HalfGridSize);

	// In case of 2D grid, it's like the actor has infinite bounds on the Z axis
	if (bUse2DGrid)
	{
		Extent.Z = HALF_WORLD_MAX1;
	}

	return FBox(Center - Extent, Center + Extent);
}

FIntVector APCGPartitionActor::GetGridCoord() const
{
	const FVector Center = GetActorLocation();
	return UPCGActorHelpers::GetCellCoord(Center, PCGGridSize, bUse2DGrid);
}

FPCGGridDescriptor APCGPartitionActor::GetGridDescriptor() const
{
	FPCGGridDescriptor GridDescriptor = FPCGGridDescriptor()
		.SetGridSize(GetPCGGridSize())
		.SetIs2DGrid(bUse2DGrid)
		.SetIsRuntime(IsRuntimeGenerated());

	// Runtime generated Partition Actors should never have any Data Layer Assets / HLOD Layer assigned as they are spawned at runtime
	// but skip this code anyways.
	if (!IsRuntimeGenerated())
	{
#if WITH_EDITORONLY_DATA
		if (GetWorld() && GetWorld()->IsPlayInEditor())
		{
			GridDescriptor.SetRuntimeHash(RuntimeGridDescriptorHash);
		}
		else
		{
			GridDescriptor.SetDataLayerAssets(GetDataLayerAssets());
			GridDescriptor.SetHLODLayer(GetHLODLayer());
		}
#else
		GridDescriptor.SetRuntimeHash(RuntimeGridDescriptorHash);
#endif
	}

	return GridDescriptor;
}

bool APCGPartitionActor::Teleport(const FVector& NewLocation)
{
	// We should not be teleporting a PA that is in use. We only teleport empty RuntimeGen PAs to their grid cell before initialization.
	ensureMsgf(bIsRuntimeGenerated, TEXT("Teleporting a non-RuntimeGenerated partition actor '%s'."), *GetActorNameOrLabel());
	ensureMsgf(LocalToOriginal.IsEmpty(), TEXT("Teleporting partition actor '%s' with %d entries in LocalToOriginal."), *GetActorNameOrLabel(), LocalToOriginal.Num());
	ensureMsgf(OriginalToLocal.IsEmpty(), TEXT("Teleporting partition actor '%s' with %d entries in OriginalToLocal (stale derived map)."), *GetActorNameOrLabel(), OriginalToLocal.Num());

	RootComponent->SetMobility(EComponentMobility::Movable);
	bool bResult = Super::SetActorLocation(NewLocation, /*bSweep=*/false, nullptr, ETeleportType::TeleportPhysics);
	RootComponent->SetMobility(EComponentMobility::Static);

	return bResult;
}

void APCGPartitionActor::RegisterPCG()
{
	if (UPCGSubsystem* Subsystem = GetSubsystem())
	{
		if (!bIsRegistered)
		{
			Subsystem->RegisterPartitionActor(this);
			bIsRegistered = true;
		}
	}
}

void APCGPartitionActor::UnregisterPCG()
{
	if (UPCGSubsystem* Subsystem = GetSubsystem())
	{
		if (bIsRegistered)
		{
			Subsystem->UnregisterPartitionActor(this);
			bIsRegistered = false;
		}
	}
}

void APCGPartitionActor::GetActorBounds(bool bOnlyCollidingComponents, FVector& Origin, FVector& BoxExtent, bool bIncludeFromChildActors) const
{
	Super::GetActorBounds(bOnlyCollidingComponents, Origin, BoxExtent, bIncludeFromChildActors);

	// To keep consistency with the other GetBounds functions, transform our result into an origin / extent formatting
	FBox Bounds(Origin - BoxExtent, Origin + BoxExtent);
	Bounds += GetFixedBounds();
	Bounds.GetCenterAndExtents(Origin, BoxExtent);
}

UPCGComponent* APCGPartitionActor::GetLocalComponent(const UPCGComponent* OriginalComponent) const
{
	return GetLocalComponent(OriginalComponent, /*bRebuildMappingOnNullEntries=*/true);
}

UPCGComponent* APCGPartitionActor::GetLocalComponent(const UPCGComponent* OriginalComponent, bool bRebuildMappingOnNullEntries) const
{
	if (const TObjectPtr<UPCGComponent>* LocalComponent = OriginalToLocal.Find(OriginalComponent))
	{
		return *LocalComponent;
	}
	else if (bRebuildMappingOnNullEntries)
	{
		bool bNeedsRebuild = OriginalToLocal.Num() != LocalToOriginal.Num();
		if (!bNeedsRebuild)
		{
			for (const auto& OriginalToLocalItem : OriginalToLocal)
			{
				if (!OriginalToLocalItem.Key.ResolveObjectPtr())
				{
					bNeedsRebuild = true;
					break;
				}
			}
		}
		
		if(bNeedsRebuild)
		{
			const_cast<APCGPartitionActor*>(this)->RebuildOriginalToLocal();
			return GetLocalComponent(OriginalComponent, /*bRebuildMappingOnNullEntries=*/false);
		}
	}

	return nullptr;
}

UPCGComponent* APCGPartitionActor::GetOriginalComponent(const UPCGComponent* LocalComponent) const
{
	const TSoftObjectPtr<UPCGComponent>* OriginalComponent = LocalToOriginal.Find(LocalComponent);
	return OriginalComponent ? (*OriginalComponent).Get() : nullptr;
}

// deprecated
void APCGPartitionActor::CleanupDeadGraphInstances(bool bRemoveNonNullOnly)
{
	CleanupDeadGraphInstancesInternal();
}

bool APCGPartitionActor::CleanupDeadGraphInstancesInternal(bool bCleanupTest, TArray<UPCGComponent*>* OutRemovedLocalComponents)
{
	bool bModified = false;

	// First find if we have any local dead instance (= nullptr) hooked to an original component.
	TSet<TObjectKey<UPCGComponent>> DeadOriginalInstances;
	for (const auto& OriginalToLocalItem : OriginalToLocal)
	{
		if (!OriginalToLocalItem.Value)
		{
			DeadOriginalInstances.Add(OriginalToLocalItem.Key);
		}
	}

	if (!DeadOriginalInstances.IsEmpty())
	{
		bModified = true;
		
		if (!bCleanupTest)
		{
			Modify();

			for (const TObjectKey<UPCGComponent>& DeadInstance : DeadOriginalInstances)
			{
				OriginalToLocal.Remove(DeadInstance);
			}

			LocalToOriginal.Remove(nullptr);
		}
	}

	// And do the same with dead original ones.
	TSet<TObjectPtr<UPCGComponent>> DeadLocalInstances;
	for (const auto& LocalToOriginalItem : LocalToOriginal)
	{
		if(IsValid(LocalToOriginalItem.Key) && !LocalToOriginalItem.Value.IsValid())
		{
			DeadLocalInstances.Add(LocalToOriginalItem.Key);
		}
	}

	if (!DeadLocalInstances.IsEmpty())
	{
		bModified = true;

		if (bCleanupTest)
		{
			if (OutRemovedLocalComponents)
			{
				OutRemovedLocalComponents->Append(DeadLocalInstances.Array());
			}

			return true;
		}

		Modify();

		for (const TObjectPtr<UPCGComponent>& DeadInstance : DeadLocalInstances)
		{
			LocalToOriginal.Remove(DeadInstance);

			if (DeadInstance)
			{
				DeadInstance->CleanupLocalImmediate(/*bRemoveComponents=*/true);
				DeadInstance->DestroyComponent();
			}
		}

		// Remove all dead entries.
		for (auto It = OriginalToLocal.CreateIterator(); It; ++It)
		{
			UPCGComponent* ResolvedKey = It.Key().ResolveObjectPtr();
			if (!ResolvedKey || !IsValid(ResolvedKey))
			{
				It.RemoveCurrent();
			}
		}
	}

	return bModified;
}

void APCGPartitionActor::AddGraphInstance(UPCGComponent* OriginalComponent)
{
	if (!OriginalComponent)
	{
		return;
	}

	// Make sure we don't have that graph twice;
	// Here we'll check if there has been some changes worth propagating or not
	UPCGComponent* LocalComponent = GetLocalComponent(OriginalComponent);

	if (LocalComponent)
	{
		// Update properties as needed and early out
		LocalComponent->SetEditingMode(OriginalComponent->GetEditingMode(), OriginalComponent->GetSerializedEditingMode());
		LocalComponent->SetPropertiesFromOriginal(OriginalComponent);
		LocalComponent->MarkAsLocalComponent();
		LocalComponent->SetGenerationGridSize(PCGGridSize);
		return;
	}

	Modify(!OriginalComponent->IsInPreviewMode());

	// Create a new local component
	LocalComponent = NewObject<UPCGComponent>(this, NAME_None, OriginalComponent->IsInPreviewMode() ? RF_Transient | RF_NonPIEDuplicateTransient : RF_NoFlags);
	LocalComponent->MarkAsLocalComponent();
	LocalComponent->SetGenerationGridSize(PCGGridSize);

	// Note: we'll place the local component prior to the SetPropertiesFromOriginal so that any code that relies on the parent-child relationship works here
	OriginalToLocal.Emplace(OriginalComponent, LocalComponent);
	LocalToOriginal.Emplace(LocalComponent, OriginalComponent);

	// Implementation note: since this is a new component, we need to use the current editing mode only for both the current & serialized editing modes
	LocalComponent->SetEditingMode(/*InEditingMode=*/OriginalComponent->GetEditingMode(), /*InSerializedEditingMode=*/OriginalComponent->GetEditingMode());
	LocalComponent->SetPropertiesFromOriginal(OriginalComponent);

	LocalComponent->RegisterComponent();

	// TODO: check if we should use a non-instanced component?
	AddInstanceComponent(LocalComponent);
}

void APCGPartitionActor::RemapGraphInstance(const UPCGComponent* OldOriginalComponent, UPCGComponent* NewOriginalComponent)
{
	check(OldOriginalComponent && NewOriginalComponent);

	UPCGComponent* LocalComponent = GetLocalComponent(OldOriginalComponent);

	if (!LocalComponent)
	{
		return;
	}

	// If the old original component was loaded, we can assume we are in a loading phase.
	// In this case, we don't want to dirty or register a transaction
	const bool bIsLoading = OldOriginalComponent->HasAnyFlags(RF_WasLoaded);

	if (!bIsLoading)
	{
		Modify(!LocalComponent->IsInPreviewMode());
	}

	OriginalToLocal.Remove(OldOriginalComponent);
	LocalToOriginal.Remove(LocalComponent);

	LocalComponent->SetEditingMode(NewOriginalComponent->GetEditingMode(), NewOriginalComponent->GetSerializedEditingMode());
	LocalComponent->SetPropertiesFromOriginal(NewOriginalComponent);
	OriginalToLocal.Emplace(NewOriginalComponent, LocalComponent);
	LocalToOriginal.Emplace(LocalComponent, NewOriginalComponent);

#if WITH_EDITOR
	// When changing original data, it means that the data we have might point to newly stale data, hence we need to force dirty here
	if (!bIsLoading)
	{
		LocalComponent->DirtyGenerated(EPCGComponentDirtyFlag::Actor);
	}
#endif
}

void APCGPartitionActor::RemapLocalComponent(UPCGComponent* OldLocalComponent, UPCGComponent* NewLocalComponent)
{
	check(OldLocalComponent && NewLocalComponent);

	// LocalToOriginal is keyed by the local component, so a reinstanced local leaves a stale key.
	// Repoint it to the new local and rebuild the derived OriginalToLocal map.
	TSoftObjectPtr<UPCGComponent> OriginalComponent;
	if (!LocalToOriginal.RemoveAndCopyValue(OldLocalComponent, OriginalComponent))
	{
		return;
	}

	LocalToOriginal.Emplace(NewLocalComponent, OriginalComponent);
	RebuildOriginalToLocal();
}

bool APCGPartitionActor::RemoveGraphInstance(UPCGComponent* OriginalComponent)
{
	UPCGComponent* LocalComponent = GetLocalComponent(OriginalComponent);

	if (!LocalComponent)
	{
		// If we don't have a local component, perhaps the original component is already dead,
		// so do some clean up
		CleanupDeadGraphInstancesInternal();
		return false;
	}

#if !WITH_EDITOR
	// When in a cooked build, we have to make sure that we are not removing the graph instance if we were generated offline.
	// Otherwise we can end up in a situation where the original component gets unloaded and the PA is not (because original's bounds are smaller).
	// This would trigger a cleanup, the PA would lose its generated resources, but would still be alive. When the original gets streamed in again,
	// we won't try to regenerate (and throw a warning saying that we have a mismatch between the generated flag on the original and local)
	// NOTE: This is only in a cooked build and not in PIE, because in PIE it is less clear about what should happen when the original dies and the
	// PA does not. Technically, we support tracking and so if the original changes we should update the PAs. This is a flaw in our flow where we don't
	// make the difference between "generated in editor and immutable" and "generated in editor but can change".
	// There is also another check when unregistering PCG Component to know if it is an unload unregister, that happens only in Editor, which should
	// cover the exact case of the PCG Component being streamed out. But deleting the original PCG Component will cleanup the PAs.
	if (OriginalComponent && !OriginalComponent->IsManagedByRuntimeGenSystem() && OriginalComponent->IsGeneratedOffline())
	{
		const AActor* OriginalActor = OriginalComponent->GetOwner();
		UE_LOGF(LogPCG, Log, "Original component on actor '%ls' got unloaded but Partition actor '%ls' will not cleanup since it was generated offline. "
			"If it needs to be cleaned up, call Cleanup on the original component first.", OriginalActor ? *OriginalActor->GetName() : TEXT("UNKNOWN"), *GetName());
		return false;
	}
#endif // !WITH_EDITOR

	Modify(!LocalComponent->IsInPreviewMode());
	OriginalToLocal.Remove(OriginalComponent);
	LocalToOriginal.Remove(LocalComponent);

	// TODO Add option to not cleanup?
	LocalComponent->CleanupLocalImmediate(/*bRemoveComponents=*/true);

	// If the component is tagged as "preview-on-load" we shouldn't actually remove it, otherwise we'll cause a change on the actor.
	if (LocalComponent->GetEditingMode() == EPCGEditorDirtyMode::Preview && LocalComponent->GetSerializedEditingMode() == EPCGEditorDirtyMode::LoadAsPreview)
	{
		LocalComponent->UnregisterComponent();
		LoadedPreviewComponents.Emplace(LocalComponent, OriginalComponent);
	}
	else
	{
		LocalComponent->DestroyComponent();
	}

	return OriginalToLocal.IsEmpty() && LoadedPreviewComponents.IsEmpty();
}

void APCGPartitionActor::RemoveLocalComponent(UPCGComponent* LocalComponent)
{
	if (!LocalComponent)
	{
		return;
	}

	UPCGComponent* OriginalComponent = GetOriginalComponent(LocalComponent);

	LocalToOriginal.Remove(LocalComponent);
	OriginalToLocal.Remove(OriginalComponent);
}

#if WITH_EDITOR
AActor* APCGPartitionActor::GetSceneOutlinerParent() const
{
	APCGWorldActor* PCGWorldActor = PCGHelpers::FindPCGWorldActor(GetWorld());
	ULevel* Level = GetLevel();
	
	if (Level && Level->IsPersistentLevel() && PCGWorldActor)
	{
		return PCGWorldActor;
	}
	else
	{
		return Super::GetSceneOutlinerParent();
	}	
}

bool APCGPartitionActor::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty && InProperty->GetFName() == AActor::GetHLODLayerPropertyName())
	{
		return false;
	}

	return true;
}

bool APCGPartitionActor::IsSelectable() const
{
	return IPCGEditorModule::Get() ? IPCGEditorModule::Get()->CanSelectPartitionActors() : true;
}
#endif // WITH_EDITOR

void APCGPartitionActor::PostCreation(const FPCGGridDescriptor& GridDescriptor)
{
	PCGGridSize = GridDescriptor.GetGridSize();
	bUse2DGrid = GridDescriptor.Is2DGrid();

#if WITH_EDITOR
	if (!GetWorld()->IsGameWorld() && !GridDescriptor.IsRuntime())
	{
		// Fetch Non External Assets and assign
		TArray<TSoftObjectPtr<UDataLayerAsset>> DescriptorDataLayerAssets;
		const UExternalDataLayerAsset* DescriptorExternalDataLayerAsset = nullptr;
		GridDescriptor.GetDataLayerAssets(DescriptorDataLayerAssets, DescriptorExternalDataLayerAsset);

		// External DataLayer should have been assigned on Spawn
		ensure(ExternalDataLayerAsset == DescriptorExternalDataLayerAsset);

		DataLayerAssets = DescriptorDataLayerAssets;

		SetHLODLayer(GridDescriptor.GetHLODLayer().LoadSynchronous());
	}

	// Set only once upon creation, can't change for a Partition Actor
	RuntimeGridDescriptorHash = GridDescriptor.GetRuntimeHash();
	SetGridSize(PCGGridSize);
	UpdateBoundsComponentExtents();

	ensure(GetGridDescriptor() == GridDescriptor);
#endif // WITH_EDITOR

	RegisterPCG();

	bWasPostCreatedLoaded = true;
}

bool APCGPartitionActor::IsSafeForDeletion() const
{
	ensure(IsInGameThread());
	for (const TObjectPtr<UPCGComponent>& PCGComponent : GetAllOriginalPCGComponents())
	{
		if (PCGComponent && (PCGComponent->IsGenerating() || PCGComponent->IsCleaningUp()))
		{
			return false;
		}
	}

	return true;
}

bool APCGPartitionActor::HasLocalPCGComponents() const
{
	return !LocalToOriginal.IsEmpty();
}

TSet<TObjectPtr<UPCGComponent>> APCGPartitionActor::GetAllLocalPCGComponents() const
{
	TSet<TObjectPtr<UPCGComponent>> ResultComponents;
	LocalToOriginal.GetKeys(ResultComponents);

	return ResultComponents;
}

TSet<TObjectPtr<UPCGComponent>> APCGPartitionActor::GetAllOriginalPCGComponents() const
{
	TSet<TObjectPtr<UPCGComponent>> ResultComponents;
	for(const TPair<TObjectKey<UPCGComponent>, TObjectPtr<UPCGComponent>>& OriginalPair : OriginalToLocal)
	{
		if (UPCGComponent* OriginalComponent = OriginalPair.Key.ResolveObjectPtr())
		{
			ResultComponents.Add(OriginalComponent);
		}
	}

	return ResultComponents;
}

UPCGSubsystem* APCGPartitionActor::GetSubsystem() const 
{
	return UPCGSubsystem::GetInstance(GetWorld());
}

#if WITH_EDITOR
bool APCGPartitionActor::ChangeTransientState(UPCGComponent* OriginalComponent, EPCGEditorDirtyMode EditingMode)
{
	check(OriginalComponent);

	// First, propagate the transient state to the matching local component if any
	if (UPCGComponent* LocalComponent = GetLocalComponent(OriginalComponent))
	{
		LocalComponent->SetEditingMode(/*CurrentEditingMode=*/EditingMode, /*SerializedEditingMode=*/EditingMode);
		LocalComponent->ChangeTransientState(EditingMode);
	}

	// Then, when switching to anything but preview, we must get rid of any still-loaded components that would have been removed otherwise
	if (EditingMode != EPCGEditorDirtyMode::Preview)
	{
		TArray<UPCGComponent*> LocalComponentsToDelete;

		for (const auto& LoadedPreviewLocalToOriginal : LoadedPreviewComponents)
		{
			if (LoadedPreviewLocalToOriginal.Value == OriginalComponent)
			{
				LocalComponentsToDelete.Add(LoadedPreviewLocalToOriginal.Key);
			}
		}

		// Streamlined version of RemoveGraphInstance
		Modify(!LocalComponentsToDelete.IsEmpty());

		for (UPCGComponent* LocalComponent : LocalComponentsToDelete)
		{
			LoadedPreviewComponents.Remove(LocalComponent);
			LocalComponent->ChangeTransientState(EditingMode);
			LocalComponent->CleanupLocalImmediate(/*bRemoveComponents=*/true);
			LocalComponent->DestroyComponent();
		}
	}

	return OriginalToLocal.IsEmpty() && LoadedPreviewComponents.IsEmpty();
}

void APCGPartitionActor::UpdateBoundsComponentExtents()
{
	// Disable BoundsComponent if in 2D
	if (BoundsComponent && !bUse2DGrid)
	{
		BoundsComponent->SetBoxExtent(GetFixedBounds().GetExtent());
	}
}
#endif // WITH_EDITOR

FString APCGPartitionActor::GetPCGPartitionActorName(uint32 GridSize, const FIntVector& GridCoords, bool bRuntimeGenerated)
{
	FPCGGridDescriptor GridDescriptor = FPCGGridDescriptor()
		.SetGridSize(GridSize)
		.SetIsRuntime(bRuntimeGenerated);
	return GetPCGPartitionActorName(GridDescriptor, GridCoords);
}

FString APCGPartitionActor::GetPCGPartitionActorName(const FPCGGridDescriptor& GridDescriptor, const FIntVector& GridCoords)
{
	return GridDescriptor.GetPartitionActorName(GridCoords);
}
