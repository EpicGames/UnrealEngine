// Copyright Epic Games, Inc. All Rights Reserved.

#include "Grid/PCGPartitionActorDesc.h"

#if WITH_EDITOR
#include "Grid/PCGPartitionActor.h"

#include "Algo/Transform.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "WorldPartition/WorldPartitionActorDescArchive.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

void FPCGPartitionActorDesc::Init(const AActor* InActor)
{
	FPartitionActorDesc::Init(InActor);

	const APCGPartitionActor* PartitionActor = CastChecked<APCGPartitionActor>(InActor);
	bUse2DGrid = PartitionActor->IsUsing2DGrid();
}

FPCGGridDescriptor FPCGPartitionActorDesc::GetGridDescriptor(const FWorldPartitionActorDescInstance* InActorDescInstance) const
{
	check(InActorDescInstance);
	FPCGGridDescriptor GridDescriptor = FPCGGridDescriptor()
		.SetGridSize(GridSize)
		.SetIs2DGrid(bUse2DGrid)
		.SetIsRuntime(false)
		.SetHLODLayer(TSoftObjectPtr<UHLODLayer>(GetHLODLayer()));

	TArray<TSoftObjectPtr<UDataLayerAsset>> DataLayerAssets;
	// Because of deprecation DataLayerAssets in ActorDescs are stored in FNames (we can convert them to SoftObjectPtrs if IsUsingDataLayerAsset() returns true)
	// TODO: store as object pointers now that all data layers have assets
	Algo::Transform(GetDataLayers(), DataLayerAssets, [](const FName& InDataLayerAsset) 
	{
		return TSoftObjectPtr<UDataLayerAsset>(FSoftObjectPath(InDataLayerAsset.ToString()));
	});
	GridDescriptor.SetDataLayerAssets(DataLayerAssets);

	return GridDescriptor;
}

FIntVector FPCGPartitionActorDesc::GetGridCoord() const
{
	return FIntVector(GridIndexX, GridIndexY, GridIndexZ);
}

void FPCGPartitionActorDesc::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	FPartitionActorDesc::Serialize(Ar);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGPartitionActorDesc)
		{
			// Try to get Grid Size and Coords from names with following pattern "PCGPartitionActor_{GridSize}_{X}_{Y}_UAID"
			// 
			// Naming was broken in 5.4 where "_{Z}" part of the SpawnParams.Name was converted into the FName Number part
			// in combination with the SpawnParams.NameMode being FActorSpawnParameters::ESpawnActorNameMode::Requested which ends up using FActorSpawnUtils::GetBaseName
			// which calls FName::GetPlainNameString() which removes the Number part (treats _{Z} as the FName number)
			//
			// The label was properly set so if we are dealing with a _UAID_ name then use the label to parse the GridSize/X/Y/Z
			const FString ActorNameStr = GetActorName().ToString();
			if (ActorNameStr.Contains("_UAID_"))
			{
				const FString ActorLabelStr = GetActorLabel().ToString();
				TArray<FString> Tokens;
				ActorLabelStr.ParseIntoArray(Tokens, TEXT("_"));
				if (Tokens[0] == TEXT("PCGPartitionActor") && Tokens.Num() == 5)
				{
					TTypeFromString<uint32>::FromString(GridSize, *Tokens[1]);
					TTypeFromString<int64>::FromString(GridIndexX, *Tokens[2]);
					TTypeFromString<int64>::FromString(GridIndexY, *Tokens[3]);
					TTypeFromString<int64>::FromString(GridIndexZ, *Tokens[4]);
				}
				else
				{
					bInvalid = true;
				}
			}
		}

		// For version prior, patch the bUse2DGrid when initializing PCG Subsystem
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGGridDescriptor)
		{
			bRequiresUse2DGridFixup = true;
		}
	}

	if (!bIsDefaultActorDesc)
	{
		// From this version on, serialize bUse2DGrid
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::PCGGridDescriptor)
		{
			Ar << bUse2DGrid;
		}
	}
}

#endif // WITH_EDITOR
