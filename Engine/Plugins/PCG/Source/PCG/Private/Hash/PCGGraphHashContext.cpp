// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hash/PCGGraphHashContext.h"

#if WITH_EDITOR

#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGPin.h"

FPCGGraphHashContext::FPCGGraphHashContext(UPCGGraphInterface* InGraph)
	: FPCGObjectHashContext(InGraph)
{
	check(InGraph);
	InGraph->OnGraphChangedDelegate.AddRaw(this, &FPCGGraphHashContext::OnGraphChanged);
	AddHashPolicy(new FPCGObjectHashPolicyPCGNoHash());
}

FPCGGraphHashContext::~FPCGGraphHashContext()
{
	if (UPCGGraphInterface* Graph = GetObject<UPCGGraphInterface>())
	{
		Graph->OnGraphChangedDelegate.RemoveAll(this);
	}
}

void FPCGGraphHashContext::OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType)
{
	ensure(InGraph == GetObject<UPCGGraphInterface>());
	// could probably filter out some changes
	OnChanged().Execute();
}

FPCGObjectHashContext* FPCGGraphHashContext::MakeInstance(UObject* InObject)
{
	return new FPCGGraphHashContext(CastChecked<UPCGGraphInterface>(InObject));
}

#endif // WITH_EDITOR