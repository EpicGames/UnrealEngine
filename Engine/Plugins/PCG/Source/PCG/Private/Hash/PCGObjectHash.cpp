// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hash/PCGObjectHash.h"

#if WITH_EDITOR

#include "PCGModule.h"

#include "Algo/Copy.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/UObjectAnnotation.h"

namespace PCGObjectHash
{
	TAutoConsoleVariable<bool> CVarAlwaysHash(
		TEXT("pcg.debug.hash.AlwaysHash"),
		false,
		TEXT("To debug hashing, set to true and the hash will always be recomputed even if the object and its dependencies didn't change"));

	TAutoConsoleVariable<bool> CVarLogDependencies(
		TEXT("pcg.debug.hash.LogDependencies"),
		false,
		TEXT("To debug hashing, set to true to log dependency hashes"));

	TAutoConsoleVariable<bool> CVarLogSkippedProperties(
		TEXT("pcg.debug.hash.LogSkippedProperties"),
		false,
		TEXT("To debug hashing, set to true to log properties not part of the hash"));

	TAutoConsoleVariable<bool> CVarLogSkippedObjects(
		TEXT("pcg.debug.hash.LogSkippedObjects"),
		false,
		TEXT("To debug hashing, set to true to log objects not part of the hash"));

	// Debug command that outputs the Hash of supported objects and all it's dependency hashes also.
	const FString GCalculatePCGObjectHashCommandName(TEXT("pcg.debug.hash"));
	FAutoConsoleCommand CalculatePCGObjectHashCommand(
		*GCalculatePCGObjectHashCommandName,
		TEXT("Calculates and outputs a hash for specified object and all its dependencies"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
		{
			if (InArgs.Num() < 1)
			{
				UE_LOGF(LogPCG, Warning, "%ls : Requires at least 1 argument. First argument should be the object path.", *GCalculatePCGObjectHashCommandName);
				return;
			}

			TArray<FString> Args(InArgs);
			FString ObjectPath;
			Args.HeapPop(ObjectPath);
			FSoftObjectPath SoftObjectPath(ObjectPath);

			UObject* ResolvedObject = SoftObjectPath.TryLoad();
			if (!ResolvedObject)
			{
				UE_LOGF(LogPCG, Warning, "%ls : Could not find object: '%ls'", *GCalculatePCGObjectHashCommandName, *SoftObjectPath.ToString());
				return;
			}

			if (FPCGObjectHash* Hash = FPCGModule::GetPCGModuleChecked().GetConstObjectHashFactory().GetOrCreateObjectHash(ResolvedObject))
			{
				Hash->GetHash(/*bInVerbose=*/true);
			}
			else
			{
				UE_LOGF(LogPCG, Warning, "%ls : Unsupported object: '%ls'", *GCalculatePCGObjectHashCommandName, *SoftObjectPath.ToString());
			}
		}));

	struct FPCGObjectHashAnnotation
	{
		bool IsDefault() const
		{
			return Ptr == nullptr;
		}
		TSharedPtr<FPCGObjectHash> Ptr;
	};

	// Global store for existing hash objects
	FUObjectAnnotationSparse<FPCGObjectHashAnnotation, /*bAutoRemove=*/true> ObjectHashAnnotation;

	// xxHash Property Hash + External dependency gathering archive (referenced objects living in a different UPackage)
	class FPCGObjectHashArchive : public FArchiveUObject
	{
	public:
		using Super = FArchiveUObject;

		static FPCGObjectHash::FHash Hash(const FPCGObjectHashContext* InContext, TSet<UObject*>& OutObjectReferences, bool bInVerbose, int32 InLevel)
		{
			check(InContext)
			if (!InContext->IsValid())
			{
				return FPCGObjectHash::FHash();
			}

			FPCGObjectHashArchive Ar(InContext, bInVerbose, InLevel);
			Ar.Compute();
			OutObjectReferences = Ar.GetObjectReferences();
			return Ar.GetHash();
		}

		const TSet<UObject*> GetObjectReferences() const
		{
			return ObjectReferences;
		}

		FPCGObjectHash::FHash GetHash() const
		{
			return LocalHash;
		}

	protected:
		FPCGObjectHashArchive(const FPCGObjectHashContext* InContext, bool bInVerbose, int32 InLevel)
			: Context(InContext), bVerbose(bInVerbose), Level(InLevel)
		{
			check(Context && Context->IsValid());

			SetIsPersistent(false);
			SetIsSaving(true);
			SetIsLoading(false);
			SetFilterEditorOnly(false);

			ArShouldSkipBulkData = true;
		}

		void Compute()
		{
			ObjectsToVisit.Reset();
			VisitedObjects.Reset();
			ObjectReferences.Reset();

			ObjectsToVisit.Add(Context->GetObject());
			while (ObjectsToVisit.Num() > 0)
			{
				UObject* CurObj = ObjectsToVisit.Pop(EAllowShrinking::No);
				if (!VisitedObjects.Contains(CurObj))
				{
					VisitedObjects.Add(CurObj);

					TGuardValue<UObject*> Guard(CurrentObject, CurObj);
					CurObj->Serialize(*this);
				}
			}

			LocalHash = Hasher.Finalize();
		}

		virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
		{
			const bool bShouldHashProperty = Context->ShouldHashProperty(CurrentObject, InProperty);
			
			if (!bShouldHashProperty && bVerbose && PCGObjectHash::CVarLogSkippedProperties.GetValueOnAnyThread())
			{
				UE_LOGF(LogPCG, Log, "[PCGHash] %ls  Property skipped: '%ls' (Type: '%ls' - Object : '%ls')", FCString::Tab(Level), *InProperty->GetName(), *CurrentObject->GetClass()->GetName(), *CurrentObject->GetPathName());
			}

			return !bShouldHashProperty;
		}

		// For now we don't try and do a deep hashing of soft reference but we at least only serialize the path once and use an index for duplicated references
		virtual FArchive& operator<<(FSoftObjectPath& Value) override 
		{ 
			if (int32* IndexPtr = SoftReferenceIndices.Find(Value))
			{
				*this << *IndexPtr;
			}
			else
			{
				int32 VisitingIndex = SoftReferenceIndices.Num();
				SoftReferenceIndices.Add(Value, VisitingIndex);
				
				// Hash index of newly visited reference
				*this << VisitingIndex;
				
				SerializeSoftObjectPath(*this, Value);
			}

			return *this;
		}

		virtual FArchive& operator<<(UObject*& InObjectReference) override
		{
			if (InObjectReference != nullptr)
			{
				int32* VisitedIndex = ObjectIndices.Find(InObjectReference);

				// Haven't visited object yet
				if (!VisitedIndex)
				{
					// Assign an index to this object reference and serialize it
					int32 VisitingIndex = ObjectIndices.Num();
					ObjectIndices.Add(InObjectReference, VisitingIndex);

					// Hash index of newly visited reference
					*this << VisitingIndex;

					// Check if object supports hashing and add it as a reference (reference hashes are used in the final object hash but are stored independently to avoid full recomputations)
					if (FPCGObjectHash* ObjectReferenceHash = FPCGModule::GetPCGModuleChecked().GetConstObjectHashFactory().GetOrCreateObjectHash(InObjectReference))
					{
						VisitedObjects.Add(InObjectReference);

						ObjectReferences.Add(InObjectReference);
					}
					else if (InObjectReference->IsIn(Context->GetObject()) && Context->ShouldHashSubObject(InObjectReference))
					{
						// Will be visited later
						ObjectsToVisit.Add(InObjectReference);
					}
					else
					{
						VisitedObjects.Add(InObjectReference);

						// Serialize object path once, next time this object is encountered its visited index will get serialized
						FString ObjectReferencePath = InObjectReference->GetPathName();
						*this << ObjectReferencePath;

						if (CVarLogSkippedObjects.GetValueOnAnyThread() && bVerbose)
						{
							UE_LOGF(LogPCG, Log, "[PCGHash] %ls  Object skipped: '%ls' (Type: '%ls')", FCString::Tab(Level), *InObjectReference->GetPathName(), *InObjectReference->GetClass()->GetName());
						}
					}
				}
				else
				{
					// Hash visisted index
					*this << *VisitedIndex;
				}
			}

			return *this;
		}

		virtual FArchive& operator<<(FLazyObjectPtr& Value) override 
		{ 
			// unsupported ptr type
			check(0);
			return *this;
		}

		virtual FArchive& operator<<(FWeakObjectPtr& Value) override
		{
			// unsupported ptr type
			check(0);
			return *this;
		}

		virtual void Serialize(void* Data, int64 Num) override
		{
			Hasher.Update((uint8*)Data, Num);
		}

		using Super::operator<<;

		virtual FArchive& operator<<(class FName& Value) override
		{
			FString NameAsString = Value.ToString();
			*this << NameAsString;
			return *this;
		}

		const FPCGObjectHashContext* Context = nullptr;

		FPCGObjectHash::FHash LocalHash;
		FXxHash64Builder Hasher;

		TSet<UObject*> ObjectReferences;
		TSet<UObject*> VisitedObjects;
		TArray<UObject*> ObjectsToVisit;
		UObject* CurrentObject = nullptr;

		// Mappings to serialize indices when reference is encountered multiple times
		TMap<FSoftObjectPath, int32> SoftReferenceIndices;
		TMap<UObject*, int32> ObjectIndices;

		bool bVerbose = false;
		int32 Level = 0;
	};
}


FPCGObjectHashContext::FPCGObjectHashContext(UObject* InObject)
	: Object(InObject)
{

}

FPCGObjectHashContext::~FPCGObjectHashContext()
{
	for (const IPCGObjectHashPolicy* Policy : HashPolicies)
	{
		delete Policy;
	}
}

bool FPCGObjectHashContext::ShouldHashProperty(const UObject* InObject, const FProperty* InProperty) const
{
	if (!ShouldHashTransientProperties() && InProperty->HasAnyPropertyFlags(CPF_Transient))
	{
		return false;
	}

	for (const IPCGObjectHashPolicy* Policy : HashPolicies)
	{
		if (!Policy->ShouldHashProperty(InObject, InProperty))
		{
			return false;
		}
	}

	return ShouldHashPropertyInternal(InObject, InProperty);
}

bool FPCGObjectHashContext::ShouldHashSubObject(const UObject* InObject) const
{
	return InObject && ShouldHashSubObjectInternal(InObject);
}

bool FPCGObjectHashPolicyPropertyMetaDataFilter::ShouldHashProperty(const UObject* InObject, const FProperty* InProperty) const
{
	return InProperty->HasMetaData(*MetaData) == bInclusionFilter;
}

FPCGObjectHash::FPCGObjectHash(FPCGObjectHashContext* InContext)
	: Context(InContext)
{
	Context->OnChanged().BindRaw(this, &FPCGObjectHash::InvalidateHash);
}

FPCGObjectHash::~FPCGObjectHash()
{
	delete Context;
}

FPCGObjectHash::FHash FPCGObjectHash::GetHash(bool bInVerbose) const
{
	return GetHashInternal(bInVerbose, /*InLevel=*/0);
}

FPCGObjectHash::FHash FPCGObjectHash::GetHashInternal(bool bInVerbose, uint32 InLevel) const
{
	FXxHash64Builder FinalHasher;

	const bool bShouldLog = bInVerbose && (InLevel == 0 || PCGObjectHash::CVarLogDependencies.GetValueOnAnyThread());
	const double StartHashTime = bShouldLog ? FPlatformTime::Seconds() : 0.0;
	if (bShouldLog)
	{
		UE_LOGF(LogPCG, Log, "[PCGHash] %lsBegin Hash: Object: '%ls' (Type: '%ls')", FCString::Tab(InLevel), *Context->GetObject()->GetPathName(), *Context->GetObject()->GetClass()->GetName());
	}

	// Check that our local hash is set (it might have been invalidated or never computed)
	if (!LocalHash.IsSet() || PCGObjectHash::CVarAlwaysHash.GetValueOnAnyThread())
	{
		TSet<UObject*> NewObjectReferences;
		LocalHash = PCGObjectHash::FPCGObjectHashArchive::Hash(Context, NewObjectReferences, bInVerbose, InLevel);
		
		// Sort dependencies for deterministic hashing
		ObjectReferences.Empty(NewObjectReferences.Num());
		Algo::Transform(NewObjectReferences, ObjectReferences, [](UObject* InObject) { return TWeakObjectPtr(InObject); });
		ObjectReferences.Sort([](const TWeakObjectPtr<UObject>& A, const TWeakObjectPtr<UObject>& B) { return A->GetFName().LexicalLess(B->GetFName()); });
	}

	// Update hasher with our local hash
	if (ensure(LocalHash.IsSet()))
	{
		FPCGObjectHash::FHash& LocalHashValue = LocalHash.GetValue();
		FinalHasher.Update(&LocalHashValue.Hash, sizeof(LocalHashValue.Hash));
	}

	// Update hasher with all hashable dependencies
	for (TWeakObjectPtr<UObject> WeakObjectReference : ObjectReferences)
	{
		UObject* ObjectReference = WeakObjectReference.Get();
		if (!ObjectReference)
		{
			continue;
		}

		if(FPCGObjectHash* ObjectReferenceHashPtr = FPCGModule::GetPCGModuleChecked().GetConstObjectHashFactory().GetOrCreateObjectHash(ObjectReference))
		{
			FPCGObjectHash::FHash ObjectReferenceHash = ObjectReferenceHashPtr->GetHashInternal(bInVerbose, InLevel + 1);
			FinalHasher.Update(&ObjectReferenceHash.Hash, sizeof(ObjectReferenceHash.Hash));
		}
	}

	FPCGObjectHash::FHash FinalHash = FinalHasher.Finalize();
	
	if (bShouldLog)
	{
		const double HashTime = FPlatformTime::Seconds() - StartHashTime;
		UE_LOGF(LogPCG, Log, "[PCGHash] %lsEnd Hash: Object: '%ls' -> Hash: '%ls' took '%ls' seconds", FCString::Tab(InLevel), *Context->GetObject()->GetPathName(), *LexToString(FinalHash.Hash), *FText::AsNumber(HashTime).ToString());
	}

	return FinalHash;
}

void FPCGObjectHashFactory::RegisterObjectHashContextFactory(UClass* InClass, FPCGOnCreateObjectHashContext InCreatePCGObjectHashContext)
{
	ensure(!CreateObjectHashContextPerClass.Contains(InClass->GetFName()));
	CreateObjectHashContextPerClass.FindOrAdd(InClass->GetFName(), InCreatePCGObjectHashContext);
}

FPCGObjectHash* FPCGObjectHashFactory::GetOrCreateObjectHash(UObject* InObject) const
{
	if (!InObject)
	{
		return nullptr;
	}

	// Object annocation maps a UObject* to some structure and handles the object lifespan, if object is destroyed, annotation gets removed.
	// This is useful to share hash objects across different object hierarchies. Ex: 2 PCG Graphs that reference the same sub graph will use the same sub graph hash object.
	PCGObjectHash::FPCGObjectHashAnnotation Annotation = PCGObjectHash::ObjectHashAnnotation.GetAnnotation(InObject);
	if (!Annotation.IsDefault())
	{
		return Annotation.Ptr.Get();
	}
		
	UClass* CurrentClass = InObject->GetClass();
	check(CurrentClass);

	// Find existing factory
	while (CurrentClass)
	{
		if (const FPCGOnCreateObjectHashContext* Delegate = CreateObjectHashContextPerClass.Find(CurrentClass->GetFName()))
		{
			FPCGObjectHashContext* HashContext = Delegate->Execute(InObject);
			if (HashContext)
			{
				TSharedPtr<FPCGObjectHash> ObjectHash = MakeShared<FPCGObjectHash>(HashContext);
				FPCGObjectHash* ObjectHashPtr = ObjectHash.Get();

				// Store annotation
				PCGObjectHash::FPCGObjectHashAnnotation NewAnnotation;
				NewAnnotation.Ptr = MoveTemp(ObjectHash);
				PCGObjectHash::ObjectHashAnnotation.AddAnnotation(InObject, MoveTemp(NewAnnotation));

				return ObjectHashPtr;
			}
		}

		CurrentClass = CurrentClass->GetSuperClass();
	}

	return nullptr;
}

#endif // WITH_EDITOR