// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hash/PCGSettingsHashContext.h"

#if WITH_EDITOR

#include "PCGSettings.h"

FPCGSettingsHashContext::FPCGSettingsHashContext(UPCGSettingsInterface* InSettings)
	: FPCGObjectHashContext(InSettings)
{
	InSettings->OnSettingsChangedDelegate.AddRaw(this, &FPCGSettingsHashContext::OnSettingsChanged);
	AddHashPolicy(new FPCGSettingsHashPolicy());
}

FPCGSettingsHashContext::~FPCGSettingsHashContext()
{
	if (UPCGSettingsInterface* Settings = GetObject<UPCGSettingsInterface>())
	{
		Settings->OnSettingsChangedDelegate.RemoveAll(this);
	}
}

FPCGObjectHashContext* FPCGSettingsHashContext::MakeInstance(UObject* InObject)
{
	return new FPCGSettingsHashContext(CastChecked<UPCGSettings>(InObject));
}

void FPCGSettingsHashContext::OnSettingsChanged(UPCGSettings* InSettings, EPCGChangeType ChangeType)
{
	ensure(InSettings == GetObject<UPCGSettings>());
	// could probably filter out some changes
	OnChanged().Execute();
}

bool FPCGSettingsHashPolicy::ShouldHashProperty(const UObject* InObject, const FProperty* InProperty) const
{
	// Omit CRC'ing data collections here as it is very slow. Rely instead on UID/ObjectPath.
	const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty);
	if (StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(FPCGDataCollection::StaticStruct()))
	{
		return false;
	}

	return Super::ShouldHashProperty(InObject, InProperty);
}

#endif // WITH_EDITOR