// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/IO/PCGIOHelpers.h"

#include "Misc/Paths.h"
#include "Serialization/CustomVersion.h"

#define LOCTEXT_NAMESPACE "PCGIOHelpers"

namespace PCG::IO
{
	namespace Constants
	{
		const FGuid FCustomExportVersion::GUID = FGuid(0x04E74488, 0x4BAC8717, 0xBBB18694, 0x39F8F3CE);
		FCustomVersionRegistration GRegisterPCGExportSelectedAttributesCustomVersion(FCustomExportVersion::GUID, FCustomExportVersion::LatestVersion, TEXT("PCGExportSelectedAttributes"));
	}

	namespace Helpers::String
	{
		void ToPrecisionString(const double Value, const int32 Precision, FString& OutString)
		{
			OutString = FString::SanitizeFloat(Value, Precision);
		}
	} // namespace Helpers::String
} // namespace PCG::IO

#undef LOCTEXT_NAMESPACE
