// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/IO/PCGJsonHelpers.h"

#include "Helpers/PCGHelpers.h"
#include "Helpers/IO/PCGIOHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Algo/Find.h"
#include "Policies/CondensedJsonPrintPolicy.h"

#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

#define LOCTEXT_NAMESPACE "PCGJson"

namespace PCG::IO::Json
{
	using JsonKeyValuesMap = TArray<TTuple<FString, TArray<TSharedPtr<FJsonValue>>>>;

	namespace Private
	{
		TSharedPtr<FJsonValueArray> DoubleArrayToJsonArray(const TConstArrayView<double> ArrayView)
		{
			TArray<TSharedPtr<FJsonValue>> JsonArray;
			for (double Value : ArrayView)
			{
				JsonArray.Emplace(MakeShared<FJsonValueNumber>(Value));
			}

			return MakeShared<FJsonValueArray>(MoveTemp(JsonArray));
		}

		Accessor::FCache CreateAccessorCache(const UPCGData* Data, const TArray<FPCGAttributePropertySelector>& Selectors, const FPCGContext* Context = nullptr)
		{
			check(Data);
			Accessor::FCache AccessorCache;

			for (const FPCGAttributePropertySelector& Selector : Selectors)
			{
				Accessor::FCacheEntry Entry;
				Entry.Selector = Selector;
				Entry.Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Data, Entry.Selector);
				Entry.Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Data, Entry.Selector);
				if (!Entry.Accessor || !Entry.Keys)
				{
					PCGLog::Metadata::LogFailToCreateAccessorError(Entry.Selector, Context);
					continue;
				}

				AccessorCache.Emplace(MoveTemp(Entry));
			}

			return AccessorCache;
		}

		TArray<TSharedPtr<FJsonValue>> AttributeToJsonValues(const Accessor::FCacheEntry& Entry)
		{
			check(Entry.Accessor.IsValid());
			return PCGMetadataAttribute::CallbackWithRightType(Entry.Accessor->GetUnderlyingType(), [&Entry]<typename T>(const T&)
			{
				TArray<TSharedPtr<FJsonValue>> JsonValues;

				// For floating point values and containers (FVector, FRotator, FTransform, etc.), use the Json numerical to maintain precision.
				if constexpr (PCG::Private::MetadataTraits<T>::IsFloatingPoint)
				{
					TArray<T> Values;
					PCGAttributeAccessorHelpers::ExtractAllValues(Entry.Accessor.Get(), Entry.Keys.Get(), Values, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);

					JsonValues.Reserve(Values.Num());
					Algo::Transform(Values, JsonValues, [&Entry](const T& Value)
					{
						return Helpers::ConvertFloatingPointType(Value);
					});
				}
				else // For everything else, automatically broadcast it to string
				{
					TArray<FString> Values;
					PCGAttributeAccessorHelpers::ExtractAllValues(Entry.Accessor.Get(), Entry.Keys.Get(), Values, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);

					JsonValues.Reserve(Values.Num());
					Algo::Transform(Values, JsonValues, [](const FString& OutValue)
					{
						return MakeShared<FJsonValueString>(OutValue);
					});
				}

				return JsonValues;
			});
		}

		void BuildKeyValuesMap(JsonKeyValuesMap& OutKeyValuesMap, const Accessor::FCache& AccessorCache)
		{
			OutKeyValuesMap.Reset();
			OutKeyValuesMap.SetNum(AccessorCache.Num());
			// Pre-process the source data into Json values.
			for (int32 CacheIndex = 0; CacheIndex < AccessorCache.Num(); ++CacheIndex)
			{
				const Accessor::FCacheEntry& Entry = AccessorCache[CacheIndex];
				FString& FieldName = OutKeyValuesMap[CacheIndex].Get<0>();
				FieldName = Entry.Selector.ToString();

				TArray<TSharedPtr<FJsonValue>>& SourceJsonValues = OutKeyValuesMap[CacheIndex].Get<1>();
				SourceJsonValues = AttributeToJsonValues(Entry);
			}
		}

		void AppendSelectionByAttribute(const TSharedPtr<FJsonObject>& InOutJsonObject, JsonKeyValuesMap& InOutKeyValueMap)
		{
			for (auto& KeyValuePair : InOutKeyValueMap)
			{
				const FString& Key = KeyValuePair.Get<0>();
				TArray<TSharedPtr<FJsonValue>>& SourceValues = KeyValuePair.Get<1>();
				InOutJsonObject->SetArrayField(Key, MoveTemp(SourceValues));
			}
		}

		void AppendSelectionByElement(const TSharedPtr<FJsonObject>& InOutJsonObject, JsonKeyValuesMap& InOutKeyValueMap, const int32 NumElements)
		{
			if (!ensure(NumElements > 0))
			{
				return;
			}

			for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
			{
				TSharedPtr<FJsonObject> ElementJsonObject = MakeShared<FJsonObject>();

				for (auto& KeyValuePair : InOutKeyValueMap)
				{
					const FString& Key = KeyValuePair.Get<0>();
					TArray<TSharedPtr<FJsonValue>>& JsonValues = KeyValuePair.Get<1>();

					// Currently only support the following Json types:
					if (!JsonValues.IsValidIndex(ElementIndex)
						|| !(JsonValues[ElementIndex]->Type == EJson::Array
							 || JsonValues[ElementIndex]->Type == EJson::Number
							 || JsonValues[ElementIndex]->Type == EJson::String
							 || JsonValues[ElementIndex]->Type == EJson::Object))
					{
						continue;
					}

					// Move the values directly
					ElementJsonObject->SetField(Key, MoveTemp(JsonValues[ElementIndex]));
				}

				InOutJsonObject->SetObjectField(FString::Printf(TEXT("element_%d"), ElementIndex), MoveTemp(ElementJsonObject));
			}

			// The map is no longer valid, so clean it.
			InOutKeyValueMap.Empty();
		}

		template <typename CharType = TCHAR, typename PrintPolicy = TCondensedJsonPrintPolicy<CharType>>
		FString ToJsonString(const TSharedPtr<FJsonObject>& JsonObject)
		{
			FString OutputString;
			const TSharedRef<TJsonWriter<CharType, PrintPolicy>> JsonWriter = TJsonWriterFactory<CharType, PrintPolicy>::Create(&OutputString);
			if (!FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter))
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("JsonSerializationError", "Serialization of Json data failed."));
				return {};
			}

			return OutputString;
		}
	} // namespace Private

	namespace Helpers
	{
		TSharedPtr<FJsonValue> ConvertFloatingPointType(const FVector2D& Value)
		{
			return Private::DoubleArrayToJsonArray({Value.X, Value.Y});
		}

		TSharedPtr<FJsonValue> ConvertFloatingPointType(const FVector& Value)
		{
			return Private::DoubleArrayToJsonArray({Value.X, Value.Y, Value.Z});
		}

		TSharedPtr<FJsonValue> ConvertFloatingPointType(const FVector4& Value)
		{
			return Private::DoubleArrayToJsonArray({Value.X, Value.Y, Value.Z, Value.W});
		}

		TSharedPtr<FJsonValue> ConvertFloatingPointType(const FQuat& Value)
		{
			return Private::DoubleArrayToJsonArray({Value.X, Value.Y, Value.Z, Value.W});
		}

		TSharedPtr<FJsonValue> ConvertFloatingPointType(const FRotator& Value)
		{
			return Private::DoubleArrayToJsonArray({Value.Pitch, Value.Yaw, Value.Roll});
		}

		TSharedPtr<FJsonValue> ConvertFloatingPointType(const FTransform& Value)
		{
			TSharedPtr<FJsonObject> JsonTransformObject = MakeShared<FJsonObject>();
			JsonTransformObject->SetArrayField(TEXT("translation"), ConvertFloatingPointType(Value.GetTranslation())->AsArray());
			JsonTransformObject->SetArrayField(TEXT("rotation"), ConvertFloatingPointType(Value.GetRotation())->AsArray());
			JsonTransformObject->SetArrayField(TEXT("scale"), ConvertFloatingPointType(Value.GetScale3D())->AsArray());

			return MakeShared<FJsonValueObject>(JsonTransformObject);
		}

		void AppendHeader(const TSharedPtr<FJsonObject>& InOutJsonObject, int32 DataVersion, int32 CustomDataVersion)
		{
			AppendVersionToHeader(InOutJsonObject, DataVersion);
		}

		void AppendVersionToHeader(const TSharedPtr<FJsonObject>& InOutJsonObject, const int32 DataVersion)
		{
			if (ensure(InOutJsonObject.IsValid()))
			{
				InOutJsonObject->SetNumberField(Constants::DefaultDataVersionKey, DataVersion);
			}
		}

		void AppendSelectionByAttribute(TSharedPtr<FJsonObject>& InOutJsonObject, const UPCGData* InData, const TArray<FPCGAttributePropertySelector>& Selectors, const FPCGContext* InOptionalContext)
		{
			if (!InOutJsonObject || !InData || Selectors.IsEmpty())
			{
				return;
			}

			const Accessor::FCache AccessorCache = Private::CreateAccessorCache(InData, Selectors, InOptionalContext);
			JsonKeyValuesMap KeyValuesMap;
			Private::BuildKeyValuesMap(KeyValuesMap, AccessorCache);
			Private::AppendSelectionByAttribute(InOutJsonObject, KeyValuesMap);
		}

		void AppendSelectionByElement(TSharedPtr<FJsonObject>& InOutJsonObject, const UPCGData* InData, const TArray<FPCGAttributePropertySelector>& Selectors, const FPCGContext* InOptionalContext)
		{
			if (!InOutJsonObject || !InData || Selectors.IsEmpty())
			{
				return;
			}

			const Accessor::FCache AccessorCache = Private::CreateAccessorCache(InData, Selectors, InOptionalContext);
			JsonKeyValuesMap KeyValuesMap;
			Private::BuildKeyValuesMap(KeyValuesMap, AccessorCache);

			FPCGMetadataDomainID DomainID = InData->GetMetadataDomainIDFromSelector(Selectors[0]);
			const FPCGAttributePropertySelector* MismatchedSelector = Algo::FindByPredicate(Selectors, [InData, &DomainID](const FPCGAttributePropertySelector& InSelector)
			{
				return InData->GetMetadataDomainIDFromSelector(InSelector) != DomainID;
			});

			if (MismatchedSelector)
			{
				FString Domain1 = Selectors[0].GetDomainString();
				Domain1 = Domain1.IsEmpty() ? TEXT("Default") : Domain1;
				FString Domain2 = MismatchedSelector->GetDomainString();
				Domain2 = Domain2.IsEmpty() ? TEXT("Default") : Domain2;

				const FText Warning = FText::Format(
					LOCTEXT("SelectorsFromMultipleDomains",
							"[Json] Could not append selection by element. All selectors must share the same domain: {0} vs {1}"),
					FText::FromString(Domain1),
					FText::FromString(Domain2));

				PCGLog::LogWarningOnGraph(Warning, InOptionalContext);

				return;
			}

			Private::AppendSelectionByElement(InOutJsonObject, KeyValuesMap, PCGHelpers::GetNumberOfElements(InData, DomainID));
		}

		FString ToJsonString(const TSharedPtr<FJsonObject>& JsonObject, const bool bPrettyJson)
		{
			return bPrettyJson
				? Private::ToJsonString<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>(JsonObject)
				: Private::ToJsonString<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>(JsonObject);
		}
	} // namespace Helpers
}  // namespace PCG::IO::Json

#undef LOCTEXT_NAMESPACE
