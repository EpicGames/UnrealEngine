// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGActorHelpers.h"

#include "PCGComponent.h"
#include "PCGIsolatedActor.h"
#include "PCGManagedResource.h"
#include "PCGModule.h"
#include "Elements/PCGSplineMeshParams.h"
#include "Helpers/PCGHelpers.h"

#include "EngineUtils.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkinnedAsset.h"
#include "Animation/AnimBank.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGActorHelpers)

#if WITH_EDITOR
#include "ChangeTracking/PCGChangeTrackingRegistry.h"

#include "Editor.h"
#include "Selection.h"
#include "UnrealEdGlobals.h"
#include "ActorFactories/ActorFactory.h"
#include "Builders/CubeBuilder.h"
#include "Components/BrushComponent.h"
#include "Engine/Level.h"
#include "Editor/IPCGEditorModule.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Blueprint.h"
#include "Modules/ModuleManager.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"

namespace UE::PCGActorHelpers::Local
{
	static float OutlinerUIRefreshDelay = 1.0f;
	static FAutoConsoleVariableRef CvarOutlinerUIRefreshDelay
	(
		TEXT("PCG.Editor.OutlinerRefreshDelay"),
		OutlinerUIRefreshDelay,
		TEXT("The delay (in seconds) before refreshing the Outliner after executing PCG tasks."),
		ECVF_Default
	);
}

#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UInstancedStaticMeshComponent* UPCGActorHelpers::GetOrCreateISMC(AActor* InTargetActor, UPCGComponent* InSourceComponent, uint64 SettingsUID, const FPCGISMCBuilderParameters& InParams)
{
	return GetOrCreateISMC(InTargetActor, InSourceComponent, SettingsUID, FPCGISMComponentBuilderParams(InParams));
}

UPCGManagedISMComponent* UPCGActorHelpers::GetOrCreateManagedISMC(AActor* InTargetActor, UPCGComponent* InSourceComponent, uint64 SettingsUID, const FPCGISMCBuilderParameters& InParams)
{
	return GetOrCreateManagedISMC(InTargetActor, InSourceComponent, SettingsUID, FPCGISMComponentBuilderParams(InParams));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// deprecated
UInstancedStaticMeshComponent* UPCGActorHelpers::GetOrCreateISMC(AActor* InTargetActor, UPCGComponent* InSourceComponent, uint64 SettingsUID, const FPCGISMComponentBuilderParams& InParams)
{
	return GetOrCreateISMC(InTargetActor, InSourceComponent, InParams);
}

// deprecated
UPCGManagedISMComponent* UPCGActorHelpers::GetOrCreateManagedISMC(AActor* InTargetActor, UPCGComponent* InSourceComponent, uint64 SettingsUID, const FPCGISMComponentBuilderParams& InParams)
{
	return GetOrCreateManagedISMC(InTargetActor, InSourceComponent, InParams);
}

UInstancedStaticMeshComponent* UPCGActorHelpers::GetOrCreateISMC(AActor* InTargetActor, IPCGGraphExecutionSource* ExecutionSource, const FPCGISMComponentBuilderParams& InParams, FPCGContext* OptionalContext)
{
	UPCGManagedISMComponent* MISMC = GetOrCreateManagedISMC(InTargetActor, ExecutionSource, InParams, OptionalContext);
	if (MISMC)
	{
		return MISMC->GetComponent();
	}
	else
	{
		return nullptr;
	}
}

UPCGManagedISMComponent* UPCGActorHelpers::GetOrCreateManagedISMC(AActor* InTargetActor, IPCGGraphExecutionSource* InExecutionSource, const FPCGISMComponentBuilderParams& InParams, FPCGContext* OptionalContext)
{
	check(InTargetActor && InExecutionSource);

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedISMC);

	UObject* ExecutionSourceObject = CastChecked<UObject>(InExecutionSource);
	
	FPCGManagedResourceContainerHelper ContainerHelper(InExecutionSource);
	if (!ContainerHelper.IsValid())
	{
		return nullptr;
	}

	FISMComponentDescriptor Descriptor(InParams.Descriptor);

	const UStaticMesh* StaticMesh = Descriptor.StaticMesh;
	if (!StaticMesh)
	{
		return nullptr;
	}

	auto AddTagsToComponent = [ExecutionSourceObject, &InParams](UInstancedStaticMeshComponent* ISMC)
	{
		ISMC->ComponentTags.AddUnique(PCGHelpers::DefaultPCGTag);
		ISMC->ComponentTags.AddUnique(ExecutionSourceObject->GetFName());
			
		for (FName ComponentTag : InParams.Descriptor.ComponentTags)
		{
			ISMC->ComponentTags.AddUnique(ComponentTag);
		}

		const TArray<FString> AdditionalComponentTags = PCGHelpers::GetStringArrayFromCommaSeparatedList(InParams.Descriptor.AdditionalCommaSeparatedTags);
		for (const FString& AdditionalComponentTag : AdditionalComponentTags)
		{
			ISMC->ComponentTags.AddUnique(FName(AdditionalComponentTag));
		}
	};

	// If the component class is invalid, default to HISM.
	// TODO: should this be part of the descriptor changes?
	if (!Descriptor.ComponentClass)
	{
		Descriptor.ComponentClass = UHierarchicalInstancedStaticMeshComponent::StaticClass();
	}

	if (InParams.bAllowDescriptorChanges)
	{
		// It's potentially less efficient to put Nanite meshes inside of HISMs so decay those to ISM in this case.
		// Note the equality here, not a IsA because we do not want to change derived types either
		if (Descriptor.ComponentClass == UHierarchicalInstancedStaticMeshComponent::StaticClass())
		{
			// Done as in InstancedStaticMesh.cpp
#if WITH_EDITOR
			const bool bMeshHasNaniteData = StaticMesh->IsNaniteEnabled();
#else
			const bool bMeshHasNaniteData = StaticMesh->GetRenderData() && StaticMesh->GetRenderData()->HasValidNaniteData();
#endif

			if (bMeshHasNaniteData)
			{
				Descriptor.ComponentClass = UInstancedStaticMeshComponent::StaticClass();
			}
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedISMC::FindMatchingMISMC);
		UPCGManagedISMComponent* MatchingResource = nullptr;
		if(InParams.SettingsCrc.IsValid())
		{
			// If we explicitely are trying to find a RF_Transient through the params or if the Execution source itself is in preview mode
			const bool bTransient = InParams.bTransient || (InExecutionSource->GetExecutionState().IsInPreviewMode());

			ContainerHelper.ForEachManagedResource([&MatchingResource, &InParams, &InTargetActor, &Descriptor, bTransient](UPCGManagedResource* InResource)
			{
				// Early out if already found a match
				if (MatchingResource)
				{
					return;
				}

				if (UPCGManagedISMComponent* Resource = Cast<UPCGManagedISMComponent>(InResource))
				{
					if (!Resource->GetSettingsCrc().IsValid() || Resource->GetSettingsCrc() != InParams.SettingsCrc || 
						(InParams.DataCrc.IsValid() && Resource->GetDataCrc() != InParams.DataCrc) ||
						!Resource->CanBeUsed())
					{
						return;
					}

					if (UInstancedStaticMeshComponent* ISMC = Resource->GetComponent())
					{
						if (IsValid(ISMC) &&
							ISMC->GetOwner() == InTargetActor &&
							ISMC->NumCustomDataFloats == InParams.NumCustomDataFloats &&
							Resource->GetDescriptor() == Descriptor &&
							ISMC->HasAnyFlags(RF_Transient) == bTransient &&
							Algo::Compare(ISMC->GetDefaultCustomPrimitiveData().Data, InParams.CustomPrimitiveData, [](const float LHS, const float RHS) { return FMath::IsNearlyEqual(LHS, RHS); }))
						{
							MatchingResource = Resource;
						}
					}
				}
			});
		}

		if (MatchingResource)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedISMC::MarkAsUsed);
			MatchingResource->MarkAsUsed();

			UInstancedStaticMeshComponent* ISMC = Cast<UInstancedStaticMeshComponent>(MatchingResource->GeneratedComponent.Get());
			if (ensure(ISMC))
			{
				ISMC->Modify(!InExecutionSource->GetExecutionState().IsInPreviewMode());
				AddTagsToComponent(ISMC);
			}

			return MatchingResource;
		}
	}

	// No matching ISM component found, let's create a new one
	InTargetActor->Modify(!InExecutionSource->GetExecutionState().IsInPreviewMode());

	FString ComponentName;

	if (Descriptor.ComponentClass == UHierarchicalInstancedStaticMeshComponent::StaticClass())
	{
		ComponentName = TEXT("HISM_");
	}
	else if (Descriptor.ComponentClass == UInstancedStaticMeshComponent::StaticClass())
	{
		ComponentName = TEXT("ISM_");
	}

	ComponentName += StaticMesh->GetName();

	EObjectFlags ObjectFlags = (InExecutionSource->GetExecutionState().IsInPreviewMode() ? RF_Transient | RF_NonPIEDuplicateTransient : RF_NoFlags);

	// Used for debug data visualization
	if (InParams.bTransient)
	{
		ObjectFlags |= RF_Transient;
	}

	UInstancedStaticMeshComponent* ISMC = NewObject<UInstancedStaticMeshComponent>(InTargetActor, Descriptor.ComponentClass, MakeUniqueObjectName(InTargetActor, Descriptor.ComponentClass, FName(ComponentName)), ObjectFlags);
	Descriptor.InitComponent(ISMC);
	ISMC->SetNumCustomDataFloats(InParams.NumCustomDataFloats);

	ISMC->RegisterComponent();
	InTargetActor->AddInstanceComponent(ISMC);

	if (!ISMC->AttachToComponent(InTargetActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, false)))
	{
		PCGLog::Component::LogComponentAttachmentFailedWarning(OptionalContext);
	}

	if (InParams.CustomPrimitiveData.Num() > FCustomPrimitiveData::NumCustomPrimitiveDataFloats)
	{
		PCGLog::LogWarningOnGraph(FText::Format(NSLOCTEXT("PCGActorHelpers", "TooManyPrimitiveDataFloats", "Too many custom primitive data {0}, max is {1}. Will be truncated."),
			InParams.CustomPrimitiveData.Num(), FCustomPrimitiveData::NumCustomPrimitiveDataFloats), OptionalContext);
	}

	ISMC->SetDefaultCustomPrimitiveDataFloatArray(0, InParams.CustomPrimitiveData);

	// Implementation note: Because we've used the FISMComponentDescriptor here (Descriptor vs InParams.Descriptor) which takes care of the loading, we still need to apply tags manually.
	AddTagsToComponent(ISMC);

	// Create managed resource on source component
	UPCGManagedISMComponent* Resource = NewObject<UPCGManagedISMComponent>(ExecutionSourceObject, NAME_None, InParams.bTransient ? RF_Transient : RF_NoFlags);
	Resource->SetComponent(ISMC);
	Resource->SetDescriptor(Descriptor);
	if (InTargetActor->GetRootComponent())
	{
		Resource->SetRootLocation(InTargetActor->GetRootComponent()->GetComponentLocation());
	}
	
	Resource->SetSettingsCrc(InParams.SettingsCrc);
	ContainerHelper.AddManagedResource(Resource);

	return Resource;
}

UPCGManagedISKMComponent* UPCGActorHelpers::GetOrCreateManagedABMC(AActor* InTargetActor, IPCGGraphExecutionSource* InExecutionSource, const FPCGSkinnedMeshComponentBuilderParams& InParams, FPCGContext* OptionalContext)
{
	check(InTargetActor && InExecutionSource);

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedABMC);

	UObject* ExecutionSourceObject = CastChecked<UObject>(InExecutionSource);

	FPCGManagedResourceContainerHelper ContainerHelper(InExecutionSource);
	if (!ContainerHelper.IsValid())
	{
		return nullptr;
	}

	FSkinnedMeshComponentDescriptor Descriptor(InParams.Descriptor);

	const USkinnedAsset* SkinnedAsset = Descriptor.SkinnedAsset;
	if (!SkinnedAsset)
	{
		return nullptr;
	}

	auto AddTagsToComponent = [ExecutionSourceObject, &InParams](UInstancedSkinnedMeshComponent* ISKMC)
	{
		ISKMC->ComponentTags.AddUnique(PCGHelpers::DefaultPCGTag);
		ISKMC->ComponentTags.AddUnique(ExecutionSourceObject->GetFName());
			
		for (const FName& ComponentTag : InParams.Descriptor.ComponentTags)
		{
			ISKMC->ComponentTags.AddUnique(ComponentTag);
		}

		const TArray<FString> AdditionalComponentTags = PCGHelpers::GetStringArrayFromCommaSeparatedList(InParams.Descriptor.AdditionalCommaSeparatedTags);
		for (const FString& AdditionalComponentTag : AdditionalComponentTags)
		{
			ISKMC->ComponentTags.AddUnique(FName(AdditionalComponentTag));
		}
	};

	if (!Descriptor.ComponentClass)
	{
		Descriptor.ComponentClass = UInstancedSkinnedMeshComponent::StaticClass();
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedABMC::FindMatchingMABMC);
		UPCGManagedISKMComponent* MatchingResource = nullptr;
		if(InParams.SettingsCrc.IsValid())
		{
			ContainerHelper.ForEachManagedResource([&MatchingResource, &InParams, &InTargetActor, &Descriptor](UPCGManagedResource* InResource)
			{
				// Early out if already found a match
				if (MatchingResource)
				{
					return;
				}

				if (UPCGManagedISKMComponent* Resource = Cast<UPCGManagedISKMComponent>(InResource))
				{
					if (!Resource->GetSettingsCrc().IsValid() || Resource->GetSettingsCrc() != InParams.SettingsCrc || !Resource->CanBeUsed())
					{
						return;
					}

					if (UInstancedSkinnedMeshComponent* ISKMC = Resource->GetComponent())
					{
						if (IsValid(ISKMC) &&
							ISKMC->GetOwner() == InTargetActor &&
							ISKMC->GetNumCustomDataFloats() == InParams.NumCustomDataFloats &&
							Resource->GetDescriptor() == Descriptor &&
							ISKMC->HasAnyFlags(RF_Transient) == InParams.bTransient)
						{
							MatchingResource = Resource;
						}
					}
				}
			});
		}

		if (MatchingResource)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedABMC::MarkAsUsed);
			MatchingResource->MarkAsUsed();

			UInstancedSkinnedMeshComponent* ISKMC = Cast<UInstancedSkinnedMeshComponent>(MatchingResource->GeneratedComponent.Get());
			if (ensure(ISKMC))
			{
				ISKMC->Modify(!InExecutionSource->GetExecutionState().IsInPreviewMode());
				AddTagsToComponent(ISKMC);
			}

			return MatchingResource;
		}
	}

	// No matching ABM component found, let's create a new one
	InTargetActor->Modify(!InExecutionSource->GetExecutionState().IsInPreviewMode());

	FString ComponentName = TEXT("ABM_") + SkinnedAsset->GetName();

	EObjectFlags ObjectFlags = (InExecutionSource->GetExecutionState().IsInPreviewMode() ? RF_Transient | RF_NonPIEDuplicateTransient : RF_NoFlags);

	// Used for debug data visualization
	if (InParams.bTransient)
	{
		ObjectFlags |= RF_Transient;
	}

	UInstancedSkinnedMeshComponent* ISKMC = NewObject<UInstancedSkinnedMeshComponent>(InTargetActor, Descriptor.ComponentClass, MakeUniqueObjectName(InTargetActor, Descriptor.ComponentClass, FName(ComponentName)), ObjectFlags);
	Descriptor.InitComponent(ISKMC);
	ISKMC->SetNumCustomDataFloats(InParams.NumCustomDataFloats);

	ISKMC->RegisterComponent();
	InTargetActor->AddInstanceComponent(ISKMC);

	if (!ISKMC->AttachToComponent(InTargetActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, false)))
	{
		PCGLog::Component::LogComponentAttachmentFailedWarning(OptionalContext);
	}
	
	// Implementation note: Because we've used the FSkinnedMeshComponentDescriptor here (Descriptor vs InParams.Descriptor) which takes care of the loading, we still need to apply tags manually.
	AddTagsToComponent(ISKMC);

	// Create managed resource on source component
	UPCGManagedISKMComponent* Resource = NewObject<UPCGManagedISKMComponent>(ExecutionSourceObject, NAME_None, InParams.bTransient ? RF_Transient : RF_NoFlags);
	Resource->SetComponent(ISKMC);
	Resource->SetDescriptor(Descriptor);
	if (InTargetActor->GetRootComponent())
	{
		Resource->SetRootLocation(InTargetActor->GetRootComponent()->GetComponentLocation());
	}
	
	Resource->SetSettingsCrc(InParams.SettingsCrc);

	ContainerHelper.AddManagedResource(Resource);

	return Resource;
}

// deprecated
USplineMeshComponent* UPCGActorHelpers::GetOrCreateSplineMeshComponent(AActor* InTargetActor, UPCGComponent* InSourceComponent, uint64 SettingsUID, const FPCGSplineMeshComponentBuilderParameters& InParams)
{
	return GetOrCreateSplineMeshComponent(InTargetActor, InSourceComponent, InParams);
}


// deprecated
UPCGManagedSplineMeshComponent* UPCGActorHelpers::GetOrCreateManagedSplineMeshComponent(AActor* InTargetActor, UPCGComponent* InSourceComponent, uint64 SettingsUID, const FPCGSplineMeshComponentBuilderParameters& InParams)
{
	return GetOrCreateManagedSplineMeshComponent(InTargetActor, InSourceComponent, InParams);
}

USplineMeshComponent* UPCGActorHelpers::GetOrCreateSplineMeshComponent(AActor* InTargetActor, IPCGGraphExecutionSource* InExecutionSource, const FPCGSplineMeshComponentBuilderParameters& InParams, FPCGContext* OptionalContext)
{
	UPCGManagedSplineMeshComponent* ManagedComponent = GetOrCreateManagedSplineMeshComponent(InTargetActor, InExecutionSource, InParams, OptionalContext);
	if (ManagedComponent)
	{
		return ManagedComponent->GetComponent();
	}
	else
	{
		return nullptr;
	}
}

UPCGManagedSplineMeshComponent* UPCGActorHelpers::GetOrCreateManagedSplineMeshComponent(AActor* InTargetActor, IPCGGraphExecutionSource* InExecutionSource, const FPCGSplineMeshComponentBuilderParameters& InParams, FPCGContext* OptionalContext)
{
	check(InTargetActor && InExecutionSource);

	FPCGManagedResourceContainerHelper ContainerHelper(InExecutionSource);
	if (!ContainerHelper.IsValid())
	{
		return nullptr;
	}

	const UStaticMesh* StaticMesh = InParams.Descriptor.StaticMesh;

	if (!StaticMesh || !InParams.Descriptor.ComponentClass)
	{
		return nullptr;
	}

	auto AttachToRoot = [InTargetActor, OptionalContext](USceneComponent* Component)
	{
		check(Component);
		USceneComponent* RootComponent = InTargetActor->GetRootComponent();

		// Implementation note: since the data passed to the params here is in world space,
		// We need the transform on the spline mesh component to be the identity - in world space, because unlike the ISMs
		// where we can set the instances and specify the data is in world space, we can't do that here.
		FAttachmentTransformRules AttachmentRules(EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, /*bInWeldSimulatedBodies=*/false);
			
		// If already attached, detach first.
		if (Component->GetAttachParent())
		{
			Component->DetachFromComponent(FDetachmentTransformRules(AttachmentRules, /*bInCallModify=*/false));
		}

		if (!Component->AttachToComponent(RootComponent, AttachmentRules))
		{
			PCGLog::Component::LogComponentAttachmentFailedWarning(OptionalContext);
		}
	};

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedSplineMeshComponent);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedSplineMeshComponent::FindMatchingManagedSplineMeshComponent);

		UPCGManagedSplineMeshComponent* MatchingResource = nullptr;
		if(InParams.SettingsCrc.IsValid())
		{
			ContainerHelper.ForEachManagedResource([&MatchingResource, &InParams, &InTargetActor](UPCGManagedResource* InResource)
			{
				// Early out if already found a match
				if (MatchingResource)
				{
					return;
				}

				if (UPCGManagedSplineMeshComponent* Resource = Cast<UPCGManagedSplineMeshComponent>(InResource))
				{
					if (!Resource->GetSettingsCrc().IsValid() || Resource->GetSettingsCrc() != InParams.SettingsCrc || !Resource->CanBeUsed())
					{
						return;
					}

					if (USplineMeshComponent* SplineMeshComponent = Resource->GetComponent())
					{
						if (IsValid(SplineMeshComponent)
							&& SplineMeshComponent->GetOwner() == InTargetActor
							&& Resource->GetDescriptor() == InParams.Descriptor
							&& Resource->GetSplineMeshParams() == InParams.SplineMeshParams)
						{
							MatchingResource = Resource;
						}
					}
				}
			});
		}

		if (MatchingResource)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::GetOrCreateManagedSplineMeshComponent::MarkAsUsed);
			MatchingResource->MarkAsUsed();

			USplineMeshComponent* SplineMeshComponent = MatchingResource->GetComponent();
			if (ensure(SplineMeshComponent))
			{
				SplineMeshComponent->Modify(!InExecutionSource->GetExecutionState().IsInPreviewMode());

				const USceneComponent* RootComponent = InTargetActor->GetRootComponent();
				
				// If mobility differs between Root Component and Spline Mesh Component, or it is not attached, we need to re-attach it
				if (SplineMeshComponent->GetAttachParent() != RootComponent 
					|| (RootComponent && SplineMeshComponent->Mobility != RootComponent->Mobility)
					|| (SplineMeshComponent->Mobility != InParams.Descriptor.Mobility))
				{
					SplineMeshComponent->Mobility = InParams.Descriptor.Mobility;
					AttachToRoot(SplineMeshComponent);
				}
			}

			return MatchingResource;
		}
	}

	// No matching component found, let's create a new one.
	InTargetActor->Modify(!InExecutionSource->GetExecutionState().IsInPreviewMode());

	FString ComponentName = TEXT("PCGSplineMeshComponent_") + StaticMesh->GetName();
	const EObjectFlags ObjectFlags = (InExecutionSource->GetExecutionState().IsInPreviewMode() ? RF_Transient | RF_NonPIEDuplicateTransient : RF_NoFlags);
	USplineMeshComponent* SplineMeshComponent = InParams.Descriptor.CreateComponent(InTargetActor, MakeUniqueObjectName(InTargetActor, InParams.Descriptor.ComponentClass, FName(ComponentName)), ObjectFlags);

	// Init Component
	{
		const FPCGSplineMeshParams& SplineMeshParams = InParams.SplineMeshParams;
		SplineMeshComponent->SetStartAndEnd(SplineMeshParams.StartPosition, SplineMeshParams.StartTangent, SplineMeshParams.EndPosition, SplineMeshParams.EndTangent);
		SplineMeshComponent->SetStartRollDegrees(SplineMeshParams.StartRollDegrees);
		SplineMeshComponent->SetEndRollDegrees(SplineMeshParams.EndRollDegrees);
		SplineMeshComponent->SetStartScale(SplineMeshParams.StartScale);
		SplineMeshComponent->SetEndScale(SplineMeshParams.EndScale);
		SplineMeshComponent->SetForwardAxis((ESplineMeshAxis::Type)SplineMeshParams.ForwardAxis);
		SplineMeshComponent->SetSplineUpDir(SplineMeshParams.SplineUpDir);
		SplineMeshComponent->SetStartOffset(SplineMeshParams.StartOffset);
		SplineMeshComponent->SetEndOffset(SplineMeshParams.EndOffset);
		SplineMeshComponent->SplineParams.NaniteClusterBoundsScale = SplineMeshParams.NaniteClusterBoundsScale;
		SplineMeshComponent->SplineBoundaryMin = SplineMeshParams.SplineBoundaryMin;
		SplineMeshComponent->SplineBoundaryMax = SplineMeshParams.SplineBoundaryMax;
		SplineMeshComponent->bSmoothInterpRollScale = SplineMeshParams.bSmoothInterpRollScale;
	}

	SplineMeshComponent->RegisterComponent();
	InTargetActor->AddInstanceComponent(SplineMeshComponent);

	AttachToRoot(SplineMeshComponent);

	UObject* ExecutionSourceObject = CastChecked<UObject>(InExecutionSource);
	SplineMeshComponent->ComponentTags.Add(ExecutionSourceObject->GetFName());
	SplineMeshComponent->ComponentTags.Add(PCGHelpers::DefaultPCGTag);

	// Create managed resource on source component
	UPCGManagedSplineMeshComponent* Resource = NewObject<UPCGManagedSplineMeshComponent>(ExecutionSourceObject);
	Resource->SetComponent(SplineMeshComponent);
	Resource->SetDescriptor(InParams.Descriptor);
	Resource->SetSplineMeshParams(InParams.SplineMeshParams);
	Resource->SetSettingsCrc(InParams.SettingsCrc);
	
	ContainerHelper.AddManagedResource(Resource);

	return Resource;
}

bool UPCGActorHelpers::DeleteActors(UWorld* World, const TArray<TSoftObjectPtr<AActor>>& ActorsToDelete)
{
	if (!World || ActorsToDelete.Num() == 0)
	{
		return true;
	}

#if WITH_EDITOR
	if (IPCGEditorModule* EditorModule = IPCGEditorModule::Get())
	{
		EditorModule->SetOutlinerUIRefreshDelay(UE::PCGActorHelpers::Local::OutlinerUIRefreshDelay);
	}
#endif

	TArray<AActor*> ActorsToDestroy;
	ActorsToDestroy.Reserve(ActorsToDelete.Num());

	// Gather actors to destroy
	for (const TSoftObjectPtr<AActor>& ManagedActor : ActorsToDelete)
	{
		// @todo_pcg: Revisit this GetWorld() check when fixing UE-215065
		if (AActor* Actor = ManagedActor.Get(); Actor && Actor->GetWorld())
		{
			ActorsToDestroy.Add(Actor);
		}
	}

#if WITH_EDITOR
	if(!ActorsToDestroy.IsEmpty())
	{
		// Deselect them if needed
		FDeselectedActorsEvent DeselectedActorsEvent(ActorsToDestroy);
	}
#endif

	// Destroy actors
	for (AActor* ActorToDestroy : ActorsToDestroy)
	{
		if (!ensure(World->DestroyActor(ActorToDestroy)))
		{
			UE_LOGF(LogPCG, Warning, "Actor %ls failed to be destroyed.", *ActorToDestroy->GetPathName());
		}
	}

	return true;
}

void UPCGActorHelpers::ForEachActorInLevel(ULevel* Level, TSubclassOf<AActor> ActorClass, TFunctionRef<bool(AActor*)> Callback)
{
	if (!Level)
	{
		return;
	}

	for (AActor* Actor : Level->Actors)
	{
		if (Actor && Actor->IsA(ActorClass))
		{
			if (!Callback(Actor))
			{
				break;
			}
		}
	}
}

void UPCGActorHelpers::ForEachActorInWorld(UWorld* World, TSubclassOf<AActor> ActorClass, TFunctionRef<bool(AActor*)> Callback)
{
	if (!World)
	{
		return;
	}

	for (TActorIterator<AActor> It(World, ActorClass); It; ++It)
	{
		if (AActor* Actor = *It)
		{
			if (!Callback(Actor))
			{
				break;
			}
		}
	}
}

AActor* UPCGActorHelpers::SpawnDefaultActor(UWorld* World, ULevel* Level, TSubclassOf<AActor> ActorClass, FName BaseName, const FTransform& Transform, AActor* Parent)
{
	if (!World || !ActorClass)
	{
		return nullptr;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.Name = MakeUniqueObjectName(Level ? Level : World->GetCurrentLevel(), ActorClass, BaseName);

	return SpawnDefaultActor(World, Level, ActorClass, Transform, SpawnParams, Parent);
}

AActor* UPCGActorHelpers::SpawnDefaultActor(UWorld* World, ULevel* Level, TSubclassOf<AActor> ActorClass, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams, AActor* Parent)
{
	FActorSpawnParameters ActorSpawnParams(InSpawnParams);
	if (Level)
	{
		ActorSpawnParams.OverrideLevel = Level;
	}

	FSpawnDefaultActorParams SpawnDefaultActorParams(World, ActorClass, Transform, ActorSpawnParams);
	
	SpawnDefaultActorParams.Parent = Parent;
	
	return SpawnDefaultActor(SpawnDefaultActorParams);
}

AActor* UPCGActorHelpers::SpawnDefaultActor(const FSpawnDefaultActorParams& Params)
{
	if (!Params.World || !Params.ActorClass)
	{
		return nullptr;
	}

	FActorSpawnParameters SpawnParams = Params.SpawnParams;
	if (!SpawnParams.OverrideLevel)
	{
		SpawnParams.OverrideLevel = Params.World->PersistentLevel;
	}

	if (PCGHelpers::IsRuntimeOrPIE() || Params.bIsPreviewActor)
	{
		SpawnParams.ObjectFlags |= RF_Transient;

		// Only allow actors to be duplicated in PIE if they aren't generated at runtime (Treat editor as viewport code path which runs runtime gen in an editor world)
		if (!Params.bIsRuntime)
		{
			SpawnParams.ObjectFlags |= RF_NonPIEDuplicateTransient;
		}
	}

#if WITH_EDITOR
	// Capture label before potentially changing it for previewing
	FString DefaultLabel = SpawnParams.Name.IsNone() ? FString() : SpawnParams.Name.ToString();

	// If we are spawning a non-runtime preview actor in a world partition world, we need to assign it an external package as Data Layers are not supported on non OFPA actors.
	// Note: this could at somepoint become an option in the FActorSpawnParameters to allow RF_Transient actors to create their own package in UWorld::SpawnActor.
	if (Params.bIsPreviewActor && !Params.bIsRuntime && SpawnParams.OverrideLevel->GetWorldPartition() && !Params.World->IsGameWorld())
	{
		// Remove trailing _C from BP generated class
		FString ActorClassName = Params.ActorClass->GetName();
		if (Cast<UBlueprint>(Params.ActorClass->ClassGeneratedBy))
		{
			ActorClassName.RemoveFromEnd(TEXT("_C"), ESearchCase::CaseSensitive);
		}

		SpawnParams.Name = FActorSpawnUtils::MakeUniqueActorName(SpawnParams.OverrideLevel, Params.ActorClass, *ActorClassName, /*bGloballyUniqueName*/true);
		
		// If Label wasn't empty add suffix to it, if not use basename as label
		if (DefaultLabel.IsEmpty())
		{
			DefaultLabel = ActorClassName;
		}
		
		SpawnParams.OverridePackage = UPCGActorHelpers::CreatePreviewPackage(SpawnParams.OverrideLevel, SpawnParams.Name.ToString());
		if (SpawnParams.OverridePackage)
		{
			SpawnParams.bCreateActorPackage = false;
		}
	}

	if (SpawnParams.InitialActorLabel.IsEmpty())
	{
		SpawnParams.InitialActorLabel = DefaultLabel;
	}

	// Avoid creating an actor package for runtime actors
	SpawnParams.bCreateActorPackage &= !Params.bIsRuntime;

	// Find External Data Layer if it exists so we can create scope that will allow actor to be properly created
	const UExternalDataLayerInstance* ExternalDataLayerInstance = nullptr;
	bool bPushedContext = false;
	// No need to do any DataLayer assignment in a game world
	if (!Params.World->IsGameWorld())
	{
		for (const UDataLayerInstance* DataLayerInstance : Params.DataLayerInstances)
		{
			if (const UExternalDataLayerInstance* Found = Cast<UExternalDataLayerInstance>(DataLayerInstance))
			{
				ExternalDataLayerInstance = Found;
				break;
			}
		}

		// Avoid relying on the Editor Context at all
		UActorEditorContextSubsystem::Get()->PushContext();
		bPushedContext = true;
	}

	ON_SCOPE_EXIT
	{ 
		if (bPushedContext)
		{
			UActorEditorContextSubsystem::Get()->PopContext();
		}
	};

	// Specify EDL we want to use if any for spawning this actor
	FScopedOverrideSpawningLevelMountPointObject EDLScope(ExternalDataLayerInstance ? ExternalDataLayerInstance->GetExternalDataLayerAsset() : nullptr);

	if (IPCGEditorModule* EditorModule = IPCGEditorModule::Get())
	{
		EditorModule->SetOutlinerUIRefreshDelay(UE::PCGActorHelpers::Local::OutlinerUIRefreshDelay);
	}
#endif // WITH_EDITOR

	AActor* NewActor = Params.World->SpawnActor(*Params.ActorClass, &Params.Transform, SpawnParams);
	
	if (!NewActor)
	{
		return nullptr;
	}

	// HACK: until UE-62747 is fixed, we have to force set the scale after spawning the actor
	NewActor->SetActorRelativeScale3D(Params.Transform.GetScale3D());

#if WITH_EDITOR
	if (!Params.World->IsGameWorld())
	{
		NewActor->SetHLODLayer(Params.HLODLayer);

		// Add remaining DataLayers (except External which was done on spawn)
		for (const UDataLayerInstance* DataLayerInstance : Params.DataLayerInstances)
		{
			if (DataLayerInstance != ExternalDataLayerInstance)
			{
				DataLayerInstance->AddActor(NewActor);
			}
		}

		// If the spawed actor is a volume, we potentially need to fix its brush/brush setup/brush component
		if (AVolume* NewVolume = Cast<AVolume>(NewActor))
		{
			UCubeBuilder* DefaultBrushBuilder = NewObject<UCubeBuilder>();
			UActorFactory::CreateBrushForVolumeActor(NewVolume, DefaultBrushBuilder);
		}
	}
#endif // WITH_EDITOR

	USceneComponent* RootComponent = NewActor->GetRootComponent();
	if (!RootComponent)
	{
		RootComponent = NewObject<USceneComponent>(NewActor, USceneComponent::GetDefaultSceneRootVariableName(), RF_Transactional);
		RootComponent->SetWorldTransform(Params.Transform);

		NewActor->SetRootComponent(RootComponent);
		NewActor->AddInstanceComponent(RootComponent);

		RootComponent->RegisterComponent();
	}

	if (Params.bForceStaticMobility)
	{
		RootComponent->Mobility = EComponentMobility::Static;
	}

#if WITH_EDITOR
	RootComponent->bVisualizeComponent = true;
#endif // WITH_EDITOR

	if (Params.Parent)
	{
		NewActor->AttachToActor(Params.Parent, FAttachmentTransformRules::KeepWorldTransform);
	}

	return NewActor;
}

#if WITH_EDITOR
UPackage* UPCGActorHelpers::CreatePreviewPackage(ULevel* InLevel, const FString& InActorName)
{
	check(InLevel);
	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> ActorPath;
	ActorPath += InLevel->GetPathName();
	ActorPath += TEXT(".Preview_");
	ActorPath += InActorName;

	UPackage* ActorPackage = ULevel::CreateActorPackage(InLevel->GetPackage(), InLevel->GetActorPackagingScheme(), ActorPath.ToString());
	if (ActorPackage)
	{
		// Set dirty flag to false before adding transient flag so that UX updates properly
		ActorPackage->SetDirtyFlag(false);
		ActorPackage->SetFlags(RF_Transient);
	}
	return ActorPackage;
}
#endif // WITH_EDITOR

FIntVector UPCGActorHelpers::GetCellCoord(FVector InPosition, uint32 InGridSize, bool bUse2DGrid)
{
	check(InGridSize > 0);

	FVector Temp = InPosition / InGridSize;

	// In case of 2D grid, Z coordinate is always 0
	return FIntVector(
		FMath::FloorToInt(Temp.X),
		FMath::FloorToInt(Temp.Y),
		bUse2DGrid ? 0 : FMath::FloorToInt(Temp.Z)
	);
}

FVector UPCGActorHelpers::GetCellCenter(const FVector& InPosition, uint32 InGridSize, bool bUse2DGrid)
{
	const FIntVector GridCoords = GetCellCoord(InPosition, InGridSize, bUse2DGrid);
	return FVector(GridCoords.X + 0.5, GridCoords.Y + 0.5, GridCoords.Z + 0.5) * InGridSize;
}

int UPCGActorHelpers::ComputeHashFromActorTagsAndReference(const AActor* InActor, const bool bIncludeTags, const bool bIncludeActorReference)
{
	if (!InActor)
	{
		return 0;
	}

	int Result = 0;
	if (bIncludeTags)
	{
		for (const FName Tag : InActor->Tags)
		{
			Result = HashCombineFast(Result, GetTypeHash(Tag));
		}
	}

	if (bIncludeActorReference)
	{
		Result = HashCombineFast(Result, GetTypeHash(InActor->GetPathName()));
	}

	return Result;
}

TArray<APCGIsolatedActor*> UPCGActorHelpers::GetIsolatedActorsOriginatingFrom(const AActor* InActor)
{
	TArray<APCGIsolatedActor*> IsolatedActors;

	if (InActor)
	{
		TSoftObjectPtr<const AActor> ActorPtr(InActor);

		ForEachActorInWorld(InActor->GetWorld(), APCGIsolatedActor::StaticClass(), [&IsolatedActors, &ActorPtr](AActor* IsolatedActorCandidate)
		{
			APCGIsolatedActor* IsolatedActor = Cast<APCGIsolatedActor>(IsolatedActorCandidate);
			if (IsolatedActor && IsolatedActor->Origin == ActorPtr)
			{
				IsolatedActors.Add(IsolatedActor);
			}

			return true;
		});
	}

	return IsolatedActors;
}

AActor* UPCGActorHelpers::IsolateFromActor(AActor* Actor, const TArray<FName>& RequiredTags)
{
	FPCGMoveResourceParams Params
	{
		.TemplateTargetClass = APCGIsolatedActor::StaticClass(),
		.RequiredTags = RequiredTags
	};

	return IsolateFromActor(Actor, Params);
}

AActor* UPCGActorHelpers::IsolateFromActor(AActor* Actor, const FPCGMoveResourceParams& InParams)
{
	UPCGComponent* PCGComponent = Actor ? Actor->GetComponentByClass<UPCGComponent>() : nullptr;

	if (!PCGComponent)
	{
		return nullptr;
	}

	// Will store data temporarily (output from the ClearPCGLink) until we move it to the created actor
	FPCGMoveResourceParams Params = InParams;

	if (!Params.TemplateTargetClass)
	{
		Params.TemplateTargetClass = APCGIsolatedActor::StaticClass();
	}

	check(Params.TemplateTargetClass);

	FPCGDataCollection TempCollection;
	bool bMoveCollectionToIsolatedActor = !Params.TargetDataCollection && Params.TemplateTargetClass->IsChildOf(APCGIsolatedActor::StaticClass());

	if (bMoveCollectionToIsolatedActor)
	{
		Params.TargetDataCollection = &TempCollection;
	}

	AActor* IsolatedActor = PCGComponent->ClearPCGLink(Params);

	// Setup isolated actor data if we're created one.
	if (APCGIsolatedActor* IsolatedPCGActor = Cast<APCGIsolatedActor>(IsolatedActor))
	{
		IsolatedPCGActor->Origin = Actor;

		// Move collection to isolated actor
		if (bMoveCollectionToIsolatedActor)
		{
			IsolatedPCGActor->Data = MoveTemp(*Params.TargetDataCollection);
		}

		// Re-outer data
		for (FPCGTaggedData& TaggedData : IsolatedPCGActor->Data.TaggedData)
		{
			TaggedData.Data->VisitDataNetwork([IsolatedPCGActor](const UPCGData* InData)
			{
				const_cast<UPCGData*>(InData)->Rename(nullptr, IsolatedPCGActor, REN_DontCreateRedirectors);
			});
		}
	}

	// @todo_pcg: Push tags onto newly created actor?

	// Remove PCG Generated tags, so they don't get ignored elsewhere.
	// This is a bit more complex, as we need to tackle the "new actor" case and the "moved in place" case.
	if(IsolatedActor && IsolatedActor != Params.GetTarget<AActor>())
	{
		IsolatedActor->ForEachComponent(/*bIncludeFromChildActors=*/true, [](UActorComponent* Component)
		{
			Component->ComponentTags.Remove(PCGHelpers::DefaultPCGTag);
		});
	}
	// In the case of a general target, we need to make sure that we untag just the proper components,
	// And not any that would have been tracked by a PCG component - especially in the "move to self" case.
	// Note that this isn't foolproof though, because we could have components that are owned by a PCG component on another actor
	// but we don't know this unfortunately.
	else if (AActor* TargetActor = Params.GetTarget<AActor>())
	{
		TArray<UPCGComponent*> PCGComponents;
		TargetActor->GetComponents<UPCGComponent>(PCGComponents);

		TargetActor->ForEachComponent(/*bIncludeFromChildActors=*/true, [&PCGComponents](UActorComponent* Component)
		{
			if (Component->ComponentTags.Contains(PCGHelpers::DefaultPCGTag))
			{
				bool bRemoveTag = true;
				const UObject* PtrOnComponent = Component;

				TArrayView<const UObject*> ViewOnComponent = MakeArrayView(&PtrOnComponent, 1);

				for (UPCGComponent* PCGComponent : PCGComponents)
				{
					bRemoveTag &= !PCGComponent->IsAnyObjectManagedByResource(ViewOnComponent);
				}

				if (bRemoveTag)
				{
					Component->ComponentTags.Remove(PCGHelpers::DefaultPCGTag);
				}
			}
		});
	}

	return IsolatedActor;
}

TArray<IPCGGraphExecutionSource*> UPCGActorHelpers::GetExecutionSourcesFromSelectionKey(const FPCGGetExecutionSourcesFromSelectionKeyParams& InParams)
{
	const AActor* InActor = Cast<const AActor>(InParams.SelectionKey.GetObjectFromPath());

	if (!InActor)
	{
		return TArray<IPCGGraphExecutionSource*>();
	}

	TArray<UActorComponent*, TInlineAllocator<64>> ActorComponents;

	if (InParams.ActorFilter == EPCGActorFilter::Self || InParams.ActorFilter == EPCGActorFilter::Original)
	{
		InActor->GetComponents(UPCGComponent::StaticClass(), ActorComponents);
	}
	else if (InParams.ActorFilter == EPCGActorFilter::Parent || (InParams.ActorFilter == EPCGActorFilter::Root && !InActor->GetParentActor()))
	{
		TArray<AActor*> ActorsToCheck;
		InActor->GetAllChildActors(ActorsToCheck, /*bIncludeDescendants=*/InParams.ActorFilter == EPCGActorFilter::Root);
		ActorsToCheck.Add(const_cast<AActor*>(InActor));
		TArray<UActorComponent*, TInlineAllocator<64>> TempActorComponents;
		for (AActor* Current : ActorsToCheck)
		{
			// TempActorComponents is reset in GetComponents
			Current->GetComponents(UPCGComponent::StaticClass(), TempActorComponents);
			ActorComponents.Append(TempActorComponents);
		}
	}

	TArray<IPCGGraphExecutionSource*> FoundExecutionSources;
	FoundExecutionSources.Reserve(ActorComponents.Num());
	for (UActorComponent* ActorComponent : ActorComponents)
	{
		if (IPCGGraphExecutionSource* ExecutionSource = Cast<IPCGGraphExecutionSource>(ActorComponent))
		{
			FoundExecutionSources.Add(ExecutionSource);
		}
	}

	return FoundExecutionSources;
}
