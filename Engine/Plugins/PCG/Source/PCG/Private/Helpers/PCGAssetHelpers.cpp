// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGAssetHelpers.h"

#include "PCGGraph.h"
#include "Elements/Blueprint/PCGBlueprintBaseElement.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/BlueprintSupport.h"

#if WITH_EDITOR

bool FPCGAssetHelpers::GetGraphAssetRegistryData(const FAssetData& AssetData, FPCGAssetHelpers::FGraphAssetOutput& Output)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Only exposing the instance if it's parent graph is defined for instances. Otherwise, it is not interesting.
	// Also march up the hierarchy to find overrides for category and description.
	// We don't look for override in titles because if we have no override for an instance but the parent has an override we will have 2 times the same entry in the palette.
	auto GetRecursiveData = [&AssetRegistryModule, &Output](const FAssetData& InAssetData, auto&& Recurse)
	{
		if (InAssetData.IsInstanceOf<UPCGGraphInstance>())
		{
			const FSoftObjectPath ParentGraph(InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, Graph)));
			if (ParentGraph.IsNull())
			{
				return false;
			}

			if(Output.Category.IsEmpty())
			{
				Output.Category = InAssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, bOverrideCategory))
					? InAssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, Category))
					: FText();
			}

			if(Output.Description.IsEmpty())
			{
				Output.Description = InAssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, bOverrideDescription))
					? InAssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, Description))
					: FText();
			}

			// Asset data are not big so that should not be that big of a deal, but they are copied all the time.
			// If we ever have performances issues, might be good to have a cache.
			const FAssetData ParentAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(ParentGraph);
			return ParentAssetData.IsValid() && Recurse(ParentAssetData, Recurse);
		}
		else
		{
			if (Output.Category.IsEmpty())
			{
				Output.Category = InAssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGGraph, Category));
			}

			if (Output.Description.IsEmpty())
			{
				Output.Description = InAssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGGraph, Description));
			}

			// Implementation note: pins are only present at the graph level
			if (UPCGGraph* Graph = Cast<UPCGGraph>(InAssetData.FastGetAsset(/*bLoad=*/false)))
			{
				Output.CachedPins = Graph->BuildCachedPins();
			}
			else
			{
				FString TempString;
				InAssetData.GetTagValue<FString>(GET_MEMBER_NAME_CHECKED(UPCGGraph, CachedPins), TempString);
				if (!TempString.IsEmpty())
				{
					FPCGAssetCachedPins::StaticStruct()->ImportText(*TempString, &Output.CachedPins, nullptr, 0, nullptr, FString{});
				}
			}

			return true;
		}
	};

	return GetRecursiveData(AssetData, GetRecursiveData);
}

bool FPCGAssetHelpers::GetBlueprintAssetRegistryData(const FAssetData& AssetData, FPCGAssetHelpers::FBlueprintAssetOutput& Output)
{
	//@todo_pcg: validate that the asset is indeed of a BP class. See PCGEditorUtils::ForEachPCGBlueprintAssetData?
	Output.GeneratedClass = FSoftClassPath(AssetData.GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath));
	Output.Category = AssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGBlueprintBaseElement, Category));
	Output.Description = AssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGBlueprintBaseElement, Description));
	Output.bOnlyExposePreconfiguredSettings = AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGBlueprintBaseElement, bOnlyExposePreconfiguredSettings));
	Output.bEnabledPreconfiguredSettings = AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UPCGBlueprintBaseElement, bEnablePreconfiguredSettings));

	if (UPCGBlueprintBaseElement* BPElement = Cast<UPCGBlueprintBaseElement>(AssetData.FastGetAsset(/*bLoad=*/false)))
	{
		Output.CachedPins = BPElement->BuildCachedPins();
	}
	else
	{
		FString TempString;
		AssetData.GetTagValue<FString>(GET_MEMBER_NAME_CHECKED(UPCGBlueprintBaseElement, CachedPins), TempString);
		if (!TempString.IsEmpty())
		{
			FPCGAssetCachedPins::StaticStruct()->ImportText(*TempString, &Output.CachedPins, nullptr, 0, nullptr, FString{});
		}
	}

	return true;
}

bool FPCGAssetHelpers::GetSettingsAssetRegistryData(const FAssetData& AssetData, FPCGAssetHelpers::FSettingsAssetOutput& Output)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Implementation note - this supports settings + instances of settings.
	// However it will not support instance of instances, but the instance class isn't built this way yet.
	auto ProcessSettings = [&AssetRegistryModule, &Output](const FAssetData& InAssetData)
	{
		if(InAssetData.IsInstanceOf<UPCGSettings>())
		{
			Output.Category = InAssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGSettings, Category));
			Output.Description = InAssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGSettings, Description));

			// Asset is already loaded, we'll use the live values instead
			if (UPCGSettings* Settings = Cast<UPCGSettings>(InAssetData.FastGetAsset(/*bLoad=*/false)))
			{
				Output.CachedPins = Settings->BuildCachedPins();
			}
			else
			{
				FString TempString;
				InAssetData.GetTagValue<FString>(GET_MEMBER_NAME_CHECKED(UPCGSettings, CachedPins), TempString);
				if (!TempString.IsEmpty())
				{
					FPCGAssetCachedPins::StaticStruct()->ImportText(*TempString, &Output.CachedPins, nullptr, 0, nullptr, FString{});
				}
			}

			return true;
		}
		else
		{
			return false;
		}
	};

	if (AssetData.IsInstanceOf<UPCGSettings>())
	{
		return ProcessSettings(AssetData);
	}
	else if (AssetData.IsInstanceOf<UPCGSettingsInstance>())
	{
		const FSoftObjectPath OriginalSettingsPath(AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGSettingsInstance, Settings)));
		if (OriginalSettingsPath.IsNull())
		{
			return false; // can't do anything here unfortunately
		}

		const FAssetData OriginalAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(OriginalSettingsPath);
		return OriginalAssetData.IsValid() && ProcessSettings(OriginalAssetData);
	}
	else
	{
		return false;
	}
}

#endif // WITH_EDITOR