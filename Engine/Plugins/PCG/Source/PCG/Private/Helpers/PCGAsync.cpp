// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGAsync.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGPoint.h"

#include "Tasks/Task.h"

namespace FPCGAsync
{
	TAutoConsoleVariable<bool> ConsoleVar::CVarDisableAsyncTimeSlicing {
		TEXT("pcg.DisableAsyncTimeSlicing"),
		false,
		TEXT("To help debug, we can disable time slicing for tasks (when executing Element from worker thread)")
	};

	TAutoConsoleVariable<bool> ConsoleVar::CVarDisableAsyncTimeSlicingOnGameThread {
		TEXT("pcg.DisableAsyncTimeSlicingOnGameThread"),
		false,
		TEXT("To help debug, we can disable time slicing for tasks (when executing Element from main thread).")
	};

	TAutoConsoleVariable<float> ConsoleVar::CVarAsyncOutOfTickBudgetInMilliseconds{
		TEXT("pcg.Async.OutOfTickBudgetInMilliseconds"),
		5.0f,
		TEXT("Allocated time in milliseconds per task when running async tasks out of tick (depends on pcg.DisableAsyncTimeSlicing being false)")
	};

	TAutoConsoleVariable<int32> ConsoleVar::CVarAsyncOverrideChunkSize{
		TEXT("pcg.AsyncOverrideChunkSize"),
		-1,
		TEXT("For quick benchmarking, we can override the value of chunk size for async processing. Any negative value is discarded.")
	};

	int32 GetNumTasks(FPCGAsyncState* AsyncState, int32 InDefaultNumTasks)
	{
		// Get number of available threads from the async state
		// Calculate Number of Tasks to spawn, if we have no AsyncState run only 1 task (reentrance)
		int32 NumTasks = AsyncState ? FMath::Max(1, InDefaultNumTasks) : 1;
		// If AsyncState limits the Max Tasks apply here
		if (AsyncState && AsyncState->NumAvailableTasks > 0)
		{
			NumTasks = FMath::Min(AsyncState->NumAvailableTasks, NumTasks);
		}

		return NumTasks;
	}

	void PrivateAsyncPointProcessing(FPCGAsyncState* AsyncState, int32 MinIterationsPerTask, int32 NumIterations, TArray<FPCGPoint>& OutPoints, const TFunction<int32(int32, int32)>& IterationInnerLoop)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncPointProcessing);

		check(MinIterationsPerTask > 0 && NumIterations >= 0);
		if (MinIterationsPerTask <= 0 || NumIterations <= 0)
		{
			return;
		}

		const int32 NumTasks = GetNumTasks(AsyncState, NumIterations / MinIterationsPerTask);
		const int32 IterationsPerTask = NumIterations / NumTasks;
		const int32 NumFutures = NumTasks - 1;
		 
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncPointProcessing::AllocatingArray);
			// Pre-reserve the out points array
			OutPoints.SetNumUninitialized(NumIterations);
		}

		// Setup [current, last, nb points] data per dispatch
		TArray<UE::Tasks::TTask<int32>> AsyncTasks;
		AsyncTasks.Reserve(NumFutures);
		const bool bInitBPContext = (AsyncState && AsyncState->bIsCallingBlueprint);

		// Launch the async tasks
		for (int32 TaskIndex = 0; TaskIndex < NumFutures; ++TaskIndex)
		{
			const int32 StartIndex = TaskIndex * IterationsPerTask;
			const int32 EndIndex = StartIndex + IterationsPerTask;

			AsyncTasks.Emplace(UE::Tasks::Launch(UE_SOURCE_LOCATION, [&IterationInnerLoop, StartIndex, EndIndex, bInitBPContext]()
			{
				LLM_SCOPE_BYTAG(PCG);

				if (bInitBPContext)
				{
					GInitRunaway(); // Reset counter as threads can run multiple workloads and might not be properly reset.
				}

				return IterationInnerLoop(StartIndex, EndIndex);
			}));
		}

		// Execute last batch locally
		int32 NumPointsWrittenOnThisThread = IterationInnerLoop(NumFutures * IterationsPerTask, NumIterations);

		// Wait/Gather results & collapse points
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncPointProcessing::WaitAndCollapseArray);

			auto CollapsePoints = [&OutPoints](int32 RangeIndex, int32 StartPointsIndex, int32 NumPointsToCollapse)
			{
				// Move points from [StartsPointIndex, StartsPointIndex + NumberPointsAdded] to [RangeIndex, RangeIndex + NumberPointsAdded]
				if (StartPointsIndex != RangeIndex)
				{
					for (int32 MoveIndex = 0; MoveIndex < NumPointsToCollapse; ++MoveIndex)
					{
						OutPoints[RangeIndex + MoveIndex] = MoveTemp(OutPoints[StartPointsIndex + MoveIndex]);
					}
				}

				return RangeIndex + NumPointsToCollapse;
			};

			int RangeIndex = 0;
			for (int32 AsyncIndex = 0; AsyncIndex < AsyncTasks.Num(); ++AsyncIndex)
			{
				const int32 StartPointsIndex = AsyncIndex * IterationsPerTask;

				UE::Tasks::TTask<int32>& AsyncTask = AsyncTasks[AsyncIndex];
				AsyncTask.Wait();
				const int32 NumberOfPointsAdded = AsyncTask.GetResult();
				RangeIndex = CollapsePoints(RangeIndex, StartPointsIndex, NumberOfPointsAdded);
			}

			// Finally, add current thread points
			RangeIndex = CollapsePoints(RangeIndex, NumFutures * IterationsPerTask, NumPointsWrittenOnThisThread);

			OutPoints.SetNum(RangeIndex);
		}
	}

	void PrivateAsyncPointProcessing(FPCGAsyncState* AsyncState, int32 MinIterationsPerTask, int32 NumIterations, TArray<FPCGPoint>& OutPoints, const TFunction<bool(int32, FPCGPoint&)>& PointFunc)
	{
		auto IterationInnerLoop = [&PointFunc, &OutPoints](int32 StartIndex, int32 EndIndex) -> int32
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncPointProcessing::InnerLoop);
			int32 NumPointsWritten = 0;

			for (int32 Index = StartIndex; Index < EndIndex; ++Index)
			{
				if (PointFunc(Index, OutPoints[StartIndex + NumPointsWritten]))
				{
					++NumPointsWritten;
				}
			}

			return NumPointsWritten;
		};

		PrivateAsyncPointProcessing(AsyncState, MinIterationsPerTask, NumIterations, OutPoints, IterationInnerLoop);
	}

	void PrivateAsyncMultiPointProcessing(FPCGAsyncState* AsyncState, int32 MinIterationsPerTask, int32 NumIterations, TArray<FPCGPoint>& OutPoints, const TFunction<TArray<FPCGPoint>(int32)>& PointFunc)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncMultiPointProcessing);
		check(MinIterationsPerTask > 0 && NumIterations >= 0);
		if (MinIterationsPerTask <= 0 || NumIterations <= 0)
		{
			return;
		}

		const int32 NumTasks = GetNumTasks(AsyncState, NumIterations / MinIterationsPerTask);
		const int32 IterationsPerTask = NumIterations / NumTasks;
		const int32 NumFutures = NumTasks - 1;

		auto IterationInnerLoop = [&PointFunc](int32 StartIndex, int32 EndIndex) -> TArray<FPCGPoint>
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncMultiPointProcessing::InnerLoop);
			TArray<FPCGPoint> OutPoints;

			for (int32 Index = StartIndex; Index < EndIndex; ++Index)
			{
				OutPoints.Append(PointFunc(Index));
			}

			return OutPoints;
		};

		TArray<UE::Tasks::TTask<TArray<FPCGPoint>>> AsyncTasks;
		AsyncTasks.Reserve(NumFutures);
		const bool bInitBPContext = (AsyncState && AsyncState->bIsCallingBlueprint);

		// Launch the async tasks
		for (int32 TaskIndex = 0; TaskIndex < NumFutures; ++TaskIndex)
		{
			const int32 StartIndex = TaskIndex * IterationsPerTask;
			const int32 EndIndex = StartIndex + IterationsPerTask;

			AsyncTasks.Emplace(UE::Tasks::Launch(UE_SOURCE_LOCATION, [&IterationInnerLoop, StartIndex, EndIndex, bInitBPContext]() -> TArray<FPCGPoint>
			{
				LLM_SCOPE_BYTAG(PCG);

				if (bInitBPContext)
				{
					GInitRunaway(); // Reset counter as threads can run multiple workloads and might not be properly reset.
				}

				return IterationInnerLoop(StartIndex, EndIndex);
			}));
		}

		TArray<FPCGPoint> PointsFromThisThread = IterationInnerLoop(NumFutures * IterationsPerTask, NumIterations);

		// Wait/Gather results & collapse points
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncMultiPointProcessing::WaitAndCollapseArray);
			for (UE::Tasks::TTask<TArray<FPCGPoint>>& AsyncTask : AsyncTasks)
			{
				AsyncTask.Wait();
				OutPoints.Append(MoveTemp(AsyncTask.GetResult()));
			}

			OutPoints.Append(MoveTemp(PointsFromThisThread));
		}
	}

	void PrivateAsyncPointFilterProcessing(FPCGAsyncState* AsyncState, int32 MinIterationsPerTask, int32 NumIterations, TArray<FPCGPoint>& InFilterPoints, TArray<FPCGPoint>& OutFilterPoints, const TFunction<bool(int32, FPCGPoint&, FPCGPoint&)>& PointFunc)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncPointFilterProcessing);
		check(MinIterationsPerTask > 0 && NumIterations >= 0);
		if (MinIterationsPerTask <= 0 || NumIterations <= 0)
		{
			return;
		}

		const int32 NumTasks = GetNumTasks(AsyncState, NumIterations / MinIterationsPerTask);
		const int32 IterationsPerTask = NumIterations / NumTasks;
		const int32 NumFutures = NumTasks - 1;

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncPointProcessing::AllocatingArray);
			// Pre-reserve the out points array
			InFilterPoints.SetNumUninitialized(NumIterations);
			OutFilterPoints.SetNumUninitialized(NumIterations);
		}

		auto IterationInnerLoop = [&PointFunc, &InFilterPoints, &OutFilterPoints](int32 StartIndex, int32 EndIndex) -> TPair<int32, int32>
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncPointProcessing::InnerLoop);
			int32 NumPointsInWritten = 0;
			int32 NumPointsOutWritten = 0;

			for (int32 Index = StartIndex; Index < EndIndex; ++Index)
			{
				if (PointFunc(Index, InFilterPoints[StartIndex + NumPointsInWritten], OutFilterPoints[StartIndex + NumPointsOutWritten]))
				{
					++NumPointsInWritten;
				}
				else
				{
					++NumPointsOutWritten;
				}
			}

			return TPair<int32, int32>(NumPointsInWritten, NumPointsOutWritten);
		};

		// Setup [current, last, nb points] data per dispatch
		TArray<UE::Tasks::TTask<TPair<int32, int32>>> AsyncTasks;
		AsyncTasks.Reserve(NumFutures);
		const bool bInitBPContext = (AsyncState && AsyncState->bIsCallingBlueprint);

		// Launch the async tasks
		for (int32 TaskIndex = 0; TaskIndex < NumFutures; ++TaskIndex)
		{
			const int32 StartIndex = TaskIndex * IterationsPerTask;
			const int32 EndIndex = StartIndex + IterationsPerTask;

			AsyncTasks.Emplace(UE::Tasks::Launch(UE_SOURCE_LOCATION, [&IterationInnerLoop, StartIndex, EndIndex, bInitBPContext]() -> TPair<int32, int32>
			{
				LLM_SCOPE_BYTAG(PCG);

				if (bInitBPContext)
				{
					GInitRunaway(); // Reset counter as threads can run multiple workloads and might not be properly reset.
				}

				return IterationInnerLoop(StartIndex, EndIndex);
			}));
		}

		// Launch remainder on current thread
		TPair<int32, int32> NumPointsWrittenOnThisThread = IterationInnerLoop(NumFutures * IterationsPerTask, NumIterations);

		// Wait/Gather results & collapse points
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncPointFilterProcessing::WaitAndCollapseArray);

			auto CollapsePoints = [&InFilterPoints, &OutFilterPoints](const TPair<int32, int32>& RangeIndices, int32 StartPointsIndex, const TPair<int32, int32>& NumberOfPointsAdded) -> TPair<int32, int32>
			{
				int32 InFilterRangeIndex = RangeIndices.Key;
				int32 OutFilterRangeIndex = RangeIndices.Value;

				// Move in-filter points
				{
					int NumInFilterPoints = NumberOfPointsAdded.Key;

					if (StartPointsIndex != InFilterRangeIndex)
					{
						for (int32 MoveIndex = 0; MoveIndex < NumInFilterPoints; ++MoveIndex)
						{
							InFilterPoints[InFilterRangeIndex + MoveIndex] = MoveTemp(InFilterPoints[StartPointsIndex + MoveIndex]);
						}
					}

					InFilterRangeIndex += NumInFilterPoints;
				}

				// Move out-filter points
				{
					int NumOutFilterPoints = NumberOfPointsAdded.Value;

					if (StartPointsIndex != OutFilterRangeIndex)
					{
						for (int32 MoveIndex = 0; MoveIndex < NumOutFilterPoints; ++MoveIndex)
						{
							OutFilterPoints[OutFilterRangeIndex + MoveIndex] = MoveTemp(OutFilterPoints[StartPointsIndex + MoveIndex]);
						}
					}

					OutFilterRangeIndex += NumOutFilterPoints;
				}

				return TPair<int32, int32>(InFilterRangeIndex, OutFilterRangeIndex);
			};

			TPair<int32, int32> RangeIndices = TPair<int32, int32>(0, 0);
			for (int32 AsyncIndex = 0; AsyncIndex < AsyncTasks.Num(); ++AsyncIndex)
			{
				const int32 StartPointsIndex = AsyncIndex * IterationsPerTask;

				UE::Tasks::TTask<TPair<int32, int32>>& AsyncTask = AsyncTasks[AsyncIndex];
				AsyncTask.Wait();
				TPair<int32, int32> NumberOfPointsAdded = AsyncTask.GetResult();

				RangeIndices = CollapsePoints(RangeIndices, StartPointsIndex, NumberOfPointsAdded);
			}

			// Finally, add this thread results
			RangeIndices = CollapsePoints(RangeIndices, NumFutures * IterationsPerTask, NumPointsWrittenOnThisThread);

			InFilterPoints.SetNum(RangeIndices.Key);
			OutFilterPoints.SetNum(RangeIndices.Value);
		}
	}

	void AsyncPointProcessing(FPCGContext* Context, int32 NumIterations, TArray<FPCGPoint>& OutPoints, const TFunction<bool(int32, FPCGPoint&)>& PointFunc)
	{
		const int32 MinIterationsPerTask = 256;

		if (Context && !Context->AsyncState.bIsRunningAsyncCall)
		{
			Context->AsyncState.bIsRunningAsyncCall = true;
			PrivateAsyncPointProcessing(&Context->AsyncState, MinIterationsPerTask, NumIterations, OutPoints, PointFunc);
			Context->AsyncState.bIsRunningAsyncCall = false;
		}
		else
		{
			// Reentrant case (no async support)
			PrivateAsyncPointProcessing(nullptr, MinIterationsPerTask, NumIterations, OutPoints, PointFunc);
		}
	}

	void AsyncPointProcessing(FPCGContext* Context, const TArray<FPCGPoint>& InPoints, TArray<FPCGPoint>& OutPoints, const TFunction<bool(const FPCGPoint&, FPCGPoint&)>& PointFunc)
	{
		const int32 MinIterationsPerTask = 256;

		auto IterationInnerLoop = [&PointFunc, &InPoints, &OutPoints](int32 StartIndex, int32 EndIndex) -> int32
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::AsyncPointProcessing::InnerLoop);
			int32 NumPointsWritten = 0;

			for (int32 Index = StartIndex; Index < EndIndex; ++Index)
			{
				if (PointFunc(InPoints[Index], OutPoints[StartIndex + NumPointsWritten]))
				{
					++NumPointsWritten;
				}
			}

			return NumPointsWritten;
		};
				
		if (Context && !Context->AsyncState.bIsRunningAsyncCall)
		{
			Context->AsyncState.bIsRunningAsyncCall = true;
			PrivateAsyncPointProcessing(&Context->AsyncState, MinIterationsPerTask, InPoints.Num(), OutPoints, IterationInnerLoop);
			Context->AsyncState.bIsRunningAsyncCall = false;
		}
		else
		{
			// Reentrant case (no async support)
			PrivateAsyncPointProcessing(nullptr, MinIterationsPerTask, InPoints.Num(), OutPoints, IterationInnerLoop);
		}
	}
		
	void AsyncPointFilterProcessing(FPCGContext* Context, int32 NumIterations, TArray<FPCGPoint>& InFilterPoints, TArray<FPCGPoint>& OutFilterPoints, const TFunction<bool(int32, FPCGPoint&, FPCGPoint&)>& PointFunc)
	{
		const int32 MinIterationsPerTask = 256;
		if (Context && !Context->AsyncState.bIsRunningAsyncCall)
		{
			Context->AsyncState.bIsRunningAsyncCall = true;
			PrivateAsyncPointFilterProcessing(&Context->AsyncState, MinIterationsPerTask, NumIterations, InFilterPoints, OutFilterPoints, PointFunc);
			Context->AsyncState.bIsRunningAsyncCall = false;
		}
		else
		{
			// Reentrant case (no async support)
			PrivateAsyncPointFilterProcessing(nullptr, MinIterationsPerTask, NumIterations, InFilterPoints, OutFilterPoints, PointFunc);
		}
	}
		
	void AsyncMultiPointProcessing(FPCGContext* Context, int32 NumIterations, TArray<FPCGPoint>& OutPoints, const TFunction<TArray<FPCGPoint>(int32)>& PointFunc)
	{
		const int32 MinIterationsPerTask = 256;
		if (Context && !Context->AsyncState.bIsRunningAsyncCall)
		{
			Context->AsyncState.bIsRunningAsyncCall = true;
			PrivateAsyncMultiPointProcessing(&Context->AsyncState, MinIterationsPerTask, NumIterations, OutPoints, PointFunc);
			Context->AsyncState.bIsRunningAsyncCall = false;
		}
		else
		{
			// Reentrant case (no async support)
			PrivateAsyncMultiPointProcessing(nullptr, MinIterationsPerTask, NumIterations, OutPoints, PointFunc);
		}
	}

	bool Private::AsyncProcessing(FPCGAsyncState& AsyncState, int32 NumIterations, TFunctionRef<void(void)> Initialize, TFunctionRef<int32(int32, int32, int32)> IterationInnerLoop, TFunctionRef<void(int32, int32, int32)> MoveDataRange, TFunctionRef<void(int32)> Finished, const bool bInEnableTimeSlicing, const int32 InChunkSize, const bool bAllowChunkSizeOverride)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAsync::AsyncProcessing);

		const int32 OverrideChunkSize = ConsoleVar::CVarAsyncOverrideChunkSize.GetValueOnAnyThread();
		const int32 ChunkSize = bAllowChunkSizeOverride && OverrideChunkSize > 0 ? OverrideChunkSize : InChunkSize;

		const bool bIsInGameThread = IsInGameThread();
		const bool bEnableTimeSlicing = bInEnableTimeSlicing && ((bIsInGameThread && !ConsoleVar::CVarDisableAsyncTimeSlicingOnGameThread.GetValueOnAnyThread()) || (!bIsInGameThread && !ConsoleVar::CVarDisableAsyncTimeSlicing.GetValueOnAnyThread()));

		if (AsyncState.NumAvailableTasks == 0 || ChunkSize <= 0 || NumIterations <= 0)
		{
			// Invalid request
			return true;
		}

		const int32 ChunksNumber = 1 + ((NumIterations - 1) / ChunkSize);

		if (AsyncState.CurrentChunkToCollapse >= ChunksNumber)
		{
			// Nothing left to do
			return true;
		}

		if (!AsyncState.bStarted)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAsync::Private::AsyncProcessing::Initialize);

			Initialize();

			AsyncState.bStarted = true;
			AsyncState.InitialNumIterations = NumIterations;

			// At the beginning, dispatch the chunks that each task needs to process.
			// It will be stored on the AsyncState so that it can be reused.
			// Each task will update its current counter directly in the AsyncState.
			// It's OK to not protect it since they all will read/write from different place in memory.

			int32 NumTasks = ChunksNumber;
			if (AsyncState.NumAvailableTasks > 0)
			{
				NumTasks = FMath::Min(AsyncState.NumAvailableTasks, NumTasks);
			}

			const int32 IterationsPerTask = ChunksNumber / NumTasks;
			
			// Some tasks will have an extra chunk of work if NumTasks doesn't divide ChunksNumber.
			const int32 ExtraChunkLimit = ChunksNumber - (IterationsPerTask * NumTasks);

			// Main thread is also doing work.
			const int32 NumFutures = NumTasks - 1;
			if (NumFutures != 0)
			{
				int32 Count = 0;
				AsyncState.TasksChunksStartEnd.Reserve(NumTasks);
				for (int32 i = 0; i < NumTasks; ++i)
				{
					const int32 NumChunkForThisTask = i < ExtraChunkLimit ? IterationsPerTask + 1 : IterationsPerTask;
					AsyncState.TasksChunksStartEnd.Emplace(Count, Count + NumChunkForThisTask);
					Count += NumChunkForThisTask;
				}

				check(Count == ChunksNumber);
			}
		}
		else
		{
			if (!ensureMsgf(NumIterations == AsyncState.InitialNumIterations,
				TEXT("NumIterations (%d) mismatch with the initial number of iterations (%d).\n"
				"It might mean that a new processing is started while the previous one is not done.\n"
				"Validate on the calling site that if time slicing is enabled, it is waiting for this\n"
				"function to return true (done) before starting a new processing."),
				NumIterations, AsyncState.InitialNumIterations))
			{
				AsyncState.Reset();
				return true;
			}
		}

		const int32 NumFutures = FMath::Max(0, AsyncState.TasksChunksStartEnd.Num() - 1);

		// Synchronisation structure to be shared between async tasks and collapsing main thread.
		struct FSynchroStruct 
		{
			// Atomic to indicate tasks to stop processing new chunks.
			std::atomic<bool> bQuit = false;

			// Queue for worker to indicate the current chunk that was processed and the number of elements written.
			// Threadsafe for MPSC: Multiple Producers (async tasks) and Single Consumer (collapsing task).
			TMpscQueue<TPair<int32, int32>> ChunkProcessedIndexAndNumElementsWrittenQueue;
		};

		// We won't stop if we are not time slicing.
		auto ShouldStop = [&AsyncState, bEnableTimeSlicing]() -> bool
		{
			return bEnableTimeSlicing && AsyncState.ShouldStop();
		};

		// Main thread will either work if there is no future, or work and collapse arrays if there are some.
		if (NumFutures == 0)
		{
			// Main thread is working
			check(ChunksNumber > 0);

			// Do at least one run, as we could fall into an infinite loop if we always have to stop before doing anything.
			do
			{
				const int32 StartReadIndex = AsyncState.CurrentChunkToCollapse * ChunkSize;
				const int32 StartWriteIndex = AsyncState.AsyncCurrentWriteIndex;
				const int32 Count = FMath::Min(ChunkSize, NumIterations - StartReadIndex);

				check(Count > 0);

				int32 NumElementsWritten;

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAsync::Private::AsyncProcessing::IterationInnerLoop);

					NumElementsWritten = IterationInnerLoop(StartReadIndex, StartWriteIndex, Count);
				}

				AsyncState.AsyncCurrentWriteIndex += NumElementsWritten;
			} while (++AsyncState.CurrentChunkToCollapse < ChunksNumber && !ShouldStop());
		}
		else
		{
			// Main thread is collapsing
			// First start the futures
			FSynchroStruct SynchroStruct{};

			// Futures are not returning anything.
			TArray<UE::Tasks::TTask<void>> AsyncTasks;
			AsyncTasks.Reserve(NumFutures);
			const bool bInitBPContext = AsyncState.bIsCallingBlueprint;

			// A job task will pick up the chunk to process, do the work on it, and push in the queue info that it processed the chunk and how many points it has written.
			auto JobTask = [&IterationInnerLoop, &SynchroStruct, ChunkSize, NumIterations, bInitBPContext](const int32 CurrentChunkToProcess)
			{
				const int32 StartReadIndex = CurrentChunkToProcess * ChunkSize;
				// We write in the same "memory space" with the async task (in preallocated output array) and will be collapsed by the main thread.
				const int32 StartWriteIndex = StartReadIndex;
				const int32 Count = FMath::Min(ChunkSize, NumIterations - StartReadIndex);

				check(Count > 0);
				int32 NumElementsWritten;

				if (bInitBPContext)
				{
					GInitRunaway(); // Reset counter as threads can run multiple workloads and might not be properly reset.
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAsync::Private::AsyncProcessing::InnerLoop);

					NumElementsWritten = IterationInnerLoop(StartReadIndex, StartWriteIndex, Count);
				}

				SynchroStruct.ChunkProcessedIndexAndNumElementsWrittenQueue.Enqueue(CurrentChunkToProcess, NumElementsWritten);
			};

			FPCGContext* CurrentContext = FPCGContext::GetContextForThread();

			// Starting all the tasks, only if the associated task has still some work to do.
			for (int32 TaskIndex = 0; TaskIndex < NumFutures; ++TaskIndex)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAsync::Private::AsyncProcessing::StartingTasks);

				if (AsyncState.TasksChunksStartEnd[TaskIndex].Get<0>() == AsyncState.TasksChunksStartEnd[TaskIndex].Get<1>())
				{
					// No work left to do.
					continue;
				}

				AsyncTasks.Emplace(UE::Tasks::Launch(UE_SOURCE_LOCATION, [&JobTask, &SynchroStruct, &StartEnd = AsyncState.TasksChunksStartEnd[TaskIndex], CurrentContext]() -> void
				{
					LLM_SCOPE_BYTAG(PCG);

					FPCGContextScope ContextScope(CurrentContext);

					// Do at least one run, as we could fall into an infinite loop if we always have to stop before doing anything.
					// Quit if we have no more work to do or we were told to stop.
					do
					{
						JobTask(StartEnd.Get<0>()++);
					} while (!SynchroStruct.bQuit && StartEnd.Get<0>() != StartEnd.Get<1>());
				}));
			}

			auto FlushQueue = [&SynchroStruct, &AsyncState]()
			{
				TPair<int32, int32> QueueItem;
				while (SynchroStruct.ChunkProcessedIndexAndNumElementsWrittenQueue.Dequeue(QueueItem))
				{
					AsyncState.ChunkToNumElementsWrittenMap.Add(QueueItem);
				}
			};

			// Collapsing needs to be done in order, so we have a map between chunk index and the number of elements written for this chunk.
			// Note that we need to collpase because points can be discarded, so we can have less points "kept" than the chunk size.
			// Do at least one run, as we could fall into an infinite loop if we always have to stop before doing anything.
			bool bHasRunOnce = false;
			while (true)
			{
				// Either we should stop because the time has elapsed or all workloads have been processed
				if (bHasRunOnce && (ShouldStop() || SynchroStruct.bQuit))
				{
					SynchroStruct.bQuit = true;

					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAsync::AsyncProcessing::Wait);
						// Wait for all futures to finish their job
						UE::Tasks::Wait(AsyncTasks);
					}

					// Make sure to dequeue everything
					FlushQueue();

					// Exit
					break;
				}

				bHasRunOnce = true;

				// Try to flush queue before trying anything.
				FlushQueue();
				
				// If we have work to do, do it.
				if (AsyncState.TasksChunksStartEnd.Last().Get<0>() != AsyncState.TasksChunksStartEnd.Last().Get<1>())
				{
					JobTask(AsyncState.TasksChunksStartEnd.Last().Get<0>()++);
					continue;
				}

				// At that point, we are on collapse duty, don't stop until we finish the collapse or we have to stop.
				// Don't check stop too often.
				constexpr int32 NumCheckStop = 20;
				int32 CheckStop = 0;
				while (AsyncState.CurrentChunkToCollapse != ChunksNumber)
				{
					if (++CheckStop == NumCheckStop)
					{
						CheckStop = 0;
						if (ShouldStop())
						{
							SynchroStruct.bQuit = true;
							break;
						}
					}

					// Try to remove the current chunk we need to process. If it is not in the map it means it has not yet been processed (or dequeued)
					// so flush the queue and try again.
					int32 NumberOfElementsWritten = 0;
					if (!AsyncState.ChunkToNumElementsWrittenMap.RemoveAndCopyValue(AsyncState.CurrentChunkToCollapse, NumberOfElementsWritten))
					{
						FlushQueue();
						continue;
					}

					TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAsync::AsyncProcessing::CollapsingNewData);
					const int32 ReadStartIndexForCurrentChunkToCollapse = AsyncState.CurrentChunkToCollapse * ChunkSize;
					check(NumberOfElementsWritten <= ChunkSize);

					// If ReadStartIndexForCurrentChunkToCollapse == AsyncState.AsyncCurrentWriteIndex, no need to copy, all elements
					// are already at the right place
					if (ReadStartIndexForCurrentChunkToCollapse == AsyncState.AsyncCurrentWriteIndex)
					{
						AsyncState.AsyncCurrentWriteIndex += NumberOfElementsWritten;
					}
					else
					{
						// Otherwise collapse
						MoveDataRange(ReadStartIndexForCurrentChunkToCollapse, AsyncState.AsyncCurrentWriteIndex, NumberOfElementsWritten);
						AsyncState.AsyncCurrentWriteIndex += NumberOfElementsWritten;
					}

					++AsyncState.CurrentChunkToCollapse;
				}

				// If we reach that point, it means we finished the collapse or we ran out of time. So indicate to stop.
				SynchroStruct.bQuit = true;
			}
		}

		const bool bIsDone = AsyncState.CurrentChunkToCollapse == ChunksNumber;

		check(bIsDone || bEnableTimeSlicing);

		if (bIsDone)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAsync::Private::AsyncProcessing::Finished);

			Finished(AsyncState.AsyncCurrentWriteIndex);
			AsyncState.Reset();
		}

		return bIsDone;
	}
}