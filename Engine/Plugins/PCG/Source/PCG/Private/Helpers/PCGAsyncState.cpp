// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGAsyncState.h"

#include "HAL/PlatformTime.h"

bool FPCGAsyncState::ShouldStop() const
{
	return FPlatformTime::Seconds() > EndTime;
}

void FPCGAsyncState::Reset()
{
	AsyncCurrentWriteIndex = 0;
	TasksChunksStartEnd.Reset();
	CurrentChunkToCollapse = 0;
	ChunkToNumElementsWrittenMap.Reset();
	InitialNumIterations = 0;

	bStarted = false;
}
