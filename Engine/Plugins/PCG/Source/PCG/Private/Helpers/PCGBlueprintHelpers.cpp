// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGBlueprintHelpers.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "PCGPoint.h"
#include "Grid/PCGPartitionActor.h"
#include "Grid/PCGLandscapeCache.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Subsystems/PCGEngineSubsystem.h"
#include "Subsystems/PCGSubsystem.h"

#include "Blueprint/BlueprintExceptionInfo.h"
#include "Engine/World.h"
#include "LandscapeProxy.h"
#include "LandscapeInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBlueprintHelpers)

UPCGGenerateGraphAsync* UPCGGenerateGraphAsync::GenerateGraphAsync(UPCGGraphInterface* InGraph, int32 InSeed)
{
	UPCGGenerateGraphAsync* AsyncCall = NewObject<UPCGGenerateGraphAsync>();

#if WITH_EDITOR
	AsyncCall->Graph = InGraph;
	AsyncCall->Seed = InSeed;
#endif

	return AsyncCall;
}

void UPCGGenerateGraphAsync::Activate()
{
#if WITH_EDITOR
	FPCGGenerateGraphParams GenerateParams;
	GenerateParams.Graph = Graph;
	GenerateParams.Seed = Seed;
	GenerateParams.GenerationCallback.BindUObject(this, &UPCGGenerateGraphAsync::OnGenerationDone);

	if (UPCGEngineSubsystem* EngineSubsystem = UPCGEngineSubsystem::Get(); ensure(EngineSubsystem))
	{
		EngineSubsystem->GenerateGraph(GenerateParams);
	}
#else
	UE_LOGF(LogPCG, Error, "GenerateGraphAsync not supported in non editor builds.")
	OnGenerationDone(nullptr, EPCGGenerationStatus::Aborted);
#endif // WITH_EDITOR
}

void UPCGGenerateGraphAsync::OnGenerationDone(IPCGGraphExecutionSource* InExecutionSource, EPCGGenerationStatus InStatus)
{
	OnCompleted.Broadcast(InStatus);

	SetReadyToDestroy();
}

void UPCGBlueprintHelpers::ThrowBlueprintException(const FText& ErrorMessage)
{
	PCGLog::LogErrorOnGraph(ErrorMessage);

	if (FFrame::GetThreadLocalTopStackFrame() && FFrame::GetThreadLocalTopStackFrame()->Object)
	{
		const FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::FatalError, ErrorMessage);
		FBlueprintCoreDelegates::ThrowScriptException(FFrame::GetThreadLocalTopStackFrame()->Object, *FFrame::GetThreadLocalTopStackFrame(), ExceptionInfo);
	}
}

int UPCGBlueprintHelpers::ComputeSeedFromPosition(const FVector& InPosition)
{
	return PCGHelpers::ComputeSeedFromPosition(InPosition);
}

void UPCGBlueprintHelpers::SetSeedFromPosition(FPCGPoint& InPoint)
{
	InPoint.Seed = ComputeSeedFromPosition(InPoint.Transform.GetLocation());
}

FRandomStream UPCGBlueprintHelpers::GetRandomStreamFromPoint(const FPCGPoint& InPoint, const UPCGSettings* OptionalSettings, const UPCGComponent* OptionalComponent)
{
	return PCGHelpers::GetRandomStreamFromSeed(InPoint.Seed, OptionalSettings, OptionalComponent);
}

FRandomStream UPCGBlueprintHelpers::GetRandomStreamFromTwoPoints(const FPCGPoint& InPointA, const FPCGPoint& InPointB, const UPCGSettings* OptionalSettings, const UPCGComponent* OptionalComponent)
{
	return PCGHelpers::GetRandomStreamFromTwoSeeds(InPointA.Seed, InPointB.Seed, OptionalSettings, OptionalComponent);
}

const UPCGSettings* UPCGBlueprintHelpers::GetSettings(FPCGContext& Context)
{
	return Context.GetInputSettings<UPCGSettings>();
}

const UPCGSettings* UPCGBlueprintHelpers::GetSettingsWithContext(const FPCGBlueprintContextHandle& ContextHandle)
{
	if (TSharedPtr<FPCGContextHandle> NativeContextHandle = ContextHandle.Handle.Pin())
	{
		if (FPCGContext* Context = NativeContextHandle->GetContext())
		{
			return GetSettings(*Context);
		}
	}

	UE_LOGF(LogPCG, Error, "UPCGBlueprintHelpers::GetSettingsWithContext : Invalid context handle");
	return nullptr;
}

UPCGData* UPCGBlueprintHelpers::GetActorData(FPCGContext& Context)
{
	UPCGComponent* PCGComponent = GetComponent(Context);
	return PCGComponent ? PCGComponent->GetActorPCGData() : nullptr;
}

UPCGData* UPCGBlueprintHelpers::GetActorDataWithContext(const FPCGBlueprintContextHandle& ContextHandle)
{
	if (TSharedPtr<FPCGContextHandle> NativeContextHandle = ContextHandle.Handle.Pin())
	{
		if (FPCGContext* Context = NativeContextHandle->GetContext())
		{
			return GetActorData(*Context);
		}
	}

	UE_LOGF(LogPCG, Error, "UPCGBlueprintHelpers::GetActorDataWithContext : Invalid context handle");
	return nullptr;
}

UPCGData* UPCGBlueprintHelpers::GetInputData(FPCGContext& Context)
{
	UPCGComponent* PCGComponent = GetComponent(Context);
	return PCGComponent ? PCGComponent->GetInputPCGData() : nullptr;
}

UPCGData* UPCGBlueprintHelpers::GetInputDataWithContext(const FPCGBlueprintContextHandle& ContextHandle)
{
	if (TSharedPtr<FPCGContextHandle> NativeContextHandle = ContextHandle.Handle.Pin())
	{
		if (FPCGContext* Context = NativeContextHandle->GetContext())
		{
			return GetInputData(*Context);
		}
	}

	UE_LOGF(LogPCG, Error, "UPCGBlueprintHelpers::GetInputDataWithContext : Invalid context handle");
	return nullptr;
}

UPCGComponent* UPCGBlueprintHelpers::GetComponent(FPCGContext& Context)
{
	return Cast<UPCGComponent>(Context.ExecutionSource.Get());
}

UPCGComponent* UPCGBlueprintHelpers::GetComponentWithContext(const FPCGBlueprintContextHandle& ContextHandle)
{
	if (TSharedPtr<FPCGContextHandle> NativeContextHandle = ContextHandle.Handle.Pin())
	{
		if (FPCGContext* Context = NativeContextHandle->GetContext())
		{
			return GetComponent(*Context);
		}
	}

	UE_LOGF(LogPCG, Error, "UPCGBlueprintHelpers::GetComponentWithContext : Invalid context handle");
	return nullptr;
}

UPCGComponent* UPCGBlueprintHelpers::GetOriginalComponent(FPCGContext& Context)
{
	UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context.ExecutionSource.Get());
	APCGPartitionActor* PartitionActor = SourceComponent ? Cast<APCGPartitionActor>(SourceComponent->GetOwner()) : nullptr;
	UPCGComponent* OriginalComponent = PartitionActor ? PartitionActor->GetOriginalComponent(SourceComponent) : nullptr;

	if (OriginalComponent)
	{
		return OriginalComponent;
	}
	else
	{
		return SourceComponent;
	}
}

UPCGComponent* UPCGBlueprintHelpers::GetOriginalComponentWithContext(const FPCGBlueprintContextHandle& ContextHandle)
{
	if (TSharedPtr<FPCGContextHandle> NativeContextHandle = ContextHandle.Handle.Pin())
	{
		if (FPCGContext* Context = NativeContextHandle->GetContext())
		{
			return GetOriginalComponent(*Context);
		}
	}

	UE_LOGF(LogPCG, Error, "UPCGBlueprintHelpers::GetOriginalComponentWithContext : Invalid context handle");
	return nullptr;
}

AActor* UPCGBlueprintHelpers::GetTargetActor(FPCGContext& Context, UPCGSpatialData* SpatialData)
{
	return Context.GetTargetActor(SpatialData);
}

AActor* UPCGBlueprintHelpers::GetTargetActorWithContext(const FPCGBlueprintContextHandle& ContextHandle, UPCGSpatialData* SpatialData)
{
	if (TSharedPtr<FPCGContextHandle> NativeContextHandle = ContextHandle.Handle.Pin())
	{
		if (FPCGContext* Context = NativeContextHandle->GetContext())
		{
			return GetTargetActor(*Context, SpatialData);
		}
	}

	UE_LOGF(LogPCG, Error, "UPCGBlueprintHelpers::GetTargetActorWithContext : Invalid context handle");
	return nullptr;
}

void UPCGBlueprintHelpers::SetExtents(FPCGPoint& InPoint, const FVector& InExtents)
{
	InPoint.SetExtents(InExtents);
}

FVector UPCGBlueprintHelpers::GetExtents(const FPCGPoint& InPoint)
{
	return InPoint.GetExtents();
}

void UPCGBlueprintHelpers::SetLocalCenter(FPCGPoint& InPoint, const FVector& InLocalCenter)
{
	InPoint.SetLocalCenter(InLocalCenter);
}

FVector UPCGBlueprintHelpers::GetLocalCenter(const FPCGPoint& InPoint)
{
	return InPoint.GetLocalCenter();
}

FBox UPCGBlueprintHelpers::GetTransformedBounds(const FPCGPoint& InPoint)
{
	return FBox(InPoint.BoundsMin, InPoint.BoundsMax).TransformBy(InPoint.Transform);
}

FBox UPCGBlueprintHelpers::GetActorBoundsPCG(AActor* InActor, bool bIgnorePCGCreatedComponents)
{
	return PCGHelpers::GetActorBounds(InActor, bIgnorePCGCreatedComponents);
}

FBox UPCGBlueprintHelpers::GetActorLocalBoundsPCG(AActor* InActor, bool bIgnorePCGCreatedComponents)
{
	return PCGHelpers::GetActorLocalBounds(InActor, bIgnorePCGCreatedComponents);
}

UPCGData* UPCGBlueprintHelpers::CreatePCGDataFromActor(AActor* InActor, bool bParseActor)
{
	return UPCGComponent::CreateActorPCGData(InActor, nullptr, bParseActor);
}

TArray<FPCGLandscapeLayerWeight> UPCGBlueprintHelpers::GetInterpolatedPCGLandscapeLayerWeights(UObject* WorldContextObject, const FVector& Location)
{
	UWorld* World = WorldContextObject ? WorldContextObject->GetWorld() : nullptr;
	if (!World)
	{
		return {};
	}

	UPCGSubsystem* PCGSubSystem = UPCGSubsystem::GetInstance(World);
	if (!PCGSubSystem)
	{
		return {};
	}

	UPCGLandscapeCache* LandscapeCache = PCGSubSystem->GetLandscapeCache();
	if (!LandscapeCache)
	{
		return {};
	}

	FBox Bounds(&Location, 1);
	TArray<TWeakObjectPtr<ALandscapeProxy>> Landscapes = PCGHelpers::GetLandscapeProxies(World, Bounds);

	if (Landscapes.IsEmpty())
	{
		return {};
	}

	FString FailureReason;

	for (TWeakObjectPtr<ALandscapeProxy> LandscapePtr : Landscapes)
	{
		ALandscapeProxy* Landscape = LandscapePtr.Get();
		if (!Landscape)
		{
			continue;
		}

		ULandscapeInfo* LandscapeInfo = Landscape->GetLandscapeInfo();
		if (!LandscapeInfo)
		{
			FailureReason = TEXT("Unable to get landscape layer weights because the landscape info is not available (landscape not registered yet?");
			continue;
		}

		const FVector LocalPoint = Landscape->LandscapeActorToWorld().InverseTransformPosition(Location);
		const FIntPoint ComponentMapKey(FMath::FloorToInt(LocalPoint.X / LandscapeInfo->ComponentSizeQuads), FMath::FloorToInt(LocalPoint.Y / LandscapeInfo->ComponentSizeQuads));

		TSharedPtr<const FPCGLandscapeCacheEntry> CacheEntry = LandscapeCache->GetCacheEntry(LandscapeInfo, ComponentMapKey, Landscape);

		if (!CacheEntry)
		{
			FailureReason = TEXT("Unable to get landscape layer weights because the cache entry is not available.");
			continue;
		}

		const FVector2D ComponentLocalPoint(LocalPoint.X - ComponentMapKey.X * LandscapeInfo->ComponentSizeQuads, LocalPoint.Y - ComponentMapKey.Y * LandscapeInfo->ComponentSizeQuads);
	
		TArray<FPCGLandscapeLayerWeight> Result;
		CacheEntry->GetInterpolatedLayerWeights(ComponentLocalPoint, Result);

		// Not needed anymore
		CacheEntry.Reset();

		Result.Sort([](const FPCGLandscapeLayerWeight& Lhs, const FPCGLandscapeLayerWeight& Rhs) {
			return Lhs.Weight > Rhs.Weight;
		});

		return Result;
	}

	if (!FailureReason.IsEmpty())
	{
		UE_LOGF(LogPCG, Warning, "[%ls] %ls", *WorldContextObject->GetName(), *FailureReason);
	}

	return {};
}

int64 UPCGBlueprintHelpers::GetTaskId(FPCGContext& Context)
{
	return static_cast<int64>(Context.TaskId);
}

int64 UPCGBlueprintHelpers::GetTaskIdWithContext(const FPCGBlueprintContextHandle& ContextHandle)
{
	if (TSharedPtr<FPCGContextHandle> NativeContextHandle = ContextHandle.Handle.Pin())
	{
		if (FPCGContext* Context = NativeContextHandle->GetContext())
		{
			return GetTaskId(*Context);
		}
	}

	UE_LOGF(LogPCG, Error, "UPCGBlueprintHelpers::GetTaskIdWithContext : Invalid context handle");
	return InvalidPCGTaskId;
}

bool UPCGBlueprintHelpers::FlushPCGCache()
{
	if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
	{
		PCGSubsystem->FlushCache();
		return true;
	}
	else
	{
		return false;
	}
}

void UPCGBlueprintHelpers::RefreshPCGRuntimeComponent(UPCGComponent* InComponent, const bool bFlushCache)
{
	if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
	{
		if (bFlushCache)
		{
			PCGSubsystem->FlushCache();
		}

		PCGSubsystem->RefreshRuntimeGenExecutionSource(InComponent, EPCGChangeType::GenerationGrid);
	}
}

UPCGData* UPCGBlueprintHelpers::DuplicateData(const UPCGData* InData, FPCGContext& Context, bool bInitializeMetadata)
{
	return InData ? InData->DuplicateData(&Context, bInitializeMetadata) : nullptr;
}

UPCGData* UPCGBlueprintHelpers::DuplicateDataWithContext(const UPCGData* InData, const FPCGBlueprintContextHandle& ContextHandle, bool bInitializeMetadata)
{
	if (TSharedPtr<FPCGContextHandle> NativeContextHandle = ContextHandle.Handle.Pin())
	{
		if (FPCGContext* Context = NativeContextHandle->GetContext())
		{
			return DuplicateData(InData, *Context, bInitializeMetadata);
		}
	}

	UE_LOGF(LogPCG, Error, "UPCGBlueprintHelpers::DuplicateDataWithContext : Invalid context handle");
	return nullptr;
}

DEFINE_FUNCTION(UPCGBlueprintHelpers::execExtractStructureAsAttributeSet)
{
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* StructPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_GET_PROPERTY(FNameProperty, PropertyPath);
	P_GET_PROPERTY(FNameProperty, OutputAttributeName);
	P_GET_PROPERTY(FBoolProperty, bExtract);
	P_GET_PROPERTY(FBoolProperty, bStrictlySanitizeNames);

	P_FINISH;

	P_NATIVE_BEGIN;

	UPCGParamData* Result = nullptr;

	const bool bHasValidStructProp = StructProp && StructProp->Struct && StructPtr;

	FPCGAttributePropertySelector InputSelector = FPCGAttributePropertySelector::CreateSelectorFromString<FPCGAttributePropertySelector>(PropertyPath.ToString());

	if (bHasValidStructProp)
	{
		PCGPropertyHelpers::FExtractorParameters Params(StructPtr, StructProp->Struct, InputSelector, OutputAttributeName, /*bInPropertyNeedsToBeVisible=*/true);
		if (bExtract)
		{
			Params.StructExtractorBehavior = EPCGStructExtractorBehavior::Extract;
			Params.ObjectExtractorBehavior = EPCGObjectExtractorBehavior::Extract;
		}

		Params.bStrictSanitizeOutputAttributeNames = bStrictlySanitizeNames;
		Result = PCGPropertyHelpers::ExtractPropertyAsAttributeSet(Params);
		if (!Result)
		{
			ThrowBlueprintException(NSLOCTEXT("PCGPropertyHelpers", "FailedToExtract", "Failed to extract the struct as a ParamData. Check the log for more information."));
		}
	}
	else
	{
		ThrowBlueprintException(NSLOCTEXT("PCGPropertyHelpers", "InvalidStruct", "Parameter passed into \"Structure To Convert\" is not a Structure."));
	}

	*static_cast<UPCGParamData**>(RESULT_PARAM) = Result;

	P_NATIVE_END;
}