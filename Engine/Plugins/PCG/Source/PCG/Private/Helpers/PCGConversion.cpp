// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGConversion.h"

#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGSettings.h"
// TODO: Included temporarily until a graph builder replaces the need for the reroute converter
#include "Elements/PCGReroute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGConversion)

namespace PCGConversion
{
#if WITH_EDITOR
	namespace Helpers
	{
		FText GetDefaultNodeTitle(TSubclassOf<UPCGSettings> Class)
		{
			if (IsValid(Class))
			{
				return CastChecked<UPCGSettings>(Class->GetDefaultObject(false))->GetDefaultNodeTitle();
			}

			return {};
		}
	}
#endif // WITH_EDITOR

	namespace Node::Helpers
	{
		bool RewireSingleEdge(UPCGEdge* Edge, UPCGPin* OldPin, UPCGPin* NewPin)
		{
			if (!Edge || !OldPin || !NewPin || OldPin == NewPin)
			{
				return false;
			}

			UPCGPin* OtherPin = Edge->InputPin == OldPin ? Edge->OutputPin : Edge->InputPin;
			check(OtherPin);
			OtherPin->BreakEdgeTo(OldPin);
			OtherPin->AddEdgeTo(NewPin);

			return true;
		}

		bool RewireAllEdges(UPCGPin* OldPin, UPCGPin* NewPin)
		{
			if (!OldPin || !NewPin || OldPin == NewPin)
			{
				return false;
			}

			while (!OldPin->Edges.IsEmpty())
			{
				RewireSingleEdge(OldPin->Edges[0], OldPin, NewPin);
			}

			return true;
		}
	}
}

FPCGConverterBase::FPCGConverterBase(UPCGGraph* InOutGraph)
	: SourceGraph(InOutGraph)
{
	if (SourceGraph)
	{
		// Mark the graph as initialized.
		CurrentStatus |= EPCGConversionStatus::InitializedGraph;

#if WITH_EDITOR
		SourceGraph->DisableNotificationsForEditor();
		SourceGraph->Modify(/*bAlwaysMarkDirty=*/false);
#endif // WITH_EDITOR
	}
}

FPCGConverterBase::~FPCGConverterBase()
{
	// Implementation Note: Each converter should roll back any changes based on the step completed, if terminated early.
#if WITH_EDITOR
	if (bGraphIsDirty)
	{
		SourceGraph->MarkPackageDirty();
	}
	SourceGraph->EnableNotificationsForEditor();
#endif // WITH_EDITOR
}

void FPCGConverterBase::PrepareData()
{
	bool bMarkGraphAsDirty = false;
	if (ExecutePrepareData(bMarkGraphAsDirty))
	{
		bGraphIsDirty |= bMarkGraphAsDirty;
		// Mark that the data was prepared.
		CurrentStatus |= EPCGConversionStatus::DataPrepared;
	}
}

void FPCGConverterBase::ApplyStructural()
{
	bool bMarkGraphAsDirty = false;
	if (ExecuteApplyStructural(bMarkGraphAsDirty))
	{
		bGraphIsDirty |= bMarkGraphAsDirty;
		CurrentStatus |= EPCGConversionStatus::StructuralChangesApplied;
	}
}

FPCGSingleNodeConverter::FPCGSingleNodeConverter(UPCGNode* InNode, TSubclassOf<UPCGSettings> InTargetSettingsClass)
	: FPCGConverterBase(InNode ? InNode->GetGraph() : nullptr)
	, SourceNode(InNode)
	, TargetSettingsClass(InTargetSettingsClass)
{
	if (SourceNode && SourceNode->GetSettings())
	{
		SetSourceInitialized();
	}
}

FPCGSingleNodeConverter::~FPCGSingleNodeConverter()
{
	// Unravel the pre-structural changes, if cancelled early.
	if (!AreStructuralChangesApplied())
	{
		if (SourceGraph && GeneratedNode)
		{
			SourceGraph->RemoveNode(GeneratedNode);
		}

		bGraphIsDirty = false;
	}
}

bool FPCGSingleNodeConverter::ExecutePrepareData(bool& bOutMarkGraphAsDirty)
{
	if (ensure(IsValid()))
	{
		GeneratedNode = SourceGraph->AddNodeOfType(TargetSettingsClass, GeneratedSettings);

		if (GeneratedNode && GeneratedSettings)
		{
			bOutMarkGraphAsDirty = true;
			if (SourceNode->HasAuthoredTitle())
			{
				GeneratedNode->NodeTitle = SourceNode->NodeTitle;
			}
#if WITH_EDITOR
			SourceNode->TransferEditorProperties(GeneratedNode);
#endif // WITH_EDITOR
			GeneratedSettings->bEnabled = SourceNode->GetSettings()->bEnabled;

			return true;
		}
	}

	return false;
}

bool FPCGSingleNodeConverter::ExecuteApplyStructural(bool& bOutMarkGraphAsDirty)
{
	if (!ensure(IsValid()))
	{
		return false;
	}

	bool bResult = true;
	// TODO: Could have better ways to match pins: based on the same name, etc.
	// For now, the assumption is that the pins should just match ordered 1:1. Do the best to fit them, otherwise discard the rest.
	const TArray<TObjectPtr<UPCGPin>>& SourceInputPins = SourceNode->GetInputPins();
	const TArray<TObjectPtr<UPCGPin>>& GeneratedInputPins = GeneratedNode->GetInputPins();
	for (int32 InputIndex = 0; InputIndex < GeneratedInputPins.Num() && InputIndex < SourceInputPins.Num(); ++InputIndex)
	{
		bResult &= PCGConversion::Node::Helpers::RewireAllEdges(SourceInputPins[InputIndex], GeneratedInputPins[InputIndex]);
	}

	const TArray<TObjectPtr<UPCGPin>>& SourceOutputPins = SourceNode->GetOutputPins();
	const TArray<TObjectPtr<UPCGPin>>& GeneratedOutputPins = GeneratedNode->GetOutputPins();
	for (int32 OutputIndex = 0; OutputIndex < GeneratedOutputPins.Num() && OutputIndex < SourceOutputPins.Num(); ++OutputIndex)
	{
		bResult &= PCGConversion::Node::Helpers::RewireAllEdges(SourceOutputPins[OutputIndex], GeneratedOutputPins[OutputIndex]);
	}

	SourceGraph->RemoveNode(SourceNode);

	return bResult;
}

void FPCGSingleNodeConverter::Finalize()
{
	if (ensure(IsValid()))
	{
		GeneratedNode->UpdateAfterSettingsChangeDuringCreation();
	}
}

FPCGRerouteDeclarationConverter::FPCGRerouteDeclarationConverter(UPCGNode* InNode, FName InNodeTitle)
	: FPCGSingleNodeConverter(InNode, UPCGRerouteSettings::StaticClass())
	, RerouteNodeTitle(InNodeTitle)
{}

bool FPCGRerouteDeclarationConverter::ExecutePrepareData(bool& bOutMarkGraphAsDirty)
{
	if (FPCGSingleNodeConverter::ExecutePrepareData(bOutMarkGraphAsDirty))
	{
		GeneratedNode->NodeTitle = RerouteNodeTitle;
		return true;
	}

	return false;
}

bool FPCGRerouteDeclarationConverter::ExecuteApplyStructural(bool& bOutMarkGraphAsDirty)
{
	if (!ensure(IsValid()))
	{
		return false;
	}

	bool bResult = true;
	const TArray<TObjectPtr<UPCGPin>>& SourceInputPins = SourceNode->GetInputPins();
	const TArray<TObjectPtr<UPCGPin>>& GeneratedInputPins = GeneratedNode->GetInputPins();
	check(SourceInputPins.Num() == 1 && GeneratedInputPins.Num() == 1);
	bResult &= PCGConversion::Node::Helpers::RewireAllEdges(SourceInputPins[0], GeneratedInputPins[0]);

	const TArray<TObjectPtr<UPCGPin>>& GeneratedOutputPins = GeneratedNode->GetOutputPins();
	checkf(GeneratedOutputPins.Num() == 1, TEXT("%s"), TEXT("This class currently only supports converting reroute nodes."));
	if (UPCGNamedRerouteDeclarationSettings* SourceSettings = Cast<UPCGNamedRerouteDeclarationSettings>(SourceNode->GetSettings()))
	{
		const TArray<TObjectPtr<UPCGPin>>& SourceOutputPins = SourceNode->GetOutputPins();
		checkf(SourceOutputPins.Num() == 2, TEXT("%s"), TEXT("This class currently only supports converting reroute nodes."));
		bResult &= PCGConversion::Node::Helpers::RewireAllEdges(SourceOutputPins[0], GeneratedOutputPins[0]);

		const TArray<UPCGNode*> AllNodes = SourceGraph->GetNodes();
		for (UPCGNode* UsageNode : AllNodes)
		{
			if (UsageNode)
			{
				const UPCGNamedRerouteUsageSettings* Settings = Cast<UPCGNamedRerouteUsageSettings>(UsageNode->GetSettings());
				if (Settings && Settings->Declaration == SourceSettings)
				{
					check(UsageNode->GetOutputPins().Num() == 1);
					bResult &= PCGConversion::Node::Helpers::RewireAllEdges(UsageNode->GetOutputPins()[0], GeneratedOutputPins[0]);
					SourceGraph->RemoveNode(UsageNode);
				}
			}
		}
	}

	SourceGraph->RemoveNode(SourceNode);

	return bResult;
}

FPCGReroutePairNodeConverter::FPCGReroutePairNodeConverter(UPCGNode* InRerouteNode, FName InNodeTitle)
	: FPCGConverterBase(InRerouteNode ? InRerouteNode->GetGraph() : nullptr)
	, SourceNode(InRerouteNode)
	, RerouteNodeTitle(InNodeTitle)
{
	if (SourceNode && SourceNode->GetSettings() && SourceNode->GetSettings()->IsA<UPCGRerouteSettings>())
	{
		SetSourceInitialized();
	}
}

void FPCGReroutePairNodeConverter::Finalize()
{
	if (ensure(IsValid()))
	{
		GeneratedDeclarationNode->UpdateAfterSettingsChangeDuringCreation();

		TArray<UPCGNode*, TInlineAllocator<16>> UsageNodes;
		for (auto [DownstreamNode, UsageNode] : DownstreamToUsageNodeMapping)
		{
			if (!UsageNodes.Contains(UsageNode))
			{
				UsageNode->UpdateAfterSettingsChangeDuringCreation();
				UsageNodes.Add(UsageNode);
			}
		}
	}
}

bool FPCGReroutePairNodeConverter::IsValid()
{
	bool bIsValid = SourceGraph != nullptr;
	if (!IsDataPrepared())
	{
		return bIsValid;
	}

	bIsValid &= GeneratedDeclarationNode != nullptr;

	for (auto [DownstreamNode, UsageNode] : DownstreamToUsageNodeMapping)
	{
		bIsValid &= (DownstreamNode != nullptr) && (UsageNode != nullptr);
	}

	return bIsValid;
}

bool FPCGReroutePairNodeConverter::ExecutePrepareData(bool& bOutMarkGraphAsDirty)
{
	if (ensure(IsValid()))
	{
		bOutMarkGraphAsDirty = true;
		GeneratedDeclarationNode = SourceGraph->AddNodeOfType(UPCGNamedRerouteDeclarationSettings::StaticClass(), GeneratedDeclarationSettings);
		DownstreamToUsageNodeMapping.Empty();

		const TArray<TObjectPtr<UPCGPin>>& SourceOutputPins = SourceNode->GetOutputPins();
		checkf(SourceOutputPins.Num() == 1, TEXT("%s"), TEXT("This class currently only supports converting reroute nodes."));

		auto TransferProperties = [this](UPCGNode* Node, UPCGSettings* Settings)
		{
			if (Node && Settings)
			{
#if WITH_EDITOR
				SourceNode->TransferEditorProperties(GeneratedDeclarationNode);
#endif // WITH_EDITOR
				GeneratedDeclarationNode->NodeTitle = RerouteNodeTitle;
				GeneratedDeclarationSettings->bEnabled = SourceNode->GetSettings()->bEnabled;

				return true;
			}

			return false;
		};

		bool bSuccess = TransferProperties(GeneratedDeclarationNode, GeneratedDeclarationSettings);

		for (const UPCGEdge* Edge : SourceOutputPins[0]->Edges)
		{
			const UPCGNode* DownstreamNode = Edge->OutputPin->Node;
			check(DownstreamNode);
			if (!DownstreamToUsageNodeMapping.Contains(DownstreamNode))
			{
				UPCGNode* UsageNode = SourceGraph->AddNodeOfType(UPCGNamedRerouteUsageSettings::StaticClass(), GeneratedUsageSettings.Emplace_GetRef());
				DownstreamToUsageNodeMapping.Add(DownstreamNode, UsageNode);
				bSuccess &= TransferProperties(UsageNode, GeneratedUsageSettings.Last());
				UsageNode->NodeTitle = RerouteNodeTitle;
#if WITH_EDITOR
				// TODO: Temporary hardcoded offset. In the future builder, we should have helpers that will position the
				// node relative to the downstream node, check the graph for overlaps, etc.
				UsageNode->SetNodePosition(DownstreamNode->PositionX - 200, DownstreamNode->PositionY);
#endif // WITH_EDITOR
				UPCGPin* DeclarationInvisiblePin = GeneratedDeclarationNode->GetOutputPin(PCGNamedRerouteConstants::InvisiblePinLabel);
				UPCGPin* UsageInvisiblePin = UsageNode->GetInputPin(PCGPinConstants::DefaultInputLabel);
				DeclarationInvisiblePin->AddEdgeTo(UsageInvisiblePin);
			}
		}

		return bSuccess;
	}

	return false;
}

bool FPCGReroutePairNodeConverter::ExecuteApplyStructural(bool& bOutMarkGraphAsDirty)
{
	if (!ensure(IsValid()))
	{
		return false;
	}

	bool bResult = true;
	const TArray<TObjectPtr<UPCGPin>>& SourceInputPins = SourceNode->GetInputPins();
	const TArray<TObjectPtr<UPCGPin>>& GeneratedInputPins = GeneratedDeclarationNode->GetInputPins();
	for (int32 InputIndex = 0; InputIndex < GeneratedInputPins.Num() && InputIndex < SourceInputPins.Num(); ++InputIndex)
	{
		bResult &= PCGConversion::Node::Helpers::RewireAllEdges(SourceInputPins[InputIndex], GeneratedInputPins[InputIndex]);
	}

	if (!DownstreamToUsageNodeMapping.IsEmpty())
	{
		const TArray<TObjectPtr<UPCGPin>>& SourceOutputPins = SourceNode->GetOutputPins();
		checkf(SourceOutputPins.Num() == 1, TEXT("%s"), TEXT("This class currently only supports converting reroute nodes."));
		for (UPCGPin* Pin : SourceOutputPins)
		{
			check(Pin);
			for (int32 EdgeIndex = 0; !Pin->Edges.IsEmpty(); ++EdgeIndex)
			{
				check(Pin->Edges[0] && Pin->Edges[0]->OutputPin && Pin->Edges[0]->OutputPin->Node)
				UPCGNode* UsageNode = DownstreamToUsageNodeMapping[Pin->Edges[0]->OutputPin->Node];
				check(UsageNode->GetOutputPins().Num() == 1);
				UPCGPin* UsagePin = UsageNode->GetOutputPin(PCGPinConstants::DefaultOutputLabel);
				bResult &= PCGConversion::Node::Helpers::RewireSingleEdge(Pin->Edges[0], Pin, UsagePin);
			}
		}
	}

	SourceGraph->RemoveNode(SourceNode);

	return bResult;
}
