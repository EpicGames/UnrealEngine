// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"

#include "PCGConversion.generated.h"

// TODO:
// - Structural helpers may transition into a Graph Builder system.
// - Factor out the N-NodeConverter into a more generic version that accepts an array of pin to pin mappings.
//   Could additionally have an "alias" class for single node conversion on top of that.

class UPCGPin;
class UPCGSettings;
class UPCGGraph;
class UPCGNode;

namespace PCGConversion
{
#if WITH_EDITOR
	namespace Helpers
	{
		/** Helper to get the default node title from the settings class. */
		FText GetDefaultNodeTitle(TSubclassOf<UPCGSettings> Class);
	}
#endif // WITH_EDITOR
}

UENUM(meta = (Bitflags))
enum class EPCGConversionStatus : uint8
{
	Uninitialized = 0,
	InitializedGraph = 1 << 0,
	InitializedSource = 1 << 1,
	DataPrepared = 1 << 2,
	StructuralChangesApplied = 1 << 3,
	Complete = InitializedSource | InitializedGraph | DataPrepared | StructuralChangesApplied
};

ENUM_CLASS_FLAGS(EPCGConversionStatus);

class FPCGConverterBase
{
public:
	explicit FPCGConverterBase(UPCGGraph* InOutGraph);
	virtual ~FPCGConverterBase();

	// Intentionally bar copy/assignment.
	FPCGConverterBase(const FPCGConverterBase&) = delete;
	FPCGConverterBase(FPCGConverterBase&&) = delete;
	FPCGConverterBase& operator=(const FPCGConverterBase&) = delete;
	FPCGConverterBase& operator=(FPCGConverterBase&&) = delete;

	/** Create whatever new data must exist in the conversion, such as a new node. */
	void PrepareData();
	/** Apply structural changes to the graph. */
	void ApplyStructural();

	/** Overridable validation query. */
	virtual bool IsValid() = 0;

	/** Get the current status of the conversion. */
	EPCGConversionStatus GetCurrentStatus() const { return CurrentStatus; }

	/** Get the conversion's corresponding graph. */
	UPCGGraph* GetGraph() const { return SourceGraph; }

	/** Should be called once the conversion's source has been initialized. */
	void SetSourceInitialized() { CurrentStatus |= EPCGConversionStatus::InitializedSource; }

	bool IsSourceInitialized() const { return !!(CurrentStatus & EPCGConversionStatus::InitializedSource); }
	bool IsGraphInitialized() const { return !!(CurrentStatus & EPCGConversionStatus::InitializedGraph); }
	bool IsDataPrepared() const { return !!(CurrentStatus & EPCGConversionStatus::DataPrepared); }
	bool AreStructuralChangesApplied() const { return !!(CurrentStatus & EPCGConversionStatus::StructuralChangesApplied); }

	/** The entire conversion process is complete. */
	bool IsComplete() const { return CurrentStatus == EPCGConversionStatus::Complete; }

protected:
	/** Overridable and intended to do the work needed to prepare the target conversion data. */
	virtual bool ExecutePrepareData(bool& bOutMarkGraphAsDirty) { return true; }
	/** Overridable and intended to do the work needed to apply structural changes during the conversion. */
	virtual bool ExecuteApplyStructural(bool& bOutMarkGraphAsDirty) { return true; }

	/** The source graph owning the conversion. This should not change during the process. */
	UPCGGraph* const SourceGraph;

	/** The graph will need to be dirtied upon completion. */
	bool bGraphIsDirty = false;

private:
	/** The completed steps of the conversion in bitflag form. */
	EPCGConversionStatus CurrentStatus = EPCGConversionStatus::Uninitialized;
};

/** Converts a single source node into a single compatible target node and rewires edges. */
class FPCGSingleNodeConverter : public FPCGConverterBase
{
public:
	explicit FPCGSingleNodeConverter(UPCGNode* InNode, TSubclassOf<UPCGSettings> InTargetSettingsClass);
	virtual ~FPCGSingleNodeConverter();

	//~Begin FPCGConverterBase interface
	/** The node conversion still has valid references. */
	virtual bool IsValid() override { return SourceGraph && (GeneratedNode || !IsDataPrepared()); }
	//~Begin FPCGConverterBase interface

	/** Finalizes the node conversion. */
	void Finalize();

	UPCGNode* GetSourceNode() const { return SourceNode; }
	UPCGNode* GetGeneratedNode() const { return GeneratedNode; }
	UPCGSettings* GetGeneratedSettings() const { return GeneratedSettings; }

protected:
	//~Begin FPCGConverterBase interface
	/** Create the new node and settings. */
	virtual bool ExecutePrepareData(bool& bOutMarkGraphAsDirty) override;
	/** Rewire the input and output pins. */
	virtual bool ExecuteApplyStructural(bool& bOutMarkGraphAsDirty) override;
	//~Begin FPCGConverterBase interface

	/** The node being converted. */
	UPCGNode* const SourceNode;
	/** The node created during the conversion. */
	UPCGNode* GeneratedNode = nullptr;
	/** The class of the target node. */
	const TSubclassOf<UPCGSettings> TargetSettingsClass;

private:
	/** The settings created during the conversion. */
	UPCGSettings* GeneratedSettings = nullptr;
};

/**
 * [EXPERIMENTAL] Converts a single source node into a single compatible target node and rewires edges.
 * TODO: This class is temporary and will likely be replaced by a non-explicit converter that utilizes a graph building system.
 */
class FPCGRerouteDeclarationConverter : public FPCGSingleNodeConverter
{
public:
	explicit FPCGRerouteDeclarationConverter(UPCGNode* InNode, FName InNodeTitle);

	//~Begin FPCGConverterBase interface
	/** Create the new node and settings. */
	virtual bool ExecutePrepareData(bool& bOutMarkGraphAsDirty) override;
	/** Rewire the input and output pins. */
	virtual bool ExecuteApplyStructural(bool& bOutMarkGraphAsDirty) override;
	//~Begin FPCGConverterBase interface

private:
	FName RerouteNodeTitle = NAME_None;
};

/**
 * [EXPERIMENTAL] Converts a single reroute node into named reroute pairs--the declaration to the upstream node and one
 * usage node for every downstream node.
 * TODO: This class is temporary and will likely be replaced by a non-explicit converter that utilizes a graph building system.
 */
class FPCGReroutePairNodeConverter final : public FPCGConverterBase
{
public:
	explicit FPCGReroutePairNodeConverter(UPCGNode* InRerouteNode, FName InNodeTitle);

	//~Begin FPCGConverterBase interface
	/** The node conversion still has valid references. */
	virtual bool IsValid() override;
	//~Begin FPCGConverterBase interface

	/** Finalizes the node conversion. */
	void Finalize();

	UPCGNode* GetSourceNode() const { return SourceNode; }
	UPCGNode* GetGeneratedDeclarationNode() const { return GeneratedDeclarationNode; }
	UPCGSettings* GetGeneratedDeclarationSettings() const { return GeneratedDeclarationSettings; }
	const TMap<const UPCGNode*, UPCGNode*>& GetDownstreamToUsageNodeMapping() const { return DownstreamToUsageNodeMapping; }
	const TArray<UPCGSettings*>& GetGeneratedUsageSettings() const { return GeneratedUsageSettings; }

protected:
	//~Begin FPCGConverterBase interface
	/** Create the new node and settings. */
	virtual bool ExecutePrepareData(bool& bOutMarkGraphAsDirty) override;
	/** Rewire the input and output pins. */
	virtual bool ExecuteApplyStructural(bool& bOutMarkGraphAsDirty) override;
	//~Begin FPCGConverterBase interface

	UPCGNode* const SourceNode;

private:
	UPCGNode* GeneratedDeclarationNode = nullptr;
	UPCGSettings* GeneratedDeclarationSettings = nullptr;

	FName RerouteNodeTitle = NAME_None;
	TArray<UPCGSettings*> GeneratedUsageSettings;
	TMap<const UPCGNode*, UPCGNode*> DownstreamToUsageNodeMapping;
};
