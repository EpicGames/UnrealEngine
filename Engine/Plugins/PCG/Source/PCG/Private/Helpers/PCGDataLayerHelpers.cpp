// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGDataLayerHelpers.h"

#if WITH_EDITOR
#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGPointData.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Utils/PCGLogErrors.h"

#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Serialization/ArchiveCrc32.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDataLayerHelpers)

#define LOCTEXT_NAMESPACE "PCGDataLayerHelpers"

namespace PCGDataLayerHelpers
{
#if WITH_EDITOR
	namespace Private
	{
		TArray<const UDataLayerInstance*> GetDataLayerInstancesFromAssets(FPCGContext* Context, const TArray<TSoftObjectPtr<UDataLayerAsset>>& DataLayerAssets, UDataLayerManager* DataLayerManager)
		{
			if (!DataLayerManager)
			{
				return {};
			}

			TArray<const UDataLayerInstance*> DataLayerInstances;
			for (const TSoftObjectPtr<UDataLayerAsset>& DataLayerAssetPtr : DataLayerAssets)
			{
				if (UDataLayerAsset* DataLayerAsset = DataLayerAssetPtr.Get())
				{
					if (const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstance(DataLayerAsset))
					{
						DataLayerInstances.Add(DataLayerInstance);
					}
					else
					{
						PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("NoDataLayerInstanceFound", "No DataLayerInstance using DataLayerAsset '{0}' found in World"), FText::FromString(DataLayerAssetPtr.ToString())), Context);
					}
				}
				else
				{
					PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("FailedToResolveDataLayerAsset", "Could not resolve DataLayerAsset '{0}', this probably means your World does not have a DataLayerInstance using it"), FText::FromString(DataLayerAssetPtr.ToString())), Context);
				}
			}

			return DataLayerInstances;
		}

		TArray<const UDataLayerInstance*> GetDataLayerInstancesFromDataLayerReferences(FPCGContext* Context, const FPCGDataLayerSettings& DataLayerSettings, ULevel* Level)
		{
			const TArray<FPCGTaggedData> DataLayersInputs = Context->InputData.GetInputsByPin(PCGDataLayerHelpers::Constants::DataLayerReferenceAttribute);

			if (DataLayersInputs.IsEmpty())
			{
				return {};
			}

			TSet<FSoftObjectPath> DataLayerSoftObjectPaths;
			for (const FPCGTaggedData& DataLayersInput : DataLayersInputs)
			{
				if (!DataLayersInput.Data)
				{
					PCGLog::LogWarningOnGraph(LOCTEXT("DataLayersWrongType", "Data layers input is not of type attribute set."), Context);
					continue;
				}

				TArray<FSoftObjectPath> SoftObjectPaths;
				if (PCGAttributeAccessorHelpers::ExtractAllValues(DataLayersInput.Data, DataLayerSettings.DataLayerReferenceAttribute, SoftObjectPaths, Context, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible, /*bQuiet=*/true))
				{
					DataLayerSoftObjectPaths.Append(SoftObjectPaths);
				}
			}

			TArray<TSoftObjectPtr<UDataLayerAsset>> DataLayerAssetPtrs;
			Algo::Transform(DataLayerSoftObjectPaths, DataLayerAssetPtrs, [](const FSoftObjectPath& DataLayerPath) { return TSoftObjectPtr<UDataLayerAsset>(DataLayerPath); });

			return GetDataLayerInstancesFromAssets(Context, DataLayerAssetPtrs, UDataLayerManager::GetDataLayerManager(Level));
		}

		void FilterDataLayerInstances(FPCGContext* Context, const FPCGDataLayerSettings& DataLayerSettings, TArray<const UDataLayerInstance*>& DataLayerInstances)
		{
			TArray<TSoftObjectPtr<UDataLayerAsset>> ExcludedDataLayerAssets = GetDataLayerAssetsFromInput(Context, PCGDataLayerHelpers::Constants::ExcludedDataLayersAttribute, DataLayerSettings.ExcludedDataLayers);
			TArray<TSoftObjectPtr<UDataLayerAsset>> IncludedDataLayerAssets = GetDataLayerAssetsFromInput(Context, PCGDataLayerHelpers::Constants::IncludedDataLayersAttribute, DataLayerSettings.IncludedDataLayers);

			int32 RemoveIndex = Algo::RemoveIf(DataLayerInstances, [&ExcludedDataLayerAssets, &IncludedDataLayerAssets](const UDataLayerInstance* DataLayerInstance)
			{
				if (const UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance))
				{
					return (!IncludedDataLayerAssets.IsEmpty() && !IncludedDataLayerAssets.Contains(DataLayerInstanceWithAsset->GetAsset()))
						|| ExcludedDataLayerAssets.Contains(DataLayerInstanceWithAsset->GetAsset());

				}

				return false;
			});
			DataLayerInstances.SetNum(RemoveIndex);
		}

		void AddDataLayerInstances(FPCGContext* Context, const FPCGDataLayerSettings& DataLayerSettings, ULevel* Level, TArray<const UDataLayerInstance*>& DataLayerInstances)
		{
			TArray<TSoftObjectPtr<UDataLayerAsset>> AddDataLayerAssets = GetDataLayerAssetsFromInput(Context, PCGDataLayerHelpers::Constants::AddDataLayersAttribute, DataLayerSettings.AddDataLayers);
			TArray<const UDataLayerInstance*> AddDataLayerInstances = GetDataLayerInstancesFromAssets(Context, AddDataLayerAssets, UDataLayerManager::GetDataLayerManager(Level));

			for (const UDataLayerInstance* AddDataLayerInstance : AddDataLayerInstances)
			{
				DataLayerInstances.AddUnique(AddDataLayerInstance);
			}
		}
	} // Private

	TArray<TSoftObjectPtr<UDataLayerAsset>> GetDataLayerAssetsFromInput(FPCGContext* Context, FName InputPinName, const FPCGAttributePropertyInputSelector& InputSelector)
	{
		const TArray<FPCGTaggedData> DataLayerAssetsInputs = Context->InputData.GetInputsByPin(InputPinName);

		if (DataLayerAssetsInputs.IsEmpty())
		{
			PCGLog::LogWarningOnGraph(NSLOCTEXT("PCGDataLayerHelpers", "NoDataLayerAssets", "No data was found on the data layer assets pin."), Context);
			return {};
		}

		TSet<FSoftObjectPath> DataLayerSoftObjectPaths;
		for (const FPCGTaggedData& DataLayerAssetsInput : DataLayerAssetsInputs)
		{
			if (!DataLayerAssetsInput.Data)
			{
				PCGLog::LogWarningOnGraph(LOCTEXT("DataLayerAssetsWrongType", "Data layer assets input is not of type attribute set."), Context);
				continue;
			}

			TArray<FSoftObjectPath> SoftObjectPaths;
			if (!PCGAttributeAccessorHelpers::ExtractAllValues(DataLayerAssetsInput.Data, InputSelector, SoftObjectPaths, Context, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible, /*bQuiet=*/true))
			{
				continue;
			}

			DataLayerSoftObjectPaths.Append(SoftObjectPaths);
		}
				
		TArray<TSoftObjectPtr<UDataLayerAsset>> ReturnValue;
		Algo::Transform(DataLayerSoftObjectPaths, ReturnValue, [](const FSoftObjectPath& AssetPath) { return TSoftObjectPtr<UDataLayerAsset>(AssetPath); });

		return ReturnValue;
	}

	TArray<TSoftObjectPtr<UDataLayerAsset>> GetDataLayerAssetsFromInput(FPCGContext* Context, FName InputPinName, const FPCGDataLayerReferenceSelector& DataLayerSelector)
	{
		if (DataLayerSelector.bAsInput)
		{
			return GetDataLayerAssetsFromInput(Context, InputPinName, DataLayerSelector.Attribute);
		}

		return DataLayerSelector.DataLayers;
	}
		
	TArray<FSoftObjectPath> GetDataLayerAssetsFromActorReferences(FPCGContext* Context, const UPCGData* Data, const FPCGAttributePropertyInputSelector& ActorReferenceAttribute)
	{			
		check(Data);
			
		TArray<FSoftObjectPath> ActorReferences;
		if (!PCGAttributeAccessorHelpers::ExtractAllValues<FSoftObjectPath>(Data, ActorReferenceAttribute, ActorReferences, Context, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible, /*bQuiet=*/true))
		{
			return {};
		}

		TSet<const UDataLayerAsset*> DataLayerAssets;
		for (const FSoftObjectPath& ActorSoftPath : ActorReferences)
		{
			AActor* Actor = Cast<AActor>(ActorSoftPath.ResolveObject());
			if (!Actor)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("UnresolvedActor", "Could not resolve actor path '{0}'."), FText::FromString(ActorSoftPath.ToString())), Context);
				continue;
			}

			DataLayerAssets.Append(GetDatalayerAssetsForActor(Actor));
		}
		
		TArray<FSoftObjectPath> DataLayerSoftObjectPaths;
		Algo::Transform(DataLayerAssets, DataLayerSoftObjectPaths, [](const UDataLayerAsset* DataLayerAsset) { return FSoftObjectPath(const_cast<UDataLayerAsset*>(DataLayerAsset)); });
		
		// Sort for determinism
		DataLayerSoftObjectPaths.Sort([](const FSoftObjectPath& A, const FSoftObjectPath& B)
		{
			return A.LexicalLess(B);
		});

		return DataLayerSoftObjectPaths;
	}
				
	TArray<const UDataLayerInstance*> GetDataLayerInstancesAndCrc(FPCGContext* Context, const FPCGDataLayerSettings& DataLayerSettings, AActor* DefaultDataLayerSource, int32& OutCrc)
	{
		TArray<const UDataLayerInstance*> DataLayerInstances;

		switch (DataLayerSettings.DataLayerSourceType)
		{
		case EPCGDataLayerSource::Self:
		{
			DataLayerInstances = DefaultDataLayerSource->GetDataLayerInstancesForLevel();
			break;
		}
		case EPCGDataLayerSource::DataLayerReferences:
		{
			DataLayerInstances = Private::GetDataLayerInstancesFromDataLayerReferences(Context, DataLayerSettings, DefaultDataLayerSource->GetLevel());
			break;
		}
		}

		Private::FilterDataLayerInstances(Context, DataLayerSettings, DataLayerInstances);

		Private::AddDataLayerInstances(Context, DataLayerSettings, DefaultDataLayerSource->GetLevel(), DataLayerInstances);

		// Sort for determinism
		DataLayerInstances.Sort([](const UDataLayerInstance& A, const UDataLayerInstance& B)
		{
			return A.GetPathName() < B.GetPathName();
		});

		// Crc DataLayerInstances
		FArchiveCrc32 Ar;
		for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
		{
			UDataLayerInstance* NonConstInstance = const_cast<UDataLayerInstance*>(DataLayerInstance);
			Ar << NonConstInstance;
		}
		OutCrc = Ar.GetCrc();

		return DataLayerInstances;
	}

	TArray<const UDataLayerAsset*> GetDatalayerAssetsForActor(AActor* InActor)
	{
		ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(InActor->GetWorld());
		ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem ? LevelInstanceSubsystem->GetOwningLevelInstance(InActor->GetLevel()) : nullptr;
		if (AActor* LevelInstanceActor = Cast<AActor>(LevelInstance))
		{
			TSet<const UDataLayerAsset*> DataLayerAssets;
			DataLayerAssets.Append(LevelInstanceActor->GetDataLayerAssets(true));
			DataLayerAssets.Append(InActor->GetDataLayerAssets(false));
			return DataLayerAssets.Array();
		}
		else
		{
			return InActor->GetDataLayerAssets(true);
		}
	}

#endif // WITH_EDITOR
}

FPCGDataLayerSettings::FPCGDataLayerSettings()
{
	DataLayerReferenceAttribute.SetAttributeName(PCGDataLayerHelpers::Constants::DataLayerReferenceAttribute);
	IncludedDataLayers.Attribute.SetAttributeName(PCGDataLayerHelpers::Constants::IncludedDataLayersAttribute);
	ExcludedDataLayers.Attribute.SetAttributeName(PCGDataLayerHelpers::Constants::ExcludedDataLayersAttribute);
	AddDataLayers.Attribute.SetAttributeName(PCGDataLayerHelpers::Constants::AddDataLayersAttribute);
}

TArray<FPCGPinProperties> FPCGDataLayerSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;

	if (DataLayerSourceType == EPCGDataLayerSource::DataLayerReferences)
	{
		FPCGPinProperties& IncludedDataLayersPin = PinProperties.Emplace_GetRef(PCGDataLayerHelpers::Constants::DataLayerReferenceAttribute, EPCGDataType::Param);
		IncludedDataLayersPin.SetRequiredPin();
	}

	if (IncludedDataLayers.bAsInput)
	{
		FPCGPinProperties& IncludedDataLayersPin = PinProperties.Emplace_GetRef(PCGDataLayerHelpers::Constants::IncludedDataLayersAttribute, EPCGDataType::Param);
		IncludedDataLayersPin.SetRequiredPin();
	}

	if (ExcludedDataLayers.bAsInput)
	{
		FPCGPinProperties& ExcludedDataLayersPin = PinProperties.Emplace_GetRef(PCGDataLayerHelpers::Constants::ExcludedDataLayersAttribute, EPCGDataType::Param);
		ExcludedDataLayersPin.SetRequiredPin();
	}

	if (AddDataLayers.bAsInput)
	{
		FPCGPinProperties& AddDataLayersPin = PinProperties.Emplace_GetRef(PCGDataLayerHelpers::Constants::AddDataLayersAttribute, EPCGDataType::Param);
		AddDataLayersPin.SetRequiredPin();
	}

	return PinProperties;
}

#if WITH_EDITOR
EPCGChangeType FPCGDataLayerSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = EPCGChangeType::None;

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FPCGDataLayerReferenceSelector, bAsInput)
		|| InPropertyName == GET_MEMBER_NAME_CHECKED(FPCGDataLayerSettings, DataLayerSourceType))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif

#undef LOCTEXT_NAMESPACE
