// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGDefaultValueContainer.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Helpers/PCGGraphParameterExtension.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Metadata/PCGMetadataAttributeTraits.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDefaultValueContainer)

const FProperty* FPCGDefaultValueContainer::CreateNewProperty(const FName PropertyName, const EPCGMetadataTypes Type)
{
	if (PropertyName == NAME_None || !PCGMetadataHelpers::MetadataTypeSupportsDefaultValues(Type))
	{
		return nullptr;
	}

	if (PropertyBag.FindPropertyDescByName(PropertyName))
	{
		PropertyBag.RemovePropertyByName(PropertyName);
	}

	const FPropertyBagPropertyDesc PropertyDesc = PCGPropertyHelpers::CreatePropertyBagDescWithMetadataType(PropertyName, Type);
	PropertyBag.AddProperties({PropertyDesc});
	if (const FPropertyBagPropertyDesc* OutPropertyDesc = PropertyBag.FindPropertyDescByName(PropertyName))
	{
		return OutPropertyDesc->CachedProperty;
	}
	else
	{
		return nullptr;
	}
}

const FProperty* FPCGDefaultValueContainer::FindProperty(const FName PropertyName) const
{
	const FPropertyBagPropertyDesc* PropertyDesc = PropertyBag.FindPropertyDescByName(PropertyName);
	return PropertyDesc ? PropertyDesc->CachedProperty : nullptr;
}

void FPCGDefaultValueContainer::RemoveProperty(const FName PropertyName)
{
	PropertyBag.RemovePropertyByName(PropertyName);
}

EPCGMetadataTypes FPCGDefaultValueContainer::GetCurrentPropertyType(const FName PropertyName) const
{
	const FProperty* Property = FindProperty(PropertyName);
	return Property ? PCGPropertyHelpers::GetMetadataTypeFromProperty(Property) : EPCGMetadataTypes::Unknown;
}

FString FPCGDefaultValueContainer::GetPropertyValueAsString(const FName PropertyName) const
{
	TValueOrError<FString, EPropertyBagResult> Result = PropertyBag.GetValueSerializedString(PropertyName);

	return Result.HasValue() ? Result.GetValue() : FString(TEXT("Error"));
}

const UPCGParamData* FPCGDefaultValueContainer::CreateParamData(FPCGContext* Context, const FName PropertyKey, const FName AttributeName) const
{
	if (const FProperty* PropertyPtr = FindProperty(PropertyKey))
	{
		const TObjectPtr<UPCGParamData> NewParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
		if (NewParamData->Metadata->CreateAttributeFromDataProperty(AttributeName, PropertyBag.GetValue().GetMemory(), PropertyPtr))
		{
			NewParamData->Metadata->AddEntry();
			return NewParamData;
		}
	}

	return nullptr;
}

void FPCGDefaultValueContainer::SerializeCrc(FArchiveCrc32& Crc32) const
{
	// const_cast safe: FArchiveCrc32 is read-only, FInstancedPropertyBag::Serialize is non-const due to FArchive API.
	const_cast<FInstancedPropertyBag&>(PropertyBag).Serialize(Crc32);
}

bool FPCGDefaultValueContainer::IsPropertyActivated(const FName PropertyName) const
{
	return ActivatedProperties.Contains(PropertyName);
}

#if WITH_EDITOR
const FProperty* FPCGDefaultValueContainer::ConvertPropertyType(const FName PropertyName, const EPCGMetadataTypes Type)
{
	if (!PCGMetadataHelpers::MetadataTypeSupportsDefaultValues(Type) || Type == GetCurrentPropertyType(PropertyName))
	{
		return nullptr;
	}

	return CreateNewProperty(PropertyName, Type);
}

bool FPCGDefaultValueContainer::SetPropertyValueFromString(const FName PropertyName, const FString& ValueString)
{
	if (PropertyName == NAME_None)
	{
		return false;
	}

	SetPropertyActivated(PropertyName, /*bIsActive=*/true);
	const EPropertyBagResult Result = PropertyBag.SetValueSerializedString(PropertyName, ValueString);
	return Result == EPropertyBagResult::Success;
}

bool FPCGDefaultValueContainer::SetPropertyActivated(const FName PropertyName, const bool bIsActivated)
{
	if ((PropertyName != NAME_None) && (bIsActivated != ActivatedProperties.Contains(PropertyName)))
	{
		if (bIsActivated)
		{
			ActivatedProperties.Add(PropertyName);
			return true;
		}
		else
		{
			ActivatedProperties.Remove(PropertyName);
			return true;
		}
	}
	else
	{
		return false;
	}
}

void FPCGDefaultValueContainer::Reset()
{
	ActivatedProperties.Reset();
	PropertyBag.Reset();
}
#endif // WITH_EDITOR
