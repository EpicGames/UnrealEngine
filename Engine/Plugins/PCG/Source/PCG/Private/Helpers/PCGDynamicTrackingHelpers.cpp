// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGDynamicTrackingHelpers.h"

#if WITH_EDITOR

#include "PCGContext.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGSettings.h"

void FPCGDynamicTrackingHelper::EnableAndInitialize(const FPCGContext* InContext, int32 OptionalNumElements)
{
	if (InContext && InContext->ExecutionSource.IsValid())
	{
		const UPCGSettings* Settings = InContext->GetOriginalSettings<UPCGSettings>();
		if (Settings && Settings->CanDynamicallyTrackKeys())
		{
			CachedExecutionSource = InContext->ExecutionSource;
			bDynamicallyTracked = true;
			DynamicallyTrackedKeysAndCulling.Reserve(OptionalNumElements);
		}
	}
}

void FPCGDynamicTrackingHelper::AddToTracking(FPCGSelectionKey&& InKey, bool bIsCulled)
{
	if (bDynamicallyTracked)
	{
		DynamicallyTrackedKeysAndCulling.AddUnique(TPair<FPCGSelectionKey, bool>(std::forward<FPCGSelectionKey>(InKey), bIsCulled));
	}
}

void FPCGDynamicTrackingHelper::Finalize(const FPCGContext* InContext)
{
	if (InContext && bDynamicallyTracked && CachedExecutionSource == InContext->ExecutionSource)
	{
		if (IPCGGraphExecutionSource* ExecutionSource = CachedExecutionSource.Get())
		{
			ExecutionSource->GetExecutionState().RegisterDynamicTracking(InContext->GetOriginalSettings<UPCGSettings>(), DynamicallyTrackedKeysAndCulling);
		}
	}
}

void FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(FPCGContext* InContext, FPCGSelectionKey&& InKey, bool bIsCulled)
{
	if (InContext)
	{
		if (IPCGGraphExecutionSource* ExecutionSource = InContext->ExecutionSource.Get())
		{
			const UPCGSettings* Settings = InContext->GetOriginalSettings<UPCGSettings>();
			
			if(Settings && Settings->CanDynamicallyTrackKeys())
			{
				TPair<FPCGSelectionKey, bool> NewPair(std::forward<FPCGSelectionKey>(InKey), bIsCulled);
				ExecutionSource->GetExecutionState().RegisterDynamicTracking(Settings, MakeArrayView(&NewPair, 1));
			}
		}
	}
}

void FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(FPCGContext* InContext, const FPCGActorSelectorSettings& InSelector)
{
	AddSingleDynamicTrackingKey(InContext, FPCGSelectionKey(InSelector), InSelector.bMustOverlapSelf);
}

#endif // WITH_EDITOR
