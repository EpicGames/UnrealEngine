// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGGraphParameterExtension.h"

#include "PCGGraph.h"

////////////
// Getters
////////////
template<> TValueOrError<float, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueFloat(PropertyName);
}
template<> TValueOrError<double, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueDouble(PropertyName);
}
template<> TValueOrError<bool, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueBool(PropertyName);
}
template<> TValueOrError<uint8, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueByte(PropertyName);
}
template<> TValueOrError<int32, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueInt32(PropertyName);
}
template<> TValueOrError<int64, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueInt64(PropertyName);
}
template<> TValueOrError<FName, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueName(PropertyName);
}
template<> TValueOrError<FString, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueString(PropertyName);
}
template<> TValueOrError<FSoftObjectPath, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	TValueOrError<FSoftObjectPath, EPropertyBagResult> ValueOrError = PropertyBag.GetValueSoftPath(PropertyName);

	// Fallback to struct approach
	if (ValueOrError.HasError())
	{
		TValueOrError<FSoftObjectPath*, EPropertyBagResult> StructPtrOrError = PropertyBag.GetValueStruct<FSoftObjectPath>(PropertyName);
		if (!StructPtrOrError.HasError() && StructPtrOrError.GetValue())
		{
			return MakeValue(*StructPtrOrError.GetValue());
		}
	}

	return ValueOrError;
}
template<> TValueOrError<FSoftClassPath, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	TValueOrError<FSoftObjectPath, EPropertyBagResult> InitialResult = PropertyBag.GetValueSoftPath(PropertyName);

	// Fallback to struct approach
	if (InitialResult.HasError())
	{
		TValueOrError<FSoftClassPath*, EPropertyBagResult> StructPtrOrError = PropertyBag.GetValueStruct<FSoftClassPath>(PropertyName);
		if (!StructPtrOrError.HasError() && StructPtrOrError.GetValue())
		{
			return MakeValue(*StructPtrOrError.GetValue());
		}
		else
		{
			return MakeError(InitialResult.GetError());
		}
	}
	else
	{
		return MakeValue(InitialResult.GetValue().ToString());
	}
}
template<> TValueOrError<TSoftObjectPtr<UObject>, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	TValueOrError<FSoftObjectPath, EPropertyBagResult> InitialResult = PropertyBag.GetValueSoftPath(PropertyName);
	if (InitialResult.HasError())
	{
		return MakeError(InitialResult.GetError());
	}
	TValueOrError<TSoftObjectPtr<UObject>, EPropertyBagResult> FinalResult(MakeValue(TSoftObjectPtr<UObject>(InitialResult.GetValue())));
	return FinalResult;
}
template<> TValueOrError<TSoftClassPtr<UObject>, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	TValueOrError<FSoftObjectPath, EPropertyBagResult> InitialResult = PropertyBag.GetValueSoftPath(PropertyName);
	if (InitialResult.HasError())
	{
		return MakeError(InitialResult.GetError());
	}
	TValueOrError<TSoftClassPtr<UObject>, EPropertyBagResult> FinalResult(MakeValue(TSoftClassPtr<UObject>(InitialResult.GetValue())));
	return FinalResult;
}
template<> TValueOrError<UObject*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueObject<UObject>(PropertyName);
}
template<> TValueOrError<UClass*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueClass(PropertyName);
}
template<> TValueOrError<FVector*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueStruct<FVector>(PropertyName);
}
template<> TValueOrError<FRotator*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueStruct<FRotator>(PropertyName);
}
template<> TValueOrError<FTransform*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueStruct<FTransform>(PropertyName);
}
template<> TValueOrError<FVector4*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueStruct<FVector4>(PropertyName);
}
template<> TValueOrError<FVector2D*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueStruct<FVector2D>(PropertyName);
}
template<> TValueOrError<FQuat*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueStruct<FQuat>(PropertyName);
}

TValueOrError<uint64, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName, const UEnum* Enum)
{
	// TODO: To stay congruent with BP Enums and Property Bag, Value converted from uint8. Update this if/when this is improved externally
	TValueOrError<uint8, EPropertyBagResult> Result =  Enum ? PropertyBag.GetValueEnum(PropertyName, Enum) : PropertyBag.GetValueByte(PropertyName);
	if (Result.HasError())
	{
		return MakeError(Result.GetError());
	}

	return MakeValue(static_cast<uint64>(Result.GetValue()));
}

////////////
// Setters
////////////
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const float& Value)
{
	return PropertyBag.SetValueFloat(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const double& Value)
{
	return PropertyBag.SetValueDouble(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const bool& bValue)
{
	return PropertyBag.SetValueBool(PropertyName, bValue);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const uint8& Value)
{
	return PropertyBag.SetValueByte(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const int32& Value)
{
	return PropertyBag.SetValueInt32(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const int64& Value)
{
	return PropertyBag.SetValueInt64(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FName& Value)
{
	return PropertyBag.SetValueName(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FString& Value)
{
	return PropertyBag.SetValueString(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FSoftObjectPath& Value)
{
	EPropertyBagResult OpResult = PropertyBag.SetValueSoftPath(PropertyName, Value);
	if (OpResult == EPropertyBagResult::TypeMismatch)
	{
		OpResult = PropertyBag.SetValueStruct(PropertyName, Value);
	}

	return OpResult;
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FSoftClassPath& Value)
{
	EPropertyBagResult OpResult = PropertyBag.SetValueSoftPath(PropertyName, Value);
	if (OpResult == EPropertyBagResult::TypeMismatch)
	{
		OpResult = PropertyBag.SetValueStruct(PropertyName, Value);
	}

	return OpResult;
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const TSoftObjectPtr<UObject>& Value)
{
	return PropertyBag.SetValueSoftPath(PropertyName, Value.ToSoftObjectPath());
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const TSoftClassPtr<UObject>& Value)
{
	return PropertyBag.SetValueSoftPath(PropertyName, Value.ToSoftObjectPath());
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, UObject* const& Value)
{
	return PropertyBag.SetValueObject<UObject>(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, UClass* const& Value)
{
	return PropertyBag.SetValueClass(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FVector& Value)
{
	return PropertyBag.SetValueStruct(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FRotator& Value)
{
	return PropertyBag.SetValueStruct(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FTransform& Value)
{
	return PropertyBag.SetValueStruct(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FVector4& Value)
{
	return PropertyBag.SetValueStruct(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FVector2D& Value)
{
	return PropertyBag.SetValueStruct(PropertyName, Value);
}
template<> EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FQuat& Value)
{
	return PropertyBag.SetValueStruct(PropertyName, Value);
}

EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const uint64& Value, const UEnum* Enum)
{
	// TODO: To stay congruent with BP Enums and Property Bag, Value converted to uint8. Update this if/when this is improved externally
	return Enum ? PropertyBag.SetValueEnum(PropertyName, static_cast<uint8>(Value), Enum) : PropertyBag.SetValueByte(PropertyName, static_cast<uint8>(Value));
}

// DEPRECATED 5.7
template<> TValueOrError<FSoftObjectPath*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueStruct<FSoftObjectPath>(PropertyName);
}
// DEPRECATED 5.7
template<> TValueOrError<FSoftClassPath*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName)
{
	return PropertyBag.GetValueStruct<FSoftClassPath>(PropertyName);
}