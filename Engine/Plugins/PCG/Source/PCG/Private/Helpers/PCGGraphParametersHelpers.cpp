// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGGraphParametersHelpers.h"

#include "PCGModule.h"
#include "PCGGraph.h"
#include "Helpers/PCGBlueprintHelpers.h"

#include "StructUtils/PropertyBag.h"
#include "Templates/ValueOrError.h"
#include "UObject/Script.h"
#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGraphParametersHelpers)

#define LOCTEXT_NAMESPACE "PCGGraphParametersHelpers"

// Helper functions for this blueprint library
namespace PCGGraphParametersHelpersPrivate
{
	static const FText InvalidGraphInterfaceError = LOCTEXT("InvalidGraphInterface", "Invalid graph interface.");
	static const FName NewGraphParameterDefaultName(LOCTEXT("NewGraphParameterDefaultName", "NewProperty").ToString());
	static constexpr uint64 MaxIncrementalNameIterations = 100;

	void ThrowBlueprintException(const FText& ErrorMessage)
	{
		UPCGBlueprintHelpers::ThrowBlueprintException(ErrorMessage);
	}

	void OnException(const EPropertyBagResult Result, const FName PropertyName)
	{
		FText ErrorMessage;
		switch (Result)
		{
		case EPropertyBagResult::Success:
			return;
		case EPropertyBagResult::TypeMismatch:
			ErrorMessage = FText::Format(LOCTEXT("TypeMismatch", "Parameter {0} is not of the right type."), FText::FromName(PropertyName));
			break;
		case EPropertyBagResult::OutOfBounds:
			ErrorMessage = FText::Format(LOCTEXT("OutOfBounds", "Parameter {0} is out of bounds."), FText::FromName(PropertyName));
			break;
		case EPropertyBagResult::PropertyNotFound:
			ErrorMessage = FText::Format(LOCTEXT("PropertyNotFound", "Parameter {0} does not exist."), FText::FromName(PropertyName));
			break;
		default:
			checkNoEntry();
		}

		ThrowBlueprintException(ErrorMessage);

		if (FPlatformMisc::IsDebuggerPresent())
		{
			ensureMsgf(false, TEXT("%s"), *ErrorMessage.ToString());
		}
	}

	// Usually returns T, but in some cases (like soft objects or struct), it returns a ptr or something else.
	// That's what U is for, and there are strict conditions on the relation between T and U.
	template <typename T, typename U = T>
	T ValidateAndReturnResult(const UPCGGraphInterface* GraphInterface, const FName PropertyName)
	{
		if (!GraphInterface)
		{
			return T{};
		}

		const TValueOrError<U, EPropertyBagResult> Result = GraphInterface->GetGraphParameter<U>(PropertyName);
		if (Result.HasError())
		{
			OnException(Result.GetError(), PropertyName);
			return T{};
		}

		if constexpr (!std::is_same_v<T, U> && std::is_pointer_v<U>)
		{
			static_assert(std::is_same_v<T, std::remove_pointer_t<U>>);
			return Result.GetValue() ? *Result.GetValue() : T{};
		}
		else if constexpr (!std::is_same_v<T, U>)
		{
			return T(Result.GetValue());
		}
		else
		{
			return Result.GetValue();
		}
	}

	template <typename T>
	void ValidateAndSetValue(UPCGGraphInterface* GraphInterface, const FName PropertyName, const T& Value)
	{
		if (!GraphInterface)
		{
			ThrowBlueprintException(InvalidGraphInterfaceError);
			return;
		}

		const EPropertyBagResult Result = GraphInterface->SetGraphParameter<T>(PropertyName, Value);
		if (Result != EPropertyBagResult::Success)
		{
			OnException(Result, PropertyName);
		}
	}

	void ValidateAndSetValue(UPCGGraphInterface* GraphInterface, const FName PropertyName, const uint64& Value, const UEnum* Enum)
	{
		if (!GraphInterface)
		{
			ThrowBlueprintException(InvalidGraphInterfaceError);
			return;
		}

		const EPropertyBagResult Result = GraphInterface->SetGraphParameter(PropertyName, Value, Enum);
		if (Result != EPropertyBagResult::Success)
		{
			OnException(Result, PropertyName);
		}
	}
}

namespace PCGGraphParameter::Helpers
{
	bool GenerateUniqueName(const UPCGGraphInterface* InGraph, FName& InOutName)
	{
		if (!InGraph)
		{
			return false;
		}

		if (InOutName == NAME_None)
		{
			InOutName = PCGGraphParametersHelpersPrivate::NewGraphParameterDefaultName;
		}

		const FInstancedPropertyBag* UserParametersStruct = InGraph->GetUserParametersStruct();
		check(UserParametersStruct);

		// Validate that the property name doesn't exist... if it does, increment until it doesn't
		uint64 NameIteration = 1;
		while (UserParametersStruct->FindPropertyDescByName(InOutName))
		{
			if (++NameIteration > PCGGraphParametersHelpersPrivate::MaxIncrementalNameIterations)
			{
				return false;
			}

			InOutName.SetNumber(NameIteration);
		}

		return true;
	}

	void LogGraphParamNamingErrors(const FName IntendedName, const EPropertyBagAlterationResult Result, const FPCGContext* InContext)
	{
		switch (Result)
		{
			case EPropertyBagAlterationResult::PropertyNameEmpty:
				PCGLog::LogErrorOnGraph(LOCTEXT("EmptyPropertyName", "Could not add graph parameter with empty name."), InContext);
				return;
			case EPropertyBagAlterationResult::PropertyNameInvalidCharacters:
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidCharacters", "Could not add graph parameter '{0}'. It contains invalid characters."), FText::FromName(IntendedName)), InContext);
				return;
			case EPropertyBagAlterationResult::TargetPropertyAlreadyExists:
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("AlreadyExists", "Could not add graph parameter '{0}'. It already exists."), FText::FromName(IntendedName)), InContext);
				return;
			case EPropertyBagAlterationResult::InternalError:
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InternalError", "Could not add graph parameter '{0}' due to internal problems with the property bag."), FText::FromName(IntendedName)), InContext);
				return;
			default: // fall-through
			case EPropertyBagAlterationResult::Success:
				return;
		}
	}
} // PCGGraphParameter::Helpers

//////////////////////////
// Begin Blueprint Library
//////////////////////////

bool UPCGGraphParametersHelpers::IsOverridden(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return GraphInterface && GraphInterface->IsGraphParameterOverridden(Name);
}

////////////
// Getters
////////////

float UPCGGraphParametersHelpers::GetFloatParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<float>(GraphInterface, Name);
}

double UPCGGraphParametersHelpers::GetDoubleParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<double>(GraphInterface, Name);
}

bool UPCGGraphParametersHelpers::GetBoolParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<bool>(GraphInterface, Name);
}

uint8 UPCGGraphParametersHelpers::GetByteParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<uint8>(GraphInterface, Name);
}

int32 UPCGGraphParametersHelpers::GetInt32Parameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<int32>(GraphInterface, Name);
}

int64 UPCGGraphParametersHelpers::GetInt64Parameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<int64>(GraphInterface, Name);
}

FName UPCGGraphParametersHelpers::GetNameParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<FName>(GraphInterface, Name);
}

FString UPCGGraphParametersHelpers::GetStringParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<FString>(GraphInterface, Name);
}

uint8 UPCGGraphParametersHelpers::GetEnumParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<uint8>(GraphInterface, Name);
}

FSoftObjectPath UPCGGraphParametersHelpers::GetSoftObjectPathParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<FSoftObjectPath>(GraphInterface, Name);
}

TSoftObjectPtr<UObject> UPCGGraphParametersHelpers::GetSoftObjectParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<TSoftObjectPtr<UObject>>(GraphInterface, Name);
}

TSoftClassPtr<UObject> UPCGGraphParametersHelpers::GetSoftClassParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<TSoftClassPtr<UObject>>(GraphInterface, Name);
}

UObject* UPCGGraphParametersHelpers::GetObjectParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<UObject*>(GraphInterface, Name);
}

UClass* UPCGGraphParametersHelpers::GetClassParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<UClass*>(GraphInterface, Name);
}

FVector UPCGGraphParametersHelpers::GetVectorParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<FVector, FVector*>(GraphInterface, Name);
}

FRotator UPCGGraphParametersHelpers::GetRotatorParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<FRotator, FRotator*>(GraphInterface, Name);
}

FTransform UPCGGraphParametersHelpers::GetTransformParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<FTransform, FTransform*>(GraphInterface, Name);
}

FVector4 UPCGGraphParametersHelpers::GetVector4Parameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<FVector4, FVector4*>(GraphInterface, Name);
}

FVector2D UPCGGraphParametersHelpers::GetVector2DParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<FVector2D, FVector2D*>(GraphInterface, Name);
}

FQuat UPCGGraphParametersHelpers::GetQuaternionParameter(const UPCGGraphInterface* GraphInterface, const FName Name)
{
	return PCGGraphParametersHelpersPrivate::ValidateAndReturnResult<FQuat, FQuat*>(GraphInterface, Name);
}


////////////
// Setters
////////////
void UPCGGraphParametersHelpers::SetFloatParameter(UPCGGraphInterface* GraphInterface, const FName Name, const float Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetDoubleParameter(UPCGGraphInterface* GraphInterface, const FName Name, const double Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetBoolParameter(UPCGGraphInterface* GraphInterface, const FName Name, const bool bValue)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, bValue);
}

void UPCGGraphParametersHelpers::SetByteParameter(UPCGGraphInterface* GraphInterface, const FName Name, const uint8 Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetInt32Parameter(UPCGGraphInterface* GraphInterface, const FName Name, const int32 Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetInt64Parameter(UPCGGraphInterface* GraphInterface, const FName Name, const int64 Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetNameParameter(UPCGGraphInterface* GraphInterface, const FName Name, const FName Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetStringParameter(UPCGGraphInterface* GraphInterface, const FName Name, const FString Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetEnumParameter(UPCGGraphInterface* GraphInterface, const FName Name, const uint8 Value, const UEnum* Enum)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value, Enum);
}

void UPCGGraphParametersHelpers::SetSoftObjectPathParameter(UPCGGraphInterface* GraphInterface, const FName Name, const FSoftObjectPath& Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetSoftObjectParameter(UPCGGraphInterface* GraphInterface, const FName Name, const TSoftObjectPtr<UObject>& Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetSoftClassParameter(UPCGGraphInterface* GraphInterface, const FName Name, const TSoftClassPtr<UObject>& Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetObjectParameter(UPCGGraphInterface* GraphInterface, const FName Name, UObject* Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetClassParameter(UPCGGraphInterface* GraphInterface, const FName Name, UClass* Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetVectorParameter(UPCGGraphInterface* GraphInterface, const FName Name, const FVector& Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetRotatorParameter(UPCGGraphInterface* GraphInterface, const FName Name, const FRotator& Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetTransformParameter(UPCGGraphInterface* GraphInterface, const FName Name, const FTransform& Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetVector4Parameter(UPCGGraphInterface* GraphInterface, const FName Name, const FVector4& Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetVector2DParameter(UPCGGraphInterface* GraphInterface, const FName Name, const FVector2D& Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

void UPCGGraphParametersHelpers::SetQuaternionParameter(UPCGGraphInterface* GraphInterface, const FName Name, const FQuat& Value)
{
	PCGGraphParametersHelpersPrivate::ValidateAndSetValue(GraphInterface, Name, Value);
}

#undef LOCTEXT_NAMESPACE
