// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGHLODHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGHLODHelpers)

#define LOCTEXT_NAMESPACE "PCGHLODHelpers"

#if WITH_EDITOR
#include "PCGContext.h"

#include "GameFramework/Actor.h"
#include "Serialization/ArchiveCrc32.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#endif

namespace PCGHLODHelpers
{
#if WITH_EDITOR
	UHLODLayer* GetHLODLayerAndCrc(FPCGContext* Context, const FPCGHLODSettings& HLODSettings, AActor* DefaultHLODLayerSource, AActor* TemplateActor, int32& OutCrc)
	{
		UHLODLayer* HLODLayer = nullptr;
		
		switch (HLODSettings.HLODSourceType)
		{
		case EPCGHLODSource::Self:
		{
			if(ensure(DefaultHLODLayerSource))
			{
				HLODLayer = DefaultHLODLayerSource->GetHLODLayer();
			}
			break;
		}
		case EPCGHLODSource::Template:
		{
			if (ensure(TemplateActor))
			{
				HLODLayer = TemplateActor->GetHLODLayer();
			}
			break;
		}
		case EPCGHLODSource::Reference:
		{
			HLODLayer = HLODSettings.HLODLayer;
			break;
		}
		}

		OutCrc = 0;
		if (HLODLayer)
		{
			FArchiveCrc32 Ar;
			FString HLODLayerPath = HLODLayer->GetPathName();
			Ar << HLODLayerPath;
			OutCrc = Ar.GetCrc();
		}

		return HLODLayer;
	}
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
