// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGHelpers.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGData.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGSettings.h"
#include "PCGSubsystem.h"
#include "PCGWorldActor.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGActorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Components/BillboardComponent.h"
#include "Landscape.h"
#include "LandscapeProxy.h"
#include "Algo/AnyOf.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGHelpers)

#if WITH_EDITOR
#include "Editor.h"
#else
#include "Engine/World.h"
#endif

#define LOCTEXT_NAMESPACE "PCGHelpers"

namespace PCGHelpers
{
	int ComputeSeed(int A)
	{
		return (A * 196314165U) + 907633515U;
	}

	int ComputeSeed(int A, int B)
	{
		return ((A * 196314165U) + 907633515U) ^ ((B * 73148459U) + 453816763U);
	}

	int ComputeSeed(int A, int B, int C)
	{
		return ((A * 196314165U) + 907633515U) ^ ((B * 73148459U) + 453816763U) ^ ((C * 34731343U) + 453816743U);
	}

	int ComputeSeedFromPosition(const FVector& InPosition)
	{
		return ComputeSeed(static_cast<int>(InPosition.X), static_cast<int>(InPosition.Y), static_cast<int>(InPosition.Z));
	}

	FRandomStream GetRandomStreamFromSeed(int32 InSeed, const UPCGSettings* OptionalSettings, const IPCGGraphExecutionSource* OptionalExecutionSource)
	{
		int Seed = InSeed;

		if (OptionalSettings && OptionalExecutionSource)
		{
			Seed = PCGHelpers::ComputeSeed(Seed, OptionalSettings->Seed, OptionalExecutionSource->GetExecutionState().GetSeed());
		}
		else if (OptionalSettings)
		{
			Seed = PCGHelpers::ComputeSeed(Seed, OptionalSettings->Seed);
		}
		else if (OptionalExecutionSource)
		{
			Seed = PCGHelpers::ComputeSeed(Seed, OptionalExecutionSource->GetExecutionState().GetSeed());
		}

		return FRandomStream(Seed);
	}

	FRandomStream GetRandomStreamFromTwoSeeds(int32 SeedA, int32 SeedB, const UPCGSettings* OptionalSettings, const IPCGGraphExecutionSource* OptionalExecutionSource)
	{
		int Seed = PCGHelpers::ComputeSeed(SeedA, SeedB);

		if (OptionalSettings && OptionalExecutionSource)
		{
			Seed = PCGHelpers::ComputeSeed(Seed, OptionalSettings->Seed, OptionalExecutionSource->GetExecutionState().GetSeed());
		}
		else if (OptionalSettings)
		{
			Seed = PCGHelpers::ComputeSeed(Seed, OptionalSettings->Seed);
		}
		else if (OptionalExecutionSource)
		{
			Seed = PCGHelpers::ComputeSeed(Seed, OptionalExecutionSource->GetExecutionState().GetSeed());
		}

		return FRandomStream(Seed);
	}

	bool IsInsideBounds(const FBox& InBox, const FVector& InPosition)
	{
		return (InPosition.X >= InBox.Min.X) && (InPosition.X < InBox.Max.X) &&
			(InPosition.Y >= InBox.Min.Y) && (InPosition.Y < InBox.Max.Y) &&
			(InPosition.Z >= InBox.Min.Z) && (InPosition.Z < InBox.Max.Z);
	}

	bool IsInsideBoundsXY(const FBox& InBox, const FVector& InPosition)
	{
		return (InPosition.X >= InBox.Min.X) && (InPosition.X < InBox.Max.X) &&
			(InPosition.Y >= InBox.Min.Y) && (InPosition.Y < InBox.Max.Y);
	}

	FBox OverlapBounds(const FBox& InA, const FBox& InB)
	{
		if (!InA.IsValid || !InB.IsValid)
		{
			return FBox(EForceInit::ForceInit);
		}
		else
		{
			return InA.Overlap(InB);
		}
	}

	FBox GetGridBounds(const AActor* Actor, const IPCGGraphExecutionSource* Source)
	{
		FBox Bounds(EForceInit::ForceInit);

		if (const APCGPartitionActor* PartitionActor = Cast<const APCGPartitionActor>(Actor))
		{
			// First, get the bounds from the partition actor
			Bounds = PartitionActor->GetFixedBounds();

			const IPCGGraphExecutionSource* OriginalSource = Source ? Source->GetExecutionState().GetOriginalSource() : nullptr;
			if (OriginalSource)
			{
				if (OriginalSource->GetExecutionState().GetTarget() != PartitionActor)
				{
					Bounds = Bounds.Overlap(OriginalSource->GetExecutionState().GetBounds());
				}
			}
		}
		// TODO: verify this works as expected in non-editor builds
		else if (const ALandscapeProxy* LandscapeActor = Cast<const ALandscapeProxy>(Actor))
		{
			Bounds = GetLandscapeBounds(LandscapeActor);
		}
		else if (Actor)
		{
			Bounds = GetActorBounds(Actor);
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Actor is invalid in GetGridBounds");
		}

		return Bounds;
	}

	FBox GetActorBounds(const AActor* InActor, bool bIgnorePCGCreatedComponents)
	{
		// Specialized version of GetComponentsBoundingBox that skips over PCG generated components
		// This is to ensure stable bounds and no timing issues (cleared ISMs, etc.)
		FBox Box(EForceInit::ForceInit);

		if (InActor)
		{
			if (const APCGPartitionActor* PartitionActor = Cast<const APCGPartitionActor>(InActor))
			{
				// Skip per-component check, return fixed bounds.
				Box = PartitionActor->GetFixedBounds();
			}
			else
			{
				const bool bNonColliding = true;

				InActor->ForEachComponent<UPrimitiveComponent>(/*bIncludeFromChildActors=*/true, [bNonColliding, bIgnorePCGCreatedComponents, &Box](const UPrimitiveComponent* InPrimComp)
				{
					// Note: we omit the IsRegistered check here (e.g. InPrimComp->IsRegistered() )
					// since this can be called in a scope where the components are temporarily unregistered
					if ((bNonColliding || InPrimComp->IsCollisionEnabled()) &&
						(!bIgnorePCGCreatedComponents || !InPrimComp->ComponentTags.Contains(DefaultPCGTag)))
					{
						Box += InPrimComp->Bounds.GetBox();
					}
				});
			}
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Actor is invalid in GetActorBounds");
		}

		return Box;
	}

	FBox GetActorLocalBounds(const AActor* InActor, bool bIgnorePCGCreatedComponents)
	{
		// Specialized version of CalculateComponentsBoundingBoxInLocalScape that skips over PCG generated components
		// This is to ensure stable bounds and no timing issues (cleared ISMs, etc.)
		FBox Box(EForceInit::ForceInit);

		if (InActor)
		{
			if (const APCGPartitionActor* PartitionActor = Cast<const APCGPartitionActor>(InActor))
			{
				// Skip per-component check, return fixed bounds only replaced on origin
				Box = PartitionActor->GetFixedBounds();
				Box = Box.MoveTo(FVector::ZeroVector);
			}
			else
			{
				const bool bNonColliding = true;

				// The following code does a bounds computation in local actor space so that Box can capture the tight bounds. 
				FTransform ActorToWorld = InActor->GetTransform();

				// The matrix inverse below seems to work well for positive scales, but seems to break down badly for non uniform
				// scales (to see, compare ActorToWorld*WorldToActor to identity) - UE-221283. The following workaround removes mirroring
				// from the actor transform, does the bounds computation, and then re-mirrors afterwards. This works well for close-to-90deg
				// actor transform rotations and relatively-uniform scales, but can result in artificial dilation of the bounds in some cases.
				const FVector ScaleSign = FVector(
					FMath::Sign(InActor->GetTransform().GetScale3D().X),
					FMath::Sign(InActor->GetTransform().GetScale3D().Y),
					FMath::Sign(InActor->GetTransform().GetScale3D().Z));

				ActorToWorld.SetScale3D(InActor->GetTransform().GetScale3D().GetAbs());

				const FTransform WorldToActor = ActorToWorld.Inverse();

				InActor->ForEachComponent<UPrimitiveComponent>(/*bIncludeFromChildActors=*/true, [bNonColliding, bIgnorePCGCreatedComponents, &WorldToActor, &Box](const UPrimitiveComponent* InPrimComp)
				{
					// Billboard requires access to its texture prevent this from running outside of game thread
					if(!InPrimComp->IsA<UBillboardComponent>())
					{
						if ((bNonColliding || InPrimComp->IsCollisionEnabled()) &&
							(!bIgnorePCGCreatedComponents || !InPrimComp->ComponentTags.Contains(DefaultPCGTag)))
						{
							const FTransform ComponentToActor = InPrimComp->GetComponentTransform() * WorldToActor;
							Box += InPrimComp->CalcBounds(ComponentToActor).GetBox();
						}
					}
				});

				// Un-mirror - see notes above.
				for (int Axis = 0; Axis < 3; ++Axis)
				{
					if (ScaleSign[Axis] < 0.0)
					{
						Box.Min[Axis] *= -1.0;
						Box.Max[Axis] *= -1.0;
						Swap(Box.Min[Axis], Box.Max[Axis]);
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Actor is invalid in GetActorLocalBounds");
		}

		return Box;
	}

	bool IsRuntimeOrPIE()
	{
#if WITH_EDITOR
		return (GEditor && GEditor->PlayWorld) || GIsPlayInEditorWorld || IsRunningGame();
#else
		return true;
#endif // WITH_EDITOR
	}

	FBox GetLandscapeBounds(const ALandscapeProxy* InLandscape)
	{
		check(InLandscape);

		if (const ALandscape* Landscape = Cast<const ALandscape>(InLandscape))
		{
			// If the landscape isn't done being loaded, we're very unlikely to want to interact with it
			if (Landscape->GetLandscapeInfo() == nullptr)
			{
				return FBox(EForceInit::ForceInit);
			}

#if WITH_EDITOR
			if (!IsRuntimeOrPIE())
			{
				return Landscape->GetCompleteBounds();
			}
			else
#endif
			{
				return Landscape->GetLoadedBounds();
			}
		}
		else
		{
			return InLandscape->GetProxyBounds();
		}
	}

	TArray<TWeakObjectPtr<ALandscapeProxy>> GetAllLandscapeProxies(UWorld* InWorld)
	{
		TArray<TWeakObjectPtr<ALandscapeProxy>> LandscapeProxies;

		for (TObjectIterator<ALandscapeProxy> It; It; ++It)
		{
			if (It->GetWorld() == InWorld)
			{
				LandscapeProxies.Add(*It);
			}
		}

		return LandscapeProxies;
	}

	ALandscape* GetLandscape(UWorld* InWorld, const FBox& InBounds)
	{
		ALandscape* Landscape = nullptr;

		if (!InBounds.IsValid)
		{
			return Landscape;
		}

		for (TObjectIterator<ALandscape> It; It; ++It)
		{
			if (IsValid(*It) && It->GetWorld() == InWorld)
			{
				const FBox LandscapeBounds = GetLandscapeBounds(*It);
				if (LandscapeBounds.IsValid && LandscapeBounds.Intersect(InBounds))
				{
					Landscape = (*It);
					break;
				}
			}
		}

		return Landscape;
	}

	TArray<TWeakObjectPtr<ALandscapeProxy>> GetLandscapeProxies(UWorld* InWorld, const FBox& InBounds)
	{
		TArray<TWeakObjectPtr<ALandscapeProxy>> LandscapeProxies;

		if (!InBounds.IsValid)
		{
			return LandscapeProxies;
		}

		for (TObjectIterator<ALandscapeProxy> It; It; ++It)
		{
			if (IsValid(*It) && It->GetWorld() == InWorld)
			{
				const FBox LandscapeBounds = GetLandscapeBounds(*It);
				if (LandscapeBounds.IsValid && LandscapeBounds.Intersect(InBounds))
				{
					LandscapeProxies.Add(*It);
				}
			}
		}

		return LandscapeProxies;
	}

	APCGWorldActor* GetPCGWorldActor(UWorld* InWorld)
	{
		return (InWorld && InWorld->GetSubsystem<UPCGSubsystem>()) ? InWorld->GetSubsystem<UPCGSubsystem>()->GetPCGWorldActor() : nullptr;
	}

	APCGWorldActor* FindPCGWorldActor(UWorld* InWorld)
	{
		return (InWorld && InWorld->GetSubsystem<UPCGSubsystem>()) ? InWorld->GetSubsystem<UPCGSubsystem>()->FindPCGWorldActor() : nullptr;
	}

	// TODO: Temporary validation during transition of allowing spaces in tags/attributes. Deprecate in 5.6.
	TArray<FString> GetStringArrayFromCommaSeparatedString(const FString& InCommaSeparatedString, const FPCGContext* InOptionalContext)
	{
#if WITH_EDITOR
		if (InCommaSeparatedString.Contains(" "))
		{
			PCGLog::LogWarningOnGraph(
				FText::Format(LOCTEXT(
					"AttributeOrTagContainsSpace", "The comma separated list '{0}' contains an internal space character, which should no longer be parsed as a separator. \n"
					"Disable 'bParseOnWhiteSpace' on the node to deprecate and update the behavior."),
					FText::FromString(InCommaSeparatedString)),
				InOptionalContext);
		}
#endif // WITH_EDITOR

		TArray<FString> Result;
		InCommaSeparatedString.ParseIntoArrayWS(Result, TEXT(","));

		return Result;
	}

	TArray<FString> GetStringArrayFromCommaSeparatedList(const FString& InCommaSeparatedString)
	{
		TArray<FString> Result;
		InCommaSeparatedString.ParseIntoArray(Result, TEXT(","));
		// Trim leading and trailing spaces
		for (FString& String : Result)
		{
			String.TrimStartAndEndInline();
		}

		return Result;
	}

#if WITH_EDITOR
	bool CanBeExpanded(UClass* ObjectClass)
	{
		// There shouldn't be any need to dig through Niagara assets + there are some issues (most likely related to loading) with parsing all their dependencies
		if (!ObjectClass ||
			ObjectClass->GetFName() == TEXT("NiagaraSystem") ||
			ObjectClass->GetFName() == TEXT("NiagaraComponent"))
		{
			return false;
		}

		return true;
	}

	void GatherDependencies(UObject* Object, TSet<TObjectPtr<UObject>>& OutDependencies, int32 MaxDepth, const TArray<UClass*>& InExcludedClasses)
	{
		UClass* ObjectClass = Object ? Object->GetClass() : nullptr;

		if (!CanBeExpanded(ObjectClass))
		{
			return;
		}
		checkSlow(ObjectClass); // CanBeExpanded checks this but static analyzer doesn't note that

		for (FProperty* Property = ObjectClass->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext)
		{
			GatherDependencies(Property, Object, OutDependencies, MaxDepth, InExcludedClasses);
		}
	}

	// Inspired by IteratePropertiesRecursive in ObjectPropertyTrace.cpp
	void GatherDependencies(FProperty* Property, const void* InContainer, TSet<TObjectPtr<UObject>>& OutDependencies, int32 MaxDepth, const TArray<UClass*>& InExcludedClasses)
	{
		// Skip any kind of internal property and the ones that are susceptible to be unstable
		if (!Property || Property->HasAnyPropertyFlags(CPF_Transient | CPF_DuplicateTransient | CPF_Deprecated))
		{
			return;
		}

		auto AddToDependenciesAndGatherRecursively = [&OutDependencies, MaxDepth, &InExcludedClasses](UObject* Object) {
			if (Object && !OutDependencies.Contains(Object))
			{
				// If we explicitly don't want to track this object, early out.
				if (!Object->GetClass() ||
					!CanBeExpanded(Object->GetClass()) ||
					Algo::AnyOf(InExcludedClasses, [InClass = Object->GetClass()](const UClass* ExcludedClass) { return InClass->IsChildOf(ExcludedClass); }))
				{
					return;
				}

				OutDependencies.Add(Object);
				if (MaxDepth != 0)
				{
					GatherDependencies(Object, OutDependencies, MaxDepth - 1, InExcludedClasses);
				}
			}
		};

		if (!InContainer || !Property)
		{
			return;
		}

		if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			UObject* Object = ObjectProperty->GetPropertyValue_InContainer(InContainer);
			AddToDependenciesAndGatherRecursively(Object);
		}
		else if (FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property))
		{
			FWeakObjectPtr WeakObject = WeakObjectProperty->GetPropertyValue_InContainer(InContainer);
			AddToDependenciesAndGatherRecursively(WeakObject.Get());
		}
		else if (FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
		{
			FSoftObjectPtr SoftObject = SoftObjectProperty->GetPropertyValue_InContainer(InContainer);
			AddToDependenciesAndGatherRecursively(SoftObject.Get());
		}
		else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			const void* StructContainer = StructProperty->ContainerPtrToValuePtr<const void>(InContainer);
			for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
			{
				GatherDependencies(*It, StructContainer, OutDependencies, MaxDepth, InExcludedClasses);
			}
		}
		else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			FScriptArrayHelper_InContainer Helper(ArrayProperty, InContainer);
			for (int32 DynamicIndex = 0; DynamicIndex < Helper.Num(); ++DynamicIndex)
			{
				const void* ValuePtr = Helper.GetRawPtr(DynamicIndex);
				GatherDependencies(ArrayProperty->Inner, ValuePtr, OutDependencies, MaxDepth, InExcludedClasses);
			}
		}
		else if (FMapProperty* MapProperty = CastField<FMapProperty>(Property))
		{
			FScriptMapHelper_InContainer Helper(MapProperty, InContainer);
			for (FScriptMapHelper::FIterator It(Helper); It; ++It)
			{
				// Key and Value are stored next to each other in memory.
				// ValueProp has an offset, so we should use the same starting address for both.
				const void* PairKeyValuePtr = Helper.GetKeyPtr(It);
				GatherDependencies(MapProperty->KeyProp, PairKeyValuePtr, OutDependencies, MaxDepth, InExcludedClasses);
				GatherDependencies(MapProperty->ValueProp, PairKeyValuePtr, OutDependencies, MaxDepth, InExcludedClasses);
			}
		}
		else if (FSetProperty* SetProperty = CastField<FSetProperty>(Property))
		{
			FScriptSetHelper_InContainer Helper(SetProperty, InContainer);
			for (FScriptSetHelper::FIterator It(Helper); It; ++It)
			{
				const void* ValuePtr = Helper.GetElementPtr(It);
				GatherDependencies(SetProperty->ElementProp, ValuePtr, OutDependencies, MaxDepth, InExcludedClasses);
			}
		}
	}

	bool IsInWorldPartitionBuilderCommandletContext()
	{
		static const UClass* WorldPartitionBuilderCommandletClass = FindObject<UClass>(nullptr, TEXT("/Script/UnrealEd.WorldPartitionBuilderCommandlet"), EFindObjectFlags::ExactClass);
		check(WorldPartitionBuilderCommandletClass);
		const UClass* RunningCommandletClass = GetRunningCommandletClass();
		return RunningCommandletClass && RunningCommandletClass->IsChildOf(WorldPartitionBuilderCommandletClass);
	}
#endif

	bool IsNewObjectAndNotDefault(const UObject* InObject, bool bCheckHierarchy)
	{
		const UObject* CurrentInspectedObject = InObject;
		while (bCheckHierarchy && CurrentInspectedObject && CurrentInspectedObject->HasAnyFlags(RF_DefaultSubObject))
		{
			CurrentInspectedObject = CurrentInspectedObject->GetOuter();
		}

		// We detect new objects if they are not a default object/archetype and/or they do not need load.
		// In some cases, where the component is a default sub object (like APCGVolume), it has no loading flags
		// even if it is loading, so we use the outer found above.
		return CurrentInspectedObject && !CurrentInspectedObject->HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad | RF_NeedPostLoad);
	}

	bool GetGenerationGridSizes(const UPCGGraph* InGraph, const APCGWorldActor* InWorldActor, PCGHiGenGrid::FSizeArray& OutGridSizes, bool& bOutHasUnbounded)
	{
		bOutHasUnbounded = false;

		if (InGraph && InGraph->IsHierarchicalGenerationEnabled())
		{
			InGraph->GetGridSizes(OutGridSizes, bOutHasUnbounded);
			return true;
		}
		else if (InWorldActor)
		{
			OutGridSizes.Add(InWorldActor->PartitionGridSize);
			return true;
		}
		else
		{
			OutGridSizes.Add(PCGHiGenGrid::UnboundedGridSize());
			return true;
		}
	}

	int32 GetGenerationGridSize(const IPCGGraphExecutionSource* InExecutionSource)
	{
		return InExecutionSource ? InExecutionSource->GetExecutionState().GetGenerationGridSize() : PCGHiGenGrid::UninitializedGridSize();
	}

	bool IsRuntimeGeneration(const IPCGGraphExecutionSource* InExecutionSource)
	{
		return InExecutionSource && InExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem();
	}

#if WITH_EDITOR
	void GetGeneratedActorsFolderPath(const AActor* InTargetActor, FString& OutFolderPath)
	{
		if (!InTargetActor)
		{
			return;
		}

		// Reserves reasonable max string length on stack, overflows to heap if exceeded.
		TStringBuilderWithBuffer<TCHAR, 1024> GeneratedActorsFolder;

		FName TargetActorFolder = InTargetActor->GetFolderPath();
		if (TargetActorFolder != NAME_None)
		{
			GeneratedActorsFolder << TargetActorFolder.ToString() << "/";
		}

		GeneratedActorsFolder << InTargetActor->GetActorLabel() << "_Generated";
		OutFolderPath = GeneratedActorsFolder;
	}

	void GetGeneratedActorsFolderPath(const AActor* InTargetActor, const FPCGContext* InContext, EPCGAttachOptions AttachOptions, FString& OutFolderPath)
	{
		if (AttachOptions == EPCGAttachOptions::Attached || AttachOptions == EPCGAttachOptions::NotAttached)
		{
			OutFolderPath = FString();
		}
		else if (AttachOptions == EPCGAttachOptions::InFolder)
		{
			GetGeneratedActorsFolderPath(InTargetActor, OutFolderPath);
		}
		else if (AttachOptions == EPCGAttachOptions::InGraphFolder && InContext && InContext->GetStack() && InContext->GetStack()->GetRootGraph())
		{
			OutFolderPath = InContext->GetStack()->GetRootGraph()->GetName() + "_Generated";
		}
		else // Generated folder
		{
			OutFolderPath = TEXT("PCG_Generated_Actors");
		}
	}
#endif

	// Note: deprecated
	void AttachToParent(AActor* InActorToAttach, AActor* InParent, EPCGAttachOptions AttachOptions, const FString& InGeneratedPath)
	{
		AttachToParent(InActorToAttach, InParent, AttachOptions, nullptr, InGeneratedPath);
	}

	void AttachToParent(AActor* InActorToAttach, AActor* InParent, EPCGAttachOptions AttachOptions, const FPCGContext* InContext, const FString& InGeneratedPath)
	{
		if (!InParent)
		{
			return;
		}

		if (AttachOptions == EPCGAttachOptions::Attached)
		{
			InActorToAttach->AttachToActor(InParent, FAttachmentTransformRules::KeepWorldTransform);
		}
#if WITH_EDITOR
		else if(AttachOptions != EPCGAttachOptions::NotAttached)
		{
			FString DefaultFolderPath;
			if (InGeneratedPath.IsEmpty())
			{
				GetGeneratedActorsFolderPath(InParent, InContext, AttachOptions, DefaultFolderPath);
			}

			const FString& FolderPath = (InGeneratedPath.IsEmpty() ? DefaultFolderPath : InGeneratedPath);
			InActorToAttach->SetFolderPath(*FolderPath);
		}
#endif
	}

	TArray<UFunction*> FindUserFunctions(TSubclassOf<UObject> ObjectClass, const TArray<FName>& FunctionNames, const TArray<const UFunction*>& FunctionPrototypes, const FPCGContext* InContext)
	{
		TArray<UFunction*> Functions;

		if (!ObjectClass)
		{
			return Functions;
		}

		for (FName FunctionName : FunctionNames)
		{
			if (FunctionName == NAME_None)
			{
				continue;
			}

			if (UFunction* Function = ObjectClass->FindFunctionByName(FunctionName))
			{
#if WITH_EDITOR
				// Implementation note: for AActors, using ProcessEvent requires the function to either be 'CallInEditor' or GAllowActorScriptExecutionInEditor to be true.
				// It might not be strictly needed in cases where the object is not an actor.
				if (ObjectClass->GetDefaultObject()->IsA<AActor>() && !Function->GetBoolMetaData(TEXT("CallInEditor")))
				{
					PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("CallInEditorFailed", "Function '{0}' in class '{1}' requires CallInEditor to be true while in-editor."), FText::FromName(FunctionName), FText::FromName(ObjectClass->GetFName())), InContext);
					continue;
				}
#endif
				for (const UFunction* Prototype : FunctionPrototypes)
				{
					if (Function->IsSignatureCompatibleWith(Prototype))
					{
						Functions.Add(Function);
						break;
					}
				}

				if (Functions.IsEmpty() || Functions.Last() != Function)
				{
					PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("ParametersIncorrect", "Function '{0}' in class '{1}' has incorrect parameters."), FText::FromName(FunctionName), FText::FromName(ObjectClass->GetFName())), InContext);
				}
			}
			else
			{
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("FunctionNotFound", "Function '{0}' was not found in class '{1}'."), FText::FromName(FunctionName), FText::FromName(ObjectClass->GetFName())), InContext);
			}
		}

		return Functions;
	}

	TFunction<float(float, float)> GetDensityMergeFunction(EPCGDensityMergeOperation InOperation)
	{
		switch (InOperation)
		{
		case EPCGDensityMergeOperation::Set: return [](float A, float B) { return B; };
		case EPCGDensityMergeOperation::Ignore: return [](float A, float B) { return A; };
		case EPCGDensityMergeOperation::Minimum: return [](float A, float B) { return FMath::Min(A, B); };
		case EPCGDensityMergeOperation::Maximum: return [](float A, float B) { return FMath::Max(A, B); };
		case EPCGDensityMergeOperation::Add: return [](float A, float B) { return A + B; };
		case EPCGDensityMergeOperation::Subtract: return [](float A, float B) { return A - B; };
		case EPCGDensityMergeOperation::Multiply: return [](float A, float B) { return A * B; };
		case EPCGDensityMergeOperation::Divide: return [](float A, float B) { return B != 0.0f ? (A / B) : 0.0f; };
		default: checkNoEntry(); return [](float, float) { return 0.0f; };
		}
	}

	TArray<int32> GetRandomIndices(FRandomStream& RandomStream, const int32 ArraySize, const int32 NumSelections)
	{
		if (ArraySize < 1 || NumSelections < 1)
		{
			return {};
		}

		const int32 N = FMath::Min(NumSelections, ArraySize);

		TArray<int32> RandomIndices;
		RandomIndices.Reserve(N);

		const int32 Max = ArraySize - NumSelections;
		for (int i = 0; i < N; ++i)
		{
			RandomIndices.Emplace(RandomStream.RandRange(0, Max));
		}

		RandomIndices.Sort();

		for (int i = 0; i < RandomIndices.Num(); ++i)
		{
			RandomIndices[i] += i;
		}

		return RandomIndices;
	}

	bool IsServer(const UWorld* InWorld)
	{
		return IsDedicatedServer(InWorld) || IsListenServer(InWorld);
	}
	
	bool IsDedicatedServer(const UWorld* InWorld)
	{
		if (InWorld)
		{
			const ENetMode NetMode = InWorld->GetNetMode();
			return NetMode == NM_DedicatedServer;
		}
		else
		{
			return false;
		}
	}

	bool IsListenServer(const UWorld* InWorld)
	{
		if (InWorld)
		{
			const ENetMode NetMode = InWorld->GetNetMode();
			return NetMode == NM_ListenServer;
		}
		else
		{
			return false;
		}
	}

	int32 GetNumberOfElements(const UPCGData* InData, const FPCGMetadataDomainID& InDomainID)
	{
		if (!InData->IsSupportedMetadataDomainID(InDomainID))
		{
			return 0;
		}

		FPCGAttributePropertyInputSelector DomainSelector;
		DomainSelector.SetAttributeName(NAME_None);
		InData->SetDomainFromDomainID(InDomainID, DomainSelector);
		TUniquePtr<const IPCGAttributeAccessorKeys> DomainKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InData, DomainSelector);
		return DomainKeys ? DomainKeys->GetNum() : 0;
	}

#if UE_ENABLE_DEBUG_DRAWING

	void DebugDrawGenerationVolume(FPCGContext* InContext, const FColor* InOverrideColor)
	{
		if (!InContext || !InContext->ExecutionSource.IsValid())
		{
			return;
		}

		const UPCGComponent* Component = Cast<UPCGComponent>(InContext->ExecutionSource.Get());
		const FColor Color = InOverrideColor ? *InOverrideColor : FColor::MakeRandomSeededColor(Component ? (2 + Component->GetGenerationGridSize()) : 0);

		PCGHelpers::ExecuteOnGameThread(TEXT("Execution debug"), [ExecutionSource = InContext->ExecutionSource, Color]()
		{
			LLM_SCOPE_BYTAG(PCG);

			UWorld* World = ExecutionSource.IsValid() ? ExecutionSource->GetExecutionState().GetWorld() : nullptr;
			if (!World)
			{
				return;
			}

			const FBox Bounds = ExecutionSource->GetExecutionState().GetBounds();
			const UPCGComponent* ComponentInner = Cast<UPCGComponent>(ExecutionSource.Get());

			// Unfortunately DrawDebugString does not work in editor.
			if (ComponentInner && World->IsGameWorld())
			{
				FString Text;
				if (ComponentInner->GetGenerationGridSize() == PCGHiGenGrid::UnboundedGridSize())
				{
					if (ComponentInner->GetOwner())
					{
						Text = ComponentInner->GetOwner()->GetName();
					}
				}
				else
				{
					const FIntVector CellCoord = UPCGActorHelpers::GetCellCoord(Bounds.GetCenter(), ComponentInner->GetGenerationGridSize(), ComponentInner->Use2DGrid());
					Text = FString::Format(TEXT("{0} ({1}, {2}, {3})"), { ComponentInner->GetGenerationGridSize(), CellCoord.X, CellCoord.Y, CellCoord.Z });
				}

				DrawDebugString(World, Bounds.GetCenter() + FVector(0.0, 0.0, 100.0), Text, /*TestBaseActor=*/nullptr, Color, /*Duration=*/0.0f);
			}

			DrawDebugBox(World, Bounds.GetCenter(), Bounds.GetExtent(), Color, /*bPersistentLines=*/false, /*LifeTime=*/0.0f);

			// Add additional boxes to visually distinguish this vis and make the wireframe look "thicker".
			DrawDebugBox(World, Bounds.GetCenter(), Bounds.GetExtent() * FVector(0.95, 1.0, 1.0), Color, /*bPersistentLines=*/false, /*LifeTime=*/0.0f);
			DrawDebugBox(World, Bounds.GetCenter(), Bounds.GetExtent() * FVector(1.0, 0.95, 1.0), Color, /*bPersistentLines=*/false, /*LifeTime=*/0.0f);
		});
	}
#endif // UE_ENABLE_DEBUG_DRAWING
}

#undef LOCTEXT_NAMESPACE
