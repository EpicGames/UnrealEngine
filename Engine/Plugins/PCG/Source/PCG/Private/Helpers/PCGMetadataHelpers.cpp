// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGMetadataHelpers.h"

#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"	
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/PCGMetadata.h"

#define LOCTEXT_NAMESPACE "PCGMetadataHelpers"

namespace PCGMetadataHelpers
{
	bool HasSameRoot(const UPCGMetadata* Metadata1, const UPCGMetadata* Metadata2)
	{
		return Metadata1 && Metadata2 && Metadata1->GetRoot() == Metadata2->GetRoot();
	}
	
	bool HasSameRoot(const FPCGMetadataDomain* Metadata1, const FPCGMetadataDomain* Metadata2)
	{
		return Metadata1 && Metadata2 && Metadata1->GetRoot() == Metadata2->GetRoot();
	}
	
	FPCGMetadataDomain* GetDefaultMetadataDomain(UPCGMetadata* InMetadata)
	{
		return InMetadata ? InMetadata->GetDefaultMetadataDomain() : nullptr;
	}

	const UPCGMetadata* GetParentMetadata(const UPCGMetadata* Metadata)
	{
		check(Metadata);
		TWeakObjectPtr<const UPCGMetadata> Parent = Metadata->GetParentPtr();

		// We're expecting the parent to either be null, or to be valid - if not, then it has been deleted
		// which is going to cause some issues.
		//check(Parent.IsExplicitlyNull() || Parent.IsValid());
		return Parent.Get();
	}

	const FPCGMetadataDomain* GetParentMetadata(const FPCGMetadataDomain* Metadata)
	{
		check(Metadata);
		return Metadata->GetParent();
	}

	void InitializeMetadataWithDataDomainCopyAndElementsNoCopy(const UPCGData* InData, UPCGData* OutData)
	{
		check(InData && OutData && InData->ConstMetadata() && OutData->MutableMetadata())

		// Initialize the data domain by copy
		const FPCGMetadataDomain* InDataDomain = InData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data);
		FPCGMetadataDomain* OutDataDomain = OutData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data);

		if (InDataDomain && OutDataDomain)
		{
			OutDataDomain->InitializeAsCopy(InDataDomain);
		}

		// And just add the attributes for the element domain since they will be copied manually.
		const FPCGMetadataDomain* InElementDomain = InData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements);
		FPCGMetadataDomain* OutElementDomain = OutData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements);
		OutElementDomain->AddAttributes(InElementDomain);
	}

	const UPCGMetadata* GetConstMetadata(const UPCGData* InData)
	{
		return InData ? InData->ConstMetadata() : nullptr;
	}

	UPCGMetadata* GetMutableMetadata(UPCGData* InData)
	{
		return InData ? InData->MutableMetadata() : nullptr;
	}

	bool CreateObjectPathGetter(const FPCGMetadataAttributeBase* InAttributeBase, TFunction<void(int64, FSoftObjectPath&)>& OutGetter)
	{
		if (!InAttributeBase)
		{
			return false;
		}

		if (InAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FString>::Id)
		{
			OutGetter = [InAttributeBase](int64 InMetadataKey, FSoftObjectPath& OutSoftObjectPath)
			{
				FString Path = static_cast<const FPCGMetadataAttribute<FString>*>(InAttributeBase)->GetValueFromItemKey(InMetadataKey);
				OutSoftObjectPath = FSoftObjectPath(Path);
			};

			return true;
		}
		else if (InAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FSoftObjectPath>::Id)
		{
			OutGetter = [InAttributeBase](int64 InMetadataKey, FSoftObjectPath& OutSoftObjectPath)
			{
				OutSoftObjectPath = static_cast<const FPCGMetadataAttribute<FSoftObjectPath>*>(InAttributeBase)->GetValueFromItemKey(InMetadataKey);
			};

			return true;
		}

		return false;
	}

	bool CreateObjectOrClassPathGetter(const FPCGMetadataAttributeBase* InAttributeBase, TFunction<void(int64, FSoftObjectPath&)>& OutGetter)
	{
		if (!InAttributeBase)
		{
			return false;
		}

		if (InAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FString>::Id)
		{
			OutGetter = [InAttributeBase](int64 InMetadataKey, FSoftObjectPath& OutSoftObjectPath)
			{
				FString Path = static_cast<const FPCGMetadataAttribute<FString>*>(InAttributeBase)->GetValueFromItemKey(InMetadataKey);
				OutSoftObjectPath = FSoftObjectPath(Path);
			};

			return true;
		}
		else if (InAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FSoftObjectPath>::Id)
		{
			OutGetter = [InAttributeBase](int64 InMetadataKey, FSoftObjectPath& OutSoftObjectPath)
			{
				OutSoftObjectPath = static_cast<const FPCGMetadataAttribute<FSoftObjectPath>*>(InAttributeBase)->GetValueFromItemKey(InMetadataKey);
			};

			return true;
		}
		else if (InAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FSoftClassPath>::Id)
		{
			OutGetter = [InAttributeBase](int64 InMetadataKey, FSoftObjectPath& OutSoftObjectPath)
			{
				OutSoftObjectPath = static_cast<const FPCGMetadataAttribute<FSoftClassPath>*>(InAttributeBase)->GetValueFromItemKey(InMetadataKey);
			};

			return true;
		}

		return false;
	}


	// @todo_pcg: Metadata -> This will copy only between default metadata. We need to handle cross domain, with restrictions?
	bool CopyAttributes(UPCGData* TargetData, const UPCGData* SourceData, const TArray<TTuple<FPCGAttributePropertyInputSelector, FPCGAttributePropertyOutputSelector, EPCGMetadataTypes>>& AttributeSelectorsWithOutputType, bool bSameOrigin, FPCGContext* OptionalContext)
	{
		check(TargetData && SourceData);
		const UPCGMetadata* SourceMetadata = SourceData->ConstMetadata();
		UPCGMetadata* TargetMetadata = TargetData->MutableMetadata();

		if (!SourceMetadata || !TargetMetadata)
		{
			return false;
		}

		bool bSuccess = false;

		for (const auto& [InputSource, OutputTarget, RequestedOutputType] : AttributeSelectorsWithOutputType)
		{
			const FPCGAttributeIdentifier LocalSourceAttribute = FPCGAttributeIdentifier(InputSource.GetName(), SourceData->GetMetadataDomainIDFromSelector(InputSource));
			const FPCGAttributeIdentifier LocalDestinationAttribute = FPCGAttributeIdentifier(OutputTarget.GetName(), TargetData->GetMetadataDomainIDFromSelector(OutputTarget));

			if (InputSource.GetSelection() == EPCGAttributePropertySelection::Attribute && !SourceMetadata->HasAttribute(LocalSourceAttribute))
			{
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("InputMissingAttribute", "Input does not have the '{0}' attribute"), InputSource.GetDisplayText()), OptionalContext);
				continue;
			}

			// We need accessors if we have a multi entry source attribute or we have extractors
			const bool bInputHasAnyExtra = !InputSource.GetExtraNames().IsEmpty();
			const bool bOutputHasAnyExtra = !OutputTarget.GetExtraNames().IsEmpty();
			const bool bSourceIsAttribute = InputSource.GetSelection() == EPCGAttributePropertySelection::Attribute;
			const bool bTargetIsAttribute = OutputTarget.GetSelection() == EPCGAttributePropertySelection::Attribute;
			// Cast is only required if it is on an output attribute that has no extra (that we will create)
			const bool bOutputTypeCast = bTargetIsAttribute && !bOutputHasAnyExtra && (RequestedOutputType != EPCGMetadataTypes::Unknown);

			const bool bNeedAccessors = bInputHasAnyExtra || bOutputHasAnyExtra || !bSourceIsAttribute || !bTargetIsAttribute || bOutputTypeCast;

			// If no accessor, copy over the attribute
			if (!bNeedAccessors)
			{
				if (bSameOrigin && LocalSourceAttribute == LocalDestinationAttribute)
				{
					// Nothing to do if we try to copy an attribute into itself in the original data.
					continue;
				}

				const FPCGMetadataAttributeBase* SourceAttribute = SourceMetadata->GetConstAttribute(LocalSourceAttribute);
				// Presence of attribute was already checked before, this should not return null
				check(SourceAttribute);

				FPCGMetadataDomain* TargetMetadataDomain = TargetMetadata->GetMetadataDomain(LocalDestinationAttribute.MetadataDomain);
				if (!TargetMetadataDomain)
				{
					PCGLog::Metadata::LogInvalidMetadataDomain(OutputTarget, OptionalContext);
					continue;
				}

				// Copy entries only if they come from the same data and they are on the same domain.
				// We'll need to keep the parent too if we copy entries, but only if we support parenting (otherwise it makes no sense as there is no parent)
				const bool bCopyEntries = bSameOrigin && SourceAttribute->GetMetadataDomain()->GetDomainID() == TargetMetadataDomain->GetDomainID();
				const bool bKeepParent = bCopyEntries && TargetMetadata->MetadataDomainSupportsParenting(LocalDestinationAttribute.MetadataDomain);
				const bool bSourceSupportsMultiEntries = SourceMetadata->MetadataDomainSupportsMultiEntries(LocalSourceAttribute.MetadataDomain);
				const bool bTargetSupportsMultiEntries = TargetMetadata->MetadataDomainSupportsMultiEntries(LocalDestinationAttribute.MetadataDomain);

				if (FPCGMetadataAttributeBase* NewAttr = TargetMetadataDomain->CopyAttribute(SourceAttribute, LocalDestinationAttribute.Name, /*bKeepParent=*/bKeepParent, /*bCopyEntries=*/bCopyEntries, /*bCopyValues=*/true))
				{
					// And finally, we create our source and target keys, to get all the metadata entry keys, to remap if we are not the same origin and that we have multiple entries.
					if (!bCopyEntries)
					{
						TUniquePtr<const IPCGAttributeAccessorKeys> SourceKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SourceData, InputSource);
						TUniquePtr<IPCGAttributeAccessorKeys> TargetKeys = PCGAttributeAccessorHelpers::CreateKeys(TargetData, OutputTarget);

						// They must exist, so we can check.
						check(SourceKeys && TargetKeys);
						
						if (SourceKeys->GetNum() > 0 && TargetKeys->GetNum() > 0)
						{
							TArray<const PCGMetadataEntryKey*> AllSourceEntryKeysPtr;
							TArray<PCGMetadataEntryKey*> AllTargetEntryKeysPtr;

							int32 NumTargetKeys = bSourceSupportsMultiEntries ? TargetKeys->GetNum() : 1;

							AllSourceEntryKeysPtr.SetNumUninitialized(FMath::Min(SourceKeys->GetNum(), NumTargetKeys));
							AllTargetEntryKeysPtr.SetNumUninitialized(NumTargetKeys);

							if (SourceKeys->GetKeys<PCGMetadataEntryKey>(0, AllSourceEntryKeysPtr) && TargetKeys->GetKeys<PCGMetadataEntryKey>(0, AllTargetEntryKeysPtr))
							{
								// Gather all the value keys
								TArray<PCGMetadataEntryKey> AllSourceEntryKeys;
								Algo::Transform(AllSourceEntryKeysPtr, AllSourceEntryKeys, [](const PCGMetadataEntryKey* KeyPtr) { return *KeyPtr; });
								TArray<PCGMetadataValueKey> ValueKeys;
								TArray<PCGMetadataValueKey> FinalValueKeys;
								ValueKeys.Reserve(AllSourceEntryKeys.Num());
								SourceAttribute->GetValueKeys(TArrayView<const PCGMetadataEntryKey>(AllSourceEntryKeys), ValueKeys);

								// Extends values keys to match target entry keys size. It will loop on value keys.
								const int32 TargetEntryKeysSize = AllTargetEntryKeysPtr.Num();
								if (ValueKeys.Num() >= TargetEntryKeysSize)
								{
									FinalValueKeys = std::move(ValueKeys);
								}
								else
								{
									FinalValueKeys.Reserve(TargetEntryKeysSize);
									while (FinalValueKeys.Num() < TargetEntryKeysSize)
									{
										FinalValueKeys.Append(TArrayView<PCGMetadataValueKey>(ValueKeys.GetData(), FMath::Min(ValueKeys.Num(), TargetEntryKeysSize - FinalValueKeys.Num())));
									}
								}

								// Make sure that the Target has some metadata entry
								// Implementation note: this is a stripped down version of UPCGMetadata::InitializeOnSet
								TArray<PCGMetadataEntryKey*> AllTargetEntryKeysPtrTemp;
								AllTargetEntryKeysPtrTemp.Reserve(AllTargetEntryKeysPtr.Num());
								for (int EntryIndex = 0; EntryIndex < AllTargetEntryKeysPtr.Num(); ++EntryIndex)
								{
									PCGMetadataEntryKey& EntryKey = *AllTargetEntryKeysPtr[EntryIndex];
									if (EntryKey == PCGInvalidEntryKey || (bTargetSupportsMultiEntries && EntryKey < TargetMetadataDomain->GetItemKeyCountForParent()))
									{
										AllTargetEntryKeysPtrTemp.Add(&EntryKey);
									}
								}

								if (!AllTargetEntryKeysPtrTemp.IsEmpty())
								{
									TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAttributeAccessor::Prepare::AddEntriesInPlace);
									TargetMetadataDomain->AddEntriesInPlace(AllTargetEntryKeysPtrTemp);
								}

								NewAttr->SetValuesFromValueKeys(AllTargetEntryKeysPtr, FinalValueKeys, /*bResetValueOnDefaultValueKey=*/true);
							}
						}
					}
				}
				else
				{
					PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("FailedCreateNewAttribute", "Failed to create new attribute '{0}'"), OutputTarget.GetDisplayText()), OptionalContext);
					continue;
				}
			}
			else // Create a new attribute of the accessed field's type manually
			{
				TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(SourceData, InputSource);
				TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(SourceData, InputSource);

				if (!InputAccessor.IsValid() || !InputKeys.IsValid())
				{
					PCGLog::Metadata::LogFailToCreateAccessorError(InputSource, OptionalContext);
					continue;
				}
				
				FPCGMetadataAttributeDesc OutputDesc{};
				if (bOutputTypeCast)
				{
					OutputDesc.ValueType = RequestedOutputType;
				}
				else
				{
					OutputDesc = InputAccessor->GetUnderlyingDesc();
				}
				
				if (bOutputTypeCast && InputAccessor->GetUnderlyingDesc().IsSameType(OutputDesc) && bSameOrigin && LocalSourceAttribute == LocalDestinationAttribute)
				{
					// Nothing to do if we try to cast an attribute on itself with the same type
					continue;
				}

				// If we have a cast, make sure it is valid
				if (bOutputTypeCast && !PCG::Private::IsBroadcastableOrConstructible(InputAccessor->GetUnderlyingDesc(), OutputDesc))
				{
					PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("CastInvalid", "Cannot convert InputAttribute '{0}' of type {1} into {2}"), InputSource.GetDisplayText(), InputAccessor->GetUnderlyingDesc().GetTypeText(), OutputDesc.GetTypeText()), OptionalContext);
					continue;
				}

				// If the target is an attribute, only create a new one if the attribute we don't have any extra.
				// If it has any extra, it will try to write to it.
				bool bShouldCopyDefaultValue = false;
				if (!bOutputHasAnyExtra && bTargetIsAttribute)
				{
					FPCGMetadataDomain* TargetMetadataDomain = TargetMetadata->GetMetadataDomainFromSelector(OutputTarget);
					if (!TargetMetadataDomain)
					{
						PCGLog::Metadata::LogInvalidMetadataDomain(OutputTarget, OptionalContext);
						continue;
					}

					OutputDesc.Name = OutputTarget.GetAttributeName();
					if (TargetMetadataDomain->HasAttribute(OutputDesc.Name))
					{
						TargetMetadataDomain->DeleteAttribute(OutputDesc.Name);
					}
					
					if (!TargetMetadataDomain->CreateAttribute(OutputDesc))
					{
						PCGLog::Metadata::LogFailToCreateAttributeError(OutputDesc, OptionalContext);
						continue;
					}

					bShouldCopyDefaultValue = true;
				}

				TUniquePtr<IPCGAttributeAccessor> OutputAccessor = PCGAttributeAccessorHelpers::CreateAccessor(TargetData, OutputTarget);
				TUniquePtr<IPCGAttributeAccessorKeys> OutputKeys = PCGAttributeAccessorHelpers::CreateKeys(TargetData, OutputTarget);

				if (!OutputAccessor.IsValid() || !OutputKeys.IsValid())
				{
					PCGLog::Metadata::LogFailToCreateAccessorError(OutputTarget, OptionalContext);
					continue;
				}

				if (OutputAccessor->IsReadOnly())
				{
					PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("OutputAccessorIsReadOnly", "Attribute/Property '{0}' is read only."), OutputTarget.GetDisplayText()), OptionalContext);
					continue;
				}

				// Final verification (if not already done), if we can put the value of input into output
				if (!bOutputTypeCast && !PCG::Private::IsBroadcastableOrConstructible(OutputDesc, OutputAccessor->GetUnderlyingDesc()))
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("CannotConvertTypes", "Cannot convert input type {0} into output type {1}"), OutputDesc.GetTypeText(), OutputAccessor->GetUnderlyingDesc().GetTypeText()), OptionalContext);
					continue;
				}

				// At this point, we are ready.
				PCGMetadataElementCommon::FCopyFromAccessorToAccessorParams Params;
				Params.InAccessor = InputAccessor.Get();
				Params.OutAccessor = OutputAccessor.Get();
				Params.IterationCount = PCGMetadataElementCommon::FCopyFromAccessorToAccessorParams::Out;

				// Do a simple copy of the default value, if it is possible and needed. It is OK to fail if the input accessor is a property for example.
				if (bShouldCopyDefaultValue)
				{
					FPCGAttributeAccessorKeysEntries DefaultEntryKeys{PCGInvalidEntryKey};
					Params.InKeys = &DefaultEntryKeys;
					Params.OutKeys = &DefaultEntryKeys;
					Params.Flags = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible | EPCGAttributeAccessorFlags::AllowSetDefaultValue;

					PCGMetadataElementCommon::CopyFromAccessorToAccessor(Params);
				}

				// Then do the real copy
				Params.InKeys = InputKeys.Get();
				Params.OutKeys = OutputKeys.Get();
				Params.Flags = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible;
				if (!PCGMetadataElementCommon::CopyFromAccessorToAccessor(Params))
				{
					PCGLog::LogWarningOnGraph(LOCTEXT("ErrorGettingSettingValues", "Error while getting/setting values"), OptionalContext);
					continue;
				}
			}

			bSuccess = true;
		}

		return bSuccess;
	}

	bool CopyAttribute(const FPCGCopyAttributeParams& InParams)
	{
		if (!InParams.TargetData || !InParams.SourceData)
		{
			return false;
		}

		TArray<TTuple<FPCGAttributePropertyInputSelector, FPCGAttributePropertyOutputSelector, EPCGMetadataTypes>> AttributeSelectors;
		FPCGAttributePropertyInputSelector InputSource = InParams.InputSource.CopyAndFixLast(InParams.SourceData);
		FPCGAttributePropertyOutputSelector OutputTarget = InParams.OutputTarget.CopyAndFixSource(&InputSource, InParams.SourceData);

		AttributeSelectors.Emplace(MoveTemp(InputSource), MoveTemp(OutputTarget), InParams.OutputType);
		return CopyAttributes(InParams.TargetData, InParams.SourceData, AttributeSelectors, InParams.bSameOrigin, InParams.OptionalContext);
	}

	void FPCGCopyAllAttributesParams::InitializeMappingFromDomainNames(const TMap<FName, FName>& MetadataDomainsMapping)
	{
		if (MetadataDomainsMapping.IsEmpty())
		{
			DomainMapping.Empty(1);
			DomainMapping = {{PCGMetadataDomainID::Default, PCGMetadataDomainID::Default}};
			return;
		}
		
		DomainMapping.Empty(MetadataDomainsMapping.Num());
		
		if (!SourceData || !TargetData)
		{
			return;
		}
		
		FPCGAttributePropertySelector TempSelector{};
	
		for (const TPair<FName, FName>& It : MetadataDomainsMapping)
		{
			TempSelector.SetDomainName(It.Key);
			const FPCGMetadataDomainID SourceDomain = SourceData->GetMetadataDomainIDFromSelector(TempSelector);
			if (!SourceDomain.IsValid())
			{
				PCGLog::Metadata::LogInvalidMetadataDomain(TempSelector);
				continue;
			}
				
			TempSelector.SetDomainName(It.Value);
			const FPCGMetadataDomainID TargetDomain = TargetData->GetMetadataDomainIDFromSelector(TempSelector);
			if (!TargetDomain.IsValid())
			{
				PCGLog::Metadata::LogInvalidMetadataDomain(TempSelector);
				continue;
			}
		
			DomainMapping.Emplace(SourceDomain, TargetDomain);
		}
	}

	void FPCGCopyAllAttributesParams::InitializeMappingForAllDomains()
	{
		DomainMapping.Reset();
		
		if (!SourceData || !TargetData)
		{
			return;
		}

		for (const FPCGMetadataDomainID SourceDomain : SourceData->GetAllSupportedMetadataDomainIDs())
		{
			if (TargetData->IsSupportedMetadataDomainID(SourceDomain))
			{
				DomainMapping.Emplace(SourceDomain, SourceDomain);
			}
		}
	}

	bool CopyAllAttributes(const UPCGData* SourceData, UPCGData* TargetData, FPCGContext* OptionalContext)
	{
		const FPCGCopyAllAttributesParams InParams
		{
			.SourceData = SourceData,
			.TargetData = TargetData,
			.OptionalContext = OptionalContext
		};

		return CopyAllAttributes(InParams);
	}

	bool CopyAllAttributes(const FPCGCopyAllAttributesParams& InParams)
	{
		if (!InParams.TargetData || !InParams.SourceData)
		{
			return false;
		}

		const UPCGMetadata* SourceMetadata = InParams.SourceData->ConstMetadata();
		if (!SourceMetadata)
		{
			return false;
		}

		TArray<TTuple<FPCGAttributePropertyInputSelector, FPCGAttributePropertyOutputSelector, EPCGMetadataTypes>> AttributeSelectors;
		TArray<FPCGAttributeIdentifier> AttributeIDs;
		TArray<EPCGMetadataTypes> AttributeTypes;
		SourceMetadata->GetAllAttributes(AttributeIDs, AttributeTypes);

		const FPCGMetadataDomainID DefaultSourceDomain = SourceMetadata->GetConstDefaultMetadataDomain()->GetDomainID();

		for (const FPCGAttributeIdentifier& AttributeID : AttributeIDs)
		{
			const bool bIsDefaultDomain = AttributeID.MetadataDomain == DefaultSourceDomain;
			const FPCGMetadataDomainID* TargetDomainId = nullptr;
			if (!InParams.DomainMapping.IsEmpty())
			{
				TargetDomainId = InParams.DomainMapping.Find(AttributeID.MetadataDomain);
				if (!TargetDomainId && bIsDefaultDomain)
				{
					TargetDomainId = InParams.DomainMapping.Find(PCGMetadataDomainID::Default);
				}

				if (!TargetDomainId && AttributeID.MetadataDomain.IsDefault())
				{
					TargetDomainId = InParams.DomainMapping.Find(DefaultSourceDomain);
				}

				if (!TargetDomainId)
				{
					// Didn't find the domain in the mapping, continue...
					continue;
				}
			}
			
			TTuple<FPCGAttributePropertyInputSelector, FPCGAttributePropertyOutputSelector, EPCGMetadataTypes>& Selectors = AttributeSelectors.Emplace_GetRef();
			Selectors.Get<0>().SetAttributeName(AttributeID.Name);
			Selectors.Get<1>().SetAttributeName(AttributeID.Name);
			Selectors.Get<2>() = EPCGMetadataTypes::Unknown;

			InParams.SourceData->SetDomainFromDomainID(AttributeID.MetadataDomain, Selectors.Get<0>());
			InParams.TargetData->SetDomainFromDomainID(TargetDomainId ? *TargetDomainId : AttributeID.MetadataDomain, Selectors.Get<1>());
		}

		return CopyAttributes(InParams.TargetData, InParams.SourceData, AttributeSelectors, /*bSameOrigin=*/false, InParams.OptionalContext);
	}

	void ComputePointWeightedAttribute(FPCGMetadataDomain* InOutMetadata, FPCGPoint& OutPoint, const TArrayView<TPair<const FPCGPoint*, float>>& InWeightedPoints, const FPCGMetadataDomain* InMetadata)
	{
		check(InOutMetadata);
		TArray<TPair<PCGMetadataEntryKey, float>, TInlineAllocator<4>> InWeightedKeys;
		InWeightedKeys.Reserve(InWeightedPoints.Num());

		for (const TPair<const FPCGPoint*, float>& WeightedPoint : InWeightedPoints)
		{
			InWeightedKeys.Emplace(WeightedPoint.Key->MetadataEntry, WeightedPoint.Value);
		}

		InOutMetadata->ComputeWeightedAttribute(OutPoint.MetadataEntry, MakeArrayView(InWeightedKeys), InMetadata);
	}

	void SetPointAttributes(FPCGMetadataDomain* InOutMetadata, const TArrayView<const FPCGPoint>& InPoints, const FPCGMetadataDomain* InMetadata, const TArrayView<FPCGPoint>& OutPoints, FPCGContext* OptionalContext)
	{
		check(InOutMetadata);
		if (!InMetadata || InMetadata->GetAttributeCount() == 0 || InOutMetadata->GetAttributeCount() == 0)
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataHelpers::SetPointAttributes);

		check(InPoints.Num() == OutPoints.Num());

		// Extract the metadata entry keys from the in & out points
		TArray<PCGMetadataEntryKey, TInlineAllocator<256>> InKeys;
		TArray<PCGMetadataEntryKey, TInlineAllocator<256>> OutKeys;

		Algo::Transform(InPoints, InKeys, [](const FPCGPoint& Point) { return Point.MetadataEntry; });
		Algo::Transform(OutPoints, OutKeys, [](const FPCGPoint& Point) { return Point.MetadataEntry; });

		InOutMetadata->SetAttributes(InKeys, InMetadata, OutKeys, OptionalContext);

		// Write back the keys on the points
		for (int KeyIndex = 0; KeyIndex < OutKeys.Num(); ++KeyIndex)
		{
			OutPoints[KeyIndex].MetadataEntry = OutKeys[KeyIndex];
		}
	}

}

#undef LOCTEXT_NAMESPACE