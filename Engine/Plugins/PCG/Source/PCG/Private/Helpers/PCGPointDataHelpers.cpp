// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGPointDataHelpers.h"

#include "Data/PCGBasePointData.h"

void PCGPointDataHelpers::WeightedAverage(const UPCGBasePointData* InPointData, TConstArrayView<TPair<int32, float>> Coefficients, UPCGBasePointData* OutPointData, int32 OutPointIndex, bool bApplyOnMetadata)
{
	if (!InPointData
		|| !OutPointData
		|| Coefficients.IsEmpty()
		|| !Algo::AllOf(Coefficients, [InPointData](const TPair<int32, float>& It) { return It.Key >= 0 && It.Key < InPointData->GetNumPoints() && It.Value >= 0.f; })
		|| OutPointIndex < 0
		|| OutPointIndex >= OutPointData->GetNumPoints())
	{
		return;
	}
	
	EPCGPointNativeProperties Properties = InPointData->GetAllocatedProperties();
	if (Properties == EPCGPointNativeProperties::None)
	{
		// No allocated properties, nothing to do
		return;
	}

	if (Coefficients.Num() == 1)
	{
		InPointData->CopyPointsTo(OutPointData, Coefficients[0].Key, OutPointIndex, /*Count=*/1);
		return;
	}

	FConstPCGPointValueRanges InPointValueRanges(InPointData);

	OutPointData->AllocateProperties(Properties);

	FPCGPointValueRanges OutPointValueRanges(OutPointData, /*bAllocate=*/false);

	int32 StrongestIndex = INDEX_NONE;
	float MaxWeight = -1.0f;
	float SumWeight = 0.0f;
	for (const auto [Index, Weight] : Coefficients)
	{
		SumWeight += Weight;
		if (Weight > MaxWeight)
		{
			StrongestIndex = Index;
			MaxWeight = Weight;
		}
	}
	
	if (FMath::IsNearlyZero(SumWeight) || !ensure(StrongestIndex != INDEX_NONE))
	{
		// Copy the first point if all the coefficients are 0
		InPointData->CopyPointsTo(OutPointData, Coefficients[0].Key, OutPointIndex, /*Count=*/1);
		return;
	}

	auto DoAverageIfAllocated = [&Coefficients, Properties, OutPointIndex, SumWeight]<typename T>(TPCGValueRange<T>& ToRange, const TConstPCGValueRange<T>& FromRange, EPCGPointNativeProperties Property)
	{
		if (EnumHasAllFlags(Properties, Property))
		{
			T& OutValue = ToRange[OutPointIndex];
			OutValue = PCG::Private::MetadataTraits<T>::ZeroValueForWeightedSum();
			
			for (const auto [Index, Weight] : Coefficients) 
			{
				OutValue = PCG::Private::MetadataTraits<T>::WeightedSum(OutValue, FromRange[Index], Weight / SumWeight);
			}
			
			if constexpr (PCG::Private::MetadataTraits<T>::InterpolationNeedsNormalization)
			{
				static_assert(PCG::Private::MetadataTraits<T>::CanNormalize);

				// We need to normalize the resulting value
				PCG::Private::MetadataTraits<T>::Normalize(OutValue);
			}
		}
	};

	DoAverageIfAllocated(OutPointValueRanges.TransformRange, InPointValueRanges.TransformRange, EPCGPointNativeProperties::Transform);
	DoAverageIfAllocated(OutPointValueRanges.DensityRange, InPointValueRanges.DensityRange, EPCGPointNativeProperties::Density);
	DoAverageIfAllocated(OutPointValueRanges.BoundsMinRange, InPointValueRanges.BoundsMinRange, EPCGPointNativeProperties::BoundsMin);
	DoAverageIfAllocated(OutPointValueRanges.BoundsMaxRange, InPointValueRanges.BoundsMaxRange, EPCGPointNativeProperties::BoundsMax);
	DoAverageIfAllocated(OutPointValueRanges.ColorRange, InPointValueRanges.ColorRange, EPCGPointNativeProperties::Color);
	DoAverageIfAllocated(OutPointValueRanges.SteepnessRange, InPointValueRanges.SteepnessRange, EPCGPointNativeProperties::Steepness);

	// For seed and metadata entry, we can't average, so take the highest coefficient (for metadata entry, only if we are not averaging metadata)
	if (EnumHasAnyFlags(Properties, EPCGPointNativeProperties::Seed))
	{
		OutPointValueRanges.SeedRange[OutPointIndex] = InPointValueRanges.SeedRange[StrongestIndex];
	}

	if (EnumHasAnyFlags(Properties, EPCGPointNativeProperties::MetadataEntry))
	{
		const UPCGMetadata* InMetadata = InPointData->ConstMetadata();
		UPCGMetadata* OutMetadata = OutPointData->MutableMetadata();
		
		if (bApplyOnMetadata && InMetadata && OutMetadata)
		{
			OutPointValueRanges.MetadataEntryRange[OutPointIndex] = OutMetadata->AddEntry();
			TArray<TPair<PCGMetadataEntryKey, float>> AttributeCoefficients;
			AttributeCoefficients.Reserve(Coefficients.Num());
			Algo::Transform(Coefficients, AttributeCoefficients, [&Range = InPointValueRanges.MetadataEntryRange](const TPair<int32, float>& It) -> TPair<PCGMetadataEntryKey, float> { return {Range[It.Key], It.Value}; });
			
			OutMetadata->ComputeWeightedAttribute(OutPointValueRanges.MetadataEntryRange[OutPointIndex], AttributeCoefficients, InMetadata);
		}
		else
		{
			OutPointValueRanges.MetadataEntryRange[OutPointIndex] = InPointValueRanges.MetadataEntryRange[StrongestIndex];
		}
	}
}
