// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#include "Containers/ContainersFwd.h"
#include "Templates/Tuple.h"

class UPCGBasePointData;

namespace PCGPointDataHelpers
{
	/**
	 * Do the weighted average of all the input points for allocated properties. Note that unallocated properties are untouched, it is the caller
	 * responsibility to make sure those are copied from in point data to out point data.
	 * @param InPointData          Input point data to read from
	 * @param Coefficients         Array of pairs of input point index and their coefficients
	 * @param OutPointData         Output point data to write to
	 * @param OutPointIndex        Output point index to write to
	 * @param bApplyOnMetadata     If the metadata should also do the average
	 */
	void WeightedAverage(const UPCGBasePointData* InPointData, TConstArrayView<TPair<int32, float>> Coefficients, UPCGBasePointData* OutPointData, int32 OutPointIndex, bool bApplyOnMetadata);
}
