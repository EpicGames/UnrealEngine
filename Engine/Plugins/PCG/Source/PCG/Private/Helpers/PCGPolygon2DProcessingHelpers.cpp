// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGPolygon2DProcessingHelpers.h"

#include "PCGModule.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPolygon2DData.h"
#include "Data/PCGSplineData.h"
#include "Metadata/PCGMetadata.h"

namespace PCGPolygon2DProcessingHelpers
{

bool GetPath(const UPCGData* InData, const FTransform& PolyTransform, double DiscretizationError, TArray<FVector2D>& OutPath)
{
	OutPath.Reset();

	TArray<FVector> Positions;

	if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(InData))
	{
		const double ErrorTolerance = FMath::Max(DiscretizationError, 0.001);
		// Implementation note: we will not remove the last point in the case of a closed spline because we want the full path regardless.
		SplineData->SplineStruct.ConvertSplineToPolyLine(ESplineCoordinateSpace::World, ErrorTolerance, Positions);
	}
	else if(const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(InData))
	{
		const TConstPCGValueRange<FTransform> TransformRange = PointData->GetConstTransformValueRange();
		Positions.Reserve(TransformRange.Num());
		for (const FTransform& Transform : TransformRange)
		{
			Positions.Emplace(Transform.GetLocation());
		}
	}
	else
	{
		// Unsupported type
		return false;
	}

	OutPath.Reserve(Positions.Num());
	for (const FVector& Position : Positions)
	{
		FVector Position2D = PolyTransform.InverseTransformPosition(Position);
		OutPath.Emplace(Position2D.X, Position2D.Y);
	}

	return true;
}

bool GetEnclosingPolygonFromPath(const UPCGData* InData, const FTransform& PolyTransform, double DiscretizationError, UE::Geometry::FAxisAlignedBox2d Bounds, UE::Geometry::FGeneralPolygon2d& OutPoly)
{
	// Idea: 
	// 1. create a polygon where the first segment(s) is the input path
	// 2. complete the polygon using the points from the bounding box,
	//      where we will select the first vertex on the left from either the other points on the bounding box or the segment start.
	TArray<FVector2D> Path;
	if (!GetPath(InData, PolyTransform, DiscretizationError, Path))
	{
		// Unsupported type
		return false;
	}

	if (Path.Num() < 2)
	{
		// Invalid path
		return false;
	}

	// Make bounds slightly larger to prevent issues with edges on the polygons that are directly on the bounds.
	const double SafeBoundarySize = FMath::Max(1.0, 0.01 * FMath::Max(Bounds.Width(), Bounds.Height()));
	Bounds.Expand(SafeBoundarySize);

	TStaticArray<FVector2D, 4> Corners;
	for (int CornerIndex = 0; CornerIndex < 4; ++CornerIndex)
	{
		Corners[CornerIndex] = Bounds.GetCorner(CornerIndex);
	}

	TArray<FVector2D> TargetPoints;

	TargetPoints.Reset();
	TargetPoints.Append(Corners);
	TargetPoints.Add(Path[0]);

	bool bDone = false;
	while (!bDone)
	{
		FVector2D LastPosition = Path.Last();
		FVector2D LastDirection = (Path.Last() - Path.Last(1)).GetSafeNormal();
		double MinAngle = DBL_MAX;
		int MinIndex = INDEX_NONE;

		// We'll select the point that has the smallest angle vs. the previous direction, counting 0 from the left, up to 2*pi at the right.
		// e.g. we're doing similar to a convex hull with point sorting.
		for (int TargetIndex = 0; TargetIndex < TargetPoints.Num(); ++TargetIndex)
		{
			// Compute angle from (Target - Last) vs last direction.
			FVector2D TargetDirection = (TargetPoints[TargetIndex] - LastPosition).GetSafeNormal();
			FVector2D ProjectedVector = FVector2D(TargetDirection | LastDirection, LastDirection ^ TargetDirection);
			double Angle = FMath::Atan2(ProjectedVector.Y, ProjectedVector.X);

			// Atan2 returns a value in the ]-pi, pi] range, but we want a value in the [0, 2pi[ range (left to right).
			if (Angle < 0)
			{
				Angle += 2.0 * UE_DOUBLE_PI;
			}

			ensure(Angle >= 0);

			if (Angle < MinAngle)
			{
				MinAngle = Angle;
				MinIndex = TargetIndex;
			}
		}

		// Two conditions here:
		// Either we're adding a point from the bounding box, or we're done with the polygon.
		if (MinIndex == TargetPoints.Num() - 1)
		{
			// we're done, we selected the first path point.
			bDone = true;
		}
		else
		{
			Path.Add(TargetPoints[MinIndex]);
			TargetPoints.RemoveAt(MinIndex);
		}
	}

	OutPoly = UE::Geometry::FGeneralPolygon2d(UE::Geometry::FPolygon2d(MoveTemp(Path)));
	return true;
}

TArray<UE::Geometry::FGeneralPolygon2d> GetPolygons(const TArray<const UPCGPolygon2DData*>& InPolygonData, bool& bTransformSet, FTransform& ReferenceTransform)
{
	TArray<UE::Geometry::FGeneralPolygon2d> OutPolys;
	OutPolys.Reserve(InPolygonData.Num());

	for (const UPCGPolygon2DData* PolyData : InPolygonData)
	{
		AddPolygon(PolyData, OutPolys, bTransformSet, ReferenceTransform);
	}

	return OutPolys;
}

void AddPolygon(const UPCGPolygon2DData* InPolygonData, TArray<UE::Geometry::FGeneralPolygon2d>& OutPolys, bool& bTransformSet, FTransform& ReferenceTransform)
{
	AddPolygon(InPolygonData, OutPolys, nullptr, nullptr, bTransformSet, ReferenceTransform);
}

void AddPolygon(const UPCGPolygon2DData* InPolygonData,
	TArray<UE::Geometry::FGeneralPolygon2d>& OutPolys,
	TArray<TArray<PCGMetadataEntryKey>>* OutPolyVertexEntryKeysList,
	TArray<const UPCGMetadata*>* OutMetadataList,
	bool& bTransformSet, 
	FTransform& ReferenceTransform)
{
	check(InPolygonData);
	const UE::Geometry::FGeneralPolygon2d& InPoly = InPolygonData->GetPolygon();
	const FTransform& InTransform = InPolygonData->GetTransform();
	TConstArrayView<PCGMetadataEntryKey> InEntryKeys = InPolygonData->GetConstVerticesEntryKeys();

	if (OutMetadataList)
	{
		OutMetadataList->Add(InPolygonData->ConstMetadata());
	}

	if (!bTransformSet)
	{
		OutPolys.Add(InPoly);

		if (OutPolyVertexEntryKeysList)
		{
			OutPolyVertexEntryKeysList->Emplace(InEntryKeys);
		}

		ReferenceTransform = InTransform;
		bTransformSet = true;
	}
	else
	{
		// @todo_pcg: we could validate that the normal are parallel.
		UE::Geometry::FGeneralPolygon2d& Poly = OutPolys.Emplace_GetRef();

		if (OutPolyVertexEntryKeysList)
		{
			OutPolyVertexEntryKeysList->Emplace(InEntryKeys);
		}

		// Important note: since this can technically flip the normal, that would in turn change the polygon orientation.
		// Unfortunately, this means that the winding of the polygon might change.
		// However, the polygon transform isn't aware of this, so we need to rebuild the polygon from scratch here instead.
		const FTransform RelativeTransform = InTransform.GetRelativeTransform(ReferenceTransform);

		Poly = InPoly;
		Poly.Transform([&RelativeTransform](const FVector2D& InVertex)
		{
			const FVector TransformedVertex = RelativeTransform.TransformPosition(FVector(InVertex, 0.0));
			return FVector2D(TransformedVertex.X, TransformedVertex.Y);
		});

		// If the outer was reversed because of the operation (due to a normal sign toggle, for example - and because the Transform doesn't update the flag),
		// We need to fix this up so that the reported type of the polygon is in line with the reported type of the outer.
		if (Poly.OuterIsClockwise() != Poly.GetOuter().IsClockwise())
		{
			if (OutPolyVertexEntryKeysList)
			{
				PCGPolygon2DData::ReversePolygon(Poly, OutPolyVertexEntryKeysList->Last());
			}
			else
			{
				Poly.Reverse();
			}

			// Finally, reupdate the outer so the flag is put back to its good value
			UE::Geometry::FPolygon2d TempOuter = Poly.GetOuter();
			Poly.SetOuter(MoveTemp(TempOuter));

			// At this point, the polygon should have been properly reset, but it might not appear so if it is degenerate, in which case we'll just take it as is.
			check(Poly.OuterIsClockwise() == InPoly.OuterIsClockwise() || FMath::Abs(Poly.SignedArea()) < UE_KINDA_SMALL_NUMBER);
		}
	}
}

void ForAllVertices(UE::Geometry::FGeneralPolygon2d& InPoly, TFunctionRef<bool(const FVector2D&, int /*VertexIndex*/, int /*SegmentIndex*/, int /*HoleIndex*/)> InFunc)
{
	bool bExit = false;
	int Offset = 0;

	auto ProcessPoly = [&InFunc, &Offset](const UE::Geometry::FPolygon2d& Poly, int HoleIndex) -> bool
	{
		const TArray<FVector2D>& Vertices = Poly.GetVertices();
		for (int VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
		{
			if (!InFunc(Vertices[VertexIndex], Offset + VertexIndex, VertexIndex, HoleIndex))
			{
				return false;
			}
		}

		Offset += Vertices.Num();
		return true;
	};

	if (!ProcessPoly(InPoly.GetOuter(), -1))
	{
		return;
	}

	int HoleIndex = 0;
	for (const UE::Geometry::FPolygon2d& Hole : InPoly.GetHoles())
	{
		if (!ProcessPoly(Hole, HoleIndex++))
		{
			return;
		}
	}
}

bool DistanceSquaredToPolygons(const FVector2D& InVertex,
	TArrayView<const UE::Geometry::FGeneralPolygon2d> InPolys,
	double Tolerance,
	int& OutPolyIndex,
	int& OutSegmentIndex,
	int& OutHoleIndex,
	double& OutT,
	double& OutMinDistance,
	double InitMinDistance)
{
	int MinPolyIndex = -1;
	int MinSegmentIndex = -1;
	int MinHoleIndex = -1;
	double MinT = 0.0;
	double MinDistance = InitMinDistance;

	for(int PolyIndex = 0; PolyIndex < InPolys.Num(); ++PolyIndex)
	{
		int HoleIndex = -1;
		int SegmentIndex = -1;
		double T = 0.0;
		const double Distance = InPolys[PolyIndex].DistanceSquared(InVertex, HoleIndex, SegmentIndex, T);
		if (Distance < MinDistance)
		{
			MinDistance = Distance;
			MinPolyIndex = PolyIndex;
			MinSegmentIndex = SegmentIndex;
			MinHoleIndex = HoleIndex;
			MinT = T;

			if (Distance < Tolerance)
			{
				break;
			}
		}
	}

	if (MinPolyIndex >= 0)
	{
		OutPolyIndex = MinPolyIndex;
		OutSegmentIndex = MinSegmentIndex;
		OutHoleIndex = MinHoleIndex;
		OutT = MinT;
		OutMinDistance = MinDistance;
	}

	return MinDistance < Tolerance;
}

// Matches the vertices from the Polys to either vertices or edges on the Primary/Secondary polygons.
TArray<PolygonVertexMapping> BuildVertexMapping(TArray<UE::Geometry::FGeneralPolygon2d>& Polys, TConstArrayView<UE::Geometry::FGeneralPolygon2d> PrimaryPolygons, TConstArrayView<UE::Geometry::FGeneralPolygon2d> SecondaryPolygons)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PCGPolygon2DProcessingHelpers::BuildVertexMapping);
	TArray<PolygonVertexMapping> VertexMappings;
	VertexMappings.SetNum(Polys.Num());

	for (int PolyIndex = 0; PolyIndex < Polys.Num(); ++PolyIndex)
	{
		UE::Geometry::FGeneralPolygon2d& Poly = Polys[PolyIndex];
		PolygonVertexMapping& PolyVertexMapping = VertexMappings[PolyIndex];

		ForAllVertices(Poly, [&PolyVertexMapping, &PrimaryPolygons, &SecondaryPolygons](const FVector2D& Vertex, int VertexIndex, int SegmentIndex, int HoleIndex)
		{
			check(VertexIndex == PolyVertexMapping.Num());
			VertexToSourcePolygonMapping& VertexMapping = PolyVertexMapping.Emplace_GetRef();

			double MinDistanceToPrimaryPolygons = DBL_MAX;
			int NearestPolyIndex = -1;
			double T = 0.0;

			// Test against primary polygons first
			bool bDone = DistanceSquaredToPolygons(Vertex, PrimaryPolygons, UE_SMALL_NUMBER, NearestPolyIndex, VertexMapping.SegmentIndex, VertexMapping.HoleIndex, T, MinDistanceToPrimaryPolygons, DBL_MAX);

			if (PrimaryPolygons.IsValidIndex(NearestPolyIndex))
			{
				VertexMapping.Poly = &PrimaryPolygons[NearestPolyIndex];
			}

			// Otherwise, test against secondary polygons
			if (!bDone)
			{
				double MinDistanceToSecondaryPolygons = DBL_MAX;
				NearestPolyIndex = -1;
				bDone = DistanceSquaredToPolygons(Vertex, SecondaryPolygons, UE_SMALL_NUMBER, NearestPolyIndex, VertexMapping.SegmentIndex, VertexMapping.HoleIndex, T, MinDistanceToSecondaryPolygons, MinDistanceToPrimaryPolygons);

				if (SecondaryPolygons.IsValidIndex(NearestPolyIndex))
				{
					VertexMapping.Poly = &SecondaryPolygons[NearestPolyIndex];
				}
			}

			// Temporarily, we'll want to ensure here, but if it happened, we might be fine with it, if the final distance is reasonable.
			if (ensure(VertexMapping.Poly))
			{
				VertexMapping.Ratio = VertexMapping.Poly->Segment(VertexMapping.SegmentIndex, VertexMapping.HoleIndex).ConvertToUnitRange(T);
				PCGPolygon2DData::GetStartEndVertexIndices(*VertexMapping.Poly, VertexMapping.SegmentIndex, VertexMapping.HoleIndex, VertexMapping.StartVertexIndex, VertexMapping.EndVertexIndex);
			}

			return true; // always process all vertices
		});
	}

	return VertexMappings;
}

void AddToPolyMetadataInfoMap(PolyToMetadataInfoMap& OutMap, const UE::Geometry::FGeneralPolygon2d& Poly, const UPCGMetadata* Metadata, const TArray<PCGMetadataEntryKey>& EntryKeys)
{
	OutMap.Add(&Poly, MetadataInfo(Metadata, &EntryKeys));
}

void AddToPolyMetadataInfoMap(PolyToMetadataInfoMap& OutMap, const TArray<UE::Geometry::FGeneralPolygon2d>& Polys, const TArray<const UPCGMetadata*>& Metadata, const TArray<TArray<PCGMetadataEntryKey>>& EntryKeys)
{
	check(Polys.Num() == Metadata.Num() && Polys.Num() == EntryKeys.Num());
	for (int i = 0; i < Polys.Num(); ++i)
	{
		OutMap.Add(&Polys[i], MetadataInfo(Metadata[i], &EntryKeys[i]));
	}
}

TArray<PCGMetadataEntryKey> ComputeEntryKeysAndInitializeAttributes(UPCGMetadata* OutMetadata, const PolygonVertexMapping& InVertexMapping, const PolyToMetadataInfoMap& InInfoMap)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PCGPolygon2DProcessingHelpers::ComputeEntryKeys);
	TArray<PCGMetadataEntryKey> EntryKeys;
	EntryKeys.Reserve(InVertexMapping.Num());

	TSet<const UPCGMetadata*> VisitedMetadata;

	for (int VertexIndex = 0; VertexIndex < InVertexMapping.Num(); ++VertexIndex)
	{
		const VertexToSourcePolygonMapping& VertexMapping = InVertexMapping[VertexIndex];
		PCGMetadataEntryKey& NewKey = EntryKeys.Add_GetRef(PCGInvalidEntryKey);

		const UE::Geometry::FGeneralPolygon2d* SourcePolygon = VertexMapping.Poly;
		if (!SourcePolygon || !InInfoMap.Contains(SourcePolygon))
		{
			continue;
		}

		const auto& [SourceMetadata, SourceEntryKeys] = InInfoMap[SourcePolygon];

		// Propagate attributes from source metadata to out metadata
		if (!VisitedMetadata.Contains(SourceMetadata))
		{
			VisitedMetadata.Add(SourceMetadata);
			OutMetadata->AddAttributes(SourceMetadata);
		}

		if (SourceEntryKeys->IsEmpty())
		{
			continue;
		}

		if (!SourceEntryKeys->IsValidIndex(VertexMapping.StartVertexIndex) ||
			!SourceEntryKeys->IsValidIndex(VertexMapping.EndVertexIndex))
		{
			continue;
		}

		const auto& StartVertexEntryKey = (*SourceEntryKeys)[VertexMapping.StartVertexIndex];
		const auto& EndVertexEntryKey = (*SourceEntryKeys)[VertexMapping.EndVertexIndex];

		if (StartVertexEntryKey != PCGInvalidEntryKey && EndVertexEntryKey != PCGInvalidEntryKey)
		{
			OutMetadata->InitializeOnSet(NewKey);

			TStaticArray<TPair<PCGMetadataEntryKey, float>, 2> Coefficients;
			Coefficients[0] = { StartVertexEntryKey, 1.0f - VertexMapping.Ratio };
			Coefficients[1] = { EndVertexEntryKey, VertexMapping.Ratio };
			OutMetadata->ComputeWeightedAttribute(NewKey, Coefficients, SourceMetadata);
		}
		else if (StartVertexEntryKey != PCGInvalidEntryKey)
		{
			OutMetadata->InitializeOnSet(NewKey, StartVertexEntryKey, SourceMetadata);
		}
		else if (EndVertexEntryKey != PCGInvalidEntryKey)
		{
			OutMetadata->InitializeOnSet(NewKey, EndVertexEntryKey, SourceMetadata);
		}
	}

	return EntryKeys;
}

TArray<PCGMetadataEntryKey> ComputeEntryKeysBySampling(const UPCGSpatialData* InputData, TConstArrayView<FVector> Positions, UPCGMetadata* OutMetadata)
{
	check(InputData);
	check(OutMetadata);

	TArray<PCGMetadataEntryKey> EntryKeys;
	EntryKeys.Reserve(Positions.Num());

	const FBox DefaultBox(FVector(-1, -1, -1), FVector(1, 1, 1));
	for (const FVector& Position : Positions)
	{
		FPCGPoint OutPoint;
		InputData->SamplePoint(FTransform(Position), DefaultBox, OutPoint, OutMetadata);
		EntryKeys.Add(OutPoint.MetadataEntry);
	}

	return EntryKeys;
}

TArray<PCGMetadataEntryKey> ComputeEntryKeysByNearestSegmentSampling(const UPCGSpatialData* InputData, double InSearchRadius, UE::Geometry::FGeneralPolygon2d& InPoly, const FTransform& InPolyTransform, UPCGMetadata* OutMetadata)
{
	check(InputData);
	check(OutMetadata);

	TArray<PCGMetadataEntryKey> EntryKeys;
	EntryKeys.Reserve(PCGPolygon2DData::PolygonVertexCount(InPoly));

	if (const UPCGSplineData* SplineData = Cast<UPCGSplineData>(InputData))
	{
		ForAllVertices(InPoly, [SplineData, &InPolyTransform, OutMetadata, &EntryKeys](const FVector2D& Vertex, int VertexIndex, int SegmentIndex, int HoleIndex)
		{
			FVector Position = InPolyTransform.TransformPosition(FVector(Vertex, 0.0));

			float InputKey = SplineData->SplineStruct.FindInputKeyClosestToWorldLocation(Position);

			PCGMetadataEntryKey& EntryKey = EntryKeys.Emplace_GetRef(PCGInvalidEntryKey);
			SplineData->WriteMetadataToEntry(InputKey, EntryKey, OutMetadata);

			return true; // always process all points
		});
	}
	else if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(InputData))
	{
		const PCGPointOctree::FPointOctree& Octree = PointData->GetPointOctree();
		const TConstPCGValueRange<FTransform> PointTransforms = PointData->GetConstTransformValueRange();
		const TConstPCGValueRange<PCGMetadataEntryKey> PointEntryKeys = PointData->GetConstMetadataEntryValueRange();
		const UPCGMetadata* SourceMetadata = PointData->ConstMetadata();

		ForAllVertices(InPoly, [&Octree, &PointTransforms, &PointEntryKeys, &InPolyTransform, OutMetadata, &EntryKeys, InSearchRadius, SourceMetadata](const FVector2D& Vertex, int VertexIndex, int SegmentIndex, int HoleIndex)
		{
			TArray<TPair<PCGMetadataEntryKey, float>> WeightedSourceEntries;
			FVector Position = InPolyTransform.TransformPosition(FVector(Vertex, 0.0));
			FBoxCenterAndExtent SearchBounds(Position, FVector(InSearchRadius, InSearchRadius, InSearchRadius));
			const double SqrSearchRadius = FMath::Square(InSearchRadius);

			Octree.FindElementsWithBoundsTest(SearchBounds, [&Position, &PointTransforms, &PointEntryKeys, &WeightedSourceEntries, SqrSearchRadius](const PCGPointOctree::FPointRef& PointRef)
			{
				const double SquaredDistance = FVector::DistSquared(Position, PointTransforms[PointRef.Index].GetLocation());
				if (SquaredDistance < SqrSearchRadius)
				{
					WeightedSourceEntries.Emplace(PointEntryKeys[PointRef.Index], 1.0f - (SquaredDistance / SqrSearchRadius));
				}
			});

			float TotalWeight = 0.0;
			for (TPair<PCGMetadataEntryKey, float>& WeightedEntry : WeightedSourceEntries)
			{
				TotalWeight += WeightedEntry.Value;
			}

			if (TotalWeight > 0)
			{
				for (TPair<PCGMetadataEntryKey, float>& WeightedEntry : WeightedSourceEntries)
				{
					WeightedEntry.Value /= TotalWeight;
				}
			}

			PCGMetadataEntryKey& EntryKey = EntryKeys.Emplace_GetRef(PCGInvalidEntryKey);
			OutMetadata->InitializeOnSet(EntryKey);
			OutMetadata->ComputeWeightedAttribute(EntryKey, WeightedSourceEntries, SourceMetadata);

			return true;
		});
	}
	else
	{
		UE_LOGF(LogPCG, Error, "Unsupported type in PCGPolygon2DProcessingHelpers::ComputeEntryKeysByNearestSegmentSampling");
	}

	return EntryKeys;
}

} // end namespace PCGPolygon2DProcessingHelpers