// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataCommon.h"

#include "Curve/GeneralPolygon2.h"

class UPCGData;
class UPCGMetadata;
class UPCGPolygon2DData;
class UPCGSpatialData;

namespace PCGPolygon2DProcessingHelpers
{
	/** Returns the path representing the input data, projected in the PolyTransform local space. */
	bool GetPath(const UPCGData* InData, const FTransform& PolyTransform, double DiscretizationError, TArray<FVector2D>& OutPath);

	/** Gets the path from the input data and builds a closed polygon using provided bounds. Output polygon will be counterclockwise. */
	bool GetEnclosingPolygonFromPath(const UPCGData* InData, const FTransform& PolyTransform, double DiscretizationError, UE::Geometry::FAxisAlignedBox2d Bounds, UE::Geometry::FGeneralPolygon2d& OutPoly);

	/** Returns the polygons from the input data, normalizing their orientation/transforms based on the input/output transform. */
	[[nodiscard]] TArray<UE::Geometry::FGeneralPolygon2d> GetPolygons(const TArray<const UPCGPolygon2DData*>& InPolygonData, bool& bTransformSet, FTransform& ReferenceTransform);

	/** Adds & normalizes a polygon to a polygon array, conforming or setting the given transform. */
	void AddPolygon(const UPCGPolygon2DData* InPolygonData, TArray<UE::Geometry::FGeneralPolygon2d>& OutPolys, bool& bTransformSet, FTransform& ReferenceTransform);
	void AddPolygon(const UPCGPolygon2DData* InPolygonData, TArray<UE::Geometry::FGeneralPolygon2d>& OutPolys, TArray<TArray<PCGMetadataEntryKey>>* OutEntryKeys, TArray<const UPCGMetadata*>* OutMetadataList, bool& bTransformSet, FTransform& ReferenceTransform);

	/** Visits all vertices in the polygon, starting from the outer. Return false in the InFunc to exit the loop prematurely. */
	void ForAllVertices(UE::Geometry::FGeneralPolygon2d& InPoly, TFunctionRef<bool(const FVector2D& /*VertexPosition*/, int /*VertexIndex*/, int /*SegmentIndex*/, int /*HoleIndex*/)> InFunc);
	
	/** Finds the nearest segment in any of the polygons and returns it's information. Note that the Out variables will be written to only if there is a distance found that's smaller than InitMinDistance. Returns true if the vertex found is inside the given tolerance (short-circuits the rest of the visit.) */
	bool DistanceSquaredToPolygons(const FVector2D& InVertex, TArrayView<const UE::Geometry::FGeneralPolygon2d> InPolys, double Tolerance, int& OutPolyIndex, int& OutSegmentIndex, int& OutHoleIndex, double& OutT, double& OutMinDistance, double InitMinDistance = DBL_MAX);

	using MetadataInfo = TPair<const UPCGMetadata*, const TArray<PCGMetadataEntryKey>*>;
	using PolyToMetadataInfoMap = TMap<const UE::Geometry::FGeneralPolygon2d*, MetadataInfo>;
	
	/** Builds a map from polygon pointer to metadata pointer & entry keys */
	void AddToPolyMetadataInfoMap(PolyToMetadataInfoMap& OutMap, const UE::Geometry::FGeneralPolygon2d& Poly, const UPCGMetadata* Metadata, const TArray<PCGMetadataEntryKey>& EntryKeys);
	void AddToPolyMetadataInfoMap(PolyToMetadataInfoMap& OutMap, const TArray<UE::Geometry::FGeneralPolygon2d>& Polys, const TArray<const UPCGMetadata*>& Metadata, const TArray<TArray<PCGMetadataEntryKey>>& EntryKeys);

	struct VertexToSourcePolygonMapping
	{
		const UE::Geometry::FGeneralPolygon2d* Poly = nullptr;
		int SegmentIndex = -1;
		int HoleIndex = -1;
		int StartVertexIndex = -1;
		int EndVertexIndex = -1;
		double Ratio = 0.0;
	};

	using PolygonVertexMapping = TArray<VertexToSourcePolygonMapping>;
	
	/** Builds a mapping for all vertices of all polys to their nearest vertex/edge in the primary & secondary polygons.
	* Warning: this is an expensive operation.
	*/
	[[nodiscard]] TArray<PolygonVertexMapping> BuildVertexMapping(TArray<UE::Geometry::FGeneralPolygon2d>& Polys, TConstArrayView<UE::Geometry::FGeneralPolygon2d> PrimaryPolygons, TConstArrayView<UE::Geometry::FGeneralPolygon2d> SecondaryPolygons);

	/** Collates vertex mapping information to allocate metadata entries with proper interpolation. */
	[[nodiscard]] TArray<PCGMetadataEntryKey> ComputeEntryKeysAndInitializeAttributes(UPCGMetadata* OutMetadata, const PolygonVertexMapping& InVertexMapping, const PolyToMetadataInfoMap& InInfoMap);

	/** Computes entry keys from given positions and the original input data (spline or points), on a sampling basis. Assumes that the positions are "on" the existing edges. */
	[[nodiscard]] TArray<PCGMetadataEntryKey> ComputeEntryKeysBySampling(const UPCGSpatialData* InputData, TConstArrayView<FVector> Positions, UPCGMetadata* OutMetadata);

	/** Computes entry keys for polygon vertices by sampling against nearest segments on spatial data (spline or points). */
	[[nodiscard]] TArray<PCGMetadataEntryKey> ComputeEntryKeysByNearestSegmentSampling(const UPCGSpatialData* InputData, double InSearchRadius, UE::Geometry::FGeneralPolygon2d& InPoly, const FTransform& InPolyTransform, UPCGMetadata* OutMetadata);

}