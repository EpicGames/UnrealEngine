// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGPropertyHelpers.h"

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Metadata/PCGMetadata.h"

#include "StructUtils/UserDefinedStruct.h"
#include "UObject/Field.h"
#include "UObject/UnrealType.h"
#include "VerseVM/VVMVerseClass.h"

#define LOCTEXT_NAMESPACE "PCGPropertyHelpers"

namespace PCGPropertyHelpers
{
	static constexpr uint64 ExcludePropertyFlags = CPF_DisableEditOnInstance;
	static constexpr uint64 IncludePropertyFlags = CPF_BlueprintVisible;

	static constexpr uint64 VerseIncludePropertyFlags = CPF_BlueprintReadOnly | CPF_Edit;

	/**
	* Expands container locations to their contents when the property passed in is an array or a set.
	* This is useful to allow extraction downstream of properties inside of arrays/sets and also to generate the list of addresses/values to look at
	* when extracting the values to the attribute set.
	* 
	* @param InContainerProperty Property that drives the container expansion.
	* @param InContainers        Container locations to expand
	* @param OutContainers       Expanded container locations. Expected to be a different array than InContainers.
	*/
	template<typename ContainerProperty, typename FirstArrayType, typename SecondArrayType>
	void ExpandContainers(const ContainerProperty* InContainerProperty, const FirstArrayType& InContainers, SecondArrayType& OutContainers)
	{
		check(OutContainers.IsEmpty() && InContainerProperty);

		static_assert(std::is_same_v<ContainerProperty, FArrayProperty> || std::is_same_v<ContainerProperty, FSetProperty>);
		using FScriptContainerHelper = std::conditional_t<std::is_same_v<ContainerProperty, FArrayProperty>, FScriptArrayHelper_InContainer, FScriptSetHelper_InContainer>;

		for (const void* Container : InContainers)
		{
			FScriptContainerHelper Helper(InContainerProperty, Container);
			int32 Offset = OutContainers.Num();
			OutContainers.SetNumUninitialized(OutContainers.Num() + Helper.Num());
			for (int32 DynamicIndex = 0; DynamicIndex < Helper.Num(); ++DynamicIndex)
			{
				OutContainers[Offset + DynamicIndex] = Helper.GetElementPtr(DynamicIndex);
			}
		}
	}
	
	struct FExtractPropertyChainParams
	{
		// Struct/Class for the current container
		const UStruct* CurrentClass = nullptr;
		
		// Property name to look for in the container class.
		FName CurrentName = NAME_None;
		
		// List of property names to continue extracting at a deeper level.
		TArrayView<const FString> NextNames;
		
		// Discard properties that are not visibile in Blueprint
		bool bNeedsToBeVisible = false;
		
		// Raw addresses for the containers. Will be written to at each recursive call.
		TArray<const void*>* OutContainers = nullptr;
		
		// Optional context used for logging.
		FPCGContext* OptionalContext = nullptr;
		
		// If we should log or not.
		bool bQuiet = false;
	};
	

	/**
	* Recursive function to go down the property chain to find the property and its container addresses.
	* @returns The last property of the chain (and its container address is in OutContainer)
	*/
	const FProperty* ExtractPropertyChain(FExtractPropertyChainParams& Params)
	{
		check(Params.CurrentClass && Params.OutContainers);

		const FProperty* Property = PCGPropertyHelpers::FindPropertyByName(Params.CurrentClass, Params.CurrentName);

		if (!Property)
		{
			if (!Params.bQuiet)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("PropertyDoesNotExist", "Property '{0}' does not exist in {1}."), FText::FromName(Params.CurrentName), FText::FromName(Params.CurrentClass->GetFName())), Params.OptionalContext);
			}

			return nullptr;
		}

		// Make sure the property is visible, if requested
		bool bVisible = true;
		if (Params.bNeedsToBeVisible)
		{
			if (Params.CurrentClass->IsA<UVerseClass>())
			{
				if (!Property->HasAnyPropertyFlags(VerseIncludePropertyFlags))
				{
					bVisible = false;
				}
			}
			else if (!Property->HasAnyPropertyFlags(IncludePropertyFlags))
			{
				bVisible = false;
			}
		}

		if (!bVisible)
		{
			if (!Params.bQuiet)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("PropertyExistsButNotVisible", "Property '{0}' does exist in {1}, but is not visible."), FText::FromName(Params.CurrentName), FText::FromName(Params.CurrentClass->GetFName())), Params.OptionalContext);
			}

			return nullptr;
		}

		auto ExtractContainers = [&OutContainers = *Params.OutContainers](UStruct*& NextClass, const auto* ContainerProperty, const FProperty* InnerProperty) -> bool
		{
			bool bPropertyNotExtractable = false;
			const FObjectProperty* InnerObjectProperty = nullptr;
			
			if (const FStructProperty* InnerStructProperty = CastField<FStructProperty>(InnerProperty))
			{
				NextClass = InnerStructProperty->Struct;
			}
			else if ((InnerObjectProperty = CastField<FObjectProperty>(InnerProperty)) != nullptr)
			{
				NextClass = InnerObjectProperty->PropertyClass;
			}
			else
			{
				bPropertyNotExtractable = true;
			}

			// If contents of the container are extractable, do so now by replacing the container entry (e.g. the array/set) with the pointer to its contents
			if (!bPropertyNotExtractable)
			{
				TArray<const void*> Subcontainers;
				PCGPropertyHelpers::ExpandContainers(ContainerProperty, OutContainers, Subcontainers);

				// If we have an object, we also need to apply the object indirection on the Subcontainers
				if (InnerObjectProperty)
				{
					for (const void*& OutContainer : Subcontainers)
					{
						OutContainer = OutContainer ? InnerObjectProperty->GetObjectPropertyValue_InContainer(OutContainer) : nullptr;
					}
				}

				OutContainers = MoveTemp(Subcontainers);
			}
			
			return bPropertyNotExtractable;
		};

		if (!Params.NextNames.IsEmpty())
		{
			UStruct* NextClass = nullptr;
			bool bPropertyNotExtractable = false;

			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				NextClass = StructProperty->Struct;
				for (const void*& OutContainer : *Params.OutContainers)
				{
					OutContainer = OutContainer ? StructProperty->ContainerPtrToValuePtr<void>(OutContainer) : nullptr;
				}
			}
			else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				NextClass = ObjectProperty->PropertyClass;
				for (const void*& OutContainer : *Params.OutContainers)
				{
					OutContainer = OutContainer ? ObjectProperty->GetObjectPropertyValue_InContainer(OutContainer) : nullptr;
				}
			}
			else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				bPropertyNotExtractable = ExtractContainers(NextClass, ArrayProperty, ArrayProperty->Inner);
			}
			else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
			{
				bPropertyNotExtractable = ExtractContainers(NextClass, SetProperty, SetProperty->ElementProp);
			}
			else
			{
				bPropertyNotExtractable = true;
			}
			
			if(bPropertyNotExtractable)
			{
				if (!Params.bQuiet)
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("PropertyIsNotExtractable", "Property '{0}' does exist in {1}, but is not extractable."), FText::FromName(Params.CurrentName), FText::FromName(Params.CurrentClass->GetFName())), Params.OptionalContext);
				}

				return nullptr;
			}
			
			FExtractPropertyChainParams NextParams = Params;
			NextParams.CurrentClass = NextClass;
			NextParams.CurrentName = FName(Params.NextNames[0]);
			NextParams.NextNames = Params.NextNames.RightChop(1);

			return ExtractPropertyChain(NextParams);
		}
		else
		{
			return Property;
		}
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FExtractorParameters::FExtractorParameters(const void* InContainer, const UStruct* InClass, const FPCGAttributePropertySelector& InPropertySelector, FName InOutputAttributeName, bool bInPropertyNeedsToBeVisible)
		: Container(InContainer), Class(InClass), OutputAttributeName(InOutputAttributeName), bPropertyNeedsToBeVisible(bInPropertyNeedsToBeVisible)
	{
		PropertySelectors.Add(InPropertySelector);
	}

	FExtractorParameters::FExtractorParameters(const void* InContainer, const UStruct* InClass, const TArray<FPCGAttributePropertySelector>& InPropertySelectors, FName InOutputAttributeName, bool bInPropertyNeedsToBeVisible)
		: Container(InContainer), Class(InClass), PropertySelectors(InPropertySelectors), OutputAttributeName(InOutputAttributeName), bPropertyNeedsToBeVisible(bInPropertyNeedsToBeVisible)
	{}

	FExtractorParameters::FExtractorParameters(const void* InContainer, const UStruct* InClass, const FString& InPropertySelectorString, FName InOutputAttributeName, bool bInPropertyNeedsToBeVisible)
		: Container(InContainer), Class(InClass), OutputAttributeName(InOutputAttributeName), bPropertyNeedsToBeVisible(bInPropertyNeedsToBeVisible)
	{
		const TArray<FString> PropertySelectorStrings = PCGHelpers::GetStringArrayFromCommaSeparatedList(InPropertySelectorString);

		PropertySelectors.Reserve(PropertySelectorStrings.Num());
		for (int StringIndex = 0; StringIndex < PropertySelectorStrings.Num(); ++StringIndex)
		{
			PropertySelectors.Add(FPCGAttributePropertySelector::CreateSelectorFromString(PropertySelectorStrings[StringIndex]));
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

EPCGMetadataTypes PCGPropertyHelpers::GetMetadataTypeFromProperty(const FProperty* InProperty)
{
	return PCGAttributeAccessorHelpers::GetMetadataTypeForProperty(InProperty);
}

FPropertyBagPropertyDesc PCGPropertyHelpers::CreatePropertyBagDescWithMetadataType(FName InPropertyName, const EPCGMetadataTypes Type)
{
	return CreatePropertyBagDescWithMetadataType(InPropertyName, FPCGMetadataAttributeDesc{.ValueType = Type});
}

FPropertyBagPropertyDesc PCGPropertyHelpers::CreatePropertyBagDescWithMetadataType(FName InPropertyName, const FPCGMetadataAttributeDesc& AttributeDesc)
{
	auto MetadataTypeToPropertyType = [](const EPCGMetadataTypes InType, const UObject* InTypeObject, EPropertyBagPropertyType& OutType, TObjectPtr<const UObject>& OutTypeObject)
	{
		OutType = EPropertyBagPropertyType::Struct;
		OutTypeObject = InTypeObject;

		switch (InType)
		{
			// Simple Types
			case EPCGMetadataTypes::Float:
				OutType = EPropertyBagPropertyType::Float;
				break;
			case EPCGMetadataTypes::Double:
				OutType = EPropertyBagPropertyType::Double;
				break;
			case EPCGMetadataTypes::Integer32:
				OutType = EPropertyBagPropertyType::Int32;
				break;
			case EPCGMetadataTypes::Integer64:
				OutType = EPropertyBagPropertyType::Int64;
				break;
			case EPCGMetadataTypes::String:
				OutType = EPropertyBagPropertyType::String;
				break;
			case EPCGMetadataTypes::Boolean:
				OutType = EPropertyBagPropertyType::Bool;
				break;
			case EPCGMetadataTypes::Name:
				OutType = EPropertyBagPropertyType::Name;
				break;
			// Value Type Objects - PropertyType is Struct by default.
			case EPCGMetadataTypes::SoftObjectPath:
				OutTypeObject = TBaseStructure<FSoftObjectPath>::Get();
				break;
			case EPCGMetadataTypes::SoftClassPath:
				OutTypeObject = TBaseStructure<FSoftClassPath>::Get();
				break;
			case EPCGMetadataTypes::Vector:
				OutTypeObject = TBaseStructure<FVector>::Get();
				break;
			case EPCGMetadataTypes::Vector2:
				OutTypeObject = TBaseStructure<FVector2d>::Get();
				break;
			case EPCGMetadataTypes::Vector4:
				OutTypeObject = TBaseStructure<FVector4>::Get();
				break;
			case EPCGMetadataTypes::Transform:
				OutTypeObject = TBaseStructure<FTransform>::Get();
				break;
			case EPCGMetadataTypes::Quaternion:
				OutTypeObject = TBaseStructure<FQuat>::Get();
				break;
			case EPCGMetadataTypes::Rotator:
				OutTypeObject = TBaseStructure<FRotator>::Get();
				break;
			case EPCGMetadataTypes::Struct:
				// Nothing to do, already set up
				break;
			case EPCGMetadataTypes::Object:
				OutType = EPropertyBagPropertyType::Object;
				break;
			case EPCGMetadataTypes::SoftObject:
				OutType = EPropertyBagPropertyType::SoftObject;
				break;
			case EPCGMetadataTypes::Class:
				OutType = EPropertyBagPropertyType::Class;
				break;
			case EPCGMetadataTypes::SoftClass:
				OutType = EPropertyBagPropertyType::SoftClass;
				break;
			case EPCGMetadataTypes::Enum:
				OutType = EPropertyBagPropertyType::Enum;
				break;
			default:
				OutType = EPropertyBagPropertyType::None;
				break;
		}
	};

	FPropertyBagPropertyDesc Result{};
	Result.Name = InPropertyName;
	MetadataTypeToPropertyType(AttributeDesc.ValueType, AttributeDesc.ValueTypeObject, Result.ValueType, Result.ValueTypeObject);
	MetadataTypeToPropertyType(AttributeDesc.KeyType, AttributeDesc.KeyTypeObject, Result.KeyType, Result.KeyTypeObject);

	for (const EPCGMetadataAttributeContainerTypes ContainerType : AttributeDesc.ContainerTypes)
	{
		EPropertyBagContainerType PropertyBagContainerType = EPropertyBagContainerType::None;
		switch (ContainerType)
		{
		case EPCGMetadataAttributeContainerTypes::Array:
			PropertyBagContainerType = EPropertyBagContainerType::Array;
			break;
		case EPCGMetadataAttributeContainerTypes::Set:
			PropertyBagContainerType = EPropertyBagContainerType::Set;
			break;
		case EPCGMetadataAttributeContainerTypes::Map:
			PropertyBagContainerType = EPropertyBagContainerType::Map;
			break;
		default:
			break;
		}

		if (PropertyBagContainerType != EPropertyBagContainerType::None)
		{
			Result.ContainerTypes.Add(PropertyBagContainerType);
		}
	}

	return Result;
}

UPCGParamData* PCGPropertyHelpers::ExtractPropertyAsAttributeSet(const PCGPropertyHelpers::FExtractorParameters& Parameters, FPCGContext* OptionalContext, TSet<FSoftObjectPath>* OptionalObjectTraversed, bool bQuiet)
{
	check(Parameters.Container && Parameters.Class);

	TConstArrayView<FPCGAttributePropertySelector> PropertySelectors = MakeArrayView(Parameters.PropertySelectors);
	static const FPCGAttributePropertySelector DefaultSelector{};

	// To keep previous behavior where no selectors meant a single default selector. 
	if (PropertySelectors.IsEmpty())
	{
		PropertySelectors = MakeArrayView(&DefaultSelector, 1);
	}

	UPCGParamData* ParamData = nullptr;
	UPCGMetadata* Metadata = nullptr;
	TArray<PCGMetadataEntryKey> Entries;
	bool bValidOperation = true;
	bool bIgnoreOutputAttributeName = (PropertySelectors.Num() > 1);
	
	const bool bContainerIsObject = Cast<UClass>(Parameters.Class) != nullptr;
	const bool bContainerIsStruct = Cast<UScriptStruct>(Parameters.Class) != nullptr;

	for (const FPCGAttributePropertySelector& PropertySelector : PropertySelectors)
	{
		TArray<const void*> Containers = { Parameters.Container };
		const FProperty* Property = nullptr;
		const FName PropertyName = PropertySelector.GetName();
		const bool bExtractRoot = PropertyName == NAME_None;
		// If Name is none, extract the container as-is, using Parameters.Class, otherwise, extract the chain.
		if (!bExtractRoot)
		{
			FExtractPropertyChainParams Params =
			{
				.CurrentClass = Parameters.Class,
				.CurrentName = PropertyName,
				.NextNames = PropertySelector.GetExtraNames(),
				.bNeedsToBeVisible = Parameters.bPropertyNeedsToBeVisible,
				.OutContainers = &Containers,
				.OptionalContext = OptionalContext,
				.bQuiet = bQuiet,
			};

			Property = ExtractPropertyChain(Params);
			if (!Property)
			{
				return nullptr;
			}
		}

		const FProperty* OriginalProperty = Property;

		// If the property is an array/set, we will work on the underlying property, only if we are flattening, and extract each element as an entry in the param data
		const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
		const FSetProperty* SetProperty = CastField<FSetProperty>(Property);
		const bool bShouldFlattenArrays = Parameters.ContainerExtractorBehavior != EPCGContainerExtractorBehavior::NoFlattenLast;
		const bool bPropertyIsAContainer = ArrayProperty || SetProperty;
		
		if (ArrayProperty && bShouldFlattenArrays)
		{
			Property = ArrayProperty->Inner;
		}
		else if (SetProperty && bShouldFlattenArrays)
		{
			Property = SetProperty->ElementProp;
		}

		using ExtractablePropertyTuple = TTuple<FString /*PropertyName*/, const FProperty* /*PropertyToExtract*/, const FProperty* /*ContainerProperty*/>;
		TArray<ExtractablePropertyTuple> ExtractableProperties;
		
		// Keep track if the extracted property is an object or not
		const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property);
		const FStructProperty* StructProperty = CastField<FStructProperty>(Property);

		bool bShouldExtractObject = Parameters.ObjectExtractorBehavior == EPCGObjectExtractorBehavior::Extract && ((bContainerIsObject && !Property) || ObjectProperty);

		bool bShouldExtractStruct = false;
		if (Parameters.StructExtractorBehavior == EPCGStructExtractorBehavior::ExtractRootOnly)
		{
			// LEGACY: Force extraction if the property is not supported by accessors, or if the property name is none
			bShouldExtractStruct = (bContainerIsStruct && !Property) || (StructProperty != nullptr && !PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(Property));
		}
		else if (Parameters.StructExtractorBehavior == EPCGStructExtractorBehavior::Extract)
		{
			bShouldExtractStruct = (bContainerIsStruct && !Property) || (StructProperty != nullptr);
		}
		
		// If we have an array as the current property, we can't extract if we are not flattening the array.		
		const bool bShouldExtract = (bShouldFlattenArrays || !bPropertyIsAContainer) && (bExtractRoot || bShouldExtractStruct || bShouldExtractObject);

		if (bShouldExtract)
		{
			const UStruct* UnderlyingClass = nullptr;
			const FProperty* ContainerProperty = nullptr;

			if (bExtractRoot)
			{
				UnderlyingClass = Parameters.Class;
			}
			else if (StructProperty)
			{
				UnderlyingClass = StructProperty->Struct;
				ContainerProperty = Property;
			}
			else if (ObjectProperty)
			{
				UnderlyingClass = ObjectProperty->PropertyClass;
				ContainerProperty = Property;
			}

			check(UnderlyingClass);

			// Re-use code from overridable params
			// Limit ourselves to not recurse into more structs.
			PCGSettingsHelpers::FPCGGetAllOverridableParamsConfig Config;
			Config.bUseSeed = true;
			Config.bExcludeSuperProperties = true;
			Config.bExtractArrays = Parameters.ContainerExtractorBehavior != EPCGContainerExtractorBehavior::FlattenLastAndDiscardNested;
			Config.bDiscardLeafStructProperty = Parameters.StructExtractorBehavior == EPCGStructExtractorBehavior::ExtractRootOnly;
			Config.MaxStructDepth = 0;
			// Can only get exposed properties and visible if requested
			if (Parameters.bPropertyNeedsToBeVisible)
			{
				Config.ShouldKeepPropertyFunc = [](const FProperty* InProperty, int32) -> bool { return InProperty && InProperty->HasAnyPropertyFlags(IncludePropertyFlags) && !InProperty->HasAnyPropertyFlags(ExcludePropertyFlags); };
			}
			TArray<FPCGSettingsOverridableParam> AllChildProperties = PCGSettingsHelpers::GetAllOverridableParams(UnderlyingClass, Config);

			for (const FPCGSettingsOverridableParam& Param : AllChildProperties)
			{
				if (ensure(!Param.PropertiesNames.IsEmpty()) && ensure(!Param.Properties.IsEmpty()) && ensure(Param.Properties[0]))
				{
					ExtractableProperties.Emplace(Param.PropertiesNames[0].ToString(), Param.Properties[0], ContainerProperty);
				}
			}
		}
		else
		{
			// For non struct/object, there is just a single property to extract with no shenanigans for address indirection.
			const bool bIsSourceName = Parameters.OutputAttributeName == PCGMetadataAttributeConstants::SourceNameAttributeName || Parameters.OutputAttributeName == PCGMetadataAttributeConstants::SourceAttributeName;
			FString AttributeName;
			if (bIgnoreOutputAttributeName || bIsSourceName)
			{
				// Make sure that the name is the authored name
				if (const UStruct* StructOwner = Property->GetOwnerStruct())
				{
					AttributeName = StructOwner->GetAuthoredNameForField(OriginalProperty);
				}

				if (AttributeName.IsEmpty())
				{
					AttributeName = Property->GetName();
				}
			}
			else
			{
				AttributeName = Parameters.OutputAttributeName.ToString();
			}

			ExtractableProperties.Emplace(std::move(AttributeName), Property, nullptr);
		}

		if (ExtractableProperties.IsEmpty())
		{
			if (!bQuiet)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("NoPropertiesFound", "No properties found to extract"), OptionalContext);
			}

			return nullptr;
		}

		// Before we need to compute all the addresses for each entry in our array/set (or just a single entry if there is no array/set)
		TArray<const void*, TInlineAllocator<16>> ExpandedContainers;
		TArrayView<const void*> ElementAddresses;
		if (SetProperty && bShouldFlattenArrays)
		{
			ExpandContainers(SetProperty, Containers, ExpandedContainers);
			ElementAddresses = MakeArrayView(ExpandedContainers);
		}
		else if (ArrayProperty && bShouldFlattenArrays)
		{
			ExpandContainers(ArrayProperty, Containers, ExpandedContainers);
			ElementAddresses = MakeArrayView(ExpandedContainers);
		}
		else
		{
			ElementAddresses = MakeArrayView(Containers);
		}
		
		// Validation that we have no null pointers
		if (Algo::AnyOf(ElementAddresses, [](const void* Addr) { return Addr == nullptr; }))
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("NullPtrContainer", "Some resolved objects were not assigned (null pointers), some results are discarded"), OptionalContext);
			return nullptr;
		}

		if (!ParamData)
		{
			// From there, we should be able to create the data.
			ParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(OptionalContext);
			check(!Metadata);
			Metadata = ParamData->MutableMetadata();
			check(Metadata);

			// Allocate entries
			TArray<PCGMetadataEntryKey, TInlineAllocator<16>> ParentEntries;
			ParentEntries.Init(PCGInvalidEntryKey, ElementAddresses.Num());
			Entries = Metadata->AddEntries(ParentEntries);
		}
		else // Pre-existing data, we're expecting the same cardinality
		{
			if (Entries.Num() != ElementAddresses.Num())
			{
				if (!bQuiet)
				{
					PCGLog::LogErrorOnGraph(LOCTEXT("InvalidCardinality", "Unable to extract because some properties are of mismatched sizes"), OptionalContext);
				}

				return nullptr;
			}
		}

		for (ExtractablePropertyTuple& ExtractableProperty : ExtractableProperties)
		{
			FString AttributeNameStr = ExtractableProperty.Get<0>();
			FName AttributeName = NAME_None;
			const FProperty* FinalProperty = ExtractableProperty.Get<1>();
			const FProperty* ContainerProperty = ExtractableProperty.Get<2>();

			// Make sure the Attribute name is sanitized, to prevent cases where property names have unsupported characters.
			FPCGMetadataAttributeBase::SanitizeName(AttributeNameStr);
			if (Parameters.bStrictSanitizeOutputAttributeNames)
			{
				AttributeName = MakeObjectNameFromDisplayLabel(AttributeNameStr, NAME_None);
			}
			else
			{
				AttributeName = *AttributeNameStr;
			}

			TUniquePtr<const IPCGAttributeAccessor> PropertyAccessor;
			
			auto CreateAccessor = [ContainerProperty, FinalProperty](bool bUseGenericAccessors)
			{
				if (ContainerProperty)
				{
					return PCGAttributeAccessorHelpers::CreatePropertyChainAccessor({ContainerProperty, FinalProperty}, bUseGenericAccessors);
				}
				else
				{
					return PCGAttributeAccessorHelpers::CreatePropertyAccessor(FinalProperty, bUseGenericAccessors);
				}
			};
			
			// First try with the old accessors
			PropertyAccessor = CreateAccessor(/*bUseGenericAccessors=*/false);
			if (!PropertyAccessor)
			{
				PropertyAccessor = CreateAccessor(/*bUseGenericAccessors=*/true);
			}

			if (!PropertyAccessor)
			{
				if (!bQuiet)
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ErrorCreatingPropertyAccessor", "Error while creating an attribute for property '{0}'. The property type is not supported by PCG."), FText::FromString(FinalProperty->GetName())), OptionalContext);
				}

				bValidOperation = false;
				break;
			}

			const FPCGAttributeAccessorKeysGenericPtrs PropertyAccessorKeys{ElementAddresses};
			const FPCGAttributePropertySelector AttributeSelector = FPCGAttributePropertySelector::CreateAttributeSelector(AttributeName);
			
			PCGAttributeAccessorHelpers::FPCGCreateAccessorWithAttributeCreationParams AttributeCreationParams =
			{
				.InData = ParamData,
				.InSelector = &AttributeSelector,
				.InMatchingAccessor = PropertyAccessor.Get(),
				.InMatchingKeysForDefaultValue = PropertyAccessorKeys.GetNum() > 0 ? &PropertyAccessorKeys : nullptr
			};
			
			const TUniquePtr<IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateAccessorWithAttributeCreation(AttributeCreationParams);
			FPCGAttributeAccessorKeysEntries AttributeKeys{TArrayView<PCGMetadataEntryKey>(Entries)};

			if (!AttributeAccessor)
			{
				if (!bQuiet)
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ErrorCreatingAttribute", "Error while creating an attribute for property '{0}'. Attribute creation failed."), FText::FromString(FinalProperty->GetName())), OptionalContext);
				}

				bValidOperation = false;
				break;
			}

			PCGMetadataElementCommon::FCopyFromAccessorToAccessorParams Params =
			{
				.IterationCount = PCGMetadataElementCommon::FCopyFromAccessorToAccessorParams::Out,
				.InAccessor = PropertyAccessor.Get(),
				.InKeys = &PropertyAccessorKeys,
				.OutAccessor = AttributeAccessor.Get(),
				.OutKeys = &AttributeKeys,
				.Flags = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible
			};
			
			if (!PCGMetadataElementCommon::CopyFromAccessorToAccessor(Params))
			{
				bValidOperation = false;
				break;
			}

			if (bShouldExtract && OptionalObjectTraversed && ObjectProperty)
			{
				TArray<FSoftObjectPath> ObjectPaths;
				ObjectPaths.SetNum(ElementAddresses.Num());
				if (PropertyAccessor->GetRange<FSoftObjectPath>(ObjectPaths, 0, PropertyAccessorKeys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
				{
					for (const FSoftObjectPath& ObjectPath : ObjectPaths)
					{
						const UObject* Object = ObjectPath.ResolveObject();
						if (IsValid(Object))
						{
							OptionalObjectTraversed->Add(Object);
						}
					}
				}

			}
		}

		if (!bValidOperation)
		{
			break;
		}
	}

	return bValidOperation ? ParamData : nullptr;
}

TArray<FInstancedStruct> PCGPropertyHelpers::ExtractAttributeSetAsArrayOfStructs(const UPCGParamData* InParamData, const UScriptStruct* InStruct, const TMap<FName, TTuple<FName, bool>>* OptionalNameMapping, FPCGContext* OptionalContext)
{
	if (!ensure(InParamData && InStruct))
	{
		return {};
	}
	
	const int32 NumElements = InParamData->ConstMetadata()->GetItemCountForChild();
	
	if (NumElements == 0)
	{
		return {};
	}
	
	TArray<FInstancedStruct> Result;
	TArray<void*> ResultContainers;
	Result.Reserve(NumElements);
	ResultContainers.Reserve(NumElements);
	for (int32 i = 0; i < NumElements; i++)
	{
		FInstancedStruct& InstancedStruct = Result.Emplace_GetRef(InStruct);
		ResultContainers.Add(InstancedStruct.GetMutableMemory());
	}
	
	return ExtractAttributeSetToContainers(InParamData, InStruct, ResultContainers, OptionalNameMapping, OptionalContext) ? Result : TArray<FInstancedStruct>{};
}

bool PCGPropertyHelpers::ExtractAttributeSetToContainers(const UPCGParamData* InParamData, const UStruct* InStruct, TArrayView<void*> InContainers, const TMap<FName, TTuple<FName, bool>>* OptionalNameMapping, FPCGContext* OptionalContext)
{
	if (!ensure(InParamData && InStruct))
	{
		return false;
	}
	
	if (InContainers.IsEmpty() || InParamData->ConstMetadata()->GetItemCountForChild() != InContainers.Num())
	{
		return false;
	}
	
	PCGSettingsHelpers::FPCGGetAllOverridableParamsConfig Config;
	Config.bExcludeSuperProperties = false;
	Config.ShouldKeepPropertyFunc = [](const FProperty* InProperty, int32) { return InProperty && InProperty->HasAnyPropertyFlags(CPF_Edit); };
	// @todo_pcg: Support arrays
	Config.bExtractArrays = false;
	Config.MaxContainersNum = 1; // Do not support more than 1 container deep.
	TArray<FPCGSettingsOverridableParam> AllProperties = PCGSettingsHelpers::GetAllOverridableParams(InStruct, Config);
	
	// Keep track of all the matched attributes and ambiguities.
	// We can have an ambiguity that is resolved by a mapping later on.
	TSet<FName> MatchedAttributes;
	TMap<FName, TArray<FString>> Ambiguities;

	for (FPCGSettingsOverridableParam& OverridableProperty : AllProperties)
	{
		const FProperty* CurrProperty = !OverridableProperty.Properties.IsEmpty() ? OverridableProperty.Properties.Last() : nullptr;
		
		if (!CurrProperty || !PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(CurrProperty))
		{
			continue;
		}
		
		FString PropertyPath = OverridableProperty.GetPropertyPath();
		const FName SimplePropertyName = OverridableProperty.PropertiesNames.Last();
		// Property can be the full path, or the last property name
		// In case of a name clash, if the last property name matches, we'll throw a warning as there is ambiguity.
		TArray<FName, TInlineAllocator<2>> TentativePropertyNames = {*PropertyPath};
		TentativePropertyNames.AddUnique(SimplePropertyName);

		TUniquePtr<const IPCGAttributeAccessor> Accessor;
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys;
		bool bCanBeDefaulted = true;
		bool bAmbiguityDetected = false;
		
		// Pass as ref to modify the values and have the real value we checked for the error message.
		FName MatchedPropertyName = NAME_None;
		for (FName& PropertyName: TentativePropertyNames)
		{
			const TTuple<FName, bool>* It = OptionalNameMapping ? OptionalNameMapping->Find(PropertyName) : nullptr;
			const bool bHasNameMapping = It != nullptr;
			if (bHasNameMapping)
			{
				PropertyName = It->Get<0>();
				bCanBeDefaulted = It->Get<1>();
			}

			FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(PropertyName);
			Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InParamData, Selector);
			Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InParamData, Selector);

			if (Accessor && Keys)
			{
				// We have an ambiguity if:
				// 1. The property has a name clash (2 properties extracted have the same last name).
				// 2. We didn't do any name matching for this one.
				// 3. We are currently evaluating the last name.
				if (OverridableProperty.bHasNameClash && !bHasNameMapping && PropertyName != PropertyPath)
				{
					Ambiguities.FindOrAdd(SimplePropertyName).Add(PropertyPath);
					bAmbiguityDetected = true;
				}

				MatchedPropertyName = PropertyName;
				break;
			}
		}

		if (bAmbiguityDetected)
		{
			continue;
		}

		if (!Accessor || !Keys)
		{
			if (!bCanBeDefaulted)
			{
				TArray<FText> TentativePropertyNamesAsText;
				TentativePropertyNamesAsText.Reserve(TentativePropertyNames.Num());
				Algo::Transform(TentativePropertyNames, TentativePropertyNamesAsText, FText::FromName);
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("FailToFindProperty", "Failed to find attribute in param data for overrides. Tested {0}"), FText::Join(INVTEXT(","), TentativePropertyNamesAsText)), OptionalContext);
				return false;
			}
			else
			{
				continue;
			}
		}

		MatchedAttributes.Add(MatchedPropertyName);

		TUniquePtr<IPCGAttributeAccessor> StructAccessor = PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(MoveTemp(OverridableProperty.Properties));
		FPCGAttributeAccessorKeysGenericPtrs StructKeys(InContainers);

		PCGMetadataElementCommon::FCopyFromAccessorToAccessorParams Params =
		{
			.IterationCount = PCGMetadataElementCommon::FCopyFromAccessorToAccessorParams::In,
			.InAccessor = Accessor.Get(),
			.InKeys = Keys.Get(),
			.OutAccessor = StructAccessor.Get(),
			.OutKeys = &StructKeys,
			.Flags = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible,
		};
		
		if (!PCGMetadataElementCommon::CopyFromAccessorToAccessor(Params))
		{
			return false;
		}
	}
	
	// Log ambiguities
	for (const auto& [PropertyName, AmbiguousProperties] : Ambiguities)
	{
		if (!MatchedAttributes.Contains(PropertyName))
		{
			for (const FString& PropertyPath : AmbiguousProperties)
			{
				PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("AmbiguityProperty", "Found attribute {0} in param data for overrides, but the property {1} need to be specified with the full path as there are multiple properties that are named {0}."), 
					FText::FromName(PropertyName), FText::FromString(PropertyPath)), OptionalContext);
			}
		}
	}

	return true;
}

const FProperty* PCGPropertyHelpers::FindPropertyByName(const UStruct* InStruct, const FName InName)
{
	if (!InStruct)
	{
		return nullptr;
	}
	else if (InStruct->IsA<UUserDefinedStruct>() || InStruct->IsA<UVerseClass>())
	{
		return FindPropertyByNameEx(InStruct, InName);
	}
	else
	{
		return FindFProperty<FProperty>(InStruct, InName);
	}
}

const FProperty* PCGPropertyHelpers::FindPropertyByNameEx(const UStruct* InStruct, const FName InName)
{
	if (!InStruct)
	{
		return nullptr;
	}

	TArray<FName, TInlineAllocator<2>> NamesToLookFor = { InName };

	if (!InName.IsValidXName())
	{
		NamesToLookFor.Add(MakeObjectNameFromDisplayLabel(InName.ToString(), NAME_None));
	}

	for (TFieldIterator<const FProperty> PropIt(InStruct, EFieldIterationFlags::IncludeSuper); PropIt; ++PropIt)
	{
		const FString PropertyNameStr = InStruct->GetAuthoredNameForField(*PropIt);
		const FName PropertyName = *PropertyNameStr;
		if (NamesToLookFor.Contains(PropertyName)
			|| (!PropertyName.IsValidXName() && NamesToLookFor.Contains(MakeObjectNameFromDisplayLabel(PropertyNameStr, NAME_None))))
		{
			return *PropIt;
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE