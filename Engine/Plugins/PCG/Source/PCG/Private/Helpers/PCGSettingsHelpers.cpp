// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGSettingsHelpers.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGEdge.h"
#include "PCGPin.h"
#include "Helpers/PCGHelpers.h"

#include "UObject/EnumProperty.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSettingsHelpers)

namespace PCGSettingsHelpers
{
	void DeprecationBreakOutParamsToNewPin(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
	{
		// Check basic conditions for which the code below should run.
		if(!InOutNode || InputPins.IsEmpty() || !InputPins[0] || InputPins[0]->Properties.AllowedTypes != EPCGDataType::Any)
		{
			return;
		}

		// Check if the node already has a param pin, if so, nothing to do.
		if (InOutNode->GetInputPin(PCGPinConstants::DefaultParamsLabel))
		{
			return;
		}

		// Also no need to add a param pin if it has no overriable params
		if (!InOutNode->GetSettings() || InOutNode->GetSettings()->OverridableParams().IsEmpty())
		{
			return;
		}

		UPCGPin* InPin = InputPins[0];

		// Add params pin with good defaults (UpdatePins will ensure pin details are correct later).
		UPCGPin* NewParamsPin = NewObject<UPCGPin>(InOutNode);
		NewParamsPin->Node = InOutNode;
		NewParamsPin->Properties.AllowedTypes = EPCGDataType::Param;
		NewParamsPin->Properties.Label = PCGPinConstants::DefaultParamsLabel;
		NewParamsPin->Properties.bAllowMultipleData = true;
		NewParamsPin->Properties.SetAllowMultipleConnections(true);
		InputPins.Add(NewParamsPin);

		// Make list of param pins that In pin is currently connected to.
		TArray<UPCGPin*> UpstreamParamPins;
		for (const UPCGEdge* Connection : InPin->Edges)
		{
			if (Connection->InputPin && Connection->InputPin->Properties.AllowedTypes == EPCGDataType::Param)
			{
				UpstreamParamPins.Add(Connection->InputPin);
			}
		}

		// Break all connections to param pins, and connect the first such pin to the new params pin on this node.
		for (UPCGPin* Pin : UpstreamParamPins)
		{
			InPin->BreakEdgeTo(Pin);

			// Params never support multiple connections as a rule (user must merge params themselves), so just connect first.
			if (!NewParamsPin->IsConnected())
			{
				NewParamsPin->AddEdgeTo(Pin);
			}
		}
	}

	template <uint32 N>
	TArray<FPCGSettingsOverridableParam> GetAllOverridableParams_Internal(const UStruct* InClass, const FPCGGetAllOverridableParamsConfig& InConfig, TArray<const FProperty*, TInlineAllocator<N>>& InAlreadySeenProperties, int32 Depth)
	{
		TArray<FName> LabelCache;

		TArray<FPCGSettingsOverridableParam> Res;

		check(InClass);

		auto GatherAliases = [](const FProperty* InProperty, FPCGSettingsOverridableParam& InParam) -> void
		{
#if WITH_EDITOR
			if (InProperty->HasMetaData(PCGObjectMetadata::OverrideAliases))
			{
				FPCGPropertyAliases Result;
				TArray<FString> TempStringArray;
				const FString& AliasesStr = InProperty->GetMetaData(PCGObjectMetadata::OverrideAliases);
				AliasesStr.ParseIntoArray(TempStringArray, TEXT(","));

				if (!TempStringArray.IsEmpty())
				{
					Algo::Transform(TempStringArray, Result.Aliases, [](const FString& In) -> FName { return FName(In); });
					// We always emplace at level 0, it will be incremented by the recursion if it is deeper than 1 level.
					InParam.MapOfAliases.Emplace(0, std::move(Result));
				}
			}
#endif // WITH_EDITOR
		};

		const UStruct* StopClass = InConfig.bExcludeSuperProperties ? InClass->GetSuperStruct() : InConfig.StopClass;
		for (const UStruct* CurClass = InClass; CurClass != nullptr && (StopClass == nullptr || CurClass  != StopClass); CurClass = CurClass->GetSuperStruct())
		{
			for (TFieldIterator<FProperty> InputIt(CurClass, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::ExcludeDeprecated); InputIt; ++InputIt)
			{
				const FProperty* Property = *InputIt;
				if (!Property)
				{
					continue;
				}

				bool bValid = true;

#if WITH_EDITOR
				auto HasMetadata = [Property](const TArray<FName>& InValues, bool bIsConjunction) -> bool
				{
					bool bFound = false;
					for (const FName& Metadata : InValues)
					{
						if (Property->HasMetaData(Metadata) && !bIsConjunction)
						{
							bFound = true;
							if (!bIsConjunction)
							{
								break;
							}
						}
						else if (bIsConjunction)
						{
							bFound = false;
							break;
						}
					}

					return bFound;
				};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (!InConfig.IncludeMetadataValues.IsEmpty())
				{
					bValid &= HasMetadata(InConfig.IncludeMetadataValues, InConfig.bIncludeMetadataIsConjunction);
				}

				if (!InConfig.ExcludeMetadataValues.IsEmpty())
				{
					bValid &= !HasMetadata(InConfig.ExcludeMetadataValues, InConfig.bExcludeMetadataIsConjunction);
				}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

				auto HasPropertyFlags = [Property](uint64 InFlags, bool bIsConjunction) -> bool
				{
					return bIsConjunction ? Property->HasAllPropertyFlags(InFlags) : Property->HasAnyPropertyFlags(InFlags);
				};
				
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (InConfig.IncludePropertyFlags != 0)
				{
					bValid &= HasPropertyFlags(InConfig.IncludePropertyFlags, InConfig.bIncludePropertyFlagsIsConjunction);
				}

				if (InConfig.ExcludePropertyFlags != 0)
				{
					bValid &= !HasPropertyFlags(InConfig.ExcludePropertyFlags, InConfig.bExcludePropertyFlagsIsConjunction);
				}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				// Don't allow to override the seed if the settings doesn't use the seed.
				if (!InConfig.bUseSeed && Property->GetOwnerClass() && Property->GetOwnerClass()->IsChildOf<UPCGSettings>())
				{
					bValid &= (Property->GetFName() != GET_MEMBER_NAME_CHECKED(UPCGSettings, Seed));
				}

				if (InConfig.ShouldKeepPropertyFunc.IsSet())
				{
					bValid &= InConfig.ShouldKeepPropertyFunc(Property, Depth);
				}

				if (!bValid)
				{
					continue;
				}

				auto RecursiveExtraction = [&InConfig, &Res, &LabelCache, Property, InClass, &GatherAliases, &InAlreadySeenProperties, Depth](const UStruct* NextClass, const FProperty* ContainerProperty)
				{
					// Reached max depth or max number of containers
					if (InConfig.MaxStructDepth == 0 || (ContainerProperty && InConfig.MaxContainersNum == 0))
					{
						return;
					}

					// Use the seed, and don't check metadata for PCG overridable.
					FPCGGetAllOverridableParamsConfig RecurseConfig = InConfig;
					InAlreadySeenProperties.Add(ContainerProperty ? ContainerProperty : Property);
					RecurseConfig.bUseSeed = true;

					// Also check if we have some filtering on the child properties. HasMetaData/GetMetaData are editor-only, so
					// this stays empty in non-editor builds, which disables the filter instead of breaking it.
					TArray<FString> ChildPropertiesFiltering;
#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					RecurseConfig.IncludeMetadataValues.Remove(PCGObjectMetadata::Overridable);
					RecurseConfig.IncludeMetadataValues.Remove(PCGObjectMetadata::OverridableCPUAndGPU);
					RecurseConfig.IncludeMetadataValues.Remove(PCGObjectMetadata::OverridableCPUAndGPUWithReadback);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

					if (Property->HasMetaData(PCGObjectMetadata::OverridableChildProperties))
					{
						const FString& ChildPropertiesMetadata = Property->GetMetaData(PCGObjectMetadata::OverridableChildProperties);
						ChildPropertiesFiltering = PCGHelpers::GetStringArrayFromCommaSeparatedList(ChildPropertiesMetadata);
					}
#endif // WITH_EDITOR

					if (RecurseConfig.MaxStructDepth > 0)
					{
						RecurseConfig.MaxStructDepth--;
					}

					if (ContainerProperty && RecurseConfig.MaxContainersNum > 0)
					{
						RecurseConfig.MaxContainersNum--;
					}

					for (FPCGSettingsOverridableParam& ChildParam : GetAllOverridableParams_Internal(NextClass, RecurseConfig, InAlreadySeenProperties, Depth + 1))
					{
						FName Label = ChildParam.Label;
						bool bHasNameClash = false;
						const int32 CachedLabelIndex = LabelCache.IndexOfByKey(Label);
						if (CachedLabelIndex != INDEX_NONE)
						{
							// If we have a clash, we will use the full path, so mark this param and the other that clashed to use the full path.
							Res[CachedLabelIndex].bHasNameClash = true;
							Res[CachedLabelIndex].Label = FName(Res[CachedLabelIndex].GetPropertyPath());
							bHasNameClash = true;
						}

						LabelCache.AddUnique(Label);

						// Also check if the current property path is filtered out
						if (!ChildPropertiesFiltering.IsEmpty() && !ChildPropertiesFiltering.Contains(ChildParam.GetPropertyPath()))
						{
							continue;
						}

						FPCGSettingsOverridableParam& Param = Res.Emplace_GetRef();
						Param.PropertiesNames.Add(*Property->GetAuthoredName());
						Param.PropertiesNames.Append(std::move(ChildParam.PropertiesNames));
						Param.NumContainers = ChildParam.NumContainers;
						if (ContainerProperty)
						{
							Param.NumContainers++;
							Param.Properties.Add(ContainerProperty);
						}

						Param.Properties.Add(Property);
						Param.Properties.Append(std::move(ChildParam.Properties));
						Param.PropertyClass = InClass;
#if WITH_EDITOR
						Param.UnderlyingType = ChildParam.UnderlyingType;
#endif //WITH_EDITOR

						if (!ChildParam.bHasNameClash && !bHasNameClash)
						{
							Param.Label = Label;
						}
						else
						{
							Param.Label = FName(Param.GetPropertyPath());
							Param.bHasNameClash = true;
						}
#if WITH_EDITOR
						Param.bSupportsGPU = ChildParam.bSupportsGPU;
						Param.bRequiresGPUReadback = ChildParam.bRequiresGPUReadback;
						GatherAliases(Property, Param);
						// For all already found aliases, add them to the current param, with the index incremented by 1.
						for (TPair<int32, FPCGPropertyAliases>& It : ChildParam.MapOfAliases)
						{
							Param.MapOfAliases.Emplace(It.Key + 1, std::move(It.Value));
						}
#endif // WITH_EDITOR
					}

					InAlreadySeenProperties.Pop(EAllowShrinking::No);
				};

				const FProperty* OriginalProperty = Property;
				const FProperty* ContainerProperty = nullptr;
				if (InConfig.bExtractArrays)
				{
					if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
					{
						ContainerProperty = ArrayProperty;
						Property = ArrayProperty->Inner;
					}
				}
				
				// Generic Attribute update: The old flow to extract property was:
				// * if it is an object, and we extract objects or instance -> RecursiveExtraction. If we can't we stop there.
				// * else if the property is supported by the accessors -> Make a param
				// * else if the property is a struct property -> RecursiveExtraction. If we can't we stop there.
				//
				// To support the new flow, we'll add to the second condition, that if we are at the end of the recursion, and we 
				// don't want to discard the leaf struct property, we create a param for this struct.
				const bool bCanRecurse = InConfig.MaxStructDepth != 0 && (!ContainerProperty || InConfig.MaxContainersNum != 0);
				const bool bIsPropertySupportedByOldAccessors = PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(Property);
				const bool bIsPropertySupportedByNewAccessors = PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(Property, /*bUseGenericAccessors=*/true);
				const bool bShouldExtractObject = Property->IsA<FObjectProperty>() && !InAlreadySeenProperties.Contains(OriginalProperty) && (InConfig.bExtractObjects || Property->HasAllPropertyFlags(CPF_InstancedReference));
				
				if (bShouldExtractObject)
				{
					RecursiveExtraction(CastFieldChecked<FObjectProperty>(Property)->PropertyClass, ContainerProperty);
				}
				else if (bIsPropertySupportedByOldAccessors || (!bCanRecurse && bIsPropertySupportedByNewAccessors && !InConfig.bDiscardLeafStructProperty))
				{
					FName Label = NAME_None;
					bool bHasNameClash = false;

					Label = *Property->GetAuthoredName();
					const int32 CachedLabelIndex = LabelCache.IndexOfByKey(Label);
					if (CachedLabelIndex != INDEX_NONE)
					{
						// If we have a clash, we will use the full path, so mark this param and the other that clashed to use the full path.
						Res[CachedLabelIndex].bHasNameClash = true;
						Res[CachedLabelIndex].Label = FName(Res[CachedLabelIndex].GetPropertyPath());
						bHasNameClash = true;
					}

					LabelCache.AddUnique(Label);

					FPCGSettingsOverridableParam& Param = Res.Emplace_GetRef();
					Param.PropertiesNames.Add(*Property->GetAuthoredName());
					if (ContainerProperty)
					{
						Param.Properties.Add(ContainerProperty);
						Param.NumContainers = 1;
					}

					Param.Properties.Add(Property);
					Param.PropertyClass = InClass;
					Param.bHasNameClash = bHasNameClash;
					Param.Label = bHasNameClash ? FName(Param.GetPropertyPath()) : Label;
#if WITH_EDITOR
					Param.bSupportsGPU = OriginalProperty->HasMetaData(PCGObjectMetadata::OverridableCPUAndGPU) || Property->HasMetaData(PCGObjectMetadata::OverridableCPUAndGPUWithReadback);
					Param.bRequiresGPUReadback = OriginalProperty->HasMetaData(PCGObjectMetadata::OverridableCPUAndGPUWithReadback);
					GatherAliases(OriginalProperty, Param);
					Param.UnderlyingType = PCGAttributeAccessorHelpers::GetMetadataTypeForProperty(Property);
#endif // WITH_EDITOR
				}
				else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property); StructProperty && bCanRecurse)
				{
					RecursiveExtraction(StructProperty->Struct, ContainerProperty);
				}
			}
		}

		return Res;
	}

	TArray<FPCGSettingsOverridableParam> GetAllOverridableParams(const UStruct* InClass, const FPCGGetAllOverridableParamsConfig& InConfig)
	{
		constexpr uint32 InlineAllocator = 16;
		// Array of already seen properties to catch infinite recursion
		TArray<const FProperty*, TInlineAllocator<InlineAllocator>> AlreadySeenProperties;

		return GetAllOverridableParams_Internal(InClass, InConfig, AlreadySeenProperties, 0);
	}

	// Thread-local variables used for the FPinTypeScopeHelper implementation
	static thread_local bool bVisitedPinsMapExists = false;
	static thread_local TMap<const UPCGPin*, FPCGDataTypeIdentifier, TInlineSetAllocator<32>> VisitedPinsMap;

	FPinTypeScopeHelper::FPinTypeScopeHelper()
	{
		if (!bVisitedPinsMapExists)
		{
			bClearMapOnDestruction = true;
			bVisitedPinsMapExists = true;
		}
	}

	FPinTypeScopeHelper::~FPinTypeScopeHelper()
	{
		if (bClearMapOnDestruction)
		{
			VisitedPinsMap.Empty();
			bVisitedPinsMapExists = false;
		}
	}

	EPCGDataType FPinTypeScopeHelper::GetCurrentPinType(const UPCGPin* InPin) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GetCurrentPinTypeID(InPin);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FPCGDataTypeIdentifier FPinTypeScopeHelper::GetCurrentPinTypeID(const UPCGPin* InPin) const
	{
		// Hitting this ensure means break of contract (scope broken), this helper should never be dynamically allocated.
		ensure(bVisitedPinsMapExists);
		if (FPCGDataTypeIdentifier* ExistingResult = bVisitedPinsMapExists ? VisitedPinsMap.Find(InPin) : nullptr)
		{
			return *ExistingResult;
		}
		else if(InPin)
		{
			FPCGDataTypeIdentifier PinResult = InPin->GetCurrentTypesID();

			if (bVisitedPinsMapExists)
			{
				VisitedPinsMap.Add(InPin, PinResult);
			}

			return PinResult;
		}
		else
		{
			return {};
		}
	}
}

PCG_API TArray<FPCGSettingsPropertyDefinition> UPCGSettingsHelpers::GetCommonlyUsedProperties(TSubclassOf<UPCGSettings> InSettingsClass)
{
	using namespace PCGSettingsHelpers;

	FPCGGetAllOverridableParamsConfig Config;
	Config.ShouldKeepPropertyFunc = [](const FProperty* InProperty, int32) -> bool { return InProperty && InProperty->HasAnyPropertyFlags(CPF_Edit) && !InProperty->HasAnyPropertyFlags(CPF_AdvancedDisplay); };
	Config.StopClass = UPCGSettings::StaticClass();

	TArray<FPCGSettingsPropertyDefinition> Properties;
	for (const FPCGSettingsOverridableParam& Param : GetAllOverridableParams(InSettingsClass, Config))
	{
		const FProperty* Property = Param.Properties.IsEmpty() ? nullptr : Param.Properties[0];
		if (Property)
		{
			FString Tooltip;
#if WITH_EDITOR
			Tooltip = Property->GetToolTipText().ToString();
#endif // WITH_EDITOR

			Properties.Add(FPCGSettingsPropertyDefinition{ 
				Param.Label.ToString() , 
				Param.GetPropertyPath(), 
				Tooltip,
				Property->GetCPPType() });
		}
	}

	return Properties;
}

