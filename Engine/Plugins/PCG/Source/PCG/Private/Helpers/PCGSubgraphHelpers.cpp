// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGSubgraphHelpers.h"

#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGInputOutputSettings.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "PCGSubgraph.h"
#include "Elements/PCGReroute.h"
#include "Elements/PCGUserParameterGet.h"

#include "Algo/AnyOf.h"
#include "Algo/Transform.h"

#define LOCTEXT_NAMESPACE "PCGSubgraphHelpers"

namespace PCGSubgraphHelpersExtra
{
	/**
	* Structure that will hold the necessary information during the collapse and execute it.
	*/
	struct FCollapsingInformation
	{
#if WITH_EDITOR
		// Keep track of all node positions to know where to spawn our subgraph node (editor only)
		FVector2D AveragePosition = FVector2D::ZeroVector;
		int32 MinX = std::numeric_limits<int32>::max();
		int32 MaxX = std::numeric_limits<int32>::min();
#endif // WITH_EDITOR

		/** First pass on the nodes to collapse, gathering information about special nodes like Graph Parameters. */
		static FCollapsingInformation CreateAndInitialize(const TArray<UPCGNode*>& InNodesToCollapse, const FInstancedPropertyBag& InGraphParameters, UPCGGraph* InOriginalGraph, TSubclassOf<UPCGBaseSubgraphSettings> InSubgraphSettingsClass);
		bool ValidateCollapseIsNotIntroducingCycle();

		void SetNewGraphAndUpdatePositions(UPCGGraph* NewGraph);

		void CopyAndRemapEmbeddedSubgraphs();
		void AddDiscoveredGraphParameters();
		void CopyNodes(const TArray<UObject*>& ExtraNodesToCollapse);
		void DeleteNodes(const TArray<UObject*>& ExtraNodesToCollapse);
		void CreateSubgraphNodeAndConnectRemainingPins();
		
		/**
		* Go through all the collapsed nodes and reconnect all of them according to the original connections.
		* Will also handle any special cases (named reroutes, graph parameters, ...)
		*/
		void ReconnectAllNodes();

		// After reconnection, clean up edges that needs to be cleaned up
		void CleanUpOriginalEdges();

		bool HasEnoughValidNodes() { return ValidNodesToCollapse.Num() > 1; }

	private:
		// Use initialize to create the struct.
		FCollapsingInformation() = default;

		// List of all the nodes that are valid to collapse and so will be copied.
		TArray<UPCGNode*> ValidNodesToCollapse;
		// By default all nodes that are collapsed are removed, but some might have to stay (like graph parameters getters)
		TArray<UPCGNode*> NodesToRemove;
		// Also some node can be mark superfluous, meaning that they can be removed if at the end, there is no edges remaining.
		TArray<UPCGNode*> SuperfluousNodes;
		// List used embedded subgraphs
		TArray<UPCGNode*> EmbeddedSubgraphNodes;

		// Keep a list of all outside pins and the name of the label to connect to.
		TArray<TPair<UPCGPin*, FName>> OutsideSubgraphPinToSubgraphPinLabel;

		// Keep a mapping between pins from the inside pin and its connected pin on the input/output node in the subgraph.
		TMap<UPCGPin*, UPCGPin*> SubgraphInsidePinToInputOutputPinMapping;
		TMap<FString, int> NameCollisionMapping;

		// Keep a mapping between original nodes and new nodes
		TMap<const UPCGNode*, UPCGNode*> OriginalToNewNodesMapping;

		UPCGGraph* OriginalGraph = nullptr;
		UPCGGraph* NewGraph = nullptr;

		TSubclassOf<UPCGBaseSubgraphSettings> SubgraphSettingsClass = UPCGSubgraphSettings::StaticClass();

		TMap<UPCGGraph*, UPCGGraph*> DuplicatedEmbeddedSubgraphs;

		/** 
		* Connect two pins that cross the subgraph boundaries. The Inside Pin is the pin that is inside the subgraph (NewGraph) and outside pin the pin in the main graph.
		* bIsSubgraphInputBoundary is set if it is the input boundary that is crossed. Otherwise, it is the output boundary.
		* Returns true if a new custom pin was added, false otherwise.
		*/
		bool ConnectPinsBetweenSubgraphBoundaries(UPCGPin* OriginalPin, UPCGPin* NewInsidePin, UPCGPin* OutsidePin, bool bIsSubgraphInputBoundary);

		/**
		* Function that will create a new pin in the input / output node of the subgraph, with a unique name that will
		* be the related to the pin passed as argument. It will be formatted like this:
		* "{OptionalNodeName} {PinName} {OptionalIndex}"
		* OptionalNodeName and OptionalIndex are added if there are any name clash.
		*/
		FName CreateNewCustomPin(UPCGGraph* InGraph, const UPCGPin* InPinToClone, bool bIsInput);

		void ReconnectNode(UPCGNode* NewNode, UPCGPin* OriginalPin);

		////////////////
		// Graph parameters specific
		////////////////

		// Keep a mapping between PropertyDesc and their getter node. Useful to discard other getters that can be removed if unused.
		TMap<const FPropertyBagPropertyDesc*, UPCGNode*> GraphParametersDescToNodeGetters;

		// Also keep a reference of the original graph parameters.
		const FInstancedPropertyBag* GraphParameters = nullptr;

		/** 
		* If a graph parameter getter is collapse, this will connect the original getter to the override pin on the subgraph node (it's not a real connect, just adding to OutsideSubgraphPinToSubgraphPinLabelMapping, for placeholder).
		* returns true if it is the first time we see that graph parameter.
		*/
		bool ConnectGraphParameterGetterToSubgraphInput(const UPCGUserParameterGetSettings* InSettings, UPCGNode* InNode);

		/**
		* Special handling if a graph parameter getter is outside the subgraph. If the graph parameter is part of the subgraph, we will use the getter inside the subgraph for the same parameter.
		* Otherwise, it is a normal boundary cross.
		*/
		void HandleExternalGraphParameterGetter(UPCGPin* OriginalInsidePin, UPCGPin* NewInsidePin, UPCGPin* OutsidePin);

		////////////////
		// Named reroute specific
		////////////////

		void HandleReconnectNamedRerouteDeclaration(UPCGNode* OriginalNode, const UPCGNamedRerouteDeclarationSettings* OriginalSettings);
		void HandleReconnectNamedRerouteUsage(UPCGNode* OriginalNode, const UPCGNamedRerouteUsageSettings* OriginalSettings);

		// List of original pins to break edges. Need to be done after the reconnection, to not alter it.
		TSet<UPCGPin*> OriginalPinsToBreakEdges;
	};

	FCollapsingInformation FCollapsingInformation::CreateAndInitialize(const TArray<UPCGNode*>& InNodesToCollapse, const FInstancedPropertyBag& InGraphParameters, UPCGGraph* InOriginalGraph, TSubclassOf<UPCGBaseSubgraphSettings> InSubgraphSettingsClass)
	{
		check(InOriginalGraph);

		FCollapsingInformation CollapseInfo{};
		CollapseInfo.GraphParameters = &InGraphParameters;
		CollapseInfo.OriginalGraph = InOriginalGraph;
		
		if (InSubgraphSettingsClass)
		{
			CollapseInfo.SubgraphSettingsClass = InSubgraphSettingsClass;
		}

		for (UPCGNode* PCGNode : InNodesToCollapse)
		{
			check(PCGNode);

			const UPCGSettings* NodeSettings = PCGNode->GetSettings();

			// Exclude input and output nodes from the subgraph.
			if (!NodeSettings || NodeSettings->IsA<UPCGGraphInputOutputSettings>())
			{
				continue;
			}

			bool bShouldRemove = true;

			if (const UPCGUserParameterGetSettings* GetSettings = Cast<UPCGUserParameterGetSettings>(NodeSettings))
			{
				bShouldRemove = false;
				// If it was connected, we want to keep this node outside, so don't remove it.
				// Otherwise, we already discovered this graph parameter, so we can remove it from the top graph
				const bool bAlreadyDiscovered = !CollapseInfo.ConnectGraphParameterGetterToSubgraphInput(GetSettings, PCGNode);
				if (bAlreadyDiscovered)
				{
					CollapseInfo.SuperfluousNodes.Add(PCGNode);
				}
			}

			if (const UPCGBaseSubgraphSettings* SubgraphSettings = Cast<UPCGBaseSubgraphSettings>(NodeSettings))
			{
				if (SubgraphSettings->GetSubgraph() && SubgraphSettings->GetSubgraph()->IsEmbeddedSubgraph())
				{
					CollapseInfo.EmbeddedSubgraphNodes.Add(PCGNode);
				}
			}

			CollapseInfo.ValidNodesToCollapse.Add(PCGNode);

#if WITH_EDITOR
			// And do all the computation to get the min, max and mean position.
			CollapseInfo.AveragePosition.X += PCGNode->PositionX;
			CollapseInfo.AveragePosition.Y += PCGNode->PositionY;
			CollapseInfo.MinX = FMath::Min(CollapseInfo.MinX, PCGNode->PositionX);
			CollapseInfo.MaxX = FMath::Max(CollapseInfo.MaxX, PCGNode->PositionY);
#endif // WITH_EDITOR

			if (bShouldRemove)
			{
				CollapseInfo.NodesToRemove.Add(PCGNode);
			}
		}

#if WITH_EDITOR
		if (CollapseInfo.ValidNodesToCollapse.Num() > 1)
		{
			// Compute the average position
			CollapseInfo.AveragePosition /= CollapseInfo.ValidNodesToCollapse.Num();
		}
#endif // WITH_EDITOR

		return CollapseInfo;
	}

	void FCollapsingInformation::SetNewGraphAndUpdatePositions(UPCGGraph* InNewGraph)
	{
		check(InNewGraph);
		NewGraph = InNewGraph;

#if WITH_EDITOR
		// Do some clean-up on input/output nodes
		constexpr int32 Padding = 300;
		NewGraph->GetInputNode()->PositionX = MinX - Padding;
		NewGraph->GetInputNode()->PositionY = AveragePosition.Y;
		NewGraph->GetOutputNode()->PositionX = MaxX + Padding;
		NewGraph->GetOutputNode()->PositionY = AveragePosition.Y;
#endif // WITH_EDITOR
	}

	void FCollapsingInformation::AddDiscoveredGraphParameters()
	{
		check(OriginalGraph && NewGraph);
		TArray<FPropertyBagPropertyDesc> GraphParametersUsed;
		Algo::Transform(GraphParametersDescToNodeGetters, GraphParametersUsed, [](const auto& It) { check(It.Key); return *It.Key; });
		NewGraph->AddUserParameters(GraphParametersUsed, OriginalGraph);
	}

	void FCollapsingInformation::CopyAndRemapEmbeddedSubgraphs()
	{
		check(OriginalGraph && NewGraph);

		// Not needed if we are creating a new embedded graph (and not an asset)
		if(NewGraph->GetEmbeddedParentGraph() == OriginalGraph)
		{
			return;
		}

		TArray<UPCGGraph*> EmbeddedSubgraphsToCopy;
		for (UPCGNode* EmbeddedSubgraphNode : EmbeddedSubgraphNodes)
		{
			UPCGBaseSubgraphSettings* SubgraphSettings = CastChecked<UPCGBaseSubgraphSettings>(EmbeddedSubgraphNode->GetSettings());
			UPCGGraph* EmbeddedSubgraph = SubgraphSettings->GetSubgraph();
			check(EmbeddedSubgraph && EmbeddedSubgraph->IsEmbeddedSubgraph());
			EmbeddedSubgraphsToCopy.AddUnique(EmbeddedSubgraph);
		}

		DuplicatedEmbeddedSubgraphs.Empty();
		NewGraph->DuplicateEmbeddedSubgraphs(EmbeddedSubgraphsToCopy, DuplicatedEmbeddedSubgraphs);

		for (UPCGNode* EmbeddedSubgraphNode : EmbeddedSubgraphNodes)
		{
			check(OriginalToNewNodesMapping.Contains(EmbeddedSubgraphNode));
			UPCGNode* NewEmbeddedSubgraphNode = OriginalToNewNodesMapping[EmbeddedSubgraphNode];
			check(NewEmbeddedSubgraphNode);

			UPCGBaseSubgraphSettings* SubgraphSettings = CastChecked<UPCGBaseSubgraphSettings>(NewEmbeddedSubgraphNode->GetSettings());
			UPCGGraph* DuplicatedEmbeddedSubgraph = DuplicatedEmbeddedSubgraphs.FindChecked(SubgraphSettings->GetSubgraph());

			SubgraphSettings->SetSubgraph(DuplicatedEmbeddedSubgraph);
		}
	}

	void FCollapsingInformation::CopyNodes(const TArray<UObject*>& ExtraNodesToCollapse)
	{
		check(NewGraph);
		for (UPCGNode* PCGNode : ValidNodesToCollapse)
		{
			check(PCGNode);

			// Reconstruct a new node, same as PCGNode, but without any edges in the new graph 
			UPCGNode* NewNode = NewGraph->ReconstructNewNode(PCGNode);
			check(NewNode);

			NewNode->NodeTitle = PCGNode->NodeTitle;
			OriginalToNewNodesMapping.Emplace(PCGNode, NewNode);
		}

#if WITH_EDITOR
		// Also duplicate the extra nodes and assign them to the new graph
		TArray<TObjectPtr<const UObject>> NewExtraGraphNodes;
		TArray<FPCGGraphCommentNodeData> NewCommentNodes;
		
		for (const UObject* ExtraNode : ExtraNodesToCollapse)
		{
			// Special treatment of comment nodes
			if (const UEdGraphNode_Comment* CommentNode = Cast<const UEdGraphNode_Comment>(ExtraNode))
			{
				NewCommentNodes.Emplace_GetRef().InitializeFromCommentNode(*CommentNode);
			}
			else
			{
				NewExtraGraphNodes.Add(DuplicateObject(ExtraNode, NewGraph));
			}
		}

		NewGraph->SetExtraEditorNodes(NewExtraGraphNodes);
		NewGraph->SetCommentNodes(std::move(NewCommentNodes));
#endif ///WITH_EDITOR
	}

	void FCollapsingInformation::DeleteNodes(const TArray<UObject*>& ExtraNodesToCollapse)
	{
		check(OriginalGraph);

		// First remove all the nodes that were valid and should be removed.
		OriginalGraph->RemoveNodes(NodesToRemove);

#if WITH_EDITOR
		for (const UObject* ExtraNode : ExtraNodesToCollapse)
		{
			// Special treatment of comment nodes
			if (const UEdGraphNode_Comment* CommentNode = Cast<const UEdGraphNode_Comment>(ExtraNode))
			{
				OriginalGraph->RemoveCommentNode(CommentNode->NodeGuid);
			}
			else
			{
				OriginalGraph->RemoveExtraEditorNode(ExtraNode);
			}
		}
#endif // WITH_EDITOR

		// Then for all the remaining nodes to remove, make sure they have no edges anymore.
		TArray<UPCGNode*> SuperfluousNodesToRemove;
		SuperfluousNodesToRemove.Reserve(SuperfluousNodes.Num());
		for (UPCGNode* SuperfluousNode : SuperfluousNodes)
		{
			check(SuperfluousNode);
			// Only remove them if they have no edges anymore
			bool bHasAnyEdges = Algo::AnyOf(SuperfluousNode->GetInputPins(), [](const UPCGPin* InPin) { return InPin && InPin->EdgeCount() > 0; })
				|| Algo::AnyOf(SuperfluousNode->GetOutputPins(), [](const UPCGPin* InPin) { return InPin && InPin->EdgeCount() > 0; });

			if (!bHasAnyEdges)
			{
				SuperfluousNodesToRemove.Add(SuperfluousNode);
			}
		}

		// Finally remove them all in one go
		OriginalGraph->RemoveNodes(SuperfluousNodesToRemove);
	}

	void FCollapsingInformation::CreateSubgraphNodeAndConnectRemainingPins()
	{
		check(OriginalGraph && NewGraph);

		UPCGSettings* DefaultNodeSettings = nullptr;
		UPCGNode* SubgraphNode = OriginalGraph->AddNodeOfType(SubgraphSettingsClass, DefaultNodeSettings);
		UPCGBaseSubgraphSettings* DefaultSubgraphSettings = CastChecked<UPCGBaseSubgraphSettings>(DefaultNodeSettings);
		DefaultSubgraphSettings->SetSubgraph(NewGraph);

		check(SubgraphNode);

#if WITH_EDITOR
		SubgraphNode->PositionX = AveragePosition.X;
		SubgraphNode->PositionY = AveragePosition.Y;
#endif // WITH_EDITOR

		SubgraphNode->UpdateAfterSettingsChangeDuringCreation();

		for (const auto& [OutsidePin, SubgraphPinLabel] : OutsideSubgraphPinToSubgraphPinLabel)
		{
			check(OutsidePin);
			UPCGNode* OutsideNode = OutsidePin->Node;
			check(OutsideNode);

			if (OutsidePin->IsOutputPin())
			{
				OutsidePin->AddEdgeTo(SubgraphNode->GetInputPin(SubgraphPinLabel));
			}
			else
			{
				SubgraphNode->GetOutputPin(SubgraphPinLabel)->AddEdgeTo(OutsidePin);
			}
		}
	}

	void FCollapsingInformation::ReconnectAllNodes()
	{
		for (UPCGNode* OriginalNode : ValidNodesToCollapse)
		{
			check(OriginalToNewNodesMapping.Contains(OriginalNode));
			UPCGNode* NewNode = OriginalToNewNodesMapping[OriginalNode];
			check(NewNode);

			// Named reroute specifics.
			if (const UPCGNamedRerouteDeclarationSettings* OriginalDeclSettings = Cast<UPCGNamedRerouteDeclarationSettings>(OriginalNode->GetSettings()))
			{
				HandleReconnectNamedRerouteDeclaration(OriginalNode, OriginalDeclSettings);
				continue;
			}
			else if (const UPCGNamedRerouteUsageSettings* OriginalUsageSettings = Cast<UPCGNamedRerouteUsageSettings>(OriginalNode->GetSettings()))
			{
				HandleReconnectNamedRerouteUsage(OriginalNode, OriginalUsageSettings);
				continue;
			}

			for (UPCGPin* OriginalInputPin : OriginalNode->GetInputPins())
			{
				ReconnectNode(NewNode, OriginalInputPin);
			}

			for (UPCGPin* OriginalOutputPin : OriginalNode->GetOutputPins())
			{
				ReconnectNode(NewNode, OriginalOutputPin);
			}
		}
	}

	void FCollapsingInformation::CleanUpOriginalEdges()
	{
		for (UPCGPin* OriginalPin : OriginalPinsToBreakEdges)
		{
			if (ensure(OriginalPin))
			{
				OriginalPin->BreakAllEdges();
			}
		}
	}

	FName FCollapsingInformation::CreateNewCustomPin(UPCGGraph* InGraph, const UPCGPin* InPinToClone, bool bIsInput)
	{
		check(InGraph && InPinToClone);

		UPCGNode* NewInputOutputNode = bIsInput ? InGraph->GetInputNode() : InGraph->GetOutputNode();
		UPCGGraphInputOutputSettings* NewInputOutputSettings = CastChecked<UPCGGraphInputOutputSettings>(NewInputOutputNode->GetSettings());
		const UPCGNode* PinToCloneNode = InPinToClone->Node;
		const UPCGSettings* PinToCloneSettings = PinToCloneNode ? PinToCloneNode->GetSettings() : nullptr;
		check(PinToCloneNode && PinToCloneSettings);

		// First try with just the pin name
		FString NewName = InPinToClone->Properties.Label.ToString();
		if (NameCollisionMapping.Contains(NewName))
		{
			// If there is a clash, use the node title on top.
			NewName = PinToCloneNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString() + " " + NewName;
			if (NameCollisionMapping.Contains(NewName))
			{
				NewName += " " + FString::FormatAsNumber(++NameCollisionMapping[NewName]);
			}
			else
			{
				NameCollisionMapping.Emplace(NewName, 1);
			}
		}
		else
		{
			NameCollisionMapping.Emplace(NewName, 1);
		}

		FPCGPinProperties NewProperties = InPinToClone->Properties;

		// For the pin type, narrow it down to the input edges (if it is an input pin)
		if (!InPinToClone->IsOutputPin())
		{
			const FPCGDataTypeIdentifier FirstPinTypeUnion = PinToCloneSettings->GetTypeUnionIDOfIncidentEdges(InPinToClone->Properties.Label);
			if (FirstPinTypeUnion.IsValid())
			{
				NewProperties.AllowedTypes = FirstPinTypeUnion;
			}
		}
		else
		{
			NewProperties.AllowedTypes = PinToCloneSettings->GetCurrentPinTypesID(InPinToClone);
		}

		NewProperties.Label = FName(NewName);
		NewProperties = NewInputOutputSettings->AddPin(NewProperties);
		NewInputOutputNode->UpdateAfterSettingsChangeDuringCreation();
		return NewProperties.Label;
	}

	void FCollapsingInformation::ReconnectNode(UPCGNode* NewNode, UPCGPin* OriginalPin)
	{
		check(NewGraph && NewNode && OriginalPin);
		const bool bIsInput = !OriginalPin->IsOutputPin();

		UPCGPin* NewPin = bIsInput ? NewNode->GetInputPin(OriginalPin->Properties.Label) : NewNode->GetOutputPin(OriginalPin->Properties.Label);
		check(NewPin);

		for (UPCGEdge* Edge : OriginalPin->Edges)
		{
			check(Edge);
			UPCGPin* OtherOriginalPin = bIsInput ? Edge->InputPin : Edge->OutputPin;
			check(OtherOriginalPin);

			const UPCGNode* OtherOriginalNode = OtherOriginalPin->Node;
			check(OtherOriginalNode);

			// If the other original is in the mapping, it is part of the subgraph. So Connect the 2, only if the new node is the input pin, as we will pass there twice (once per edge extremity)
			if (UPCGNode** OtherNewNodePtr = OriginalToNewNodesMapping.Find(OtherOriginalNode))
			{
				if (bIsInput)
				{
					check(OtherOriginalPin->IsOutputPin());
					UPCGPin* OtherNewPin = (*OtherNewNodePtr)->GetOutputPin(OtherOriginalPin->Properties.Label);
					check(OtherNewPin);
					OtherNewPin->AddEdgeTo(NewPin);
				}
			}
			else
			{
				if (const UPCGUserParameterGetSettings* GraphParameterSettings = Cast<const UPCGUserParameterGetSettings>(OtherOriginalNode->GetSettings()))
				{
					HandleExternalGraphParameterGetter(OriginalPin, NewPin, OtherOriginalPin);
				}
				else if (Cast<const UPCGUserParameterGetSettings>(NewNode->GetSettings()) != nullptr)
				{
					// If the inside node is a graph parameter that goes outside, we have nothing to do, the edge already exists on the original node and the original node won't be deleted.
				}
				else
				{
					ConnectPinsBetweenSubgraphBoundaries(OriginalPin, NewPin, OtherOriginalPin, bIsInput);
				}
			}
		}
	}

	// Returns true if a custom pin was created.
	bool FCollapsingInformation::ConnectPinsBetweenSubgraphBoundaries(UPCGPin* OriginalInsidePin, UPCGPin* NewInsidePin, UPCGPin* OutsidePin, bool bIsSubgraphInputBoundary)
	{
		check(OriginalInsidePin && NewInsidePin && OutsidePin && NewGraph);

		bool bCustomPinCreated = false;

		// IO pins are a 1 for 1 with the pins that are inside the subgraph.
		// So we keep a mapping between the subgraph IO pins and the inside pins, and only create a new IO pin if we have a new inside pin that cross boundaries.
		UPCGPin* EdgeInputPin = bIsSubgraphInputBoundary ? OutsidePin : OriginalInsidePin;
		UPCGPin** SubgraphIOPinPtr = SubgraphInsidePinToInputOutputPinMapping.Find(EdgeInputPin);
		UPCGPin* SubgraphIOPin = nullptr;
		if (!SubgraphIOPinPtr)
		{
			FName SubgraphPinLabel = CreateNewCustomPin(NewGraph, EdgeInputPin, bIsSubgraphInputBoundary);
			SubgraphIOPin = bIsSubgraphInputBoundary ? NewGraph->GetInputNode()->GetOutputPin(SubgraphPinLabel) : NewGraph->GetOutputNode()->GetInputPin(SubgraphPinLabel);
			check(SubgraphIOPin);
			SubgraphInsidePinToInputOutputPinMapping.Emplace(EdgeInputPin, SubgraphIOPin);

			bCustomPinCreated = true;
		}
		else
		{
			SubgraphIOPin = *SubgraphIOPinPtr;
		}

		check(SubgraphIOPin);

		// And we keep a list of all the outside pins and the label they should connect to, when the subgraph node will be created later.
		if (bCustomPinCreated || !bIsSubgraphInputBoundary)
		{
			OutsideSubgraphPinToSubgraphPinLabel.Emplace(OutsidePin, SubgraphIOPin->Properties.Label);
		}
		
		if (bCustomPinCreated || bIsSubgraphInputBoundary)
		{
			if (bIsSubgraphInputBoundary)
			{
				SubgraphIOPin->AddEdgeTo(NewInsidePin);
			}
			else
			{
				NewInsidePin->AddEdgeTo(SubgraphIOPin);
			}
		}

		return bCustomPinCreated;
	}

	bool FCollapsingInformation::ConnectGraphParameterGetterToSubgraphInput(const UPCGUserParameterGetSettings* InSettings, UPCGNode* InNode)
	{
		check(InSettings && InNode);
		const FPropertyBagPropertyDesc* PropertyDesc = GraphParameters->FindPropertyDescByID(InSettings->PropertyGuid);

		if (ensure(PropertyDesc) && !GraphParametersDescToNodeGetters.Contains(PropertyDesc))
		{
			GraphParametersDescToNodeGetters.Emplace(PropertyDesc, InNode);
			check(InNode->GetOutputPins().Num() == 1);
			UPCGPin* GraphParameterPin = InNode->GetOutputPins()[0];
			check(GraphParameterPin);
			OutsideSubgraphPinToSubgraphPinLabel.Emplace(GraphParameterPin, GraphParameterPin->Properties.Label);

			return true;
		}

		// Already connected or invalid.
		return false;
	}

	void FCollapsingInformation::HandleExternalGraphParameterGetter(UPCGPin* OriginalInsidePin, UPCGPin* NewInsidePin, UPCGPin* OutsidePin)
	{
		check(OriginalInsidePin && NewInsidePin && OutsidePin);
		UPCGNode* ExternalGraphParameterNode = OutsidePin->Node;
		check(ExternalGraphParameterNode);
		const UPCGUserParameterGetSettings* Settings = CastChecked<const UPCGUserParameterGetSettings>(ExternalGraphParameterNode->GetSettings());

		const FPropertyBagPropertyDesc* PropertyDesc = GraphParameters->FindPropertyDescByID(Settings->PropertyGuid);
		check(PropertyDesc);

		// If the graph parameter exists, we should use that new getter inside the subgraph, and mark the external as superfluous.
		if (UPCGNode** ExistingGraphParameterNode = GraphParametersDescToNodeGetters.Find(PropertyDesc))
		{
			check(*ExistingGraphParameterNode);
			UPCGNode* ExistingGraphParameterNodeInsideSubgraph = OriginalToNewNodesMapping.FindChecked(*ExistingGraphParameterNode);
			check(ExistingGraphParameterNodeInsideSubgraph && ExistingGraphParameterNodeInsideSubgraph->GetOutputPins().Num() == 1);

			ExistingGraphParameterNodeInsideSubgraph->GetOutputPins()[0]->AddEdgeTo(NewInsidePin);
			SuperfluousNodes.Add(ExternalGraphParameterNode);
		}
		// If the external graph parameter is not part of the subgraph, we treat it as a normal node.
		else
		{
			ConnectPinsBetweenSubgraphBoundaries(OriginalInsidePin, NewInsidePin, OutsidePin, /*bIsSubgraphInputBoundary=*/true);
		}
	}

	void FCollapsingInformation::HandleReconnectNamedRerouteDeclaration(UPCGNode* OriginalDeclNode, const UPCGNamedRerouteDeclarationSettings* OriginalDeclSettings)
	{
		check(OriginalDeclNode && OriginalDeclSettings);
		UPCGNode* NewDeclNode = OriginalToNewNodesMapping.FindChecked(OriginalDeclNode);
		check(NewDeclNode);

		// Connect all the input pins normally
		for (UPCGPin* OriginalInputPin : OriginalDeclNode->GetInputPins())
		{
			ReconnectNode(NewDeclNode, OriginalInputPin);
		}

		// Connect all the output pins normally, except the Invisible Pin
		UPCGPin* InvisiblePin = OriginalDeclNode->GetOutputPin(PCGNamedRerouteConstants::InvisiblePinLabel);
		check(InvisiblePin);
		for (UPCGPin* OriginalOutputPin : OriginalDeclNode->GetOutputPins())
		{
			if (!ensure(OriginalOutputPin) || OriginalOutputPin == InvisiblePin)
			{
				continue;
			}

			ReconnectNode(NewDeclNode, OriginalOutputPin);
		}

		// Then for the invisible pin, only check for usages that could be outside. If they are outside, we don't remove the original declaration node, 
		// we make sure to create a new output pin to the subgraph and connect the original to this pin. We also break all the edges at the output, as they would be rewired normally.
		// For the usage inside, they will be patched and rewired in the Usage case.
		for (const UPCGEdge* Edge : InvisiblePin->Edges)
		{
			const UPCGPin* OtherPin = Edge->OutputPin;
			check(OtherPin && OtherPin != InvisiblePin);
			const UPCGNode* OtherNode = OtherPin->Node;
			check(OtherNode);
			const UPCGNamedRerouteUsageSettings* OtherUsageSettings = Cast<UPCGNamedRerouteUsageSettings>(OtherNode->GetSettings());
			if (!OriginalToNewNodesMapping.Contains(OtherNode) && OtherUsageSettings && NodesToRemove.Contains(OriginalDeclNode))
			{
				NodesToRemove.RemoveSwap(OriginalDeclNode);
				UPCGPin* OriginalInputPin = OriginalDeclNode->GetInputPin(PCGPinConstants::DefaultInputLabel);
				check(OriginalInputPin);
				OriginalPinsToBreakEdges.Add(OriginalInputPin);

				UPCGPin* OriginalOutputPin = OriginalDeclNode->GetOutputPin(PCGPinConstants::DefaultOutputLabel);
				check(OriginalOutputPin);
				OriginalPinsToBreakEdges.Add(OriginalOutputPin);

				UPCGPin* NewOutputPin = NewDeclNode->GetOutputPin(PCGPinConstants::DefaultOutputLabel);
				check(NewOutputPin);
				ConnectPinsBetweenSubgraphBoundaries(OriginalOutputPin, NewOutputPin, OriginalInputPin, /*bIsInput=*/false);
				break;
			}
		}
	}

	void FCollapsingInformation::HandleReconnectNamedRerouteUsage(UPCGNode* OriginalUsageNode, const UPCGNamedRerouteUsageSettings* OriginalUsageSettings)
	{
		check(OriginalUsageNode && OriginalUsageSettings)
		UPCGNode* NewUsageNode = OriginalToNewNodesMapping.FindChecked(OriginalUsageNode);
		check(NewUsageNode);
		UPCGNamedRerouteUsageSettings* NewUsageSettings = CastChecked<UPCGNamedRerouteUsageSettings>(NewUsageNode->GetSettings());

		const UPCGNamedRerouteDeclarationSettings* OriginalDeclSettings = OriginalUsageSettings->Declaration;
		if (!ensure(OriginalDeclSettings))
		{
			return;
		}

		UPCGNode* OriginalDeclNode = CastChecked<UPCGNode>(OriginalDeclSettings->GetOuter());
		UPCGNode* NewDeclNode = nullptr;
		if (UPCGNode** NewDeclNodePtr = OriginalToNewNodesMapping.Find(OriginalDeclNode))
		{
			NewDeclNode = *NewDeclNodePtr;
		}
		else
		{
			// If the declaration is outside of the subgraph, we need to create a new declaration inside the subgraph.
			NewDeclNode = NewGraph->ReconstructNewNode(OriginalDeclNode);
			check(NewDeclNode);
			NewDeclNode->NodeTitle = OriginalDeclNode->NodeTitle;
			OriginalToNewNodesMapping.Emplace(OriginalDeclNode, NewDeclNode);

			UPCGPin* OriginalOutputPin = OriginalDeclNode->GetOutputPin(PCGPinConstants::DefaultOutputLabel);
			UPCGPin* NewInputPin = NewDeclNode->GetInputPin(PCGPinConstants::DefaultInputLabel);
			check(OriginalOutputPin && NewInputPin);
			ConnectPinsBetweenSubgraphBoundaries(OriginalOutputPin, NewInputPin, OriginalOutputPin, /*bIsInput=*/true);
		}

		// Patch and connect the edges.
		check(NewDeclNode);
		NewUsageSettings->Declaration = CastChecked<UPCGNamedRerouteDeclarationSettings>(NewDeclNode->GetSettings());
		UPCGPin* InvisiblePin = NewDeclNode->GetOutputPin(PCGNamedRerouteConstants::InvisiblePinLabel);
		UPCGPin* InputUsagePin = NewUsageNode->GetInputPin(PCGPinConstants::DefaultInputLabel);
		check(InvisiblePin && InputUsagePin);
		InvisiblePin->AddEdgeTo(InputUsagePin);

		// And finally connect all the output pins normally
		for (UPCGPin* OriginalOutputPin : OriginalUsageNode->GetOutputPins())
		{
			ReconnectNode(NewUsageNode, OriginalOutputPin);
		}
	}

	/** Pre validation before starting the collapse, to verify that we won't introduce a cycle by collapsing. */
	bool FCollapsingInformation::ValidateCollapseIsNotIntroducingCycle()
	{
		const TSet<UPCGNode*> NodesToCollapse{ ValidNodesToCollapse };

		for (const UPCGNode* Node : NodesToCollapse)
		{
			if (!Node)
			{
				continue;
			}

			// We can only switch boundaries once, otherwise it will introduce cycle
			// Also keep track of all seen nodes, so our algorithm stops even if there are cycles already existing.
			TSet<const UPCGNode*> SeenNodes;

			auto Visitor = [&NodesToCollapse, &SeenNodes](const UPCGNode* CurrentNode, bool bInSubgraphBoundary, auto&& VisitorRecurse) -> bool
			{
				if (!CurrentNode || SeenNodes.Contains(CurrentNode))
				{
					//Nothing to do
					return true;
				}

				SeenNodes.Add(CurrentNode);

				for (const UPCGPin* OutputPin : CurrentNode->GetOutputPins())
				{
					if (!OutputPin)
					{
						continue;
					}

					for (const UPCGEdge* Edge : OutputPin->Edges)
					{
						if (!Edge)
						{
							continue;
						}

						if (const UPCGNode* OtherNode = Edge->OutputPin ? Edge->OutputPin->Node : nullptr)
						{
							const bool bNodeInSubgraphBoundary = NodesToCollapse.Contains(OtherNode);
							if (!bInSubgraphBoundary && bNodeInSubgraphBoundary)
							{
								// We crossed twice, we can't collapse
								return false;
							}

							if (!VisitorRecurse(OtherNode, bNodeInSubgraphBoundary, VisitorRecurse))
							{
								return false;
							}
						}
					}
				}

				return true;
			};

			if (!Visitor(Node, /*bIsInSubgraphBoundary=*/true, Visitor))
			{
				return false;
			}
		}

		return true;
	}
}

UPCGGraph* FPCGSubgraphHelpers::CollapseIntoSubgraph(UPCGGraph* InOriginalGraph, const TArray<UPCGNode*>& InNodesToCollapse, const TArray<UObject*>& InExtraNodesToCollapse, UPCGGraph* OptionalPreAllocatedGraph)
{
	FText OutFailReason;
	UPCGGraph* OutSubgraph = CollapseIntoSubgraphWithReason(InOriginalGraph, InNodesToCollapse, InExtraNodesToCollapse, OutFailReason, OptionalPreAllocatedGraph);
	if (!OutSubgraph)
	{
		UE_LOGF(LogPCG, Warning, "%ls", *OutFailReason.ToString());
	}

	return OutSubgraph;
}

UPCGGraph* FPCGSubgraphHelpers::CollapseIntoSubgraphWithReason(UPCGGraph* InOriginalGraph, const TArray<UPCGNode*>& InNodesToCollapse, const TArray<UObject*>& InExtraNodesToCollapse, FText& OutFailReason, UPCGGraph* OptionalPreAllocatedGraph, TSubclassOf<UPCGBaseSubgraphSettings> InSubgraphSettingsClass)
{
	// Don't collapse into a subgraph if you don't have at least 2 nodes
	if (!InOriginalGraph)
	{
		OutFailReason = LOCTEXT("InvalidGraph", "Original Graph is invalid.");
		return nullptr;
	}

	if (InNodesToCollapse.Num() < 2)
	{
		OutFailReason = LOCTEXT("CollapseLessThanTwoNodes", "Try to collapse less than 2 nodes.");
		return nullptr;
	}

	const FInstancedPropertyBag* GraphParameters = InOriginalGraph->GetUserParametersStruct();
	check(GraphParameters);

	PCGSubgraphHelpersExtra::FCollapsingInformation CollapseInfo = PCGSubgraphHelpersExtra::FCollapsingInformation::CreateAndInitialize(InNodesToCollapse, *GraphParameters, InOriginalGraph, InSubgraphSettingsClass);

	// If we have at most 1 valid node to collapse, just exit
	if (!CollapseInfo.HasEnoughValidNodes())
	{
		OutFailReason = LOCTEXT("ValidLessThanTwoNodes", "There were less than 2 PCG nodes selected, abort");
		return nullptr;
	}

	if (!CollapseInfo.ValidateCollapseIsNotIntroducingCycle())
	{
		OutFailReason = LOCTEXT("CycleDetected", "Collapsing would introduce a cycle (A -> B -> C situation when we try to collapse A and C, B would be connected to both ends). Abort.");
		return nullptr;
	}

	// Create a new subgraph if necessary.
	UPCGGraph* NewPCGGraph = OptionalPreAllocatedGraph;
	if (!NewPCGGraph)
	{
		NewPCGGraph = NewObject<UPCGGraph>();
	}

	CollapseInfo.SetNewGraphAndUpdatePositions(NewPCGGraph);

#if WITH_EDITOR
	// Disable all notifications for both graphs as we do our changes.
	InOriginalGraph->DisableNotificationsForEditor();
	NewPCGGraph->DisableNotificationsForEditor();
#endif // WITH_EDITOR

	CollapseInfo.AddDiscoveredGraphParameters();
	CollapseInfo.CopyNodes(InExtraNodesToCollapse);
	CollapseInfo.CopyAndRemapEmbeddedSubgraphs();
	CollapseInfo.ReconnectAllNodes();
	CollapseInfo.CleanUpOriginalEdges();
	CollapseInfo.DeleteNodes(InExtraNodesToCollapse);
	CollapseInfo.CreateSubgraphNodeAndConnectRemainingPins();

#if WITH_EDITOR
	// Trigger a structural change for original graph and flush all notifications for subgraph and re-enable notifications for both.
	InOriginalGraph->NotifyGraphChanged(EPCGChangeType::Structural);
	NewPCGGraph->NotifyGraphChanged(EPCGChangeType::Structural);

	InOriginalGraph->EnableNotificationsForEditor();
	NewPCGGraph->EnableNotificationsForEditor();
#endif // WITH_EDITOR

	return NewPCGGraph;
}

UPCGNode* FPCGSubgraphHelpers::SpawnNodeAndConnect(UPCGGraph* InGraph, UPCGPin* InUpstreamPin, UPCGPin* InDownstreamPin, TSubclassOf<UPCGSettings> InSettingsClass, const TFunction<bool(UPCGSettings*)>& PostSettingsCreation)
{
	check(InGraph && InUpstreamPin && InDownstreamPin && InSettingsClass);
	UPCGSettings* Settings;
	UPCGNode* NewNode = InGraph->AddNodeOfType(InSettingsClass, Settings);
	
	if (NewNode)
	{
		if (PostSettingsCreation.IsSet() && PostSettingsCreation(Settings))
		{
			NewNode->UpdateAfterSettingsChangeDuringCreation();	
		}
		
		UPCGNode* UpstreamNode = InUpstreamPin->Node;
		UPCGNode* DownstreamNode = InDownstreamPin->Node;
		check(UpstreamNode && DownstreamNode);

#if WITH_EDITOR
		// Put the new node between A & B but make it stay within a radius of B to keep things tidy.
		NewNode->PositionX = (UpstreamNode->PositionX + DownstreamNode->PositionX) / 2;
		NewNode->PositionY = (UpstreamNode->PositionY + DownstreamNode->PositionY) / 2;

		// A hand tweaked distance that keeps it reasonably close.
		constexpr float MaxDistFromB = 200;
		const FVector2D OffsetFromB(NewNode->PositionX - DownstreamNode->PositionX, NewNode->PositionY - DownstreamNode->PositionY);
		const float Dist = OffsetFromB.Length();
		if (Dist > MaxDistFromB)
		{
			const float Scale = MaxDistFromB / Dist;
			NewNode->PositionX = DownstreamNode->PositionX + Scale * OffsetFromB.X;
			NewNode->PositionY = DownstreamNode->PositionY + Scale * OffsetFromB.Y;
		}
#endif // WITH_EDITOR

		// Connect by default the first in and out pins of the new node.
		if (ensure(!NewNode->GetInputPins().IsEmpty()))
		{
			check(NewNode->GetInputPins()[0]);
			const FName NewNodeInputPinLabel = NewNode->GetInputPins()[0]->Properties.Label;
			UpstreamNode->AddEdgeTo(InUpstreamPin->Properties.Label, NewNode, NewNodeInputPinLabel);
		}

		if (ensure(!NewNode->GetOutputPins().IsEmpty()))
		{
			check(NewNode->GetOutputPins()[0]);
			const FName NewNodeOutputPinLabel = NewNode->GetOutputPins()[0]->Properties.Label;
			NewNode->AddEdgeTo(NewNodeOutputPinLabel, DownstreamNode, InDownstreamPin->Properties.Label);
		}
	}
	
	return NewNode;
}

#undef LOCTEXT_NAMESPACE