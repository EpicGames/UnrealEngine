// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGTagHelpers.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

namespace PCG
{
namespace Private
{
	FParseTagResult::FParseTagResult(const FString& InTag)
	{
		*this = ParseTag(InTag);
	}

	FParseTagResult::FParseTagResult(FName InTag)
	{
		*this = ParseTag(InTag);
	}

	FParseTagResult ParseTag(const FString& InTag)
	{
		FParseTagResult Result;

		int32 DividerPosition = INDEX_NONE;
		if (InTag.FindChar(':', DividerPosition))
		{
			FString LeftSide = InTag.Left(DividerPosition);
			FString RightSide = InTag.RightChop(DividerPosition + 1);

			if (LeftSide.IsEmpty())
			{
				// Tag doesn't have an attribute - ignore
			}
			else
			{
				if (RightSide.IsNumeric())
				{
					Result.NumericValue = FCString::Atod(*RightSide);
				}
				else if (RightSide.Equals(TEXT("True"), ESearchCase::IgnoreCase))
				{
					Result.BooleanValue = true;
				}
				else if (RightSide.Equals(TEXT("False"), ESearchCase::IgnoreCase))
				{
					Result.BooleanValue = false;
				}

				Result.Attribute = MoveTemp(LeftSide);
				Result.Value = MoveTemp(RightSide);
			}
		}
		else
		{
			Result.Attribute = InTag;
		}

		// Finally, sanitize the attribute name if needed
		FString OriginalAttribute = Result.Attribute;
		if (FPCGMetadataAttributeBase::SanitizeName(Result.Attribute))
		{
			Result.OriginalAttribute = MoveTemp(OriginalAttribute);
		}

		return Result;
	}

	FParseTagResult ParseTag(FName InTag)
	{
		return ParseTag(InTag.ToString());
	}

	bool CreateAttributeFromTag(const FString& InTag, UPCGMetadata* InMetadata, FParseTagResult* OutResult)
	{
		FParseTagResult TagData(InTag);
		const bool bCreateSuccess = CreateAttributeFromTag(TagData, InMetadata);

		if (OutResult)
		{
			*OutResult = MoveTemp(TagData);
		}

		return bCreateSuccess;
	}

	bool CreateAttributeFromTag(const FParseTagResult& InTagData, UPCGMetadata* InMetadata)
	{
		return SetAttributeFromTag(InTagData, InMetadata, PCGInvalidEntryKey, ESetAttributeFromTagFlags::OverwriteAttributeIfDifferentType);
	}

	bool SetAttributeFromTag(const FString& InTag, UPCGMetadata* InMetadata, PCGMetadataEntryKey InKey, ESetAttributeFromTagFlags Flags, FParseTagResult* OutResult, TOptional<FName> OptionalAttributeName)
	{
		check(InMetadata);
		return SetAttributeFromTag(InTag, InMetadata->GetDefaultMetadataDomain(), InKey, Flags, OutResult, std::move(OptionalAttributeName));
	}
	
	bool SetAttributeFromTag(const FParseTagResult& InTagData, UPCGMetadata* InMetadata, PCGMetadataEntryKey InKey, ESetAttributeFromTagFlags Flags, TOptional<FName> OptionalAttributeName)
	{
		check(InMetadata);
		return SetAttributeFromTag(InTagData, InMetadata->GetDefaultMetadataDomain(), InKey, Flags, std::move(OptionalAttributeName));
	}

	bool SetAttributeFromTag(const FString& InTag, FPCGMetadataDomain* InMetadata, PCGMetadataEntryKey InEntryKey, ESetAttributeFromTagFlags Flags, FParseTagResult* OutResult, TOptional<FName> OptionalAttributeName)
	{
		FParseTagResult TagData = ParseTag(InTag);
		const bool bSetSuccess = SetAttributeFromTag(TagData, InMetadata, InEntryKey, Flags, std::move(OptionalAttributeName));

		if (OutResult)
		{
			*OutResult = MoveTemp(TagData);
		}

		return bSetSuccess;
	}

	bool SetAttributeFromTag(const FParseTagResult& TagData, FPCGMetadataDomain* InMetadata, PCGMetadataEntryKey InEntryKey, ESetAttributeFromTagFlags Flags, TOptional<FName> OptionalAttributeName)
	{
		check(InMetadata);

		if (TagData.IsValid())
		{
			const FName AttributeName = OptionalAttributeName.Get(/*DefaultValue=*/ *TagData.Attribute);

			auto SetValue = [InMetadata, &AttributeName, InEntryKey, Flags](auto DefaultValue, auto Value)
			{
				using AttributeType = decltype(Value);

				const bool bCanCreateAttribute = !!((Flags & ESetAttributeFromTagFlags::CreateAttribute) | (Flags & ESetAttributeFromTagFlags::OverwriteAttributeIfDifferentType));
				const bool bCanOverwriteAttribute = !!(Flags & ESetAttributeFromTagFlags::OverwriteAttributeIfDifferentType);

				FPCGMetadataAttribute<AttributeType>* Attribute = nullptr;

				if (bCanCreateAttribute)
				{
					Attribute = InMetadata->FindOrCreateAttribute<AttributeType>(AttributeName, DefaultValue, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/bCanOverwriteAttribute);
				}
				else
				{
					Attribute = InMetadata->GetMutableTypedAttribute<AttributeType>(AttributeName);
				}

				if (Attribute)
				{
					if (InEntryKey != PCGInvalidEntryKey)
					{
						Attribute->SetValue(InEntryKey, Value);
					}
					else if (!!(Flags & ESetAttributeFromTagFlags::SetDefaultValue))
					{
						Attribute->SetDefaultValue(Value);
					}

					return true;
				}
				else
				{
					return false;
				}
			};

			if (TagData.HasNumericValue())
			{
				return SetValue(0.0, TagData.NumericValue.GetValue());
			}
			else if (TagData.HasBooleanValue())
			{
				return SetValue(false, TagData.BooleanValue.GetValue());
			}
			else if (TagData.HasValue())
			{
				return SetValue(FString(), TagData.Value.GetValue());
			}
			else // Default boolean tag, assumes default is false.
			{
				return SetValue(false, true);
			}
		}
		
		return false;
	}

	// DEPRECATED 5.6
	bool SetAttributeFromTag(const FString& InTag, UPCGMetadata* InMetadata, PCGMetadataEntryKey InEntryKey, bool bInCanCreateAttribute, FParseTagResult* OutResult)
	{
		return SetAttributeFromTag(InTag, InMetadata, InEntryKey, bInCanCreateAttribute ? ESetAttributeFromTagFlags::OverwriteAttributeIfDifferentType : ESetAttributeFromTagFlags::None, OutResult);
	}

	// DEPRECATED 5.6
	bool SetAttributeFromTag(const FParseTagResult& TagData, UPCGMetadata* InMetadata, PCGMetadataEntryKey InEntryKey, bool bInCanCreateAttribute)
	{
		return SetAttributeFromTag(TagData, InMetadata, InEntryKey, bInCanCreateAttribute ? ESetAttributeFromTagFlags::OverwriteAttributeIfDifferentType : ESetAttributeFromTagFlags::None);
	}
}
}