// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGWorldQueryHelpers.h"

#include "PCGComponent.h"
#include "PCGSubsystem.h"
#include "Data/PCGPointData.h"
#include "Data/PCGWorldData.h"
#include "Grid/PCGLandscapeCache.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadataDomain.h"

#include "LandscapeProxy.h"
#include "StaticMeshResources.h"
#include "Algo/AnyOf.h"
#include "Components/BrushComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/HitResult.h"
#include "Engine/OverlapResult.h"
#include "Engine/StaticMesh.h"
#include "Kismet/GameplayStatics.h"
#include "Materials/MaterialInterface.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGWorldQueryHelpers)

namespace PCGWorldQueryHelpers
{
	FTransform GetOrthonormalImpactTransform(const FHitResult& Hit)
	{
		// Implementation note: this uses the same orthonormalization process as the landscape cache
		ensure(Hit.ImpactNormal.IsNormalized());
		const FVector ArbitraryVector = (FMath::Abs(Hit.ImpactNormal.Y) < (1.f - UE_KINDA_SMALL_NUMBER) ? FVector::YAxisVector : FVector::ZAxisVector);
		const FVector XAxis = (ArbitraryVector ^ Hit.ImpactNormal).GetSafeNormal();
		const FVector YAxis = (Hit.ImpactNormal ^ XAxis);

		return FTransform(XAxis, YAxis, Hit.ImpactNormal, Hit.ImpactPoint);
	}

	bool FilterRayHitResult(const FPCGFilterHitResultParams& InParams, bool& OutPassesFilter)
	{
		const UPrimitiveComponent* HitComponent = InParams.HitResult.GetComponent();
		if (!HitComponent)
		{
			// Not handled
			return false;
		}

		OutPassesFilter = FilterCommonQueryResults(InParams.QueryParams, HitComponent, InParams.OriginatingSource, InParams.FilteredObjectReferences);

		if (OutPassesFilter && InParams.QueryParams.bIgnoreBackfaceHits)
		{
			// If it's a landscape, we cull if the normal is negative in Z direction (landscape normal is always the +Z axis). If not, then we cull if the impact normal and the ray are headed in the same direction
			if (InParams.HitResult.bStartPenetrating || (InParams.HitResult.GetActor() && InParams.HitResult.GetActor()->IsA<ALandscapeProxy>() && InParams.HitResult.ImpactNormal.Z < 0) || (InParams.HitResult.TraceEnd - InParams.HitResult.TraceStart).Dot(InParams.HitResult.ImpactNormal) > 0)
			{
				OutPassesFilter = false;
			}
		}

		// Handled
		return true;
	}

	bool FilterOverlapResult(const FPCGFilterOverlapResultParams& InParams, bool& OutPassesFilter)
	{
		const UPrimitiveComponent* OverlapComponent = InParams.OverlapResult.GetComponent();
		if (!OverlapComponent)
		{
			// Not handled
			return false;
		}

		OutPassesFilter = FilterCommonQueryResults(InParams.QueryParams, OverlapComponent, InParams.OriginatingSource, InParams.FilteredObjectReferences);

		// Handled
		return true;
	}

	bool FilterCommonQueryResults(
		const FPCGWorldCommonQueryParams& QueryParams,
		const UPrimitiveComponent* TriggeredComponent,
		const TWeakInterfacePtr<IPCGGraphExecutionSource> OriginatingSource,
		const TSet<FObjectKey>& FilteredObjectReferences)
	{
		check(TriggeredComponent);

		// Skip invisible walls / triggers / volumes
		if (TriggeredComponent->IsA<UBrushComponent>())
		{
			return false;
		}

		// Skip "No collision" type actors
		if (!TriggeredComponent->IsQueryCollisionEnabled() || TriggeredComponent->GetCollisionResponseToChannel(QueryParams.CollisionChannel) != ECR_Block)
		{
			return false;
		}

		// Skip to-be-cleaned-up PCG-created objects
		if (TriggeredComponent->ComponentHasTag(PCGHelpers::MarkedForCleanupPCGTag) || (TriggeredComponent->GetOwner() && TriggeredComponent->GetOwner()->ActorHasTag(PCGHelpers::MarkedForCleanupPCGTag)))
		{
			return false;
		}

		// Optionally skip all PCG created objects
		if (QueryParams.bIgnorePCGHits && (TriggeredComponent->ComponentHasTag(PCGHelpers::DefaultPCGTag) || (TriggeredComponent->GetOwner() && TriggeredComponent->GetOwner()->ActorHasTag(PCGHelpers::DefaultPCGActorTag))))
		{
			return false;
		}

		// Skip self-generated PCG objects optionally
		const UObject* OriginatingObject = Cast<UObject>(OriginatingSource.Get());
		if (QueryParams.bIgnoreSelfHits && OriginatingObject && TriggeredComponent->ComponentTags.Contains(OriginatingObject->GetFName()))
		{
			return false;
		}

		// Additional filter as provided in the QueryParams base class
		if (QueryParams.ActorTagFilter != EPCGWorldQueryFilter::None
			|| QueryParams.ActorClassFilter != EPCGWorldQueryFilter::None
			|| QueryParams.ActorFilterFromInput != EPCGWorldQueryFilter::None)
		{
			AActor* Actor = TriggeredComponent->GetOwner();
			if (!Actor)
			{
				return false;
			}
			
			bool bSoftInclude = false;
			bool bHasIncludeFilter = false;
			bool bHardExclude = false;

			// Exclude and requires will force the exclude if the criteria doesn't match (Hard exclude)
			// Include and requires will have a soft include if the criteria does match
			// Also keep track if we have an include filter. If we didn't, and we have not a hard exclude, it should be included (if we had just exclude filters).
			auto UpdateIncludeExcludeStates = [&bSoftInclude, &bHardExclude, &bHasIncludeFilter](const EPCGWorldQueryFilter Filter, const bool bFoundMatch)
			{
				bHardExclude |= ((Filter == EPCGWorldQueryFilter::Exclude && bFoundMatch) || (Filter == EPCGWorldQueryFilter::Require && !bFoundMatch));
				
				const bool bIsIncludeFilter = (Filter == EPCGWorldQueryFilter::Include) || (Filter == EPCGWorldQueryFilter::Require);
				bSoftInclude |= (bIsIncludeFilter && bFoundMatch);
				bHasIncludeFilter |= bIsIncludeFilter;
			};

			if (QueryParams.ActorTagFilter != EPCGWorldQueryFilter::None)
			{
				const bool bFoundMatch = Algo::AnyOf(Actor->Tags, [QueryParams](const FName Tag) { return QueryParams.ParsedActorTagsList.Contains(Tag); });
				UpdateIncludeExcludeStates(QueryParams.ActorTagFilter, bFoundMatch);
			}

			// No need to check if we already know we are going to discard it.
			if (!bHardExclude && QueryParams.ActorClassFilter != EPCGWorldQueryFilter::None)
			{
				const bool bFoundMatch = Actor->GetClass() && Actor->GetClass()->IsChildOf(QueryParams.ActorClass);
				UpdateIncludeExcludeStates(QueryParams.ActorClassFilter, bFoundMatch);
			}

			if (!bHardExclude && QueryParams.ActorFilterFromInput != EPCGWorldQueryFilter::None)
			{
				const bool bFoundMatch = FilteredObjectReferences.Contains(FObjectKey(Actor));
				UpdateIncludeExcludeStates(QueryParams.ActorFilterFromInput, bFoundMatch);
			}

			if (bHardExclude || (!bSoftInclude && bHasIncludeFilter))
			{
				return false;
			}
		}

		// Landscape or not, include it if this is true
		if (QueryParams.SelectLandscapeHits == EPCGWorldQuerySelectLandscapeHits::Include)
		{
			return true;
		}

		bool bTriggeredOnLandscape = TriggeredComponent->GetOwner() && TriggeredComponent->GetOwner()->IsA<ALandscapeProxy>();

		// If excluding, skip. If requiring, and it's not a landscape, skip.
		if ((bTriggeredOnLandscape && QueryParams.SelectLandscapeHits == EPCGWorldQuerySelectLandscapeHits::Exclude) ||
			(!bTriggeredOnLandscape && QueryParams.SelectLandscapeHits == EPCGWorldQuerySelectLandscapeHits::Require))
		{
			return false;
		}

		return true;
	}

	TOptional<FHitResult> FilterRayHitResults(
		const FPCGWorldRaycastQueryParams& QueryParams,
		const TWeakInterfacePtr<IPCGGraphExecutionSource> OriginatingSource,
		const TArray<FHitResult>& HitResults,
		const TSet<FObjectKey>& FilteredObjectReferences)
	{
		const FPCGPhysicsRegistry& PhysicsRegistry = FPCGModule::GetConstPhysicsRegistry();

		for (const FHitResult& Hit : HitResults)
		{
			const FPCGFilterHitResultParams FilterHitParams(Hit, QueryParams, OriginatingSource, FilteredObjectReferences);

			if (PhysicsRegistry.FilterHitResult(FilterHitParams))
			{
				return Hit;
			}
		}

		return {};
	}

	TOptional<FOverlapResult> FilterOverlapResults(
		const FPCGWorldVolumetricQueryParams& QueryParams,
		const TWeakInterfacePtr<IPCGGraphExecutionSource> OriginatingSource,
		const TArray<FOverlapResult>& OverlapResults,
		const TSet<FObjectKey>& FilteredObjectReferences)
	{
		const FPCGPhysicsRegistry& PhysicsRegistry = FPCGModule::GetConstPhysicsRegistry();

		for (const FOverlapResult& Overlap : OverlapResults)
		{
			const FPCGFilterOverlapResultParams FilterOverlapParams(Overlap, QueryParams, OriginatingSource, FilteredObjectReferences);

			if (PhysicsRegistry.FilterOverlapResult(FilterOverlapParams))
			{
				return Overlap;
			}
		}

		return {};
	}

	bool CreateRayHitAttributes(const FPCGWorldRaycastQueryParams& QueryParams, UPCGMetadata* OutMetadata)
	{
		return CreateRayHitAttributes(QueryParams, OutMetadata ? OutMetadata->GetDefaultMetadataDomain() : nullptr);
	}

	bool CreateRayHitAttributes(const FPCGWorldRaycastQueryParams& QueryParams, FPCGMetadataDomain* OutMetadata)
	{
		if (!OutMetadata)
		{
			return false;
		}

		auto CreateAttribute = [OutMetadata]<typename Type>(FName AttributeName, bool bShouldCreate, const Type& DefaultValue) -> bool
		{
			return !bShouldCreate || OutMetadata->FindOrCreateAttribute<Type>(AttributeName, DefaultValue, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true);
		};

		bool bResult = true;
		// Default T/F Impact to true, as most cases misses will be ignored completely
		bResult &= CreateAttribute(PCGWorldQueryConstants::ImpactAttribute, QueryParams.bGetImpact, true);
		bResult &= CreateAttribute(PCGWorldQueryConstants::ImpactPointAttribute, QueryParams.bGetImpactPoint, FVector::ZeroVector);
		bResult &= CreateAttribute(PCGWorldQueryConstants::ImpactNormalAttribute, QueryParams.bGetImpactNormal, FVector::ZeroVector);
		bResult &= CreateAttribute(PCGWorldQueryConstants::ImpactReflectionAttribute, QueryParams.bGetReflection, FVector::ZeroVector);
		bResult &= CreateAttribute(PCGWorldQueryConstants::ImpactDistanceAttribute, QueryParams.bGetDistance, 0.0);
		bResult &= CreateAttribute(PCGWorldQueryConstants::LocalImpactPointAttribute, QueryParams.bGetLocalImpactPoint, FVector::ZeroVector);
		bResult &= CreateAttribute(PCGPointDataConstants::ActorReferenceAttribute, QueryParams.bGetReferenceToActorHit, FSoftObjectPath());
		bResult &= CreateAttribute(PCGWorldQueryConstants::PhysicalMaterialReferenceAttribute, QueryParams.bGetReferenceToPhysicalMaterial, FSoftObjectPath());
		bResult &= CreateAttribute(PCGWorldQueryConstants::RenderMaterialReferenceAttribute, QueryParams.bGetReferenceToRenderMaterial, FSoftObjectPath());
		bResult &= CreateAttribute(PCGWorldQueryConstants::RenderMaterialIndexAttribute, QueryParams.bTraceComplex && QueryParams.bGetRenderMaterialIndex, int32{0});
		bResult &= CreateAttribute(PCGWorldQueryConstants::StaticMeshReferenceAttribute, QueryParams.bGetReferenceToStaticMesh, FSoftObjectPath());
		bResult &= CreateAttribute(PCGWorldQueryConstants::ElementIndexAttribute, QueryParams.bGetElementIndex, int32{0});
		bResult &= CreateAttribute(PCGWorldQueryConstants::UVCoordAttribute, QueryParams.bTraceComplex && QueryParams.bGetUVCoords, FVector2D::ZeroVector);
		bResult &= CreateAttribute(PCGWorldQueryConstants::FaceIndexAttribute, QueryParams.bTraceComplex && QueryParams.bGetFaceIndex, int32{0});
		bResult &= CreateAttribute(PCGWorldQueryConstants::SectionIndexAttribute, QueryParams.bTraceComplex && QueryParams.bGetSectionIndex, int32{0});

		return bResult;
	}

	bool ApplyRayMissMetadata(const FPCGWorldRaycastQueryParams& QueryParams, int64& OutMetadataEntry, UPCGMetadata* OutMetadata)
	{
		if (!QueryParams.bGetImpact)
		{
			return true;
		}

		if (!OutMetadata)
		{
			return false;
		}

		if (FPCGMetadataAttribute<bool>* Attribute = OutMetadata->FindOrCreateAttribute<bool>(PCGWorldQueryConstants::ImpactAttribute, true, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true))
		{
			OutMetadata->InitializeOnSet(OutMetadataEntry);
			Attribute->SetValue(OutMetadataEntry, false);
			return true;
		}

		return false;
	}

	template<class T>
	bool ApplyAttribute(const FPCGApplyHitResultAttributesParams& InParams, FName AttributeName, const T& Value, bool bShouldApply = true)
	{
		if (!bShouldApply)
		{
			return true;
		}

		if (FPCGMetadataAttribute<T>* Attribute = InParams.Metadata.FindOrCreateAttribute<T>(AttributeName))
		{
			InParams.Metadata.InitializeOnSet(InParams.MetadataEntry);
			Attribute->SetValue(InParams.MetadataEntry, Value);
			return true;
		}

		return false;
	};

	bool ApplyHitResultAttributes(const FPCGApplyHitResultAttributesParams& InParams, bool& bOutSuccess)
	{
		bOutSuccess = true;

		AActor* HitActor = InParams.HitResult.GetActor();
		UPrimitiveComponent* HitComponent = InParams.HitResult.GetComponent();

		if (!HitActor && !HitComponent)
		{
			// Unhandled
			return false;
		}

		bOutSuccess &= ApplyAttribute(InParams, PCGPointDataConstants::ActorReferenceAttribute, FSoftObjectPath(HitActor), InParams.QueryParams.bGetReferenceToActorHit);

		// Handle landscape related hit result
		if (ALandscapeProxy* Landscape = Cast<ALandscapeProxy>(HitActor))
		{
			if (const UMaterialInterface* RenderMaterial = Landscape->GetLandscapeMaterial())
			{
				bOutSuccess &= ApplyAttribute(InParams, PCGWorldQueryConstants::RenderMaterialReferenceAttribute, FSoftObjectPath(RenderMaterial), InParams.QueryParams.bGetReferenceToRenderMaterial);
			}

			if (InParams.QueryParams.bApplyMetadataFromLandscape && InParams.World && InParams.World->GetSubsystem<UPCGSubsystem>())
			{
				if (UPCGLandscapeCache* LandscapeCache = InParams.World->GetSubsystem<UPCGSubsystem>()->GetLandscapeCache())
				{
					const TArray<FName> Layers = LandscapeCache->GetLayerNames(Landscape);
					for (const FName& Layer : Layers)
					{
						InParams.Metadata.FindOrCreateAttribute<float>(Layer);
					}

					LandscapeCache->SampleMetadataOnPoint(Landscape, InParams.Transform, InParams.MetadataEntry, &InParams.Metadata);
				}
			}
		}
		// Handle primitive component related hit result
		else if (HitComponent)
		{
			if (InParams.QueryParams.bGetLocalImpactPoint)
			{
				const FVector LocalHitLocation = HitComponent->GetComponentToWorld().InverseTransformPosition(InParams.HitResult.ImpactPoint);
				bOutSuccess &= ApplyAttribute(InParams, PCGWorldQueryConstants::LocalImpactPointAttribute, LocalHitLocation);
			}

			// Handle static mesh component related hit result
			if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(HitComponent))
			{
				if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
				{
					bOutSuccess &= ApplyAttribute(InParams, PCGWorldQueryConstants::StaticMeshReferenceAttribute, FSoftObjectPath(StaticMesh), InParams.QueryParams.bGetReferenceToStaticMesh);
				}

				// Implementation note: FaceIndex will return -1 if complex queries are disabled
				bOutSuccess &= ApplyAttribute(InParams, PCGWorldQueryConstants::FaceIndexAttribute, InParams.HitResult.FaceIndex, InParams.QueryParams.bTraceComplex && InParams.QueryParams.bGetFaceIndex);

				if (InParams.QueryParams.bTraceComplex)
				{
					int32 SectionIndex = INDEX_NONE;
					int32 MaterialIndex = INDEX_NONE;
					UMaterialInterface* RenderMaterial = nullptr;

					// Attempt to get these values from LOD0, only if needed.
					if (InParams.QueryParams.bGetSectionIndex
						|| InParams.QueryParams.bGetRenderMaterialIndex
						|| (InParams.QueryParams.bGetReferenceToRenderMaterial && !InParams.QueryParams.bUseRenderMaterialIndex))
					{
						if (InParams.HitResult.FaceIndex != INDEX_NONE
							&& StaticMeshComponent->GetStaticMesh()
							&& StaticMeshComponent->GetStaticMesh()->GetRenderData()
							&& !StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources.IsEmpty())
						{
							const FStaticMeshLODResources& LOD = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[0];
							for (int32 CurrentIndex = 0; CurrentIndex < LOD.Sections.Num(); ++CurrentIndex)
							{
								const FStaticMeshSection& Section = LOD.Sections[CurrentIndex];
								const uint32 HitFaceIndex = static_cast<uint32>(InParams.HitResult.FaceIndex);

								// Find the section by iterating through the indices 3 at a time.
								const uint32 FirstFaceIndex = Section.FirstIndex / 3u;
								const uint32 LastFaceIndex = FirstFaceIndex + Section.NumTriangles - 1;

								// Find the section index
								if (HitFaceIndex >= FirstFaceIndex && HitFaceIndex <= LastFaceIndex)
								{
									SectionIndex = CurrentIndex;
									MaterialIndex = Section.MaterialIndex;
									break;
								}
							}
						}
					}

					if (InParams.QueryParams.bUseRenderMaterialIndex)  // Use explicit Render Material Index
					{
						RenderMaterial = HitComponent->GetMaterial(InParams.QueryParams.RenderMaterialIndex);
						bOutSuccess &= ApplyAttribute(InParams, PCGWorldQueryConstants::RenderMaterialIndexAttribute, InParams.QueryParams.RenderMaterialIndex, InParams.QueryParams.bGetRenderMaterialIndex);
					}
					else
					{
						RenderMaterial = HitComponent->GetMaterialFromCollisionFaceIndex(InParams.HitResult.FaceIndex, SectionIndex);
						bOutSuccess &= ApplyAttribute(InParams, PCGWorldQueryConstants::RenderMaterialIndexAttribute, MaterialIndex, InParams.QueryParams.bGetRenderMaterialIndex);
					}

					bOutSuccess &= ApplyAttribute(InParams, PCGWorldQueryConstants::SectionIndexAttribute, SectionIndex, InParams.QueryParams.bGetSectionIndex);
					bOutSuccess &= ApplyAttribute(InParams, PCGWorldQueryConstants::RenderMaterialReferenceAttribute, FSoftObjectPath(RenderMaterial), InParams.QueryParams.bGetReferenceToRenderMaterial);

					if (InParams.QueryParams.bGetUVCoords && UPhysicsSettings::Get()->bSupportUVFromHitResults)
					{
						FVector2D UVCoords;
						if (UGameplayStatics::FindCollisionUV(InParams.HitResult, InParams.QueryParams.UVChannel, UVCoords))
						{
							bOutSuccess &= ApplyAttribute(InParams, PCGWorldQueryConstants::UVCoordAttribute, UVCoords);
						}
					}
				}
				else // @todo_pcg: Address other primitive types here
				{
					if (InParams.QueryParams.bGetReferenceToRenderMaterial)
					{
						const UMaterialInterface* RenderMaterial = HitComponent->GetMaterial(InParams.QueryParams.RenderMaterialIndex);
						bOutSuccess &= RenderMaterial && ApplyAttribute(InParams, PCGWorldQueryConstants::RenderMaterialReferenceAttribute, FSoftObjectPath(RenderMaterial));
					}
				}
			}
		}

		// Handled
		return true;
	}

	// TODO: Add an option to create and apply ranges
	bool ApplyRayHitMetadata(
		const FHitResult& HitResult,
		const FPCGWorldRaycastQueryParams& QueryParams,
		const FVector& RayDirection,
		const FTransform& InTransform, 
		int64& OutMetadataEntry,
		UPCGMetadata* OutMetadata,
		TWeakObjectPtr<UWorld> World)
	{
		if (!OutMetadata)
		{
			return false;
		}

		FVector ReflectionVector = FVector::ZeroVector;
		if (QueryParams.bGetReflection)
		{
			ReflectionVector = RayDirection - 2.0 * (FVector(HitResult.ImpactNormal) | RayDirection) * FVector(HitResult.ImpactNormal);
			ReflectionVector.Normalize();
		}

		bool bResult = true;
		
		FPCGApplyHitResultAttributesParams Params(HitResult, QueryParams, RayDirection, InTransform, World.Get(), OutMetadataEntry, *OutMetadata);
		
		// Note: The T/F Impact attribute is true by default, so no need to set it directly.
		bResult &= ApplyAttribute(Params, PCGWorldQueryConstants::ImpactPointAttribute, FVector(HitResult.ImpactPoint), QueryParams.bGetImpactPoint);
		bResult &= ApplyAttribute(Params, PCGWorldQueryConstants::ImpactNormalAttribute, FVector(HitResult.ImpactNormal), QueryParams.bGetImpactNormal);
		bResult &= ApplyAttribute(Params, PCGWorldQueryConstants::ImpactReflectionAttribute, ReflectionVector, QueryParams.bGetReflection);
		bResult &= ApplyAttribute(Params, PCGWorldQueryConstants::ImpactDistanceAttribute, (HitResult.ImpactPoint - HitResult.TraceStart).Length(), QueryParams.bGetDistance);
		bResult &= ApplyAttribute(Params, PCGWorldQueryConstants::PhysicalMaterialReferenceAttribute, FSoftObjectPath(HitResult.PhysMaterial.Get()), QueryParams.bGetReferenceToPhysicalMaterial);
		bResult &= ApplyAttribute(Params, PCGWorldQueryConstants::ElementIndexAttribute, static_cast<int32>(HitResult.ElementIndex), QueryParams.bGetElementIndex);

		// Extra Apply based on hit type
		bResult &= FPCGModule::GetConstPhysicsRegistry().ApplyHitResultAttributes(Params);

		return bResult;
	}
}
