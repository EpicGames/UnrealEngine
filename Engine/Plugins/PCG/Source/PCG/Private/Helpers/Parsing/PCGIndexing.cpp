// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Parsing/PCGIndexing.h"

#include "Algo/MaxElement.h"

namespace PCGIndexing
{
	bool FPCGIndexRange::ContainsIndex(const int32 Index) const
	{
		return Index >= StartIndex && Index < EndIndex;
	}

	int32 FPCGIndexRange::GetIndexCount() const
	{
		return FMath::Max(EndIndex - StartIndex, 0);
	}

	bool FPCGIndexRange::operator<(const FPCGIndexRange& OtherRange) const
	{
		return StartIndex < OtherRange.StartIndex;
	}

	bool FPCGIndexRange::operator==(const FPCGIndexRange& OtherRange) const
	{
		return StartIndex == OtherRange.StartIndex && EndIndex == OtherRange.EndIndex;
	}

	bool FPCGIndexRange::operator!=(const FPCGIndexRange& OtherRange) const
	{
		return !operator==(OtherRange);
	}

	FPCGIndexCollection::FPCGIndexCollection(const TArrayView<const int32> InArrayView)
	{
		AddIndices(InArrayView, /*bGrowIfNeeded=*/true);
	}

	FPCGIndexCollection::FPCGIndexCollection(const TArrayView<const int32> InArrayView, const int32 InArraySize)
		: ArraySize(FMath::Max(InArraySize, 0))
	{
		AddIndices(InArrayView);
	}

	bool FPCGIndexCollection::AddRange(const int32 StartIndex, const int32 EndIndex)
	{
		if (!ensure(IsValid()))
		{
			return false;
		}

		const FPCGIndexRange NewRange = AdjustIndicesAndCreateRange(StartIndex, EndIndex);

		if (!RangeIsValid(NewRange))
		{
			return false;
		}

		// First element added, early out
		if (IndexRanges.IsEmpty())
		{
			IndexRanges.Add(NewRange);
			return true;
		}

		// Insert after the new start index is larger, so we can start overlap comparing at the index prior
		int32 InsertionIndex = 0;
		const int32 PreviousSize = IndexRanges.Num();
		while (InsertionIndex < PreviousSize && NewRange.StartIndex > IndexRanges[InsertionIndex].StartIndex)
		{
			++InsertionIndex;
		}

		IndexRanges.Insert(NewRange, InsertionIndex);

		// Start of the overlap checking
		const int32 OverlapIndex = FMath::Max(InsertionIndex - 1, 0);

		// Since they inserted in sorting order, insert and resolve overlaps until there are none
		do
		{
			if (!CheckOverlap(IndexRanges[OverlapIndex], IndexRanges[OverlapIndex + 1]))
			{
				break;
			}

			IndexRanges[OverlapIndex] = MergeRanges(IndexRanges[OverlapIndex], IndexRanges[OverlapIndex + 1]);
			IndexRanges.RemoveAt(OverlapIndex + 1);
		}
		while (OverlapIndex + 1 < IndexRanges.Num());

		return true;
	}

	bool FPCGIndexCollection::AddRange(const FPCGIndexRange& NewRange)
	{
		return AddRange(NewRange.StartIndex, NewRange.EndIndex);
	}

	bool FPCGIndexCollection::RangeIsValid(const FPCGIndexRange& Range) const
	{
		return IsValid()
			&& Range.StartIndex >= 0
			&& Range.StartIndex <= ArraySize
			&& Range.EndIndex >= 0
			&& Range.EndIndex <= ArraySize
			&& Range.StartIndex < Range.EndIndex;
	}

	bool FPCGIndexCollection::ContainsIndex(const int32 InIndex) const
	{
		if (!IsValid())
		{
			return false;
		}

		// Employ a binary search since the collection is already sorted - O(log n) lookup
		const int32 LowerIndex = Algo::LowerBoundBy(IndexRanges, InIndex, [](const FPCGIndexRange& Range) { return Range.EndIndex; });
		return LowerIndex < IndexRanges.Num() && IndexRanges[LowerIndex].ContainsIndex(InIndex);
	}

	int32 FPCGIndexCollection::GetArraySize() const
	{
		return ArraySize;
	}

	int32 FPCGIndexCollection::GetTotalRangeCount() const
	{
		return IndexRanges.Num();
	}

	int32 FPCGIndexCollection::GetTotalIndexCount() const
	{
		int32 TotalIndexCount = 0;

		for (const FPCGIndexRange& Range : IndexRanges)
		{
			check(Range.GetIndexCount() > 0);
			TotalIndexCount += Range.GetIndexCount();
		}

		return TotalIndexCount;
	}

	bool FPCGIndexCollection::IsValid() const
	{
		return ArraySize > 0;
	}

	bool FPCGIndexCollection::IsEmpty() const
	{
		return IndexRanges.IsEmpty();
	}

	void FPCGIndexCollection::ForEach(const TFunctionRef<bool(FPCGIndexRange)> Action) const
	{
		for (const FPCGIndexRange& Range : IndexRanges)
		{
			if (Action(Range))
			{
				break;
			}
		}
	}

	bool FPCGIndexCollection::operator==(const FPCGIndexCollection& Other) const
	{
		return (ArraySize == Other.ArraySize) && (IndexRanges == Other.IndexRanges);
	}

	FPCGIndexCollection& FPCGIndexCollection::operator+=(const FPCGIndexCollection& Other)
	{
		ArraySize = FMath::Max(ArraySize, Other.ArraySize);
		for (const FPCGIndexRange& Range : Other.IndexRanges)
		{
			AddRange(Range);
		}

		return *this;
	}

	FPCGIndexCollection& FPCGIndexCollection::operator+=(const TArrayView<const int32>& InArrayView)
	{
		AddIndices(InArrayView, /*bGrowIfNeeded=*/true);
		return *this;
	}

	FPCGIndexRange FPCGIndexCollection::AdjustIndicesAndCreateRange(int32 StartIndex, int32 EndIndex) const
	{
		// Convert both indices to a positive range
		StartIndex = (StartIndex >= 0) ? FMath::Min(StartIndex, ArraySize) : FMath::Min(ArraySize + StartIndex, ArraySize);
		EndIndex = (EndIndex >= 0) ? FMath::Min(EndIndex, ArraySize) : FMath::Min(ArraySize + EndIndex, ArraySize);

		return FPCGIndexRange(StartIndex, (EndIndex == StartIndex) ? EndIndex + 1 : EndIndex);
	}

	bool FPCGIndexCollection::CheckOverlap(const FPCGIndexRange& FirstRange, const FPCGIndexRange& SecondRange) const
	{
		check(RangeIsValid(FirstRange) && RangeIsValid(SecondRange));

		return FirstRange.StartIndex <= SecondRange.EndIndex && SecondRange.StartIndex <= FirstRange.EndIndex;
	}

	FPCGIndexRange FPCGIndexCollection::MergeRanges(const FPCGIndexRange& FirstRange, const FPCGIndexRange& SecondRange) const
	{
		// Safety check
		check(CheckOverlap(FirstRange, SecondRange));

		const int32 MergedStart = FMath::Min(FirstRange.StartIndex, SecondRange.StartIndex);
		const int32 MergedEnd = FMath::Max(FirstRange.EndIndex, SecondRange.EndIndex);

		return FPCGIndexRange(MergedStart, MergedEnd);
	}

	void FPCGIndexCollection::AddIndices(const TArrayView<const int32> InArrayView, const bool bGrowIfNeeded)
	{
		// @todo_pcg: Consider a divide and conquer approach for very large merges.
		if (!InArrayView.IsEmpty())
		{
			// If the indices contained in the array view are larger than the current array size, expand
			if (bGrowIfNeeded)
			{
				if (const int32* InMax = Algo::MaxElement(InArrayView); InMax && *InMax > 0)
				{
					ArraySize = FMath::Max(*InMax + 1, ArraySize);
				}
			}

			if (!IsValid())
			{
				// Edge Case: Negative indices only, which require the ArraySize to be positive.
				return;
			}

			// Remove invalid indices from the inbound array.
			TArray<int32, TInlineAllocator<128>> Indices(InArrayView);
			for (int32 i = Indices.Num() - 1; i >= 0; --i)
			{
				Indices[i] = (Indices[i] >= 0 ? Indices[i] : ArraySize + Indices[i]);
				// Invalid negative index... discard
				if (Indices[i] < 0)
				{
					Indices.RemoveAtSwap(i, EAllowShrinking::No);
				}
			}

			if (Indices.IsEmpty())
			{
				return;
			}

			// Sort the indices to make the range compilation easier.
			Indices.Sort();

			int32 RangeStart = Indices[0];
			int32 RangeEnd = RangeStart;

			// For the remaining indices, walk through and distill them into ranges.
			for (int32 i = 1; i < Indices.Num(); ++i)
			{
				// Skip duplicate indices (ranges can't contain them, but an array of indices might).
				if (Indices[i] <= RangeEnd)
				{
					continue;
				}

				// Grow the working range, if needed.
				if (Indices[i] == RangeEnd + 1)
				{
					RangeEnd = Indices[i];
				}
				else if (Indices[i] > RangeEnd + 1)
				{
					// Range complete - add to the collection and start a new one.
					AddRange(RangeStart, RangeEnd + 1);
					RangeStart = Indices[i];
					RangeEnd = Indices[i];
				}
			}

			// Add the final range
			AddRange(RangeStart, RangeEnd + 1);
		}
	}
} // namespace PCGIndexing
