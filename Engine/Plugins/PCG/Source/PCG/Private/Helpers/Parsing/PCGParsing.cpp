// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Parsing/PCGParsing.h"

#include "Internationalization/Regex.h"
#include "Templates/ValueOrError.h"

namespace PCGParser
{
	namespace Constants
	{
		static constexpr const TCHAR* IndexElementToken = TEXT(",");
		static constexpr const TCHAR* IndexRangeToken = TEXT(":");
		static constexpr const TCHAR* IndexRangePattern = TEXT("^(-?\\d*):(-?\\d*)$");
	}

	// TODO: Consider passing error messages as well for more precise feedback
	EPCGParserResult ParseIndexRanges(PCGIndexing::FPCGIndexCollection& OutIndexCollection, const FStringView& InputString)
	{
		FString PurgedInputString(InputString);
		PurgedInputString.RemoveSpacesInline();

		TArray<FString> ElementSubstrings;
		// First parse the string into comma separated array
		PurgedInputString.ParseIntoArray(ElementSubstrings, Constants::IndexElementToken, /*InCullEmpty=*/true);

		if (ElementSubstrings.IsEmpty())
		{
			return EPCGParserResult::EmptyExpression;
		}

		for (const FString& ElementSubstring : ElementSubstrings)
		{
			// Range case with the ':' character
			if (ElementSubstring.Contains(Constants::IndexRangeToken))
			{
				FRegexMatcher RangeMatcher(FRegexPattern(Constants::IndexRangePattern), ElementSubstring);
				if (RangeMatcher.FindNext())
				{
					FString StartIndexString = RangeMatcher.GetCaptureGroup(1);
					FString EndIndexString = RangeMatcher.GetCaptureGroup(2);

					if ((!StartIndexString.IsEmpty() && !StartIndexString.IsNumeric()) ||
						(!EndIndexString.IsEmpty() && !EndIndexString.IsNumeric()))
					{
						return EPCGParserResult::InvalidCharacter;
					}

					// Empty start index defaults to the array start
					const int32 StartRangeIndex = StartIndexString.IsEmpty() ? 0 : FCString::Atoi(*StartIndexString);
					// Empty end index defaults to the max value, to be truncated down to the array max
					const int32 EndRangeIndex = EndIndexString.IsEmpty() ? std::numeric_limits<int32>::max() : FCString::Atoi(*EndIndexString);

					if (!OutIndexCollection.AddRange(StartRangeIndex, EndRangeIndex))
					{
						return EPCGParserResult::InvalidExpression;
					}
				}
				else
				{
					return EPCGParserResult::InvalidExpression;
				}
			}
			else // Single index element case
			{
				if (!ElementSubstring.IsNumeric())
				{
					return EPCGParserResult::InvalidCharacter;
				}

				const int32 Index = FCString::Atoi(*ElementSubstring);
				// Adding the same index twice will adjust the end index appropriately
				if (!OutIndexCollection.AddRange(Index, Index))
				{
					return EPCGParserResult::InvalidExpression;
				}
			}
		}

		return EPCGParserResult::Success;
	}
}
