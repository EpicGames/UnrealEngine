// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceDataPackers/PCGInstanceDataPackerBase.h"

#include "PCGContext.h"
#include "PCGElement.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "MeshSelectors/PCGMeshSelectorBase.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGInstanceDataPackerBase)

#define LOCTEXT_NAMESPACE "PCGInstanceDataPackerBase"

bool PCGInstanceDataPackerBase::FPackedDataCommonParams::ValidateOffsets(int32 OffsetIndex, int32 NumOfCustomFloats) const
{
	if (!OptionalOffsets.IsValidIndex(OffsetIndex))
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidOffsetIndex", "[PCGInstanceDataPackerBase::ValidateOffsets] Offsets are expected to be at least {0} elements."), OffsetIndex), OptionalContext);
		return false;
	}
	else if (OptionalOffsets[OffsetIndex] < NumOfCustomFloats)
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidOffset", "[PCGInstanceDataPackerBase::ValidateOffsets] Offset {0} at index {1} is too small for the number of expected floats ({2})"), OptionalOffsets[OffsetIndex], OffsetIndex, NumOfCustomFloats), OptionalContext);
		return false;
	}
	else
	{
		return true;
	}
}

bool PCGInstanceDataPackerBase::FPackedDataCommonParams::ValidateNumOfFloats(int32 NumOfCustomFloats) const
{
	check(OutPackedCustomData);
	
	if (OutPackedCustomData->NumCustomDataFloats != 0 && OutPackedCustomData->NumCustomDataFloats < NumOfCustomFloats)
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("NotEnoughFloats", "[PCGInstanceDataPackerBase::ValidateNumOfFloats] Expected to have at least {0} custom floats, but was explicitly set to {1}, which is not enough."), NumOfCustomFloats, OutPackedCustomData->NumCustomDataFloats), OptionalContext);
		return false;
	}
	else
	{
		return true;
	}
}

void PCGInstanceDataPackerBase::FPackedDataCommonParams::ZeroOutCustomData()
{
	if (OutPackedCustomData)
	{
		OutPackedCustomData->CustomData.SetNumZeroed(OutPackedCustomData->NumCustomDataFloats * NumInstances);
	}
}

void PCGInstanceDataPackerBase::PackCustomDataFromAccessors(FPackedDataFromAccessorParams& InParams)
{
	check(InParams.CommonParams.OutPackedCustomData);
	FPCGPackedCustomData& OutPackedCustomData = *InParams.CommonParams.OutPackedCustomData;

	const int32 NumInstances = InParams.CommonParams.NumInstances;
	const int32 TotalNumCustomDataFloats = OutPackedCustomData.NumCustomDataFloats * NumInstances;

	// Zero out the custom data to avoid sending garbage data to the GPU in case of error.
	auto ZeroOut = [&InParams]()
	{
		InParams.CommonParams.ZeroOutCustomData();
	};

	if (!ensure(InParams.Accessors.Num() == InParams.AccessorKeys.Num() || (InParams.AccessorKeys.Num() == 1)))
	{
		ZeroOut();
		return;
	}

	if (!Algo::AllOf(InParams.Accessors) || !Algo::AllOf(InParams.AccessorKeys))
	{
		ZeroOut();
		return;
	}

	TConstArrayView<int32> OptionalOffsets = InParams.CommonParams.OptionalOffsets;

	// Validate that if we have custom offsets, they match the number of accessors
	if (!ensure(OptionalOffsets.IsEmpty() || (InParams.Accessors.Num() == OptionalOffsets.Num())))
	{
		// Zero out the custom data to avoid sending garbage data to the GPU.
		ZeroOut();
		return;
	}

	OutPackedCustomData.CustomData.SetNumUninitialized(TotalNumCustomDataFloats);

	if (TotalNumCustomDataFloats == 0)
	{
		return;
	}

	if (!OptionalOffsets.IsEmpty())
	{
		OutPackedCustomData.Mask.Init(false, OutPackedCustomData.NumCustomDataFloats);
	}

	// Index of the accessor's attribute/property for a single instance (e.g. if the
	// accessors are [float, FVector, float], AccessorPackedIndex will be [0, 1, 4])
	int AccessorPackedIndex = 0;
	const EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible;

	for (int AccessorIndex = 0; AccessorIndex < InParams.Accessors.Num(); ++AccessorIndex)
	{
		const IPCGAttributeAccessor* Accessor = InParams.Accessors[AccessorIndex].Get();
		const IPCGAttributeAccessorKeys* Keys = InParams.AccessorKeys.Num() == 1 ? InParams.AccessorKeys[0].Get() : InParams.AccessorKeys[AccessorIndex].Get();

		if (!OptionalOffsets.IsEmpty())
		{
			// Make sure that the offset if not writing on existing data.
			const int32 Offset = OptionalOffsets[AccessorIndex];
			const int32 TypePackingSize = GetTypePackingSize(Accessor->GetUnderlyingType());
			if (Offset < AccessorPackedIndex || (Offset + TypePackingSize) > OutPackedCustomData.NumCustomDataFloats)
			{
				PCGLog::LogErrorOnGraph(LOCTEXT("InvalidOffsets", "While packing custom data, offsets are overlapping or out of bounds."), InParams.CommonParams.OptionalContext);
				ZeroOut();
				return;
			}
			
			AccessorPackedIndex = OptionalOffsets[AccessorIndex];
			OutPackedCustomData.Mask.SetRange(AccessorPackedIndex, TypePackingSize, true);
		}

		check(Accessor && Keys);

		switch (Accessor->GetUnderlyingType())
		{
		case PCG::Private::MetadataTypes<bool>::Id: // fall-through
		case PCG::Private::MetadataTypes<float>::Id: // fall-through
		case PCG::Private::MetadataTypes<double>::Id: // fall-through
		case PCG::Private::MetadataTypes<int32>::Id: // fall-through
		case PCG::Private::MetadataTypes<int64>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](float Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;
				OutPackedCustomData.CustomData[PackedIndex] = Value;
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<float>(*Keys, *Accessor, PackFloats, Flags, PCGMetadataElementCommon::DefaultChunkSize, NumInstances));

			AccessorPackedIndex += 1;

			break;
		}
		case PCG::Private::MetadataTypes<FRotator>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](const FRotator& Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;

				OutPackedCustomData.CustomData[PackedIndex + 0] = static_cast<float>(Value.Roll);
				OutPackedCustomData.CustomData[PackedIndex + 1] = static_cast<float>(Value.Pitch);
				OutPackedCustomData.CustomData[PackedIndex + 2] = static_cast<float>(Value.Yaw);
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<FRotator>(*Keys, *Accessor, PackFloats, Flags, PCGMetadataElementCommon::DefaultChunkSize, NumInstances));

			AccessorPackedIndex += 3;

			break;
		}
		case PCG::Private::MetadataTypes<FVector2D>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](const FVector2D& Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;

				OutPackedCustomData.CustomData[PackedIndex + 0] = static_cast<float>(Value.X);
				OutPackedCustomData.CustomData[PackedIndex + 1] = static_cast<float>(Value.Y);
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<FVector2D>(*Keys, *Accessor, PackFloats, Flags, PCGMetadataElementCommon::DefaultChunkSize, NumInstances));

			AccessorPackedIndex += 2;

			break;
		}
		case PCG::Private::MetadataTypes<FVector>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](const FVector& Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;

				OutPackedCustomData.CustomData[PackedIndex + 0] = static_cast<float>(Value.X);
				OutPackedCustomData.CustomData[PackedIndex + 1] = static_cast<float>(Value.Y);
				OutPackedCustomData.CustomData[PackedIndex + 2] = static_cast<float>(Value.Z);
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<FVector>(*Keys, *Accessor, PackFloats, Flags, PCGMetadataElementCommon::DefaultChunkSize, NumInstances));

			AccessorPackedIndex += 3;

			break;
		}
		case PCG::Private::MetadataTypes<FVector4>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](const FVector4& Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;

				OutPackedCustomData.CustomData[PackedIndex + 0] = static_cast<float>(Value.X);
				OutPackedCustomData.CustomData[PackedIndex + 1] = static_cast<float>(Value.Y);
				OutPackedCustomData.CustomData[PackedIndex + 2] = static_cast<float>(Value.Z);
				OutPackedCustomData.CustomData[PackedIndex + 3] = static_cast<float>(Value.W);
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<FVector4>(*Keys, *Accessor, PackFloats, Flags, PCGMetadataElementCommon::DefaultChunkSize, NumInstances));

			AccessorPackedIndex += 4;

			break;
		}
		case PCG::Private::MetadataTypes<FQuat>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](const FQuat& Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;

				OutPackedCustomData.CustomData[PackedIndex + 0] = static_cast<float>(Value.X);
				OutPackedCustomData.CustomData[PackedIndex + 1] = static_cast<float>(Value.Y);
				OutPackedCustomData.CustomData[PackedIndex + 2] = static_cast<float>(Value.Z);
				OutPackedCustomData.CustomData[PackedIndex + 3] = static_cast<float>(Value.W);
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<FQuat>(*Keys, *Accessor, PackFloats, Flags, PCGMetadataElementCommon::DefaultChunkSize, NumInstances));

			AccessorPackedIndex += 4;

			break;
		}
		default:
			break;
		}
	}
}

void PCGInstanceDataPackerBase::PackCustomDataFromAttributes(FPackedDataFromAttributesParams& InParams)
{
	check(InParams.CommonParams.OutPackedCustomData);

	if (!InParams.InData || InParams.CommonParams.NumInstances <= 0 || !InParams.InData->ConstMetadata())
	{
		return;
	}

	if (InParams.Attributes.IsEmpty() && !InParams.AttributeIdentifiers.IsEmpty())
	{
		// Extract Attributes and start again
		TArray<const FPCGMetadataAttributeBase*> Attributes;
		Attributes.Reserve(InParams.AttributeIdentifiers.Num());
		for (const FPCGAttributeIdentifier& AttributeIdentifier : InParams.AttributeIdentifiers)
		{
			const FPCGMetadataAttributeBase* Attribute = InParams.InData->ConstMetadata()->GetConstAttribute(AttributeIdentifier);
			if (Attribute)
			{
				Attributes.Add(Attribute);
			}
		}

		if (Attributes.IsEmpty())
		{
			// Nothing to do
			InParams.CommonParams.ZeroOutCustomData();
			return;
		}

		InParams.Attributes = Attributes;
		return PackCustomDataFromAttributes(InParams);
	}

	if (!Algo::AllOf(InParams.Attributes) || InParams.Attributes.IsEmpty())
	{
		InParams.CommonParams.ZeroOutCustomData();
		return;
	}

	if (!InParams.CommonParams.OptionalOffsets.IsEmpty() && (InParams.CommonParams.OptionalOffsets.Num() != InParams.Attributes.Num()))
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MismatchOffsetSelectors2", "[PCGInstanceDataPackerBase::PackCustomDataFromAttributes] There is a mismatch between the number of attributes ({0}) and offsets ({1})"),
			InParams.Attributes.Num(), InParams.CommonParams.OptionalOffsets.Num()), InParams.CommonParams.OptionalContext);
		
		InParams.CommonParams.ZeroOutCustomData();
		return;
	}

	// Verify if we have multi-domain
	const bool bHasMultiDomain = Algo::AnyOf(InParams.Attributes, [FirstAttribute = InParams.Attributes[0]](const FPCGMetadataAttributeBase* Attribute) { return Attribute->GetMetadataDomain() != FirstAttribute->GetMetadataDomain(); });

	if (bHasMultiDomain && !InParams.OptionalIndices.IsEmpty())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("MultiDomainWithIndices", "Provided attributes are on multiple domains, while providing custom indices. This is unsupported."), InParams.CommonParams.OptionalContext);
		InParams.CommonParams.ZeroOutCustomData();
		return;
	}
	
	TArray<TUniquePtr<const IPCGAttributeAccessor>> Accessors;
	TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> AccessorKeys;
	TArray<const PCGMetadataEntryKey> FilteredKeys;

	int32 NumCustomDataFloats = 0;

	// If we have optional indices, we need first to extract all the keys that will be read from.
	if (!InParams.OptionalIndices.IsEmpty())
	{
		TUniquePtr<const IPCGAttributeAccessorKeys> TempKey = PCGAttributeAccessorHelpers::CreateConstKeys(InParams.Attributes[0]);
		check(TempKey);
		
		TArray<const PCGMetadataEntryKey*> AllKeys;
		FilteredKeys.Empty(InParams.OptionalIndices.Num());

		if (TempKey->GetKeys<PCGMetadataEntryKey>(0, AllKeys))
		{
			Algo::Transform(InParams.OptionalIndices, FilteredKeys, [&AllKeys](const int32 Index) { return AllKeys.IsValidIndex(Index) ? *AllKeys[Index] : PCGInvalidEntryKey; });
			AccessorKeys.Emplace(MakeUnique<FPCGAttributeAccessorKeysEntries>(FilteredKeys));
		}
		else
		{
			AccessorKeys.Emplace(MoveTemp(TempKey));
		}
	}

	for (int32 i = 0; i < InParams.Attributes.Num(); ++i)
	{
		const FPCGMetadataAttributeBase* Attribute = InParams.Attributes[i];
		check(Attribute);
		if (!InParams.CommonParams.OptionalOffsets.IsEmpty())
		{
			if (!InParams.CommonParams.ValidateOffsets(i, NumCustomDataFloats))
			{
				return;
			}
			
			NumCustomDataFloats = InParams.CommonParams.OptionalOffsets[i];
		}
		
		const FPCGMetadataDomain* MetadataDomain = Attribute->GetMetadataDomain();
		
		if (Attribute && AddTypeToPacking(Attribute->GetTypeId(), NumCustomDataFloats))
		{
			Accessors.Emplace(PCGAttributeAccessorHelpers::CreateConstAccessor(Attribute, MetadataDomain));
			
			if (bHasMultiDomain)
			{
				AccessorKeys.Emplace(PCGAttributeAccessorHelpers::CreateConstKeys(Attribute));
			}
		}
	}

	if (Accessors.IsEmpty())
	{
		InParams.CommonParams.ZeroOutCustomData();
		return;
	}

	// When all is done, verify that the number of custom data float is valid.
	if (InParams.CommonParams.OutPackedCustomData->NumCustomDataFloats != 0)
	{
		if (!InParams.CommonParams.ValidateNumOfFloats(NumCustomDataFloats))
		{
			InParams.CommonParams.ZeroOutCustomData();
			return;
		}
	}
	else
	{
		InParams.CommonParams.OutPackedCustomData->NumCustomDataFloats = NumCustomDataFloats;
	}

	FPackedDataFromAccessorParams Params =
	{
		.CommonParams = InParams.CommonParams,
		.Accessors = MakeConstArrayView(Accessors),
		.AccessorKeys = MakeConstArrayView(AccessorKeys),
	};

	PackCustomDataFromAccessors(Params);
}

void PCGInstanceDataPackerBase::PackCustomData(FPackedDataParams& InParams)
{
	check(InParams.CommonParams.OutPackedCustomData);

	if (!InParams.InData || InParams.CommonParams.NumInstances <= 0 || !InParams.InData->ConstMetadata())
	{
		InParams.CommonParams.ZeroOutCustomData();
		return;
	}

	if (!InParams.CommonParams.OptionalOffsets.IsEmpty() && (InParams.CommonParams.OptionalOffsets.Num() != InParams.Selectors.Num()))
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MismatchOffsetSelectors", "[PCGInstanceDataPackerBase::PackCustomData] There is a mismatch between the number of selectors ({0}) and offsets ({1})"),
			InParams.Selectors.Num(), InParams.CommonParams.OptionalOffsets.Num()), InParams.CommonParams.OptionalContext);
		InParams.CommonParams.ZeroOutCustomData();
		return;
	}

	TArray<TUniquePtr<const IPCGAttributeAccessor>> Accessors;
	TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> AccessorKeys;

	int32 NumCustomDataFloats = 0;

	for (int32 i = 0; i < InParams.Selectors.Num(); ++i)
	{
		const FPCGAttributePropertyInputSelector& Selector = InParams.Selectors[i];

		if (!InParams.CommonParams.OptionalOffsets.IsEmpty())
		{
			if (!InParams.CommonParams.ValidateOffsets(i, NumCustomDataFloats))
			{
				InParams.CommonParams.ZeroOutCustomData();
				return;
			}
			
			NumCustomDataFloats = InParams.CommonParams.OptionalOffsets[i];
		}
		
		FPCGAttributePropertyInputSelector FinalSelector = Selector.CopyAndFixLast(InParams.InData);

		TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InParams.InData, FinalSelector);
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InParams.InData, FinalSelector);
		
		if (!Accessor || !Keys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(FinalSelector, InParams.CommonParams.OptionalContext);
			continue;
		}
		
		if (AddTypeToPacking(Accessor->GetUnderlyingType(), NumCustomDataFloats))
		{
			Accessors.Emplace(MoveTemp(Accessor));
			AccessorKeys.Emplace(MoveTemp(Keys));
		}
	}

	if (Accessors.IsEmpty())
	{
		InParams.CommonParams.ZeroOutCustomData();
		return;
	}

	// When all is done, verify that the number of custom data float is valid.
	if (InParams.CommonParams.OutPackedCustomData->NumCustomDataFloats != 0)
	{
		if (!InParams.CommonParams.ValidateNumOfFloats(NumCustomDataFloats))
		{
			InParams.CommonParams.ZeroOutCustomData();
			return;
		}
	}
	else
	{
		InParams.CommonParams.OutPackedCustomData->NumCustomDataFloats = NumCustomDataFloats;
	}

	FPackedDataFromAccessorParams Params =
	{
		.CommonParams = InParams.CommonParams,
		.Accessors = MakeConstArrayView(Accessors),
		.AccessorKeys = MakeConstArrayView(AccessorKeys)
	};

	PackCustomDataFromAccessors(Params);
}

bool PCGInstanceDataPackerBase::AddTypeToPacking(int TypeId, int32& OutNumCustomDataFloats)
{
	const int32 TypePackingSize = GetTypePackingSize(TypeId);
	if (TypePackingSize > 0)
	{
		OutNumCustomDataFloats += TypePackingSize;
		return true;
	}
	else
	{
		return false;
	}
}

int32 PCGInstanceDataPackerBase::GetTypePackingSize(int TypeId)
{
	switch (TypeId)
	{
	case PCG::Private::MetadataTypes<bool>::Id: // fall-through
	case PCG::Private::MetadataTypes<float>::Id: // fall-through
	case PCG::Private::MetadataTypes<double>::Id: // fall-through
	case PCG::Private::MetadataTypes<int32>::Id: // fall-through
	case PCG::Private::MetadataTypes<int64>::Id:
		return 1;
	case PCG::Private::MetadataTypes<FVector2D>::Id:
		return 2;
	case PCG::Private::MetadataTypes<FRotator>::Id: // fall-through
	case PCG::Private::MetadataTypes<FVector>::Id:
		return 3;
	case PCG::Private::MetadataTypes<FVector4>::Id: // fall-through
	case PCG::Private::MetadataTypes<FQuat>::Id:
		return 4;
	default:
		return 0;
	}
}

TConstArrayView<float> FPCGPackedCustomData::GetViewForIndex(int32 Index) const
{
	
	if (NumCustomDataFloats == 0 || (Index + 1) * NumCustomDataFloats > CustomData.Num())
	{
		return {};
	}
	else
	{
		return MakeConstArrayView(CustomData.GetData() + (Index * NumCustomDataFloats), NumCustomDataFloats);
	}
}

void FPCGPackedCustomData::Reset()
{
	NumCustomDataFloats = 0;
	CustomData.Reset();
	Mask.Reset();
}

void UPCGInstanceDataPackerBase::PackInstances_Implementation(FPCGContext& Context, const UPCGSpatialData* InSpatialData, const FPCGMeshInstanceList& InstanceList, FPCGPackedCustomData& OutPackedCustomData) const
{
	PCGE_LOG_C(Error, GraphAndLog, &Context, LOCTEXT("InstanceDataPackerBaseFailed", "Unable to execute InstanceDataPacker pure virtual base function, override the PackInstances function or use a default implementation."));
}

void UPCGInstanceDataPackerBase::PackCustomDataFromAttributes(const FPCGMeshInstanceList& InstanceList, const UPCGMetadata* Metadata, const TArray<FName>& AttributeNames, FPCGPackedCustomData& OutPackedCustomData) const
{
	const TArray<FPCGAttributeIdentifier> AttributeIdentifiers = FPCGAttributeIdentifier::TransformNameArray(AttributeNames);
	
	PCGInstanceDataPackerBase::FPackedDataFromAttributesParams Params =
	{
		.CommonParams =
		{
			.NumInstances = InstanceList.InstancesIndices.Num(),
			.OutPackedCustomData = &OutPackedCustomData,
		},
		.OptionalIndices = MakeConstArrayView(InstanceList.InstancesIndices),
		.InData = InstanceList.PointData.Get(),
		.AttributeIdentifiers = MakeConstArrayView(AttributeIdentifiers)
	};

	return PCGInstanceDataPackerBase::PackCustomDataFromAttributes(Params);
}

void UPCGInstanceDataPackerBase::PackCustomDataFromAttributes(const FPCGMeshInstanceList& InstanceList, const TArray<const FPCGMetadataAttributeBase*>& Attributes, FPCGPackedCustomData& OutPackedCustomData) const
{
	PCGInstanceDataPackerBase::FPackedDataFromAttributesParams Params =
	{
		.CommonParams =
		{
			.NumInstances = InstanceList.InstancesIndices.Num(),
			.OutPackedCustomData = &OutPackedCustomData,
		},
		.OptionalIndices = MakeConstArrayView(InstanceList.InstancesIndices),
		.InData = InstanceList.PointData.Get(),
		.Attributes = Attributes
	};

	return PCGInstanceDataPackerBase::PackCustomDataFromAttributes(Params);
}

void UPCGInstanceDataPackerBase::PackCustomDataFromAccessors(const FPCGMeshInstanceList& InstanceList,
	TArray<TUniquePtr<const IPCGAttributeAccessor>> Accessors,
	TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> AccessorKeys,
	FPCGPackedCustomData& OutPackedCustomData) const
{
	PCGInstanceDataPackerBase::FPackedDataFromAccessorParams Params =
	{
		.CommonParams =
		{
			.NumInstances = InstanceList.InstancesIndices.Num(),
			.OutPackedCustomData = &OutPackedCustomData,
		},
		.Accessors = MakeConstArrayView(Accessors),
		.AccessorKeys = MakeConstArrayView(AccessorKeys),
	};

	return PCGInstanceDataPackerBase::PackCustomDataFromAccessors(Params);
}

#undef LOCTEXT_NAMESPACE