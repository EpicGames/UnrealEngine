// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceDataPackers/PCGInstanceDataPackerByAttribute.h"

#include "PCGContext.h"
#include "PCGElement.h"
#include "Data/PCGSpatialData.h"
#include "InstanceDataPackers/PCGInstanceDataPackerBase.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "MeshSelectors/PCGMeshSelectorBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGInstanceDataPackerByAttribute)

#define LOCTEXT_NAMESPACE "PCGInstanceDataPackerByAttribute"

#if WITH_EDITOR
void UPCGInstanceDataPackerByAttribute::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (FName AttributeName : AttributeNames)
	{
		FPCGAttributePropertyInputSelector& Selector = AttributeSelectors.Emplace_GetRef();
		Selector.SetAttributeName(AttributeName);
	}

	AttributeNames.Reset();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR

void UPCGInstanceDataPackerByAttribute::PackInstances_Implementation(FPCGContext& Context, const UPCGSpatialData* InSpatialData, const FPCGMeshInstanceList& InstanceList, FPCGPackedCustomData& OutPackedCustomData) const
{
	if (!InSpatialData || !InSpatialData->Metadata)
	{
		PCGLog::InputOutput::LogInvalidInputDataError(&Context);
		return;
	}

	TArray<TUniquePtr<const IPCGAttributeAccessor>> SelectedAccessors;
	TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> SelectedKeys;

	SelectedAccessors.Reserve(AttributeSelectors.Num());
	SelectedKeys.Reserve(AttributeSelectors.Num());

	// Find attributes and calculate NumCustomDataFloats
	for (const FPCGAttributePropertyInputSelector& Selector : AttributeSelectors)
	{
		TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InSpatialData, Selector);
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys;
		const UPCGBasePointData* PointData = InstanceList.PointData.Get();
		TArray<const PCGMetadataEntryKey> ExtractedKeys;
		
		if (InSpatialData == PointData)
		{
			Keys = MakeUnique<const FPCGAttributeAccessorKeysPointsSubset>(PointData, InstanceList.InstancesIndices);
		}
		else
		{
			// Convert indices to entry keys
			ExtractedKeys.Reserve(InstanceList.InstancesIndices.Num());
			Algo::Transform(InstanceList.InstancesIndices, ExtractedKeys, [](const int32 Index) -> PCGMetadataEntryKey{ return Index; });
			Keys = MakeUnique<const FPCGAttributeAccessorKeysEntries>(ExtractedKeys);
		}

		if (!Accessor.IsValid() || !Keys.IsValid())
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(Selector, &Context);
			continue;
		}

		if (!AddTypeToPacking(Accessor->GetUnderlyingType(), OutPackedCustomData))
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("AttributeInvalidType", "Attribute/property '{0}' is not a valid type - skipped."), Selector.GetDisplayText()), &Context);
			continue;
		}

		SelectedAccessors.Add(std::move(Accessor));
		SelectedKeys.Add(std::move(Keys));
	}

	PackCustomDataFromAccessors(InstanceList, std::move(SelectedAccessors), std::move(SelectedKeys), OutPackedCustomData);
}

bool UPCGInstanceDataPackerByAttribute::GetAttributeNames(TArray<FName>* OutNames)
{
	if (OutNames)
	{
		for (const FPCGAttributePropertyInputSelector& AttributeSelector : AttributeSelectors)
		{
			OutNames->Add(AttributeSelector.GetAttributeName());
		}
	}

	return true;
}

TOptional<TConstArrayView<FPCGAttributePropertyInputSelector>> UPCGInstanceDataPackerByAttribute::GetAttributeSelectors() const
{
	return MakeConstArrayView(AttributeSelectors);
}

#undef LOCTEXT_NAMESPACE
