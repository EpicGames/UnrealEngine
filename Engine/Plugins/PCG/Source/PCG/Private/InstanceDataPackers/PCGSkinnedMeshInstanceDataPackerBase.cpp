// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceDataPackers/PCGSkinnedMeshInstanceDataPackerBase.h"

#include "PCGElement.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "MeshSelectors/PCGSkinnedMeshSelector.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSkinnedMeshInstanceDataPackerBase)

void UPCGSkinnedMeshInstanceDataPackerBase::PackInstances_Implementation(FPCGContext& Context, const UPCGSpatialData* InSpatialData, const FPCGSkinnedMeshInstanceList& InstanceList, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const
{
	PCGE_LOG_C(Error, GraphAndLog, &Context, NSLOCTEXT("PCGSkinnedMeshInstanceDataPackerBase", "SkinnedMeshInstanceDataPackerBaseFailed", "Unable to execute SkinnedMeshInstanceDataPacker pure virtual base function, override the PackInstances function or use a default implementation."));
}

bool UPCGSkinnedMeshInstanceDataPackerBase::AddTypeToPacking(int TypeId, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const
{
	switch (TypeId)
	{
	case PCG::Private::MetadataTypes<bool>::Id: // fall-through
	case PCG::Private::MetadataTypes<float>::Id: // fall-through
	case PCG::Private::MetadataTypes<double>::Id: // fall-through
	case PCG::Private::MetadataTypes<int32>::Id: // fall-through
	case PCG::Private::MetadataTypes<int64>::Id:
		OutPackedCustomData.NumCustomDataFloats += 1;
		break;
	case PCG::Private::MetadataTypes<FVector2D>::Id:
		OutPackedCustomData.NumCustomDataFloats += 2;
		break;
	case PCG::Private::MetadataTypes<FRotator>::Id: // fall-through
	case PCG::Private::MetadataTypes<FVector>::Id:
		OutPackedCustomData.NumCustomDataFloats += 3;
		break;
	case PCG::Private::MetadataTypes<FVector4>::Id: // fall-through
	case PCG::Private::MetadataTypes<FQuat>::Id:
		OutPackedCustomData.NumCustomDataFloats += 4;
		break;
	default:
		return false;
	}

	return true;
}

void UPCGSkinnedMeshInstanceDataPackerBase::PackCustomDataFromAttributes(const FPCGSkinnedMeshInstanceList& InstanceList, const UPCGMetadata* Metadata, const TArray<FName>& AttributeNames, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const
{
	if (!Metadata)
	{
		UE_LOGF(LogPCG, Error, "Invalid metadata");
		return;
	}

	TArray<const FPCGMetadataAttributeBase*> Attributes;
	for (const FName& AttributeName : AttributeNames)
	{
		Attributes.Add(Metadata->GetConstAttribute(AttributeName));
	}

	PackCustomDataFromAttributes(InstanceList, Attributes, OutPackedCustomData);
}

void UPCGSkinnedMeshInstanceDataPackerBase::PackCustomDataFromAttributes(const FPCGSkinnedMeshInstanceList& InstanceList, const TArray<const FPCGMetadataAttributeBase*>& Attributes, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const
{
	const UPCGPointData* PointData = InstanceList.PointData.Get();
	for (const int32 Index : InstanceList.InstancePointIndices)
	{
		const PCGMetadataEntryKey EntryKey = PointData ? PointData->GetPoints()[Index].MetadataEntry : Index;
		for (const FPCGMetadataAttributeBase* AttributeBase : Attributes)
		{
			check(AttributeBase);

			switch (AttributeBase->GetTypeId())
			{
			case PCG::Private::MetadataTypes<bool>::Id:
			{
				const FPCGMetadataAttribute<bool>* Attribute = static_cast<const FPCGMetadataAttribute<bool>*>(AttributeBase);
				check(Attribute);

				const bool Value = Attribute->GetValueFromItemKey(EntryKey);
				OutPackedCustomData.CustomData.Add(Value);
				break;
			}
			case PCG::Private::MetadataTypes<float>::Id:
			{
				const FPCGMetadataAttribute<float>* Attribute = static_cast<const FPCGMetadataAttribute<float>*>(AttributeBase);
				check(Attribute);

				const float Value = Attribute->GetValueFromItemKey(EntryKey);
				OutPackedCustomData.CustomData.Add(Value);
				break;
			}
			case PCG::Private::MetadataTypes<double>::Id:
			{
				const FPCGMetadataAttribute<double>* Attribute = static_cast<const FPCGMetadataAttribute<double>*>(AttributeBase);
				check(Attribute);

				const double Value = Attribute->GetValueFromItemKey(EntryKey);
				OutPackedCustomData.CustomData.Add(Value);
				break;
			}
			case PCG::Private::MetadataTypes<int32>::Id:
			{
				const FPCGMetadataAttribute<int32>* Attribute = static_cast<const FPCGMetadataAttribute<int32>*>(AttributeBase);
				check(Attribute);

				const float Value = static_cast<float>(Attribute->GetValueFromItemKey(EntryKey));
				OutPackedCustomData.CustomData.Add(Value);
				break;
			}
			case PCG::Private::MetadataTypes<int64>::Id:
			{
				const FPCGMetadataAttribute<int64>* Attribute = static_cast<const FPCGMetadataAttribute<int64>*>(AttributeBase);
				check(Attribute);

				const float Value = static_cast<float>(Attribute->GetValueFromItemKey(EntryKey));
				OutPackedCustomData.CustomData.Add(Value);
				break;
			}
			case PCG::Private::MetadataTypes<FRotator>::Id:
			{
				const FPCGMetadataAttribute<FRotator>* Attribute = static_cast<const FPCGMetadataAttribute<FRotator>*>(AttributeBase);
				check(Attribute);

				const FRotator Value = Attribute->GetValueFromItemKey(EntryKey);
				OutPackedCustomData.CustomData.Add(Value.Roll);
				OutPackedCustomData.CustomData.Add(Value.Pitch);
				OutPackedCustomData.CustomData.Add(Value.Yaw);
				break;
			}
			case PCG::Private::MetadataTypes<FVector2D>::Id:
			{
				const FPCGMetadataAttribute<FVector2D>* Attribute = static_cast<const FPCGMetadataAttribute<FVector2D>*>(AttributeBase);
				check(Attribute);

				const FVector2D Value = Attribute->GetValueFromItemKey(EntryKey);
				OutPackedCustomData.CustomData.Add(Value.X);
				OutPackedCustomData.CustomData.Add(Value.Y);
				break;
			}
			case PCG::Private::MetadataTypes<FVector>::Id:
			{
				const FPCGMetadataAttribute<FVector>* Attribute = static_cast<const FPCGMetadataAttribute<FVector>*>(AttributeBase);
				check(Attribute);

				const FVector Value = Attribute->GetValueFromItemKey(EntryKey);
				OutPackedCustomData.CustomData.Add(Value.X);
				OutPackedCustomData.CustomData.Add(Value.Y);
				OutPackedCustomData.CustomData.Add(Value.Z);
				break;
			}
			case PCG::Private::MetadataTypes<FVector4>::Id:
			{
				const FPCGMetadataAttribute<FVector4>* Attribute = static_cast<const FPCGMetadataAttribute<FVector4>*>(AttributeBase);
				check(Attribute);

				const FVector4 Value = Attribute->GetValueFromItemKey(EntryKey);
				OutPackedCustomData.CustomData.Add(Value.X);
				OutPackedCustomData.CustomData.Add(Value.Y);
				OutPackedCustomData.CustomData.Add(Value.Z);
				OutPackedCustomData.CustomData.Add(Value.W);
				break;
			}
			case PCG::Private::MetadataTypes<FQuat>::Id:
			{
				const FPCGMetadataAttribute<FQuat>* Attribute = static_cast<const FPCGMetadataAttribute<FQuat>*>(AttributeBase);
				check(Attribute);

				const FQuat Value = Attribute->GetValueFromItemKey(EntryKey);
				OutPackedCustomData.CustomData.Add(Value.X);
				OutPackedCustomData.CustomData.Add(Value.Y);
				OutPackedCustomData.CustomData.Add(Value.Z);
				OutPackedCustomData.CustomData.Add(Value.W);
				break;
			}
			default:
				break;
			}
		}
	}
}

void UPCGSkinnedMeshInstanceDataPackerBase::PackCustomDataFromAccessors(const FPCGSkinnedMeshInstanceList& InstanceList, TArray<TUniquePtr<const IPCGAttributeAccessor>> Accessors, TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> AccessorKeys, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const
{
	const int32 TotalNumCustomDataFloats = OutPackedCustomData.NumCustomDataFloats * InstanceList.Instances.Num();

	if (!ensure(Accessors.Num() == AccessorKeys.Num()))
	{
		// Zero out the custom data to avoid sending garbage data to the GPU.
		OutPackedCustomData.CustomData.SetNumZeroed(TotalNumCustomDataFloats);
		return;
	}

	OutPackedCustomData.CustomData.SetNumUninitialized(TotalNumCustomDataFloats);

	if (TotalNumCustomDataFloats == 0)
	{
		return;
	}

	// Index of the accessor's attribute/property for a single instance (e.g. if the
	// accessors are [float, FVector, float], AccessorPackedIndex will be [0, 1, 4])
	int AccessorPackedIndex = 0;
	const EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible;

	for (int AccessorIndex = 0; AccessorIndex < Accessors.Num(); ++AccessorIndex)
	{
		const IPCGAttributeAccessor* Accessor = Accessors[AccessorIndex].Get();
		const IPCGAttributeAccessorKeys* Keys = AccessorKeys[AccessorIndex].Get();

		check(Accessor && Keys);

		switch (Accessor->GetUnderlyingType())
		{
		case PCG::Private::MetadataTypes<bool>::Id: // fall-through
		case PCG::Private::MetadataTypes<float>::Id: // fall-through
		case PCG::Private::MetadataTypes<double>::Id: // fall-through
		case PCG::Private::MetadataTypes<int32>::Id: // fall-through
		case PCG::Private::MetadataTypes<int64>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](float Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;
				OutPackedCustomData.CustomData[PackedIndex] = Value;
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<float>(*Keys, *Accessor, PackFloats, Flags));

			AccessorPackedIndex += 1;

			break;
		}
		case PCG::Private::MetadataTypes<FRotator>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](const FRotator& Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;

				OutPackedCustomData.CustomData[PackedIndex + 0] = static_cast<float>(Value.Roll);
				OutPackedCustomData.CustomData[PackedIndex + 1] = static_cast<float>(Value.Pitch);
				OutPackedCustomData.CustomData[PackedIndex + 2] = static_cast<float>(Value.Yaw);
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<FRotator>(*Keys, *Accessor, PackFloats, Flags));

			AccessorPackedIndex += 3;

			break;
		}
		case PCG::Private::MetadataTypes<FVector2D>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](const FVector2D& Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;

				OutPackedCustomData.CustomData[PackedIndex + 0] = static_cast<float>(Value.X);
				OutPackedCustomData.CustomData[PackedIndex + 1] = static_cast<float>(Value.Y);
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<FVector2D>(*Keys, *Accessor, PackFloats, Flags));

			AccessorPackedIndex += 2;

			break;
		}
		case PCG::Private::MetadataTypes<FVector>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](const FVector& Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;

				OutPackedCustomData.CustomData[PackedIndex + 0] = static_cast<float>(Value.X);
				OutPackedCustomData.CustomData[PackedIndex + 1] = static_cast<float>(Value.Y);
				OutPackedCustomData.CustomData[PackedIndex + 2] = static_cast<float>(Value.Z);
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<FVector>(*Keys, *Accessor, PackFloats, Flags));

			AccessorPackedIndex += 3;

			break;
		}
		case PCG::Private::MetadataTypes<FVector4>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](const FVector4& Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;

				OutPackedCustomData.CustomData[PackedIndex + 0] = static_cast<float>(Value.X);
				OutPackedCustomData.CustomData[PackedIndex + 1] = static_cast<float>(Value.Y);
				OutPackedCustomData.CustomData[PackedIndex + 2] = static_cast<float>(Value.Z);
				OutPackedCustomData.CustomData[PackedIndex + 3] = static_cast<float>(Value.W);
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<FVector4>(*Keys, *Accessor, PackFloats, Flags));

			AccessorPackedIndex += 4;

			break;
		}
		case PCG::Private::MetadataTypes<FQuat>::Id:
		{
			auto PackFloats = [&OutPackedCustomData, AccessorPackedIndex](const FQuat& Value, int32 Index)
			{
				const int PackedIndex = Index * OutPackedCustomData.NumCustomDataFloats + AccessorPackedIndex;

				OutPackedCustomData.CustomData[PackedIndex + 0] = static_cast<float>(Value.X);
				OutPackedCustomData.CustomData[PackedIndex + 1] = static_cast<float>(Value.Y);
				OutPackedCustomData.CustomData[PackedIndex + 2] = static_cast<float>(Value.Z);
				OutPackedCustomData.CustomData[PackedIndex + 3] = static_cast<float>(Value.W);
			};

			ensure(PCGMetadataElementCommon::ApplyOnAccessor<FQuat>(*Keys, *Accessor, PackFloats, Flags));

			AccessorPackedIndex += 4;

			break;
		}
		default:
			break;
		}
	}
}
