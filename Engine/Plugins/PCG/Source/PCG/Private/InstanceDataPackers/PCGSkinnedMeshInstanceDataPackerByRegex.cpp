// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceDataPackers/PCGSkinnedMeshInstanceDataPackerByRegex.h"

#include "PCGContext.h"
#include "PCGElement.h"
#include "Data/PCGSpatialData.h"
#include "InstanceDataPackers/PCGSkinnedMeshInstanceDataPackerBase.h"

#include "Internationalization/Regex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSkinnedMeshInstanceDataPackerByRegex)

#define LOCTEXT_NAMESPACE "PCGSkinnedMeshInstanceDataPackerByRegex"

void UPCGSkinnedMeshInstanceDataPackerByRegex::PackInstances_Implementation(FPCGContext& Context, const UPCGSpatialData* InSpatialData, const FPCGSkinnedMeshInstanceList& InstanceList, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const
{
	if (!InSpatialData || !InSpatialData->Metadata)
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, LOCTEXT("InvalidInputData", "Invalid input data"));
		return;
	}
	
	TArray<FName> AttributeNames;
	TArray<EPCGMetadataTypes> AttributeTypes;
	InSpatialData->Metadata->GetAttributes(AttributeNames, AttributeTypes);

	TArray<const FPCGMetadataAttributeBase*> SelectedAttributes;
	TSet<PCGMetadataAttributeKey> FoundAttributes;

	for (const FString& Regex : RegexPatterns)
	{
		const FRegexPattern Pattern(Regex);
	
		// Match our Regex Pattern against every Attribute
		for (int Index = 0; Index < AttributeNames.Num(); ++Index)
		{
			const FName& AttributeName = AttributeNames[Index];
			FRegexMatcher RegexMatcher(Pattern, AttributeName.ToString());
	
			if (!RegexMatcher.FindNext()) 
			{
				continue;
			}
	
			const FPCGMetadataAttributeBase* AttributeBase = InSpatialData->Metadata->GetConstAttribute(AttributeName);
			check(AttributeBase);

			// Avoid adding the same attribute multiple times if it is captured by different regex patterns
			if (FoundAttributes.Contains(AttributeBase->AttributeId))
			{
				continue;
			}

			FoundAttributes.Add(AttributeBase->AttributeId);
	
			if (!AddTypeToPacking(AttributeBase->GetTypeId(), OutPackedCustomData))
			{
				PCGE_LOG_C(Warning, GraphAndLog, &Context, FText::Format(LOCTEXT("InvalidAttributeType", "Attribute name '{0}' is not of a valid type"), FText::FromName(AttributeName)));
				continue;
			}
	
			SelectedAttributes.Add(AttributeBase);
		}
	}
	
	PackCustomDataFromAttributes(InstanceList, SelectedAttributes, OutPackedCustomData);
}

#undef LOCTEXT_NAMESPACE
