// Copyright Epic Games, Inc. All Rights Reserved.

#include "Landscape/PCGLandscapeCVars.h"

namespace PCGLandscapeCVars
{
	TAutoConsoleVariable<int> CVarLandscapeRefreshTimeDelay(
		TEXT("pcg.LandscapeRefreshTimeDelayMS"),
		1000,
		TEXT("Time in MS between a landscape change and PCG refresh. Set it to 0 or negative value to disable the delay."));

	TAutoConsoleVariable<bool> CVarLandscapeDisableRefreshTracking(
		TEXT("pcg.LandscapeDisableRefreshTracking"),
		false,
		TEXT("Completely disable landscape refresh when it changes."));

	TAutoConsoleVariable<bool> CVarLandscapeDisableRefreshTrackingInLandscapeEditingMode(
		TEXT("pcg.LandscapeDisableRefreshTrackingInLandscapeEditingMode"),
		false,
		TEXT("Disable landscape refresh when it changes in landscape editing mode."));

	TAutoConsoleVariable<bool> CVarLandscapeForceRefreshRuntimeGen(
		TEXT("pcg.LandscapeForceRefreshRuntimeGen"),
		false,
		TEXT("Force landscape change events to refresh runtime generated execution sources even when editor refresh tracking is disabled."));
}