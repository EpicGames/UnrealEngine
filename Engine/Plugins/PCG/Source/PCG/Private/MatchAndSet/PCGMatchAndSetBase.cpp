// Copyright Epic Games, Inc. All Rights Reserved.

#include "MatchAndSet/PCGMatchAndSetBase.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGPointMatchAndSet.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMatchAndSetBase)

#define LOCTEXT_NAMESPACE "PCGMatchAndSetBase"

#if WITH_EDITOR
void UPCGMatchAndSetBase::PostLoad()
{
	Super::PostLoad();

	if (Type == EPCGMetadataTypes::String)
	{
		if (StringMode_DEPRECATED == EPCGMetadataTypesConstantStructStringMode::SoftObjectPath)
		{
			Type = EPCGMetadataTypes::SoftObjectPath;
		}
		else if (StringMode_DEPRECATED == EPCGMetadataTypesConstantStructStringMode::SoftClassPath)
		{
			Type = EPCGMetadataTypes::SoftClassPath;
		}
	}
}
#endif

void UPCGMatchAndSetBase::SetType(EPCGMetadataTypes InType)
{
	Type = InType;
}

void UPCGMatchAndSetBase::MatchAndSet_Implementation(
	FPCGContext& Context,
	const UPCGPointMatchAndSetSettings* InSettings,
	const UPCGPointData* InPointData,
	UPCGPointData* OutPointData) const
{
	PCGE_LOG_C(Error, GraphAndLog, &Context, LOCTEXT("MatchAndSetBaseFailed", "Unable to execute MatchAndSet pure virtual base function, override the MatchAndSet function or use a default implementation."));
}

bool UPCGMatchAndSetBase::CreateAttributeIfNeeded(FPCGContext& Context, const FPCGAttributePropertySelector& Selector, const FPCGMetadataTypesConstantStruct& ConstantValue, UPCGPointData* OutPointData, const UPCGPointMatchAndSetSettings* InSettings) const
{
	check(OutPointData && OutPointData->Metadata);

	check(OutPointData->Metadata);
	if (Selector.GetSelection() == EPCGAttributePropertySelection::Attribute)
	{
		FName DestinationAttribute = Selector.GetName();

		if (!OutPointData->Metadata->HasAttribute(DestinationAttribute) ||
			OutPointData->Metadata->GetConstAttribute(DestinationAttribute)->GetTypeId() != static_cast<uint16>(InSettings->SetTargetType))
		{
			auto CreateAttribute = [OutPointData, &DestinationAttribute](auto&& Value)
			{
				using ConstantType = std::decay_t<decltype(Value)>;
				return PCGMetadataElementCommon::ClearOrCreateAttribute<ConstantType>(OutPointData->Metadata, DestinationAttribute) != nullptr;
			};

			if (!ConstantValue.Dispatcher(CreateAttribute))
			{
				PCGE_LOG_C(Error, GraphAndLog, &Context, FText::Format(LOCTEXT("AttributeCreationFailed", "Unable to create attribute '{0}' on Point data"), FText::FromName(DestinationAttribute)));
				return false;
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
