// Copyright Epic Games, Inc. All Rights Reserved.

#include "MatchAndSet/PCGMatchAndSetByAttribute.h"

#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGPointMatchAndSet.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMatchAndSetByAttribute)

#define LOCTEXT_NAMESPACE "PCGMatchAndSetByAttributeElement"

FPCGMatchAndSetByAttributeEntry::FPCGMatchAndSetByAttributeEntry()
{
	ValueToMatch.bAllowsTypeChange = false;
	Value.bAllowsTypeChange = false;
}

#if WITH_EDITOR
void FPCGMatchAndSetByAttributeEntry::OnPostLoad()
{
	ValueToMatch.OnPostLoad();
	Value.OnPostLoad();
}
#endif

void UPCGMatchAndSetByAttribute::SetType(EPCGMetadataTypes InType)
{
	for (FPCGMatchAndSetByAttributeEntry& Entry : Entries)
	{
		Entry.Value.Type = InType;
	}

	Super::SetType(InType);
}

void UPCGMatchAndSetByAttribute::SetSourceType(EPCGMetadataTypes InType)
{
	for (FPCGMatchAndSetByAttributeEntry& Entry : Entries)
	{
		Entry.ValueToMatch.Type = InType;
	}
}

#if WITH_EDITOR
void UPCGMatchAndSetByAttribute::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		const FName& PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGMatchAndSetByAttribute, MatchSourceType))
		{
			SetSourceType(MatchSourceType);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGMatchAndSetByAttribute, Entries))
		{
			// Some changes in the entries (add, insert) might require us to re-set the type
			SetType(Type);
			SetSourceType(MatchSourceType);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPCGMatchAndSetByAttribute::PostLoad()
{
	Super::PostLoad();

	if (MatchSourceType == EPCGMetadataTypes::String)
	{
		if (MatchSourceStringMode_DEPRECATED == EPCGMetadataTypesConstantStructStringMode::SoftObjectPath)
		{
			MatchSourceType = EPCGMetadataTypes::SoftObjectPath;
		}
		else if (MatchSourceStringMode_DEPRECATED == EPCGMetadataTypesConstantStructStringMode::SoftClassPath)
		{
			MatchSourceType = EPCGMetadataTypes::SoftClassPath;
		}
	}

	for (FPCGMatchAndSetByAttributeEntry& Entry : Entries)
	{
		Entry.OnPostLoad();
	}
}
#endif // WITH_EDITOR

void UPCGMatchAndSetByAttribute::MatchAndSet_Implementation(
	FPCGContext& Context,
	const UPCGPointMatchAndSetSettings* InSettings,
	const UPCGPointData* InPointData,
	UPCGPointData* OutPointData) const
{
	check(InSettings && InPointData && OutPointData);
	check(OutPointData->Metadata);
	check(InPointData->GetPoints().Num() == OutPointData->GetMutablePoints().Num());

	if (Entries.Num() == 0)
	{
		return;
	}

	FPCGAttributePropertyInputSelector InputSource;
	InputSource.SetAttributeName(MatchSourceAttribute);
	InputSource = InputSource.CopyAndFixLast(InPointData);

	// Deprecation old behavior, if SetTarget was None, we took last created in OutPointData
	FPCGAttributePropertyOutputSelector SetTarget = InSettings->SetTarget.CopyAndFixSource(&InputSource, OutPointData);

	// Create attribute if needed
	if (!CreateAttributeIfNeeded(Context, SetTarget, Entries[0].Value, OutPointData, InSettings))
	{
		return; // Failed adding attribute
	}

	TArray<TUniquePtr<const IPCGAttributeAccessor>> EntryMatchSourceAccessors;
	TArray<TUniquePtr<const IPCGAttributeAccessor>> EntrySetValueAccessors;
	EntryMatchSourceAccessors.Reserve(Entries.Num());
	EntrySetValueAccessors.Reserve(Entries.Num());

	TUniquePtr<const IPCGAttributeAccessorKeys> EntryKeys = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();

	auto CreateConstantMatchSourceAccessor = [&EntryMatchSourceAccessors](auto&& Value)
	{
		using ConstantType = std::decay_t<decltype(Value)>;
		EntryMatchSourceAccessors.Emplace(MakeUnique<FPCGConstantValueAccessor<ConstantType>>(std::forward<decltype(Value)>(Value)));
	};

	auto CreateConstantSetValueAccessor = [&EntrySetValueAccessors](auto&& Value)
	{
		using ConstantType = std::decay_t<decltype(Value)>;
		EntrySetValueAccessors.Emplace(MakeUnique<FPCGConstantValueAccessor<ConstantType>>(std::forward<decltype(Value)>(Value)));
	};

	for (const FPCGMatchAndSetByAttributeEntry& Entry : Entries)
	{
		Entry.ValueToMatch.Dispatcher(CreateConstantMatchSourceAccessor);
		Entry.Value.Dispatcher(CreateConstantSetValueAccessor);
	}

	//TODO: implement async loop?
	TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InPointData, InputSource);
	TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InPointData, InputSource);

	if (!InputAccessor.IsValid() || !InputKeys.IsValid())
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, LOCTEXT("InputAccessorCreationFailed", "Failed to create input accessor or iterator in MatchAndSet"));
		return;
	}

	// TODO: this should be more complex type - see what's done in the compare element
	TUniquePtr<IPCGAttributeAccessor> SetTargetAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutPointData, SetTarget);
	TUniquePtr<IPCGAttributeAccessorKeys> SetTargetKeys = PCGAttributeAccessorHelpers::CreateKeys(OutPointData, SetTarget);

	if (!SetTargetAccessor.IsValid() || !SetTargetKeys.IsValid())
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, LOCTEXT("FailedToCreateTargetAccessor", "Failed to create output accessor or iterator"));
		return;
	}

	if (SetTargetAccessor->IsReadOnly())
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, FText::Format(LOCTEXT("SetTargetAccessorIsReadOnly", "Attribute/Property '{0}' is read only."), SetTarget.GetDisplayText()));
		return;
	}

	auto MatchAndSetOperation = [&InputAccessor, &InputKeys, &EntryMatchSourceAccessors, &EntrySetValueAccessors, &EntryKeys, &SetTargetAccessor, &SetTargetKeys](auto MatchDummy)
	{
		using MatchType = decltype(MatchDummy);
		auto GetValueIndex = [&InputAccessor, &InputKeys, &EntryMatchSourceAccessors, &EntryKeys](int32 Index) -> int32
		{
			MatchType Value{};
			EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::AllowBroadcast;

			InputAccessor->Get<MatchType>(Value, Index, *InputKeys, Flags);

			for (int32 SourceIndex = 0; SourceIndex < EntryMatchSourceAccessors.Num(); ++SourceIndex)
			{
				MatchType SourceValue{};
				EntryMatchSourceAccessors[SourceIndex]->Get<MatchType>(SourceValue, *EntryKeys, Flags);

				if (PCG::Private::MetadataTraits<MatchType>::Equal(Value, SourceValue))
				{
					return SourceIndex;
				}
			}

			return -1;
		};

		auto SetValueOperation = [&GetValueIndex, &EntrySetValueAccessors, &EntryKeys, &SetTargetAccessor, &SetTargetKeys](auto SetValueDummy)
		{
			using OutputType = decltype(SetValueDummy);
			EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::AllowBroadcast;

			// Cache values to set
			TArray<OutputType> Values;
			Values.Reserve(EntrySetValueAccessors.Num());

			for (TUniquePtr<const IPCGAttributeAccessor>& InputAccessor : EntrySetValueAccessors)
			{
				InputAccessor->Get<OutputType>(Values.Emplace_GetRef(), *EntryKeys, Flags);
			}

			// Set values on points
			const int32 NumberOfElements = SetTargetKeys->GetNum();
			for (int32 ElementIndex = 0; ElementIndex < NumberOfElements; ++ElementIndex)
			{
				const int32 ValueIndex = GetValueIndex(ElementIndex);
				if (ValueIndex >= 0)
				{
					SetTargetAccessor->Set<OutputType>(Values[ValueIndex], ElementIndex, *SetTargetKeys, Flags);
				}
			}

			return true;
		};

		return PCGMetadataAttribute::CallbackWithRightType(SetTargetAccessor->GetUnderlyingType(), SetValueOperation);
	};

	if (!PCGMetadataAttribute::CallbackWithRightType(InputAccessor->GetUnderlyingType(), MatchAndSetOperation))
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, LOCTEXT("ErrorGettingSettingValues", "Error while getting/setting values in the MatchAndSet"));
	}
}

bool UPCGMatchAndSetByAttribute::ValidatePreconditions_Implementation(const UPCGPointData* InPointData) const
{
	if (!InPointData || !InPointData->Metadata)
	{
		return false;
	}

	// Check if source attribute exists
	FName AttributeName = ((MatchSourceAttribute == NAME_None) ? InPointData->Metadata->GetLatestAttributeNameOrNone() : MatchSourceAttribute);

	if (!InPointData->Metadata->HasAttribute(AttributeName))
	{
		return false;
	}

	// Check if type matches
	const FPCGMetadataAttributeBase* AttributeToMatch = InPointData->Metadata->GetConstAttribute(AttributeName);
	return AttributeToMatch && AttributeToMatch->GetTypeId() == static_cast<uint16>(MatchSourceType);
}

#undef LOCTEXT_NAMESPACE
