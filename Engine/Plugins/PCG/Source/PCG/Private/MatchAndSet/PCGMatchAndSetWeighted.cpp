// Copyright Epic Games, Inc. All Rights Reserved.

#include "MatchAndSet/PCGMatchAndSetWeighted.h"

#include "Math/RandomStream.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGPointMatchAndSet.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMatchAndSetWeighted)

#define LOCTEXT_NAMESPACE "PCGMatchAndSetWeightedEntry"

FPCGMatchAndSetWeightedEntry::FPCGMatchAndSetWeightedEntry()
{
	Value.bAllowsTypeChange = false;
}

#if WITH_EDITOR
void FPCGMatchAndSetWeightedEntry::OnPostLoad()
{
	Value.OnPostLoad();
}
#endif

void UPCGMatchAndSetWeighted::SetType(EPCGMetadataTypes InType)
{
	for (FPCGMatchAndSetWeightedEntry& Entry : Entries)
	{
		Entry.Value.Type = InType;
	}

	Super::SetType(InType);
}

#if WITH_EDITOR
void UPCGMatchAndSetWeighted::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		const FName& PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGMatchAndSetWeighted, Entries))
		{
			// Some changes in the array might (such as insert or new) might require us to re-set the type
			SetType(Type);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPCGMatchAndSetWeighted::PostLoad()
{
	Super::PostLoad();

	for (FPCGMatchAndSetWeightedEntry& Entry : Entries)
	{
		Entry.OnPostLoad();
	}
}
#endif // WITH_EDITOR

void UPCGMatchAndSetWeighted::MatchAndSet_Implementation(
	FPCGContext& Context,
	const UPCGPointMatchAndSetSettings* InSettings,
	const UPCGPointData* InPointData,
	UPCGPointData* OutPointData) const
{
	check(InSettings && InPointData && OutPointData);
	check(OutPointData->Metadata);
	check(InPointData->GetPoints().Num() == OutPointData->GetMutablePoints().Num());

	if (Entries.Num() == 0)
	{
		return;
	}

	const FPCGAttributePropertySelector& SetTarget = InSettings->SetTarget;

	// Create attribute if needed
	if (!CreateAttributeIfNeeded(Context, SetTarget, Entries[0].Value, OutPointData, InSettings))
	{
		return; // Failed adding attribute
	}

	TArray<TUniquePtr<const IPCGAttributeAccessor>> EntryValueAccessors;
	EntryValueAccessors.Reserve(Entries.Num());
	TUniquePtr<const IPCGAttributeAccessorKeys> EntryKeys = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();

	auto CreateConstantAccessor = [&EntryValueAccessors](auto&& Value)
	{
		using ConstantType = std::decay_t<decltype(Value)>;
		EntryValueAccessors.Emplace(MakeUnique<FPCGConstantValueAccessor<ConstantType>>(std::forward<decltype(Value)>(Value)));
	};

	TArray<int> CumulativeWeights;

	int TotalWeight = 0;

	for (const FPCGMatchAndSetWeightedEntry& Entry : Entries)
	{
		TotalWeight += Entry.Weight;
		CumulativeWeights.Add(TotalWeight);
		Entry.Value.Dispatcher(CreateConstantAccessor);
	}

	if (TotalWeight <= 0)
	{
		return;
	}

	// TODO: implement async loop?
	const IPCGGraphExecutionSource* ExecutionSource = Context.ExecutionSource.Get();
	const TArray<FPCGPoint>& InPoints = InPointData->GetPoints();

	TUniquePtr<IPCGAttributeAccessor> SetTargetAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutPointData, SetTarget);
	TUniquePtr<IPCGAttributeAccessorKeys> SetTargetKeys = PCGAttributeAccessorHelpers::CreateKeys(OutPointData, SetTarget);

	if (!SetTargetAccessor.IsValid() || !SetTargetKeys.IsValid())
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, LOCTEXT("FailedToCreateTargetAccessor", "Failed to create output accessor or iterator"));
		return;
	}

	if (SetTargetAccessor->IsReadOnly())
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, FText::Format(LOCTEXT("SetTargetAccessorIsReadOnly", "Attribute/Property '{0}' is read only."), SetTarget.GetDisplayText()));
		return;
	}

	auto GetValueIndexFromElementIndex = [&InPoints, InSettings, ExecutionSource, &CumulativeWeights, TotalWeight](int32 Index) -> int32
	{
		const FPCGPoint& Point = InPoints[Index];

		FRandomStream RandomSource = PCGHelpers::GetRandomStreamFromSeed(Point.Seed, InSettings, ExecutionSource);
		int RandomWeightedPick = RandomSource.RandRange(0, TotalWeight - 1);

		int RandomPick = 0;
		while (RandomPick < CumulativeWeights.Num() && CumulativeWeights[RandomPick] <= RandomWeightedPick)
		{
			++RandomPick;
		}

		return RandomPick;
	};

	auto MatchAndSetOperation = [&GetValueIndexFromElementIndex, &EntryKeys, &EntryValueAccessors, &SetTargetAccessor, &SetTargetKeys](auto Dummy)
	{
		using OutputType = decltype(Dummy);
		EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::AllowBroadcast;

		// Cache values to set
		TArray<OutputType> Values;
		Values.Reserve(EntryValueAccessors.Num());

		for (TUniquePtr<const IPCGAttributeAccessor>& InputAccessor : EntryValueAccessors)
		{
			InputAccessor->Get<OutputType>(Values.Emplace_GetRef(), *EntryKeys, Flags);
		}

		// Set values on points
		const int32 NumberOfElements = SetTargetKeys->GetNum();
		for (int32 ElementIndex = 0; ElementIndex < NumberOfElements; ++ElementIndex)
		{
			const int32 ValueIndex = GetValueIndexFromElementIndex(ElementIndex);
			SetTargetAccessor->Set<OutputType>(Values[ValueIndex], ElementIndex, *SetTargetKeys, Flags);
		}

		return true;
	};

	if (!PCGMetadataAttribute::CallbackWithRightType(SetTargetAccessor->GetUnderlyingType(), MatchAndSetOperation))
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, LOCTEXT("ErrorGettingSettingValues", "Error while getting/setting values in the MatchAndSet"));
	}
}

#undef LOCTEXT_NAMESPACE
