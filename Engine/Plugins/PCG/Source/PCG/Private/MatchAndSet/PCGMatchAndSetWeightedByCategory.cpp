// Copyright Epic Games, Inc. All Rights Reserved.

#include "MatchAndSet/PCGMatchAndSetWeightedByCategory.h"

#include "MatchAndSet/PCGMatchAndSetWeighted.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGPointMatchAndSet.h"
#include "Helpers/PCGHelpers.h"
#include "Math/RandomStream.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMatchAndSetWeightedByCategory)

#define LOCTEXT_NAMESPACE "PCGMatchAndSetWeightedByCategoryElement"

FPCGMatchAndSetWeightedByCategoryEntryList::FPCGMatchAndSetWeightedByCategoryEntryList()
{
	CategoryValue.bAllowsTypeChange = false;
}

#if WITH_EDITOR
void FPCGMatchAndSetWeightedByCategoryEntryList::OnPostLoad()
{
	CategoryValue.OnPostLoad();

	for (FPCGMatchAndSetWeightedEntry& Entry : WeightedEntries)
	{
		Entry.OnPostLoad();
	}
} 
#endif

void FPCGMatchAndSetWeightedByCategoryEntryList::SetType(EPCGMetadataTypes InType)
{
	for (FPCGMatchAndSetWeightedEntry& Entry : WeightedEntries)
	{
		Entry.Value.Type = InType;
	}
}

int FPCGMatchAndSetWeightedByCategoryEntryList::GetTotalWeight() const
{
	int TotalWeight = 0;

	for (const FPCGMatchAndSetWeightedEntry& Entry : WeightedEntries)
	{
		TotalWeight += Entry.Weight;
	}

	return TotalWeight;
}

void UPCGMatchAndSetWeightedByCategory::SetType(EPCGMetadataTypes InType)
{
	for (FPCGMatchAndSetWeightedByCategoryEntryList& Category : Categories)
	{
		Category.SetType(InType);
	}

	Super::SetType(InType);
}

void UPCGMatchAndSetWeightedByCategory::SetCategoryType(EPCGMetadataTypes InType)
{
	for (FPCGMatchAndSetWeightedByCategoryEntryList& Category : Categories)
	{
		Category.CategoryValue.Type = InType;
	}
}

#if WITH_EDITOR
void UPCGMatchAndSetWeightedByCategory::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		const FName& PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGMatchAndSetWeightedByCategory, CategoryType))
		{
			SetCategoryType(CategoryType);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FPCGMatchAndSetWeightedByCategoryEntryList, WeightedEntries))
		{
			// Changes in any weighted entries (which store the end values) - due to add or insert - need to update type
			SetType(Type);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGMatchAndSetWeightedByCategory, Categories))
		{
			SetCategoryType(CategoryType);
			SetType(Type);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPCGMatchAndSetWeightedByCategory::PostLoad()
{
	Super::PostLoad();

	if (CategoryType == EPCGMetadataTypes::String)
	{
		if (CategoryStringMode_DEPRECATED == EPCGMetadataTypesConstantStructStringMode::SoftObjectPath)
		{
			CategoryType = EPCGMetadataTypes::SoftObjectPath;
		}
		else if (CategoryStringMode_DEPRECATED == EPCGMetadataTypesConstantStructStringMode::SoftClassPath)
		{
			CategoryType = EPCGMetadataTypes::SoftClassPath;
		}
	}

	for (FPCGMatchAndSetWeightedByCategoryEntryList& Category : Categories)
	{
		Category.OnPostLoad();
	}
}
#endif // WITH_EDITOR

void UPCGMatchAndSetWeightedByCategory::MatchAndSet_Implementation(
	FPCGContext& Context,
	const UPCGPointMatchAndSetSettings* InSettings,
	const UPCGPointData* InPointData,
	UPCGPointData* OutPointData) const
{
	// Early validation
	check(InSettings && InPointData && OutPointData);
	check(OutPointData->Metadata);
	check(InPointData->GetPoints().Num() == OutPointData->GetMutablePoints().Num());

	bool bHasAtLeastOneNonEmptyEntry = false;
	int32 DefaultCategoryIndex = INDEX_NONE;
	const FPCGMetadataTypesConstantStruct* FirstValueForTypeExtraction = nullptr;
	
	for(int32 CategoryIndex = 0; CategoryIndex < Categories.Num(); ++CategoryIndex)
	{
		const FPCGMatchAndSetWeightedByCategoryEntryList& Category = Categories[CategoryIndex];

		if (Category.bIsDefault && DefaultCategoryIndex == INDEX_NONE)
		{
			DefaultCategoryIndex = CategoryIndex;
		}

		if (Category.GetTotalWeight() > 0)
		{
			bHasAtLeastOneNonEmptyEntry = true;

			check(Category.WeightedEntries.Num() > 0);
			FirstValueForTypeExtraction = &Category.WeightedEntries[0].Value;
		}
	}

	if (!bHasAtLeastOneNonEmptyEntry)
	{
		return;
	}

	FPCGAttributePropertyInputSelector InputSource;
	InputSource.SetAttributeName(CategoryAttribute);
	InputSource = InputSource.CopyAndFixLast(InPointData);

	// Deprecation old behavior, if SetTarget was None, we took last created in OutPointData
	FPCGAttributePropertyOutputSelector SetTarget = InSettings->SetTarget.CopyAndFixSource(&InputSource, OutPointData);

	// Create attribute if needed
	if (!CreateAttributeIfNeeded(Context, SetTarget, *FirstValueForTypeExtraction, OutPointData, InSettings))
	{
		return; // Failed adding attribute
	}

	TArray<TUniquePtr<const IPCGAttributeAccessor>> EntryCategoryAccessors;
	TArray<TArray<TUniquePtr<const IPCGAttributeAccessor>>> EntryValueAccessors;
	EntryCategoryAccessors.Reserve(Categories.Num());
	EntryValueAccessors.Reserve(Categories.Num());

	TUniquePtr<const IPCGAttributeAccessorKeys> EntryKeys = MakeUnique<FPCGAttributeAccessorKeysSingleObjectPtr<void>>();

	auto CreateConstantCategoryAccessor = [&EntryCategoryAccessors](auto&& Value)
	{
		using ConstantType = std::decay_t<decltype(Value)>;
		EntryCategoryAccessors.Emplace(MakeUnique<FPCGConstantValueAccessor<ConstantType>>(std::forward<decltype(Value)>(Value)));
	};

	for (const FPCGMatchAndSetWeightedByCategoryEntryList& Category : Categories)
	{
		Category.CategoryValue.Dispatcher(CreateConstantCategoryAccessor);

		TArray<TUniquePtr<const IPCGAttributeAccessor>>& ValuesAccessor = EntryValueAccessors.Emplace_GetRef();
		for (const FPCGMatchAndSetWeightedEntry& WeightedEntry : Category.WeightedEntries)
		{
			WeightedEntry.Value.Dispatcher([&ValuesAccessor](auto&& Value)
			{
				using ConstantType = std::decay_t<decltype(Value)>;
				ValuesAccessor.Emplace(MakeUnique<FPCGConstantValueAccessor<ConstantType>>(std::forward<decltype(Value)>(Value)));
			});
		}
	}

	//TODO: implement async loop?
	const IPCGGraphExecutionSource* ExecutionSource = Context.ExecutionSource.Get();
	const TArray<FPCGPoint>& InPoints = InPointData->GetPoints();

	TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InPointData, InputSource);
	TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InPointData, InputSource);

	if (!InputAccessor.IsValid() || !InputKeys.IsValid())
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, LOCTEXT("InputAccessorFailed", "Failed to create input accessor or iterator in MatchAndSet"));
		return;
	}

	TUniquePtr<IPCGAttributeAccessor> SetTargetAccessor = PCGAttributeAccessorHelpers::CreateAccessor(OutPointData, SetTarget);
	TUniquePtr<IPCGAttributeAccessorKeys> SetTargetKeys = PCGAttributeAccessorHelpers::CreateKeys(OutPointData, SetTarget);

	if (!SetTargetAccessor.IsValid() || !SetTargetKeys.IsValid())
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, LOCTEXT("FailedToCreateTargetAccessor", "Failed to create output accessor or iterator"));
		return;
	}

	if (SetTargetAccessor->IsReadOnly())
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, FText::Format(LOCTEXT("SetTargetAccessorIsReadOnly", "Attribute/Property '{0}' is read only."), SetTarget.GetDisplayText()));
		return;
	}

	auto GetValueIndex = [this, InSettings, ExecutionSource, &InPoints](int32 CategoryIndex, int32 PointIndex) -> int32
	{
		const FPCGMatchAndSetWeightedByCategoryEntryList& Category = Categories[CategoryIndex];
		const int32 TotalWeight = Category.GetTotalWeight();

		if (TotalWeight == 0)
		{
			return INDEX_NONE;
		}

		const FPCGPoint& Point = InPoints[PointIndex];

		FRandomStream RandomSource = PCGHelpers::GetRandomStreamFromSeed(Point.Seed, InSettings, ExecutionSource);
		int RandomWeightedPick = RandomSource.RandRange(0, TotalWeight - 1);

		int RandomPick = 0;
		while (RandomPick < Category.WeightedEntries.Num() && RandomWeightedPick >= Category.WeightedEntries[RandomPick].Weight)
		{
			RandomWeightedPick -= Category.WeightedEntries[RandomPick].Weight;
			++RandomPick;
		}

		return RandomPick;
	};

	auto MatchAndSetOperation = [&GetValueIndex, &InputAccessor, &InputKeys, &EntryCategoryAccessors, &EntryValueAccessors, &EntryKeys, DefaultCategoryIndex, &SetTargetAccessor, &SetTargetKeys](auto MatchDummy)
	{
		using MatchType = decltype(MatchDummy);
		auto GetCategoryIndex = [&InputAccessor, &InputKeys, &EntryCategoryAccessors, &EntryKeys, DefaultCategoryIndex](int32 Index) -> int32
		{
			EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::AllowBroadcast;

			MatchType Value{};
			InputAccessor->Get<MatchType>(Value, Index, *InputKeys, Flags);

			for (int32 CategoryIndex = 0; CategoryIndex < EntryCategoryAccessors.Num(); ++CategoryIndex)
			{
				MatchType SourceValue{};
				EntryCategoryAccessors[CategoryIndex]->Get<MatchType>(SourceValue, *EntryKeys, Flags);

				if (PCG::Private::MetadataTraits<MatchType>::Equal(Value, SourceValue))
				{
					return CategoryIndex;
				}
			}

			return DefaultCategoryIndex;
		};

		auto SetValueOperation = [&GetCategoryIndex, &GetValueIndex, &EntryValueAccessors, &EntryKeys, &SetTargetAccessor, &SetTargetKeys](auto SetValueDummy)
		{
			using OutputType = decltype(SetValueDummy);
			EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::AllowBroadcast;

			// Cache values to set
			TArray<TArray<OutputType>> Values;
			Values.Reserve(EntryValueAccessors.Num());
			for (const TArray<TUniquePtr<const IPCGAttributeAccessor>>& CategoryValueAccessor : EntryValueAccessors)
			{
				TArray<OutputType>& CategoryValues = Values.Emplace_GetRef();
				CategoryValues.Reserve(CategoryValueAccessor.Num());

				for (const TUniquePtr<const IPCGAttributeAccessor>& ValueAccessor : CategoryValueAccessor)
				{
					ValueAccessor->Get<OutputType>(CategoryValues.Emplace_GetRef(), *EntryKeys, Flags);
				}
			}

			// Set values on points
			const int32 NumberOfElements = SetTargetKeys->GetNum();
			for (int32 ElementIndex = 0; ElementIndex < NumberOfElements; ++ElementIndex)
			{
				const int32 CategoryIndex = GetCategoryIndex(ElementIndex);
				if (CategoryIndex >= 0)
				{
					const int32 ValueIndex = GetValueIndex(CategoryIndex, ElementIndex);
					if (ValueIndex >= 0)
					{
						SetTargetAccessor->Set<OutputType>(Values[CategoryIndex][ValueIndex], ElementIndex, *SetTargetKeys, Flags);
					}
				}
			}

			return true;
		};

		return PCGMetadataAttribute::CallbackWithRightType(SetTargetAccessor->GetUnderlyingType(), SetValueOperation);
	};

	if (!PCGMetadataAttribute::CallbackWithRightType(InputAccessor->GetUnderlyingType(), MatchAndSetOperation))
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, LOCTEXT("ErrorGettingSettingValues", "Error while getting/setting values in the MatchAndSet"));
	}
}

bool UPCGMatchAndSetWeightedByCategory::ValidatePreconditions_Implementation(const UPCGPointData* InPointData) const
{
	if (!InPointData || !InPointData->Metadata)
	{
		return false;
	}

	FName AttributeName = ((CategoryAttribute == NAME_None) ? InPointData->Metadata->GetLatestAttributeNameOrNone() : CategoryAttribute);
	
	if (!InPointData->Metadata->HasAttribute(AttributeName))
	{
		return false;
	}

	// Check if type matches
	const FPCGMetadataAttributeBase* AttributeToMatch = InPointData->Metadata->GetConstAttribute(AttributeName);
	return AttributeToMatch && AttributeToMatch->GetTypeId() == static_cast<uint16>(CategoryType);
}

#undef LOCTEXT_NAMESPACE
