// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSelectors/PCGISMDescriptor.h"

#include "Helpers/PCGHelpers.h"

#include "Components/InstancedStaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGISMDescriptor)

FPCGSoftISMComponentDescriptor::FPCGSoftISMComponentDescriptor()
	: FSoftISMComponentDescriptor()
{
	// Override defaults from base class
	ComponentClass = UInstancedStaticMeshComponent::StaticClass();
	Mobility = EComponentMobility::Static;
	bGenerateOverlapEvents = 0;
	BodyInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}

FPCGSoftISMComponentDescriptor::FPCGSoftISMComponentDescriptor(const FISMComponentDescriptor& Other)
	: Super(Other)
{}

void FPCGSoftISMComponentDescriptor::InitFrom(const UStaticMeshComponent* Component, bool bInitBodyInstance)
{
	ComponentTags = Component->ComponentTags;
	Super::InitFrom(Component, bInitBodyInstance);
}

void FPCGSoftISMComponentDescriptor::InitComponent(UInstancedStaticMeshComponent* ISMComponent) const
{
	ISMComponent->ComponentTags = ComponentTags;

	const TArray<FString> AdditionalTags = PCGHelpers::GetStringArrayFromCommaSeparatedList(AdditionalCommaSeparatedTags);
	for (const FString& AdditionalTag : AdditionalTags)
	{
		ISMComponent->ComponentTags.Emplace(AdditionalTag);
	}

	Super::InitComponent(ISMComponent);
}

bool FPCGSoftISMComponentDescriptor::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(FSoftISMComponentDescriptor::StaticStruct()->GetFName()))
	{
		FSoftISMComponentDescriptor TempDescriptor;
		FSoftISMComponentDescriptor::StaticStruct()->SerializeItem(Slot, &TempDescriptor, nullptr);
		*static_cast<FSoftISMComponentDescriptor*>(this) = TempDescriptor;
		return true;
	}

	return false;
}