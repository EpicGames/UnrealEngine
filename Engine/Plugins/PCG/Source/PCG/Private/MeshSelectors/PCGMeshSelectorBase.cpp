// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSelectors/PCGMeshSelectorBase.h"

#include "PCGElement.h"
#include "Elements/PCGStaticMeshSpawnerContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMeshSelectorBase)

#define LOCTEXT_NAMESPACE "PCGMeshSelectorBase"


bool UPCGMeshSelectorBase::SelectMeshInstances(
	FPCGStaticMeshSpawnerContext& Context,
	const UPCGStaticMeshSpawnerSettings* Settings,
	const UPCGBasePointData* InPointData,
	TArray<FPCGMeshInstanceList>& OutMeshInstances,
	UPCGBasePointData* OutPointData) const
{
	// This method should be overwritten by the subclasses but in case that subclass hasn't yet been updated try and call the old signature
	const UPCGPointData* TypedInPointData = Cast<UPCGPointData>(InPointData);
	UPCGPointData* TypedOutPointData = Cast<UPCGPointData>(OutPointData);

	const FText Message = FText::Format(LOCTEXT("SelectMeshInstancesNotImplemented", "MeshSelector class '{0}' needs to implement SelectMeshInstances"), FText::FromString(GetClass()->GetName()));
	if (TypedInPointData && TypedOutPointData)
	{
		PCGE_LOG_C(Warning, GraphAndLog, &Context, Message);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SelectInstances(Context, Settings, TypedInPointData, OutMeshInstances, TypedOutPointData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, Message);
		return true;
	}
}

void FPCGMeshMaterialOverrideHelper::Initialize(
	FPCGContext& InContext,
	bool bInUseMaterialOverrideAttributes,
	const TArray<TSoftObjectPtr<UMaterialInterface>>& InStaticMaterialOverrides,
	const TArray<FName>& InMaterialOverrideAttributeNames,
	const UPCGMetadata* InMetadata
)
{
	check(!bIsInitialized);

	bUseMaterialOverrideAttributes = bInUseMaterialOverrideAttributes;
	StaticMaterialOverrides = InStaticMaterialOverrides;
	MaterialOverrideAttributeNames = InMaterialOverrideAttributeNames;
	Metadata = InMetadata;

	Initialize(InContext);

	bIsInitialized = true;
}

void FPCGMeshMaterialOverrideHelper::Initialize(
	FPCGContext& InContext,
	bool bInByAttributeOverride,
	const TArray<FName>& InMaterialOverrideAttributeNames,
	const UPCGMetadata* InMetadata
)
{
	check(!bIsInitialized);

	bUseMaterialOverrideAttributes = bInByAttributeOverride;
	MaterialOverrideAttributeNames = InMaterialOverrideAttributeNames;
	Metadata = InMetadata;

	Initialize(InContext);

	bIsInitialized = true;
}

void FPCGMeshMaterialOverrideHelper::Initialize(FPCGContext& InContext)
{
	// Perform data setup & validation up-front
	if (bUseMaterialOverrideAttributes)
	{
		if (!Metadata)
		{
			PCGE_LOG_C(Error, GraphAndLog, &InContext, LOCTEXT("MetadataMissing", "Data has no metadata"));
			return;
		}

		for (const FName& MaterialOverrideAttributeName : MaterialOverrideAttributeNames)
		{
			if (!Metadata->HasAttribute(MaterialOverrideAttributeName))
			{
				PCGE_LOG_C(Warning, GraphAndLog, &InContext, FText::Format(LOCTEXT("AttributeMissing", "Attribute '{0}' for material overrides is not present in the metadata"), FText::FromName(MaterialOverrideAttributeName)));
				continue;
			}

			const FPCGMetadataAttributeBase* MaterialAttributeBase = Metadata->GetConstAttribute(MaterialOverrideAttributeName);
			check(MaterialAttributeBase);
			
			if (!PCG::Private::IsOfTypes<FSoftObjectPath, FString>(MaterialAttributeBase->GetTypeId()))
			{
				PCGE_LOG_C(Error, GraphAndLog, &InContext, LOCTEXT("AttributeInvalidType", "Material override attribute is not of valid type (FSoftObjectPath or FString)."));
				return;
			}

			MaterialAttributes.Add(MaterialAttributeBase);
		}

		ValueKeyToOverrideMaterials.SetNum(MaterialOverrideAttributeNames.Num());
		WorkingMaterialOverrides.Reserve(MaterialOverrideAttributeNames.Num());
	}

	bIsValid = true;
}

void FPCGMeshMaterialOverrideHelper::Reset()
{
	MaterialAttributes.Reset();
	ValueKeyToOverrideMaterials.Reset();
	WorkingMaterialOverrides.Reset();
	bIsInitialized = false;
	bIsValid = false;
	bUseMaterialOverrideAttributes = false;
	StaticMaterialOverrides.Reset();
	MaterialOverrideAttributeNames.Reset();
	Metadata = nullptr;
}

const TArray<TSoftObjectPtr<UMaterialInterface>>& FPCGMeshMaterialOverrideHelper::GetMaterialOverrides(PCGMetadataEntryKey EntryKey)
{
	check(bIsValid);
	if (bUseMaterialOverrideAttributes)
	{
		WorkingMaterialOverrides.Reset();

		for (int32 MaterialIndex = 0; MaterialIndex < MaterialAttributes.Num(); ++MaterialIndex)
		{
			const FPCGMetadataAttributeBase* MaterialAttribute = MaterialAttributes[MaterialIndex];
			PCGMetadataValueKey MaterialValueKey = MaterialAttribute->GetValueKey(EntryKey);
			TSoftObjectPtr<UMaterialInterface>* NewMaterial = ValueKeyToOverrideMaterials[MaterialIndex].Find(MaterialValueKey);
			TSoftObjectPtr<UMaterialInterface> Material = nullptr;

			if (!NewMaterial)
			{
				FSoftObjectPath MaterialPath;
				if (MaterialAttribute->GetTypeId() == PCG::Private::MetadataTypes<FSoftObjectPath>::Id)
				{
					MaterialPath = static_cast<const FPCGMetadataAttribute<FSoftObjectPath>*>(MaterialAttribute)->GetValue(MaterialValueKey);
				}
				else if (MaterialAttribute->GetTypeId() == PCG::Private::MetadataTypes<FString>::Id)
				{
					MaterialPath = FSoftObjectPath(static_cast<const FPCGMetadataAttribute<FString>*>(MaterialAttribute)->GetValue(MaterialValueKey));
				}
				else
				{
					ensure(false);
					continue;
				}

				Material = TSoftObjectPtr<UMaterialInterface>(MaterialPath);
				ValueKeyToOverrideMaterials[MaterialIndex].Add(MaterialValueKey, Material);
			}
			else
			{
				Material = *NewMaterial;
			}

			WorkingMaterialOverrides.Add(Material);
		}

		return WorkingMaterialOverrides;
	}
	else
	{
		return StaticMaterialOverrides;
	}
}

#undef LOCTEXT_NAMESPACE
