// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSelectors/PCGMeshSelectorByAttribute.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGStaticMeshSpawner.h"
#include "Elements/PCGStaticMeshSpawnerContext.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "MeshSelectors/PCGMeshSelectorBase.h"
#include "Metadata/PCGObjectPropertyOverride.h"
#include "Metadata/PCGMetadataPartitionCommon.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Algo/AnyOf.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMeshSelectorByAttribute)

#define LOCTEXT_NAMESPACE "PCGMeshSelectorByAttribute"

namespace PCGMeshSelectorAttribute
{
	// Returns variation based on mesh, material overrides and reverse culling
	FPCGMeshInstanceList& GetInstanceList(
		TArray<FPCGMeshInstanceList>& InstanceLists,
		const FPCGSoftISMComponentDescriptor& TemplateDescriptor,
		TSoftObjectPtr<UStaticMesh> Mesh,
		const TArray<TSoftObjectPtr<UMaterialInterface>>& MaterialOverrides,
		bool bReverseCulling,
		const UPCGBasePointData* InPointData,
		const int AttributePartitionIndex = INDEX_NONE)
	{
		for (FPCGMeshInstanceList& InstanceList : InstanceLists)
		{
			if (InstanceList.Descriptor.StaticMesh == Mesh &&
				InstanceList.Descriptor.bReverseCulling == bReverseCulling &&
				InstanceList.Descriptor.OverrideMaterials == MaterialOverrides &&
				InstanceList.AttributePartitionIndex == AttributePartitionIndex)
			{
				return InstanceList;
			}
		}

		FPCGMeshInstanceList& NewInstanceList = InstanceLists.Emplace_GetRef(TemplateDescriptor);
		NewInstanceList.Descriptor.StaticMesh = Mesh;
		NewInstanceList.Descriptor.OverrideMaterials = MaterialOverrides;
		NewInstanceList.Descriptor.bReverseCulling = bReverseCulling;
		NewInstanceList.AttributePartitionIndex = AttributePartitionIndex;
		NewInstanceList.PointData = InPointData;

		return NewInstanceList;
	}
}

void UPCGMeshSelectorByAttribute::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (bOverrideMaterials_DEPRECATED)
	{
		MaterialOverrideMode_DEPRECATED = EPCGMeshSelectorMaterialOverrideMode::StaticOverride;
		bOverrideMaterials_DEPRECATED = false;
	}

	if (bOverrideCollisionProfile_DEPRECATED ||
		CollisionProfile_DEPRECATED.Name != UCollisionProfile::NoCollision_ProfileName ||
		MaterialOverrides_DEPRECATED.Num() > 0 ||
		MaterialOverrideMode_DEPRECATED != EPCGMeshSelectorMaterialOverrideMode::NoOverride ||
		CullStartDistance_DEPRECATED != 0 ||
		CullEndDistance_DEPRECATED != 0 ||
		WorldPositionOffsetDisableDistance_DEPRECATED != 0)
	{
		if (bOverrideCollisionProfile_DEPRECATED)
		{
			TemplateDescriptor.bUseDefaultCollision = false;
			TemplateDescriptor.BodyInstance.SetCollisionProfileName(CollisionProfile_DEPRECATED.Name);
		}
		else
		{
			TemplateDescriptor.bUseDefaultCollision = true;
		}

		if (MaterialOverrideMode_DEPRECATED != EPCGMeshSelectorMaterialOverrideMode::NoOverride)
		{
			TemplateDescriptor.OverrideMaterials = MaterialOverrides_DEPRECATED;
		}
		
		TemplateDescriptor.InstanceStartCullDistance = CullStartDistance_DEPRECATED;
		TemplateDescriptor.InstanceEndCullDistance = CullEndDistance_DEPRECATED;
		TemplateDescriptor.WorldPositionOffsetDisableDistance = WorldPositionOffsetDisableDistance_DEPRECATED;

		bUseAttributeMaterialOverrides = (MaterialOverrideMode_DEPRECATED == EPCGMeshSelectorMaterialOverrideMode::ByAttributeOverride);

		bOverrideCollisionProfile_DEPRECATED = false;
		CollisionProfile_DEPRECATED = UCollisionProfile::NoCollision_ProfileName;
		MaterialOverrideMode_DEPRECATED = EPCGMeshSelectorMaterialOverrideMode::NoOverride;
		MaterialOverrides_DEPRECATED.Reset();
		CullStartDistance_DEPRECATED = 0;
		CullEndDistance_DEPRECATED = 0;
		WorldPositionOffsetDisableDistance_DEPRECATED = 0;
	}
#endif

	// TODO: Remove if/when FBodyInstance is updated or replaced
	// Necessary to update the collision Response Container from the Response Array
	TemplateDescriptor.PostLoadFixup(this);
}

bool UPCGMeshSelectorByAttribute::SelectMeshInstances(
	FPCGStaticMeshSpawnerContext& Context,
	const UPCGStaticMeshSpawnerSettings* Settings,
	const UPCGBasePointData* InPointData,
	TArray<FPCGMeshInstanceList>& OutMeshInstances,
	UPCGBasePointData* OutPointData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelectorByAttribute::SelectInstances);

	if (!InPointData)
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, LOCTEXT("InputMissingData", "Missing input data"));
		return true;
	}

	if (!InPointData->Metadata)
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, LOCTEXT("InputMissingMetadata", "Unable to get metadata from input"));
		return true;
	}

	if (!InPointData->Metadata->HasAttribute(AttributeName))
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, FText::Format(LOCTEXT("AttributeNotInMetadata", "Attribute '{0}' is not in the metadata"), FText::FromName(AttributeName)));
		return true;
	}

	const FPCGMetadataAttributeBase* AttributeBase = InPointData->Metadata->GetConstAttribute(AttributeName);
	check(AttributeBase);

	// Validate that the "mesh" attribute is of the right type
	if (!PCG::Private::IsOfTypes<FSoftObjectPath, FString>(AttributeBase->GetTypeId()))
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, FText::Format(LOCTEXT("AttributeInvalidType", "Attribute '{0}' is not of valid type (must be FString or FSoftObjectPath)"), FText::FromName(AttributeName)));
		return true;
	}

	FPCGMeshMaterialOverrideHelper& MaterialOverrideHelper = Context.MaterialOverrideHelper;
	if (!MaterialOverrideHelper.IsInitialized())
	{
		MaterialOverrideHelper.Initialize(Context, bUseAttributeMaterialOverrides, TemplateDescriptor.OverrideMaterials, MaterialOverrideAttributes, InPointData->Metadata);
	}

	if (!MaterialOverrideHelper.IsValid())
	{
		return true;
	}

	// ByAttribute takes in SoftObjectPaths per point in the metadata, so we can pass those directly into the outgoing pin if it exists
	if (Context.CurrentPointIndex == 0 && OutPointData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelectorByAttribute::SetupOutPointData);

		const int32 NumPoints = InPointData->GetNumPoints();
		OutPointData->SetNumPoints(NumPoints);
		InPointData->CopyPointsTo(OutPointData, 0, 0, InPointData->GetNumPoints());

		OutPointData->Metadata->DeleteAttribute(Settings->OutAttributeName);
		OutPointData->Metadata->CopyAttribute(InPointData->Metadata, AttributeName, Settings->OutAttributeName);
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelectorByAttribute::SelectEntries);

	if (!Context.bPartitionDone)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelectorByAttribute::SelectEntries::BuildingPartition);

		TArray<FPCGObjectPropertyOverrideDescription> PropertyOverrides = Settings->StaticMeshComponentPropertyOverrides;
		const FString StaticMeshPropertyString = GET_MEMBER_NAME_CHECKED(FPCGSoftISMComponentDescriptor, StaticMesh).ToString();

		// Add the Static mesh override to the list only if not already provided
		if (!Algo::AnyOf(PropertyOverrides, [&StaticMeshPropertyString](const FPCGObjectPropertyOverrideDescription& PropertyOverride) { return PropertyOverride.PropertyTarget == StaticMeshPropertyString; }))
		{
			FPCGAttributePropertyInputSelector MeshSelector;
			MeshSelector.SetAttributeName(AttributeName);
			PropertyOverrides.Emplace(MeshSelector, StaticMeshPropertyString);
		}

		static FPCGSoftISMComponentDescriptor DummyDescriptor{};

		// Dummy descriptor, to initialize the overrides and do some validation
		FPCGObjectOverrides Overrides(&DummyDescriptor);
		Overrides.Initialize(PropertyOverrides, &DummyDescriptor, InPointData, &Context);

		// Validate all the selectors are actual FPCGSoftISMComponentDescriptor properties
		TArray<FPCGAttributePropertySelector> ValidSelectorOverrides = Overrides.GetAllInputSelectors();

		// Remove all overrides that target data-level specifically
		// @todo_pcg: warning - there`s an assumption that the override attributes aren't global attributes, which we really need to change.
		for (int SelectorIndex = ValidSelectorOverrides.Num() - 1; SelectorIndex >= 0; --SelectorIndex)
		{
			if(InPointData->GetMetadataDomainIDFromSelector(ValidSelectorOverrides[SelectorIndex]) == PCGMetadataDomainID::Data)
			{
				ValidSelectorOverrides.RemoveAt(SelectorIndex, EAllowShrinking::No);
			}
		}

		// If there are valid overrides, partition the points on those attributes so that an instance can be created for each
		if (!ValidSelectorOverrides.IsEmpty())
		{
			Context.AttributeOverridePartition = PCGMetadataPartitionCommon::AttributeGenericPartition(InPointData, ValidSelectorOverrides, &Context, Settings->bSilenceOverrideAttributeNotFoundErrors);
		}

		// Set the descriptors to match the partition count. Uninitialized, because it will be copied from the template below
		Context.OverriddenDescriptors.Reserve(Context.AttributeOverridePartition.Num());
		for (int I = 0; I < Context.AttributeOverridePartition.Num(); ++I)
		{
			FPCGSoftISMComponentDescriptor& Descriptor = Context.OverriddenDescriptors.Add_GetRef(TemplateDescriptor);

			// If partition is empty (which can happen, esp. for the default partition on the default value, we'll just skip it here.
			if (Context.AttributeOverridePartition[I].IsEmpty())
			{
				continue;
			}

			// Use the Object Override to map the user's input selector and property to the descriptor
			Overrides.UpdateTemplateObject(&Descriptor);

			// Since they are already partitioned and identical, we can just use the value on the first point
			check(!Context.AttributeOverridePartition[I].IsEmpty());
			int32 AnyPointIndexOnThisPartition = Context.AttributeOverridePartition[I][0];
			Overrides.Apply(AnyPointIndexOnThisPartition);
		}

		// Given partitioning is expensive, check if we're out of time for this frame
		Context.bPartitionDone = true;
		if (Context.ShouldStop())
		{
			return false;
		}
	}

	// Assign points to entries
	int32 CurrentPartitionIndex = Context.CurrentPointIndex; // misnomer but we're reusing another concept from the context
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelectorByAttribute::SelectEntries::PushingPointsToInstanceLists);

		const TConstPCGValueRange<FTransform> InTransformRange = InPointData->GetConstTransformValueRange();
		const TConstPCGValueRange<int64> InMetadataEntryRange = InPointData->GetConstMetadataEntryValueRange();
				
		// TODO: Revisit this when attribute partitioning is returned in a more optimized form
		// The partition index is used to assign the point to the correct partition's instance
		while(CurrentPartitionIndex < Context.AttributeOverridePartition.Num())
		{
			const int32 PartitionIndex = CurrentPartitionIndex++;
			TArray<int32>& Partition = Context.AttributeOverridePartition[PartitionIndex];
			const FPCGSoftISMComponentDescriptor& CurrentPartitionDescriptor = Context.OverriddenDescriptors[PartitionIndex];
			
			if (Partition.IsEmpty() || CurrentPartitionDescriptor.StaticMesh.IsNull())
			{
				continue;
			}

			// Setup data for mesh bounds computation
			if (OutPointData && Settings->bApplyMeshBoundsToPoints)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelectorByAttribute::SelectEntries::PushPointsToApplyMeshBounds);
				TArray<int32>& PointIndices = Context.MeshToOutPoints.FindOrAdd(CurrentPartitionDescriptor.StaticMesh).FindOrAdd(OutPointData);
				PointIndices.Append(Partition);
			}

			// Separate the inverse determinant instances so we can push them to a different ISM
			TArray<int32> ReverseInstances;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelectorByAttribute::SelectEntries::BuildReverseInstances);
				int32 WriteIndex = 0;
				for (int32 ReadIndex = 0; ReadIndex < Partition.Num(); ++ReadIndex)
				{
					const FTransform& InTransform = InTransformRange[Partition[ReadIndex]];
					if (InTransform.GetDeterminant() < 0)
					{
						ReverseInstances.Add(Partition[ReadIndex]);
					}
					else
					{
						if (WriteIndex != ReadIndex)
						{
							Partition[WriteIndex] = Partition[ReadIndex];
						}
						++WriteIndex;
					}
				}

				Partition.SetNum(WriteIndex);
			}

			bool bHasInvalidRotation = false;

			auto AddPointsToInstanceList = [&OutMeshInstances, &CurrentPartitionDescriptor, &MaterialOverrideHelper, &InTransformRange, &InMetadataEntryRange, &bHasInvalidRotation, PartitionIndex, InPointData](const TArray<int32>& PointIndices, bool bReverseTransform)
			{
				if (MaterialOverrideHelper.OverridesMaterials())
				{
					for(int32 PointIndex : PointIndices)
					{
						const FTransform& InTransform = InTransformRange[PointIndex];

						if (!InTransform.IsRotationNormalized())
						{
							bHasInvalidRotation = true;
							continue;
						}

						const int64& InMetadataEntry = InMetadataEntryRange[PointIndex];
						FPCGMeshInstanceList& InstanceList = PCGMeshSelectorAttribute::GetInstanceList(OutMeshInstances, CurrentPartitionDescriptor, CurrentPartitionDescriptor.StaticMesh, MaterialOverrideHelper.GetMaterialOverrides(InMetadataEntry), bReverseTransform, InPointData, PartitionIndex);
						InstanceList.Instances.Emplace(InTransform);
						InstanceList.InstancesIndices.Emplace(PointIndex);
					}
				}
				else
				{
					FPCGMeshInstanceList& InstanceList = PCGMeshSelectorAttribute::GetInstanceList(OutMeshInstances, CurrentPartitionDescriptor, CurrentPartitionDescriptor.StaticMesh, CurrentPartitionDescriptor.OverrideMaterials, bReverseTransform, InPointData, PartitionIndex);

					check(InstanceList.Instances.Num() == InstanceList.InstancesIndices.Num());
					const int32 InstanceOffset = InstanceList.Instances.Num();
					InstanceList.Instances.Reserve(InstanceOffset + PointIndices.Num());
					InstanceList.InstancesIndices.Reserve(InstanceOffset + PointIndices.Num());

					for (int32 PointIndex : PointIndices)
					{
						const FTransform& InTransform = InTransformRange[PointIndex];

						if (!InTransform.IsRotationNormalized())
						{
							bHasInvalidRotation = true;
							continue;
						}

						InstanceList.Instances.Emplace(InTransform);
						InstanceList.InstancesIndices.Emplace(PointIndex);
					}
				}
			};

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelectorByAttribute::SelectEntries::AddPointsToInstanceList);
				AddPointsToInstanceList(Partition, false);
				AddPointsToInstanceList(ReverseInstances, true);

				// @todo_pcg: we should log this only once, not once-per-call, but this requires a change in the context.
				if (bHasInvalidRotation)
				{
					PCGLog::Validation::LogInvalidRotationsSkipped();
				}
			}

			if (Context.ShouldStop())
			{
				break;
			}
		}
	}

	Context.CurrentPointIndex = CurrentPartitionIndex; // misnomer, but we're using the same context
	return CurrentPartitionIndex == Context.AttributeOverridePartition.Num();
}

#undef LOCTEXT_NAMESPACE
