// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSelectors/PCGMeshSelectorPrimitiveData.h"

#include "PCGParamData.h"
#include "Elements/PCGStaticMeshSpawner.h"
#include "Elements/PCGStaticMeshSpawnerContext.h"

#include "Algo/Compare.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMeshSelectorPrimitiveData)

#define LOCTEXT_NAMESPACE "PCGMeshSelectorPrimitiveData"

namespace PCGMeshSelectorPrimitiveData
{
	// Returns variation based on mesh, material overrides and reverse culling
	FPCGMeshInstanceList& GetInstanceList(
		TArray<FPCGMeshInstanceList>& InstanceLists,
		const FPCGSoftISMComponentDescriptor& TemplateDescriptor,
		TSoftObjectPtr<UStaticMesh> Mesh,
		const TArray<TSoftObjectPtr<UMaterialInterface>>& MaterialOverrides,
		bool bReverseCulling,
		const UPCGBasePointData* InPointData,
		FPCGPackedCustomData InCustomData,
		const int AttributePartitionIndex)
	{
		TConstArrayView<float> PrimitiveCustomData = InCustomData.GetViewForIndex(AttributePartitionIndex);
		
		for (FPCGMeshInstanceList& InstanceList : InstanceLists)
		{
			if (InstanceList.Descriptor.StaticMesh == Mesh &&
				InstanceList.Descriptor.bReverseCulling == bReverseCulling &&
				InstanceList.Descriptor.OverrideMaterials == MaterialOverrides &&
				InstanceList.AttributePartitionIndex == AttributePartitionIndex &&
				Algo::Compare(InstanceList.CustomPrimitiveData, PrimitiveCustomData, [](const float LHS, const float RHS) { return FMath::IsNearlyEqual(LHS, RHS);}))
			{
				return InstanceList;
			}
		}

		FPCGMeshInstanceList& NewInstanceList = InstanceLists.Emplace_GetRef(TemplateDescriptor);
		NewInstanceList.Descriptor.StaticMesh = Mesh;
		NewInstanceList.Descriptor.OverrideMaterials = MaterialOverrides;
		NewInstanceList.Descriptor.bReverseCulling = bReverseCulling;
		NewInstanceList.AttributePartitionIndex = AttributePartitionIndex;
		NewInstanceList.PointData = InPointData;
		NewInstanceList.CustomPrimitiveData.Append(PrimitiveCustomData);

		return NewInstanceList;
	}
}

bool UPCGMeshSelectorPrimitiveData::SelectMeshInstances(
	FPCGStaticMeshSpawnerContext& Context,
	const UPCGStaticMeshSpawnerSettings* Settings,
	const UPCGBasePointData* InPointData,
	TArray<FPCGMeshInstanceList>& OutMeshInstances,
	UPCGBasePointData* OutPointData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelectorPrimitiveData::SelectInstances);

	if (!InPointData || !InPointData->ConstMetadata())
	{
		PCGLog::InputOutput::LogInvalidInputDataError(&Context);
		return true;
	}

	if (!InPointData->Metadata->HasAttribute(PrimitiveIndexAttribute))
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, FText::Format(LOCTEXT("AttributeNotInMetadata", "Attribute '{0}' is not in the metadata"), FText::FromName(PrimitiveIndexAttribute)));
		return true;
	}

	// Read the data table and create MeshInstanceList out of it
	const TArray<FPCGTaggedData> PrimitiveTableInput = Context.InputData.GetInputsByPin(PCGStaticMeshSpawner::PrimitiveTablePinLabel);
	if (PrimitiveTableInput.IsEmpty())
	{
		return true;
	}

	if (PrimitiveTableInput.Num() > 1)
	{
		PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGStaticMeshSpawner::PrimitiveTablePinLabel, &Context);
	}

	const UPCGParamData* PrimitiveTableData = Cast<const UPCGParamData>(PrimitiveTableInput[0].Data);
	if (!PrimitiveTableData || !PrimitiveTableData->ConstMetadata())
	{
		PCGLog::InputOutput::LogInvalidInputDataError(&Context);
		return true;
	}

	const bool bUseAttributeMaterialOverrides = !MaterialOverrideAttributes.IsEmpty();

	const FPCGAttributePropertyInputSelector MeshSelector = FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(MeshAttribute);
	TArray<FSoftObjectPath> Meshes;

	if (!PCGAttributeAccessorHelpers::ExtractAllValues<FSoftObjectPath>(PrimitiveTableData, MeshSelector, Meshes, &Context, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
	{
		return true;
	}

	if (Meshes.IsEmpty())
	{
		return true;
	}

	TArray<FPCGSoftISMComponentDescriptor> Descriptors;
	TArray<TArray<float>> CustomPrimitiveData;
	Descriptors.Reserve(Meshes.Num());
	for (int32 i = 0; i < Meshes.Num(); ++i)
	{
		FPCGSoftISMComponentDescriptor& Descriptor = Descriptors.Emplace_GetRef();
		Descriptor.StaticMesh = Meshes[i];

		FPCGMeshMaterialOverrideHelper MaterialOverrideHelper;
		MaterialOverrideHelper.Initialize(Context, bUseAttributeMaterialOverrides, Descriptor.OverrideMaterials, MaterialOverrideAttributes, PrimitiveTableData->ConstMetadata());
		if (MaterialOverrideHelper.IsValid())
		{
			// Param data keys are between 0 and Num() - 1;
			Descriptor.OverrideMaterials = MaterialOverrideHelper.GetMaterialOverrides(PCGMetadataEntryKey{i});	
		}

		FPCGObjectOverrides Overrides(&Descriptor);
		Overrides.Initialize(PrimitiveOverrideAttributes, &Descriptor, PrimitiveTableData, &Context);
		if (Overrides.IsValid())
		{
			Overrides.Apply(i);
		}

		Context.MeshToOutPoints.FindOrAdd(Descriptor.StaticMesh).FindOrAdd(OutPointData);
	}

	FPCGPackedCustomData CustomData;
	PackCustomPrimitiveData(PrimitiveTableData, Meshes.Num(), CustomData, &Context);

	// @todo_pcg: Time Slice, maybe partition is better?
	const FPCGAttributePropertyInputSelector PrimitiveIndexSelector = FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(PrimitiveIndexAttribute);
	const TUniquePtr<const IPCGAttributeAccessor> PrimitiveIndexAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InPointData, PrimitiveIndexSelector);
	const TUniquePtr<const IPCGAttributeAccessorKeys> PrimitiveIndexKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InPointData, PrimitiveIndexSelector);

	if (!PrimitiveIndexAccessor || !PrimitiveIndexKeys)
	{
		PCGLog::Metadata::LogFailToCreateAccessorError(PrimitiveIndexSelector, &Context);
		return true;
	}

	const TConstPCGValueRange<FTransform> InTransformRange = InPointData->GetConstTransformValueRange();

	TArray<TPair</*MeshInstanceIndex*/int32, /*ReverseMeshInstanceIndex*/int32>> PrimitiveToMeshInstanceMapping;
	PrimitiveToMeshInstanceMapping.Init({INDEX_NONE, INDEX_NONE}, Descriptors.Num());

	bool bHasInvalidRotations = false;

	PCGMetadataElementCommon::ApplyOnAccessor<int32>(*PrimitiveIndexKeys, *PrimitiveIndexAccessor, [&InTransformRange, &OutMeshInstances, &Descriptors, &PrimitiveToMeshInstanceMapping, &Context, bUseAttributeMaterialOverrides, InPointData, OutPointData, &CustomData, &bHasInvalidRotations](int32 PrimitiveIndex, int32 PointIndex)
	{
		if (!Descriptors.IsValidIndex(PrimitiveIndex))
		{
			return;
		}

		const FPCGSoftISMComponentDescriptor& Descriptor = Descriptors[PrimitiveIndex];
		const FTransform& InTransform = InTransformRange[PointIndex];

		if (!InTransform.IsRotationNormalized())
		{
			bHasInvalidRotations = true;
			return;
		}

		const bool bReverseInstance = InTransform.GetDeterminant() < 0;
		int32& MeshInstanceIndex = bReverseInstance ? PrimitiveToMeshInstanceMapping[PrimitiveIndex].Value : PrimitiveToMeshInstanceMapping[PrimitiveIndex].Key;
		if (MeshInstanceIndex == INDEX_NONE)
		{
			FPCGMeshInstanceList& InstanceList = PCGMeshSelectorPrimitiveData::GetInstanceList(OutMeshInstances, Descriptor, Descriptor.StaticMesh, Descriptor.OverrideMaterials, bReverseInstance, InPointData, CustomData, PrimitiveIndex);
			MeshInstanceIndex = static_cast<uint32>(&InstanceList - OutMeshInstances.GetData());
		}

		check(MeshInstanceIndex != INDEX_NONE);
		FPCGMeshInstanceList& InstanceList = OutMeshInstances[MeshInstanceIndex];
		Context.MeshToOutPoints[Descriptor.StaticMesh][OutPointData].Add(PointIndex);
		InstanceList.Instances.Emplace(InTransform);
		InstanceList.InstancesIndices.Emplace(PointIndex);

	}, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);

	if (bHasInvalidRotations)
	{
		PCGLog::Validation::LogInvalidRotationsSkipped();
	}

	return true;
}

void UPCGMeshSelectorPrimitiveData::PackCustomPrimitiveData(const UPCGData* InputData, int32 NumInstances, FPCGPackedCustomData& OutCustomPrimitiveData, FPCGContext* OptionalContext) const
{
	if (!CustomPrimitiveDataAttributes.IsEmpty())
	{
		PCGInstanceDataPackerBase::FPackedDataParams Params =
		{
			.CommonParams =
			{
				.NumInstances = NumInstances,
				.OutPackedCustomData = &OutCustomPrimitiveData,
				.OptionalContext = OptionalContext
			},
			.InData = InputData,
			.Selectors = CustomPrimitiveDataAttributes
		};

		PCGInstanceDataPackerBase::PackCustomData(Params);
	}
}

#undef LOCTEXT_NAMESPACE
