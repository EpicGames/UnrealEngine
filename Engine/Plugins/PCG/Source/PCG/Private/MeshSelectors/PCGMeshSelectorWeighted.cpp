// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSelectors/PCGMeshSelectorWeighted.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGStaticMeshSpawner.h"
#include "Elements/PCGStaticMeshSpawnerContext.h"
#include "Helpers/PCGHelpers.h"
#include "MeshSelectors/PCGMeshSelectorBase.h"

#include "Engine/StaticMesh.h"
#include "Math/RandomStream.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMeshSelectorWeighted)

#define LOCTEXT_NAMESPACE "PCGMeshSelectorWeighted"

namespace PCGMeshSelectorWeighted
{
	FPCGMeshInstanceList& GetInstanceList(
		TArray<FPCGMeshInstanceList>& InstanceLists,
		bool bUseMaterialOverrides,
		const TArray<TSoftObjectPtr<UMaterialInterface>>& InMaterialOverrides,
		bool bInReverseCulling,
		const UPCGBasePointData* InPointData)
	{
		check(InstanceLists.Num() > 0);

		// First look through previously existing values - note that we scope this to prevent issues with the 0 index access which might become invalid below
		{
			const TArray<TSoftObjectPtr<UMaterialInterface>>& MaterialOverrides = (bUseMaterialOverrides ? InMaterialOverrides : InstanceLists[0].Descriptor.OverrideMaterials);
			for (FPCGMeshInstanceList& InstanceList : InstanceLists)
			{
				if (InstanceList.Descriptor.bReverseCulling == bInReverseCulling &&
					InstanceList.Descriptor.OverrideMaterials == MaterialOverrides)
				{
					return InstanceList;
				}
			}
		}

		FPCGMeshInstanceList& NewInstanceList = InstanceLists.Emplace_GetRef();
		NewInstanceList.Descriptor = InstanceLists[0].Descriptor;
		NewInstanceList.Descriptor.bReverseCulling = bInReverseCulling;
		NewInstanceList.PointData = InPointData;

		if (bUseMaterialOverrides)
		{
			NewInstanceList.Descriptor.OverrideMaterials = InMaterialOverrides;
		}

		return NewInstanceList;
	}
}

FPCGMeshSelectorWeightedEntry::FPCGMeshSelectorWeightedEntry(TSoftObjectPtr<UStaticMesh> InMesh, int InWeight)
	: Weight(InWeight)
{
	Descriptor.StaticMesh = InMesh;
#if WITH_EDITOR
	DisplayName = InMesh.ToSoftObjectPath().GetAssetFName();
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void FPCGMeshSelectorWeightedEntry::ApplyDeprecation()
{
	if (!Mesh_DEPRECATED.IsNull() ||
		bOverrideCollisionProfile_DEPRECATED ||
		CollisionProfile_DEPRECATED.Name != UCollisionProfile::NoCollision_ProfileName ||
		bOverrideMaterials_DEPRECATED ||
		MaterialOverrides_DEPRECATED.Num() > 0 ||
		CullStartDistance_DEPRECATED != 0 ||
		CullEndDistance_DEPRECATED != 0 ||
		WorldPositionOffsetDisableDistance_DEPRECATED != 0)
	{
		Descriptor.StaticMesh = Mesh_DEPRECATED;
		
		if (bOverrideCollisionProfile_DEPRECATED)
		{
			Descriptor.bUseDefaultCollision = false;
			Descriptor.BodyInstance.SetCollisionProfileName(CollisionProfile_DEPRECATED.Name);
		}
		else
		{
			Descriptor.bUseDefaultCollision = true;
		}

		Descriptor.InstanceStartCullDistance = CullStartDistance_DEPRECATED;
		Descriptor.InstanceEndCullDistance = CullEndDistance_DEPRECATED;
		Descriptor.WorldPositionOffsetDisableDistance = WorldPositionOffsetDisableDistance_DEPRECATED;

		if (bOverrideMaterials_DEPRECATED)
		{
			Descriptor.OverrideMaterials = MaterialOverrides_DEPRECATED;
		}

		Mesh_DEPRECATED.Reset();
		bOverrideCollisionProfile_DEPRECATED = false;
		CollisionProfile_DEPRECATED = UCollisionProfile::NoCollision_ProfileName;
		bOverrideMaterials_DEPRECATED = false;
		MaterialOverrides_DEPRECATED.Reset();
		CullStartDistance_DEPRECATED = 0;
		CullEndDistance_DEPRECATED = 0;		
		WorldPositionOffsetDisableDistance_DEPRECATED = 0;
	}
}
#endif

bool UPCGMeshSelectorWeighted::SelectMeshInstances(
	FPCGStaticMeshSpawnerContext& Context,
	const UPCGStaticMeshSpawnerSettings* Settings, 
	const UPCGBasePointData* InPointData,
	TArray<FPCGMeshInstanceList>& OutMeshInstances,
	UPCGBasePointData* OutPointData) const
{
	if (!InPointData)
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, LOCTEXT("MissingInputData", "Missing input data"));
		return true;
	}

	TArray<TArray<FPCGMeshInstanceList>>& MeshInstances = Context.WeightedMeshInstances;
	TArray<int>& CumulativeWeights = Context.CumulativeWeights;

	// TODO: Remove this log once the other selection modes are available
	if (!Settings->StaticMeshComponentPropertyOverrides.IsEmpty())
	{
		PCGE_LOG_C(Log, LogOnly, &Context, LOCTEXT("AttributeToPropertyOverrideUnavailable", "Attribute to Property Overrides are only currently available with the 'By Attribute' Selector"));
	}

	// Setup
	if (Context.CurrentPointIndex == 0)
	{
		int TotalWeight = 0;

		// Prepare common mesh setups which we will use as a kind of map
		for (const FPCGMeshSelectorWeightedEntry& Entry : MeshEntries)
		{
			if (Entry.Weight <= 0)
			{
				PCGE_LOG_C(Verbose, LogOnly, &Context, LOCTEXT("EntryWithNegativeWeight", "Entry found with weight <= 0"));
				continue;
			}

			TArray<FPCGMeshInstanceList>& PickEntry = MeshInstances.Emplace_GetRef();
			FPCGMeshInstanceList& MeshInstanceList = PickEntry.Emplace_GetRef(Entry.Descriptor);
			MeshInstanceList.PointData = InPointData;

			TotalWeight += Entry.Weight;
			CumulativeWeights.Add(TotalWeight);
		}

		if (TotalWeight <= 0)
		{
			return true;
		}
	}

	FPCGMeshMaterialOverrideHelper& MaterialOverrideHelper = Context.MaterialOverrideHelper;
	if (!MaterialOverrideHelper.IsInitialized())
	{
		MaterialOverrideHelper.Initialize(Context, bUseAttributeMaterialOverrides, MaterialOverrideAttributes, InPointData->Metadata);
	}

	if (!MaterialOverrideHelper.IsValid())
	{
		return true;
	}

	FPCGMetadataAttribute<FString>* OutAttribute = nullptr;

	FPCGPointValueRanges OutRanges;
	
	if (OutPointData)
	{
		OutRanges = FPCGPointValueRanges(OutPointData);

		check(OutPointData->Metadata);

		if (!OutPointData->Metadata->HasAttribute(Settings->OutAttributeName)) 
		{
			PCGE_LOG_C(Error, GraphAndLog, &Context, FText::Format(LOCTEXT("AttributeNotInMetadata", "Out attribute '{0}' is not in the metadata"), FText::FromName(Settings->OutAttributeName)));
		}

		FPCGMetadataAttributeBase* OutAttributeBase = OutPointData->Metadata->GetMutableAttribute(Settings->OutAttributeName);

		if (OutAttributeBase)
		{
			if (OutAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FString>::Id)
			{
				OutAttribute = static_cast<FPCGMetadataAttribute<FString>*>(OutAttributeBase);
			}
			else
			{
				PCGE_LOG_C(Error, GraphAndLog, &Context, LOCTEXT("AttributeNotFString", "Out attribute is not of valid type FString"));
			}
		}
	}

	// Assign points to entries
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGStaticMeshSpawnerElement::Execute::SelectEntries);
		
		const FConstPCGPointValueRanges InRanges(InPointData);

		int32 CurrentPointIndex = Context.CurrentPointIndex;
		int32 LastCheckpointIndex = CurrentPointIndex;
		constexpr int32 TimeSlicingCheckFrequency = 1024;
		TMap<TSoftObjectPtr<UStaticMesh>, PCGMetadataValueKey>& MeshToValueKey = Context.MeshToValueKey;
		const int32 TotalWeight = CumulativeWeights.Last();
		bool bHasInvalidRotations = false;

		while (CurrentPointIndex < InPointData->GetNumPoints())
		{
			const int32 InPointIndex = CurrentPointIndex++;

			const FTransform& InTransform = InRanges.TransformRange[InPointIndex];

			if (!InTransform.IsRotationNormalized())
			{
				bHasInvalidRotations = true;
				continue;
			}

			const int64& InMetadataEntry = InRanges.MetadataEntryRange[InPointIndex];
			const int32& InSeed = InRanges.SeedRange[InPointIndex];
			
			FRandomStream RandomSource = PCGHelpers::GetRandomStreamFromSeed(InSeed, Settings, Context.ExecutionSource.Get());
			const int RandomWeightedPick = RandomSource.RandRange(0, TotalWeight - 1);

			int RandomPick = 0;
			while(RandomPick < MeshInstances.Num() && CumulativeWeights[RandomPick] <= RandomWeightedPick)
			{
				++RandomPick;
			}

			if(RandomPick < MeshInstances.Num())
			{
				const bool bNeedsReverseCulling = (InTransform.GetDeterminant() < 0);
				FPCGMeshInstanceList& InstanceList = PCGMeshSelectorWeighted::GetInstanceList(MeshInstances[RandomPick], bUseAttributeMaterialOverrides, MaterialOverrideHelper.GetMaterialOverrides(InMetadataEntry), bNeedsReverseCulling, InPointData);
				InstanceList.Instances.Emplace(InTransform);
				InstanceList.InstancesIndices.Emplace(CurrentPointIndex - 1); // - 1 because it is already incremented.

				const TSoftObjectPtr<UStaticMesh>& Mesh = InstanceList.Descriptor.StaticMesh;

				if (OutPointData && OutAttribute)
				{
					OutRanges.SetFromValueRanges(Context.CurrentWriteIndex, InRanges, InPointIndex);

					PCGMetadataValueKey* OutValueKey = MeshToValueKey.Find(Mesh);
					if (!OutValueKey)
					{
						PCGMetadataValueKey ValueKey = OutAttribute->AddValue(Mesh.ToSoftObjectPath().ToString());
						OutValueKey = &MeshToValueKey.Add(Mesh, ValueKey);
					}

					check(OutValueKey);

					OutPointData->Metadata->InitializeOnSet(OutRanges.MetadataEntryRange[Context.CurrentWriteIndex]);
					OutAttribute->SetValueFromValueKey(OutRanges.MetadataEntryRange[Context.CurrentWriteIndex], *OutValueKey);

					++Context.CurrentWriteIndex;

					if (Settings->bApplyMeshBoundsToPoints)
					{
						TArray<int32>& PointIndices = Context.MeshToOutPoints.FindOrAdd(Mesh).FindOrAdd(OutPointData);
						// CurrentPointIndex - 1, because CurrentPointIndex is incremented at the beginning of the loop
						PointIndices.Emplace(CurrentPointIndex - 1);
					}
				}
			}

			// Check if we should stop here and continue in a subsequent call
			if (CurrentPointIndex - LastCheckpointIndex >= TimeSlicingCheckFrequency)
			{
				if (Context.ShouldStop())
				{
					break;
				}
				else
				{
					LastCheckpointIndex = CurrentPointIndex;
				}
			}
		}

		// @todo_pcg: we should log this only once, not once-per-call, but this requires a change in the context.
		if (bHasInvalidRotations)
		{
			PCGLog::Validation::LogInvalidRotationsSkipped();
		}

		Context.CurrentPointIndex = CurrentPointIndex;
	}

	if (Context.CurrentPointIndex == InPointData->GetNumPoints())
	{
		if (OutPointData)
		{
			OutPointData->SetNumPoints(Context.CurrentWriteIndex);
		}

		// Finally, collapse to OutMeshInstances
		for (TArray<FPCGMeshInstanceList>& PickedMeshInstances : MeshInstances)
		{
			for (FPCGMeshInstanceList& PickedMeshInstanceEntry : PickedMeshInstances)
			{
				OutMeshInstances.Emplace(MoveTemp(PickedMeshInstanceEntry));
			}
		}

		return true;
	}
	else
	{
		return false;
	}
}

void UPCGMeshSelectorWeighted::PostLoad()
{
	Super::PostLoad();

	for (FPCGMeshSelectorWeightedEntry& Entry : MeshEntries)
	{
#if WITH_EDITOR
		Entry.ApplyDeprecation();
#endif // WITH_EDITOR

		// TODO: Remove if/when FBodyInstance is updated or replaced
		// Necessary to update the collision Response Container from the Response Array
		Entry.Descriptor.PostLoadFixup(this);
	}

#if WITH_EDITOR
	RefreshDisplayNames();
#endif // WITH_EDITOR
}

void UPCGMeshSelectorWeighted::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

#if WITH_EDITOR
	RefreshDisplayNames();
#endif // WITH_EDITOR
}

void UPCGMeshSelectorWeighted::PostEditImport()
{
	Super::PostEditImport();

#if WITH_EDITOR
	RefreshDisplayNames();
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UPCGMeshSelectorWeighted::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FPCGSoftISMComponentDescriptor, StaticMesh))
	{
		RefreshDisplayNames();
	}
}

void UPCGMeshSelectorWeighted::RefreshDisplayNames()
{
	for (FPCGMeshSelectorWeightedEntry& Entry : MeshEntries)
	{
		Entry.DisplayName = Entry.Descriptor.StaticMesh.ToSoftObjectPath().GetAssetFName();
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
