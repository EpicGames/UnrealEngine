// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSelectors/PCGMeshSelectorWeightedByCategory.h"

#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGStaticMeshSpawner.h"
#include "Elements/PCGStaticMeshSpawnerContext.h"
#include "Helpers/PCGHelpers.h"
#include "MeshSelectors/PCGMeshSelectorWeighted.h"

#include "Engine/StaticMesh.h"
#include "Math/RandomStream.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMeshSelectorWeightedByCategory)

#define LOCTEXT_NAMESPACE "PCGMeshSelectorWeightedByCategory"

#if WITH_EDITOR
void FPCGWeightedByCategoryEntryList::ApplyDeprecation()
{
	for (FPCGMeshSelectorWeightedEntry& Entry : WeightedMeshEntries)
	{
		Entry.ApplyDeprecation();
	}
}
#endif

bool UPCGMeshSelectorWeightedByCategory::SelectMeshInstances(
	FPCGStaticMeshSpawnerContext& Context,
	const UPCGStaticMeshSpawnerSettings* Settings,
	const UPCGBasePointData* InPointData,
	TArray<FPCGMeshInstanceList>& OutMeshInstances,
	UPCGBasePointData* OutPointData) const
{
	if (!InPointData)
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, LOCTEXT("InputMissingData", "Missing input data"));
		return true;
	}

	if (!InPointData->Metadata)
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, LOCTEXT("InputMissingMetadata", "Unable to get metadata from input"));
		return true;
	}

	if (!InPointData->Metadata->HasAttribute(CategoryAttribute))
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, FText::Format(LOCTEXT("InputMissingAttribute", "Attribute '{0}' is not in the metadata"), FText::FromName(CategoryAttribute)));
		return true;
	}

	const FPCGMetadataAttributeBase* AttributeBase = InPointData->Metadata->GetConstAttribute(CategoryAttribute);
	check(AttributeBase);

	// TODO: support enum type as well
	if (AttributeBase->GetTypeId() != PCG::Private::MetadataTypes<FString>::Id)
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, FText::Format(LOCTEXT("AttributeInvalidType", "Attribute '{0}' is not of valid type FString"), FText::FromName(CategoryAttribute)));
		return true;
	}

	const FPCGMetadataAttribute<FString>* Attribute = static_cast<const FPCGMetadataAttribute<FString>*>(AttributeBase);

	// maps a CategoryEntry ValueKey to the meshes and precomputed weight data 
	TMap<PCGMetadataValueKey, FPCGInstancesAndWeights>& CategoryEntryToInstancesAndWeights = Context.CategoryEntryToInstancesAndWeights;

	// unmarked points will fallback to the MeshEntries associated with the DefaultValueKey
	PCGMetadataValueKey DefaultValueKey = PCGDefaultValueKey;

	// TODO: Remove this log once the other selection modes are available
	if (!Settings->StaticMeshComponentPropertyOverrides.IsEmpty())
	{
		PCGE_LOG_C(Log, LogOnly, &Context, LOCTEXT("AttributeToPropertyOverrideUnavailable", "Attribute to Property Overrides are only currently available with the 'By Attribute' Selector"));
	}

	// Setup
	if (Context.CurrentPointIndex == 0)
	{
		const FString AttributeDefaultValue = Attribute->GetValue(PCGDefaultValueKey);

		for (const FPCGWeightedByCategoryEntryList& Entry : Entries)
		{
			if (Entry.WeightedMeshEntries.Num() == 0)
			{
				PCGE_LOG_C(Verbose, LogOnly, &Context, FText::Format(LOCTEXT("EmptyEntryInCategory", "Empty entry found in category '{0}'"), FText::FromString(Entry.CategoryEntry)));
				continue;
			}

			PCGMetadataValueKey ValueKey = Attribute->FindValue(Entry.CategoryEntry);

			if (ValueKey == PCGNotFoundValueKey)
			{
				PCGE_LOG_C(Verbose, LogOnly, &Context, FText::Format(LOCTEXT("UnusedCategory", "Unused category '{0}'. Not a valid value for attribute '{1}'."), 
					FText::FromString(Entry.CategoryEntry), FText::FromName(CategoryAttribute)));
				continue;
			}

			FPCGInstancesAndWeights* InstancesAndWeights = CategoryEntryToInstancesAndWeights.Find(ValueKey);

			if (InstancesAndWeights)
			{
				PCGE_LOG_C(Warning, GraphAndLog, &Context, FText::Format(LOCTEXT("DuplicateEntry", "Duplicate entry found in category '{0}'. Subsequent entries are ignored."), FText::FromString(Entry.CategoryEntry)));
				continue;
			}

			if (Entry.IsDefault)
			{
				if (DefaultValueKey == PCGDefaultValueKey)
				{
					DefaultValueKey = ValueKey;
				}
				else
				{
					PCGE_LOG_C(Warning, GraphAndLog, &Context, LOCTEXT("DuplicateDefaultEntry", "Duplicate default entry found. Subsequent default entries are ignored."));
				}
			}

			InstancesAndWeights = &CategoryEntryToInstancesAndWeights.Add(ValueKey, FPCGInstancesAndWeights());

			int TotalWeight = 0;
			for (const FPCGMeshSelectorWeightedEntry& WeightedEntry : Entry.WeightedMeshEntries)
			{
				if (WeightedEntry.Weight <= 0)
				{
					PCGE_LOG_C(Verbose, LogOnly, &Context, FText::Format(LOCTEXT("ZeroOrNegativeWeight", "Entry found with weight <= 0 in category '{0}'"), FText::FromString(Entry.CategoryEntry)));
					continue;
				}

				TArray<FPCGMeshInstanceList>& PickEntry = InstancesAndWeights->MeshInstances.Emplace_GetRef();
				FPCGMeshInstanceList& MeshInstanceList = PickEntry.Emplace_GetRef(WeightedEntry.Descriptor);
				MeshInstanceList.PointData = InPointData;

				// precompute the weights
				TotalWeight += WeightedEntry.Weight;
				InstancesAndWeights->CumulativeWeights.Add(TotalWeight);
			}

			// if no weighted entries were collected, discard this InstancesAndWeights
			if (InstancesAndWeights->CumulativeWeights.Num() == 0)
			{
				CategoryEntryToInstancesAndWeights.Remove(ValueKey);
			}
		}
	}

	FPCGMeshMaterialOverrideHelper& MaterialOverrideHelper = Context.MaterialOverrideHelper;
	if (!MaterialOverrideHelper.IsInitialized())
	{
		MaterialOverrideHelper.Initialize(Context, bUseAttributeMaterialOverrides, MaterialOverrideAttributes, InPointData->Metadata);
	}

	if (!MaterialOverrideHelper.IsValid())
	{
		return true;
	}

	FPCGMetadataAttribute<FString>* OutAttribute = nullptr;

	FPCGPointValueRanges OutRanges;

	if (OutPointData)
	{
		OutRanges = FPCGPointValueRanges(OutPointData);
		
		check(OutPointData->Metadata);

		if (!OutPointData->Metadata->HasAttribute(Settings->OutAttributeName)) 
		{
			PCGE_LOG_C(Error, GraphAndLog, &Context, FText::Format(LOCTEXT("AttributeNotInMetadata", "Out attribute '{0}' is not in the metadata"), FText::FromName(Settings->OutAttributeName)));
		}

		FPCGMetadataAttributeBase* OutAttributeBase = OutPointData->Metadata->GetMutableAttribute(Settings->OutAttributeName);

		if (OutAttributeBase)
		{
			if (OutAttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FString>::Id)
			{
				OutAttribute = static_cast<FPCGMetadataAttribute<FString>*>(OutAttributeBase);
			}
			else
			{
				PCGE_LOG_C(Error, GraphAndLog, &Context, LOCTEXT("TypeNotFString", "Out attribute is not of valid type FString"));
			}
		}
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGStaticMeshSpawnerElement::Execute::SelectEntries);

	int32 CurrentPointIndex = Context.CurrentPointIndex;
	int32 LastCheckpointIndex = CurrentPointIndex;
	constexpr int32 TimeSlicingCheckFrequency = 1024;
	TMap<TSoftObjectPtr<UStaticMesh>, PCGMetadataValueKey>& MeshToValueKey = Context.MeshToValueKey;
	bool bHasInvalidRotations = false;

	// Assign points to entries
	const FConstPCGPointValueRanges InRanges(InPointData);

	while (CurrentPointIndex < InPointData->GetNumPoints())
	{
		const int32 InPointIndex = CurrentPointIndex++;

		const FTransform& InTransform = InRanges.TransformRange[InPointIndex];

		if (!InTransform.IsRotationNormalized())
		{
			bHasInvalidRotations = true;
			continue;
		}

		const int64& InMetadataEntry = InRanges.MetadataEntryRange[InPointIndex];
		const int32& InSeed = InRanges.SeedRange[InPointIndex];

		const PCGMetadataValueKey ValueKey = Attribute->GetValueKey(InMetadataEntry);

		// if no mesh list was processed for this attribute value, fallback to the default mesh list
		FPCGInstancesAndWeights* InstancesAndWeights = CategoryEntryToInstancesAndWeights.Find(ValueKey);
		if (!InstancesAndWeights)
		{
			if (DefaultValueKey != PCGDefaultValueKey)
			{
				InstancesAndWeights = CategoryEntryToInstancesAndWeights.Find(DefaultValueKey);
				check(InstancesAndWeights);
			}
			else
			{
				continue;
			}
		}

		const int TotalWeight = InstancesAndWeights->CumulativeWeights.Last();

		FRandomStream RandomSource = PCGHelpers::GetRandomStreamFromSeed(InSeed, Settings, Context.ExecutionSource.Get());
		const int RandomWeightedPick = RandomSource.RandRange(0, TotalWeight - 1);

		int RandomPick = 0;
		while(RandomPick < InstancesAndWeights->MeshInstances.Num() && InstancesAndWeights->CumulativeWeights[RandomPick] <= RandomWeightedPick)
		{
			++RandomPick;
		}

		if(RandomPick < InstancesAndWeights->MeshInstances.Num())
		{
			const bool bNeedsReverseCulling = (InTransform.GetDeterminant() < 0);
			FPCGMeshInstanceList& InstanceList = PCGMeshSelectorWeighted::GetInstanceList(InstancesAndWeights->MeshInstances[RandomPick], bUseAttributeMaterialOverrides, MaterialOverrideHelper.GetMaterialOverrides(InMetadataEntry), bNeedsReverseCulling, InPointData);
			InstanceList.Instances.Emplace(InTransform);
			InstanceList.InstancesIndices.Emplace(CurrentPointIndex - 1); // - 1 because it is already incremented.

			const TSoftObjectPtr<UStaticMesh>& Mesh = InstanceList.Descriptor.StaticMesh;

			if (OutPointData && OutAttribute)
			{
				OutRanges.SetFromValueRanges(Context.CurrentWriteIndex, InRanges, InPointIndex);

				PCGMetadataValueKey* OutValueKey = MeshToValueKey.Find(Mesh);
				if (!OutValueKey)
				{
					PCGMetadataValueKey TempValueKey = OutAttribute->AddValue(Mesh.ToSoftObjectPath().ToString());
					OutValueKey = &MeshToValueKey.Add(Mesh, TempValueKey);
				}

				check(OutValueKey);

				OutPointData->Metadata->InitializeOnSet(OutRanges.MetadataEntryRange[Context.CurrentWriteIndex]);
				OutAttribute->SetValueFromValueKey(OutRanges.MetadataEntryRange[Context.CurrentWriteIndex], *OutValueKey);

				++Context.CurrentWriteIndex;

				if (Settings->bApplyMeshBoundsToPoints)
				{
					TArray<int32>& PointIndices = Context.MeshToOutPoints.FindOrAdd(Mesh).FindOrAdd(OutPointData);
					// CurrentPointIndex - 1, because CurrentPointIndex is incremented at the beginning of the loop
					PointIndices.Emplace(CurrentPointIndex - 1);
				}
			}
		}

		// Check if we should stop here and continue in a subsequent call
		if (CurrentPointIndex - LastCheckpointIndex >= TimeSlicingCheckFrequency)
		{
			if (Context.ShouldStop())
			{
				break;
			}
			else
			{
				LastCheckpointIndex = CurrentPointIndex;
			}
		}
	}

	// @todo_pcg: we should log this only once, not once-per-call, but this requires a change in the context.
	if (bHasInvalidRotations)
	{
		PCGLog::Validation::LogInvalidRotationsSkipped();
	}

	Context.CurrentPointIndex = CurrentPointIndex;

	if (CurrentPointIndex == InPointData->GetNumPoints())
	{
		if (OutPointData)
		{
			OutPointData->SetNumPoints(Context.CurrentWriteIndex);
		}

		// Collapse to OutMeshInstances
		for (TPair<PCGMetadataValueKey, FPCGInstancesAndWeights>& Entry : CategoryEntryToInstancesAndWeights)
		{
			for (TArray<FPCGMeshInstanceList>& PickedMeshInstances : Entry.Value.MeshInstances)
			{
				for (FPCGMeshInstanceList& PickedMeshInstanceEntry : PickedMeshInstances)
				{
					OutMeshInstances.Emplace(MoveTemp(PickedMeshInstanceEntry));
				}
			}
		}

		return true;
	}
	else
	{
		return false;
	}
}

void UPCGMeshSelectorWeightedByCategory::PostLoad()
{
	Super::PostLoad();

	for (FPCGWeightedByCategoryEntryList& Entry : Entries)
	{
#if WITH_EDITOR
		Entry.ApplyDeprecation();
#endif // WITH_EDITOR

		// TODO: Remove if/when FBodyInstance is updated or replaced
		// Necessary to update the collision Response Container from the Response Array
		for (FPCGMeshSelectorWeightedEntry& WeightedEntry : Entry.WeightedMeshEntries)
		{
			WeightedEntry.Descriptor.PostLoadFixup(this);
		}
	}

#if WITH_EDITOR
	RefreshDisplayNames();
#endif // WITH_EDITOR
}

void UPCGMeshSelectorWeightedByCategory::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

#if WITH_EDITOR
	RefreshDisplayNames();
#endif // WITH_EDITOR
}

void UPCGMeshSelectorWeightedByCategory::PostEditImport()
{
	Super::PostEditImport();

#if WITH_EDITOR
	RefreshDisplayNames();
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UPCGMeshSelectorWeightedByCategory::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FPCGSoftISMComponentDescriptor, StaticMesh))
	{
		RefreshDisplayNames();
	}
}

void UPCGMeshSelectorWeightedByCategory::RefreshDisplayNames()
{
	for (FPCGWeightedByCategoryEntryList& Entry : Entries)
	{
		for (FPCGMeshSelectorWeightedEntry& WeightedEntry : Entry.WeightedMeshEntries)
		{
			WeightedEntry.DisplayName = WeightedEntry.Descriptor.StaticMesh.ToSoftObjectPath().GetAssetFName();
		}
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
