// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSelectors/PCGSkinnedMeshDescriptor.h"

#include "Helpers/PCGHelpers.h"

#include "Components/InstancedSkinnedMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSkinnedMeshDescriptor)

FPCGSoftSkinnedMeshComponentDescriptor::FPCGSoftSkinnedMeshComponentDescriptor()
: FSoftSkinnedMeshComponentDescriptor()
{
	// Override defaults from base class
	ComponentClass = UInstancedSkinnedMeshComponent::StaticClass();
	Mobility = EComponentMobility::Static;
	BodyInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	// TODO: bGenerateOverlapEvents = 0;
}

FPCGSoftSkinnedMeshComponentDescriptor::FPCGSoftSkinnedMeshComponentDescriptor(const FSkinnedMeshComponentDescriptor& Other)
	: Super(Other)
{}

void FPCGSoftSkinnedMeshComponentDescriptor::InitFrom(const UInstancedSkinnedMeshComponent* Component, bool bInitBodyInstance)
{
	ComponentTags = Component->ComponentTags;
	Super::InitFrom(Component, bInitBodyInstance);
}

void FPCGSoftSkinnedMeshComponentDescriptor::InitComponent(UInstancedSkinnedMeshComponent* ABMComponent) const
{
	ABMComponent->ComponentTags = ComponentTags;

	const TArray<FString> AdditionalTags = PCGHelpers::GetStringArrayFromCommaSeparatedList(AdditionalCommaSeparatedTags);
	for (const FString& AdditionalTag : AdditionalTags)
	{
		ABMComponent->ComponentTags.Emplace(AdditionalTag);
	}

	Super::InitComponent(ABMComponent);
}

int32 FPCGSoftSkinnedMeshComponentDescriptor::GetOrAddAnimationIndex(const FSoftAnimBankItem& BankItem)
{
	int32 BankIndex = INDEX_NONE;

	// GW-TODO: 
#if 0
	for (int32 ExistingBankIndex = 0; ExistingBankIndex < BankItems.Num(); ++ExistingBankIndex)
	{
		if (BankItems[ExistingBankIndex] == BankItem)
		{
			BankIndex = ExistingBankIndex;
			break;
		}
	}

	if (BankIndex == INDEX_NONE)
	{
		BankIndex = BankItems.Num();
		BankItems.Emplace(BankItem);
	}
#endif

	return BankIndex;
}
