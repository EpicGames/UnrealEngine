// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSelectors/PCGSkinnedMeshSelector.h"

#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGSkinnedMeshSpawner.h"
#include "Elements/PCGSkinnedMeshSpawnerContext.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "MeshSelectors/PCGSkinnedMeshSelector.h"
#include "Metadata/PCGObjectPropertyOverride.h"
#include "Metadata/PCGMetadataPartitionCommon.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Algo/AnyOf.h"
#include "Animation/AnimBank.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSkinnedMeshSelector)

#define LOCTEXT_NAMESPACE "PCGSkinnedMeshSelector"

namespace PCGSkinnedMeshSelector
{
	// Returns variation based on mesh, material overrides and reverse culling
	FPCGSkinnedMeshInstanceList& GetInstanceList(
		TArray<FPCGSkinnedMeshInstanceList>& InstanceLists,
		const FPCGSoftSkinnedMeshComponentDescriptor& TemplateDescriptor,
		TSoftObjectPtr<USkinnedAsset> Asset,
		const TArray<TSoftObjectPtr<UMaterialInterface>>& MaterialOverrides,
		bool bReverseCulling,
		const UPCGPointData* InPointData,
		const int AttributePartitionIndex = INDEX_NONE)
	{
		for (FPCGSkinnedMeshInstanceList& InstanceList : InstanceLists)
		{
			if (InstanceList.Descriptor.SkinnedAsset == Asset &&
				// TODO: InstanceList.Descriptor.bReverseCulling == bReverseCulling &&
				// TODO: InstanceList.Descriptor.OverrideMaterials == MaterialOverrides &&
				InstanceList.AttributePartitionIndex == AttributePartitionIndex)
			{
				return InstanceList;
			}
		}

		FPCGSkinnedMeshInstanceList& NewInstanceList = InstanceLists.Emplace_GetRef(TemplateDescriptor);
		NewInstanceList.Descriptor.SkinnedAsset = Asset;
		// TODO: NewInstanceList.Descriptor.OverrideMaterials = MaterialOverrides;
		// TODO: NewInstanceList.Descriptor.bReverseCulling = bReverseCulling;
		NewInstanceList.AttributePartitionIndex = AttributePartitionIndex;
		NewInstanceList.PointData = InPointData;

		return NewInstanceList;
	}
}

void UPCGSkinnedMeshSelector::PostLoad()
{
	Super::PostLoad();

	// TODO: Remove if/when FBodyInstance is updated or replaced
	// Necessary to update the collision Response Container from the Response Array
	TemplateDescriptor.PostLoadFixup(this);
}

bool UPCGSkinnedMeshSelector::SelectInstances(
	FPCGSkinnedMeshSpawnerContext& Context,
	const UPCGSkinnedMeshSpawnerSettings* Settings,
	const UPCGPointData* InPointData,
	TArray<FPCGSkinnedMeshInstanceList>& OutMeshInstances,
	UPCGPointData* OutPointData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelector::SelectInstances);

	if (!InPointData)
	{
		PCGLog::InputOutput::LogInvalidInputDataError(&Context);
		return true;
	}

	if (!InPointData->Metadata)
	{
		PCGLog::Metadata::LogInvalidMetadata(&Context);
		return true;
	}

	const FPCGAttributePropertyInputSelector MeshAttributeSelector = MeshAttribute.CopyAndFixLast(InPointData);
	const FName MeshAttributeName = MeshAttributeSelector.GetAttributeName();

	if(!InPointData->Metadata->HasAttribute(MeshAttributeName))
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, FText::Format(LOCTEXT("MeshAttributeNotInMetadata", "Mesh Attribute '{0}' is not in the metadata."), FText::FromName(MeshAttributeName)));
		return true;
	}

	const FPCGMetadataAttributeBase* MeshAttributeBase = InPointData->Metadata->GetConstAttribute(MeshAttributeName);
	check(MeshAttributeBase);

	// Validate that the "mesh" attribute is of the right type
	if (!PCG::Private::IsOfTypes<FSoftObjectPath, FString>(MeshAttributeBase->GetTypeId()))
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, FText::Format(LOCTEXT("MeshAttributeInvalidType", "Mesh Attribute '{0}' is not of valid type (must be FString or FSoftObjectPath)."), FText::FromName(MeshAttributeName)));
		return true;
	}

	FPCGMeshMaterialOverrideHelper& MaterialOverrideHelper = Context.MaterialOverrideHelper;
	if (!MaterialOverrideHelper.IsInitialized())
	{
		MaterialOverrideHelper.Initialize(Context, bUseAttributeMaterialOverrides, {} /*TemplateDescriptor.OverrideMaterials*/, MaterialOverrideAttributes, InPointData->Metadata);
	}

	if (!MaterialOverrideHelper.IsValid())
	{
		return true;
	}

	// ByAttribute takes in SoftObjectPaths per point in the metadata, so we can pass those directly into the outgoing pin if it exists
	if (Context.CurrentPointIndex == 0 && OutPointData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMeshSelector::SetupOutPointData);
		OutPointData->SetPoints(InPointData->GetPoints());
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSkinnedMeshSelector::SelectEntries);

	if (!Context.bPartitionDone)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSkinnedMeshSelector::SelectEntries::BuildingPartition);

		TArray<FPCGObjectPropertyOverrideDescription> PropertyOverrides = Settings->SkinnedMeshComponentPropertyOverrides;
		const FString SkinnedAssetPropertyString = GET_MEMBER_NAME_CHECKED(FPCGSoftSkinnedMeshComponentDescriptor, SkinnedAsset).ToString();

		// Add the skinned asset override to the list only if not already provided
		if (!Algo::AnyOf(PropertyOverrides, [&SkinnedAssetPropertyString](const FPCGObjectPropertyOverrideDescription& PropertyOverride) { return PropertyOverride.PropertyTarget == SkinnedAssetPropertyString; }))
		{
			PropertyOverrides.Emplace(MeshAttribute, SkinnedAssetPropertyString);
		}

		static FPCGSoftSkinnedMeshComponentDescriptor DummyDescriptor{};
		
		// Dummy descriptor, to initialize the overrides and do some validation
		FPCGObjectOverrides Overrides(&DummyDescriptor);
		Overrides.Initialize(PropertyOverrides, &DummyDescriptor, InPointData, &Context);

		// Validate all the selectors are actual FPCGSoftSkinnedMeshComponentDescriptor properties
		TArray<FPCGAttributePropertySelector> ValidSelectorOverrides = Overrides.GetAllInputSelectors();

		// If there are valid overrides, partition the points on those attributes so that an instance can be created for each
		if (!ValidSelectorOverrides.IsEmpty())
		{
			Context.AttributeOverridePartition = PCGMetadataPartitionCommon::AttributeGenericPartition(InPointData, ValidSelectorOverrides, &Context, Settings->bSilenceOverrideAttributeNotFoundErrors);
		}

		// Set the descriptors to match the partition count. Uninitialized, because it will be copied from the template below
		Context.OverriddenDescriptors.Reserve(Context.AttributeOverridePartition.Num());
		for (int I = 0; I < Context.AttributeOverridePartition.Num(); ++I)
		{
			FPCGSoftSkinnedMeshComponentDescriptor& Descriptor = Context.OverriddenDescriptors.Add_GetRef(TemplateDescriptor);

			// If partition is empty (which can happen, esp. for the default partition on the default value, we'll just skip it here.
			if (Context.AttributeOverridePartition[I].IsEmpty())
			{
				continue;
			}

			// Use the Object Override to map the user's input selector and property to the descriptor
			Overrides.UpdateTemplateObject(&Descriptor);

			// Since they are already partitioned and identical, we can just use the value on the first point
			check(!Context.AttributeOverridePartition[I].IsEmpty());
			int32 AnyPointIndexOnThisPartition = Context.AttributeOverridePartition[I][0];
			Overrides.Apply(AnyPointIndexOnThisPartition);
		}

		Context.PointProviderValues.Reset();
		Context.PointAnimationValues.Reset();

		// Given partitioning is expensive, check if we're out of time for this frame
		Context.bPartitionDone = true;
		if (Context.ShouldStop())
		{
			return false;
		}
	}

	// Assign points to entries
	int32 CurrentPartitionIndex = Context.CurrentPointIndex; // misnomer but we're reusing another concept from the context
	const TArray<FPCGPoint>& Points = InPointData->GetPoints();

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSkinnedMeshSelector::SelectEntries::PushingPointsToInstanceLists);

		if (Context.PointProviderValues.IsEmpty() || Context.PointAnimationValues.IsEmpty())
		{
			// GW-TODO: 
			// PCGAttributeAccessorHelpers::ExtractAllValues(InPointData, BankAttribute, Context.PointProviderValues, &Context);
			// PCGAttributeAccessorHelpers::ExtractAllValues(InPointData, SequenceIndexAttribute, Context.PointAnimationValues, &Context);
		}

		// TODO: Revisit this when attribute partitioning is returned in a more optimized form
		// The partition index is used to assign the point to the correct partition's instance
		while (CurrentPartitionIndex < Context.AttributeOverridePartition.Num())
		{
			const int32 PartitionIndex = CurrentPartitionIndex++;
			TArray<int32>& Partition = Context.AttributeOverridePartition[PartitionIndex];
			const FPCGSoftSkinnedMeshComponentDescriptor& CurrentPartitionDescriptor = Context.OverriddenDescriptors[PartitionIndex];
			
			if (Partition.IsEmpty() || CurrentPartitionDescriptor.SkinnedAsset.IsNull())
			{
				continue;
			}

			// Setup data for mesh bounds computation
			if (OutPointData && Settings->bApplyMeshBoundsToPoints)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSkinnedMeshSelector::SelectEntries::PushPointsToApplyMeshBounds);
				TArray<int32>& PointIndices = Context.MeshToOutPoints.FindOrAdd(CurrentPartitionDescriptor.SkinnedAsset).FindOrAdd(OutPointData);
				PointIndices.Append(Partition);
			}

			// Separate the inverse determinant instances so we can push them to a different ABM
			TArray<int32> ReverseInstances;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSkinnedMeshSelector::SelectEntries::BuildReverseInstances);
				int32 WriteIndex = 0;
				for (int32 ReadIndex = 0; ReadIndex < Partition.Num(); ++ReadIndex)
				{
					const FPCGPoint& Point = Points[Partition[ReadIndex]];
					if (Point.Transform.GetDeterminant() < 0)
					{
						ReverseInstances.Add(Partition[ReadIndex]);
					}
					else
					{
						if (WriteIndex != ReadIndex)
						{
							Partition[WriteIndex] = Partition[ReadIndex];
						}
						++WriteIndex;
					}
				}

				Partition.SetNum(WriteIndex);
			}

			bool bHasInvalidRotations = false;

			auto AddPointsToInstanceList = [&OutMeshInstances, &CurrentPartitionDescriptor, &Context, &MaterialOverrideHelper, &Points, &bHasInvalidRotations, PartitionIndex, InPointData](const TArray<int32>& PointIndices, bool bReverseTransform)
			{
				if (MaterialOverrideHelper.OverridesMaterials())
				{
					for (int32 PointIndex : PointIndices)
					{
						const FPCGPoint& Point = Points[PointIndex];

						if (!Point.Transform.IsRotationNormalized())
						{
							bHasInvalidRotations = true;
							continue;
						}

						FPCGSkinnedMeshInstanceList& InstanceList = PCGSkinnedMeshSelector::GetInstanceList(
							OutMeshInstances,
							CurrentPartitionDescriptor,
							CurrentPartitionDescriptor.SkinnedAsset,
							MaterialOverrideHelper.GetMaterialOverrides(Point.MetadataEntry),
							bReverseTransform,
							InPointData,
							PartitionIndex
						);

						// GW-TODO: 
						//FSoftAnimBankItem BankItem;
						//{
						//	BankItem.BankAsset = Context.PointBankValues[PointIndex];
						//	BankItem.SequenceIndex = Context.PointSequenceIndexValues[PointIndex];
						//}

						FPCGSkinnedMeshInstance Instance;
						Instance.AnimationIndex = INDEX_NONE;// InstanceList.Descriptor.GetOrAddAnimationIndex(BankItem);
						Instance.Transform = Point.Transform;

						InstanceList.Instances.Emplace(Instance);
						InstanceList.InstancePointIndices.Emplace(PointIndex);
					}
				}
				else
				{
					TArray<TSoftObjectPtr<UMaterialInterface>> DummyMaterialList;
					FPCGSkinnedMeshInstanceList& InstanceList = PCGSkinnedMeshSelector::GetInstanceList(
						OutMeshInstances,
						CurrentPartitionDescriptor,
						CurrentPartitionDescriptor.SkinnedAsset,
						DummyMaterialList,
						bReverseTransform,
						InPointData,
						PartitionIndex
					);

					check(InstanceList.Instances.Num() == InstanceList.InstancePointIndices.Num());
					const int32 InstanceOffset = InstanceList.Instances.Num();
					InstanceList.Instances.Reserve(InstanceOffset + PointIndices.Num());
					InstanceList.InstancePointIndices.Reserve(InstanceOffset + PointIndices.Num());

					for (int32 PointIndex : PointIndices)
					{
						const FPCGPoint& Point = Points[PointIndex];

						if (!Point.Transform.IsRotationNormalized())
						{
							bHasInvalidRotations = true;
							continue;
						}

						FSoftAnimBankItem BankItem;
						// GW-TODO: 
						//{
							//BankItem.BankAsset = Context.PointProviderValues[PointIndex];
							//BankItem.SequenceIndex = Context.PointAnimationValues[PointIndex];
						//}
						
						FPCGSkinnedMeshInstance Instance;
						Instance.AnimationIndex = InstanceList.Descriptor.GetOrAddAnimationIndex(BankItem);
						Instance.Transform = Point.Transform;

						InstanceList.Instances.Emplace(Instance);
						InstanceList.InstancePointIndices.Emplace(PointIndex);
					}
				}
			};

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSkinnedMeshSelector::SelectEntries::AddPointsToInstanceList);
				AddPointsToInstanceList(Partition, false);
				AddPointsToInstanceList(ReverseInstances, true);

				// @todo_pcg: we should log this only once, not once-per-call, but this requires a change in the context.
				if (bHasInvalidRotations)
				{
					PCGLog::Validation::LogInvalidRotationsSkipped();
				}
			}

			if (Context.ShouldStop())
			{
				break;
			}
		}
	}

	Context.CurrentPointIndex = CurrentPartitionIndex; // misnomer, but we're using the same context
	return CurrentPartitionIndex == Context.AttributeOverridePartition.Num();
}

#undef LOCTEXT_NAMESPACE
