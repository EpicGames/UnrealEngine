// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/Accessors/IPCGAttributeAccessor.h"

#include "Utils/PCGLogErrors.h"

#define LOCTEXT_NAMESPACE "PCGAttributeAccessor"

namespace PCGAttributeAccessorPrivate
{
	PCG::Private::FTransformFunc GetTransformFunc(const FPCGMetadataAttributeDesc& ReadDesc, const FPCGMetadataAttributeDesc& WriteDesc, PCG::Private::EPCGAttributeAccessorOperation OpType)
	{
		const uint16 ReadValueType = static_cast<uint16>(ReadDesc.ValueType);
		const uint16 WriteValueType = static_cast<uint16>(WriteDesc.ValueType);
		
		check(OpType == PCG::Private::EPCGAttributeAccessorOperation::Broadcast || OpType == PCG::Private::EPCGAttributeAccessorOperation::Construct);
		return OpType == PCG::Private::EPCGAttributeAccessorOperation::Broadcast
			? PCG::Private::GetBroadcastTransformFunc(ReadValueType, WriteValueType)
			: PCG::Private::GetConstructibleTransformFunc(WriteValueType, ReadValueType);
	}
}

bool IPCGAttributeAccessor::CopyTo(const IPCGAttributeAccessorKeys& SrcKeys, IPCGAttributeAccessor& DestAccessor, IPCGAttributeAccessorKeys& DestKeys, int32 InputIndex, int32 OutputIndex, int32 Count, EPCGAttributeAccessorFlags Flags) const
{
	if (Count <= 0)
	{
		// Nothing to do
		return true;
	}
	
	// Can't copy to read only accessors
	if (DestAccessor.IsReadOnly())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("ReadOnlyDest", "[PCGAttributeAccessor::CopyTo] Destination accessor is read only."));
		return false;
	}
	
	// Or outside range
	if (OutputIndex >= DestKeys.GetNum())
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("OutsideRange", "[PCGAttributeAccessor::CopyTo] Try to write outside of destination accessor range. (Index = {0}, MaxIndex = {1})."), OutputIndex, DestKeys.GetNum() - 1));
		return false;
	}
	
	// Can only copy matching containers
	const FPCGMetadataAttributeDesc& OriginalReadDesc = GetUnderlyingDesc();
	FPCGMetadataAttributeDesc ReadDesc = OriginalReadDesc;
	const FPCGMetadataAttributeDesc& WriteDesc = DestAccessor.GetUnderlyingDesc();
	PCG::Private::EPCGAttributeAccessorOperation OpType = PCG::Private::GetOpType(ReadDesc, WriteDesc, Flags);
	bool bSingleValueToArray = false;
	
	// Check the special case where we can convert multi values into a single container
	// @todo_pcg Support sets
	if (OpType == PCG::Private::EPCGAttributeAccessorOperation::Invalid && !!(Flags & EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer) && OutputIndex == 0 && DestKeys.GetNum() == 1 && ReadDesc.IsSingleValue() && (WriteDesc.IsArray()))
	{
		ReadDesc.ContainerTypes = {EPCGMetadataAttributeContainerTypes::Array};
		OpType = PCG::Private::GetOpType(ReadDesc, WriteDesc, Flags);
		bSingleValueToArray = true;
	}

	if (ReadDesc.ContainerTypes != WriteDesc.ContainerTypes)
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MismatchContainer", "[PCGAttributeAccessor::CopyTo] Mismatch between containers {0} vs {1}"), FText::FromString(OriginalReadDesc.GetTypeString()), FText::FromString(WriteDesc.GetTypeString())));
		return false;
	}

	const bool bIsSingleValue = ReadDesc.IsSingleValue();
	const bool bIsArray = ReadDesc.IsArray();
	if (OpType == PCG::Private::EPCGAttributeAccessorOperation::Invalid || (!bIsSingleValue && !bIsArray && OpType != PCG::Private::EPCGAttributeAccessorOperation::SameType))
	{
		PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ImpossibleConversion", "[PCGAttributeAccessor::CopyTo] Can't convert {0} to {1}"), FText::FromString(OriginalReadDesc.GetTypeString()), FText::FromString(WriteDesc.GetTypeString())));
		return false;
	}

	if (bIsSingleValue)
	{
		// First of all, if we are complex types, and we have no conversion, try by pointer
		const bool bIsComplexType = ReadDesc.ContainsObject() || ReadDesc.ValueType == EPCGMetadataTypes::Struct 
			|| ReadDesc.ValueType == EPCGMetadataTypes::String || ReadDesc.ValueType == EPCGMetadataTypes::SoftObjectPath 
			|| ReadDesc.ValueType == EPCGMetadataTypes::SoftClassPath;
		
		if (bIsComplexType && OpType == PCG::Private::EPCGAttributeAccessorOperation::SameType)
		{
			TArray<const void*> TempValues;
			TempValues.SetNumUninitialized(Count);
			
			PCG::Private::FOutValues OutValues{TInPlaceType<PCG::Private::FOutValuesByPtr>{}, TempValues, nullptr, &ReadDesc};
			PCG::Private::FInValues InValues{TInPlaceType<PCG::Private::FInValuesByPtr>{}, TempValues, &WriteDesc};
			
			// Only try by pointer if we support getting/setting by pointer.
			if (SupportsGet(OutValues) && DestAccessor.SupportsSet(InValues))
			{
				return GetRangeVirtual(MoveTemp(OutValues), Count, InputIndex, SrcKeys)
					&& DestAccessor.SetRangeVirtual(MoveTemp(InValues), Count, OutputIndex, DestKeys, Flags);
			}
		}
		
		// Otherwise we'll need buffers
		PCG::FPCGAccessorBuffer ReadBuffer;
		ReadBuffer.SetupAndAllocate(Count, ReadDesc);
		if (!ensure(ReadBuffer.Num() == Count))
		{
			return false;
		}
		
		if (!GetRangeVirtual(PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, ReadBuffer.GetOwnedMemoryPtr(), Count, &ReadDesc}, Count, InputIndex, SrcKeys))
		{
			return false;
		}
		
		// If we are the same type, we can write directly into the dest accessor, otherwise we need a second buffer to apply the transform to
		PCG::FPCGAccessorBuffer WriteBuffer;
		PCG::FPCGAccessorBuffer* BufferToWrite = &ReadBuffer;
		if (OpType != PCG::Private::EPCGAttributeAccessorOperation::SameType)
		{
			WriteBuffer.SetupAndAllocate(Count, WriteDesc);
			if (!ensure(WriteBuffer.Num() == Count))
			{
				return false;
			}
			
			PCG::Private::FTransformFunc TransformFunc = PCGAttributeAccessorPrivate::GetTransformFunc(ReadDesc, WriteDesc, OpType);
			check(TransformFunc);
			(*TransformFunc)(WriteBuffer.GetOwnedMemoryPtr(), ReadBuffer.GetOwnedMemoryPtr(), Count);
			
			BufferToWrite = &WriteBuffer;
		}

		check(BufferToWrite);
		return DestAccessor.SetRangeVirtual(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, BufferToWrite->GetOwnedMemoryPtr(), Count, &WriteDesc}, Count, OutputIndex, DestKeys, Flags);
	}
	else if (bIsArray)
	{
		check(WriteDesc.IsArray());

		TArray<TTuple<const void*, int32>, TInlineAllocator<256>> Temp;
		Temp.SetNumUninitialized(bSingleValueToArray ? 1 : Count);
		
		PCG::FPCGAccessorBuffer SingleValuesBuffer{};
		
		bool bSuccess = false;
		if (bSingleValueToArray)
		{
			ReadDesc.ContainerTypes.Empty();
			SingleValuesBuffer.SetupAndAllocate(Count, ReadDesc);
			bSuccess = GetRangeVirtual(PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, SingleValuesBuffer.GetOwnedMemoryPtr(), Count}, Count, InputIndex, SrcKeys);
			Temp[0] = MakeTuple(SingleValuesBuffer.GetOwnedMemoryPtr(), Count);
		}
		else
		{
			bSuccess = GetRangeVirtual(PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, Temp}, Count, InputIndex, SrcKeys);
		}
		
		if (!bSuccess)
		{
			return false;
		}

		if (OpType == PCG::Private::EPCGAttributeAccessorOperation::SameType)
		{
			return DestAccessor.SetRangeVirtual(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, Temp}, Temp.Num(), OutputIndex, DestKeys, Flags);
		}
		else
		{
			const int32 WriteCount = Temp.Num();

			// We need buffers for conversion here
			TArray<PCG::FPCGAccessorBuffer> Buffers;
			Buffers.SetNum(WriteCount);
			
			FPCGMetadataAttributeDesc SingleWriteDesc{};
			SingleWriteDesc.ValueType = WriteDesc.ValueType;
			SingleWriteDesc.ValueTypeObject = WriteDesc.ValueTypeObject;
			
			PCG::Private::FTransformFunc TransformFunc = PCGAttributeAccessorPrivate::GetTransformFunc(ReadDesc, WriteDesc, OpType);
			check(TransformFunc);

			for (auto [Buffer, TempValue] : UE::Zip(Buffers, Temp))
			{
				auto& [DataPtr, ArrayNum] = TempValue;
				
				Buffer.SetupAndAllocate(ArrayNum, SingleWriteDesc);

				TransformFunc(Buffer.GetOwnedMemoryPtr(), DataPtr, ArrayNum);
				DataPtr = Buffer.GetOwnedMemoryPtr();
			}

			return DestAccessor.SetRangeVirtual(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, Temp}, WriteCount, OutputIndex, DestKeys, Flags);
		}
	}

	// @todo_pcg: Set and maps

	return false;
}

bool IPCGAttributeAccessor::GetRange(PCG::Private::FOutValues OutValues, const FPCGMetadataAttributeDesc& WriteDesc, int32 Count, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) const
{
	using namespace PCG::Private;
	
	if (Count <= 0)
	{
		// Nothing to do
		return true;
	}
	
	EPCGAttributeAccessorOperation OpType = PCG::Private::GetOpType(UnderlyingDesc, WriteDesc, Flags);
		
	if (OpType == EPCGAttributeAccessorOperation::Invalid)
	{
		return false;
	}
	
	if (OpType == EPCGAttributeAccessorOperation::SameType)
	{
		return GetRangeVirtual(MoveTemp(OutValues), Count, Index, Keys);
	}

	PCG::Private::FTransformFunc TransformFunc = PCGAttributeAccessorPrivate::GetTransformFunc(UnderlyingDesc, WriteDesc, OpType);
	check(TransformFunc);
	
	// Handle the conversion for each type of input.
	return Visit([this, Count, Index, &Keys, &WriteDesc, OpType, TransformFunc](auto&& OutValue) -> bool
	{
		using T = std::decay_t<decltype(OutValue)>;
		if constexpr (std::is_same_v<T, FOutValuesByValue>)
		{
			if (!WriteDesc.IsSingleValue())
			{
				return false;
			}

			check(IsPCGType(static_cast<uint16>(UnderlyingDesc.ValueType)));

			PCG::FPCGAccessorBuffer ReadBuffer;
			ReadBuffer.SetupAndAllocate(Count, UnderlyingDesc);
			if (!ensure(ReadBuffer.Num() == Count))
			{
				return false;
			}
			
			if (!GetRangeVirtual(FOutValues{TInPlaceType<FOutValuesByValue>{}, ReadBuffer.GetOwnedMemoryPtr(), Count, &UnderlyingDesc}, Count, Index, Keys))
			{
				return false;
			}
			
			(*TransformFunc)(OutValue.OutValues, ReadBuffer.GetOwnedMemoryPtr(), Count);
			return true;
		}
		else if constexpr (std::is_same_v<T, FOutValuesAsArray>)
		{
			// Buffers MUST be present.
			if (!WriteDesc.IsArray() || OutValue.Buffers.Num() != OutValue.OutValues.Num())
			{
				return false;
			}
			
			if (!GetRangeVirtual(FOutValues{TInPlaceType<FOutValuesAsArray>{}, OutValue.OutValues}, Count, Index, Keys))
			{
				return false;
			}
			
			const FPCGMetadataAttributeDesc SingleWriteDesc = WriteDesc.ConvertToSingleValue();
			
			for (auto [PullValues, Buffer] : UE::Zip(OutValue.OutValues, OutValue.Buffers))
			{
				auto& [DataPtr, ArrayNum] = PullValues;
				Buffer.SetupAndAllocate(ArrayNum, SingleWriteDesc);
				check(Buffer.IsOwningMemory() && Buffer.Num() == ArrayNum);
				
				(*TransformFunc)(Buffer.GetOwnedMemoryPtr(), DataPtr, ArrayNum);
				DataPtr = Buffer.GetOwnedMemoryPtr();
			}
			
			return true;
		}
		else if constexpr (std::is_same_v<T, FOutValuesAsSet> || std::is_same_v<T, FOutValuesAsMap> || std::is_same_v<T, FOutValuesByPtr>)
		{
			// Not supported
			return false;
		}
		else
		{
			static_assert(!std::is_same_v<T, T>, "Missing case in Visit");
			return false;
		}
	}, MoveTemp(OutValues));
}

bool IPCGAttributeAccessor::SetRange(PCG::Private::FInValues InValues, const FPCGMetadataAttributeDesc& ReadDesc, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
{
	using namespace PCG::Private;
	
	if (Count <= 0)
	{
		// Nothing to do
		return true;
	}

	const bool bSupportsConversion = InValues.IsType<FInValuesByValue>() || InValues.IsType<FInValuesAsArray>();
	EPCGAttributeAccessorOperation OpType = GetOpType(ReadDesc, UnderlyingDesc, Flags);

	if (OpType == PCG::Private::EPCGAttributeAccessorOperation::Invalid)
	{
		// Check the special case where we can convert multi values into a single container
		if (!!(Flags & EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer) && Index == 0 && Keys.GetNum() == 1 && ReadDesc.IsSingleValue() && (UnderlyingDesc.IsArray() || UnderlyingDesc.IsSet()) && (InValues.IsType<FInValuesByValue>() || InValues.IsType<FInValuesByPtr>()))
		{
			FPCGMetadataAttributeDesc ReadDescAsContainer = ReadDesc;
			ReadDescAsContainer.ContainerTypes = UnderlyingDesc.ContainerTypes;
			OpType = PCG::Private::GetOpType(ReadDescAsContainer, UnderlyingDesc, Flags);

			// If we have values by pointer, we unfortunately have to convert them for arrays (but can use as-is for sets)
			// Supports conversion for arrays
			if (OpType != PCG::Private::EPCGAttributeAccessorOperation::Invalid && UnderlyingDesc.IsArray())
			{
				PCG::FPCGAccessorBuffer ReadBuffer;
				TTuple<const void*, int32> InValuesAsArray;
				
				if (const FInValuesByPtr* InValuesByPtr = InValues.TryGet<FInValuesByPtr>())
				{
					if (!ReadBuffer.CopyFromPointers(InValuesByPtr->InValues, ReadDesc))
					{
						return false;
					}

					InValuesAsArray = {ReadBuffer.GetOwnedMemoryPtr(), Count};
				}
				else
				{
					const FInValuesByValue& InValuesByValue = InValues.Get<FInValuesByValue>();
					InValuesAsArray = {InValuesByValue.InValues, InValuesByValue.Count};
				}
				
				return SetRange_Internal(
					PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, MakeArrayView(&InValuesAsArray, 1), &ReadDescAsContainer},
					ReadDescAsContainer,
					1,
					Index,
					Keys,
					Flags,
					OpType
				);
			}
			// @todo_pcg: Add support for conversion for sets
			else if (OpType == PCG::Private::EPCGAttributeAccessorOperation::SameType && UnderlyingDesc.IsSet())
			{
				TArray<const void*> SetValues;
				
				if (const FInValuesByPtr* InValuesByPtr = InValues.TryGet<FInValuesByPtr>())
				{
					SetValues = InValuesByPtr->InValues;
				}
				else
				{
					const FInValuesByValue& InValuesByValue = InValues.Get<FInValuesByValue>();
					SetValues.Reserve(InValuesByValue.Count);
					
					// Use the known element size if we know it, otherwise get it as a property
					int32 ElementSize = GetElementSize(static_cast<uint16>(ReadDesc.ValueType));
					if (ElementSize <= 0)
					{
						FPCGAttributeProperty AttributeProperty(ReadDesc);
						ElementSize = AttributeProperty.IsValid() ? AttributeProperty.GetProperty()->Inner->GetElementSize() : -1;
					}
					
					if (ElementSize <= 0)
					{
						return false;
					}

					for (int32 i = 0; i < InValuesByValue.Count; i++)
					{
						SetValues.Add(static_cast<const uint8*>(InValuesByValue.InValues) + i * ElementSize);
					}
				}
				
				return SetRange_Internal(
					PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsSet>{}, MakeArrayView(&SetValues, 1), &ReadDescAsContainer},
					ReadDescAsContainer,
					1,
					Index,
					Keys,
					Flags,
					OpType
				);
			}

			return false;
		}
	}

	if (OpType == EPCGAttributeAccessorOperation::Invalid || (!bSupportsConversion && OpType != EPCGAttributeAccessorOperation::SameType))
	{
		return false;
	}

	return SetRange_Internal(MoveTemp(InValues), ReadDesc, Count, Index, Keys, Flags, OpType);
}

bool IPCGAttributeAccessor::SupportsGet(const PCG::Private::FOutValues& OutValues) const
{
	using namespace PCG::Private;
	
	if (!UnderlyingDesc.IsValid())
	{
		return false;
	}

	return Visit([this]<typename T>(const T&) -> bool
	{
		if constexpr (std::is_same_v<T, FOutValuesByPtr>)
		{
			return false;
		}
		else if constexpr (std::is_same_v<T, FOutValuesByValue>)
		{
			return UnderlyingDesc.IsSingleValue();
		}
		else if constexpr (std::is_same_v<T, FOutValuesAsArray>)
		{
			return UnderlyingDesc.IsArray();
		}
		else if constexpr (std::is_same_v<T, FOutValuesAsSet>)
		{
			return UnderlyingDesc.IsSet();
		}
		else if constexpr (std::is_same_v<T, FOutValuesAsMap>)
		{
			return UnderlyingDesc.IsMap();
		}
		else
		{
			static_assert(!std::is_same_v<T, T>, "Missing variant case");
			return false;
		}
	}, OutValues);
}

bool IPCGAttributeAccessor::SupportsSet(const PCG::Private::FInValues& InValues) const
{
	using namespace PCG::Private;
	
	if (!UnderlyingDesc.IsValid())
	{
		return false;
	}

	return Visit([this]<typename T>(const T&) -> bool
	{
		if constexpr (std::is_same_v<T, FInValuesByPtr> || std::is_same_v<T, FInValuesSubset>)
		{
			return false;
		}
		else if constexpr (std::is_same_v<T, FInValuesByValue>)
		{
			return UnderlyingDesc.IsSingleValue();
		}
		else if constexpr (std::is_same_v<T, FInValuesAsArray>)
		{
			return UnderlyingDesc.IsArray();
		}
		else if constexpr (std::is_same_v<T, FInValuesAsSet>)
		{
			return UnderlyingDesc.IsSet();
		}
		else if constexpr (std::is_same_v<T, FInValuesAsMap>)
		{
			return UnderlyingDesc.IsMap();
		}
		else
		{
			static_assert(!std::is_same_v<T, T>, "Missing variant case");
			return false;
		}
	}, InValues);
}

bool IPCGAttributeAccessor::GetRange_Internal(PCG::Private::FOutValues OutValues, const FPCGMetadataAttributeDesc& WriteDesc, int32 Count, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags, PCG::Private::EPCGAttributeAccessorOperation OpType) const
{
	using namespace PCG::Private;

	switch (OpType)
	{
	case EPCGAttributeAccessorOperation::SameType:
	{
		return GetRangeVirtual(MoveTemp(OutValues), Count, Index, Keys);
	}
	case EPCGAttributeAccessorOperation::Construct: // Fall-through
	case EPCGAttributeAccessorOperation::Broadcast:
	{
		check(OutValues.IsType<FOutValuesByValue>() && IsPCGType(static_cast<uint16>(UnderlyingDesc.ValueType)));
		
		PCG::Private::FTransformFunc TransformFunc = PCGAttributeAccessorPrivate::GetTransformFunc(UnderlyingDesc, WriteDesc, OpType);
		check(TransformFunc);
			
		PCG::FPCGAccessorBuffer ReadBuffer;
		ReadBuffer.SetupAndAllocate(Count, UnderlyingDesc);
		if (!ensure(ReadBuffer.Num() == Count))
		{
			return false;
		}
		
		if (!GetRangeVirtual(FOutValues{TInPlaceType<FOutValuesByValue>{}, ReadBuffer.GetOwnedMemoryPtr(), Count, &UnderlyingDesc}, Count, Index, Keys))
		{
			return false;
		}
		
		(*TransformFunc)(OutValues.Get<FOutValuesByValue>().OutValues, ReadBuffer.GetOwnedMemoryPtr(), Count);
		return true;
	}
	default:
		checkNoEntry();
		return false;
	}
}

bool IPCGAttributeAccessor::SetRange_Internal(PCG::Private::FInValues InValues, const FPCGMetadataAttributeDesc& ReadDesc, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags, PCG::Private::EPCGAttributeAccessorOperation OpType)
{
	using namespace PCG::Private;

	switch (OpType)
	{
	case EPCGAttributeAccessorOperation::SameType:
	{
		return SetRangeVirtual(MoveTemp(InValues), Count, Index, Keys, Flags);
	}
	case EPCGAttributeAccessorOperation::Construct: // Fall-through
	case EPCGAttributeAccessorOperation::Broadcast:
	{
		check((InValues.IsType<FInValuesByValue>() || InValues.IsType<FInValuesAsArray>()) && IsPCGType(static_cast<uint16>(UnderlyingDesc.ValueType)));

		if (InValues.IsType<FInValuesByValue>())
		{
			PCG::FPCGAccessorBuffer WriteBuffer;
			WriteBuffer.SetupAndAllocate(Count, UnderlyingDesc);
			if (!ensure(WriteBuffer.Num() == Count))
			{
				return false;
			}
		
			PCG::Private::FTransformFunc TransformFunc = PCGAttributeAccessorPrivate::GetTransformFunc(ReadDesc, UnderlyingDesc, OpType);
			check(TransformFunc);
			(*TransformFunc)(WriteBuffer.GetOwnedMemoryPtr(), InValues.Get<FInValuesByValue>().InValues, Count);

			return SetRangeVirtual(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, WriteBuffer.GetOwnedMemoryPtr(), Count, &UnderlyingDesc}, Count, Index, Keys, Flags);
		}
		else // if (InValues.IsType<FInValuesAsArray>())
		{
			FPCGMetadataAttributeDesc SingleWriteDesc{};
			SingleWriteDesc.ValueType = UnderlyingDesc.ValueType;
			SingleWriteDesc.ValueTypeObject = UnderlyingDesc.ValueTypeObject;

			FTransformFunc TransformFunc = PCGAttributeAccessorPrivate::GetTransformFunc(ReadDesc, UnderlyingDesc, OpType);

			check(TransformFunc);

			FInValuesAsArray& InValuesAsArray = InValues.Get<FInValuesAsArray>();
			check(InValuesAsArray.InValues.Num() == Count);
			TArray<PCG::FPCGAccessorBuffer> Buffers;
			Buffers.Reserve(Count);

			for (auto& [DataPtr, ArrayNum] : InValuesAsArray.InValues)
			{
				if (ArrayNum == 0)
				{
					continue;
				}

				PCG::FPCGAccessorBuffer& Buffer = Buffers.Emplace_GetRef();
				Buffer.SetupAndAllocate(ArrayNum, SingleWriteDesc);
				check(Buffer.GetOwnedMemoryPtr() != nullptr);
				TransformFunc(Buffer.GetOwnedMemoryPtr(), DataPtr, ArrayNum);
				// The DataPtr in the values now point to the own memory
				DataPtr = Buffer.GetOwnedMemoryPtr();
			}

			return SetRangeVirtual(MoveTemp(InValues), Count, Index, Keys, Flags);
		}
	}
	default:
		checkNoEntry();
		return false;
	}
}

#undef LOCTEXT_NAMESPACE