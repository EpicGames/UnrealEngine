// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/Accessors/PCGAttributeAccessorFactory.h"

#include "PCGModule.h"
#include "Data/PCGSpatialData.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

namespace PCGAttributeAccessorFactory
{
	TUniquePtr<IPCGAttributeAccessor> CreateDefaultAccessor(UPCGData* InData, const FPCGAttributePropertySelector& InSelector, const bool bQuiet)
	{
		check(InData);
		TUniquePtr<IPCGAttributeAccessor> Accessor;
	
		if (InSelector.GetSelection() == EPCGAttributePropertySelection::ExtraProperty)
		{
			Accessor = PCGAttributeAccessorHelpers::CreateExtraAccessor(InSelector.GetExtraProperty());

			if (!Accessor.IsValid())
			{
				if (!bQuiet)
				{
					UE_LOGF(LogPCG, Error, "[FPCGAttributeAccessorFactory::DefaultAccessor] Expected to select an extra property '%ls' but the data doesn't support it or is only constant.", *InSelector.GetName().ToString());
				}
				
				return nullptr;
			}
		}
		else if (InSelector.GetSelection() == EPCGAttributePropertySelection::Attribute)
		{
			UPCGMetadata* Metadata = InData->MutableMetadata();
			if (!Metadata)
			{
				PCGLog::Metadata::LogInvalidMetadata();
				return nullptr;
			}
			
			const FPCGMetadataDomainID DomainID = InData->GetMetadataDomainIDFromSelector(InSelector);
			if (!DomainID.IsValid())
			{
				PCGLog::Metadata::LogInvalidMetadataDomain(InSelector);
				return nullptr;
			}

			FPCGMetadataDomain* MetadataDomain = Metadata->GetMetadataDomain(DomainID);
			FPCGMetadataAttributeBase* Attribute = MetadataDomain ? MetadataDomain->GetMutableAttribute(InSelector.GetAttributeName()) : nullptr;
		
			Accessor = PCGAttributeAccessorHelpers::CreateAccessor(Attribute, MetadataDomain, bQuiet);
		}

		return Accessor;
	}
	
	TUniquePtr<const IPCGAttributeAccessor> CreateDefaultConstAccessor(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, const bool bQuiet)
	{
		check(InData);
		TUniquePtr<const IPCGAttributeAccessor> Accessor;
	
		if (InSelector.GetSelection() == EPCGAttributePropertySelection::ExtraProperty)
		{
			if (InSelector.GetExtraProperty() == EPCGExtraProperties::NumElements)
			{
				if (InSelector.GetDomainName() != PCGDataConstants::DataDomainName || !InData->IsSupportedMetadataDomainID(PCGMetadataDomainID::Elements))
				{
					if (!bQuiet)
					{
						UE_LOGF(LogPCG, Error, "[FPCGAttributeAccessorFactory::DefaultConstAccessor] Num elements is only supported on @Data domain and for data that have elements.");
					}
					
					return nullptr;
				}
				
				return MakeUnique<FPCGConstantValueAccessor<int32>>(PCGHelpers::GetNumberOfElements(InData));
			}
			
			Accessor = PCGAttributeAccessorHelpers::CreateExtraAccessor(InSelector.GetExtraProperty());

			if (!Accessor.IsValid())
			{
				if (!bQuiet)
				{
					UE_LOGF(LogPCG, Error, "[FPCGAttributeAccessorFactory::DefaultAccessor] Expected to select an extra property '%ls' but the data doesn't support it.", *InSelector.GetName().ToString());
				}
				
				return nullptr;
			}
		}
		else if (InSelector.GetSelection() == EPCGAttributePropertySelection::Attribute)
		{
			const UPCGMetadata* Metadata = InData->ConstMetadata();
			if (!Metadata)
			{
				PCGLog::Metadata::LogInvalidMetadata();
				return nullptr;
			}
			
			const FPCGMetadataDomainID DomainID = InData->GetMetadataDomainIDFromSelector(InSelector);
			if (!DomainID.IsValid())
			{
				PCGLog::Metadata::LogInvalidMetadataDomain(InSelector);
				return nullptr;
			}

			const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomain(DomainID);
			const FPCGMetadataAttributeBase* Attribute = MetadataDomain ? MetadataDomain->GetConstAttribute(InSelector.GetAttributeName()) : nullptr;
		
			Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Attribute, MetadataDomain, bQuiet);
		}

		return Accessor;
	}
	
	TUniquePtr<IPCGAttributeAccessorKeys> CreateDefaultAccessorKeys(UPCGData* InData, const FPCGAttributePropertySelector& InSelector, const bool bQuiet)
	{
		UPCGMetadata* Metadata = InData->MutableMetadata();
		if (!Metadata)
		{
			PCGLog::Metadata::LogInvalidMetadata();
			return nullptr;
		}
		
		const FPCGMetadataDomainID DomainID = InData->GetMetadataDomainIDFromSelector(InSelector);
		if (!DomainID.IsValid())
		{
			PCGLog::Metadata::LogInvalidMetadataDomain(InSelector);
			return nullptr;
		}

		FPCGMetadataDomain* MetadataDomain = Metadata->GetMetadataDomain(DomainID);
		return MetadataDomain ? MakeUnique<FPCGAttributeAccessorKeysEntries>(MetadataDomain) : nullptr;
	}
	
	TUniquePtr<const IPCGAttributeAccessorKeys> CreateDefaultConstAccessorKeys(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, const bool bQuiet)
	{
		if (InSelector.GetSelection() == EPCGAttributePropertySelection::ExtraProperty && InSelector.GetExtraProperty() == EPCGExtraProperties::NumElements)
		{
			if (InSelector.GetDomainName() == PCGDataConstants::DataDomainName && InData->IsSupportedMetadataDomainID(PCGMetadataDomainID::Elements))
			{
				// Make a dummy key
				return MakeUnique<const FPCGAttributeAccessorKeysEntries>(PCGInvalidEntryKey);
			}
			else
			{
				return nullptr;
			}
		}
		
		const UPCGMetadata* Metadata = InData->ConstMetadata();
		if (!Metadata)
		{
			PCGLog::Metadata::LogInvalidMetadata();
			return nullptr;
		}
		
		const FPCGMetadataDomainID DomainID = InData->GetMetadataDomainIDFromSelector(InSelector);
		if (!DomainID.IsValid())
		{
			PCGLog::Metadata::LogInvalidMetadataDomain(InSelector);
			return nullptr;
		}

		const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomain(DomainID);
		const bool bAddDefaultValueIfEmpty = MetadataDomain ? (MetadataDomain->GetDomainID() == PCGMetadataDomainID::Data) : false;
		return MetadataDomain ? MakeUnique<const FPCGAttributeAccessorKeysEntries>(MetadataDomain, /*bAddDefaultValueIfEmpty=*/bAddDefaultValueIfEmpty) : nullptr;
	}
}

template <typename PCGData, typename Func>
decltype(auto) FPCGAttributeAccessorFactory::CallOnMethod(PCGData* InData, Func Callback) const
{
	using AccessorType = decltype(Callback(FPCGAttributeAccessorMethods{}));

	if (!InData)
	{
		return AccessorType();
	}

	TSubclassOf<UPCGData> CurrentClass = InData->GetClass();
	check(CurrentClass);

	// Current Class will dereference to child of UPCGData classes, until we have the super class of UPCGData, which will dereference to nullptr
	while (*CurrentClass)
	{
		if (const FPCGAttributeAccessorMethods* Methods = AccessorMethods.Find(CurrentClass))
		{
			AccessorType Accessor = Callback(*Methods);
			if (Accessor.IsValid())
			{
				return Accessor;
			}
		}

		CurrentClass = CurrentClass->GetSuperClass();
	}

	return AccessorType();
}

#if WITH_EDITOR
void FPCGAttributeAccessorMethods::FillSelectorMenuEntryFromEnum(const UEnum* EnumType, const TArrayView<const FText> InMenuHierarchy)
{
	if (!EnumType)
	{
		return;
	}

	FPCGAttributeSelectorMenu* CurrentMenu = &AttributeSelectorMenu;
	for (const FText& MenuLabel : InMenuHierarchy)
	{
		FPCGAttributeSelectorMenu* It = CurrentMenu->SubMenus.FindByPredicate([&MenuLabel](const FPCGAttributeSelectorMenu& Menu) { return Menu.Label.EqualTo(MenuLabel); });
		if (!It)
		{
			It = &CurrentMenu->SubMenus.Emplace_GetRef();
			It->Label = MenuLabel;
		}

		check(It);
		CurrentMenu = It;
	}

	check(CurrentMenu);

	static const FString EnumMetadataDomain_MetadataFlag = PCGObjectMetadata::EnumMetadataDomain.ToString();

	const int32 NumEnums = EnumType->ContainsExistingMax() ? EnumType->NumEnums() - 1 : EnumType->NumEnums();
	for (int32 i = 0; i < NumEnums; ++i)
	{
		if (EnumType->HasMetaData(TEXT("Hidden"), i))
		{
			continue;
		}

		// Not const to be able to move them.
		FText EnumName = EnumType->GetDisplayNameTextByIndex(i);
		FText Tooltip = EnumType->GetToolTipTextByIndex(i);
		const FString InvariantName = EnumType->GetNameStringByIndex(i); // Use string version as it strips out the namespace.
		FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreatePropertySelector(*InvariantName);		

		if (EnumType->HasMetaData(*EnumMetadataDomain_MetadataFlag, i))
		{
			const FString MetadataDomain = EnumType->GetMetaData(*EnumMetadataDomain_MetadataFlag, i);
			Selector.SetDomainName(*MetadataDomain);
		}

		const bool bReadOnly = EnumType->HasMetaData(*PCGObjectMetadata::PropertyReadOnly.ToString(), i);

		CurrentMenu->Entries.Emplace(std::move(EnumName), std::move(Tooltip), std::move(Selector), bReadOnly);
	}
}
#endif // WITH_EDITOR

FPCGAttributeAccessorFactory& FPCGAttributeAccessorFactory::GetMutableInstance()
{
	return FPCGModule::GetMutableAttributeAccessorFactory();
}

const FPCGAttributeAccessorFactory& FPCGAttributeAccessorFactory::GetInstance()
{
	return FPCGModule::GetConstAttributeAccessorFactory();
}

TUniquePtr<IPCGAttributeAccessor> FPCGAttributeAccessorFactory::CreateSimpleAccessor(UPCGData* InData,
	const FPCGAttributePropertySelector& InSelector, const bool bQuiet) const
{
	return CallOnMethod(InData, [&InData, &InSelector, bQuiet](const FPCGAttributeAccessorMethods& Methods)
	{
		return Methods.CreateAccessorFunc ? Methods.CreateAccessorFunc(InData, InSelector, bQuiet) : TUniquePtr<IPCGAttributeAccessor>();
	});
}

TUniquePtr<const IPCGAttributeAccessor> FPCGAttributeAccessorFactory::CreateSimpleConstAccessor(const UPCGData* InData,
	const FPCGAttributePropertySelector& InSelector, const bool bQuiet) const
{
	return CallOnMethod(InData, [&InData, &InSelector, bQuiet](const FPCGAttributeAccessorMethods& Methods)
	{
		return Methods.CreateConstAccessorFunc ? Methods.CreateConstAccessorFunc(InData, InSelector, bQuiet) : TUniquePtr<const IPCGAttributeAccessor>();
	});
}

TUniquePtr<IPCGAttributeAccessorKeys> FPCGAttributeAccessorFactory::CreateSimpleKeys(UPCGData* InData,
	const FPCGAttributePropertySelector& InSelector, const bool bQuiet) const
{
	if (!InData)
	{
		if (!bQuiet)
		{
			UE_LOGF(LogPCG, Error, "[FPCGAttributeAccessorFactory::CreateSimpleKeys] Can't create keys with no input data.");
		}
		
		return {};
	}
	
	return CallOnMethod(InData, [&InData, &InSelector, bQuiet](const FPCGAttributeAccessorMethods& Methods)
	{
		return Methods.CreateAccessorKeysFunc ? Methods.CreateAccessorKeysFunc(InData, InSelector, bQuiet) : TUniquePtr<IPCGAttributeAccessorKeys>();
	});
}

TUniquePtr<const IPCGAttributeAccessorKeys> FPCGAttributeAccessorFactory::CreateSimpleConstKeys(const UPCGData* InData,
	const FPCGAttributePropertySelector& InSelector, const bool bQuiet) const
{
	if (!InData)
	{
		if (!bQuiet)
		{
			UE_LOGF(LogPCG, Error, "[FPCGAttributeAccessorFactory::CreateSimpleConstKeys] Can't create keys with no input data.");
		}
		
		return {};
	}
	
	return CallOnMethod(InData, [&InData, &InSelector, bQuiet](const FPCGAttributeAccessorMethods& Methods)
	{
		return Methods.CreateConstAccessorKeysFunc ? Methods.CreateConstAccessorKeysFunc(InData, InSelector, bQuiet) : TUniquePtr<const IPCGAttributeAccessorKeys>();
	});
}

#if WITH_EDITOR
void FPCGAttributeAccessorFactory::ForEachSelectorMenu(TFunctionRef<void(const FPCGAttributeSelectorMenu&)> Callback) const
{
	for (const auto& [Class, Methods] : AccessorMethods)
	{
		Callback(Methods.AttributeSelectorMenu);
	}
}
#endif // WITH_EDITOR

void FPCGAttributeAccessorFactory::RegisterMethods(TSubclassOf<UPCGData> PCGDataClass, FPCGAttributeAccessorMethods&& InAccessorMethods)
{
	check(*PCGDataClass);
	
	if (!ensure(!AccessorMethods.Contains(PCGDataClass)))
	{
		UE_LOGF(LogPCG, Error, "Trying to register %ls accessor methods multiple times, will be ignored.", *PCGDataClass->GetName());
		return;
	}
	
	AccessorMethods.Add(PCGDataClass, std::move(InAccessorMethods));
}

void FPCGAttributeAccessorFactory::UnregisterMethods(TSubclassOf<UPCGData> PCGDataClass)
{
	AccessorMethods.Remove(PCGDataClass);
}

void FPCGAttributeAccessorFactory::RegisterDefaultMethods()
{
	FPCGAttributeAccessorMethods DefaultMethods
	{
		.CreateAccessorFunc = PCGAttributeAccessorFactory::CreateDefaultAccessor,
		.CreateConstAccessorFunc = PCGAttributeAccessorFactory::CreateDefaultConstAccessor,
		.CreateAccessorKeysFunc = PCGAttributeAccessorFactory::CreateDefaultAccessorKeys,
		.CreateConstAccessorKeysFunc = PCGAttributeAccessorFactory::CreateDefaultConstAccessorKeys,
	};

	RegisterMethods<UPCGData>(std::move(DefaultMethods));
}

void FPCGAttributeAccessorFactory::UnregisterDefaultMethods()
{
	UnregisterMethods<UPCGData>();
}
