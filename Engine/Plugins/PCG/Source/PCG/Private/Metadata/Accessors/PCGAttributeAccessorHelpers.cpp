// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "PCGData.h"
#include "PCGParamData.h"
#include "PCGSettings.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"
#include "Metadata/Accessors/PCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpersInternal.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"
#include "Metadata/Accessors/PCGAttributeExtractor.h"

#include "StructUtils/UserDefinedStruct.h"
#include "UObject/EnumProperty.h"
#include "VerseVM/VVMVerseClass.h"

namespace PCGAttributeAccessorHelpers
{
	TUniquePtr<IPCGAttributeAccessor> CreateChainAccessor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess)
	{
		if (!InAccessor.IsValid())
		{
			bOutSuccess = false;
			return TUniquePtr<IPCGAttributeAccessor>();
		}

		bOutSuccess = true;

		auto Chain = [&Accessor = InAccessor, Name, &bOutSuccess](auto Dummy) -> TUniquePtr<IPCGAttributeAccessor>
		{
			using AccessorType = decltype(Dummy);

			if constexpr (PCG::Private::IsOfTypes<AccessorType, FVector2D, FVector, FVector4>())
			{
				return PCGAttributeExtractor::CreateVectorExtractor<AccessorType>(MoveTemp(Accessor), Name, bOutSuccess);
			}
			else if constexpr (PCG::Private::IsOfTypes<AccessorType, FTransform>())
			{
				return PCGAttributeExtractor::CreateTransformExtractor(MoveTemp(Accessor), Name, bOutSuccess);
			}
			else if constexpr (PCG::Private::IsOfTypes<AccessorType, FQuat>())
			{
				return PCGAttributeExtractor::CreateQuatExtractor(MoveTemp(Accessor), Name, bOutSuccess);
			}
			else if constexpr (PCG::Private::IsOfTypes<AccessorType, FRotator>())
			{
				return PCGAttributeExtractor::CreateRotatorExtractor(MoveTemp(Accessor), Name, bOutSuccess);
			}
			else if constexpr (PCG::Private::IsOfTypes<AccessorType, FString, FName, FSoftObjectPath, FSoftClassPath>())
			{
				return PCGAttributeExtractor::CreateStringExtractor<AccessorType>(MoveTemp(Accessor), Name, bOutSuccess);
			}
			else
			{
				bOutSuccess = false;
				return std::move(Accessor);
			}
		};

		return PCGMetadataAttribute::CallbackWithRightType(InAccessor->GetUnderlyingType(), Chain);
	}
	
	TUniquePtr<IPCGAttributeAccessor> CreateChainAccessor(TUniquePtr<IPCGAttributeAccessor> InAccessor, const FPCGAttributePropertySelector& InSelector, bool& bOutSuccess, bool bQuiet)
	{
		if (!InAccessor.IsValid())
		{
			bOutSuccess = false;
			return {};
		}
		
		bOutSuccess = true;
		for (const FString& ExtraName : InSelector.GetExtraNames())
		{
			InAccessor = CreateChainAccessor(std::move(InAccessor), FName(ExtraName), bOutSuccess);
			if (!bOutSuccess)
			{
				if (!bQuiet)
				{
					UE_LOGF(LogPCG, Error, "[PCGAttributeAccessorHelpers::CreateChainAccessor] Extra selectors don't match existing properties.");
				}
				
				return {};
			}
		}

		return InAccessor;
	}

	bool GetPropertyChain(const TArray<FName>& InPropertyNames, const UStruct* InStruct, TArray<const FProperty*>& OutProperties)
	{
		check(InStruct);

		const UStruct* CurrentStruct = InStruct;
		OutProperties.Reserve(InPropertyNames.Num());

		for (int32 i = 0; i < InPropertyNames.Num(); ++i)
		{
			const FProperty* Property = nullptr;
			const FName PropertyName = InPropertyNames[i];

			Property = PCGPropertyHelpers::FindPropertyByName(CurrentStruct, PropertyName);

			if (!Property)
			{
				UE_LOGF(LogPCG, Error, "Property '%ls' does not exist in %ls.", *PropertyName.ToString(), *CurrentStruct->GetName());
				return false;
			}

			OutProperties.Add(Property);

			// If the property is a container, we also need to add the inner property
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				Property = ArrayProperty->Inner;
				OutProperties.Add(Property);
			}

			// Check for a struct or object for all properties except the last one.
			if (i < InPropertyNames.Num() - 1)
			{
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					CurrentStruct = StructProperty->Struct;
				}
				else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
				{
					CurrentStruct = ObjectProperty->PropertyClass;
				}
				else
				{
					UE_LOGF(LogPCG, Error, "Property '%ls' does exist in % ls, but is not extractable.", *PropertyName.ToString(), *CurrentStruct->GetName());
					return false;
				}
			}
		}

		return true;
	}
}

TUniquePtr<IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreatePropertyAccessor(const FProperty* InProperty, bool bUseGenericAccessor)
{
	if (bUseGenericAccessor)
	{
		if (FPCGMetadataAttributeDesc::CreateFromProperty(InProperty).IsValid())
		{
			return MakeUnique<FPCGPropertyGenericAccessor>(InProperty);
		}
		else
		{
			return nullptr;
		}
	}
	else
	{
		return PCGAttributeAccessorHelpers::Internal::DispatchPropertyTypes(InProperty, [](auto SignatureDummy, const auto* TypedProperty) -> TUniquePtr<IPCGAttributeAccessor>
		{
			using TypedAccessor = typename decltype(SignatureDummy)::Type;
			return MakeUnique<TypedAccessor>(TypedProperty);
		});
	}
}

TUniquePtr<IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreatePropertyAccessor(const FName InPropertyName, const UStruct* InStruct, bool bUseGenericAccessor)
{
	if (bUseGenericAccessor)
	{
		if (const FProperty* Property = PCGPropertyHelpers::FindPropertyByName(InStruct, InPropertyName); Property && FPCGMetadataAttributeDesc::CreateFromProperty(Property).IsValid())
		{
			return MakeUnique<FPCGPropertyGenericAccessor>(Property);
		}
		else
		{
			return nullptr;
		}
	}
	else
	{
		return PCGAttributeAccessorHelpers::Internal::DispatchPropertyTypes(InPropertyName, InStruct, [](auto SignatureDummy, const auto* TypedProperty) -> TUniquePtr<IPCGAttributeAccessor>
		{
			using TypedAccessor = typename decltype(SignatureDummy)::Type;
			return MakeUnique<TypedAccessor>(TypedProperty);
		});
	}
}

TUniquePtr<IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(TArray<const FProperty*>&& InProperties, bool bUseGenericAccessor)
{
	if (InProperties.IsEmpty())
	{
		return TUniquePtr<IPCGAttributeAccessor>{};
	}
	if (bUseGenericAccessor)
	{
		if (FPCGMetadataAttributeDesc::CreateFromProperty(InProperties.Last()).IsValid())
		{
			return MakeUnique<FPCGPropertyGenericAccessor>(InProperties.Last(), MoveTemp(InProperties));
		}
		else
		{
			return nullptr;
		}
	}
	else
	{
		return PCGAttributeAccessorHelpers::Internal::DispatchPropertyTypes(InProperties.Last(), [&InProperties](auto SignatureDummy, const auto* TypedProperty) -> TUniquePtr<IPCGAttributeAccessor>
		{
			using TypedAccessor = typename decltype(SignatureDummy)::Type;
			return MakeUnique<TypedAccessor>(TypedProperty, MoveTemp(InProperties));
		});
	}
}

TUniquePtr<IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(const TArray<FName>& InPropertyNames, const UStruct* InStruct, bool bUseGenericAccessor)
{
	TArray<const FProperty*> PropertyChain;
	if (!PCGAttributeAccessorHelpers::GetPropertyChain(InPropertyNames, InStruct, PropertyChain))
	{
		return TUniquePtr<IPCGAttributeAccessor>{};
	}
	else
	{
		return PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(std::move(PropertyChain), bUseGenericAccessor);
	}
}

bool PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(const FProperty* InProperty, bool bUseGenericAccessor)
{
	if (bUseGenericAccessor)
	{
		return FPCGMetadataAttributeDesc::CreateFromProperty(InProperty).IsValid();
	}
	else
	{
		return PCGAttributeAccessorHelpers::Internal::DispatchPropertyTypes(InProperty, [](auto, const auto*) -> bool
		{
			return true;
		});
	}
}

bool PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(const FName InPropertyName, const UStruct* InStruct, bool bUseGenericAccessor)
{
	if (bUseGenericAccessor)
	{
		if (const FProperty* Property = PCGPropertyHelpers::FindPropertyByName(InStruct, InPropertyName))
		{
			return FPCGMetadataAttributeDesc::CreateFromProperty(Property).IsValid();
		}
		else
		{
			return false;
		}
	}
	else
	{
		return PCGAttributeAccessorHelpers::Internal::DispatchPropertyTypes(InPropertyName, InStruct, [](auto, const auto*) -> bool
		{
			return true;
		});
	}
}

bool PCGAttributeAccessorHelpers::IsPropertyAccessorChainSupported(const TArray<FName>& InPropertyNames, const UStruct* InStruct, bool bUseGenericAccessor)
{
	TArray<const FProperty*> PropertyChain;
	return PCGAttributeAccessorHelpers::GetPropertyChain(InPropertyNames, InStruct, PropertyChain) &&
		!PropertyChain.IsEmpty() &&
		PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(PropertyChain.Last(), bUseGenericAccessor);
}

EPCGMetadataTypes PCGAttributeAccessorHelpers::GetMetadataTypeForProperty(const FProperty* InProperty, bool bUseGenericAccessor)
{
	if (bUseGenericAccessor)
	{
		return FPCGMetadataAttributeDesc::CreateFromProperty(InProperty).ValueType;
	}
	else
	{
		bool bFound = false;
		const EPCGMetadataTypes Result = PCGAttributeAccessorHelpers::Internal::DispatchPropertyTypes(InProperty, [&bFound, InProperty](auto SignatureDummy, const auto* TypedProperty) -> EPCGMetadataTypes
		{
			bFound = true;
			using TypedAccessor = decltype(SignatureDummy)::Type;
			return static_cast<EPCGMetadataTypes>(PCG::Private::MetadataTypes<typename TypedAccessor::Type>::Id);
		});

		return bFound ? Result : EPCGMetadataTypes::Unknown;
	}
}

TUniquePtr<IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreateExtraAccessor(EPCGExtraProperties InExtraProperties)
{
	switch (InExtraProperties)
	{
	case EPCGExtraProperties::Index:
		return MakeUnique<FPCGIndexAccessor>();
	default:
		return TUniquePtr<IPCGAttributeAccessor>();
	}
}

TUniquePtr<const IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreateConstAccessorForOverrideParamWithResult(const FPCGDataCollection& InInputData, const FPCGSettingsOverridableParam& InParam, AccessorParamResult* OutResult)
{
	bool bFromGlobalParamsPin = false;
	TArray<FPCGTaggedData> InputParamData = InInputData.GetParamsByPin(InParam.Label);
	if (InputParamData.IsEmpty())
	{
		// If it is empty, try with the Overrides pin (Global Params)
		bFromGlobalParamsPin = true;
		InputParamData = InInputData.GetParamsByPin(PCGPinConstants::DefaultParamsLabel);
	}

	if (OutResult)
	{
		OutResult->bHasMultipleAttributeSetsOnOverridePin = InputParamData.Num() > 1;
	}

	const UPCGParamData* ParamData = !InputParamData.IsEmpty() ? Cast<UPCGParamData>(InputParamData[0].Data) : nullptr;

	if (OutResult && ParamData && !bFromGlobalParamsPin)
	{
		OutResult->bPinConnected = true;
	}

	if (ParamData && ParamData->Metadata && ParamData->Metadata->GetAttributeCount() > 0)
	{
		// If the param only has a single attribute and is not from the global Params pin, use this one. Otherwise we need perfect name matching, either the property name or its full path if there is a name clash.
		const FName AttributeName = (ParamData->Metadata->GetAttributeCount() == 1 && !bFromGlobalParamsPin) 
			? ParamData->Metadata->GetLatestAttributeNameOrNone() 
			: (!InParam.bHasNameClash ? InParam.PropertiesNames.Last() : FName(InParam.GetPropertyPath()));

		if (OutResult)
		{
			OutResult->AttributeName = AttributeName;
		}

		FPCGAttributePropertyInputSelector InputSelector{};
		InputSelector.SetAttributeName(AttributeName);
		TUniquePtr<const IPCGAttributeAccessor> Result = PCGAttributeAccessorHelpers::CreateConstAccessor(ParamData, InputSelector);

		if (!Result)
		{
			// If we didn't find it, try some aliases.
			for (const FName& Alias : InParam.GenerateAllPossibleAliases())
			{
				InputSelector.SetAttributeName(Alias);
				Result = PCGAttributeAccessorHelpers::CreateConstAccessor(ParamData, InputSelector);
				if (Result)
				{
					if (OutResult)
					{
						OutResult->bUsedAliases = true;
						OutResult->AliasUsed = Alias;
					}

					break;
				}
			}
		}

		if (Result && OutResult && ParamData && ParamData->Metadata)
		{
			OutResult->bHasMultipleDataInAttributeSet = ParamData->Metadata->GetLocalItemCount() > 1;
			OutResult->bHasNoEntry = ParamData->Metadata->GetLocalItemCount() == 0;
			
			if (InParam.NumContainers > 0)
			{
				OutResult->Keys = PCGAttributeAccessorHelpers::CreateConstKeys(ParamData, InputSelector);
			}
		}

		return Result;
	}

	return TUniquePtr<const IPCGAttributeAccessor>{};
}

TUniquePtr<const IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreateConstAccessor(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet)
{
	TUniquePtr<IPCGAttributeAccessor> Accessor;
	
	if (InData)
	{
		// Temporary const_cast for the chain accessor.
		Accessor.Reset(const_cast<IPCGAttributeAccessor*>(FPCGAttributeAccessorFactory::GetInstance().CreateSimpleConstAccessor(InData, InSelector, bQuiet).Release()));
	}
	else
	{
		// For backward compatibility, no data is point data (or extra property)
		if (InSelector.GetSelection() == EPCGAttributePropertySelection::ExtraProperty)
		{
			Accessor = CreateExtraAccessor(InSelector.GetExtraProperty());
		}
		else
		{
			Accessor = UPCGPointData::CreateStaticAccessor(InSelector, bQuiet);
		}
	}

	if (!Accessor.IsValid())
	{
		return {};
	}

	return CreateChainAccessor(std::move(Accessor), InSelector, bQuiet);
}

TUniquePtr<const IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreateConstAccessor(const FPCGMetadataAttributeBase* InAttribute, const UPCGMetadata* InMetadata, bool bQuiet)
{
	if (!InMetadata || !InAttribute)
	{
		return {};
	}
	else
	{
		return CreateConstAccessor(InAttribute, InMetadata->GetConstDefaultMetadataDomain(), bQuiet);
	}
}

TUniquePtr<const IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreateConstAccessor(const FPCGMetadataAttributeBase* InAttribute, const FPCGMetadataDomain* InMetadata, bool bQuiet)
{
	if (!InMetadata || !InAttribute)
	{
		return {};
	}
	else
	{
		return MakeUnique<const FPCGAttributeGenericAccessor>(InAttribute, InMetadata);
	}
}

TUniquePtr<IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreateAccessor(UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet)
{
	TUniquePtr<IPCGAttributeAccessor> Accessor;
	
	if (InData)
	{
		// Temporary const_cast for the chain accessor.
		Accessor = FPCGAttributeAccessorFactory::GetInstance().CreateSimpleAccessor(InData, InSelector, bQuiet);
	}
	else
	{
		// For backward compatibility, no data is point data (or extra property)
		if (InSelector.GetSelection() == EPCGAttributePropertySelection::ExtraProperty)
		{
			Accessor = CreateExtraAccessor(InSelector.GetExtraProperty());
		}
		else
		{
			Accessor = UPCGPointData::CreateStaticAccessor(InSelector, bQuiet);
		}
	}

	if (!Accessor.IsValid())
	{
		return {};
	}

	Accessor = CreateChainAccessor(std::move(Accessor), InSelector, bQuiet);
	
	if (Accessor && Accessor->IsReadOnly())
	{
		if (!bQuiet)
		{
			UE_LOGF(LogPCG, Error, "[PCGAttributeAccessorHelpers::CreateAccessor] Attribute can not be written into, since it is read-only.");
		}
				
		return {};
	}

	// Set the cached selector
	if (InData)
	{
		InData->SetLastSelector(InSelector);
	}

	return Accessor;
}

TUniquePtr<IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreateAccessor(FPCGMetadataAttributeBase* InAttribute, UPCGMetadata* InMetadata, bool bQuiet)
{
	if (!InMetadata || !InAttribute)
	{
		return {};
	}
	else
	{
		return CreateAccessor(InAttribute, InMetadata->GetDefaultMetadataDomain());
	}
}

TUniquePtr<IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreateAccessor(FPCGMetadataAttributeBase* InAttribute, FPCGMetadataDomain* InMetadata, bool bQuiet)
{
	if (!InMetadata || !InAttribute)
	{
		return {};
	}
	else
	{
		return MakeUnique<FPCGAttributeGenericAccessor>(InAttribute, InMetadata);
	}
}

TUniquePtr<IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreateAccessorWithAttributeCreation(UPCGData* InData, const FPCGAttributePropertySelector& InSelector, const IPCGAttributeAccessor* InMatchingAccessor, EPCGAttributeAccessorFlags InTypeMatching, bool bQuiet)
{
	FPCGCreateAccessorWithAttributeCreationParams Params =
	{
		.InData = InData,
		.InSelector = &InSelector,
		.InMatchingAccessor = InMatchingAccessor,
		.InTypeMatching = InTypeMatching,
		.bQuiet = bQuiet
	};

	return CreateAccessorWithAttributeCreation(Params);
}

TUniquePtr<IPCGAttributeAccessor> PCGAttributeAccessorHelpers::CreateAccessorWithAttributeCreation(const FPCGCreateAccessorWithAttributeCreationParams& InParams)
{
	if (!InParams.InData || !InParams.InSelector)
	{
		return nullptr;
	}

	TUniquePtr<IPCGAttributeAccessor> Result = CreateAccessor(InParams.InData, *InParams.InSelector, InParams.bQuiet);

	if (!InParams.InSelector->IsBasicAttribute())
	{
		return Result;
	}

	bool bValid = !!Result;
	FPCGMetadataAttributeDesc MatchingDesc{};
	if (InParams.InMatchingAccessor)
	{
		MatchingDesc = InParams.InMatchingAccessor->GetUnderlyingDesc();
	}
	else
	{
		MatchingDesc = InParams.InExpectedDesc;
	}

	// The underlying type of the accessor is expected to be valid (otherwise it is ill-formed), so this is mainly to catch if InExpectedDesc is valid.
	if (!MatchingDesc.IsValid())
	{
		return Result;
	}
	
	// Force objects/soft objects to be paths, as attributes do not support objects.
	MatchingDesc.ConvertObjectsToSoftPath();

	if (bValid && !!(InParams.InTypeMatching & EPCGAttributeAccessorFlags::StrictType))
	{
		bValid &= MatchingDesc.IsSameType(Result->GetUnderlyingDesc());
	}

	if (bValid && !!(InParams.InTypeMatching & EPCGAttributeAccessorFlags::AllowBroadcast))
	{
		bValid &= PCG::Private::IsBroadcastable(MatchingDesc, Result->GetUnderlyingDesc());
	}

	if (bValid && !!(InParams.InTypeMatching & EPCGAttributeAccessorFlags::AllowConstructible))
	{
		bValid &= PCG::Private::IsConstructible(MatchingDesc, Result->GetUnderlyingDesc());
	}

	if (!bValid)
	{
		Result.Reset();

		// We didn't find the attribute in the data, or we can't Broadcast/Construct, so create a new one.
		// Honour the selector's domain — falling back to the default domain would silently ignore a
		// user-specified domain (e.g. @Data) and create the attribute in the wrong place.
		UPCGMetadata* Metadata = InParams.InData->MutableMetadata();
		const FPCGMetadataDomainID DomainID = InParams.InData->GetMetadataDomainIDFromSelector(*InParams.InSelector);
		FPCGMetadataDomain* MetadataDomain = Metadata ? Metadata->GetMetadataDomain(DomainID) : nullptr;
		if (!MetadataDomain)
		{
			return Result;
		}

		const FName AttributeName = InParams.InSelector->GetName();
		if (MetadataDomain->HasAttribute(AttributeName))
		{
			MetadataDomain->DeleteAttribute(AttributeName);
		}
		
		MatchingDesc.Name = AttributeName;
		
		FPCGMetadataAttributeBase* Attribute =  MetadataDomain->CreateAttribute(MatchingDesc, /*bAllowInterpolation=*/true, /*bOverrideParent=*/false);
		if (!Attribute)
		{
			return nullptr;
		}
		
		Result = MakeUnique<FPCGAttributeGenericAccessor>(Attribute, MetadataDomain);

		if (InParams.InMatchingAccessor)
		{
			FPCGAttributeAccessorKeysEntries InvalidKey(PCGInvalidEntryKey);
			InParams.InMatchingAccessor->CopyTo(InParams.InMatchingKeysForDefaultValue ? *InParams.InMatchingKeysForDefaultValue : InvalidKey, *Result, InvalidKey, /*Index=*/0, /*Count=*/1, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible | EPCGAttributeAccessorFlags::AllowSetDefaultValue);
		}
	}

	return Result;
}

TUniquePtr<const IPCGAttributeAccessorKeys> PCGAttributeAccessorHelpers::CreateConstKeys(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector)
{
	return FPCGAttributeAccessorFactory::GetInstance().CreateSimpleConstKeys(InData, InSelector, /*bQuiet=*/ false);
}

TUniquePtr<IPCGAttributeAccessorKeys> PCGAttributeAccessorHelpers::CreateKeys(UPCGData* InData, const FPCGAttributePropertySelector& InSelector)
{
	return FPCGAttributeAccessorFactory::GetInstance().CreateSimpleKeys(InData, InSelector, /*bQuiet=*/ false);
}

TUniquePtr<const IPCGAttributeAccessorKeys> PCGAttributeAccessorHelpers::CreateConstKeys(const FPCGMetadataAttributeBase* InAttribute)
{
	const UPCGData* Data = InAttribute && InAttribute->GetMetadata() ? Cast<const UPCGData>(InAttribute->GetMetadata()->GetOuter()) : nullptr;
	
	if (!Data || !InAttribute->GetMetadataDomain())
	{
		return nullptr;
	}

	FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(InAttribute->Name);
	Data->SetDomainFromDomainID(InAttribute->GetMetadataDomain()->GetDomainID(), Selector);
		
	return PCGAttributeAccessorHelpers::CreateConstKeys(Data, Selector);
}

TUniquePtr<IPCGAttributeAccessorKeys> PCGAttributeAccessorHelpers::CreateKeys(FPCGMetadataAttributeBase* InAttribute)
{
	UPCGData* Data = InAttribute && InAttribute->GetMetadata() ? Cast<UPCGData>(InAttribute->GetMetadata()->GetOuter()) : nullptr;
	
	if (!Data || !InAttribute->GetMetadataDomain())
	{
		return nullptr;
	}

	FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(InAttribute->Name);
	Data->SetDomainFromDomainID(InAttribute->GetMetadataDomain()->GetDomainID(), Selector);
		
	return PCGAttributeAccessorHelpers::CreateKeys(Data, Selector);
}
