// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "PCGPoint.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPolygon2DData.h"
#include "Data/PCGSplineData.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttribute.h"

//////////////////////////////////////////////////////////////////// 
FPCGAttributeAccessorKeysEntries::FPCGAttributeAccessorKeysEntries(const FPCGMetadataAttributeBase* Attribute)
	: FPCGAttributeAccessorKeysEntries(Attribute->GetMetadata())
{
	// Deprecated
}

FPCGAttributeAccessorKeysEntries::FPCGAttributeAccessorKeysEntries(PCGMetadataEntryKey EntryKey)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ false)
{
	ExtractedEntries.Add(EntryKey);
	Entries = TArrayView<PCGMetadataEntryKey>(ExtractedEntries);
}

FPCGAttributeAccessorKeysEntries::FPCGAttributeAccessorKeysEntries(const UPCGMetadata* Metadata, bool bAddDefaultValueIfEmpty)
	: FPCGAttributeAccessorKeysEntries(Metadata ? Metadata->GetConstDefaultMetadataDomain() : nullptr, bAddDefaultValueIfEmpty)
{}

FPCGAttributeAccessorKeysEntries::FPCGAttributeAccessorKeysEntries(UPCGMetadata* Metadata, bool bAddDefaultValueIfEmpty)
	: FPCGAttributeAccessorKeysEntries(Metadata ? Metadata->GetDefaultMetadataDomain() : nullptr, bAddDefaultValueIfEmpty)
{}

FPCGAttributeAccessorKeysEntries::FPCGAttributeAccessorKeysEntries(const FPCGMetadataDomain* Metadata, bool bAddDefaultValueIfEmpty)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ true)
{
	InitializeFromMetadata(Metadata, bAddDefaultValueIfEmpty);
	Entries = TArrayView<PCGMetadataEntryKey>(ExtractedEntries);
}

FPCGAttributeAccessorKeysEntries::FPCGAttributeAccessorKeysEntries(FPCGMetadataDomain* Metadata, bool bAddDefaultValueIfEmpty)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ false)
{
	InitializeFromMetadata(Metadata, bAddDefaultValueIfEmpty);
	Entries = TArrayView<PCGMetadataEntryKey>(ExtractedEntries);
}

FPCGAttributeAccessorKeysEntries::FPCGAttributeAccessorKeysEntries(const TArrayView<PCGMetadataEntryKey>& InEntries)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/false)
	, Entries(InEntries)
{

}

FPCGAttributeAccessorKeysEntries::FPCGAttributeAccessorKeysEntries(const TArrayView<const PCGMetadataEntryKey>& InEntries)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/true)
	, Entries(const_cast<PCGMetadataEntryKey*>(InEntries.GetData()), InEntries.Num())
{

}

void FPCGAttributeAccessorKeysEntries::InitializeFromMetadata(const FPCGMetadataDomain* Metadata, bool bAddDefaultValueIfEmpty)
{
	if (!Metadata)
	{
		return;
	}

	check(ExtractedEntries.IsEmpty());
	const PCGMetadataEntryKey ItemKeyUpperBound = Metadata->GetItemCountForChild();

	if (ItemKeyUpperBound > 0)
	{
		ExtractedEntries.Reserve(ItemKeyUpperBound);

		for (PCGMetadataEntryKey Entry = 0; Entry < ItemKeyUpperBound; ++Entry)
		{
			ExtractedEntries.Add(Entry);
		}
	}

	if (ExtractedEntries.IsEmpty() && bAddDefaultValueIfEmpty)
	{
		ExtractedEntries.Add(PCGInvalidEntryKey);
	}
}

FPCGAttributeAccessorKeysEntries::FPCGAttributeAccessorKeysEntries()
	: IPCGAttributeAccessorKeys(/*bReadOnly=*/true)
{
}

bool FPCGAttributeAccessorKeysEntries::GetMetadataEntryKeys(int32 InStart, TArrayView<PCGMetadataEntryKey*> OutEntryKeys)
{
	return PCGAttributeAccessorKeys::GetKeys(Entries, InStart, OutEntryKeys, [](PCGMetadataEntryKey& Key) -> PCGMetadataEntryKey* { return &Key; });
}

bool FPCGAttributeAccessorKeysEntries::GetMetadataEntryKeys(int32 InStart, TArrayView<const PCGMetadataEntryKey*> OutEntryKeys) const
{
	return PCGAttributeAccessorKeys::GetKeys(Entries, InStart, OutEntryKeys, [](const PCGMetadataEntryKey& Key) -> const PCGMetadataEntryKey* { return &Key; });
}

////////////////////////////////////////////////////////////////////

FPCGAttributeAccessorKeysPoints::FPCGAttributeAccessorKeysPoints(const TArrayView<FPCGPoint>& InPoints)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ false)
	, Points(InPoints)
{}

FPCGAttributeAccessorKeysPoints::FPCGAttributeAccessorKeysPoints(const TArrayView<const FPCGPoint>& InPoints)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ true)
	, Points(const_cast<FPCGPoint*>(InPoints.GetData()), InPoints.Num())
{}

FPCGAttributeAccessorKeysPoints::FPCGAttributeAccessorKeysPoints(FPCGPoint& InPoint)
	: FPCGAttributeAccessorKeysPoints(TArrayView<FPCGPoint>(&InPoint, 1))
{}

FPCGAttributeAccessorKeysPoints::FPCGAttributeAccessorKeysPoints(const FPCGPoint& InPoint)
	: FPCGAttributeAccessorKeysPoints(TArrayView<const FPCGPoint>(&InPoint, 1))
{}

bool FPCGAttributeAccessorKeysPoints::IsClassSupported(const UStruct* InClass) const
{
	return PCGAttributeAccessorKeys::IsClassSupported<FPCGPoint>(InClass);
}

bool FPCGAttributeAccessorKeysPoints::GetKeyIndices(int32 InStart, int32 InCount, TArray<int32>& OutKeyIndices, bool& bOutContiguous) const
{
	if (Points.IsEmpty())
	{
		return false;
	}

	// Optimization (avoid allocating indices memory)
	if (InStart + InCount <= Points.Num())
	{
		bOutContiguous = true;
		return true;
	}

	OutKeyIndices.SetNumUninitialized(InCount);
	for (int32 Index = 0; Index < InCount; ++Index)
	{
		OutKeyIndices[Index] = (InStart + Index) % Points.Num();
	}

	return true;
}

bool FPCGAttributeAccessorKeysPoints::GetPointKeys(int32 InStart, TArrayView<FPCGPoint*> OutPoints)
{
	return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutPoints, [](FPCGPoint& Point) -> FPCGPoint* { return &Point; });
}

bool FPCGAttributeAccessorKeysPoints::GetPointKeys(int32 InStart, TArrayView<const FPCGPoint*> OutPoints) const
{
	return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutPoints, [](const FPCGPoint& Point) -> const FPCGPoint* { return &Point; });
}

bool FPCGAttributeAccessorKeysPoints::GetGenericObjectKeys(int32 InStart, TArrayView<void*> OutObjects)
{
	return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutObjects, [](FPCGPoint& Point) -> void* { return &Point; });
}

bool FPCGAttributeAccessorKeysPoints::GetGenericObjectKeys(int32 InStart, TArrayView<const void*> OutObjects) const
{
	return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutObjects, [](const FPCGPoint& Point) -> const void* { return &Point; });
}

bool FPCGAttributeAccessorKeysPoints::GetMetadataEntryKeys(int32 InStart, TArrayView<PCGMetadataEntryKey*> OutEntryKeys)
{
	return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutEntryKeys, [](FPCGPoint& Point) -> PCGMetadataEntryKey* { return &(Point.MetadataEntry); });
}

bool FPCGAttributeAccessorKeysPoints::GetMetadataEntryKeys(int32 InStart, TArrayView<const PCGMetadataEntryKey*> OutEntryKeys) const
{
	return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutEntryKeys, [](const FPCGPoint& Point) -> const PCGMetadataEntryKey* { return &(Point.MetadataEntry); });
}

////////////////////////////////////////////////////////////////////

FPCGAttributeAccessorKeysPointsSubset::FPCGAttributeAccessorKeysPointsSubset(const TArrayView<FPCGPoint>& InPoints, const TArrayView<const int32>& InPointIndices)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ false)
{
	check(InPoints.Num() == InPointIndices.Num());
	Points.Reserve(InPointIndices.Num());
	Algo::Transform(InPointIndices, Points, [&InPoints](const int32 Index) -> FPCGPoint* { return &InPoints[Index]; });
}

FPCGAttributeAccessorKeysPointsSubset::FPCGAttributeAccessorKeysPointsSubset(const TArrayView<const FPCGPoint>& InPoints, const TArrayView<const int32>& InPointIndices)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ true)
{
	Points.Reserve(InPointIndices.Num());
	Algo::Transform(InPointIndices, Points, [&InPoints](const int32 Index) -> FPCGPoint* { return const_cast<FPCGPoint*>(&InPoints[Index]); });
}

FPCGAttributeAccessorKeysPointsSubset::FPCGAttributeAccessorKeysPointsSubset(TArray<FPCGPoint*> InPointPtrs)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ false)
	, Points(std::move(InPointPtrs))
{}

FPCGAttributeAccessorKeysPointsSubset::FPCGAttributeAccessorKeysPointsSubset(TArray<const FPCGPoint*> InPointPtrs)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ true)
	, Points(std::move(*reinterpret_cast<TArray<FPCGPoint*>*>(&InPointPtrs)))
{}

FPCGAttributeAccessorKeysPointsSubset::FPCGAttributeAccessorKeysPointsSubset(const UPCGBasePointData* InPointData, const TArrayView<const int32>& InPointIndices)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ true)
	, PointData(const_cast<UPCGBasePointData*>(InPointData))
	, PointIndices(InPointIndices)
{}

FPCGAttributeAccessorKeysPointsSubset::FPCGAttributeAccessorKeysPointsSubset(UPCGBasePointData* InPointData, const TArrayView<const int32>& InPointIndices)
	: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ false)
	, PointData(InPointData)
	, PointIndices(InPointIndices)
{}

bool FPCGAttributeAccessorKeysPointsSubset::IsClassSupported(const UStruct* InClass) const
{
	return PCGAttributeAccessorKeys::IsClassSupported<FPCGPoint>(InClass);
}
bool FPCGAttributeAccessorKeysPointsSubset::GetKeyIndices(int32 InStart, int32 InCount, TArray<int32>& OutKeyIndices, bool& bOutContiguous) const
{
	bOutContiguous = false;

	if (PointIndices.Num())
	{
		OutKeyIndices.SetNumUninitialized(InCount);
		// @todo_pcg: could implement a memcpy version if InStart+Count <= PointIndices.Num()
		for (int32 Index = 0; Index < InCount; ++Index)
		{
			const int32 KeyIndex = (InStart + Index) % PointIndices.Num();
			OutKeyIndices[Index] = PointIndices[KeyIndex];
		}

		return true;
	}

	return false;
}

bool FPCGAttributeAccessorKeysPointsSubset::GetPointKeys(int32 InStart, TArrayView<FPCGPoint*> OutPoints)
{
	if (Points.Num() > 0)
	{
		return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutPoints, [](FPCGPoint* Point) -> FPCGPoint* { return Point; });
	}
	else if (UPCGPointData* LocalPointData = Cast<UPCGPointData>(PointData))
	{
		TArray<FPCGPoint>& LocalPoints = LocalPointData->GetMutablePoints();
		return PCGAttributeAccessorKeys::GetKeys(PointIndices, InStart, OutPoints, [&LocalPoints](int32 Index) -> FPCGPoint* { return &LocalPoints[Index]; });
	}
	else
	{
		return false;
	}
}

bool FPCGAttributeAccessorKeysPointsSubset::GetPointKeys(int32 InStart, TArrayView<const FPCGPoint*> OutPoints) const
{
	if (Points.Num() > 0)
	{
		return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutPoints, [](const FPCGPoint* Point) -> const FPCGPoint* { return Point; });
	}
	else if (UPCGPointData* LocalPointData = Cast<UPCGPointData>(PointData))
	{
		const TArray<FPCGPoint>& LocalPoints = LocalPointData->GetPoints();
		return PCGAttributeAccessorKeys::GetKeys(PointIndices, InStart, OutPoints, [&LocalPoints](int32 Index) -> const FPCGPoint* { return &LocalPoints[Index]; });
	}
	else
	{
		return false;
	}
}

bool FPCGAttributeAccessorKeysPointsSubset::GetGenericObjectKeys(int32 InStart, TArrayView<void*> OutObjects)
{
	if (Points.Num() > 0)
	{
		return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutObjects, [](FPCGPoint* Point) -> void* { return Point; });
	}
	else if (UPCGPointData* LocalPointData = Cast<UPCGPointData>(PointData))
	{
		TArray<FPCGPoint>& LocalPoints = LocalPointData->GetMutablePoints();
		return PCGAttributeAccessorKeys::GetKeys(PointIndices, InStart, OutObjects, [&LocalPoints](int32 Index) -> void* { return &LocalPoints[Index]; });
	}
	else
	{
		return false;
	}
}

bool FPCGAttributeAccessorKeysPointsSubset::GetGenericObjectKeys(int32 InStart, TArrayView<const void*> OutObjects) const
{
	if (Points.Num() > 0)
	{
		return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutObjects, [](const FPCGPoint* Point) -> const void* { return Point; });
	}
	else if (UPCGPointData* LocalPointData = Cast<UPCGPointData>(PointData))
	{
		const TArray<FPCGPoint>& LocalPoints = LocalPointData->GetPoints();
		return PCGAttributeAccessorKeys::GetKeys(PointIndices, InStart, OutObjects, [&LocalPoints](int32 Index) -> const void* { return &LocalPoints[Index]; });
	}
	else
	{
		return false;
	}
}

bool FPCGAttributeAccessorKeysPointsSubset::GetMetadataEntryKeys(int32 InStart, TArrayView<PCGMetadataEntryKey*> OutEntryKeys)
{
	if (Points.Num() > 0)
	{
		return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutEntryKeys, [](FPCGPoint* Point) -> PCGMetadataEntryKey* { return &(Point->MetadataEntry); });
	}
	else if (PointIndices.Num() && PointData)
	{
		TPCGValueRange<int64> MetadataEntryKeys = PointData->GetMetadataEntryValueRange();

		for (int32 Index = 0; Index < OutEntryKeys.Num(); ++Index)
		{
			const int32 PointIndex = (InStart + Index) % PointIndices.Num();
			OutEntryKeys[Index] = &MetadataEntryKeys[PointIndices[PointIndex]];
		}

		return true;
	}

	return false;
}

bool FPCGAttributeAccessorKeysPointsSubset::GetMetadataEntryKeys(int32 InStart, TArrayView<const PCGMetadataEntryKey*> OutEntryKeys) const
{
	if (Points.Num() > 0)
	{
		return PCGAttributeAccessorKeys::GetKeys(Points, InStart, OutEntryKeys, [](const FPCGPoint* Point) -> const PCGMetadataEntryKey* { return &(Point->MetadataEntry); });
	}
	else if (PointIndices.Num() > 0 && PointData)
	{
		const TConstPCGValueRange<int64> MetadataEntryKeys = PointData->GetConstMetadataEntryValueRange();

		for (int32 Index = 0; Index < OutEntryKeys.Num(); ++Index)
		{
			const int32 PointIndex = (InStart + Index) % PointIndices.Num();
			OutEntryKeys[Index] = &MetadataEntryKeys[PointIndices[PointIndex]];
		}

		return true;
	}

	return false;
}

/////////////////////////////////////////////////////////
template<typename PolyLineType>
FPCGAttributeAccessorKeysPolyLineData<PolyLineType>::FPCGAttributeAccessorKeysPolyLineData()
	: FPCGAttributeAccessorKeysSingleObjectPtr<PolyLineType>()
{}

template<typename PolyLineType>
FPCGAttributeAccessorKeysPolyLineData<PolyLineType>::FPCGAttributeAccessorKeysPolyLineData(PolyLineType* InPtr, bool bInGlobalData)
	: FPCGAttributeAccessorKeysSingleObjectPtr<PolyLineType>(InPtr)
	, bGlobalData(bInGlobalData)
{}

template<typename PolyLineType>
FPCGAttributeAccessorKeysPolyLineData<PolyLineType>::FPCGAttributeAccessorKeysPolyLineData(const PolyLineType* InPtr, bool bInGlobalData)
	: FPCGAttributeAccessorKeysSingleObjectPtr<PolyLineType>(InPtr)
	, bGlobalData(bInGlobalData)
{}

template<typename PolyLineType>
int32 FPCGAttributeAccessorKeysPolyLineData<PolyLineType>::GetNum() const
{
	if (!this->Ptr)
	{
		return 0; 
	}
	else if (bGlobalData)
	{
		return 1;
	}
	else
	{
		return this->Ptr->GetNumVertices();
	}
}

// Instantiate templated classes
template class FPCGAttributeAccessorKeysPolyLineData<UPCGSplineData>;
template class FPCGAttributeAccessorKeysPolyLineData<UPCGPolygon2DData>;

/////////////////////////////////////////////////////////
template<typename PolyLineType>
FPCGAttributeAccessorKeysPolyLineDataEntries<PolyLineType>::FPCGAttributeAccessorKeysPolyLineDataEntries(const PolyLineType* InPolyLineData)
	: Ptr(InPolyLineData)
{
	check(InPolyLineData);

	bIsReadOnly = true;
	// const_cast the view
	TConstArrayView<PCGMetadataEntryKey> ConstEntries = InPolyLineData->GetConstVerticesEntryKeys();
	Entries = MakeArrayView(const_cast<PCGMetadataEntryKey*>(ConstEntries.GetData()), ConstEntries.Num());
	
	if (Entries.IsEmpty())
	{
		ExtractedEntries = { PCGInvalidEntryKey };
		Entries = MakeArrayView(ExtractedEntries);
	}
}

template<typename PolyLineType>
FPCGAttributeAccessorKeysPolyLineDataEntries<PolyLineType>::FPCGAttributeAccessorKeysPolyLineDataEntries(PolyLineType* InPolyLineData)
	: Ptr(InPolyLineData)
{
	check(InPolyLineData);
	
	bIsReadOnly = false;
	InPolyLineData->AllocateMetadataEntries();
	Entries = InPolyLineData->GetMutableVerticesEntryKeys();
}

template<typename PolyLineType>
int32 FPCGAttributeAccessorKeysPolyLineDataEntries<PolyLineType>::GetNum() const
{
	return Ptr ? Ptr->GetNumVertices() : 0;
}

// Instantiate templated classes
template class FPCGAttributeAccessorKeysPolyLineDataEntries<UPCGSplineData>;
template class FPCGAttributeAccessorKeysPolyLineDataEntries<UPCGPolygon2DData>;