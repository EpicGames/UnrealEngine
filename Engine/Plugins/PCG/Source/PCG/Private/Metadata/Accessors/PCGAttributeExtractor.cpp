// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/Accessors/PCGAttributeExtractor.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"
#include "Metadata/Accessors/PCGCustomAccessor.h"

#include "Internationalization/Regex.h"

namespace PCGAttributeExtractor
{
	template <typename VectorType>
	double GetAt(const VectorType& Value, int32 Index)
	{
		check(Index >= 0 && Index < 4);
		return Value[Index];
	}

	template <>
	double GetAt<FQuat>(const FQuat& Value, int32 Index)
	{
		check(Index >= 0 && Index < 4);
		switch (Index)
		{
		case 0:
			return Value.X;
		case 1:
			return Value.Y;
		case 2:
			return Value.Z;
		default:
			return Value.W;
		}
	}

	template <typename VectorType>
	void SetAt(VectorType& Value, double In, int32 Index)
	{
		check(Index >= 0 && Index < 4);
		Value[Index] = In;
	}

	template <>
	void SetAt<FQuat>(FQuat& Value, double In, int32 Index)
	{
		check(Index >= 0 && Index < 4);
		switch (Index)
		{
		case 0:
			Value.X = In;
			break;
		case 1:
			Value.Y = In;
			break;
		case 2:
			Value.Z = In;
			break;
		default:
			Value.W = In;
			break;
		}
	}

	// Works for Vec2, Vec3, Vec4 and Quat (same as Vec4)
	template <typename VectorType>
	TUniquePtr<IPCGAttributeAccessor> CreateVectorExtractor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess)
	{
		if ((Name == PCGAttributeExtractorConstants::VectorLength) || (Name == PCGAttributeExtractorConstants::VectorSize))
		{
			bOutSuccess = true;
			return MakeUnique<FPCGChainAccessor<double, VectorType>>(std::move(InAccessor),
				[](const VectorType& Value) -> double { return Value.Size(); });
		}
		else if (Name == PCGAttributeExtractorConstants::VectorSquaredLength)
		{
			bOutSuccess = true;
			return MakeUnique<FPCGChainAccessor<double, VectorType>>(std::move(InAccessor),
				[](const VectorType& Value) -> double { return Value.SizeSquared(); });
		}
		else if (Name == PCGAttributeExtractorConstants::VectorNormalized)
		{
			if constexpr (std::is_same_v<FQuat, VectorType>)
			{
				bOutSuccess = true;
				return MakeUnique<FPCGChainAccessor<VectorType, VectorType>>(std::move(InAccessor),
					[](const VectorType& Value) -> VectorType { return Value.GetNormalized(); });
			}
			else
			{
				bOutSuccess = true;
				return MakeUnique<FPCGChainAccessor<VectorType, VectorType>>(std::move(InAccessor),
					[](const VectorType& Value) -> VectorType { return Value.GetSafeNormal(); });
			}
		}

		const FString NameStr = Name.ToString();
		const int32 NameLen = NameStr.Len();

		// Name should have at least 1 char and max 4
		if (NameLen <= 0 || NameLen > 4)
		{
			// Failed
			bOutSuccess = false;
			return InAccessor;
		}

		// We should match only X,Y,Z or W, for the whole name.
		// We also support R, G, B and A. But we cannot mix both.
		FString MatchStringXYZW;
		FString MatchStringRGBA;
		if constexpr (std::is_same_v<FVector2D, VectorType>)
		{
			MatchStringXYZW = FString::Printf(TEXT("[XY]{%d}"), NameLen);
			MatchStringRGBA = FString::Printf(TEXT("[RG]{%d}"), NameLen);
		}
		else if constexpr (std::is_same_v<FVector, VectorType>)
		{
			MatchStringXYZW = FString::Printf(TEXT("[XYZ]{%d}"), NameLen);
			MatchStringRGBA = FString::Printf(TEXT("[RGB]{%d}"), NameLen);
		}
		else
		{
			MatchStringXYZW = FString::Printf(TEXT("[XYZW]{%d}"), NameLen);
			MatchStringRGBA = FString::Printf(TEXT("[RGBA]{%d}"), NameLen);
		}

		const FRegexPattern RegexPatternXYZW(MatchStringXYZW, ERegexPatternFlags::CaseInsensitive);
		const FRegexPattern RegexPatternRGBA(MatchStringRGBA, ERegexPatternFlags::CaseInsensitive);

		TArray<int32, TInlineAllocator<4>> Indexes;

		FRegexMatcher RegexMatcherXYZW(RegexPatternXYZW, *NameStr);
		FRegexMatcher RegexMatcherRGBA(RegexPatternRGBA, *NameStr);

		if (RegexMatcherXYZW.FindNext() || RegexMatcherRGBA.FindNext())
		{
			for (const TCHAR Char : NameStr)
			{
				if (Char == 'R' || Char == 'r' || Char == 'X' || Char == 'x')
				{
					Indexes.Add(0);
				}
				else if (Char == 'G' || Char == 'g' || Char == 'Y' || Char == 'y')
				{
					Indexes.Add(1);
				}
				else if (Char == 'B' || Char == 'b' || Char == 'Z' || Char == 'z')
				{
					Indexes.Add(2);
				}
				else if (Char == 'A' || Char == 'a' || Char == 'W' || Char == 'w')
				{
					Indexes.Add(3);
				}
				else // Safeguard, should be caught by the regex
				{
					ensure(false);
					bOutSuccess = false;
					return InAccessor;
				}
			}
		}
		else
		{
			// Failed
			bOutSuccess = false;
			return InAccessor;
		}

		bOutSuccess = true;

		if (Indexes.Num() == 1)
		{
			return MakeUnique<FPCGChainAccessor<double, VectorType>>(std::move(InAccessor),
				[Indexes](const VectorType& Value) -> double { return GetAt(Value, Indexes[0]); },
				[Indexes](VectorType& Value, const double& In) -> void { SetAt(Value, In, Indexes[0]); });
		}
		else if (Indexes.Num() == 2)
		{
			return MakeUnique<FPCGChainAccessor<FVector2D, VectorType>>(std::move(InAccessor),
				[Indexes](const VectorType& Value) -> FVector2D { return FVector2D(GetAt(Value, Indexes[0]), GetAt(Value, Indexes[1])); },
				[Indexes](VectorType& Value, const FVector2D& In) -> void { SetAt(Value, In.X, Indexes[0]); SetAt(Value, In.Y, Indexes[1]);});
		}
		else if (Indexes.Num() == 3)
		{
			return MakeUnique<FPCGChainAccessor<FVector, VectorType>>(std::move(InAccessor),
				[Indexes](const VectorType& Value) -> FVector { return FVector(GetAt(Value, Indexes[0]), GetAt(Value, Indexes[1]), GetAt(Value, Indexes[2])); },
				[Indexes](VectorType& Value, const FVector& In) -> void { SetAt(Value, In.X, Indexes[0]); SetAt(Value, In.Y, Indexes[1]); SetAt(Value, In.Z, Indexes[2]); });
		}
		else
		{
			return MakeUnique<FPCGChainAccessor<FVector4, VectorType>>(std::move(InAccessor),
				[Indexes](const VectorType& Value) -> FVector4 { return FVector4(GetAt(Value, Indexes[0]), GetAt(Value, Indexes[1]), GetAt(Value, Indexes[2]), GetAt(Value, Indexes[3])); },
				[Indexes](VectorType& Value, const FVector4& In) -> void { SetAt(Value, In.X, Indexes[0]); SetAt(Value, In.Y, Indexes[1]); SetAt(Value, In.Z, Indexes[2]); SetAt(Value, In.W, Indexes[3]); });
		}
	}

	TUniquePtr<IPCGAttributeAccessor> CreateRotatorExtractor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess)
	{
		bOutSuccess = true;

		if (Name == PCGAttributeExtractorConstants::RotatorPitch)
		{
			return MakeUnique<FPCGChainAccessor<double, FRotator>>(std::move(InAccessor),
				[](const FRotator& Value) -> double { return Value.Pitch; },
				[](FRotator& Value, const double& In) -> void { Value.Pitch = In; });
		}

		if (Name == PCGAttributeExtractorConstants::RotatorRoll)
		{
			return MakeUnique<FPCGChainAccessor<double, FRotator>>(std::move(InAccessor),
				[](const FRotator& Value) -> double { return Value.Roll; },
				[](FRotator& Value, const double& In) -> void { Value.Roll = In; });
		}

		if (Name == PCGAttributeExtractorConstants::RotatorYaw)
		{
			return MakeUnique<FPCGChainAccessor<double, FRotator>>(std::move(InAccessor),
				[](const FRotator& Value) -> double { return Value.Yaw; },
				[](FRotator& Value, const double& In) -> void { Value.Yaw = In; });
		}

		if (Name == PCGAttributeExtractorConstants::RotatorEulerAngles)
		{
			return MakeUnique<FPCGChainAccessor<FVector, FRotator>>(std::move(InAccessor),
				[](const FRotator& Value) -> FVector { return Value.Euler(); },
				[](FRotator& Value, const FVector& In) -> void { Value = FRotator::MakeFromEuler(In); });
		}

		// Read-only
		if (Name == PCGAttributeExtractorConstants::RotatorForward)
		{
			return MakeUnique<FPCGChainAccessor<FVector, FRotator>>(std::move(InAccessor),
				[](const FRotator& Value) -> FVector { return Value.Quaternion().GetForwardVector(); });
		}

		if (Name == PCGAttributeExtractorConstants::RotatorRight)
		{
			return MakeUnique<FPCGChainAccessor<FVector, FRotator>>(std::move(InAccessor),
				[](const FRotator& Value) -> FVector { return Value.Quaternion().GetRightVector(); });
		}

		if (Name == PCGAttributeExtractorConstants::RotatorUp)
		{
			return MakeUnique<FPCGChainAccessor<FVector, FRotator>>(std::move(InAccessor),
				[](const FRotator& Value) -> FVector { return Value.Quaternion().GetUpVector(); });
		}

		bOutSuccess = false;
		return InAccessor;
	}

	TUniquePtr<IPCGAttributeAccessor> CreateQuatExtractor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess)
	{
		bOutSuccess = true;

		if (Name == PCGAttributeExtractorConstants::RotatorPitch)
		{
			return MakeUnique<FPCGChainAccessor<double, FQuat>>(std::move(InAccessor),
				[](const FQuat& Value) -> double { return Value.Rotator().Pitch; },
				[](FQuat& Value, const double& In) -> void { FRotator Temp = Value.Rotator(); Temp.Pitch = In; Value = Temp.Quaternion(); });
		}

		if (Name == PCGAttributeExtractorConstants::RotatorRoll)
		{
			return MakeUnique<FPCGChainAccessor<double, FQuat>>(std::move(InAccessor),
				[](const FQuat& Value) -> double { return Value.Rotator().Roll; },
				[](FQuat& Value, const double& In) -> void { FRotator Temp = Value.Rotator(); Temp.Roll = In; Value = Temp.Quaternion(); });
		}

		if (Name == PCGAttributeExtractorConstants::RotatorYaw)
		{
			return MakeUnique<FPCGChainAccessor<double, FQuat>>(std::move(InAccessor),
				[](const FQuat& Value) -> double { return Value.Rotator().Yaw; },
				[](FQuat& Value, const double& In) -> void { FRotator Temp = Value.Rotator(); Temp.Yaw = In; Value = Temp.Quaternion(); });
		}

		if (Name == PCGAttributeExtractorConstants::RotatorEulerAngles)
		{
			return MakeUnique<FPCGChainAccessor<FVector, FQuat>>(std::move(InAccessor),
				[](const FQuat& Value) -> FVector { return Value.Euler(); },
				[](FQuat& Value, const FVector& In) -> void { Value = FQuat::MakeFromEuler(In); });
		}

		// Read-only
		if (Name == PCGAttributeExtractorConstants::RotatorForward)
		{
			return MakeUnique<FPCGChainAccessor<FVector, FQuat>>(std::move(InAccessor),
				[](const FQuat& Value) -> FVector { return Value.GetForwardVector(); });
		}

		if (Name == PCGAttributeExtractorConstants::RotatorRight)
		{
			return MakeUnique<FPCGChainAccessor<FVector, FQuat>>(std::move(InAccessor),
				[](const FQuat& Value) -> FVector { return Value.GetRightVector(); });
		}

		if (Name == PCGAttributeExtractorConstants::RotatorUp)
		{
			return MakeUnique<FPCGChainAccessor<FVector, FQuat>>(std::move(InAccessor),
				[](const FQuat& Value) -> FVector { return Value.GetUpVector(); });
		}

		// Quat also support vector extractor
		return CreateVectorExtractor<FQuat>(std::move(InAccessor), Name, bOutSuccess);
	}

	TUniquePtr<IPCGAttributeAccessor> CreateTransformExtractor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess)
	{
		bOutSuccess = true;

		if ((Name == PCGAttributeExtractorConstants::TransformLocation) || (Name == TEXT("Position")))
		{
			return MakeUnique<FPCGChainAccessor<FVector, FTransform>>(std::move(InAccessor),
				[](const FTransform& Value) -> FVector { return Value.GetLocation(); },
				[](FTransform& Value, const FVector& In) -> void { Value.SetLocation(In); });
		}

		if ((Name == PCGAttributeExtractorConstants::TransformScale) || (Name == TEXT("Scale3D")))
		{
			return MakeUnique<FPCGChainAccessor<FVector, FTransform>>(std::move(InAccessor),
				[](const FTransform& Value) -> FVector { return Value.GetScale3D(); },
				[](FTransform& Value, const FVector& In) -> void { Value.SetScale3D(In); });
		}

		if (Name == PCGAttributeExtractorConstants::TransformRotation)
		{
			return MakeUnique<FPCGChainAccessor<FQuat, FTransform>>(std::move(InAccessor),
				[](const FTransform& Value) -> FQuat { return Value.GetRotation(); },
				[](FTransform& Value, const FQuat& In) -> void { Value.SetRotation(In); });
		}

		bOutSuccess = false;
		return InAccessor;
	}

	template <typename StringType>
	TUniquePtr<IPCGAttributeAccessor> CreateStringExtractor(TUniquePtr<IPCGAttributeAccessor> InAccessor, const FName Name, bool& bOutSuccess)
	{
		bOutSuccess = true;

		if (Name == PCGAttributeExtractorConstants::StringLength)
		{
			if constexpr (std::is_same_v<StringType, FString>)
			{
				return MakeUnique<FPCGChainAccessor<int32, FString>>(MoveTemp(InAccessor),
					[](const FString& Value) -> int32 { return Value.Len(); });
			}
			else
			{
				return MakeUnique<FPCGChainAccessor<int32, StringType>>(MoveTemp(InAccessor),
					[](const StringType& Value) -> int32 { return Value.ToString().Len(); });
			}
		}

		bOutSuccess = false;
		return InAccessor;
	}

	// Template instantiation for all vectors types + quat
	template TUniquePtr<IPCGAttributeAccessor> CreateVectorExtractor<FVector2D>(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
	template TUniquePtr<IPCGAttributeAccessor> CreateVectorExtractor<FVector>(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
	template TUniquePtr<IPCGAttributeAccessor> CreateVectorExtractor<FVector4>(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
	template TUniquePtr<IPCGAttributeAccessor> CreateVectorExtractor<FQuat>(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
	template TUniquePtr<IPCGAttributeAccessor> CreateStringExtractor<FString>(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
	template TUniquePtr<IPCGAttributeAccessor> CreateStringExtractor<FName>(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
	template TUniquePtr<IPCGAttributeAccessor> CreateStringExtractor<FSoftObjectPath>(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
	template TUniquePtr<IPCGAttributeAccessor> CreateStringExtractor<FSoftClassPath>(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
}
