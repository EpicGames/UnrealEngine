// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/Accessors/PCGPropertyAccessor.h"

#include "Algo/AnyOf.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "UObject/EnumProperty.h"

#define LOCTEXT_NAMESPACE "PCGPropertyAccessor"

namespace PCGPropertyAccessor
{
	static constexpr const TCHAR* NullStr = TEXT("null");

	void LogErrorForInvalidObjectWrite(const UClass* PropertyClass, const UClass* ObjectClass, const FSoftObjectPath& Value)
	{
#if !NO_LOGGING
		PCGLog::LogErrorOnGraph(FText::Format(
			LOCTEXT("ObjectSetRangeError", "[FPCGPropertyObjectPtrAccessor::SetRange] Tried to write into an Object property of class \"{0}\" with an object of class \"{1}\" ({2}). This is invalid."),
			FText::FromString(PropertyClass ? PropertyClass->GetName() : NullStr),
			FText::FromString(ObjectClass ? ObjectClass->GetName() : NullStr),
			FText::FromString(Value.ToString())));
#endif // !NO_LOGGING
	}

	void LogErrorForInvalidClassWrite(const UClass* MetaClass, const UClass* ObjectMetaClass, const FSoftClassPath& Value)
	{
#if !NO_LOGGING
		PCGLog::LogErrorOnGraph(FText::Format(
			LOCTEXT("ClassSetRangeError", "[FPCGPropertyObjectPtrAccessor::SetRange] Tried to write into an Class property of metaclass \"{0}\" with a class of metaclass \"{1}\" ({2}). This is invalid."),
			FText::FromString(MetaClass ? MetaClass->GetName() : NullStr),
			FText::FromString(ObjectMetaClass ? ObjectMetaClass->GetName() : NullStr),
			FText::FromString(Value.ToString())));
#endif // !NO_LOGGING
	}
	
	void LogErrorForInvalidEnumWrite(const UEnum* Enum, const int64 Value)
	{
#if !NO_LOGGING
		PCGLog::LogErrorOnGraph(FText::Format(
			LOCTEXT("EnumSetRangeError", "[FPCGEnumPropertyAccessor::SetRange] Tried to write into an Enum \"{0}\" a value \"{1}\" that is not a valid value for this enum."),
			FText::FromString(Enum->GetName()),
			Value));
#endif // !NO_LOGGING
	}
}

IPCGPropertyChain::IPCGPropertyChain(const FProperty* Property, TArray<const FProperty*>&& ExtraProperties)
	: PropertyChain(std::move(ExtraProperties))
{
	// Fix property chain
	if (PropertyChain.IsEmpty() || PropertyChain.Last() != Property)
	{
		PropertyChain.Add(Property);
	}
}

const UStruct* IPCGPropertyChain::GetTopPropertyStruct() const
{
	if (!PropertyChain.IsEmpty() && PropertyChain[0])
	{
		return PropertyChain[0]->GetOwnerStruct();
	}
	else
	{
		return nullptr;
	}
}

FPCGEnumPropertyAccessor::FPCGEnumPropertyAccessor(const FEnumProperty* InProperty, TArray<const FProperty*>&& ExtraProperties)
	: Super(/*bInReadOnly=*/ false)
	, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
	, Property(InProperty)
{
	check(Property);

	const UEnum* Enum = Property->GetEnum();
	bHasMaxValue = Enum && Enum->ContainsExistingMax();
	EnumMaxValue = bHasMaxValue ? Enum->GetMaxEnumValue() : -1;
}


bool FPCGEnumPropertyAccessor::GetRangeImpl(TArrayView<int64> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
{
	return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> Type
	{
		return Property->GetUnderlyingProperty()->GetSignedIntPropertyValue(PropertyAddressData);
	});
}

bool FPCGEnumPropertyAccessor::SetRangeImpl(TArrayView<const int64> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
{
	bool bWriteOK = true;
	const UEnum* Enum = Property->GetEnum();

	const bool bIterateOK = PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this, &bWriteOK, Enum](void* PropertyAddressData, const int64& Value) -> void
	{
		if (Enum && (!Enum->IsValidEnumValue(Value) || (bHasMaxValue && Value == EnumMaxValue)))
		{
			PCGPropertyAccessor::LogErrorForInvalidEnumWrite(Enum, Value);
			bWriteOK = false;
			return;
		}
		
		Property->GetUnderlyingProperty()->SetIntPropertyValue(PropertyAddressData, Value);
	});

	return bIterateOK && bWriteOK;
}

FPCGPropertyGenericAccessor::FPCGPropertyGenericAccessor(const FProperty* InProperty, TArray<const FProperty*>&& ExtraProperties)
	: Super(/*bInReadOnly=*/ false, static_cast<int16>(EPCGMetadataTypes::Unknown))
	, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
	, Property(InProperty)
{
	check(Property);
	UnderlyingDesc = FPCGMetadataAttributeDesc::CreateFromProperty(InProperty);
	UnderlyingType = static_cast<int16>(UnderlyingDesc.ValueType);

	// Make sure we never create accessors that are not supported.
	check(UnderlyingDesc.IsValid());

	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		ObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner);
	}
	else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
	{
		ObjectProperty = CastField<FObjectProperty>(SetProperty->ElementProp);
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
	{
		KeyObjectProperty = CastField<FObjectProperty>(MapProperty->KeyProp);
		ObjectProperty = CastField<FObjectProperty>(MapProperty->ValueProp);
	}
	else
	{
		ObjectProperty = CastField<FObjectProperty>(Property);
	}

	ClassProperty = CastField<FClassProperty>(ObjectProperty);
	KeyClassProperty = CastField<FClassProperty>(KeyObjectProperty);

	bNeedSlowPath = ObjectProperty || KeyObjectProperty;
}

bool FPCGPropertyGenericAccessor::SupportsGet(const PCG::Private::FOutValues& OutValues) const
{
	using namespace PCG::Private;
	if (OutValues.IsType<FOutValuesByPtr>())
	{
		return UnderlyingDesc.IsValid() && UnderlyingDesc.IsSingleValue();
	}
	else
	{
		return Super::SupportsGet(OutValues);
	}
}
	
bool FPCGPropertyGenericAccessor::SupportsSet(const PCG::Private::FInValues& InValues) const
{
	using namespace PCG::Private;
	if (InValues.IsType<FInValuesByPtr>())
	{
		return UnderlyingDesc.IsValid() && UnderlyingDesc.IsSingleValue();
	}
	else
	{
		return Super::SupportsSet(InValues);
	}
}

bool FPCGPropertyGenericAccessor::GetRangeVirtual(PCG::Private::FOutValues OutValues, int32 Count, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
{
	TArray<const void*, TInlineAllocator<256>> ContainerKeys;
	PCGPropertyAccessor::GetContainerKeys(Index, Count, Keys, ContainerKeys);
	if (ContainerKeys.IsEmpty())
	{
		return false;
	}

	// Update the addresses of all. We want to ignore the last property to call the getter on the container if it is a single value and it cannot be memcopied.
	const bool bDataCanBeMemcpy = Property->HasAnyPropertyFlags(CPF_IsPlainOldData) && !Property->HasGetter();
	const bool bShouldOperateOnContainer = OutValues.IsType<PCG::Private::FOutValuesByValue>() && !bDataCanBeMemcpy;
	PCGPropertyAccessor::AddressOffset<const void*>(GetPropertyChain(), ContainerKeys, /*bIgnoreLastProperty=*/bShouldOperateOnContainer);
	
	// Validate that we have no null pointers
	if (Algo::AnyOf(ContainerKeys, [](const void* Addr) { return Addr == nullptr;}))
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NullPtrContainerGet", "Some resolved objects were not assigned (null pointers), some results are discarded"));
		return false;
	}

	return Visit([this, Count, &ContainerKeys, bDataCanBeMemcpy](auto&& OutValues) -> bool
	{
		using T = std::decay_t<decltype(OutValues)>;
		if constexpr (std::is_same_v<T, PCG::Private::FOutValuesByValue> || std::is_same_v<T, PCG::Private::FOutValuesByPtr>)
		{
			if (!UnderlyingDesc.IsSingleValue())
			{
				return false;
			}
			
			if constexpr(std::is_same_v<T, PCG::Private::FOutValuesByValue>)
			{
				uint8* RawOutValues = static_cast<uint8*>(OutValues.OutValues);
				const int32 StructSize = Property->GetElementSize();
				
				if (bDataCanBeMemcpy)
				{
					for (int32 i = 0; i < Count; ++i)
					{
						FMemory::Memcpy(RawOutValues + i * StructSize, ContainerKeys[i], StructSize);
					}
				}
				else
				{
					for (int32 i = 0; i < Count; ++i)
					{
						Property->GetValue_InContainer(ContainerKeys[i], RawOutValues + i * StructSize);
					}
				}
			}
			else // if constexpr (std::is_same_v<T, PCG::Private::FOutValuesByPtr>)
			{
				FMemory::Memcpy(OutValues.OutValues.GetData(), ContainerKeys.GetData(), sizeof(const void*) * Count);
			}

			return true;
		}
		else if constexpr (std::is_same_v<T, PCG::Private::FOutValuesAsArray>)
		{
			if (!UnderlyingDesc.IsArray())
			{
				return false;
			}
			
			const FArrayProperty* ArrayProperty = CastFieldChecked<const FArrayProperty>(Property);
			for (int32 i = 0; i < Count; ++i)
			{
				FScriptArrayHelper Helper(ArrayProperty, ContainerKeys[i]);
				OutValues.OutValues[i] = {Helper.GetRawPtr(), Helper.Num()};
			}
			
			return true;
		}
		else if constexpr (std::is_same_v<T, PCG::Private::FOutValuesAsSet>)
		{
			if (!UnderlyingDesc.IsSet())
			{
				return false;
			}
			
			for (int32 i = 0; i < Count; ++i)
			{
				new (OutValues.OutValues[i]) FScriptSetHelper(CastFieldChecked<const FSetProperty>(Property), ContainerKeys[i]);
			}
			
			return true;
		}
		else if constexpr (std::is_same_v<T, PCG::Private::FOutValuesAsMap>)
		{
			if (!UnderlyingDesc.IsMap())
			{
				return false;
			}
			
			for (int32 i = 0; i < Count; ++i)
			{
				new (OutValues.OutValues[i]) FScriptMapHelper(CastFieldChecked<const FMapProperty>(Property), ContainerKeys[i]);
			}
			
			return true;
		}
		else
		{
			static_assert(!std::is_same_v<T, T>, "Missing case in the visit");
			return false;
		}
	}, OutValues);
}

bool FPCGPropertyGenericAccessor::SetRangeVirtual(PCG::Private::FInValues InValues, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
{
	TArray<void*, TInlineAllocator<256>> ContainerKeys;
	PCGPropertyAccessor::GetContainerKeys(Index, Count, Keys, ContainerKeys);
	if (ContainerKeys.IsEmpty())
	{
		return false;
	}

	// Update the addresses of all. We want to ignore the last property to call the setter on the container if it is a single value and it can not be memcopied.
	const bool bDataCanBeMemcpy = Property->HasAnyPropertyFlags(CPF_IsPlainOldData) && !Property->HasSetter();
	const bool bShouldOperateOnContainer = InValues.IsType<PCG::Private::FInValuesByValue>() && !bDataCanBeMemcpy;
	PCGPropertyAccessor::AddressOffset<void*>(GetPropertyChain(), ContainerKeys, /*bIgnoreLastProperty=*/bShouldOperateOnContainer);
	
	// Validate that we have no null pointers
	if (Algo::AnyOf(ContainerKeys, [](const void* Addr) { return Addr == nullptr;}))
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NullPtrContainerSet", "Some resolved objects were not assigned (null pointers), some results are discarded"));
		return false;
	}

	return Visit([this, Count, &ContainerKeys, bDataCanBeMemcpy](auto&& InValues) -> bool
	{
		using T = std::decay_t<decltype(InValues)>;
		if constexpr (std::is_same_v<T, PCG::Private::FInValuesByValue>)
		{
			if (!UnderlyingDesc.IsSingleValue())
			{
				return false;
			}
			
			check(InValues.Count == Count);
			
			// Forced to go the slow path for objects
			if (bNeedSlowPath)
			{
				return SetRange_Objects_Values(InValues, ContainerKeys);
			}

			const uint8* RawInValues = static_cast<const uint8*>(InValues.InValues);
			const int32 StructSize = Property->GetElementSize();
			
			if (bDataCanBeMemcpy)
			{
				for (int32 i = 0; i < Count; ++i)
				{
					FMemory::Memcpy(ContainerKeys[i], RawInValues + i * StructSize, StructSize);
				}
			}
			else
			{
				for (int32 i = 0; i < Count; ++i)
				{
					Property->SetValue_InContainer(ContainerKeys[i], RawInValues + i * StructSize);
				}
			}

			return true;
		}
		else if constexpr (std::is_same_v<T, PCG::Private::FInValuesByPtr>)
		{
			if (!UnderlyingDesc.IsSingleValue())
			{
				return false;
			}

			check(InValues.InValues.Num() == Count);

			// Forced to go the slow path for objects
			if (bNeedSlowPath)
			{
				return SetRange_Objects_Ptr(InValues, ContainerKeys);
			}

			for (int32 i = 0; i < Count; ++i)
			{
				if (InValues.InValues[i])
				{
					Property->CopyCompleteValue(ContainerKeys[i], InValues.InValues[i]);
				}
			}

			return true;
		}
		else if constexpr (std::is_same_v<T, PCG::Private::FInValuesAsArray>)
		{
			if (!UnderlyingDesc.IsArray())
			{
				return false;
			}
			
			bool bSuccess = true;
			const FArrayProperty* ArrayProperty = CastFieldChecked<const FArrayProperty>(Property);
			for (int32 i = 0; i < Count; ++i)
			{
				FScriptArrayHelper Helper(ArrayProperty, ContainerKeys[i]);
				const int32 Num = InValues.InValues[i].template Get<1>();
				if (ArrayProperty->Inner->HasAnyPropertyFlags(CPF_IsPlainOldData))
				{
					Helper.EmptyAndAddUninitializedValues(Num);
				}
				else
				{
					Helper.EmptyAndAddValues(Num);
				}
				
				// Forced to go the slow path for objects
				if (bNeedSlowPath)
				{
					bSuccess &= SetRange_Objects_Arrays(Helper, InValues, i);
				}
				else
				{
					PCG::Private::CopyArray(ArrayProperty, Helper.GetRawPtr(), InValues.InValues[i].template Get<0>(), Num);
				}
			}
			
			return bSuccess;
		}
		else if constexpr (std::is_same_v<T, PCG::Private::FInValuesAsSet>)
		{
			if (!UnderlyingDesc.IsSet())
			{
				return false;
			}

			bool bSuccess = true;
			const FSetProperty* SetProperty = CastFieldChecked<const FSetProperty>(Property);
			for (int32 i = 0; i < Count; ++i)
			{
				FScriptSetHelper Helper(SetProperty, ContainerKeys[i]);
				const int32 Num = InValues.InValues[i].Num();
				Helper.EmptyElements(Num);

				// Forced to go the slow path for objects
				if (bNeedSlowPath)
				{
					bSuccess &= SetRange_Objects_Sets(Helper, InValues, i);
				}
				else
				{
					for (int j = 0; j < Num; ++j)
					{
						Helper.AddElement(InValues.InValues[i][j]);
					}
				}
			}
			
			return bSuccess;
		}
		else if constexpr (std::is_same_v<T, PCG::Private::FInValuesAsMap>)
		{
			if (!UnderlyingDesc.IsMap())
			{
				return false;
			}
			
			bool bSuccess = true;
			const FMapProperty* MapProperty = CastFieldChecked<const FMapProperty>(Property);
			for (int32 i = 0; i < Count; ++i)
			{
				FScriptMapHelper Helper(MapProperty, ContainerKeys[i]);
				const int32 Num = InValues.InValues[i].Num();
				Helper.EmptyValues(Num);

				// Forced to go the slow path for objects
				if (bNeedSlowPath)
				{
					bSuccess &= SetRange_Objects_Maps(Helper, InValues, i);
				}
				else
				{
					for (int j = 0; j < Num; ++j)
					{
						Helper.AddPair(InValues.InValues[i][j].Key, InValues.InValues[i][j].Value);
					}
				}
			}
						
			return bSuccess;
		}
		else if constexpr (std::is_same_v<T, PCG::Private::FInValuesSubset>)
		{
			// Subset is unsupported.
			return false;
		}
		else
		{
			static_assert(!std::is_same_v<T, T>, "Missing case in the visit");
			return false;
		}
	}, InValues);
}

bool FPCGPropertyGenericAccessor::SetRange_Objects_Values(const PCG::Private::FInValuesByValue& InValues, TArrayView<void*> ContainerKeys) const
{
	check(ObjectProperty);

	bool bSuccess = true;

	TConstArrayView<TObjectPtr<UObject>> ObjectValues = MakeConstArrayView(static_cast<const TObjectPtr<UObject>*>(InValues.InValues), InValues.Count);
	for (int32 i = 0; i < InValues.Count; ++i)
	{
		if (!ValidateValue(ObjectValues[i], /*bUseKeyProperty=*/false))
		{
			// It is fine to use the Object property as a class property is also an object property.
			ObjectProperty->SetValue_InContainer(ContainerKeys[i], TObjectPtr<UObject>{});
			bSuccess = false;
		}
		else
		{
			ObjectProperty->SetValue_InContainer(ContainerKeys[i], ObjectValues[i]);
		}
	}

	return bSuccess;
}

bool FPCGPropertyGenericAccessor::SetRange_Objects_Ptr(const PCG::Private::FInValuesByPtr& InValues, TArrayView<void*> ContainerKeys) const
{
	check(ObjectProperty);

	bool bSuccess = true;

	for (int32 i = 0; i < InValues.InValues.Num(); ++i)
	{
		const TObjectPtr<UObject>* ObjectValue = static_cast<const TObjectPtr<UObject>*>(InValues.InValues[i]);
		if (!ObjectValue || !ValidateValue(*ObjectValue, /*bUseKeyProperty=*/false))
		{
			// It is fine to use the Object property as a class property is also an object property.
			ObjectProperty->SetValue_InContainer(ContainerKeys[i], TObjectPtr<UObject>{});
			bSuccess = false;
		}
		else
		{
			ObjectProperty->SetValue_InContainer(ContainerKeys[i], *ObjectValue);
		}
	}

	return bSuccess;
}

bool FPCGPropertyGenericAccessor::SetRange_Objects_Arrays(FScriptArrayHelper& Helper, const PCG::Private::FInValuesAsArray& InValues, int32 Index) const
{
	check(ObjectProperty);

	bool bSuccess = true;

	auto [DataPtr, ArrayNum] = InValues.InValues[Index];
	TConstArrayView<TObjectPtr<UObject>> ObjectValues = MakeConstArrayView(static_cast<const TObjectPtr<UObject>*>(DataPtr), ArrayNum);
	for (int32 i = 0; i < ArrayNum; ++i)
	{
		if (!ValidateValue(ObjectValues[i], /*bUseKeyProperty=*/false))
		{
			ObjectProperty->SetValue_InContainer(Helper.GetElementPtr(i), TObjectPtr<UObject>{});
			bSuccess = false;
		}
		else
		{
			ObjectProperty->SetValue_InContainer(Helper.GetElementPtr(i), ObjectValues[i]);
		}
	}

	return bSuccess;
}

bool FPCGPropertyGenericAccessor::SetRange_Objects_Sets(FScriptSetHelper& Helper, const PCG::Private::FInValuesAsSet& InValues, int32 Index) const
{
	TObjectPtr<UObject> EmptyObject{};

	check(ObjectProperty);

	bool bSuccess = true;

	TArrayView<const void*> Values = InValues.InValues[Index];
	for (int32 i = 0; i < Values.Num(); ++i)
	{
		const TObjectPtr<UObject>* ObjectValue = static_cast<const TObjectPtr<UObject>*>(Values[i]);
		if (!ValidateValue(*ObjectValue, /*bUseKeyProperty=*/false))
		{
			// Null object would have already been added if success is false
			if (bSuccess)
			{
				Helper.AddElement(&EmptyObject);
			}

			bSuccess = false;
		}
		else
		{
			Helper.AddElement(ObjectValue);
		}
	}

	return bSuccess;
}

bool FPCGPropertyGenericAccessor::SetRange_Objects_Maps(FScriptMapHelper& Helper, const PCG::Private::FInValuesAsMap& InValues, int32 Index) const
{
	TObjectPtr<UObject> EmptyObject{};

	check(ObjectProperty || KeyObjectProperty);

	bool bSuccess = true;

	TArrayView<TPair<const void*, const void*>> Values = InValues.InValues[Index];
	for (int32 i = 0; i < Values.Num(); ++i)
	{
		auto [Key, Value] = Values[i];

		if ((ObjectProperty || ClassProperty) && !ValidateValue(*static_cast<const TObjectPtr<UObject>*>(Value), /*bUseKeyProperty=*/false))
		{
			Value = &EmptyObject;
			bSuccess = false;
		}

		if ((KeyObjectProperty || KeyClassProperty) && !ValidateValue(*static_cast<const TObjectPtr<UObject>*>(Key), /*bUseKeyProperty=*/true))
		{
			Key = &EmptyObject;
			bSuccess = false;
		}

		Helper.AddPair(Key, Value);
	}

	return bSuccess;
}

bool FPCGPropertyGenericAccessor::ValidateValue(const TObjectPtr<UObject>& InValue, const bool bUseKeyProperty) const
{
	const FObjectProperty* WorkingObjectProperty = bUseKeyProperty ? KeyObjectProperty : ObjectProperty;
	const FClassProperty* WorkingClassProperty = bUseKeyProperty ? KeyClassProperty : ClassProperty;

	if (InValue && WorkingClassProperty)
	{
		if (const UClass* Class = Cast<const UClass>(InValue.Get()); !Class || !Class->IsChildOf(WorkingClassProperty->MetaClass))
		{
			PCGPropertyAccessor::LogErrorForInvalidClassWrite(WorkingClassProperty->MetaClass, Class, InValue.GetPath());
			return false;
		}
	}
	else if (InValue && !InValue->GetClass()->IsChildOf(WorkingObjectProperty->PropertyClass))
	{
		PCGPropertyAccessor::LogErrorForInvalidObjectWrite(WorkingObjectProperty->PropertyClass, InValue->GetClass(), InValue.GetPath());
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
