// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGCustomVersion.h"
#include "PCGData.h"
#include "Helpers/PCGMetadataHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttribute.h"

#include "Serialization/ArchiveCrc32.h"
#include "String/ParseTokens.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributePropertySelector)

namespace PCGAttributePropertySelectorConstants
{
	static const TCHAR* PropertyPrefix = TEXT("$");
	static const TCHAR* ExtraSeparator = TEXT(".");
	static const TCHAR* DomainPrefix = TEXT("@");
	static const TCHAR PropertyPrefixChar = PropertyPrefix[0];
	static const TCHAR DomainPrefixChar = DomainPrefix[0];
	static const TCHAR ExtraSeparatorChar = ExtraSeparator[0];

	static const FString ExportTextLeftSentinel = TEXT("PCGBegin(");
	static const FString ExportTextRightSentinel = TEXT(")PCGEnd");
}

namespace PCGAttributePropertySelector
{
	bool IsReservedAttributeName(const FName& InName)
	{
		return InName == PCGMetadataAttributeConstants::LastAttributeName ||
			InName == PCGMetadataAttributeConstants::LastCreatedAttributeName ||
			InName == PCGMetadataAttributeConstants::SourceAttributeName ||
			InName == PCGMetadataAttributeConstants::SourceNameAttributeName;
	}
}

bool FPCGAttributePropertySelector::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FPCGCustomVersion::GUID);

	// Don't actually serialize, just write the custom version for PostSerialize
	return false;
}

void FPCGAttributePropertySelector::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.IsLoading() && Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Ar.CustomVer(FPCGCustomVersion::GUID) < FPCGCustomVersion::AttributePropertySelectorDeprecatePointProperties
			&& (Selection == EPCGAttributePropertySelection::PointProperty || Selection == EPCGAttributePropertySelection::Property))
		{
			SetPointProperty(PointProperty_DEPRECATED, /*bResetExtraNames=*/false);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITOR
}

bool FPCGAttributePropertySelector::ExportTextItem(FString& ValueStr, FPCGAttributePropertySelector const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	// String guarded by sentinels, don't use `"` because it can be used in the selector.
	TStringBuilder<256> StringBuilder;
	StringBuilder.Append(PCGAttributePropertySelectorConstants::ExportTextLeftSentinel);
	StringBuilder.Append(ToString());
	StringBuilder.Append(PCGAttributePropertySelectorConstants::ExportTextRightSentinel);

	ValueStr += StringBuilder.ToString();
	return true;
}

bool FPCGAttributePropertySelector::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	const FStringView BufferView(Buffer);

	using PCGAttributePropertySelectorConstants::ExportTextLeftSentinel;
	using PCGAttributePropertySelectorConstants::ExportTextRightSentinel;

	// Look for the first occurence of the left and right sentinel
	int32 Start = BufferView.Find(ExportTextLeftSentinel);
	const int32 End = BufferView.Find(ExportTextRightSentinel);

	if (Start == INDEX_NONE || End == INDEX_NONE)
	{
		// Didn't find our sentinels, abort
		return false;
	}

	// Offset our start accounting the size of the left sentinel
	Start += ExportTextLeftSentinel.Len();

	Update(FString(BufferView.SubStr(Start, End - Start)));

	// Offset buffer to the end of the right sentinel.
	Buffer += (End + ExportTextRightSentinel.Len());
	return true;
}

FName FPCGAttributePropertySelector::GetName() const
{
	switch (GetSelection())
	{
	case EPCGAttributePropertySelection::ExtraProperty:
	{
		if (const UEnum* EnumPtr = StaticEnum<EPCGExtraProperties>())
		{
			return FName(EnumPtr->GetNameStringByValue((int64)ExtraProperty));
		}
		else
		{
			return NAME_None;
		}
	}
	case EPCGAttributePropertySelection::Attribute:
	{
		return GetAttributeName();
	}
	case EPCGAttributePropertySelection::Property:
	{
		return GetPropertyName();
	}
	default:
		return NAME_None;
	}
}

bool FPCGAttributePropertySelector::SetAttributeName(FName InAttributeName, bool bResetExtraNames)
{
	bool bHasChanged = false;
	if (bResetExtraNames)
	{
		bHasChanged |= ResetExtraNames();
	}

	if (Selection != EPCGAttributePropertySelection::Attribute || GetAttributeName() != InAttributeName)
	{
		Selection = EPCGAttributePropertySelection::Attribute;
		AttributeName = InAttributeName;
		bHasChanged = true;
	}

	return bHasChanged;
}

bool FPCGAttributePropertySelector::SetDomainName(FName InDomainName, bool bResetExtraNames)
{
	bool bHasChanged = false;
	if (bResetExtraNames)
	{
		bHasChanged |= ResetExtraNames();
	}

	if (GetDomainName() != InDomainName)
	{
		DomainName = InDomainName;
		bHasChanged = true;
	}

	return bHasChanged;
}

bool FPCGAttributePropertySelector::SetPropertyName(FName InPropertyName, bool bResetExtraNames)
{
	bool bHasChanged = false;
	if (bResetExtraNames)
	{
		bHasChanged |= ResetExtraNames();
	}

	if (Selection != EPCGAttributePropertySelection::Property || GetPropertyName() != InPropertyName)
	{
		Selection = EPCGAttributePropertySelection::Property;
		PropertyName = InPropertyName;
		bHasChanged = true;
	}

	return bHasChanged;
}

bool FPCGAttributePropertySelector::SetPointProperty(EPCGPointProperties InPointProperty, bool bResetExtraNames)
{
	UEnum* EnumPtr = StaticEnum<EPCGPointProperties>();
	check(EnumPtr);
	
	const bool bHasChanged = SetPropertyName(*EnumPtr->GetNameStringByValue(static_cast<int64>(InPointProperty)), bResetExtraNames);

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Not doing this will break the CDO for all nodes that set this explicitly in their constructor.
	PointProperty_DEPRECATED = InPointProperty;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
	
	return bHasChanged;
}

bool FPCGAttributePropertySelector::SetExtraProperty(EPCGExtraProperties InExtraProperty, bool bResetExtraNames)
{
	bool bHasChanged = false;
	if (bResetExtraNames)
	{
		bHasChanged |= ResetExtraNames();
	}

	if (Selection != EPCGAttributePropertySelection::ExtraProperty || InExtraProperty != ExtraProperty)
	{
		Selection = EPCGAttributePropertySelection::ExtraProperty;
		ExtraProperty = InExtraProperty;
		bHasChanged = true;
	}

	return bHasChanged;
}

FString FPCGAttributePropertySelector::GetDomainString(bool bAddLeadingQualifier) const
{
	const FName Domain = GetDomainName();
	if (Domain == NAME_None)
	{
		return FString{};
	}
	
	if (bAddLeadingQualifier)
	{
		return FString(PCGAttributePropertySelectorConstants::DomainPrefix) + Domain.ToString();
	}
	else
	{
		return Domain.ToString();
	}
}

FString FPCGAttributePropertySelector::GetAttributePropertyString(bool bAddPropertyQualifier) const
{
	const FName Name = GetName();
	// Add a '$' if it is a property
	if (bAddPropertyQualifier && Selection != EPCGAttributePropertySelection::Attribute && Name != NAME_None)
	{
		return FString(PCGAttributePropertySelectorConstants::PropertyPrefix) + Name.ToString();
	}
	else
	{
		return Name.ToString();
	}
}

FString FPCGAttributePropertySelector::GetAttributePropertyAccessorsString(bool bAddLeadingSeparator) const
{
	if (!ExtraNames.IsEmpty())
	{
		FString LeadingSeparatorString;
		if (bAddLeadingSeparator)
		{
			LeadingSeparatorString = FString(PCGAttributePropertySelectorConstants::ExtraSeparator);
		}

		return LeadingSeparatorString + FString::Join(ExtraNames, PCGAttributePropertySelectorConstants::ExtraSeparator);
	}
	else
	{
		return FString();
	}
}

FString FPCGAttributePropertySelector::ToString(bool bSkipDomain) const
{
	const FString Domain = !bSkipDomain ? GetDomainString(/*bAddLeadingQualifier=*/true) : FString{};
	const FString Attribute = GetAttributePropertyString(/*bAddPropertyQualifier=*/true);
	const FString Accessors = GetAttributePropertyAccessorsString(/*bAddLeadingSeparator*/true);

	if (Domain.IsEmpty())
	{
		return Attribute + Accessors;
	}
	else
	{
		return Domain + PCGAttributePropertySelectorConstants::ExtraSeparator + Attribute + Accessors;
	}
}

bool FPCGAttributePropertySelector::operator==(const FPCGAttributePropertySelector& Other) const
{
	return IsSame(Other);
}

bool FPCGAttributePropertySelector::IsSame(const FPCGAttributePropertySelector& Other, bool bIncludeExtraNames) const
{
	if (GetSelection() != Other.GetSelection() || DomainName != Other.DomainName || (bIncludeExtraNames && ExtraNames != Other.ExtraNames))
	{
		return false;
	}

	switch (GetSelection())
	{
	case EPCGAttributePropertySelection::Attribute:
		return GetAttributeName() == Other.GetAttributeName();
	case EPCGAttributePropertySelection::Property:
		return GetPropertyName() == Other.GetPropertyName();
	case EPCGAttributePropertySelection::ExtraProperty:
		return ExtraProperty == Other.ExtraProperty;
	default:
		return false;
	}
}

void FPCGAttributePropertySelector::ImportFromOtherSelector(const FPCGAttributePropertySelector& InOther)
{
	Selection = InOther.GetSelection();
	DomainName = InOther.GetDomainName();

	switch (GetSelection())
	{
	case EPCGAttributePropertySelection::Attribute:
		SetAttributeName(InOther.GetAttributeName());
		break;
	case EPCGAttributePropertySelection::Property:
		SetPropertyName(InOther.GetPropertyName());
		break;
	case EPCGAttributePropertySelection::ExtraProperty:
		SetExtraProperty(InOther.GetExtraProperty());
		break;
	default:
		break;
	}

	ExtraNames = InOther.GetExtraNames();
}

bool FPCGAttributePropertySelector::IsValid() const
{
	const FName ThisAttributeName = GetName();
	static const FName EmptyName = TEXT("");

	if (!ExtraNames.IsEmpty() && ThisAttributeName == EmptyName)
	{
		return false;
	}

	return (Selection != EPCGAttributePropertySelection::Attribute) ||
		PCGAttributePropertySelector::IsReservedAttributeName(ThisAttributeName) ||
		FPCGMetadataAttributeBase::IsValidName(ThisAttributeName);
}

bool FPCGAttributePropertySelector::Reset()
{
	static const FPCGAttributePropertySelector EmptySelector{};
	const bool bHasChanged = EmptySelector != *this;
	*this = EmptySelector;
	return bHasChanged;
}

bool FPCGAttributePropertySelector::ResetExtraNames()
{
	if (!ExtraNames.IsEmpty())
	{
		ExtraNames.Empty();
		return true;
	}
	else
	{
		return false;
	}
}

bool FPCGAttributePropertySelector::Update(const FStringView NewValue)
{
	if (NewValue.IsEmpty())
	{
		return Reset();
	}
	
	TArray<FString> NewValues;
	UE::String::ParseTokens(
		NewValue,
		FStringView(PCGAttributePropertySelectorConstants::ExtraSeparator),
		[&NewValues](FStringView Token) { NewValues.Emplace(Token); },
		UE::String::EParseTokensOptions::IgnoreCase
	);

	check(!NewValues.IsEmpty())

	// TODO: If we ever have to support multiple domains, this has to change
	const bool bHasDomainName = NewValues[0].Len() > 1 && NewValues[0][0] == PCGAttributePropertySelectorConstants::DomainPrefixChar && !PCGAttributePropertySelector::IsReservedAttributeName(*NewValues[0]);
	const int32 PropertyIndex = bHasDomainName ? 1 : 0;
	const bool bHasProperty = NewValues.Num() > PropertyIndex && !NewValues[PropertyIndex].IsEmpty() && NewValues[PropertyIndex][0] == PCGAttributePropertySelectorConstants::PropertyPrefixChar;

	const int32 ExtraNameIndex = PropertyIndex + 1;
	
	TArray<FString> ExtraNamesTemp;
	if (NewValues.Num() > ExtraNameIndex)
	{
		ExtraNamesTemp.Append(&NewValues[ExtraNameIndex], NewValues.Num() - ExtraNameIndex);
	}

	bool bHasChanged = ExtraNamesTemp != ExtraNames;
	ExtraNames = MoveTemp(ExtraNamesTemp);

	const FName NewDomainName = bHasDomainName ? FName{*NewValues[0].RightChop(1)} : FName{NAME_None};
	bHasChanged |= SetDomainName(NewDomainName, /*bResetExtraNames=*/ false);

	if (NewValues.Num() <= PropertyIndex)
	{
		return SetAttributeName(NAME_None, /*bResetExtraNames=*/ false) || bHasChanged;
	}
	
	if (bHasProperty)
	{
		const FString NewNameWithoutPrefix = NewValues[PropertyIndex].RightChop(1);
		const UEnum* EnumPtr = StaticEnum<EPCGExtraProperties>();
		check(EnumPtr);
		
		const int32 Index = EnumPtr->GetIndexByNameString(NewNameWithoutPrefix);
		if (Index != INDEX_NONE)
		{
			return SetExtraProperty(static_cast<EPCGExtraProperties>(EnumPtr->GetValueByIndex(Index)), /*bResetExtraNames=*/ false) || bHasChanged;
		}
		else
		{
			return SetPropertyName(*NewNameWithoutPrefix, /*bResetExtraNames=*/ false) || bHasChanged;
		}
	}
	else
	{
		return SetAttributeName(*NewValues[PropertyIndex], /*bResetExtraNames=*/ false) || bHasChanged;
	}
}

bool FPCGAttributePropertySelector::Update(const char* NewValue)
{
	FString Temp(NewValue);
	return Update(MakeStringView(Temp));
}

void FPCGAttributePropertySelector::AddToCrc(FArchiveCrc32& Ar) const
{
	EPCGAttributePropertySelection RealSelection = GetSelection();
	Ar << RealSelection;
	Ar << const_cast<FName&>(DomainName);
	Ar << const_cast<TArray<FString>&>(ExtraNames);

	switch (RealSelection)
	{
	case EPCGAttributePropertySelection::Attribute:
		Ar << const_cast<FName&>(AttributeName);
		break;
	case EPCGAttributePropertySelection::Property:
		Ar << const_cast<FName&>(PropertyName);
		break;
	case EPCGAttributePropertySelection::ExtraProperty:
		Ar << ExtraProperty;
		break;
	default:
		break;
	}
}

uint32 GetTypeHash(const FPCGAttributePropertySelector& Selector)
{
	uint32 Hash = HashCombine(GetTypeHash(Selector.Selection), GetTypeHash(Selector.DomainName));
	
	switch (Selector.GetSelection())
	{
	case EPCGAttributePropertySelection::Attribute:
		Hash = HashCombine(Hash, GetTypeHash(Selector.AttributeName));
		break;
	case EPCGAttributePropertySelection::Property:
		Hash = HashCombine(Hash, GetTypeHash(Selector.PropertyName));
		break;
	case EPCGAttributePropertySelection::ExtraProperty:
		Hash = HashCombine(Hash, GetTypeHash(Selector.ExtraProperty));
		break;
	default:
		break;
	}

	for (const FString& ExtraName : Selector.ExtraNames)
	{
		Hash = HashCombine(Hash, GetTypeHash(ExtraName));
	}

	return Hash;
}

EPCGPointProperties FPCGAttributePropertySelector::GetPointProperty() const
{
	UEnum* EnumPtr = StaticEnum<EPCGPointProperties>();
	check(EnumPtr);
		
	const int32 Index = EnumPtr->GetIndexByName(PropertyName);
	if (Index != INDEX_NONE)
	{
		return static_cast<EPCGPointProperties>(EnumPtr->GetValueByIndex(Index));
	}
	else
	{
		return EPCGPointProperties::Invalid;
	}
}

bool FPCGAttributePropertySelector::IsBasicAttribute() const
{
	return Selection == EPCGAttributePropertySelection::Attribute && ExtraNames.IsEmpty();
}

///////////////////////////////////////////////////////////////////////

FPCGAttributePropertyInputSelector::FPCGAttributePropertyInputSelector()
{
	AttributeName = PCGMetadataAttributeConstants::LastAttributeName;
}

FPCGAttributePropertyInputSelector FPCGAttributePropertyInputSelector::CopyAndFixLast(const UPCGData* InData) const
{
	if (Selection == EPCGAttributePropertySelection::Attribute)
	{
		// For each case, append extra names to the newly created selector.
		if (AttributeName == PCGMetadataAttributeConstants::LastAttributeName && InData && InData->HasCachedLastSelector())
		{
			FPCGAttributePropertyInputSelector Selector = InData->GetCachedLastSelector();
			Selector.ExtraNames.Append(ExtraNames);
			return Selector;
		}
		else if (AttributeName == PCGMetadataAttributeConstants::LastCreatedAttributeName && InData)
		{
			if (const UPCGMetadata* Metadata = PCGMetadataHelpers::GetConstMetadata(InData))
			{
				FPCGAttributePropertyInputSelector Selector;
				Selector.SetAttributeName(Metadata->GetLatestAttributeNameOrNone());
				Selector.ExtraNames.Append(ExtraNames);
				return Selector;
			}
		}
	}

	return *this;
}

bool FPCGAttributePropertyInputSelector::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(FPCGAttributePropertySelector::StaticStruct()->GetFName()))
	{
		FPCGAttributePropertyInputSelector::StaticStruct()->SerializeItem(Slot, this, nullptr);
		return true;
	}
	else if (Tag.GetType().GetName() == NAME_NameProperty)
	{
		FName Value;
		Slot << Value;
		SetAttributeName(Value);
		return true;
	}

	return false;
}

void FPCGAttributePropertyInputSelector::ApplyDeprecation(int32 InPCGCustomVersion)
{
	if ((InPCGCustomVersion < FPCGCustomVersion::UpdateAttributePropertyInputSelector) && (Selection == EPCGAttributePropertySelection::Attribute) && (AttributeName == PCGMetadataAttributeConstants::LastAttributeName))
	{
		AttributeName = PCGMetadataAttributeConstants::LastCreatedAttributeName;
	}
}

///////////////////////////////////////////////////////////////////////

bool FPCGAttributePropertyOutputNoSourceSelector::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(FPCGAttributePropertySelector::StaticStruct()->GetFName()))
	{
		StaticStruct()->SerializeItem(Slot, this, nullptr);
		return true;
	}
	else if (Tag.GetType().GetName() == NAME_NameProperty)
	{
		FName Value;
		Slot << Value;
		SetAttributeName(Value);
		return true;
	}

	return false;
}

FPCGAttributePropertyOutputSelector::FPCGAttributePropertyOutputSelector()
{
	AttributeName = PCGMetadataAttributeConstants::SourceAttributeName;
}

FPCGAttributePropertyOutputSelector FPCGAttributePropertyOutputSelector::CopyAndFixSource(const FPCGAttributePropertyInputSelector* InSourceSelector, const UPCGData* InOptionalData) const
{
	if (Selection == EPCGAttributePropertySelection::Attribute)
	{
		// For each case, append extra names to the newly created selector.
		if (AttributeName == PCGMetadataAttributeConstants::SourceAttributeName && InSourceSelector)
		{
			FPCGAttributePropertyOutputSelector Selector = FPCGAttributePropertySelector::CreateFromOtherSelector<FPCGAttributePropertyOutputSelector>(*InSourceSelector);
			Selector.ExtraNames.Append(ExtraNames);
			return Selector;
		}
		else if (AttributeName == PCGMetadataAttributeConstants::SourceNameAttributeName && InSourceSelector)
		{
			FPCGAttributePropertyOutputSelector Selector;
			Selector.SetAttributeName(InSourceSelector->GetName());
			Selector.ExtraNames.Append(ExtraNames);
			return Selector;
		}
		// Only for deprecation
		else if (AttributeName == PCGMetadataAttributeConstants::LastCreatedAttributeName && InOptionalData)
		{
			if (const UPCGMetadata* Metadata = PCGMetadataHelpers::GetConstMetadata(InOptionalData))
			{
				FPCGAttributePropertyOutputSelector Selector;
				Selector.SetAttributeName(Metadata->GetLatestAttributeNameOrNone());
				Selector.ExtraNames.Append(ExtraNames);
				return Selector;
			}
		}
	}

	return *this;
}

///////////////////////////////////////////////////////////////////////

bool UPCGAttributePropertySelectorBlueprintHelpers::SetPointProperty(FPCGAttributePropertySelector& Selector, EPCGPointProperties InPointProperty, bool bResetExtraNames)
{
	return Selector.SetPointProperty(InPointProperty, bResetExtraNames);
}

bool UPCGAttributePropertySelectorBlueprintHelpers::SetAttributeName(FPCGAttributePropertySelector& Selector, FName InAttributeName, bool bResetExtraNames)
{
	return Selector.SetAttributeName(InAttributeName, bResetExtraNames);
}

bool UPCGAttributePropertySelectorBlueprintHelpers::SetPropertyName(FPCGAttributePropertySelector& Selector, FName InPropertyName, bool bResetExtraNames)
{
	return Selector.SetPropertyName(InPropertyName, bResetExtraNames);
}

bool UPCGAttributePropertySelectorBlueprintHelpers::SetDomainName(FPCGAttributePropertySelector& Selector, FName InDomainName, bool bResetExtraNames)
{
	return Selector.SetDomainName(InDomainName, bResetExtraNames);
}

bool UPCGAttributePropertySelectorBlueprintHelpers::SetExtraProperty(FPCGAttributePropertySelector& Selector, EPCGExtraProperties InExtraProperty, bool bResetExtraNames)
{
	return Selector.SetExtraProperty(InExtraProperty, bResetExtraNames);
}

EPCGAttributePropertySelection UPCGAttributePropertySelectorBlueprintHelpers::GetSelection(const FPCGAttributePropertySelector& Selector)
{
	return Selector.GetSelection();
}

EPCGPointProperties UPCGAttributePropertySelectorBlueprintHelpers::GetPointProperty(const FPCGAttributePropertySelector& Selector)
{
	return Selector.GetPointProperty();
}

FName UPCGAttributePropertySelectorBlueprintHelpers::GetAttributeName(const FPCGAttributePropertySelector& Selector)
{
	return Selector.GetAttributeName();
}

EPCGExtraProperties UPCGAttributePropertySelectorBlueprintHelpers::GetExtraProperty(const FPCGAttributePropertySelector& Selector)
{
	return Selector.GetExtraProperty();
}

const TArray<FString>& UPCGAttributePropertySelectorBlueprintHelpers::GetExtraNames(const FPCGAttributePropertySelector& Selector)
{
	return Selector.GetExtraNames();
}

FName UPCGAttributePropertySelectorBlueprintHelpers::GetPropertyName(const FPCGAttributePropertySelector& Selector)
{
	return Selector.GetPropertyName();
}

FName UPCGAttributePropertySelectorBlueprintHelpers::GetDomainName(const FPCGAttributePropertySelector& Selector)
{
	return Selector.GetDomainName();
}

FName UPCGAttributePropertySelectorBlueprintHelpers::GetName(const FPCGAttributePropertySelector& Selector)
{
	return Selector.GetName();
}

FPCGAttributePropertyInputSelector UPCGAttributePropertySelectorBlueprintHelpers::CopyAndFixLast(const FPCGAttributePropertyInputSelector& Selector, const UPCGData* InData)
{
	return Selector.CopyAndFixLast(InData);
}

FPCGAttributePropertyOutputSelector UPCGAttributePropertySelectorBlueprintHelpers::CopyAndFixSource(const FPCGAttributePropertyOutputSelector& OutputSelector, const FPCGAttributePropertyInputSelector& InputSelector, const UPCGData* InOptionalData)
{
	return OutputSelector.CopyAndFixSource(&InputSelector, InOptionalData);
}
