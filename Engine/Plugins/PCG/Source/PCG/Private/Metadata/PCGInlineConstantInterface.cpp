// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGInlineConstantInterface.h"

#include "PCGParamData.h"
#include "PCGSettings.h"
#include "Helpers/PCGMetadataHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGInlineConstantInterface)

#define LOCTEXT_NAMESPACE "PCGInlineConstantInterface"

namespace PCGInlineConstant::Helpers
{
	bool IsValidDefaultValueType(const EPCGMetadataTypes Type)
	{
		return Type != EPCGMetadataTypes::Unknown && PCGMetadataHelpers::MetadataTypeSupportsDefaultValues(Type);
	}

	FString GetZeroValueSerializedString(const EPCGMetadataTypes InType)
	{
		return PCGMetadataAttribute::CallbackWithRightType(static_cast<uint16>(InType), []<typename T>(const T&)
		{
			const T Value = PCG::Private::MetadataTraits<T>::ZeroValue();
			if constexpr (requires { TBaseStructure<T>::Get(); })
			{
				FString Result;
				TBaseStructure<T>::Get()->ExportText(Result, &Value, /*Defaults=*/nullptr, /*OwnerObject=*/nullptr, PPF_None, /*ExportRootScope=*/nullptr);
				return Result;
			}
			else
			{
				return LexToString(Value);
			}
		});
	}
}

bool FPCGInlineConstantState::DefaultValuesAreEnabled() const
{
	return true;
}

bool FPCGInlineConstantState::IsPinDefaultValueEnabled(const FName PinLabel) const
{
	return DefaultValueMap.Contains(PinLabel) && DefaultValues.FindProperty(PinLabel);
}

bool FPCGInlineConstantState::IsPinDefaultValueActivated(const FName PinLabel) const
{
	return IsPinDefaultValueEnabled(PinLabel) && DefaultValues.IsPropertyActivated(PinLabel);
}

EPCGMetadataTypes FPCGInlineConstantState::GetPinDefaultValueType(const FName PinLabel) const
{
	const FPCGPinDefaultValueInfo* DefaultValueInfo = DefaultValueMap.Find(PinLabel);
	return DefaultValueInfo ? DefaultValueInfo->CurrentType : EPCGMetadataTypes::Unknown;
}

const UPCGParamData* FPCGInlineConstantState::CreateDefaultValueParamData(FPCGContext* InContext, const FName PropertyKey, const FName AttributeName) const
{
	const FPCGPinDefaultValueInfo* DefaultValueInfo = DefaultValueMap.Find(PropertyKey);
	if (!DefaultValueInfo)
	{
		return nullptr;
	}

	if (!ensure(DefaultValues.GetCurrentPropertyType(PropertyKey) == DefaultValueInfo->CurrentType))
	{
		return nullptr;
	}

	return DefaultValues.CreateParamData(InContext, PropertyKey, AttributeName);
}

#if WITH_EDITOR
bool FPCGInlineConstantState::IsPinDefaultValueMetadataTypeValid(const FName PinLabel, const EPCGMetadataTypes DataType) const
{
	const FPCGPinDefaultValueInfo* DefaultValueInfo = DefaultValueMap.Find(PinLabel);
	return DefaultValueInfo && (DefaultValueInfo->AvailableTypes.IsEmpty() || DefaultValueInfo->AvailableTypes.Contains(DataType));
}

bool FPCGInlineConstantState::PinIsMapped(const FName PinLabel) const
{
	return DefaultValueMap.Contains(PinLabel);
}

EPCGMetadataTypes FPCGInlineConstantState::GetPinInitialDefaultValueType(const FName PinLabel) const
{
	const FPCGPinDefaultValueInfo* DefaultValueInfo = DefaultValueMap.Find(PinLabel);
	return DefaultValueInfo ? DefaultValueInfo->InitialType : EPCGMetadataTypes::Unknown;
}

FString FPCGInlineConstantState::GetPinDefaultValueAsString(const FName PinLabel) const
{
	if (PinLabel != NAME_None)
	{
		return DefaultValues.FindProperty(PinLabel) ? DefaultValues.GetPropertyValueAsString(PinLabel) : GetPinInitialDefaultValueString(PinLabel);
	}

	return FString{};
}

FString FPCGInlineConstantState::GetPinInitialDefaultValueString(const FName PinLabel) const
{
	if (const FPCGPinDefaultValueInfo* DefaultValueInfo = PinLabel != NAME_None ? DefaultValueMap.Find(PinLabel) : nullptr)
	{
		return PCGInlineConstant::Helpers::GetZeroValueSerializedString(DefaultValueInfo->InitialType);
	}

	return FString{};
}

EPCGSettingDefaultValueExtraFlags FPCGInlineConstantState::GetDefaultValueExtraFlags(const FName PinLabel) const
{
	const FPCGPinDefaultValueInfo* DefaultValueInfo = DefaultValueMap.Find(PinLabel);
	return DefaultValueInfo ? DefaultValueInfo->ExtraFlags : EPCGSettingDefaultValueExtraFlags::None;
}

bool FPCGInlineConstantState::AddDefaultValueToPin(const FName PinLabel, FPCGPinDefaultValueInfo::FInitParams InitParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGInlineConstantState::AddDefaultValueToPin);

	if (DefaultValueMap.Contains(PinLabel))
	{
		return false;
	}

	DefaultValues.CreateNewProperty(PinLabel, InitParams.InitialType);
	DefaultValues.SetPropertyActivated(PinLabel, InitParams.bStartActivated);
	DefaultValueMap.Add(PinLabel, FPCGPinDefaultValueInfo(MoveTemp(InitParams)));

	return true;
}

bool FPCGInlineConstantState::RemoveDefaultValueFromPin(const FName PinLabel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGInlineConstantState::RemoveDefaultValueFromPin);

	if (!DefaultValueMap.Contains(PinLabel))
	{
		return false;
	}

	DefaultValueMap.Remove(PinLabel);
	DefaultValues.RemoveProperty(PinLabel);

	return true;
}

bool FPCGInlineConstantState::ResetDefaultValues()
{
	bool bDefaultValuesReset = false;

	// Validate a reset is needed
	for (const TTuple<FName, FPCGPinDefaultValueInfo>& DefaultValue : DefaultValueMap)
	{
		if (DefaultValue.Key != NAME_None && DefaultValues.FindProperty(DefaultValue.Key))
		{
			bDefaultValuesReset = true;
			break;
		}
	}

	if (!bDefaultValuesReset)
	{
		return false;
	}

	for (const TTuple<FName, FPCGPinDefaultValueInfo>& DefaultValue : DefaultValueMap)
	{
		ResetDefaultValueInternal(DefaultValue.Key);
	}

	return true;
}

bool FPCGInlineConstantState::ResetDefaultValue(const FName PinLabel)
{
	if (PinLabel == NAME_None || !DefaultValues.FindProperty(PinLabel) || !DefaultValueMap.Contains(PinLabel))
	{
		return false;
	}

	ResetDefaultValueInternal(PinLabel);
	return true;
}

bool FPCGInlineConstantState::SetPinDefaultValue(const FName PinLabel, const FString& DefaultValue, const bool bCreateIfNeeded)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGInlineConstantState::SetPinDefaultValue);

	if (PinLabel == NAME_None)
	{
		return false;
	}

	const FProperty* Property = DefaultValues.FindProperty(PinLabel);
	if (Property && DefaultValue == DefaultValues.GetPropertyValueAsString(PinLabel))
	{
		return false;
	}

	if (!Property && bCreateIfNeeded)
	{
		const EPCGMetadataTypes Type = GetPinInitialDefaultValueType(PinLabel);
		DefaultValues.CreateNewProperty(PinLabel, Type);
	}

	return DefaultValues.SetPropertyValueFromString(PinLabel, DefaultValue);
}

bool FPCGInlineConstantState::ConvertPinDefaultValueMetadataType(const FName PinLabel, const EPCGMetadataTypes DataType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGInlineConstantState::ConvertPinDefaultValueMetadataType);

	if (!ensure(IsPinDefaultValueActivated(PinLabel)))
	{
		return false;
	}

	if (PinLabel == NAME_None || !IsPinDefaultValueMetadataTypeValid(PinLabel, DataType))
	{
		return false;
	}

	if (DefaultValues.ConvertPropertyType(PinLabel, DataType))
	{
		DefaultValueMap[PinLabel].CurrentType = DataType;
	}

	return true;
}

bool FPCGInlineConstantState::SetPinDefaultValueIsActivated(const FName PinLabel, const bool bIsActivated)
{
	if (!ensure(IsPinDefaultValueEnabled(PinLabel)))
	{
		return false;
	}

	if (DefaultValues.IsPropertyActivated(PinLabel) == bIsActivated)
	{
		return false;
	}

	DefaultValues.SetPropertyActivated(PinLabel, bIsActivated);

	return true;
}

void FPCGInlineConstantState::ResetDefaultValueInternal(const FName PinLabel)
{
	if (PinLabel != NAME_None && DefaultValues.FindProperty(PinLabel))
	{
		if (!ensure(DefaultValueMap.Contains(PinLabel)))
		{
			return;
		}

		DefaultValues.RemoveProperty(PinLabel);
		DefaultValues.CreateNewProperty(PinLabel, DefaultValueMap[PinLabel].InitialType);
		DefaultValueMap[PinLabel].CurrentType = DefaultValueMap[PinLabel].InitialType;
	}
}
#endif // WITH_EDITOR

const UPCGParamData* IPCGSettingsInlineConstant::CreateDefaultValueParamData(FPCGContext* InContext, const FName PropertyKey, const FName AttributeName) const
{
	const FPCGInlineConstantState* State = GetInlineConstantState();
	return State ? State->CreateDefaultValueParamData(InContext, PropertyKey, AttributeName) : nullptr;
}

bool IPCGSettingsInlineConstant::DefaultValuesAreEnabled() const
{
	const FPCGInlineConstantState* State = GetInlineConstantState();
	return State && State->DefaultValuesAreEnabled();
}

bool IPCGSettingsInlineConstant::IsPinDefaultValueEnabled(const FName PinLabel) const
{
	const FPCGInlineConstantState* State = GetInlineConstantState();
	if (State && State->IsPinDefaultValueEnabled(PinLabel))
	{
		return true;
	}

	// Unregistered pins are eligible if their initial type supports default values.
	return PCGInlineConstant::Helpers::IsValidDefaultValueType(GetPinInitialDefaultValueType(PinLabel));
}

bool IPCGSettingsInlineConstant::IsPinDefaultValueActivated(const FName PinLabel) const
{
	const FPCGInlineConstantState* State = GetInlineConstantState();
	if (State && State->IsPinDefaultValueEnabled(PinLabel))
	{
		return State->IsPinDefaultValueActivated(PinLabel);
	}

	// Treat unregistered pins as activated if their initial type supports default values.
	// First user edit materializes the entry in state via lazy init in SetPinDefaultValueIsActivated.
	return PCGInlineConstant::Helpers::IsValidDefaultValueType(GetPinInitialDefaultValueType(PinLabel));
}

EPCGMetadataTypes IPCGSettingsInlineConstant::GetPinDefaultValueType(const FName PinLabel) const
{
	const FPCGInlineConstantState* State = GetInlineConstantState();
	if (State)
	{
		const EPCGMetadataTypes Type = State->GetPinDefaultValueType(PinLabel);
		if (Type != EPCGMetadataTypes::Unknown)
		{
			return Type;
		}
	}

	// Use the initial type for unregistered pins.
	return GetPinInitialDefaultValueType(PinLabel);
}

#if WITH_EDITOR
bool IPCGSettingsInlineConstant::IsPinDefaultValueMetadataTypeValid(const FName PinLabel, const EPCGMetadataTypes DataType) const
{
	const FPCGInlineConstantState* State = GetInlineConstantState();
	return State && State->IsPinDefaultValueMetadataTypeValid(PinLabel, DataType);
}

void IPCGSettingsInlineConstant::ResetDefaultValues()
{
	if (FPCGInlineConstantState* State = GetMutableInlineConstantState())
	{
		if (UPCGSettings* Settings = Cast<UPCGSettings>(this))
		{
			Settings->Modify();

			if (State->ResetDefaultValues())
			{
				Settings->OnSettingsChangedDelegate.Broadcast(Settings, EPCGChangeType::Settings);
			}
		}
	}
}

void IPCGSettingsInlineConstant::ResetDefaultValue(const FName PinLabel)
{
	if (FPCGInlineConstantState* State = GetMutableInlineConstantState())
	{
		if (UPCGSettings* Settings = Cast<UPCGSettings>(this))
		{
			Settings->Modify();

			if (State->ResetDefaultValue(PinLabel))
			{
				Settings->OnSettingsChangedDelegate.Broadcast(Settings, EPCGChangeType::Settings);
			}
		}
	}
}

void IPCGSettingsInlineConstant::SetPinDefaultValue(const FName PinLabel, const FString& DefaultValue, const bool bCreateIfNeeded)
{
	if (FPCGInlineConstantState* State = GetMutableInlineConstantState())
	{
		UPCGSettings* Settings = Cast<UPCGSettings>(this);
		if (Settings)
		{
			Settings->Modify();
		}

		if (State->SetPinDefaultValue(PinLabel, DefaultValue, bCreateIfNeeded))
		{
			if (Settings)
			{
				Settings->OnSettingsChangedDelegate.Broadcast(Settings, EPCGChangeType::Settings);
			}
		}
	}
}

void IPCGSettingsInlineConstant::ConvertPinDefaultValueMetadataType(const FName PinLabel, const EPCGMetadataTypes DataType)
{
	if (FPCGInlineConstantState* State = GetMutableInlineConstantState())
	{
		UPCGSettings* Settings = Cast<UPCGSettings>(this);
		if (Settings)
		{
			Settings->Modify();
		}

		if (State->ConvertPinDefaultValueMetadataType(PinLabel, DataType))
		{
			if (Settings)
			{
				Settings->OnSettingsChangedDelegate.Broadcast(Settings, EPCGChangeType::Node | EPCGChangeType::Edge | EPCGChangeType::Settings);
			}
		}
	}
}

void IPCGSettingsInlineConstant::SetPinDefaultValueIsActivated(const FName PinLabel, const bool bIsActivated, const bool bDirtySettings)
{
	FPCGInlineConstantState* State = GetMutableInlineConstantState();
	if (!State)
	{
		return;
	}

	// Lazy initialization -- if the pin hasn't been registered in state yet, create it now.
	if (!State->PinIsMapped(PinLabel))
	{
		const EPCGMetadataTypes InitialType = GetPinInitialDefaultValueType(PinLabel);
		if (!PCGInlineConstant::Helpers::IsValidDefaultValueType(InitialType))
		{
			return;
		}

		FPCGPinDefaultValueInfo::FInitParams InitParams;
		InitParams.InitialType = InitialType;
		InitParams.bStartActivated = true;
		State->AddDefaultValueToPin(PinLabel, MoveTemp(InitParams));
	}

	if (ensure(State->IsPinDefaultValueEnabled(PinLabel)))
	{
		UPCGSettings* Settings = bDirtySettings ? Cast<UPCGSettings>(this) : nullptr;
		if (Settings)
		{
			Settings->Modify();
		}

		if (State->SetPinDefaultValueIsActivated(PinLabel, bIsActivated))
		{
			if (Settings)
			{
				Settings->OnSettingsChangedDelegate.Broadcast(Settings, EPCGChangeType::Node | EPCGChangeType::Edge | EPCGChangeType::Settings);
			}
		}
	}
}

FString IPCGSettingsInlineConstant::GetPinDefaultValueAsString(const FName PinLabel) const
{
	const FPCGInlineConstantState* State = GetInlineConstantState();
	if (State && State->IsPinDefaultValueEnabled(PinLabel))
	{
		return State->GetPinDefaultValueAsString(PinLabel);
	}

	// Zero value string for unregistered pins.
	const EPCGMetadataTypes InitialType = GetPinInitialDefaultValueType(PinLabel);
	if (PCGInlineConstant::Helpers::IsValidDefaultValueType(InitialType))
	{
		return PCGInlineConstant::Helpers::GetZeroValueSerializedString(InitialType);
	}

	return FString{};
}

FString IPCGSettingsInlineConstant::GetPinInitialDefaultValueString(const FName PinLabel) const
{
	const FPCGInlineConstantState* State = GetInlineConstantState();
	return State ? State->GetPinInitialDefaultValueString(PinLabel) : FString{};
}

EPCGSettingDefaultValueExtraFlags IPCGSettingsInlineConstant::GetDefaultValueExtraFlags(const FName PinLabel) const
{
	const FPCGInlineConstantState* State = GetInlineConstantState();
	return State ? State->GetDefaultValueExtraFlags(PinLabel) : EPCGSettingDefaultValueExtraFlags::None;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
