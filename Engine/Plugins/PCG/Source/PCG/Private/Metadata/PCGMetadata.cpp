// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGMetadata.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGData.h"
#include "PCGPoint.h"

#include "Algo/Transform.h"
#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadata)

////////////////////////////
// UPCGMetadata
////////////////////////////

template <typename Func>
void UPCGMetadata::FindFixOrCreateDomainInitializeParams(const FPCGMetadataInitializeParams& InParams, const FPCGMetadataDomainID DomainID, const FPCGMetadataDomain& OtherMetadataDomain, Func&& InFunc)
{
	// Will look for the DomainID in the "Mapping" array (that is expected to be an array of tuple with the first element being a DomainID)
	// If it finds it, it returns a pointer on the second element of the tuple, nullptr otherwise (like a Find function on a TMap)
	auto FindForDomain = []<typename T>(const FPCGMetadataDomainID& DomainID, const TArray<TTuple<FPCGMetadataDomainID, T>>& Mapping) -> const T*
	{
		if (const auto* It = Mapping.FindByPredicate([&DomainID](const auto& Item) { return Item.template Get<0>() == DomainID;}))
		{
			return &It->template Get<1>();
		}
		else
		{
			return nullptr;
		}
	};

	const FPCGMetadataDomainInitializeParams* Params = nullptr;
	FPCGMetadataDomainInitializeParams TempParams{nullptr};
	
	Params = FindForDomain(DomainID, InParams.DomainInitializeParams);

	// If we didn't find it, and it is the default domain, retry with default identifier.
	if (!Params && InParams.Parent->DefaultDomain == DomainID)
	{
		Params = FindForDomain(PCGMetadataDomainID::Default, InParams.DomainInitializeParams);
	}

	// If we didn't find it, and domain ID is default, retry with the default domain.
	if (!Params && DomainID.IsDefault())
	{
		Params = FindForDomain(InParams.Parent->DefaultDomain, InParams.DomainInitializeParams);
	}

	// Validate that the parenting is valid, if not update it
	if (!Params || !Params->Parent || Params->Parent != &OtherMetadataDomain)
	{
		if (Params)
		{
			TempParams = *Params;
			TempParams.Parent = &OtherMetadataDomain;
		}
		else
		{
			TempParams = FPCGMetadataDomainInitializeParams(&OtherMetadataDomain);
		}
		
		Params = &TempParams;
	}

	// If we have a mapping for domains, find it there.
	FPCGMetadataDomainID OtherDomain = DomainID;
	if (const FPCGMetadataDomainID* It = FindForDomain(DomainID, InParams.DomainMapping))
	{
		OtherDomain = *It;
	}

	if (FPCGMetadataDomain* MetadataDomain = FindOrCreateMetadataDomain(OtherDomain))
	{
		check(Params);
		InFunc(*MetadataDomain, *Params);
	}
}

template <typename Func, typename ...Args>
decltype(auto) UPCGMetadata::WithMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID, Func InFunc, Args&& ...InArgs)
{
	if (FPCGMetadataDomain* FoundMetadataDomain = GetMetadataDomain(InMetadataDomainID))
	{
		return Invoke(InFunc, *FoundMetadataDomain, std::forward<Args>(InArgs)...);
	}
	else
	{
		UE_LOGF(LogPCG, Error, "Failed to find MetadataDomain with id %ls", *InMetadataDomainID.DebugName.ToString());
		
		using ReturnType = std::invoke_result_t<Func, FPCGMetadataDomain&, Args...>;
		if constexpr (!std::is_same_v<ReturnType, void>)
		{
			return ReturnType{};
		}
	}
}

template <typename Func>
decltype(auto) UPCGMetadata::WithMetadataDomain_Lambda(const FPCGMetadataDomainID& InMetadataDomainID, Func InFunc)
{
	if (FPCGMetadataDomain* FoundMetadataDomain = GetMetadataDomain(InMetadataDomainID))
	{
		return InFunc(FoundMetadataDomain);
	}
	else
	{
		UE_LOGF(LogPCG, Error, "Failed to find domain with id %ls", *InMetadataDomainID.DebugName.ToString());
		
		using ReturnType = std::invoke_result_t<Func, FPCGMetadataDomain*>;
		if constexpr (!std::is_same_v<ReturnType, void>)
		{
			return ReturnType{};
		}
	}
}

template <typename Func, typename ...Args>
decltype(auto) UPCGMetadata::WithConstMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID, Func InFunc, Args&& ...InArgs) const
{
	if (const FPCGMetadataDomain* FoundMetadataDomain = GetConstMetadataDomain(InMetadataDomainID))
	{
		return Invoke(InFunc, *FoundMetadataDomain, std::forward<Args>(InArgs)...);
	}
	else
	{
		UE_LOGF(LogPCG, Error, "Failed to find domain with id %ls", *InMetadataDomainID.DebugName.ToString());
		
		using ReturnType = typename std::invoke_result_t<Func, FPCGMetadataDomain&, Args...>;
		if constexpr (!std::is_same_v<ReturnType, void>)
		{
			return ReturnType{};
		}
	}
}

template <typename Func>
void UPCGMetadata::ForEachValidUniqueConstMetadataDomain(Func InFunc) const
{
	for (const auto& It : MetadataDomains)
	{
		const FPCGMetadataDomainID DomainID = It.Key;
		if (DomainID.IsDefault())
		{
			continue;
		}
		
		const FPCGMetadataDomain* MetadataDomain = It.Value.Get();
		if (!MetadataDomain)
		{
			continue;
		}

		InFunc(It.Key, *MetadataDomain);
	}
}

template <typename Func>
void UPCGMetadata::ForEachValidUniqueMetadataDomain(Func InFunc)
{
	for (const auto& It : MetadataDomains)
	{
		const FPCGMetadataDomainID DomainID = It.Key;
		if (DomainID.IsDefault())
		{
			continue;
		}
		
		FPCGMetadataDomain* MetadataDomain = It.Value.Get();
		if (!MetadataDomain)
		{
			continue;
		}

		InFunc(It.Key, *MetadataDomain);
	}
}

FPCGMetadataDomain* UPCGMetadata::GetMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID)
{
	return InMetadataDomainID.IsValid() ? FindOrCreateMetadataDomain(InMetadataDomainID) : nullptr;
}

FPCGMetadataDomain* UPCGMetadata::GetMetadataDomainFromSelector(const FPCGAttributePropertySelector& InSelector)
{
	const UPCGData* OwnerData = Cast<UPCGData>(GetOuter());
	return OwnerData ? GetMetadataDomain(OwnerData->GetMetadataDomainIDFromSelector(InSelector)) : nullptr;
}

const FPCGMetadataDomain* UPCGMetadata::GetConstMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID) const
{
	const TSharedPtr<FPCGMetadataDomain>* FoundMetadataDomain = MetadataDomains.Find(InMetadataDomainID);
	return FoundMetadataDomain ? FoundMetadataDomain->Get() : nullptr;
}

const FPCGMetadataDomain* UPCGMetadata::GetConstMetadataDomainFromSelector(const FPCGAttributePropertySelector& InSelector) const
{
	const UPCGData* OwnerData = Cast<UPCGData>(GetOuter());
	return OwnerData ? GetConstMetadataDomain(OwnerData->GetMetadataDomainIDFromSelector(InSelector)) : nullptr;
}

FPCGMetadataDomain* UPCGMetadata::FindOrCreateMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID)
{
	TSharedPtr<FPCGMetadataDomain>* FoundMetadataDomain = MetadataDomains.Find(InMetadataDomainID);
	if (!FoundMetadataDomain)
	{
		return nullptr;
	}

	if (!FoundMetadataDomain->IsValid())
	{
		PCG::TScopeLock Lock(MetadataDomainsLock);
		
		FoundMetadataDomain = MetadataDomains.Find(InMetadataDomainID);
		if (!FoundMetadataDomain->IsValid())
		{
			CreateMetadataDomain_Unsafe(InMetadataDomainID);
		}
		
	}

	return FoundMetadataDomain->Get();
}

FPCGMetadataDomain* UPCGMetadata::CreateMetadataDomain_Unsafe(const FPCGMetadataDomainID& InMetadataDomainID)
{
	check(MetadataDomains.Contains(InMetadataDomainID) && !MetadataDomains[InMetadataDomainID].IsValid());
	if (!InMetadataDomainID.IsDefault() && InMetadataDomainID == DefaultDomain)
	{
		// Nothing to do
	}
	else if (!MetadataDomains[InMetadataDomainID].IsValid())
	{
		MetadataDomains[InMetadataDomainID] = MakeShared<FPCGMetadataDomain>(this, InMetadataDomainID);
	}
	
	return MetadataDomains[InMetadataDomainID].Get();
}

FPCGMetadataInitializeParams::FPCGMetadataInitializeParams(const UPCGMetadata* InParent, const TArray<PCGMetadataEntryKey>* InOptionalEntriesToCopy)
	: Parent(InParent)
{
	if (IsValid(InParent))
	{
		FPCGMetadataDomainInitializeParams& DefaultDomainParams = DomainInitializeParams.Emplace_GetRef(PCGMetadataDomainID::Default, FPCGMetadataDomainInitializeParams(InParent->GetConstDefaultMetadataDomain())).Get<1>();
	
		if (InOptionalEntriesToCopy)
		{
			DefaultDomainParams.OptionalEntriesToCopy.Emplace(PCGValueRangeHelpers::MakeConstValueRange(*InOptionalEntriesToCopy));
		}
	}
}

FPCGMetadataInitializeParams::FPCGMetadataInitializeParams(const UPCGMetadata* InParent,
	const TSet<FName>& InFilteredAttributes,
	EPCGMetadataFilterMode InFilterMode,
	EPCGStringMatchingOperator InMatchOperator,
	const TArray<PCGMetadataEntryKey>* InOptionalEntriesToCopy)
		: FPCGMetadataInitializeParams(InParent, InOptionalEntriesToCopy)
{
	// We added the default in the ctor just above. It will be the first element.
	FPCGMetadataDomainInitializeParams& DefaultDomainParams = DomainInitializeParams[0].Get<1>();
	DefaultDomainParams.FilterMode = InFilterMode;
	DefaultDomainParams.MatchOperator = InMatchOperator;
	if (!InFilteredAttributes.IsEmpty())
	{
		DefaultDomainParams.FilteredAttributes = InFilteredAttributes;
	}
}

void FPCGMetadataInitializeParams::PopulateDomainInitializeParamsFromParent()
{
	if (!Parent)
	{
		return;
	}

	// -1 to remove the default
	DomainInitializeParams.Empty(Parent->MetadataDomains.Num() - 1);

	for (const auto& It : Parent->MetadataDomains)
	{
		if (It.Key.IsDefault() || !It.Value)
		{
			continue;
		}
		
		DomainInitializeParams.Emplace(It.Key, FPCGMetadataDomainInitializeParams(It.Value.Get()));
	}
}

UPCGMetadata::UPCGMetadata(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Always initialize the default domain
	SetupDomain(PCGMetadataDomainID::Default, /*bIsDefault=*/true);
	CreateMetadataDomain_Unsafe(PCGMetadataDomainID::Default);
}

void UPCGMetadata::SetupDomain(FPCGMetadataDomainID DomainID, bool bIsDefault)
{
	check(!MetadataDomains.Contains(DomainID));
	if (bIsDefault && !DomainID.IsDefault())
	{
		check(MetadataDomains.Contains(PCGMetadataDomainID::Default) && MetadataDomains[PCGMetadataDomainID::Default].IsValid());
		// If the default domain is not Default, we need to change it
		if (!DefaultDomain.IsDefault())
		{
			check(MetadataDomains.Contains(DefaultDomain) && MetadataDomains[DefaultDomain] == MetadataDomains[PCGMetadataDomainID::Default]);
			MetadataDomains[DefaultDomain].Reset();
		}
		
		DefaultDomain = DomainID;
		// Map the default domain to the DefaultID domain and update domain to the not-default domain ID.
		TSharedPtr<FPCGMetadataDomain> DefaultIDLayer = MetadataDomains[PCGMetadataDomainID::Default];
		MetadataDomains.Emplace(DomainID, DefaultIDLayer)->DomainID = DomainID;
		// Make sure to update the cached value for allow for multiple entries, as it might not have been setup correctly.
		DefaultIDLayer->bSupportMultiEntries = MetadataDomainSupportsMultiEntries(DomainID);
	}
	else
	{
		MetadataDomains.Emplace(DomainID);
	}
}

void UPCGMetadata::SetupDomainsFromPCGDataType(const TSubclassOf<UPCGData>& PCGDataType)
{
	const UPCGData* CDO = *PCGDataType ? CastChecked<const UPCGData>(PCGDataType->GetDefaultObject()) : nullptr;
	if (CDO)
	{
		SetupDomainsFromOtherMetadataIfNeeded(CDO->ConstMetadata());
	}
}

void UPCGMetadata::Serialize(FArchive& InArchive)
{
	LLM_SCOPE_BYTAG(PCG);
	Super::Serialize(InArchive);

	InArchive.UsingCustomVersion(FPCGCustomVersion::GUID);

	TArray<FPCGMetadataDomainID> DomainIDs;
	
	if (InArchive.IsLoading())
	{
		if (InArchive.CustomVer(FPCGCustomVersion::GUID) < FPCGCustomVersion::MultiLevelMetadata)
		{
			const UPCGData* Outer = Cast<const UPCGData>(GetOuter());
			FPCGMetadataDomain* DefaultMetadataDomain = GetDefaultMetadataDomain();
			check(DefaultMetadataDomain);
			DefaultMetadataDomain->Serialize(InArchive);
		}
		else
		{
			FPCGMetadataDomainID ArchiveDefaultDomain;
			InArchive << ArchiveDefaultDomain;

			if (ArchiveDefaultDomain != DefaultDomain)
			{
				UE_LOGF(LogPCG, Warning, "Mismatch between default metadata domains while loading. "
								 "You should make sure to update the serialized metadata with the new default. "
								 "Serialized domain: %ls ; Current domain: %ls",
								 *ArchiveDefaultDomain.DebugName.ToString(), *DefaultDomain.DebugName.ToString());
			}
			
			InArchive << DomainIDs;
			
			for (const FPCGMetadataDomainID& DomainID : DomainIDs)
			{
				bool bIsValid = false;
				InArchive << bIsValid;
				
				// If the domain is the default domain, nothing to do.
				if (!DomainID.IsDefault())
				{
					FPCGMetadataDomain* NewMetadataDomain = FindOrCreateMetadataDomain(DomainID);
					check(NewMetadataDomain);
					if (bIsValid)
					{
						NewMetadataDomain->Serialize(InArchive);
					}
				}
			}
		}
	}
	else
	{
		MetadataDomains.GetKeys(DomainIDs);
		// Sort the domains to always serialize in the same order.
		DomainIDs.Sort();

		InArchive << DefaultDomain;
		InArchive << DomainIDs;
		for (const FPCGMetadataDomainID& DomainID : DomainIDs)
		{
			bool bIsValid = MetadataDomains[DomainID].IsValid();
			InArchive << bIsValid;

			// Don't serialize the default domain
			if (bIsValid &&  !DomainID.IsDefault())
			{
				MetadataDomains[DomainID]->Serialize(InArchive);
			}
		}
	}
}

void UPCGMetadata::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMetadata::GetResourceSizeEx);
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Note: we will not gather memory from the parent, but only "local" memory here
	// This might report as not quite all the memory we need (in the context of the cache, for example)
	// but it will not also grossly over-evalute the memory needed.
	// Gather from domains
	ForEachValidUniqueConstMetadataDomain([&CumulativeResourceSize](FPCGMetadataDomainID DomainID, const FPCGMetadataDomain& Domain)
	{
		Domain.GetResourceSizeEx(CumulativeResourceSize);
	});
}

FPCGMetadataAttributeBase* UPCGMetadata::CreateAttribute(FPCGAttributeIdentifier AttributeName, const FPCGMetadataAttributeDesc& AttributeDesc, bool bAllowsInterpolation, bool bOverrideParent)
{
	FPCGMetadataDomain* FoundSubMetadata = FindOrCreateMetadataDomain(AttributeName.MetadataDomain);
	return FoundSubMetadata ? FoundSubMetadata->CreateAttribute(AttributeDesc, bAllowsInterpolation, bOverrideParent) : nullptr;
}

void UPCGMetadata::K2_InitializeAsCopy(const UPCGMetadata* InMetadataToCopy, const TArray<int64>& InOptionalEntriesToCopy)
{
	if (!IsValid(InMetadataToCopy))
	{
		return;
	}

	InitializeAsCopy(FPCGMetadataInitializeParams(InMetadataToCopy, !InOptionalEntriesToCopy.IsEmpty() ? &InOptionalEntriesToCopy : nullptr));
}

void UPCGMetadata::K2_InitializeAsCopyWithAttributeFilter(const UPCGMetadata* InMetadataToCopy,
	const TSet<FName>& InFilteredAttributes,
	const TArray<int64>& InOptionalEntriesToCopy,
	EPCGMetadataFilterMode InFilterMode,
	EPCGStringMatchingOperator InMatchOperator)
{
	if (!IsValid(InMetadataToCopy))
	{
		return;
	}

	InitializeAsCopy(FPCGMetadataInitializeParams(InMetadataToCopy, InFilteredAttributes, InFilterMode, InMatchOperator, !InOptionalEntriesToCopy.IsEmpty() ? &InOptionalEntriesToCopy : nullptr));
}

void UPCGMetadata::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMetadata::AddToCrc);
	const UPCGData* Data = Cast<UPCGData>(GetOuter());
	check(Data);
	
	TArray<FPCGMetadataDomainID> DomainIDs;
	MetadataDomains.GetKeys(DomainIDs);
	DomainIDs.Sort();

	for (const FPCGMetadataDomainID& DomainID : DomainIDs)
	{
		// No need to CRC the default (since it is an alias to one domain)
		if (DomainID.IsDefault())
		{
			continue;
		}

		bool bIsValid = MetadataDomains[DomainID].IsValid();
		Ar << bIsValid;

		if (bIsValid)
		{
			MetadataDomains[DomainID]->AddToCrc(Ar, Data, bFullDataCrc);
		}
	}
}

void UPCGMetadata::Initialize(const UPCGMetadata* InParent)
{
	if (!IsValid(InParent))
	{
		return;
	}

	Initialize(FPCGMetadataInitializeParams(InParent));
}

void UPCGMetadata::Initialize(const UPCGMetadata* InParent, bool bAddAttributesFromParent)
{
	if (!IsValid(InParent))
	{
		return;
	}
	
	// If we are adding attributes from parent, then we use exclude filter with empty list so
	// that all parameters added. Otherwise use include filter with empty list so none are added.
	const FPCGMetadataInitializeParams Params(InParent, TSet<FName>(), bAddAttributesFromParent ? EPCGMetadataFilterMode::ExcludeAttributes : EPCGMetadataFilterMode::IncludeAttributes);
	Initialize(Params);
}

void UPCGMetadata::InitializeWithAttributeFilter(const UPCGMetadata* InParent, const TSet<FName>& InFilteredAttributes, EPCGMetadataFilterMode InFilterMode, EPCGStringMatchingOperator InMatchOperator)
{
	if (!IsValid(InParent))
	{
		return;
	}

	Initialize(FPCGMetadataInitializeParams(InParent, InFilteredAttributes, InFilterMode, InMatchOperator));
}

void UPCGMetadata::Initialize(const FPCGMetadataInitializeParams& InParams)
{
	if (Parent || InParams.Parent == this || !InParams.Parent)
	{
		// Already initialized, or invalid parent; note that while that might be constructed as a warning, there are legit cases where this is correct
		return;
	}
	
	// If we have a domain mapping, we can't have "dynamic" domains
	if (InParams.DomainMapping.IsEmpty())
	{
		SetupDomainsFromOtherMetadataIfNeeded(InParams.Parent);
	}

	Parent = InParams.Parent;
	
	Parent->ForEachValidUniqueConstMetadataDomain([this, &InParams](const FPCGMetadataDomainID& DomainID, const FPCGMetadataDomain& OtherMetadataDomain)
	{
		auto InitializeFunc = [](FPCGMetadataDomain& CurrentMetadataDomain, const FPCGMetadataDomainInitializeParams& DomainParams)
		{
			CurrentMetadataDomain.Initialize(DomainParams);
		};
		
		FindFixOrCreateDomainInitializeParams(InParams, DomainID, OtherMetadataDomain, std::move(InitializeFunc));
	});
}

void UPCGMetadata::InitializeAsCopy(const UPCGMetadata* InMetadataToCopy, const TArray<PCGMetadataEntryKey>* EntriesToCopy)
{
	if (!IsValid(InMetadataToCopy))
	{
		return;
	}

	InitializeAsCopy(FPCGMetadataInitializeParams(InMetadataToCopy, EntriesToCopy));
}

void UPCGMetadata::InitializeAsCopyWithAttributeFilter(const UPCGMetadata* InMetadataToCopy, const TSet<FName>& InFilteredAttributes, EPCGMetadataFilterMode InFilterMode, const TArray<PCGMetadataEntryKey>* EntriesToCopy, EPCGStringMatchingOperator InMatchOperator)
{
	if (!IsValid(InMetadataToCopy))
	{
		return;
	}

	InitializeAsCopy(FPCGMetadataInitializeParams(InMetadataToCopy, InFilteredAttributes, InFilterMode, InMatchOperator, EntriesToCopy));
}

void UPCGMetadata::InitializeAsCopy(const FPCGMetadataInitializeParams& InParams)
{
	if (!InParams.Parent)
	{
		return;
	}
	
	if (Parent)
	{
		UE_LOGF(LogPCG, Error, "Metadata has already been initialized.");
		return;
	}

	// If any metadata domain sets its parent, we need to parent this metadata to the metadata to copy, to preserve the hierarchy.
	// But in case of partial copy, the parenting is not necessary.
	bool bShouldParent = false;

	SetupDomainsFromOtherMetadataIfNeeded(InParams.Parent);
		
	InParams.Parent->ForEachValidUniqueConstMetadataDomain([this, &bShouldParent, &InParams](const FPCGMetadataDomainID& DomainID, const FPCGMetadataDomain& OtherMetadataDomain)
	{
		auto InitializeFunc = [&bShouldParent, &InParams](FPCGMetadataDomain& CurrentMetadataDomain, const FPCGMetadataDomainInitializeParams& DomainParams)
		{
			CurrentMetadataDomain.InitializeAsCopy(DomainParams);

			// We still validate that the parent was set correctly and it matches the metadata to copy.
			if (CurrentMetadataDomain.Parent && CurrentMetadataDomain.Parent->TopMetadata == InParams.Parent)
			{
				bShouldParent = true;
			}
			else
			{
				CurrentMetadataDomain.Parent = nullptr;
			}
		};
		
		FindFixOrCreateDomainInitializeParams(InParams, DomainID, OtherMetadataDomain, std::move(InitializeFunc));
	});

	if (bShouldParent)
	{
		Parent = InParams.Parent;
	}
}

void UPCGMetadata::AddAttributes(const UPCGMetadata* InOther)
{
	if (!IsValid(InOther))
	{
		return;
	}

	AddAttributes(FPCGMetadataInitializeParams(InOther));
}

void UPCGMetadata::AddAttributesFiltered(const UPCGMetadata* InOther, const TSet<FName>& InFilteredAttributes, EPCGMetadataFilterMode InFilterMode, EPCGStringMatchingOperator InMatchOperator)
{
	if (!IsValid(InOther))
	{
		return;
	}

	AddAttributes(FPCGMetadataInitializeParams(InOther, InFilteredAttributes, InFilterMode, InMatchOperator));
}

void UPCGMetadata::AddAttributes(const FPCGMetadataInitializeParams& InParams)
{
	if (!InParams.Parent)
	{
		return;
	}

	bool bAddSucceeded = false;

	InParams.Parent->ForEachValidUniqueConstMetadataDomain([this, &InParams, &bAddSucceeded](const FPCGMetadataDomainID& DomainID, const FPCGMetadataDomain& OtherMetadataDomain)
	{
		auto InitializeFunc = [&bAddSucceeded](FPCGMetadataDomain& CurrentMetadataDomain, const FPCGMetadataDomainInitializeParams& DomainParams)
		{
			bAddSucceeded |= CurrentMetadataDomain.AddAttributes(DomainParams);
		};

		FindFixOrCreateDomainInitializeParams(InParams, DomainID, OtherMetadataDomain, std::move(InitializeFunc));
	});
}

void UPCGMetadata::BP_AddAttribute(const UPCGMetadata* InOther, FName AttributeName)
{
	AddAttribute(InOther, AttributeName);
}

void UPCGMetadata::AddAttribute(const UPCGMetadata* InOther, FPCGAttributeIdentifier AttributeName)
{
	if (!InOther || !InOther->HasAttribute(AttributeName) || HasAttribute(AttributeName))
	{
		return;
	}

	CopyAttribute(InOther->GetConstAttribute(AttributeName), AttributeName, /*bKeepParent=*/InOther == Parent, /*bCopyEntries=*/false, /*bCopyValues=*/false);
}

void UPCGMetadata::CopyAttributes(const UPCGMetadata* InOther)
{
	if (!InOther || InOther == Parent)
	{
		return;
	}

	if (GetItemCountForChild() != InOther->GetItemCountForChild())
	{
		UE_LOGF(LogPCG, Error, "Mismatch in copy attributes since the entries do not match");
		return;
	}

	InOther->ForEachValidUniqueConstMetadataDomain([this](const FPCGMetadataDomainID& DomainID, const FPCGMetadataDomain& OtherMetadataDomain)
	{
		if (FPCGMetadataDomain* CurrentMetadataDomain = FindOrCreateMetadataDomain(DomainID))
		{
			CurrentMetadataDomain->CopyAttributes(&OtherMetadataDomain);
		}
	});
}

void UPCGMetadata::BP_CopyAttribute(const UPCGMetadata* InOther, FName AttributeToCopy, FName NewAttributeName)
{
	CopyAttribute(InOther, AttributeToCopy, NewAttributeName); 
}

void UPCGMetadata::CopyAttribute(const UPCGMetadata* InOther, FPCGAttributeIdentifier AttributeToCopy, FName NewAttributeName)
{
	if (!InOther)
	{
		return;
	}
	else if (HasAttribute(NewAttributeName) || !InOther->HasAttribute(AttributeToCopy))
	{
		return;
	}
	else if (InOther == Parent)
	{
		CopyExistingAttribute(AttributeToCopy, NewAttributeName);
		return;
	}

	if (GetItemCountForChild() != InOther->GetItemCountForChild())
	{
		UE_LOGF(LogPCG, Error, "Mismatch in copy attributes since the entries do not match");
		return;
	}
	
	CopyAttribute(InOther->GetConstAttribute(AttributeToCopy), NewAttributeName, /*bKeepParent=*/false, /*bCopyEntries=*/true, /*bCopyValues=*/true);
}

const UPCGMetadata* UPCGMetadata::GetRoot() const
{
	if (Parent)
	{
		return Parent->GetRoot();
	}
	else
	{
		return this;
	}
}

bool UPCGMetadata::HasParent(const UPCGMetadata* InTentativeParent) const
{
	if (!InTentativeParent)
	{
		return false;
	}

	const UPCGMetadata* HierarchicalParent = Parent.Get();
	while (HierarchicalParent && HierarchicalParent != InTentativeParent)
	{
		HierarchicalParent = HierarchicalParent->Parent.Get();
	}

	return HierarchicalParent == InTentativeParent;
}

void UPCGMetadata::Flatten()
{
	// Check if we have a UPCGData owner, if so call it, otherwise just call FlattenImpl
	if (UPCGData* Owner = Cast<UPCGData>(GetOuter()))
	{
		Owner->Flatten();
	}
	else
	{
		FlattenImpl();
	}
}

void UPCGMetadata::FlattenImpl()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMetadata::FlattenImpl);
	Modify();

	ForEachValidUniqueMetadataDomain([](const FPCGMetadataDomainID& DomainID, FPCGMetadataDomain& MetadataDomain) { MetadataDomain.FlattenImpl();});

	Parent = nullptr;
}

bool UPCGMetadata::FlattenAndCompress(const TArray<PCGMetadataEntryKey>& InEntryKeysToKeep)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMetadata::FlattenAndCompress);
	
	ForEachValidUniqueMetadataDomain([&InEntryKeysToKeep](const FPCGMetadataDomainID& DomainID, FPCGMetadataDomain& MetadataDomain){ MetadataDomain.FlattenAndCompress(InEntryKeysToKeep); });

	Parent = nullptr;
	return true;
}

bool UPCGMetadata::FlattenAndCompress(const TMap<FPCGMetadataDomainID, TArrayView<const PCGMetadataEntryKey>>& InEntryKeysToKeepMapping)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGMetadata::FlattenAndCompress);
	
	ForEachValidUniqueMetadataDomain([this, &InEntryKeysToKeepMapping](const FPCGMetadataDomainID& DomainID, FPCGMetadataDomain& MetadataDomain)
	{
		const TArrayView<const PCGMetadataEntryKey>* EntryKeysToKeep = InEntryKeysToKeepMapping.Find(MetadataDomain.GetDomainID());
		if (!EntryKeysToKeep && MetadataDomain.GetDomainID() == DefaultDomain)
		{
			EntryKeysToKeep = InEntryKeysToKeepMapping.Find(PCGMetadataDomainID::Default);
		}
		
		if (EntryKeysToKeep)
		{
			MetadataDomain.FlattenAndCompress(*EntryKeysToKeep);
		}
		else
		{
			MetadataDomain.FlattenImpl();
		}
	});

	Parent = nullptr;
	return true;
}

FPCGMetadataAttributeBase* UPCGMetadata::GetMutableAttribute(FPCGAttributeIdentifier AttributeName)
{
	return WithMetadataDomain(AttributeName.MetadataDomain, &FPCGMetadataDomain::GetMutableAttribute, AttributeName.Name);
}

const FPCGMetadataAttributeBase* UPCGMetadata::GetConstAttribute(FPCGAttributeIdentifier AttributeName) const
{
	return WithConstMetadataDomain(AttributeName.MetadataDomain, &FPCGMetadataDomain::GetConstAttribute, AttributeName.Name);
}

bool UPCGMetadata::BP_HasAttribute(FName AttributeName) const
{
	return HasAttribute(AttributeName);
}

bool UPCGMetadata::HasAttribute(FPCGAttributeIdentifier AttributeName) const
{
	return WithConstMetadataDomain(AttributeName.MetadataDomain, &FPCGMetadataDomain::HasAttribute, AttributeName.Name);
}

bool UPCGMetadata::HasCommonAttributes(const UPCGMetadata* InMetadata) const
{
	if (!InMetadata)
	{
		return false;
	}
	
	bool bHasCommonAttribute = false;
	ForEachValidUniqueConstMetadataDomain([InMetadata, &bHasCommonAttribute](const FPCGMetadataDomainID& DomainID, const FPCGMetadataDomain& MetadataDomain)
	{
		if (bHasCommonAttribute)
		{
			return;
		}
		
		const FPCGMetadataDomain* OtherMetadataDomain = InMetadata->GetConstMetadataDomain(DomainID);
		bHasCommonAttribute = OtherMetadataDomain && MetadataDomain.HasCommonAttributes(OtherMetadataDomain);
	});

	return bHasCommonAttribute;
}

int32 UPCGMetadata::GetAttributeCount() const
{
	int32 Count = 0;
	ForEachValidUniqueConstMetadataDomain([&Count](const FPCGMetadataDomainID& DomainID, const FPCGMetadataDomain& MetadataDomain) { Count += MetadataDomain.GetAttributeCount(); });
	
	return Count;
}

void UPCGMetadata::GetAttributes(TArray<FName>& AttributeNames, TArray<EPCGMetadataTypes>& AttributeTypes) const
{
	AttributeNames.Reset();
	AttributeTypes.Reset();

	if (const FPCGMetadataDomain* DefaultMetadataDomain = GetConstDefaultMetadataDomain())
	{
		DefaultMetadataDomain->GetAttributes(AttributeNames, AttributeTypes);
	}
}

void UPCGMetadata::GetAllAttributes(TArray<FPCGAttributeIdentifier>& AttributeNames, TArray<EPCGMetadataTypes>& AttributeTypes) const
{
	AttributeNames.Reset();
	AttributeTypes.Reset();

	TArray<FName> SubAttributeNames;

	ForEachValidUniqueConstMetadataDomain([&AttributeNames, &AttributeTypes, &SubAttributeNames](const FPCGMetadataDomainID& DomainID, const FPCGMetadataDomain& MetadataDomain)
	{
		SubAttributeNames.Empty();
		MetadataDomain.GetAttributes(SubAttributeNames, AttributeTypes);
		Algo::Transform(SubAttributeNames, AttributeNames, [&DomainID](const FName Name) { return FPCGAttributeIdentifier(Name, DomainID);});
	});
}

FName UPCGMetadata::GetLatestAttributeNameOrNone() const
{
	const FPCGMetadataDomain* DefaultMetadataDomain = GetConstDefaultMetadataDomain();
	check(DefaultMetadataDomain);
	return DefaultMetadataDomain->GetLatestAttributeNameOrNone();
}

bool UPCGMetadata::ParentHasAttribute(FPCGAttributeIdentifier AttributeName) const
{
	return Parent && Parent->HasAttribute(AttributeName);
}

#define PCG_IMPL_CREATE_TYPED_ATTRIBUTE(FuncName, ArgType) \
UPCGMetadata* UPCGMetadata::FuncName(FName AttributeName, ArgType DefaultValue, bool bAllowsInterpolation, bool bOverrideParent) \
{ \
	CreateAttribute<std::remove_const_t<std::remove_reference_t<ArgType>>>(AttributeName, DefaultValue, bAllowsInterpolation, bOverrideParent); \
	return this; \
}

PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateInteger32Attribute, int32)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateInteger64Attribute, int64)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateFloatAttribute, float)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateDoubleAttribute, double)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateVectorAttribute, FVector)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateVector4Attribute, FVector4)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateVector2Attribute, FVector2D)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateRotatorAttribute, FRotator)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateQuatAttribute, FQuat)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateTransformAttribute, FTransform)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateStringAttribute, FString)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateNameAttribute, FName)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateBoolAttribute, bool)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateSoftObjectPathAttribute, const FSoftObjectPath&)
PCG_IMPL_CREATE_TYPED_ATTRIBUTE(CreateSoftClassPathAttribute, const FSoftClassPath&)

#undef PCG_IMPL_CREATE_TYPED_ATTRIBUTE

bool UPCGMetadata::CreateAttributeFromProperty(FPCGAttributeIdentifier AttributeName, const UObject* Object, const FProperty* InProperty)
{
	FPCGMetadataDomain* FoundMetadataDomain = FindOrCreateMetadataDomain(AttributeName.MetadataDomain);
	return FoundMetadataDomain ? FoundMetadataDomain->CreateAttributeFromProperty(AttributeName.Name, Object, InProperty) : false;
}

bool UPCGMetadata::CreateAttributeFromDataProperty(FPCGAttributeIdentifier AttributeName, const void* Data, const FProperty* InProperty)
{
	FPCGMetadataDomain* FoundMetadataDomain = FindOrCreateMetadataDomain(AttributeName.MetadataDomain);
	return FoundMetadataDomain ? FoundMetadataDomain->CreateAttributeFromDataProperty(AttributeName.Name, Data, InProperty) : false;
}

bool UPCGMetadata::SetAttributeFromProperty(FPCGAttributeIdentifier AttributeName, PCGMetadataEntryKey& EntryKey, const UObject* Object, const FProperty* InProperty, bool bCreate)
{
	FPCGMetadataDomain* FoundMetadataDomain = FindOrCreateMetadataDomain(AttributeName.MetadataDomain);
	return FoundMetadataDomain ? FoundMetadataDomain->SetAttributeFromProperty(AttributeName.Name, EntryKey, Object, InProperty, bCreate) : false;
}

bool UPCGMetadata::SetAttributeFromDataProperty(FPCGAttributeIdentifier AttributeName, PCGMetadataEntryKey& EntryKey, const void* Data, const FProperty* InProperty, bool bCreate)
{
	FPCGMetadataDomain* FoundMetadataDomain = FindOrCreateMetadataDomain(AttributeName.MetadataDomain);
	return FoundMetadataDomain ? FoundMetadataDomain->SetAttributeFromDataProperty(AttributeName.Name, EntryKey, Data, InProperty, bCreate) : false;
}

bool UPCGMetadata::BP_CopyExistingAttribute(FName AttributeToCopy, FName NewAttributeName, bool bKeepParent)
{
	return CopyExistingAttribute(AttributeToCopy, NewAttributeName, bKeepParent);
}

bool UPCGMetadata::CopyExistingAttribute(FPCGAttributeIdentifier AttributeToCopy, FName NewAttributeName, bool bKeepParent)
{
	return CopyAttribute(AttributeToCopy, NewAttributeName, bKeepParent, /*bCopyEntries=*/true, /*bCopyValues=*/true) != nullptr;
}

FPCGMetadataAttributeBase* UPCGMetadata::CopyAttribute(FPCGAttributeIdentifier AttributeToCopy, FName NewAttributeName, bool bKeepParent, bool bCopyEntries, bool bCopyValues)
{
	return WithMetadataDomain_Lambda(AttributeToCopy.MetadataDomain, [&](FPCGMetadataDomain* MetadataDomain) { return MetadataDomain->CopyAttribute(AttributeToCopy.Name, NewAttributeName, bKeepParent, bCopyEntries, bCopyValues); });
}

FPCGMetadataAttributeBase* UPCGMetadata::CopyAttribute(const FPCGMetadataAttributeBase* OriginalAttribute, FPCGAttributeIdentifier NewAttributeName, bool bKeepParent, bool bCopyEntries, bool bCopyValues)
{
	check(OriginalAttribute);
	const FPCGMetadataDomain* OriginalMetadataDomain = OriginalAttribute->GetMetadataDomain();
	check(OriginalMetadataDomain);

#if DO_GUARD_SLOW
	const UPCGMetadata* RootMetadata = GetRoot();
	checkSlow((RootMetadata && RootMetadata->MetadataDomains.Contains(OriginalMetadataDomain->DomainID) && RootMetadata->MetadataDomains[OriginalMetadataDomain->DomainID].Get() == OriginalMetadataDomain->GetRoot()) || !bKeepParent);
#endif

	FPCGMetadataAttributeBase* Attribute = WithMetadataDomain_Lambda(NewAttributeName.MetadataDomain, [&](FPCGMetadataDomain* MetadataDomain) -> FPCGMetadataAttributeBase*
	{
		// TODO: Maybe the operation needs to be handled differently? 
		if (OriginalMetadataDomain->DomainID != MetadataDomain->DomainID)
		{
			// UE_LOGF(LogPCG, Error, "[Metadata - CopyAttribute] Can't copy across domains.");
			// return nullptr;
		}

		return MetadataDomain->CopyAttribute(OriginalAttribute, NewAttributeName.Name, bKeepParent, bCopyEntries, bCopyValues);
	});
	
	if (!Attribute)
	{
		UE_LOGF(LogPCG, Error, "[Metadata - CopyAttribute] Metadata domain does not exist in current metadata or copy failed.");
	}

	return Attribute;
}

bool UPCGMetadata::BP_RenameAttribute(FName AttributeToRename, FName NewAttributeName)
{
	return RenameAttribute(AttributeToRename, NewAttributeName);
}

bool UPCGMetadata::RenameAttribute(FPCGAttributeIdentifier AttributeToRename, FName NewAttributeName)
{
	if (FPCGMetadataDomain* FoundMetadataDomain = GetMetadataDomain(AttributeToRename.MetadataDomain))
	{
		return FoundMetadataDomain->RenameAttribute(AttributeToRename.Name, NewAttributeName);
	}
	else
	{
		return false;
	}
}

void UPCGMetadata::BP_ClearAttribute(FName AttributeToClear)
{
	ClearAttribute(AttributeToClear);
}

void UPCGMetadata::ClearAttribute(FPCGAttributeIdentifier AttributeToClear)
{
	return WithMetadataDomain(AttributeToClear.MetadataDomain, &FPCGMetadataDomain::ClearAttribute, AttributeToClear.Name);
}

void UPCGMetadata::BP_DeleteAttribute(FName AttributeToDelete)
{
	DeleteAttribute(AttributeToDelete);
}

void UPCGMetadata::DeleteAttribute(FPCGAttributeIdentifier AttributeToDelete)
{
	return WithMetadataDomain(AttributeToDelete.MetadataDomain, &FPCGMetadataDomain::DeleteAttribute, AttributeToDelete.Name);
}

bool UPCGMetadata::ChangeAttributeType(FPCGAttributeIdentifier AttributeName, const FPCGMetadataAttributeDesc& AttributeNewDesc)
{
	// Need to differentiate overloads.
	return WithMetadataDomain(AttributeName.MetadataDomain, static_cast<bool(FPCGMetadataDomain::*)(FName, const FPCGMetadataAttributeDesc&)>(&FPCGMetadataDomain::ChangeAttributeType), AttributeName.Name, AttributeNewDesc);
}

bool UPCGMetadata::ChangeAttributeType(FPCGAttributeIdentifier AttributeName, int16 AttributeNewType)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Need to differentiate overloads.
	return WithMetadataDomain(AttributeName.MetadataDomain, static_cast<bool(FPCGMetadataDomain::*)(FName, int16)>(&FPCGMetadataDomain::ChangeAttributeType), AttributeName.Name, AttributeNewType);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

int64 UPCGMetadata::GetItemCountForChild() const
{
	return WithConstMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::GetItemCountForChild);
}

int64 UPCGMetadata::GetLocalItemCount() const
{
	return WithConstMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::GetLocalItemCount);
}

int64 UPCGMetadata::AddEntry(int64 ParentEntry)
{
	return WithMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::AddEntry, ParentEntry);
}

TArray<int64> UPCGMetadata::AddEntries(TArrayView<const int64> ParentEntryKeys)
{
	return WithMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::AddEntries, ParentEntryKeys);
}

void UPCGMetadata::AddEntriesInPlace(TArrayView<int64*> ParentEntryKeys)
{
	return WithMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::AddEntriesInPlace, ParentEntryKeys);
}

int64 UPCGMetadata::AddEntryPlaceholder()
{
	return WithMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::AddEntryPlaceholder);
}

void UPCGMetadata::AddDelayedEntries(const TArray<TTuple<int64, int64>>& AllEntries)
{
	return WithMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::AddDelayedEntries, AllEntries);
}

bool UPCGMetadata::InitializeOnSet(PCGMetadataEntryKey& InOutKey, PCGMetadataEntryKey InParentKeyA, const UPCGMetadata* InParentMetadataA, PCGMetadataEntryKey InParentKeyB, const UPCGMetadata* InParentMetadataB)
{
	return WithMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::InitializeOnSet, InOutKey, InParentKeyA, InParentMetadataA ? InParentMetadataA->GetConstMetadataDomain(PCGMetadataDomainID::Default) : nullptr, InParentKeyB, InParentMetadataB ? InParentMetadataB->GetConstMetadataDomain(PCGMetadataDomainID::Default) : nullptr);
}

PCGMetadataEntryKey UPCGMetadata::GetParentKey(PCGMetadataEntryKey LocalItemKey) const
{
	return WithConstMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::GetParentKey, LocalItemKey);
}

void UPCGMetadata::GetParentKeys(TArrayView<PCGMetadataEntryKey> LocalItemKeys, const TBitArray<>* Mask) const
{
	return WithConstMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::GetParentKeys, LocalItemKeys, Mask);
}

void UPCGMetadata::MergePointAttributes(const FPCGPoint& InPointA, const FPCGPoint& InPointB, FPCGPoint& OutPoint, EPCGMetadataOp Op)
{
	MergeAttributes(InPointA.MetadataEntry, this, InPointB.MetadataEntry, this, OutPoint.MetadataEntry, Op);
}

void UPCGMetadata::MergePointAttributesSubset(const FPCGPoint& InPointA, const UPCGMetadata* InMetadataA, const UPCGMetadata* InMetadataSubsetA, const FPCGPoint& InPointB, const UPCGMetadata* InMetadataB, const UPCGMetadata* InMetadataSubsetB, FPCGPoint& OutPoint, EPCGMetadataOp Op)
{
	MergeAttributesSubset(InPointA.MetadataEntry, InMetadataA, InMetadataSubsetA, InPointB.MetadataEntry, InMetadataB, InMetadataSubsetB, OutPoint.MetadataEntry, Op);
}

void UPCGMetadata::MergeAttributes(PCGMetadataEntryKey InKeyA, const UPCGMetadata* InMetadataA, PCGMetadataEntryKey InKeyB, const UPCGMetadata* InMetadataB, PCGMetadataEntryKey& OutKey, EPCGMetadataOp Op)
{
	MergeAttributesSubset(InKeyA, InMetadataA, InMetadataA, InKeyB, InMetadataB, InMetadataB, OutKey, Op);
}

void UPCGMetadata::MergeAttributesSubset(PCGMetadataEntryKey InKeyA, const UPCGMetadata* InMetadataA, const UPCGMetadata* InMetadataSubsetA, PCGMetadataEntryKey InKeyB, const UPCGMetadata* InMetadataB, const UPCGMetadata* InMetadataSubsetB, PCGMetadataEntryKey& OutKey, EPCGMetadataOp Op)
{
	// TODO Support more domains?
	const FPCGMetadataDomain* InMetadataDomainA = InMetadataA->GetConstMetadataDomain(PCGMetadataDomainID::Default);
	const FPCGMetadataDomain* InMetadataDomainSubsetA = InMetadataSubsetA->GetConstMetadataDomain(PCGMetadataDomainID::Default);
	const FPCGMetadataDomain* InMetadataDomainB = InMetadataB->GetConstMetadataDomain(PCGMetadataDomainID::Default);
	const FPCGMetadataDomain* InMetadataDomainSubsetB = InMetadataSubsetB->GetConstMetadataDomain(PCGMetadataDomainID::Default);

	return WithMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::MergeAttributesSubset, InKeyA, InMetadataDomainA, InMetadataDomainSubsetA, InKeyB, InMetadataDomainB, InMetadataDomainSubsetB, OutKey, Op); 
}

void UPCGMetadata::ResetWeightedAttributes(PCGMetadataEntryKey& OutKey)
{
	return WithMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::ResetWeightedAttributes, OutKey);
}

void UPCGMetadata::AccumulateWeightedAttributes(PCGMetadataEntryKey InKey, const UPCGMetadata* InMetadata, float Weight, bool bSetNonInterpolableAttributes, PCGMetadataEntryKey& OutKey)
{
	// TODO: Support other domains?
	return WithMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::AccumulateWeightedAttributes, InKey, InMetadata ? InMetadata->GetConstMetadataDomain(PCGMetadataDomainID::Default) : nullptr, Weight, bSetNonInterpolableAttributes, OutKey);
}

void UPCGMetadata::ComputePointWeightedAttribute(FPCGPoint& OutPoint, const TArrayView<TPair<const FPCGPoint*, float>>& InWeightedPoints, const UPCGMetadata* InMetadata)
{
	// TODO: Support other domains?
	return WithMetadataDomain_Lambda(PCGMetadataDomainID::Default, [&](FPCGMetadataDomain* MetadataDomain)
	{
		PCGMetadataHelpers::ComputePointWeightedAttribute(MetadataDomain, OutPoint, InWeightedPoints, InMetadata->GetConstMetadataDomain(PCGMetadataDomainID::Default));
	});
}

void UPCGMetadata::ComputeWeightedAttribute(PCGMetadataEntryKey& OutKey, const TArrayView<TPair<PCGMetadataEntryKey, float>>& InWeightedKeys, const UPCGMetadata* InMetadata)
{
	if (!InMetadata || InWeightedKeys.IsEmpty())
	{
		return;
	}

	// TODO: Support other domains?
	return WithMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::ComputeWeightedAttribute, OutKey, InWeightedKeys, InMetadata->GetConstMetadataDomain(PCGMetadataDomainID::Default));
}

int64 UPCGMetadata::GetItemKeyCountForParent() const
{
	return WithConstMetadataDomain(PCGMetadataDomainID::Default, &FPCGMetadataDomain::GetItemKeyCountForParent);
}

void UPCGMetadata::SetAttributes(PCGMetadataEntryKey InKey, const UPCGMetadata* InMetadata, PCGMetadataEntryKey& OutKey)
{
	if (!InMetadata)
	{
		return;
	}

	// TODO: Support other domains?
	return WithMetadataDomain_Lambda(PCGMetadataDomainID::Default, [&InKey, InMetadataDomain = InMetadata->GetConstMetadataDomain(PCGMetadataDomainID::Default), &OutKey](FPCGMetadataDomain* MetadataDomain)
	{
		return MetadataDomain->SetAttributes(InKey, InMetadataDomain, OutKey);
	});
}

void UPCGMetadata::SetPointAttributes(const TArrayView<const FPCGPoint>& InPoints, const UPCGMetadata* InMetadata, const TArrayView<FPCGPoint>& OutPoints, FPCGContext* OptionalContext)
{
	if (!InMetadata || InMetadata->GetAttributeCount() == 0 || GetAttributeCount() == 0)
	{
		return;
	}

	// TODO: Support other domains?
	return WithMetadataDomain_Lambda(PCGMetadataDomainID::Default, [&](FPCGMetadataDomain* MetadataDomain)
	{
		PCGMetadataHelpers::SetPointAttributes(MetadataDomain, InPoints, InMetadata->GetConstMetadataDomain(PCGMetadataDomainID::Default), OutPoints, OptionalContext);
	});
}

void UPCGMetadata::SetAttributes(const TArrayView<const PCGMetadataEntryKey>& InOriginalKeys, const UPCGMetadata* InMetadata, const TArrayView<PCGMetadataEntryKey>* InOutOptionalKeys, FPCGContext* OptionalContext)
{
	if (!InMetadata || InMetadata->GetAttributeCount() == 0 || GetAttributeCount() == 0 || InOriginalKeys.IsEmpty())
	{
		return;
	}

	// TODO: Support other domains?
	return WithMetadataDomain_Lambda(PCGMetadataDomainID::Default, [&](FPCGMetadataDomain* MetadataDomain)
	{
		return MetadataDomain->SetAttributes(InOriginalKeys, InMetadata->GetConstMetadataDomain(PCGMetadataDomainID::Default), InOutOptionalKeys, OptionalContext);
	});
}

void UPCGMetadata::SetAttributes(const TArrayView<const PCGMetadataEntryKey>& InKeys, const UPCGMetadata* InMetadata, const TArrayView<PCGMetadataEntryKey>& OutKeys, FPCGContext* OptionalContext)
{
	SetAttributes(InKeys, InMetadata, &OutKeys, OptionalContext);
}

void UPCGMetadata::MergeAttributesByKey(int64 KeyA, const UPCGMetadata* MetadataA, int64 KeyB, const UPCGMetadata* MetadataB, int64 TargetKey, EPCGMetadataOp Op, int64& OutKey)
{
	OutKey = TargetKey;
	MergeAttributes(KeyA, MetadataA, KeyB, MetadataB, OutKey, Op);
}

void UPCGMetadata::SetAttributesByKey(int64 Key, const UPCGMetadata* Metadata, int64 TargetKey, int64& OutKey)
{
	OutKey = TargetKey;
	SetAttributes(Key, Metadata, OutKey);
}

void UPCGMetadata::ResetWeightedAttributesByKey(int64 TargetKey, int64& OutKey)
{
	OutKey = TargetKey;
	ResetWeightedAttributes(OutKey);
}

void UPCGMetadata::AccumulateWeightedAttributesByKey(PCGMetadataEntryKey Key, const UPCGMetadata* Metadata, float Weight, bool bSetNonInterpolableAttributes, int64 TargetKey, int64& OutKey)
{
	OutKey = TargetKey;
	AccumulateWeightedAttributes(Key, Metadata, Weight, bSetNonInterpolableAttributes, OutKey);
}

void UPCGMetadata::MergePointAttributes(const FPCGPoint& PointA, const UPCGMetadata* MetadataA, const FPCGPoint& PointB, const UPCGMetadata* MetadataB, UPARAM(ref) FPCGPoint& TargetPoint, EPCGMetadataOp Op)
{
	MergeAttributes(PointA.MetadataEntry, MetadataA, PointB.MetadataEntry, MetadataB, TargetPoint.MetadataEntry, Op);
}

void UPCGMetadata::SetPointAttributes(const FPCGPoint& Point, const UPCGMetadata* Metadata, FPCGPoint& OutPoint)
{
	SetAttributes(Point.MetadataEntry, Metadata, OutPoint.MetadataEntry);
}

void UPCGMetadata::ResetPointWeightedAttributes(FPCGPoint& OutPoint)
{
	ResetWeightedAttributes(OutPoint.MetadataEntry);
}

void UPCGMetadata::AccumulatePointWeightedAttributes(const FPCGPoint& InPoint, const UPCGMetadata* InMetadata, float Weight, bool bSetNonInterpolableAttributes, FPCGPoint& OutPoint)
{
	AccumulateWeightedAttributes(InPoint.MetadataEntry, InMetadata, Weight, bSetNonInterpolableAttributes, OutPoint.MetadataEntry);
}

void UPCGMetadata::SetLastCachedSelectorOnOwner(FName AttributeName, FPCGMetadataDomainID DomainID)
{
	if (UPCGData* OwnerData = Cast<UPCGData>(GetOuter()))
	{
		FPCGAttributePropertyInputSelector Selector;
		Selector.SetAttributeName(AttributeName);
		if (!DomainID.IsDefault() && DomainID != DefaultDomain)
		{
			OwnerData->SetDomainFromDomainID(DomainID, Selector);
		}

		OwnerData->SetLastSelector(Selector);
	}
}

bool UPCGMetadata::MetadataDomainSupportsMultiEntries(const FPCGMetadataDomainID& InDomainID) const
{
	const UPCGData* Data = Cast<UPCGData>(GetOuter());
	if (!Data)
	{
		Data = GetDefault<UPCGData>();
	}

	check(Data);
	return Data->MetadataDomainSupportsMultiEntries(InDomainID);
}

bool UPCGMetadata::MetadataDomainSupportsParenting(const FPCGMetadataDomainID& InDomainID) const
{
	const UPCGData* Data = Cast<UPCGData>(GetOuter());
	if (!Data)
	{
		Data = GetDefault<UPCGData>();
	}

	check(Data);
	return Data->MetadataDomainSupportsParenting(InDomainID);
}

void UPCGMetadata::SetupDomainsFromOtherMetadataIfNeeded(const UPCGMetadata* OtherMetadata)
{
	// Only do this if our outer is not a PCG data and we are not setuped.
	// To be used with caution, only with floating metadata.
	if (!OtherMetadata || OtherMetadata->DefaultDomain.IsDefault() || !DefaultDomain.IsDefault() || (GetOuter() && GetOuter()->IsA<UPCGData>()))
	{
		return;
	}

	for (const auto& It : OtherMetadata->MetadataDomains)
	{
		if (MetadataDomains.Contains(It.Key))
		{
			continue;
		}
		
		SetupDomain(It.Key, It.Key == OtherMetadata->DefaultDomain);
	}
}

FPCGMetadataDomainHandle UPCGMetadata::BP_GetMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID)
{
	FPCGMetadataDomainHandle Result{};
	if (FindOrCreateMetadataDomain(InMetadataDomainID) != nullptr)
	{
		Result.bConst = false;
		Result.DomainHandle = MetadataDomains[InMetadataDomainID].ToWeakPtr();
	}
	
	return Result;
}

FPCGMetadataDomainHandle UPCGMetadata::BP_GetConstMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID) const
{
	FPCGMetadataDomainHandle Result{};
	if (const TSharedPtr<FPCGMetadataDomain>* Domain = MetadataDomains.Find(InMetadataDomainID))
	{
		Result.bConst = true;
		Result.DomainHandle = Domain->ToWeakPtr();
	}
	
	return Result;
}