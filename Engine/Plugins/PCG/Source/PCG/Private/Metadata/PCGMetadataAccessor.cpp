// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGMetadataAccessor.h"

#include "PCGModule.h"
#include "PCGPoint.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataAccessor)

#define LOCTEXT_NAMESPACE "PCGMetadataAccessorHelpers"

namespace PCGMetadataAccessorHelpers
{
	static const FText InvalidAttributeNameFormat = LOCTEXT("InvalidAttributeNameFormat", "[PCG] Invalid attribute name({0})");
	static const FText InvalidTypeFormat = LOCTEXT("InvalidTypeFormat", "[PCG] Attribute {0} is of type {1} and it is not the requested type {2}");
	static const FText NoMetadata = LOCTEXT("NoMetadata", "[PCG] No metadata provided");
	static const FText InvalidValue = LOCTEXT("InvalidValue", "[PCG] Value is invalid.");
	static const FTextFormat InvalidAttribute = LOCTEXT("InvalidAttribute", "[PCG] Could not find attribute {0} in metadata");
	static const FText InvalidValueType = LOCTEXT("InvalidValueType", "[PCG] Value is not of a valid type.");

	void OnException(const FText& ErrorMessage)
	{
		UPCGBlueprintHelpers::ThrowBlueprintException(ErrorMessage);
	}
	
	bool SetAttributeValue(PCGMetadataEntryKey& OutKey, FPCGMetadataDomain* Metadata, const FName AttributeName, const FProperty* ValueProperty, const void* ValuePtr)
	{
		if (!Metadata)
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(LOCTEXT("InvalidMetadataOrConst", "[PCG] Metadata is invalid or const."));
			return false;
		}
	
		if (!ValueProperty || !ValuePtr)
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(InvalidValue);
			return false;
		}
	
		FPCGMetadataAttributeBase* BaseAttribute = Metadata->GetMutableAttribute(AttributeName);
		if (!BaseAttribute)
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(FText::Format(InvalidAttribute, FText::FromName(AttributeName)));
			return false;
		}
	
		TUniquePtr<const IPCGAttributeAccessor> PropertyAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(ValueProperty);
		if (!PropertyAccessor)
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(InvalidValueType);
			return false;
		}
	
		TUniquePtr<IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateAccessor(BaseAttribute, Metadata);
		check(AttributeAccessor);
	
		FPCGAttributeAccessorKeysEntries AttributeKeys(MakeArrayView(&OutKey, 1));
		const FPCGAttributeAccessorKeysGenericPtrs PropertyKeys(MakeArrayView<const void*>(&ValuePtr, 1));

		const bool bSuccess = PropertyAccessor->CopyTo(PropertyKeys, *AttributeAccessor, AttributeKeys, /*Index=*/0, /*Count=*/1, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
		if (!bSuccess)
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(FText::Format(LOCTEXT("InvalidOpTypeWrite", "[PCG] Cannot write value ({0}) into the attribute {1} ({2})"), 
				FText::FromString(PropertyAccessor->GetUnderlyingDesc().GetTypeString()), FText::FromName(AttributeName), FText::FromString(AttributeAccessor->GetUnderlyingDesc().GetTypeString())));
		}
		
		return bSuccess;
	}
	
	bool GetAttributeValue(PCGMetadataEntryKey InKey, const FPCGMetadataDomain* Metadata, const FName AttributeName, const FProperty* ValueProperty, void* ValuePtr)
	{
		if (!Metadata)
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(LOCTEXT("InvalidMetadata", "[PCG] Metadata is invalid"));
			return false;
		}
	
		if (!ValueProperty || !ValuePtr)
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(InvalidValue);
			return false;
		}
	
		const FPCGMetadataAttributeBase* BaseAttribute = Metadata->GetConstAttribute(AttributeName);
		if (!BaseAttribute)
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(FText::Format(InvalidAttribute, FText::FromName(AttributeName)));
			return false;
		}
	
		TUniquePtr<IPCGAttributeAccessor> PropertyAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(ValueProperty);
		if (!PropertyAccessor)
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(InvalidValueType);
			return false;
		}
	
		TUniquePtr<const IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(BaseAttribute, Metadata);
		check(AttributeAccessor);
	
		const FPCGAttributeAccessorKeysEntries AttributeKeys(MakeArrayView(&InKey, 1));
		FPCGAttributeAccessorKeysGenericPtrs PropertyKeys(MakeArrayView<void*>(&ValuePtr, 1));

		const bool bSuccess = AttributeAccessor->CopyTo(AttributeKeys, *PropertyAccessor, PropertyKeys, /*Index=*/0, /*Count=*/1, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
		if (!bSuccess)
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(FText::Format(LOCTEXT("InvalidOpTypeRead", "[PCG] Cannot read value ({0}) from the attribute {1} ({2})"), 
				FText::FromString(PropertyAccessor->GetUnderlyingDesc().GetTypeString()), FText::FromName(AttributeName), FText::FromString(AttributeAccessor->GetUnderlyingDesc().GetTypeString())));
		}
		
		return bSuccess;
	}
}

/** Key-based implementations */
template<typename T>
T UPCGMetadataAccessorHelpers::GetAttribute(PCGMetadataEntryKey Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	if (!Metadata)
	{
		PCGMetadataAccessorHelpers::OnException(PCGMetadataAccessorHelpers::NoMetadata);
		return T{};
	}

	const FPCGMetadataAttributeBase* AttributeBase = Metadata->GetConstAttribute(AttributeName);
	if (!AttributeBase)
	{
		PCGMetadataAccessorHelpers::OnException(FText::Format(PCGMetadataAccessorHelpers::InvalidAttributeNameFormat, FText::FromName(AttributeName)));
		return T{};
	}

	if (AttributeBase->GetTypeId() == PCG::Private::MetadataTypes<T>::Id)
	{
		return static_cast<const FPCGMetadataAttribute<T>*>(AttributeBase)->GetValueFromItemKey(Key);
	}
	else if constexpr (std::is_same_v<T, FString>)
	{
		// Legacy path - allow reading soft object/class paths using string accessor.
		if (AttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FSoftObjectPath>::Id)
		{
			return static_cast<const FPCGMetadataAttribute<FSoftObjectPath>*>(AttributeBase)->GetValueFromItemKey(Key).ToString();
		}
		else if (AttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FSoftClassPath>::Id)
		{
			return static_cast<const FPCGMetadataAttribute<FSoftClassPath>*>(AttributeBase)->GetValueFromItemKey(Key).ToString();
		}
	}

	PCGMetadataAccessorHelpers::OnException(FText::Format(
		PCGMetadataAccessorHelpers::InvalidTypeFormat,
		FText::FromName(AttributeName),
		FText::FromString(PCG::Private::GetTypeName(AttributeBase->GetTypeId())),
		FText::FromString(PCG::Private::GetTypeName(PCG::Private::MetadataTypes<T>::Id))));

	return T{};
}

template<typename T>
void UPCGMetadataAccessorHelpers::SetAttribute(PCGMetadataEntryKey& Key, UPCGMetadata* Metadata, FName AttributeName, const T& Value)
{
	if (!Metadata)
	{
		PCGMetadataAccessorHelpers::OnException(PCGMetadataAccessorHelpers::NoMetadata);
		return;
	}

	Metadata->InitializeOnSet(Key);

	if (Key == PCGInvalidEntryKey)
	{
		PCGMetadataAccessorHelpers::OnException(LOCTEXT("NoMetadataEntry", "[PCG] Metadata key has no entry, therefore can't set values"));
		return;
	}

	FPCGMetadataAttribute<T>* Attribute = static_cast<FPCGMetadataAttribute<T>*>(Metadata->GetMutableAttribute(AttributeName));
	if (Attribute && Attribute->GetTypeId() == PCG::Private::MetadataTypes<T>::Id)
	{
		Attribute->SetValue(Key, Value);
	}
	else if (Attribute)
	{
		PCGMetadataAccessorHelpers::OnException(FText::Format(PCGMetadataAccessorHelpers::InvalidTypeFormat, FText::FromName(AttributeName), FText::FromString(PCG::Private::GetTypeName(Attribute->GetTypeId())), FText::FromString(PCG::Private::GetTypeName(PCG::Private::MetadataTypes<T>::Id))));
	}
	else
	{
		PCGMetadataAccessorHelpers::OnException(FText::Format(PCGMetadataAccessorHelpers::InvalidAttributeNameFormat, FText::FromName(AttributeName)));
	}
}

bool UPCGMetadataAccessorHelpers::HasAttributeSetByMetadataKey(PCGMetadataEntryKey Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	if (!Metadata)
	{
		PCGMetadataAccessorHelpers::OnException(PCGMetadataAccessorHelpers::NoMetadata);
		return false;
	}

	// Early out: the point has no metadata entry assigned
	if (Key == PCGInvalidEntryKey)
	{
		return false;
	}

	if (const FPCGMetadataAttributeBase* Attribute = Metadata->GetConstAttribute(AttributeName))
	{
		return Attribute->HasNonDefaultValue(Key);
	}
	else
	{
		PCGMetadataAccessorHelpers::OnException(FText::Format(PCGMetadataAccessorHelpers::InvalidAttributeNameFormat, FText::FromName(AttributeName)));
		return false;
	}
}

int32 UPCGMetadataAccessorHelpers::GetInteger32AttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<int32>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetInteger32AttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, int32 Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

int64 UPCGMetadataAccessorHelpers::GetInteger64AttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<int64>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetInteger64AttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, int64 Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

float UPCGMetadataAccessorHelpers::GetFloatAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<float>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetFloatAttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, float Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

double UPCGMetadataAccessorHelpers::GetDoubleAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<double>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetDoubleAttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, double Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

FVector UPCGMetadataAccessorHelpers::GetVectorAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FVector>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetVectorAttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FVector& Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

FVector4 UPCGMetadataAccessorHelpers::GetVector4AttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FVector4>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetVector4AttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FVector4& Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

FVector2D UPCGMetadataAccessorHelpers::GetVector2AttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FVector2D>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetVector2AttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FVector2D& Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

FRotator UPCGMetadataAccessorHelpers::GetRotatorAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FRotator>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetRotatorAttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FRotator& Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

FQuat UPCGMetadataAccessorHelpers::GetQuatAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FQuat>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetQuatAttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FQuat& Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

FTransform UPCGMetadataAccessorHelpers::GetTransformAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FTransform>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetTransformAttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FTransform& Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

FString UPCGMetadataAccessorHelpers::GetStringAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FString>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetStringAttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FString& Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

bool UPCGMetadataAccessorHelpers::GetBoolAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<bool>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetBoolAttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, bool Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

FSoftObjectPath UPCGMetadataAccessorHelpers::GetSoftObjectPathAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FSoftObjectPath>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetSoftObjectPathAttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FSoftObjectPath& Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

FSoftClassPath UPCGMetadataAccessorHelpers::GetSoftClassPathAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FSoftClassPath>(Key, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetSoftClassPathAttributeByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FSoftClassPath& Value)
{
	SetAttribute(Key, Metadata, AttributeName, Value);
}

bool UPCGMetadataAccessorHelpers::SetAttributeFromPropertyByMetadataKey(int64& Key, UPCGMetadata* Metadata, FName AttributeName, const UObject* Object, FName PropertyName)
{
	if (!Object)
	{
		return false;
	}

	const FProperty* Property = FindFProperty<FProperty>(Object->GetClass(), PropertyName);
	if (!Property)
	{
		return false;
	}

	return Metadata->SetAttributeFromProperty(AttributeName, Key, Object, Property, /*bCreate=*/ true);
}

/** Point-based implementations */
void UPCGMetadataAccessorHelpers::CopyPoint(const FPCGPoint& InPoint, FPCGPoint& OutPoint, bool bCopyMetadata, const UPCGMetadata* InMetadata, UPCGMetadata* OutMetadata)
{
	// Copy standard properties
	OutPoint = InPoint;

	// If we want to copy the metadata, then at least the out metadata must not be null.
	if (bCopyMetadata)
	{
		InitializeMetadata(OutPoint, OutMetadata, InPoint, InMetadata);
	}
	else
	{
		OutPoint.MetadataEntry = PCGInvalidEntryKey;
	}
}

void UPCGMetadataAccessorHelpers::InitializeMetadata(FPCGPoint& Point, UPCGMetadata* Metadata)
{
	Point.MetadataEntry = Metadata ? Metadata->AddEntry() : PCGInvalidEntryKey;
}

void UPCGMetadataAccessorHelpers::InitializeMetadata(FPCGPoint& Point, UPCGMetadata* Metadata, const FPCGPoint& ParentPoint, const UPCGMetadata* ParentMetadata)
{
	if (Metadata)
	{
		// If we're not given the parent metadata, we'll assume it is the current metadata's parent
		if (!ParentMetadata || Metadata->HasParent(ParentMetadata))
		{
			Point.MetadataEntry = Metadata->AddEntry(ParentPoint.MetadataEntry);
		}
		else
		{
			// Conceptual parent isn't in the metadata hierarchy, therefore we must set attributes if any
			Point.MetadataEntry = Metadata->AddEntry();
			Metadata->SetPointAttributes(ParentPoint, ParentMetadata, Point);
		}
	}
	else
	{
		Point.MetadataEntry = PCGInvalidEntryKey;
	}
}

void UPCGMetadataAccessorHelpers::InitializeMetadataWithParent(FPCGPoint& Point, UPCGMetadata* Metadata, const FPCGPoint& ParentPoint, const UPCGMetadata* ParentMetadata)
{
	Point.MetadataEntry = Metadata ? (Metadata->HasParent(ParentMetadata) ? Metadata->AddEntry(ParentPoint.MetadataEntry) : Metadata->AddEntry()) : PCGInvalidEntryKey;
}

int32 UPCGMetadataAccessorHelpers::GetInteger32Attribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<int32>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetInteger32Attribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, int32 Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

int64 UPCGMetadataAccessorHelpers::GetInteger64Attribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<int64>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetInteger64Attribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, int64 Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

float UPCGMetadataAccessorHelpers::GetFloatAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<float>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetFloatAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, float Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

double UPCGMetadataAccessorHelpers::GetDoubleAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<double>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetDoubleAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, double Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

FVector UPCGMetadataAccessorHelpers::GetVectorAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FVector>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetVectorAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FVector& Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

FVector4 UPCGMetadataAccessorHelpers::GetVector4Attribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FVector4>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetVector4Attribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FVector4& Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

FVector2D UPCGMetadataAccessorHelpers::GetVector2Attribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FVector2D>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetVector2Attribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FVector2D& Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

FRotator UPCGMetadataAccessorHelpers::GetRotatorAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FRotator>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetRotatorAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FRotator& Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

FQuat UPCGMetadataAccessorHelpers::GetQuatAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FQuat>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetQuatAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FQuat& Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

FTransform UPCGMetadataAccessorHelpers::GetTransformAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FTransform>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetTransformAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FTransform& Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

FString UPCGMetadataAccessorHelpers::GetStringAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FString>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetStringAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FString& Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

FName UPCGMetadataAccessorHelpers::GetNameAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FName>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetNameAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FName& Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

bool UPCGMetadataAccessorHelpers::GetBoolAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<bool>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetBoolAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, bool Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

FSoftObjectPath UPCGMetadataAccessorHelpers::GetSoftObjectPathAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FSoftObjectPath>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetSoftObjectPathAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FSoftObjectPath& Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

FSoftClassPath UPCGMetadataAccessorHelpers::GetSoftClassPathAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return GetAttribute<FSoftClassPath>(Point.MetadataEntry, Metadata, AttributeName);
}

void UPCGMetadataAccessorHelpers::SetSoftClassPathAttribute(FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FSoftClassPath& Value)
{
	SetAttribute(Point.MetadataEntry, Metadata, AttributeName, Value);
}

bool UPCGMetadataAccessorHelpers::HasAttributeSet(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName)
{
	return HasAttributeSetByMetadataKey(Point.MetadataEntry, Metadata, AttributeName);
}

namespace PCGMetadataAccessorHelpers
{
	// Since the code differ only from the key and metadata, factorize all the code in this template function
	// KeyType will be either be PCGMetadataEntryKey& (Set) or PCGMetadataEntryKey (Get)
	// MetadataType will be either be FPCGMetadataDomain (Set) or const FPCGMetadataDomain (Get)
	template <typename KeyType, typename MetadataType>
	void GetSetAttributeValue(KeyType Key, MetadataType* Metadata, UObject* Context, FFrame& Stack, RESULT_DECL)
	{
		P_GET_PROPERTY(FNameProperty, Name)
	
		// Read wildcard value input.
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.MostRecentProperty = nullptr;
		Stack.StepCompiledIn<FProperty>(nullptr);
	
		const FProperty* TargetProperty = Stack.MostRecentProperty;
		void* TargetPtr = Stack.MostRecentPropertyContainer;

		P_FINISH;

		P_NATIVE_BEGIN;
		
		static_assert(std::is_const_v<MetadataType> != std::is_reference_v<KeyType>);
		
		if constexpr (std::is_const_v<MetadataType>)
		{
			*static_cast<bool*>(RESULT_PARAM) = PCGMetadataAccessorHelpers::GetAttributeValue(Key, Metadata, Name, TargetProperty, TargetPtr);
		}
		else
		{
			*static_cast<bool*>(RESULT_PARAM) = PCGMetadataAccessorHelpers::SetAttributeValue(Key, Metadata, Name, TargetProperty, TargetPtr);
		}

		P_NATIVE_END;
	}
}

DEFINE_FUNCTION(UPCGMetadataAccessorHelpers::execSetAttributeValueByMetadataKey)
{
	P_GET_OBJECT(UPCGMetadata, Metadata)
	P_GET_PROPERTY_REF(FInt64Property, Key)

	PCGMetadataAccessorHelpers::GetSetAttributeValue<PCGMetadataEntryKey&, FPCGMetadataDomain>(Key, Metadata ? Metadata->GetDefaultMetadataDomain() : nullptr, Context, Stack, RESULT_PARAM);
}

DEFINE_FUNCTION(UPCGMetadataAccessorHelpers::execGetAttributeValueByMetadataKey)
{
	P_GET_OBJECT(const UPCGMetadata, Metadata)
	P_GET_PROPERTY(FInt64Property, Key)
	
	PCGMetadataAccessorHelpers::GetSetAttributeValue<PCGMetadataEntryKey, const FPCGMetadataDomain>(Key, Metadata ? Metadata->GetConstDefaultMetadataDomain() : nullptr, Context, Stack, RESULT_PARAM);
}

DEFINE_FUNCTION(UPCGMetadataAccessorHelpers::execSetAttributeValue)
{
	P_GET_OBJECT(UPCGMetadata, Metadata)
	P_GET_STRUCT_REF(FPCGPoint, Point)
	
	PCGMetadataAccessorHelpers::GetSetAttributeValue<PCGMetadataEntryKey&, FPCGMetadataDomain>(Point.MetadataEntry, Metadata ? Metadata->GetDefaultMetadataDomain() : nullptr, Context, Stack, RESULT_PARAM);
}

DEFINE_FUNCTION(UPCGMetadataAccessorHelpers::execGetAttributeValue)
{
	P_GET_OBJECT(const UPCGMetadata, Metadata)
	P_GET_STRUCT_REF(FPCGPoint, Point)
	
	PCGMetadataAccessorHelpers::GetSetAttributeValue<PCGMetadataEntryKey, const FPCGMetadataDomain>(Point.MetadataEntry, Metadata ? Metadata->GetConstDefaultMetadataDomain() : nullptr, Context, Stack, RESULT_PARAM);
}

DEFINE_FUNCTION(UPCGMetadataAccessorHelpers::execSetAttributeValueByMetadataKeyOnDomain)
{
	P_GET_STRUCT(FPCGMetadataDomainHandle, MetadataDomain)
	P_GET_PROPERTY_REF(FInt64Property, Key)
	
	PCGMetadataAccessorHelpers::GetSetAttributeValue<PCGMetadataEntryKey&, FPCGMetadataDomain>(Key, MetadataDomain.GetMutableDomain(), Context, Stack, RESULT_PARAM);
}

DEFINE_FUNCTION(UPCGMetadataAccessorHelpers::execGetAttributeValueByMetadataKeyOnDomain)
{
	P_GET_STRUCT(FPCGMetadataDomainHandle, MetadataDomain)
	P_GET_PROPERTY(FInt64Property, Key)

	PCGMetadataAccessorHelpers::GetSetAttributeValue<PCGMetadataEntryKey, const FPCGMetadataDomain>(Key, MetadataDomain.GetConstDomain(), Context, Stack, RESULT_PARAM);
}

#undef LOCTEXT_NAMESPACE