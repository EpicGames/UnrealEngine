// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGMetadataAttribute.h"

#include "Graph/PCGGraphCache.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataDomain.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Serialization/ArchiveCountMem.h"
#include "UObject/TextProperty.h"

namespace PCG::Private
{
	struct FPartitionResultBucketIndices
	{
		FPartitionResultBucketIndices(int32 InCount) {}

		void AddNewUniqueValue(int32 FoundValueKey)
		{
			check(!Result.IsValidIndex(FoundValueKey));
			Result.Emplace();
		}

		void AddValue(int32 FoundValueKey, int32 i)
		{
			check(Result.IsValidIndex(FoundValueKey));
			Result[FoundValueKey].Add(i);
		}

		void Finish(TArray<int32>& UniqueValuesIndices) {}

		TArray<TArray<int32>> Result;
	};

	struct FPartitionResultBucketBitArray
	{
		FPartitionResultBucketBitArray(int32 InCount)
		{
			check(InCount > 0);
			Count = InCount;
		}
		
		void AddNewUniqueValue(int32 FoundValueKey)
		{
			check(!Result.IsValidIndex(FoundValueKey));
			Result.Emplace_GetRef().Init(false, Count);
		}

		void AddValue(int32 FoundValueKey, int32 i)
		{
			check(Result.IsValidIndex(FoundValueKey));
			Result[FoundValueKey][i] = true;
		}

		void Finish(TArray<int32>& UniqueValuesIndices) {}

		TArray<TBitArray<>> Result;
		int32 Count = -1;
	};

	struct FPartitionResultUniqueIndices
	{
		FPartitionResultUniqueIndices(int32 InCount)
		{
			Result.Get<1>().Reserve(InCount);
		}
		
		void AddNewUniqueValue(int32 FoundValueKey) {}

		void AddValue(int32 FoundValueKey, int32 i)
		{
			Result.Get<1>().Emplace(FoundValueKey);
		}

		void Finish(TArray<int32>& UniqueValuesIndices)
		{
			Result.Get<0>() = MoveTemp(UniqueValuesIndices);
		}

		TTuple<TArray<int32>, TArray<PCGMetadataValueKey>> Result;
	};

	// Do a partition and write into one of the three result structs: FPartitionResultBucketIndices, FPartitionResultBucketBitArray, FPartitionResultUniqueIndices
	// Only support single values or arrays
	template <typename FPartitionResult>
	FPartitionResult MakePartition(TConstArrayView<TTuple<const void*, int32>> Values, const FProperty* InProperty)
	{
		static_assert(std::is_same_v<FPartitionResult, FPartitionResultBucketIndices> || std::is_same_v<FPartitionResult, FPartitionResultBucketBitArray> || std::is_same_v<FPartitionResult, FPartitionResultUniqueIndices>);
		
		const int32 Count = Values.Num();
		check(Count > 0);

		TArray<int32> UniqueValuesIndices;

		FPartitionResult Result{Count};

		auto Compare = [InProperty, Values](int32 LHS, int32 RHS) -> bool
		{
			auto [DataPtr1, NumElements1] = Values[LHS];
			auto [DataPtr2, NumElements2] = Values[RHS];
			return CompareArrays(InProperty, DataPtr1, NumElements1, DataPtr2, NumElements2);
		};

		// Hash-accelerated deduplication: O(n) average case.
		// Algorithm:
		//	- For each value in our incoming values
		//		* Get the value pointer
		//		* Compute the hash for this value
		//		* Find the hash in our map of already computed hashes
		//		* For each unique value with the same hash (collision), do the comparison
		//		* If it is found, we have our value key
		//		* If it is not found, we have a new unique value, so add it to our map.
		//
		// This reduces drastically the amount of comparisons to do at the cost of extra memory.
		if (InProperty->HasAllPropertyFlags(CPF_HasGetValueTypeHash))
		{
			TMap<uint32, TArray<int32, TInlineAllocator<4>>> HashBuckets;

			for (int32 i = 0; i < Count; ++i)
			{
				int32 FoundValueKey = INDEX_NONE;
				auto [ValuePtr, ArrayNum] = Values[i];
				TOptional<int32> Hash = ComputeHash(InProperty, ValuePtr, ArrayNum);

				if (Hash.IsSet())
				{
					if (TArray<int32, TInlineAllocator<4>>* Bucket = HashBuckets.Find(*Hash))
					{
						// Check within collision bucket using Identical for correctness
						for (const int32 UniqueIdx : *Bucket)
						{
							if (Compare(i, UniqueValuesIndices[UniqueIdx]))
							{
								FoundValueKey = UniqueIdx;
								break;
							}
						}
					}

					if (FoundValueKey == INDEX_NONE)
					{
						FoundValueKey = UniqueValuesIndices.Add(i);
						HashBuckets.FindOrAdd(*Hash).Add(FoundValueKey);
						Result.AddNewUniqueValue(FoundValueKey);
					}
				}
				else
				{
					// Failed hash (null pointer) - treat as unique (consistent with Compare returning false for nulls)
					FoundValueKey = UniqueValuesIndices.Add(i);
					Result.AddNewUniqueValue(FoundValueKey);
				}

				Result.AddValue(FoundValueKey, i);
			}
		}
		else
		{
			// Fallback for types without hash support: O(n*m) where n = Count and m is the number of unique values (worst case: all unique values -> n^2)
			for (int32 i = 0; i < Count; ++i)
			{
				int32 FoundValueKey = INDEX_NONE;
				for (int32 j = 0; j < UniqueValuesIndices.Num(); ++j)
				{
					if (Compare(i, UniqueValuesIndices[j]))
					{
						FoundValueKey = j;
						break;
					}
				}

				if (FoundValueKey == INDEX_NONE)
				{
					FoundValueKey = UniqueValuesIndices.Add(i);
					Result.AddNewUniqueValue(FoundValueKey);
				}

				Result.AddValue(FoundValueKey, i);
			}
		}

		Result.Finish(UniqueValuesIndices);

		return Result;
	}
}

FPCGAttributeDefaultValue::~FPCGAttributeDefaultValue()
{
	Destroy();
}

void FPCGAttributeDefaultValue::Init(TSharedPtr<FPCGAttributeProperty> InUnderlyingProperty)
{
	if (!InUnderlyingProperty || !InUnderlyingProperty->IsValid())
	{
		return;
	}

	// Check if we already have a property set. If the property is the same type, we can keep the allocated memory.
	// Otherwise we have to destroy it.
	if (UnderlyingProperty && UnderlyingProperty->IsValid() && Memory)
	{
		if (UnderlyingProperty->GetProperty()->SameType(InUnderlyingProperty->GetProperty()))
		{
			UnderlyingProperty = MoveTemp(InUnderlyingProperty);
			return;
		}
	}

	Destroy();

	UnderlyingProperty = MoveTemp(InUnderlyingProperty);
	Memory = UnderlyingProperty->GetProperty()->Inner->AllocateAndInitializeValue();
}

void FPCGAttributeDefaultValue::CopyFrom(const void* SrcData)
{
	check(Memory);
	UnderlyingProperty->Copy(Memory, SrcData, /*Count=*/1);
}

void FPCGAttributeDefaultValue::Destroy()
{
	if (Memory)
	{
		if (ensureMsgf(UnderlyingProperty && UnderlyingProperty->IsValid(),
			TEXT("Memory is allocated but we do not have an underlying property. This is ill-formed and will leak memory")))
		{
			UnderlyingProperty->GetProperty()->Inner->DestroyValue(Memory);
		}

		FMemory::Free(Memory);
	}

	Memory = nullptr;
}

namespace PCGMetadataAttributeConstants
{
	const FName LastAttributeName = TEXT("@Last");
	const FName LastCreatedAttributeName = TEXT("@LastCreated");
	const FName SourceAttributeName = TEXT("@Source");
	const FName SourceNameAttributeName = TEXT("@SourceName");
}

namespace PCGMetadataAttributeBase
{
	static constexpr TCHAR AllowedSpecialCharacters[4] = {' ', '_', '-', '/'};

	bool IsValidNameCharacter(TCHAR Character)
	{
		if (FChar::IsAlpha(Character) || FChar::IsDigit(Character))
		{
			return true;
		}

		for (const TCHAR AllowedSpecialCharacter : AllowedSpecialCharacters)
		{
			if (AllowedSpecialCharacter == Character)
			{
				return true;
			}
		}

		return false;
	}
}

FPCGMetadataAttributeBase::FPCGMetadataAttributeBase(FPCGMetadataDomain* InMetadata, FName InName, const FPCGMetadataAttributeBase* InParent, bool bInAllowsInterpolation)
	: Metadata(InMetadata)
	, Parent(InParent)
	, bAllowsInterpolation(bInAllowsInterpolation)
	, Name(InName)
{
}

FPCGMetadataAttributeBase::FPCGMetadataAttributeBase(const FPCGMetadataAttributeDesc& InAttributeDesc, FPCGMetadataDomain* InMetadata, bool bInAllowsInterpolation)
	: Metadata(InMetadata)
	, bAllowsInterpolation(bInAllowsInterpolation)
	, CachedDesc(InAttributeDesc)
	, UnderlyingProperty(MakeShared<FPCGAttributeProperty>(CachedDesc))
	, Name(InAttributeDesc.Name)
{
	// @todo_pcg support multi-containers
	if (!ensure(CachedDesc.ContainerTypes.Num() <= 1))
	{
		UnderlyingProperty.Reset();
		CachedDesc = FPCGMetadataAttributeDesc{};
		return;
	}

	Init();
}

FPCGMetadataAttributeBase::FPCGMetadataAttributeBase(TNotNull<const FPCGMetadataAttributeBase*> InParent, FPCGMetadataDomain* InMetadata, bool bInAllowsInterpolation, TOptional<FName> InName)
	: Metadata(InMetadata)
	, Parent(InParent)
	, bAllowsInterpolation(bInAllowsInterpolation)
	, CachedDesc(InParent->CachedDesc)
	, UnderlyingProperty(InParent->UnderlyingProperty)
	, Name(InName.Get(InParent->CachedDesc.Name))
{
	CachedDesc.Name = InName.Get(InParent->CachedDesc.Name);

	Init();

	ValueKeyOffset = InParent->GetValueKeyOffsetForChild();

	if (ensure(DefaultValue.IsValid() && InParent->DefaultValue.IsValid()))
	{
		// Copy the default value from the parent.
		DefaultValue.CopyFrom(InParent->DefaultValue.GetRawPtr());
	}
}

FPCGMetadataAttributeBase::FPCGMetadataAttributeBase(const FPCGMetadataAttributeDesc& InAttributeDesc, const FPCGMetadataAttributeBase* InParent, FPCGMetadataDomain* InMetadata, bool bInAllowsInterpolation)
	: Metadata(InMetadata)
	, Parent(InParent)
	, bAllowsInterpolation(bInAllowsInterpolation)
	, CachedDesc(InAttributeDesc)
	, UnderlyingProperty(InParent ? InParent->UnderlyingProperty : MakeShared<FPCGAttributeProperty>(CachedDesc))
	, Name(InAttributeDesc.Name)
{
	check(!InParent || InParent->GetAttributeDesc().IsSameType(CachedDesc));

	Init();

	ValueKeyOffset = InParent ? InParent->GetValueKeyOffsetForChild() : 0;

	if (InParent && ensure(DefaultValue.IsValid() && InParent->DefaultValue.IsValid()))
	{
		// Copy the default value from the parent.
		DefaultValue.CopyFrom(InParent->DefaultValue.GetRawPtr());
	}
}

FPCGMetadataAttributeBase::~FPCGMetadataAttributeBase()
{
	// We need to make sure to empty the array so the elements are destroyed and the memory is freed.
	if (TOptional<FScriptArrayHelper> Helper = MakeArrayHelper(); ensure(Values.IsEmpty() || Helper.IsSet()) && Helper)
	{
		Helper->EmptyValues();
	}
}


void FPCGMetadataAttributeBase::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
{
	const int SizeLevel = FPCGGraphCache::CacheSizeDetailLevel();
	// Lightweight - skip
	if (SizeLevel == 2)
	{
		return;
	}

	size_t ComputedResourceSize = 0;

	{
		PCG::TSharedScopeLock ReadLock(EntryMapLock);
		ComputedResourceSize += EntryToValueKeyMap.GetAllocatedSize();
	}

	// Full
	if (SizeLevel == 0)
	{
		// Note - can replace this with serialize call once it's there.
		FArchiveCountMem Archive(nullptr);
		// Serialize desc
		FPCGMetadataAttributeDesc::StaticStruct()->SerializeItem(FStructuredArchiveFromArchive(Archive).GetSlot(), const_cast<FPCGMetadataAttributeDesc*>(&CachedDesc), /*Defaults*/nullptr);

		if (UnderlyingProperty && UnderlyingProperty->GetProperty())
		{
			PCG::TSharedScopeLock ReadLock(ValueLock);
			if (UnderlyingProperty->IsPlainOldData())
			{
				// Here we add one for the default value.
				ComputedResourceSize += (1 + Values.Num()) * UnderlyingProperty->GetInnerElementSize();
			}
			else
			{
				if (UnderlyingProperty->GetProperty()->Inner && DefaultValue.GetRawPtr())
				{
					// Serialize default value
					UnderlyingProperty->GetProperty()->Inner->SerializeItem(FStructuredArchiveFromArchive(Archive).GetSlot(), const_cast<void*>(DefaultValue.GetRawPtr()), /*Defaults*/nullptr);
				}

				// Serialize values
				if (Values.Num() > 0)
				{
					UnderlyingProperty->GetProperty()->SerializeItem(FStructuredArchiveFromArchive(Archive).GetSlot(), const_cast<FScriptArray*>(&Values), /*Defaults*/nullptr);
				}
			}
		}

		ComputedResourceSize += Archive.GetNum();
	}
	// Approximation
	else if (UnderlyingProperty && UnderlyingProperty->GetProperty())
	{
		PCG::TSharedScopeLock ReadLock(ValueLock);
		// Approximation of the actual size - we'll ignore the attribute descriptor and just count the size of elements (and not their ancillary memory) + 1 for the default value.
		ComputedResourceSize += (1 + Values.Num()) * UnderlyingProperty->GetInnerElementSize();
	}

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(ComputedResourceSize);
}

PCGMetadataValueKey FPCGMetadataAttributeBase::GetValueKeyOffsetForChild() const
{
	PCG::TSharedScopeLock ReadLock(ValueLock);
	return Values.Num() + ValueKeyOffset;
}

void FPCGMetadataAttributeBase::BaseSerialize(FPCGMetadataDomain* InMetadata, FArchive& InArchive)
{
	InArchive << EntryToValueKeyMap;
	Metadata = InMetadata;

	int32 ParentAttributeId = (Parent ? Parent->AttributeId : -1);
	InArchive << ParentAttributeId;

	if (InArchive.IsLoading())
	{
		ensure(ParentAttributeId < 0 || Metadata->GetParent());
		if (ParentAttributeId >= 0 && Metadata->GetParent())
		{
			Parent = Metadata->GetParent()->GetConstAttributeById(ParentAttributeId);
			check(Parent);
		}
	}

	//Type id should already be known by then, so no need to serialize it
	InArchive << Name;
	InArchive << AttributeId;
}

void FPCGMetadataAttributeBase::Serialize(FPCGMetadataDomain* InMetadata, FArchive& InArchive)
{
	BaseSerialize(InMetadata, InArchive);

	// If we don't have a parent, we need to serialize the CachedDesc
	if (!Parent)
	{
		FPCGMetadataAttributeDesc::StaticStruct()->SerializeItem(FStructuredArchiveFromArchive(InArchive).GetSlot(), &CachedDesc, /*Defaults*/nullptr);
		if (InArchive.IsLoading() && !UnderlyingProperty)
		{
			UnderlyingProperty = MakeShared<FPCGAttributeProperty>(CachedDesc);
		}
	}
	else if (InArchive.IsLoading() )
	{
		CachedDesc = Parent->CachedDesc;
		CachedDesc.Name = Name;
		UnderlyingProperty = Parent->UnderlyingProperty;
	}

	if (InArchive.IsLoading() && !DefaultValue.IsValid())
	{
		// Initialize the attribute
		Init();
	}

	if (UnderlyingProperty && UnderlyingProperty->IsValid())
	{
		// Then serialize the default value
		if (DefaultValue.IsValid())
		{
			UnderlyingProperty->GetProperty()->Inner->SerializeItem(FStructuredArchiveFromArchive(InArchive).GetSlot(), DefaultValue.GetRawPtr(), /*Defaults*/nullptr);
		}

		// And finally serialize the values
		UnderlyingProperty->GetProperty()->SerializeItem(FStructuredArchiveFromArchive(InArchive).GetSlot(), &Values, /*Defaults*/nullptr);

		if (InArchive.IsLoading())
		{
			ValueKeyOffset = GetParent() ? GetParent()->GetValueKeyOffsetForChild() : 0;
		}
	}
}

void FPCGMetadataAttributeBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	// @todo_pcg: We should register all the stored objects
	// We'll have to see the cost of this, and if we don't hold object references for too long.
}

void FPCGMetadataAttributeBase::SerializeValuesForEntryKeys(TConstArrayView<PCGMetadataEntryKey> EntryKeys, FArchive& InArchive) const
{
	if (!UnderlyingProperty.IsValid())
	{
		return;
	}

	TArray<PCGMetadataValueKey> ValueKeys;
	GetValueKeys(EntryKeys, ValueKeys);
	check(ValueKeys.Num() == EntryKeys.Num());
	
	SerializeValues(ValueKeys, InArchive);
}

void FPCGMetadataAttributeBase::SerializeValues(TConstArrayView<PCGMetadataValueKey> ValueKeys, FArchive& InArchive) const
{
	if (!UnderlyingProperty.IsValid())
	{
		return;
	}

	const FProperty* InnerProperty = UnderlyingProperty->GetProperty()->Inner;

	PCG::TSharedScopeLock ReadLock(ValueLock);
	
	for (const PCGMetadataValueKey ValueKey : ValueKeys)
	{
		if (void* ValuePtr = const_cast<void*>(GetReadAddressForValueKey_Unsafe(ValueKey)))
		{
			InnerProperty->SerializeItem(FStructuredArchiveFromArchive(InArchive).GetSlot(), ValuePtr, /*Defaults*/nullptr);
		}
	}
}

TArray<PCGMetadataValueKey> FPCGMetadataAttributeBase::AddValues_Internal(const PCG::Private::FInValues& InValues, int32 Count, int32 ExistingIndex, bool bLockless, TOptional<TConstArrayView<int32>> OptionalIndices)
{
	using namespace PCG::Private;
	
	check(!OptionalIndices || Count == OptionalIndices->Num());
	
	if (const FInValuesSubset* InValuesSubset = InValues.TryGet<FInValuesSubset>())
	{
		if (OptionalIndices)
		{
			// If we have multiple layers of subsets, we need to extract the indices.
			// Note that the order we extract the layers is inner to outer, so Optional indices MUST be always smaller than the indices in the subset.
			// For example, if we have FirstLayer = Subset(OriginalValues, [0,1,3]) and SecondLayer = Subset(FirstLayer, [0, 2]), the first time we call AddValues_Internal,
			// we pass `SecondLayer`. We then call the function again with FirstLayer and OptionalIndices = [0, 2].
			// The second time, we call the function again with OriginalValues and OptionalIndices = [0, 3] (First and Third indices ([0, 2]) of the FirstLayer subset indices [0, 1, 3])
			check(InValuesSubset->Indices.Num() >= OptionalIndices->Num())

			TArray<int32, TInlineAllocator<256>> IndicesSubset;
			IndicesSubset.Reserve(OptionalIndices->Num());
			for (int32 i : *OptionalIndices)
			{
				IndicesSubset.Add(InValuesSubset->Indices[i]);
			}
			
			return AddValues_Internal(*static_cast<const FInValues*>(InValuesSubset->InValues), IndicesSubset.Num(), ExistingIndex, bLockless, MakeConstArrayView(IndicesSubset));
		}

		check(InValuesSubset->Indices.Num() <= Count);
		
		// Otherwise, forward the call with the underlying values and its indices.
		return AddValues_Internal(*static_cast<const FInValues*>(InValuesSubset->InValues), InValuesSubset->Indices.Num(), ExistingIndex, bLockless, MakeConstArrayView(InValuesSubset->Indices));
	}

	TArray<PCGMetadataValueKey> ValueKeys;

	int32 StartWriteIndex = INDEX_NONE;

	TOptional<FScriptArrayHelper> Helper = MakeArrayHelper();
	if (!Helper.IsSet() || Count <= 0)
	{
		return ValueKeys;
	}

	// Hard-requirement that when we write to the default value, we can only write a single value
	if (!ensure(ExistingIndex != ExistingIndexForDefaultValue || Count == 1))
	{
		Count = 1;
	}

	{
		PCG::TUniqueScopeLock WriteLock(ValueLock, /*bShouldLock=*/!bLockless);
		if (ExistingIndex == INDEX_NONE)
		{
			// We should never allocate values if we are not already locked.
			check(!bLockless || ValueLock.IsLocked());
			StartWriteIndex = AllocateValues_Unsafe(Count);
		}
		else if (ExistingIndex != ExistingIndexForDefaultValue)
		{
			// Expected to be already allocated.
			check(Helper->Num() >= ExistingIndex + Count);
			StartWriteIndex = ExistingIndex;
		}
		else
		{
			// Expected the default value to always be allocated.
			check(DefaultValue.GetRawPtr() != nullptr);
			StartWriteIndex = ExistingIndex;
		}

		check(StartWriteIndex != INDEX_NONE);

		auto GetRawPtr = [this, &Helper, StartWriteIndex](const int32 Index)
		{
			return StartWriteIndex == ExistingIndexForDefaultValue ? DefaultValue.GetRawPtr() : Helper->GetRawPtr(Index);
		};
		
		auto Loop = [&OptionalIndices, Count, StartWriteIndex](auto Callback)
		{
			if (!OptionalIndices)
			{
				for (int32 i = 0; i < Count; ++i)
				{
					Callback(/*ReadIndex=*/i, /*WriteIndex=*/StartWriteIndex + i);
				}
			}
			else
			{
				for (int32 i = 0; i < Count; ++i)
				{
					Callback(/*ReadIndex=*/(*OptionalIndices)[i], /*WriteIndex=*/StartWriteIndex + i);
				}
			}
		};

		Visit([this, &GetRawPtr, StartWriteIndex, Count, &OptionalIndices, &Loop](auto&& Value)
			{
				using T = std::decay_t<decltype(Value)>;
				if constexpr(std::is_same_v<T, FInValuesByValue>)
				{
					if (!OptionalIndices)
					{
						const void* SrcPtr = Value.InValues;
						void* DestPtr = GetRawPtr(StartWriteIndex);
						UnderlyingProperty->Copy(DestPtr, SrcPtr, Count);
					}
					else
					{
						for (int32 i = 0; i < OptionalIndices->Num(); ++i)
						{
							const void* SrcPtr = UnderlyingProperty->GetPtrInArray(Value.InValues, (*OptionalIndices)[i]);
							void* DestPtr = GetRawPtr(StartWriteIndex + i);
							UnderlyingProperty->Copy(DestPtr, SrcPtr, 1);
						}
					}
				}
				else if constexpr(std::is_same_v<T, FInValuesByPtr>)
				{
					Loop([this, &GetRawPtr, &Value](int32 ReadIndex, int32 WriteIndex)
					{
						const void* SrcPtr = Value.InValues[ReadIndex];
						void* DestPtr = GetRawPtr(WriteIndex);
						UnderlyingProperty->Copy(DestPtr, SrcPtr, 1);
					});
				}
				else if constexpr(std::is_same_v<T, FInValuesSubset>)
				{
					// Must be caught earlier
					checkNoEntry();
				}
				else if constexpr (std::is_same_v<T, FInValuesAsSet>)
				{
					const FSetProperty* InnerProperty = CastFieldChecked<FSetProperty>(UnderlyingProperty->GetProperty()->Inner);

					Loop([InnerProperty, &GetRawPtr, &Value](int32 ReadIndex, int32 WriteIndex)
					{
						FScriptSetHelper OtherHelper(InnerProperty, GetRawPtr(WriteIndex));
						const int32 Num = Value.InValues[ReadIndex].Num();

						for (int j = 0; j < Num; ++j)
						{
							OtherHelper.AddElement(Value.InValues[ReadIndex][j]);
						}
					});
				}
				else if constexpr (std::is_same_v<T, FInValuesAsMap>)
				{
					const FMapProperty* InnerProperty = CastFieldChecked<FMapProperty>(UnderlyingProperty->GetProperty()->Inner);

					Loop([InnerProperty, &GetRawPtr, &Value](int32 ReadIndex, int32 WriteIndex)
					{
						FScriptMapHelper OtherHelper(InnerProperty, GetRawPtr(WriteIndex));
						const int32 Num = Value.InValues[ReadIndex].Num();

						for (int j = 0; j < Num; ++j)
						{
							OtherHelper.AddPair(Value.InValues[ReadIndex][j].Key, Value.InValues[ReadIndex][j].Value);
						}
					});
				}
				else if constexpr (std::is_same_v<T, FInValuesAsArray>)
				{
					const FArrayProperty* InnerProperty = CastFieldChecked<FArrayProperty>(UnderlyingProperty->GetProperty()->Inner);

					Loop([InnerProperty, &GetRawPtr, &Value](int32 ReadIndex, int32 WriteIndex)
					{
						FScriptArrayHelper OtherHelper(InnerProperty, GetRawPtr(WriteIndex));
						const int32 Num = Value.InValues[ReadIndex].template Get<1>();
						if (InnerProperty->HasAnyPropertyFlags(CPF_IsPlainOldData))
						{
							OtherHelper.AddUninitializedValues(Num);
						}
						else
						{
							OtherHelper.AddValues(Num);
						}

						PCG::Private::CopyArray(InnerProperty, OtherHelper.GetRawPtr(), Value.InValues[ReadIndex].template Get<0>(), Num);
					});
				}
				else
				{
					static_assert(!std::is_same_v<T, T>, "Missing variant case for AddValues_Internal");
				}
			}, InValues);
	}

	ValueKeys.SetNum(Count);
	for (int ValueIndex = 0; ValueIndex < Count; ++ValueIndex)
	{
		ValueKeys[ValueIndex] = ValueKeyOffset + StartWriteIndex + ValueIndex;
	}

	return ValueKeys;
}

TArray<PCGMetadataValueKey> FPCGMetadataAttributeBase::AddCompressedValues(PCG::Private::FInValues InUniqueValues, int32 Count, TArray<PCGMetadataValueKey>& FoundValueKeys)
{
	using namespace PCG::Private;

	// For now only support FInValuesByValue/FInValuesSubset/FInValuesAsArray/FInValuesByPtr
	if (!ensure(InUniqueValues.IsType<FInValuesByValue>() || InUniqueValues.IsType<FInValuesSubset>() || InUniqueValues.IsType<FInValuesAsArray>() || InUniqueValues.IsType<FInValuesByPtr>()))
	{
		return {};
	}
	
	// FoundValueKeys should always be bigger than Count (since FoundValueKeys should represent the full range of values to add, while InUniqueValues/Count represent the unique values)
	if (!ensure(Count <= FoundValueKeys.Num()))
	{
		return {};
	}

	const bool bHasDuplicateValues = (Count != FoundValueKeys.Num());

	TArray<PCGMetadataValueKey> FoundUniqueValueKeys;
	TArray<PCGMetadataValueKey>& FoundKeys = (bHasDuplicateValues ? FoundUniqueValueKeys : FoundValueKeys);

	// Implementation note: when we don't have any duplicate values, the previously-set values in FoundValueKeys will be wiped out - this is intended
	int32 ValueKeySet = 0;
	bool bAtLeastOneValueNotFound = !FindValues(FoundKeys, InUniqueValues, Count, &ValueKeySet);

	if (bAtLeastOneValueNotFound)
	{
		// When we validated that we have values to add, we still have to go through again on the partial result because we can have race conditions
		// where 2 threads go through FindValues at the same time and both tries to add the values, breaking the assumption that value
		// keys are unique.
		PCG::TUniqueScopeLock Lock(ValueLock);

		// Call the ArrayView version since the Array version would reset the array.
		bAtLeastOneValueNotFound = !FindValues(MakeArrayView(FoundKeys), InUniqueValues, Count, &ValueKeySet, /*bShouldLock=*/false);

		if (bAtLeastOneValueNotFound)
		{
			// If there is at least one value not found, go through all the found keys (from the original unique values)
			// and keep the indices of all the original values that need to be added.
			FInValuesSubset ValueSubset{};
			ValueSubset.InValues = &InUniqueValues;
			ValueSubset.Indices.Reserve(Count);

			for (int ValueIndex = 0; ValueIndex < Count; ++ValueIndex)
			{
				if (FoundKeys[ValueIndex] == PCGNotFoundValueKey)
				{
					ValueSubset.Indices.Add(ValueIndex);
				}
			}

			const int32 SubCount = ValueSubset.Indices.Num();
			TArray<PCGMetadataValueKey> NewValueKeys = AddValues_Internal(FInValues{TInPlaceType<FInValuesSubset>{}, MoveTemp(ValueSubset)}, SubCount, /*ExistingIndex=*/INDEX_NONE, /*bLockless=*/true);

			int32 NewlyAddedValueKeyIndex = 0;
			for (int ValueIndex = 0; ValueIndex < Count; ++ValueIndex)
			{
				if (FoundKeys[ValueIndex] == PCGNotFoundValueKey)
				{
					FoundKeys[ValueIndex] = NewValueKeys[NewlyAddedValueKeyIndex++];
				}
			}
		}
	}

	// Remap to full array if needed
	if (bHasDuplicateValues)
	{
		for (PCGMetadataValueKey& ValueToRemap : FoundValueKeys)
		{
			ValueToRemap = FoundUniqueValueKeys[ValueToRemap];
		}
	}

	return FoundValueKeys;
}

TArray<PCGMetadataValueKey> FPCGMetadataAttributeBase::AddCompressedValues(const PCG::Private::FInValues& InValues, int32 Count)
{
	using namespace PCG::Private;
	
	if (!UnderlyingProperty || !UnderlyingProperty->IsValid() || Count <= 0 || !ensure(InValues.IsType<FInValuesByValue>() || InValues.IsType<FInValuesAsArray>() || InValues.IsType<FInValuesByPtr>()))
	{
		return {};
	}

	check(UnderlyingProperty->CompressData());

	// Since we're getting raw values here, we might have duplicates
	// so we should aim to remove duplicates here so we preserve our 'compress data' idea, otherwise it will break other foundational blocks (e.g. partition)
	const FProperty* InnerProperty = UnderlyingProperty->GetProperty()->Inner;

	const bool bIsArray = InValues.IsType<FInValuesAsArray>();
	const bool bIsPtr = InValues.IsType<FInValuesByPtr>();
	InnerProperty = bIsArray ? CastFieldChecked<FArrayProperty>(InnerProperty)->Inner : InnerProperty;

	check(InnerProperty);
	check(bIsArray == CachedDesc.IsArray())
	
	TArray<TTuple<const void*, int32>> ValuesToCompare;
	TConstArrayView<TTuple<const void*, int32>> ValuesToCompareView;
	if (const FInValuesByValue* InValuesByValue = InValues.TryGet<FInValuesByValue>())
	{
		ValuesToCompare.Reserve(Count);
		for (int32 i = 0; i < Count; ++i)
		{
			ValuesToCompare.Emplace(UnderlyingProperty->GetPtrInArray(InValuesByValue->InValues, i), 1);
		}

		ValuesToCompareView = MakeConstArrayView(ValuesToCompare);
	}
	else if (const FInValuesByPtr* InValuesByPtr = InValues.TryGet<FInValuesByPtr>())
	{
		ValuesToCompare.Reserve(Count);
		for (int32 i = 0; i < Count; ++i)
		{
			ValuesToCompare.Emplace(InValuesByPtr->InValues[i], 1);
		}

		ValuesToCompareView = MakeConstArrayView(ValuesToCompare);
	}
	else if (const FInValuesAsArray* InValuesAsArray = InValues.TryGet<FInValuesAsArray>())
	{
		ValuesToCompareView = InValuesAsArray->InValues;
	}
	else
	{
		checkNoEntry();
	}

	auto [UniqueValuesIndices, FoundValueKeys] = MakePartition<FPartitionResultUniqueIndices>(ValuesToCompareView, InnerProperty).Result;

	const int32 UniqueCount = UniqueValuesIndices.Num();
	return AddCompressedValues(FInValues{TInPlaceType<FInValuesSubset>{}, &InValues, MoveTemp(UniqueValuesIndices)}, UniqueCount, FoundValueKeys);
}

bool FPCGMetadataAttributeBase::FindValues(TArray<PCGMetadataValueKey>& OutValueKeys, const PCG::Private::FInValues& InValues, int32 Count, int32* OutValueKeysSet, bool bShouldLock) const
{
	OutValueKeys.Init(PCGNotFoundValueKey, Count);

	return FindValues(MakeArrayView(OutValueKeys), InValues, Count, OutValueKeysSet, bShouldLock);
}

bool FPCGMetadataAttributeBase::FindValues(TArrayView<PCGMetadataValueKey> OutValueKeys, const PCG::Private::FInValues& InValues, int32 Count, int32* OutValueKeysSet, bool bShouldLock) const
{
	check(OutValueKeys.Num() == Count);
	int TempValueKeysSet = 0;
	int& ValueKeysSet = OutValueKeysSet ? *OutValueKeysSet : TempValueKeysSet;
	FindValuesInternal(InValues, Count, OutValueKeys, ValueKeysSet, /*bIsRoot=*/true, /*bShouldLock=*/bShouldLock);

	return (ValueKeysSet == Count);
}

void FPCGMetadataAttributeBase::FindValuesInternal(const PCG::Private::FInValues& InValues, int32 Count, TArrayView<PCGMetadataValueKey> ValueKeys, int& ValueKeysSet, bool bIsRoot, bool bShouldLock) const
{
	using namespace PCG::Private;

	check(Count == ValueKeys.Num());

	if (!ensure(UnderlyingProperty))
	{
		return;
	}
	
	// For now only support FInValuesByValue/FInValuesSubset/FInValuesAsArray/FInValuesByPtr
	const TArray<int32>* OptionalIndices = nullptr;
	const void* UnderlyingValues = nullptr;
	TArrayView<TTuple<const void*, int32>> UnderlyingValuesAsArray;
	TConstArrayView<const void*> UnderlyingValuesAsPtr;

	bool bIsArray = false;
	bool bIsPtr = false;

	Visit([this, &OptionalIndices, &UnderlyingValues, &UnderlyingValuesAsArray, &UnderlyingValuesAsPtr, &bIsArray, &bIsPtr](auto&& InValue)
	{
		using T = std::decay_t<decltype(InValue)>;
		if constexpr (std::is_same_v<T, FInValuesAsSet> || std::is_same_v<T, FInValuesAsMap>)
		{
			// Unsupported
			return;
		}
		else if constexpr (std::is_same_v<T, FInValuesByValue>)
		{
			UnderlyingValues = InValue.InValues;
		}
		else if constexpr (std::is_same_v<T, FInValuesByPtr>)
		{
			UnderlyingValuesAsPtr = InValue.InValues;
			bIsPtr = true;
		}
		else if constexpr (std::is_same_v<T, FInValuesSubset>)
		{
			const FInValues* UnderlyingVariant = static_cast<const FInValues*>(InValue.InValues);
			OptionalIndices = &InValue.Indices;

			if (const FInValuesByValue* UnderlyingValue = UnderlyingVariant->TryGet<FInValuesByValue>())
			{
				UnderlyingValues = UnderlyingValue->InValues;
			}
			else if (const FInValuesAsArray* UnderlyingValueAsArray = UnderlyingVariant->TryGet<FInValuesAsArray>())
			{
				UnderlyingValuesAsArray = UnderlyingValueAsArray->InValues;
				bIsArray = true;
			}
			else if (const FInValuesByPtr* UnderlyingValueAsPtr = UnderlyingVariant->TryGet<FInValuesByPtr>())
			{
				UnderlyingValuesAsPtr = UnderlyingValueAsPtr->InValues;
				bIsPtr = true;
			}
			else
			{
				// Unsupported
				return;
			}
		}
		else if constexpr (std::is_same_v<T, FInValuesAsArray>)
		{
			UnderlyingValuesAsArray = InValue.InValues;
			bIsArray = true;
		}
		else
		{
			static_assert(!std::is_same_v<T, T>, "Missing variant case for FindValuesInternal");
		}
	}, InValues);

	if (!UnderlyingValues && UnderlyingValuesAsArray.IsEmpty() && UnderlyingValuesAsPtr.IsEmpty())
	{
		// Unsupported
		return;
	}

	// Can only have one at the same time.
	const int Set = (UnderlyingValues != nullptr) + !UnderlyingValuesAsArray.IsEmpty() + !UnderlyingValuesAsPtr.IsEmpty();
	check(Set == 1);
	check(!(bIsArray && bIsPtr));

	const FProperty* InnerProperty = UnderlyingProperty->GetProperty()->Inner;

	InnerProperty = bIsArray ? CastFieldChecked<FArrayProperty>(InnerProperty)->Inner : InnerProperty;

	check(InnerProperty);
	check(bIsArray == CachedDesc.IsArray())

	auto Compare = [this, InnerProperty, UnderlyingValues, UnderlyingValuesAsArray, UnderlyingValuesAsPtr, bIsArray, bIsPtr](int32 InValueIndex, PCGMetadataValueKey ValueKey) -> bool
	{
		if (bIsArray)
		{
			// This is a FScriptArray, we need to access the data ptr and the number of elements
			const void* ArrayValue = GetReadAddressForValueKey_Unsafe(ValueKey);
			const void* DataPtr1 = ArrayValue ? static_cast<const FScriptArray*>(ArrayValue)->GetData() : nullptr;
			const int32 NumElements1 = ArrayValue ? static_cast<const FScriptArray*>(ArrayValue)->Num() : 0;
			
			auto [DataPtr2, NumElements2] = UnderlyingValuesAsArray[InValueIndex];

			return CompareArrays(InnerProperty, DataPtr1, NumElements1, DataPtr2, NumElements2);
		}
		else
		{
			const void* Value1 = GetReadAddressForValueKey_Unsafe(ValueKey);
			const void* Value2 = bIsPtr ? UnderlyingValuesAsPtr[InValueIndex] : UnderlyingProperty->GetPtrInArray(UnderlyingValues, InValueIndex);
			return CompareArrays(InnerProperty, Value1, 1, Value2, 1);
		}
	};

	if (bIsRoot)
	{
		for (int ValueIndex = 0; ValueIndex < Count; ++ValueIndex)
		{
			if (ValueKeys[ValueIndex] != PCGNotFoundValueKey)
			{
				continue;
			}

			if (Compare(OptionalIndices ? (*OptionalIndices)[ValueIndex] : ValueIndex, PCGDefaultValueKey))
			{
				ValueKeys[ValueIndex] = PCGDefaultValueKey;
				++ValueKeysSet;
			}
		}
	}

	// First look for existing values at this level
	if (ValueKeysSet != Count)
	{
		PCG::TSharedScopeLock ReadLock(ValueLock, /*bShouldLock=*/bShouldLock);
		for (int ValueIndex = 0; ValueIndex < Count; ++ValueIndex)
		{
			if (ValueKeys[ValueIndex] != PCGNotFoundValueKey)
			{
				continue;
			}

			for (PCGMetadataValueKey ValueKey = 0; ValueKey < Values.Num(); ++ValueKey)
			{
				const PCGMetadataValueKey OffsetKey = ValueKey + ValueKeyOffset;
				if (Compare(OptionalIndices ? (*OptionalIndices)[ValueIndex] : ValueIndex, OffsetKey))
				{
					ValueKeys[ValueIndex] = OffsetKey;
					++ValueKeysSet;
					break;
				}
			}
		}
	}

	// Then, if there are unfound values remaining, check in the parent.
	if (ValueKeysSet != Count)
	{
		if (const FPCGMetadataAttributeBase* ThisParent = GetParent())
		{
			// No need for lock for parent as it is assumed it is immutable.
			ThisParent->FindValuesInternal(InValues, Count, ValueKeys, ValueKeysSet, /*bIsRoot=*/false, /*bShouldLock=*/false);
		}
	}
}

void FPCGMetadataAttributeBase::Init()
{
	if (!ensure(UnderlyingProperty) || !ensure(!DefaultValue.IsValid()) || !UnderlyingProperty->IsValid())
	{
		return;
	}

	// In order to be consistent between multiple equivalent descriptors, we will convert structs that matches a predefined type.
	CachedDesc.FixLegacyTypeId();

	// Initialize the default value
	DefaultValue.Init(UnderlyingProperty);
}

void FPCGMetadataAttributeBase::Flatten()
{
	// Implementation notes:
	// We don't need to flatten the EntryToValueKeyMap - this will have been taken care of in the metadata flatten

	// Flatten values, from root to current attribute
	if (Parent && UnderlyingProperty)
	{
		PCG::TUniqueScopeLock WriteLock(ValueLock);

		TArray<const FScriptArray*> OriginalValues;

		int32 ValueCount = 0;

		const FPCGMetadataAttributeBase* Current = this;
		while (Current)
		{
			ValueCount += Current->Values.Num();
			OriginalValues.Add(&Current->Values);

			if (Current->Parent)
			{
				Current = Current->Parent;
			}
			else
			{
				Current = nullptr;
			}
		}

		// Swap the current values to an empty one
		FScriptArray NewValues{};
		FScriptArrayHelper NewValuesHelper{UnderlyingProperty->GetProperty(), &NewValues};

		if (UnderlyingProperty->IsPlainOldData())
		{
			NewValuesHelper.AddUninitializedValues(ValueCount);
		}
		else
		{
			NewValuesHelper.AddValues(ValueCount);
		}

		// Then copy all the values in reverse order (oldest first)
		int32 Count = 0;
		for (int32 ValuesIndex = OriginalValues.Num() - 1; ValuesIndex >= 0; --ValuesIndex)
		{
			FScriptArrayHelper OtherHelper{UnderlyingProperty->GetProperty(), OriginalValues[ValuesIndex]};
			const int32 OtherCount = OtherHelper.Num();
			UnderlyingProperty->Copy(NewValuesHelper.GetRawPtr(Count), OtherHelper.GetRawPtr(0), /*Count=*/OtherCount);
			Count += OtherCount;
		}

		// Finally move the new values in place of the old values.
		if (TOptional<FScriptArrayHelper> Helper = MakeArrayHelper(); ensure(Helper.IsSet()))
		{
			Helper->MoveAssign(&NewValues);
		}
	}

	// Reset value offset, and lose parent
	ValueKeyOffset = 0;
	Parent = nullptr;
}

void FPCGMetadataAttributeBase::FlattenAndCompress(const TArrayView<const PCGMetadataEntryKey>& InEntryKeysToKeep)
{
	// No entries, we can just delete everything in the attribute.
	if (InEntryKeysToKeep.IsEmpty() || !UnderlyingProperty)
	{
		Reset();
		return;
	}

	TArray<PCGMetadataValueKey> AllValueKeys;
	TArray<PCGMetadataValueKey> AllUniqueValueKeys;
	AllValueKeys.Reserve(InEntryKeysToKeep.Num());
	bool bUseValueKeys = UsesValueKeys();
	if (bUseValueKeys)
	{
		AllUniqueValueKeys.Reserve(InEntryKeysToKeep.Num());
	}

	// First gather all value keys associated with the entry keys to keep.
	// If we compress data, we also store the unique value keys used (that is not default).
	// We need to keep the number of values that are not the default value.
	int32 NonDefaultValuesNum = 0;
	for (const PCGMetadataEntryKey& EntryKey : InEntryKeysToKeep)
	{
		AllValueKeys.Add(GetValueKey(EntryKey));
		if (AllValueKeys.Last() != PCGDefaultValueKey)
		{
			NonDefaultValuesNum++;
			if (bUseValueKeys)
			{
				AllUniqueValueKeys.AddUnique(AllValueKeys.Last());
			}
		}
	}

	// Then for each value key (or unique values keys), gather the value in a NewValues array
	// and also keep a mapping between old value key and new value key.
	// Only done if the old value key is not the default one.
	TMap<PCGMetadataValueKey, PCGMetadataValueKey> ValueKeyMapping;
	FScriptArray NewValues{};
	FScriptArrayHelper NewValuesHelper{UnderlyingProperty->GetProperty(), &NewValues};

	const TArray<PCGMetadataValueKey>& AllValueKeysRef = !bUseValueKeys ? AllValueKeys : AllUniqueValueKeys;
	const int32 NumValuesToAllocate = !bUseValueKeys ? NonDefaultValuesNum : AllUniqueValueKeys.Num();
	if (UnderlyingProperty->IsPlainOldData())
	{
		NewValuesHelper.AddUninitializedValues(NumValuesToAllocate);
	}
	else
	{
		NewValuesHelper.AddValues(NumValuesToAllocate);
	}

	ValueKeyMapping.Reserve(NumValuesToAllocate);

	int32 Count = 0;
	for (PCGMetadataValueKey ValueKey : AllValueKeysRef)
	{
		if (ValueKey != PCGDefaultValueKey)
		{
			ValueKeyMapping.Add(ValueKey, Count);

			// @todo_pcg: Improve to find all contiguous values before copying.
			const void* SrcPtr = GetReadAddressForValueKey_Unsafe(ValueKey);
			UnderlyingProperty->Copy(NewValuesHelper.GetRawPtr(Count++), SrcPtr, /*Count=*/1);
		}
	}

	// Move the new values in place of the old values.
	if (TOptional<FScriptArrayHelper> Helper = MakeArrayHelper(); ensure(Helper.IsSet()))
	{
		PCG::TUniqueScopeLock WriteLock(ValueLock);
		Helper->MoveAssign(&NewValues);
	}

	// And finally, create a new entry to value mapping.
	// Logic is that each entry to keep will have their "index" as new entry key
	// (like if the entries to keep are [25, 47, 54], the new entries would be [0, 1, 2]).
	// So the operation is:
	// All pairs Old EK -> Old VK transform to New EK -> New VK.
	TMap<PCGMetadataEntryKey, PCGMetadataValueKey> NewMap;
	for (int32 i = 0; i < InEntryKeysToKeep.Num(); ++i)
	{
		PCGMetadataValueKey ValueKey = AllValueKeys[i];
		if (ValueKey != PCGDefaultValueKey && ensure(InEntryKeysToKeep[i] != PCGInvalidEntryKey))
		{
			NewMap.Add(i, ValueKeyMapping[ValueKey]);
		}
	}

	// And move the map.
	{
		PCG::TUniqueScopeLock WriteLock(EntryMapLock);
		EntryToValueKeyMap = MoveTemp(NewMap);
	}

	// At the end, reset value offset, and lose parent.
	ValueKeyOffset = 0;
	Parent = nullptr;
}

void FPCGMetadataAttributeBase::Reset()
{
	ValueKeyOffset = 0;
	Parent = nullptr;

	{
		PCG::TUniqueScopeLock WriteLock(EntryMapLock);
		EntryToValueKeyMap.Empty();
	}

	if (TOptional<FScriptArrayHelper> Helper = MakeArrayHelper(); Helper.IsSet())
	{
		PCG::TUniqueScopeLock WriteLock(ValueLock);
		Helper->EmptyValues();
	}
}

FPCGMetadataAttributeBase* FPCGMetadataAttributeBase::Copy(FName NewName, FPCGMetadataDomain* InMetadata, bool bKeepParent, bool bCopyEntries, bool bCopyValues) const
{
	// If we copy an attribute where we don't want to keep the parent, while copying entries and/or values, we'll lose data.
	// In that case, we will copy all the data from this attribute and all its ancestors.

	// We can't keep the parent if we don't have the same root.
	checkSlow(!bKeepParent || PCGMetadataHelpers::HasSameRoot(Metadata, InMetadata));

	// Validate that the new name is valid
	if (!IsValidName(NewName))
	{
		UE_LOGF(LogPCG, Error, "Try to create a new attribute with an invalid name: %ls", *NewName.ToString());
		return nullptr;
	}

	// This copies to a new attribute.
	FPCGMetadataAttributeBase* AttributeCopy = nullptr;
	if (bKeepParent)
	{
		AttributeCopy = new FPCGMetadataAttributeBase(this, InMetadata, bAllowsInterpolation, NewName);
	}
	else
	{
		FPCGMetadataAttributeDesc NewDesc = CachedDesc;
		NewDesc.Name = NewName;
		AttributeCopy = new FPCGMetadataAttributeBase(NewDesc, InMetadata, bAllowsInterpolation);
	}

	CopyInternal(AttributeCopy, bKeepParent, bCopyEntries, bCopyValues);

	return AttributeCopy;
}

FPCGMetadataAttributeBase* FPCGMetadataAttributeBase::CopyToAnotherType(int16 Type) const
{
	// This was done to support casting from one type to another, where we knew at compile time both types (source and target).
	// Without it, we would need a way at runtime to know what is compatible and how to convert the values. In the meantime, this is not supported.
	return nullptr;
}

void FPCGMetadataAttributeBase::CopyInternal(FPCGMetadataAttributeBase* NewAttribute, bool bKeepParent, bool bCopyEntries, bool bCopyValues, FCopyFunctor CopyFunctor) const
{
	check(NewAttribute);

	if (!CopyFunctor)
	{
		// If we do not have an explicit functor, just use the default copy.
		CopyFunctor = [&UnderlyingProperty = this->UnderlyingProperty](void* DestData, const void* SrcData, const int32 Count)
		{
			UnderlyingProperty->Copy(DestData, SrcData, Count);
		};
	}

	check(CopyFunctor);

	if (ensure(DefaultValue.IsValid()) && ensure(NewAttribute->DefaultValue.IsValid()))
	{
		// Copy this default value to the attribute copy.
		CopyFunctor(NewAttribute->DefaultValue.GetRawPtr(), DefaultValue.GetRawPtr(), 1);
	}

	// Gather the chain of parents if we don't keep the parent and we want to copy entries/values.
	// We always have at least one item, "this".
	TArray<const FPCGMetadataAttributeBase*, TInlineAllocator<2>> Parents = { this };
	if (!bKeepParent && (bCopyEntries || bCopyValues))
	{
		const FPCGMetadataDomain* CurrentMetadata = Metadata;
		const FPCGMetadataAttributeBase* Current = this;

		const FPCGMetadataDomain* ParentMetadata = PCGMetadataHelpers::GetParentMetadata(CurrentMetadata);
		while (ParentMetadata && Current->Parent)
		{
			CurrentMetadata = ParentMetadata;
			Current = Current->Parent;
			Parents.Add(Current);

			ParentMetadata = PCGMetadataHelpers::GetParentMetadata(CurrentMetadata);
		}
	}

	if (bCopyEntries)
	{
		// We go backwards, since we need to preserve order (root -> this)
		// Latest entry in our Parents array is the root.
		for (int32 i = Parents.Num() - 1; i >= 0; --i)
		{
			const FPCGMetadataAttributeBase* Current = Parents[i];

			// Only need to lock if Current == this, since the parents are assumed to be const.
			PCG::TSharedScopeLock ReadLock(Current->EntryMapLock, /*bShouldLock=*/Current == this);
			NewAttribute->EntryToValueKeyMap.Append(Current->EntryToValueKeyMap);
		}
	}

	if (bCopyValues)
	{
		// We go backwards, since we need to preserve order (root -> this)
		// Latest entry in our Parents array is the root.
		if (TOptional<FScriptArrayHelper> Helper = NewAttribute->MakeArrayHelper(); Helper.IsSet())
		{
			// Make sure to start from a fresh start, as we already have a default value allocated.
			Helper->EmptyValues();

			for (int32 i = Parents.Num() - 1; i >= 0; --i)
			{
				const FPCGMetadataAttributeBase* Current = Parents[i];

				// Only need to lock if Current == this, since the parents are assumed to be const.
				PCG::TSharedScopeLock ReadLock(Current->ValueLock, /*bShouldLock=*/Current == this);

				TOptional<FScriptArrayHelper> ParentHelper = Current->MakeArrayHelper();
				if (!ParentHelper.IsSet())
				{
					continue;
				}

				const int32 NumValuesToCopy = ParentHelper->Num();
				if (NumValuesToCopy <= 0)
				{
					continue;
				}

				const int32 StartIndex = NewAttribute->AllocateValues_Unsafe(NumValuesToCopy);

				if (StartIndex == INDEX_NONE)
				{
					continue;
				}

				CopyFunctor(Helper->GetRawPtr(StartIndex), ParentHelper->GetRawPtr(0), NumValuesToCopy);
			}
		}
	}
}

void FPCGMetadataAttributeBase::SetValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttribute, PCGMetadataEntryKey InEntryKey)
{
	using namespace PCG::Private;

	if (!InAttribute || !InAttribute->GetAttributeDesc().IsSameType(CachedDesc))
	{
		return;
	}

	TOptional<FScriptArrayHelper> ThisHelper = MakeArrayHelper();
	if (!ThisHelper.IsSet())
	{
		return;
	}

	check(ItemKey != PCGInvalidEntryKey);

	const void* SrcPtr = InAttribute->GetReadAddressFromEntryKey_Unsafe(InEntryKey);
	if (!SrcPtr)
	{
		return;
	}

	PCGMetadataValueKey ValueKeyToAdd = PCGNotFoundValueKey;

	if (UnderlyingProperty->CompressData())
	{
		bool bValueKeyFound = false;
		if (CachedDesc.IsArray())
		{
			// This is a FScriptArray, we need to access the data ptr and the number of elements
			const void* DataPtr = static_cast<const FScriptArray*>(SrcPtr)->GetData();
			const int32 NumElements = static_cast<const FScriptArray*>(SrcPtr)->Num();
			TTuple<const void*, int32> DataTuple(DataPtr, NumElements);
			bValueKeyFound = FindValues(MakeArrayView(&ValueKeyToAdd, 1), FInValues{TInPlaceType<FInValuesAsArray>{}, MakeArrayView(&DataTuple, 1)}, 1);
		}
		else
		{
			bValueKeyFound = FindValues(MakeArrayView(&ValueKeyToAdd, 1), FInValues{TInPlaceType<FInValuesByValue>{}, SrcPtr, 1}, 1);
		}
		check(!bValueKeyFound || ValueKeyToAdd != PCGNotFoundValueKey);
	}

	// If we are not compressed or we haven't find the value, we need to add it.
	if (ValueKeyToAdd == PCGNotFoundValueKey)
	{
		PCG::TUniqueScopeLock WriteLock(ValueLock);

		// Add the value in the buffer
		const int32 NewValueIndex = ThisHelper->AddValue();

		// If InAttribute == this, SrcPtr might be dangling now because of possible reallocation, so fix it there
		if (InAttribute == this)
		{
			SrcPtr = InAttribute->GetReadAddressFromEntryKey_Unsafe(InEntryKey);
		}

		check(SrcPtr);

		ValueKeyToAdd = ValueKeyOffset + NewValueIndex;
		void* DestPtr = ThisHelper->GetRawPtr(NewValueIndex);
		UnderlyingProperty->Copy(DestPtr, SrcPtr);
	}

	if (ensure(ValueKeyToAdd != PCGNotFoundValueKey))
	{
		PCG::TUniqueScopeLock WriteLock(EntryMapLock);
		EntryToValueKeyMap.Emplace(ItemKey, ValueKeyToAdd);
	}
}

void FPCGMetadataAttributeBase::SetZeroValue(PCGMetadataEntryKey ItemKey)
{
	// Not supported, will need to be specialized in the templated versions.
}

void FPCGMetadataAttributeBase::AccumulateValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA,
	PCGMetadataEntryKey InEntryKeyA, float Weight)
{
	// Not supported, will need to be specialized in the templated versions.
}

void FPCGMetadataAttributeBase::SetWeightedValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttribute,
	const TArrayView<const TPair<PCGMetadataEntryKey, float>>& InWeightedKeys)
{
	// Not supported, will need to be specialized in the templated versions.
}

void FPCGMetadataAttributeBase::SetValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA,
	PCGMetadataEntryKey InEntryKeyA, const FPCGMetadataAttributeBase* InAttributeB, PCGMetadataEntryKey InEntryKeyB, EPCGMetadataOp Op)
{
	// Not supported, will need to be specialized in the templated versions.
}

bool FPCGMetadataAttributeBase::IsEqualToDefaultValue(PCGMetadataValueKey ValueKey) const
{
	if (ValueKey == PCGDefaultValueKey)
	{
		return true;
	}

	if (ensure(UnderlyingProperty) && UnderlyingProperty->CompressData())
	{
		return false;
	}
	else
	{
		return AreValuesEqual(ValueKey, PCGDefaultValueKey);
	}
}

void FPCGMetadataAttributeBase::SetDefaultValueToFirstEntry()
{
	if (Values.Num() == 0 || Values.Num() > 1 || !ensure(UnderlyingProperty) || !DefaultValue.IsValid())
	{
		return;
	}

	{
		PCG::TUniqueScopeLock WriteLock(ValueLock);

		// First value key of this attribute.
		const void* SrcPtr = GetReadAddressForValueKey_Unsafe(ValueKeyOffset);
		if (!SrcPtr)
		{
			return;
		}

		void* DestPtr = DefaultValue.GetRawPtr();

		UnderlyingProperty->Copy(DestPtr, SrcPtr);
	}
}

bool FPCGMetadataAttributeBase::UsesValueKeys() const
{
	return ensure(UnderlyingProperty) ? UnderlyingProperty->CompressData() : false;
}

bool FPCGMetadataAttributeBase::AreValuesEqualForEntryKeys(PCGMetadataEntryKey EntryKey1, PCGMetadataEntryKey EntryKey2) const
{
	return AreValuesEqual(GetValueKey(EntryKey1), GetValueKey(EntryKey2));
}

bool FPCGMetadataAttributeBase::AreValuesEqual(PCGMetadataValueKey ValueKey1, PCGMetadataValueKey ValueKey2) const
{
	if (!ensure(UnderlyingProperty))
	{
		return false;
	}

	if (ValueKey1 == ValueKey2)
	{
		return true;
	}

	{
		PCG::TSharedScopeLock ReadLock(ValueLock);

		const void* Value1 = GetReadAddressForValueKey_Unsafe(ValueKey1);
		const void* Value2 = GetReadAddressForValueKey_Unsafe(ValueKey2);

		if (Value1 == nullptr || Value2 == nullptr)
		{
			return false;
		}

		return UnderlyingProperty->GetProperty()->Inner->Identical(Value1, Value2);
	}
}

/**
 * In this function, we want to gather all the values are that given by the values keys, but only those who are at this level (aka. ValueKey >= ValueKeyOffset).
 * UnretrievedValues will hold all the values that didn't managed to get because they are stored higher in the attribute hierarchy. The algorithm stops when
 * all the values are retrieved.
 * Extraction of the values will depend on the FOutValues variant, since we need to distinguished between those 5 possibilities:
 * - Output by Values (copy)
 * - Output by Pointer
 * - Output as Array
 * - Output as Set
 * - Output as Map
 *
 * On top of that, in order to reduce as much as we can virtual function calls, we also track for the different un-retrieved values if they are contiguous in memory, for input AND output.
 * Values are contiguous in memory if their index in the output array are contiguous AND the values keys associated with them are also contiguous.
 * For example, if we have indices {0, 1, 2, 5} (because we still haven't retrieved values for those indices), and we have value keys {1, 2, 8, 9}, we have a contiguous
 * range for the first two (their indices AND values keys are contiguous), but not for the rest.
 * So we can do 3 extractions (in this example [Index:ValueKey]): [0:1 ,1:2], [2:8] and [5:9] and optimize the first one to reduce the number of virtual calls.
 */
void FPCGMetadataAttributeBase::GetValues_Internal(const TArrayView<const PCGMetadataValueKey> ValueKeys, PCG::Private::FOutValues OutValues, TBitArray<>& UnretrievedValues) const
{
	using namespace PCG::Private;

	bool bFoundAllKeys = true;
	TConstSetBitIterator<> It(UnretrievedValues);
	if (!It)
	{
		return;
	}

	const FPCGMetadataAttributeBase* ThisParent = static_cast<const FPCGMetadataAttributeBase*>(GetParent());
	TOptional<FScriptArrayHelper> Helper = MakeArrayHelper();
	if (!Helper)
	{
		return;
	}

	{
		PCG::TSharedScopeLock ReadLock(ValueLock);

		struct FContinuousRange
		{
			void Reset()
			{
				StartIndex = INDEX_NONE;
				StartValueKey = INDEX_NONE;
				Count = 0;
			}

			void Init(int32 InStartIndex, int64 InStartValueKey)
			{
				StartIndex = InStartIndex;
				StartValueKey = InStartValueKey;
				Count = 1;
			}

			int32 StartIndex = INDEX_NONE;
			int64 StartValueKey = INDEX_NONE;
			int32 Count = 0;
		};

		FContinuousRange ContinuousRange{};

		auto RetrieveValues = [this, &OutValues, &Helper](FContinuousRange& Range)
		{
			const uint8* SrcPtr = nullptr;
			if (Range.StartValueKey == PCG::Private::ExistingIndexForDefaultValue)
			{
				check(Range.Count == 1);
				SrcPtr = static_cast<const uint8*>(DefaultValue.GetRawPtr());
			}
			else
			{
				SrcPtr = Helper->GetRawPtr(Range.StartValueKey);
			}

			check(SrcPtr);

			// Since the values are stored in an array, they are all contiguous in memory so we can just do pointer arithmetic using the
			// element size.
			auto ApplyOnLoop = [this, &Range, SrcPtr](auto Func)
			{
				for (int32 i = 0; i < Range.Count; ++i)
				{
					Func(static_cast<const uint8*>(UnderlyingProperty->GetPtrInArray(SrcPtr, i)), Range.StartIndex + i);
				}
			};

			Visit([this, &ApplyOnLoop, SrcPtr, &Range](auto&& OutValue)
				{
					using T = std::decay_t<decltype(OutValue)>;
					if constexpr (std::is_same_v<T, FOutValuesAsArray>)
					{
						ApplyOnLoop([this, &OutValue](const uint8* SrcPtr, int32 Index)
						{
							FScriptArrayHelper OtherHelper(static_cast<FArrayProperty*>(UnderlyingProperty->GetProperty()->Inner), SrcPtr);
							OutValue.OutValues[Index].template Get<0>() = OtherHelper.GetRawPtr();
							OutValue.OutValues[Index].template Get<1>() = OtherHelper.Num();
						});
					}
					else if constexpr (std::is_same_v<T, FOutValuesAsSet>)
					{
						ApplyOnLoop([this, &OutValue](const uint8* SrcPtr, int32 Index)
						{
							new (OutValue.OutValues[Index]) FScriptSetHelper(static_cast<FSetProperty*>(UnderlyingProperty->GetProperty()->Inner), SrcPtr);
						});
					}
					else if constexpr (std::is_same_v<T, FOutValuesAsMap>)
					{
						ApplyOnLoop([this, &OutValue](const uint8* SrcPtr, int32 Index)
						{
							new (OutValue.OutValues[Index]) FScriptMapHelper(static_cast<FMapProperty*>(UnderlyingProperty->GetProperty()->Inner), SrcPtr);
						});
					}
					else if constexpr (std::is_same_v<T, FOutValuesByPtr>)
					{
						ApplyOnLoop([this, &OutValue](const uint8* SrcPtr, int32 Index)
						{
							OutValue.OutValues[Index] = SrcPtr;
						});
					}
					else if constexpr (std::is_same_v<T, FOutValuesByValue>)
					{
						void* DestPtr = const_cast<void*>(UnderlyingProperty->GetPtrInArray(OutValue.OutValues, Range.StartIndex));
						UnderlyingProperty->Copy(DestPtr, SrcPtr, Range.Count);
					}
					else
					{
						static_assert(!std::is_same_v<T, T>, "Missing variant case for GetValues_Internal");
					}
			}, OutValues);
		};

		for (; It; ++It)
		{
			const int32 Index = It.GetIndex();
			const PCGMetadataValueKey ValueKey = ValueKeys[Index];

			int32 ValueKeyIndex = INDEX_NONE;

			if (ValueKey == PCGDefaultValueKey)
			{
				ValueKeyIndex = PCG::Private::ExistingIndexForDefaultValue;
			}
			else if (ValueKey >= ValueKeyOffset)
			{
				int32 ValueIndex = ValueKey - ValueKeyOffset;
				ValueKeyIndex = ValueIndex < Values.Num() ? ValueIndex : PCG::Private::ExistingIndexForDefaultValue;
			}
			else if (!ThisParent)
			{
				ValueKeyIndex = PCG::Private::ExistingIndexForDefaultValue;
			}
			else
			{
				bFoundAllKeys = false;
			}

			if (ValueKeyIndex < 0)
			{
				// Reached the end
				if (ContinuousRange.StartIndex != INDEX_NONE)
				{
					RetrieveValues(ContinuousRange);
					ContinuousRange.Reset();
				}

				// Then copy the default value if the value key is PCG::Private::ExistingIndexForDefaultValue
				if (ValueKeyIndex == PCG::Private::ExistingIndexForDefaultValue)
				{
					ContinuousRange.Init(Index, PCG::Private::ExistingIndexForDefaultValue);
					RetrieveValues(ContinuousRange);
					ContinuousRange.Reset();
				}
			}
			else
			{
				// Check if we are still in the same continuous range. If so, increment and continue. We also need to mark the value as retrieved.
				if (ContinuousRange.StartIndex != INDEX_NONE && ContinuousRange.StartIndex + ContinuousRange.Count == Index && ContinuousRange.StartValueKey + ContinuousRange.Count == ValueKeyIndex)
				{
					ContinuousRange.Count++;
					UnretrievedValues[Index] = false;
					continue;
				}
				else if (ContinuousRange.StartIndex != INDEX_NONE)
				{
					// End of the range
					RetrieveValues(ContinuousRange);
					ContinuousRange.Reset();
				}

				ContinuousRange.Init(Index, ValueKeyIndex);
			}

			// If the value key index is not none, the value was (or will be) retrieved.
			if (ValueKeyIndex != INDEX_NONE)
			{
				UnretrievedValues[Index] = false;
			}
		}

		// One last time
		if (ContinuousRange.StartIndex != INDEX_NONE)
		{
			// End of the range
			RetrieveValues(ContinuousRange);
			ContinuousRange.Reset();
		}
	}

	ensure(ThisParent || bFoundAllKeys);

	if (ThisParent && !bFoundAllKeys)
	{
		ThisParent->GetValues_Internal(ValueKeys, OutValues, UnretrievedValues);
	}
}

FInstancedPropertyBag FPCGMetadataAttributeBase::BuildStructForDebug(PCGMetadataEntryKey EntryKey) const
{
	if (!ensure(UnderlyingProperty))
	{
		return {};
	}

	FInstancedPropertyBag Result;

	// Property system doesn't like a None name for the properties, so use `_` to differentiate.
	FPropertyBagPropertyDesc Desc{Name == NAME_None ? "_None_" : Name, UnderlyingProperty->GetProperty()->Inner};

	const UPropertyBag* PropertyBag = UPropertyBag::GetOrCreateFromDescs(MakeConstArrayView(&Desc, 1));
	Result.InitializeFromBagStruct(PropertyBag);

	const FProperty* ResultProperty = PropertyBag->GetPropertyDescs()[0].CachedProperty;

	void* DestPtr = ResultProperty->ContainerPtrToValuePtr<void>(Result.GetMutableValue().GetMemory());

	{
		PCG::TSharedScopeLock ReadLock(ValueLock);
		if (const void* SrcPtr = GetReadAddressFromEntryKey_Unsafe(EntryKey))
		{
			ResultProperty->CopyCompleteValue(DestPtr, SrcPtr);
		}
	}

	return Result;
}

TOptional<FScriptArrayHelper> FPCGMetadataAttributeBase::MakeArrayHelper() const
{
	if (!ensure(UnderlyingProperty) || !UnderlyingProperty->GetProperty())
	{
		return {};
	}

	return FScriptArrayHelper(UnderlyingProperty->GetProperty(), &Values);
}

const void* FPCGMetadataAttributeBase::GetReadAddressForValueKey_Unsafe(const PCGMetadataValueKey InValueKey) const
{
	if (InValueKey == PCGDefaultValueKey)
	{
		check(DefaultValue.IsValid());
		return DefaultValue.GetRawPtr();
	}

	if (InValueKey < ValueKeyOffset)
	{
		return ensure(Parent) ? Parent->GetReadAddressForValueKey_Unsafe(InValueKey) : nullptr;
	}

	const int32 ValueIndex = InValueKey - ValueKeyOffset;

	TOptional<FScriptArrayHelper> Helper = MakeArrayHelper();
	if (!Helper || ValueIndex >= Values.Num())
	{
		return nullptr;
	}

	return Helper->GetRawPtr(ValueIndex);
}

const void* FPCGMetadataAttributeBase::GetReadAddressFromEntryKey_Unsafe(const PCGMetadataEntryKey InEntryKey) const
{
	return GetReadAddressForValueKey_Unsafe(GetValueKey(InEntryKey));
}

void FPCGMetadataAttributeBase::SetValueFromProperty(PCGMetadataEntryKey ItemKey, const void* SrcPtr, const FProperty* Property)
{
	using namespace PCG::Private;

	if (!SrcPtr || !Property)
	{
		return;
	}

	if (TOptional<FScriptArrayHelper> Helper = MakeArrayHelper(); Helper.IsSet())
	{
		if (!UnderlyingProperty->GetProperty()->Inner->SameType(Property))
		{
			return;
		}

		const int32 ExistingIndex = ItemKey == PCGInvalidEntryKey && ensure(DefaultValue.IsValid()) ? PCG::Private::ExistingIndexForDefaultValue : INDEX_NONE;

		PCGMetadataValueKey ValueKeyToAdd = PCGNotFoundValueKey;

		if (UnderlyingProperty->CompressData() && ItemKey != PCGInvalidEntryKey)
		{
			bool bValueKeyFound = false;
			if (CachedDesc.IsSingleValue())
			{
				bValueKeyFound = FindValues(MakeArrayView(&ValueKeyToAdd, 1), FInValues{TInPlaceType<FInValuesByValue>{}, SrcPtr, 1}, 1);
			}
			else if (CachedDesc.IsArray())
			{
				const FArrayProperty* InnerProperty = CastFieldChecked<FArrayProperty>(UnderlyingProperty->GetProperty()->Inner);
				FScriptArrayHelper ArrayHelper(InnerProperty, SrcPtr);
				TTuple<const void*, int32> ArrayData[1] = {MakeTuple(ArrayHelper.GetElementPtr(), ArrayHelper.Num())};

				bValueKeyFound = FindValues(MakeArrayView(&ValueKeyToAdd, 1), FInValues{TInPlaceType<FInValuesAsArray>{}, ArrayData}, 1);
			}

			check(!bValueKeyFound || ValueKeyToAdd != PCGNotFoundValueKey);
		}

		if (ValueKeyToAdd == PCGNotFoundValueKey)
		{
			SetValuesFromValueKeys({ItemKey}, AddValues_Internal(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, SrcPtr, 1}, 1, ExistingIndex), /*bResetDefaultValueOnDefaultValueKey=*/ItemKey==PCGInvalidEntryKey);
		}
		else
		{
			SetValueFromValueKey(ItemKey, ValueKeyToAdd);
		}
	}
}

int32 FPCGMetadataAttributeBase::PreallocateValues(TArrayView<PCGMetadataEntryKey*> EntryKeys, bool bLockless)
{
	int32 StartIndex = INDEX_NONE;

	if (!ensure(UnderlyingProperty))
	{
		return StartIndex;
	}

	if (!UnderlyingProperty->CompressData())
	{
		PCG::TUniqueScopeLock WriteValueLock(ValueLock, /*bShouldLock=*/!bLockless);
		StartIndex = AllocateValues_Unsafe(EntryKeys.Num());
	}

	{
		PCG::TUniqueScopeLock WriteEntryMapLock(EntryMapLock, /*bShouldLock=*/!bLockless);
		EntryToValueKeyMap.Reserve(EntryToValueKeyMap.Num() + EntryKeys.Num());

		if (!UnderlyingProperty->CompressData() && StartIndex != INDEX_NONE)
		{
			for (int32 i = 0; i < EntryKeys.Num(); ++i)
			{
				const PCGMetadataValueKey ValueKey = StartIndex + i + ValueKeyOffset;
				EntryToValueKeyMap.Emplace(*EntryKeys[i], ValueKey);
			}
		}
	}

	return StartIndex;
}

void FPCGMetadataAttributeBase::Prepare(int32 Count)
{
	PCG::TUniqueScopeLock Lock(EntryMapLock);
	EntryToValueKeyMap.Reserve(EntryToValueKeyMap.Num() + Count);
}

int32 FPCGMetadataAttributeBase::AllocateValues_Unsafe(int32 Count)
{
	if (TOptional<FScriptArrayHelper> Helper = MakeArrayHelper(); Helper.IsSet())
	{
		const int32 StartIndex = Helper->Num();

		if (UnderlyingProperty->IsPlainOldData())
		{
			Helper->AddUninitializedValues(Count);
		}
		else
		{
			Helper->AddValues(Count);
		}

		return StartIndex;
	}
	else
	{
		return INDEX_NONE;
	}
}

const UPCGMetadata* FPCGMetadataAttributeBase::GetMetadata() const
{
	return Metadata ? Metadata->GetTopMetadata() : nullptr;
}

void FPCGMetadataAttributeBase::SetValueFromValueKey(PCGMetadataEntryKey EntryKey, PCGMetadataValueKey ValueKey, bool bResetValueOnDefaultValueKey)
{
	PCG::TUniqueScopeLock WriteLock(EntryMapLock);
	SetValueFromValueKey_Unsafe(EntryKey, ValueKey, bResetValueOnDefaultValueKey);
}

void FPCGMetadataAttributeBase::SetValueFromValueKey_Unsafe(PCGMetadataEntryKey EntryKey, PCGMetadataValueKey ValueKey, bool bResetValueOnDefaultValueKey, bool bAllowInvalidEntries)
{
	if (EntryKey == PCGInvalidEntryKey)
	{
		check(bAllowInvalidEntries);
		return;
	}

	if (ValueKey == PCGDefaultValueKey && bResetValueOnDefaultValueKey)
	{
		EntryToValueKeyMap.Remove(EntryKey);
	}
	else
	{
		EntryToValueKeyMap.FindOrAdd(EntryKey) = ValueKey;
	}
}

void FPCGMetadataAttributeBase::SetValuesFromValueKeys(const TArrayView<const TTuple<PCGMetadataEntryKey, PCGMetadataValueKey>>& EntryValuePairs, bool bResetValueOnDefaultValueKey)
{
	if (EntryValuePairs.IsEmpty())
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(EntryMapLock);
	for (const TTuple<PCGMetadataEntryKey, PCGMetadataValueKey>& EntryValuePair : EntryValuePairs)
	{
		SetValueFromValueKey_Unsafe(EntryValuePair.Key, EntryValuePair.Value, bResetValueOnDefaultValueKey, /*bAllowInvalidEntries=*/true);
	}
}

void FPCGMetadataAttributeBase::SetValuesFromValueKeys(const TArrayView<const PCGMetadataEntryKey>& EntryKeys, const TArrayView<const PCGMetadataValueKey>& ValueKeys, bool bResetValueOnDefaultValueKey)
{
	SetValuesFromValueKeys(PCGValueRangeHelpers::MakeConstValueRange(EntryKeys), ValueKeys, bResetValueOnDefaultValueKey);
}

void FPCGMetadataAttributeBase::SetValuesFromValueKeys(TConstPCGValueRange<PCGMetadataEntryKey> EntryKeys, const TArrayView<const PCGMetadataValueKey>& ValueKeys, bool bResetValueOnDefaultValueKey)
{
	if (EntryKeys.IsEmpty() || EntryKeys.Num() != ValueKeys.Num())
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(EntryMapLock);
	for (int32 i = 0; i < EntryKeys.Num(); ++i)
	{
		SetValueFromValueKey_Unsafe(EntryKeys[i], ValueKeys[i], bResetValueOnDefaultValueKey, /*bAllowInvalidEntries=*/true);
	}
}

void FPCGMetadataAttributeBase::SetValuesFromValueKeys(const TArrayView<const PCGMetadataEntryKey * const>& EntryKeys, const TArrayView<const PCGMetadataValueKey>& ValueKeys, bool bResetValueOnDefaultValueKey)
{
	if (EntryKeys.IsEmpty() || EntryKeys.Num() != ValueKeys.Num())
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(EntryMapLock);
	for (int32 i = 0; i < EntryKeys.Num(); ++i)
	{
		SetValueFromValueKey_Unsafe(*EntryKeys[i], ValueKeys[i], bResetValueOnDefaultValueKey, /*bAllowInvalidEntries=*/true);
	}
}

template <typename BucketType>
TArray<BucketType> FPCGMetadataAttributeBase::AttributePartition(TConstArrayView<PCGMetadataEntryKey> EntryKeys) const
{
	using namespace PCG::Private;

	if (!UnderlyingProperty || !UnderlyingProperty->IsValid() || EntryKeys.IsEmpty())
	{
		return {};
	}

	TArray<PCGMetadataValueKey> ValueKeys;
	GetValueKeys(EntryKeys, ValueKeys);
	check(EntryKeys.Num() == ValueKeys.Num());

	if (UnderlyingProperty->CompressData())
	{
		// If the data is already compressed there is nothing to do, just convert all the entry keys to their value keys
		TMap<PCGMetadataValueKey, int32> MappingUniqueValueKeyToIndex;
		TArray<BucketType> Result;
		const int32 NumEntries = EntryKeys.Num();

		for (int32 i = 0; i < NumEntries; ++i)
		{
			int32* Index = MappingUniqueValueKeyToIndex.Find(ValueKeys[i]);
			if (!Index)
			{
				Index = &MappingUniqueValueKeyToIndex.Add(ValueKeys[i], MappingUniqueValueKeyToIndex.Num());
				if constexpr (std::is_same_v<BucketType, TBitArray<>>)
				{
					Result.Emplace_GetRef().Init(false, NumEntries);
				}
				else
				{
					Result.Emplace();
				}
			}

			if constexpr (std::is_same_v<BucketType, TBitArray<>>)
			{
				Result[*Index][i] = true;
			}
			else
			{
				Result[*Index].Add(i);
			}
		}

		return Result;
	}
	else if (CachedDesc.IsSet() || CachedDesc.IsMap())
	{
		// @todo_pcg Support set and maps
		return {};
	}
	else
	{
		const FProperty* InnerProperty = UnderlyingProperty->GetProperty()->Inner;

		const bool bIsArray = CachedDesc.IsArray();
		InnerProperty = bIsArray ? CastFieldChecked<FArrayProperty>(InnerProperty)->Inner : InnerProperty;
		
		check(InnerProperty);

		{
			PCG::TSharedScopeLock ReadLock(ValueLock);

			TArray<TTuple<const void*, int32>> ValuesToCompare;
			ValuesToCompare.Reserve(ValueKeys.Num());

			for (int32 i = 0; i < ValueKeys.Num(); ++i)
			{
				const void* ValuePtr = GetReadAddressForValueKey_Unsafe(ValueKeys[i]);
				if (bIsArray)
				{
					if (const FScriptArray* ArrayPtr = static_cast<const FScriptArray*>(ValuePtr))
					{
						ValuesToCompare.Emplace(ArrayPtr->GetData(), ArrayPtr->Num());
					}
					else
					{
						// If there was nothing at this value key, compare to an empty array.
						ValuesToCompare.Emplace(nullptr, 0);
					}
				}
				else
				{
					ValuesToCompare.Emplace(ValuePtr, 1);
				}
			}

			return MakePartition<std::conditional_t<std::is_same_v<BucketType, TBitArray<>>, FPartitionResultBucketBitArray, FPartitionResultBucketIndices>>(ValuesToCompare, InnerProperty).Result;
		}
	}
}

template TArray<TArray<int32>> FPCGMetadataAttributeBase::AttributePartition<TArray<int32>>(TConstArrayView<PCGMetadataEntryKey> EntryKeys) const;
template TArray<TBitArray<>> FPCGMetadataAttributeBase::AttributePartition<TBitArray<>>(TConstArrayView<PCGMetadataEntryKey> EntryKeys) const;

PCGMetadataValueKey FPCGMetadataAttributeBase::GetValueKey(PCGMetadataEntryKey EntryKey) const
{
	if (EntryKey == PCGInvalidEntryKey)
	{
		return PCGDefaultValueKey;
	}

	PCGMetadataValueKey ValueKey = PCGDefaultValueKey;
	bool bFoundKey = false;

	{
		PCG::TSharedScopeLock ReadLock(EntryMapLock);
		if (const PCGMetadataValueKey* FoundLocalKey = EntryToValueKeyMap.Find(EntryKey))
		{
			ValueKey = *FoundLocalKey;
			bFoundKey = true;
		}
	}

	if (!bFoundKey && Parent)
	{
		return Parent->GetValueKey(Metadata->GetParentKey(EntryKey));
	}
	else
	{
		return ValueKey;
	}
}

void FPCGMetadataAttributeBase::GetValueKeys(const TArrayView<const PCGMetadataEntryKey>& EntryKeys, TArray<PCGMetadataValueKey>& OutValueKeys) const
{
	GetValueKeys(PCGValueRangeHelpers::MakeConstValueRange<PCGMetadataEntryKey>(EntryKeys), OutValueKeys);
}

void FPCGMetadataAttributeBase::GetValueKeys(TConstPCGValueRange<PCGMetadataEntryKey> EntryKeys, TArray<PCGMetadataValueKey>& OutValueKeys) const
{
	if (EntryKeys.IsEmpty())
	{
		return;
	}

	OutValueKeys.SetNumUninitialized(EntryKeys.Num());
	// Bitset with all unset values. If we have any unset value, we will ask the parent for those.
	TBitArray<> UnsetValues(true, EntryKeys.Num());

	GetValueKeys_Internal(EntryKeys, OutValueKeys, UnsetValues, /*bOwnerOfEntryKeysView=*/false);
}

void FPCGMetadataAttributeBase::GetValueKeys(TArrayView<PCGMetadataEntryKey> EntryKeys, TArray<PCGMetadataValueKey>& OutValueKeys) const
{
	GetValueKeys(PCGValueRangeHelpers::MakeValueRange<PCGMetadataEntryKey>(EntryKeys), OutValueKeys);
}

void FPCGMetadataAttributeBase::GetValueKeys(TPCGValueRange<PCGMetadataEntryKey> EntryKeys, TArray<PCGMetadataValueKey>& OutValueKeys) const
{
	if (EntryKeys.IsEmpty())
	{
		return;
	}

	OutValueKeys.SetNumUninitialized(EntryKeys.Num());
	// Bitset with all unset values. If we have any unset value, we will ask the parent for those.
	TBitArray<> UnsetValues(true, EntryKeys.Num());

	GetValueKeys_Internal(PCGValueRangeHelpers::MakeConstValueRange(EntryKeys), OutValueKeys, UnsetValues, /*bOwnerOfEntryKeysView=*/true);
}

void FPCGMetadataAttributeBase::GetValueKeys_Internal(TConstPCGValueRange<PCGMetadataEntryKey> EntryKeys, TArrayView<PCGMetadataValueKey> OutValueKeys, TBitArray<>& UnsetValues, bool bOwnerOfEntryKeysView) const
{
	check(EntryKeys.Num() == OutValueKeys.Num() && OutValueKeys.Num() == UnsetValues.Num());

	bool bFoundAllKeys = true;
	TConstSetBitIterator<> It(UnsetValues);
	if (!It)
	{
		return;
	}

	{
		PCG::TSharedScopeLock ReadLock(EntryMapLock);

		for (; It; ++It)
		{
			const int32 Index = It.GetIndex();
			const PCGMetadataEntryKey EntryKey = EntryKeys[Index];

			auto SetValueKey = [Index, &OutValueKeys, &UnsetValues](PCGMetadataValueKey ValueKey)
				{
					OutValueKeys[Index] = ValueKey;
					UnsetValues[Index] = false;
				};

			if (EntryKey == PCGInvalidEntryKey)
			{
				SetValueKey(PCGDefaultValueKey);
			}
			else if (const PCGMetadataValueKey* FoundLocalKey = EntryToValueKeyMap.Find(EntryKey))
			{
				SetValueKey(*FoundLocalKey);
			}
			else if (!Parent)
			{
				SetValueKey(PCGDefaultValueKey);
			}
			else
			{
				bFoundAllKeys = false;
			}
		}
	}

	ensure(Parent || bFoundAllKeys);

	if (Parent && !bFoundAllKeys)
	{
		auto ParentCall = [this, &OutValueKeys, &UnsetValues](TConstPCGValueRange<PCGMetadataEntryKey> CurrentEntryKeys)
		{
			// Before querying the parent, we need to update all our entry keys to get them in the parent referential.
			// At that point, we are owner of our memory, so it is safe to cast
			Metadata->GetParentKeysWithRange(PCGValueRangeHelpers::MakeValueRange_Unsafe(CurrentEntryKeys), &UnsetValues);
			Parent->GetValueKeys_Internal(CurrentEntryKeys, OutValueKeys, UnsetValues, /*bOwnerOfEntryKeysView=*/true);
		};

		// If the input data is coming from outside, we need to copy it to be able to modify it.
		// Only do it there, because we don't have to pay the cost of the copy if we don't have to check the parent.
		if (!bOwnerOfEntryKeysView)
		{
			TArray<PCGMetadataEntryKey, TInlineAllocator<256>> CopiedKeys;
			CopiedKeys.Reserve(EntryKeys.Num());
			Algo::Transform(EntryKeys, CopiedKeys, [](const PCGMetadataEntryKey& It) { return It; });
			ParentCall(PCGValueRangeHelpers::MakeConstValueRange(CopiedKeys));
		}
		else
		{
			ParentCall(EntryKeys);
		}
	}
}

bool FPCGMetadataAttributeBase::HasNonDefaultValue(PCGMetadataEntryKey EntryKey) const
{
	return GetValueKey(EntryKey) != PCGDefaultValueKey;
}

void FPCGMetadataAttributeBase::ClearEntries()
{
	EntryToValueKeyMap.Reset();
}

bool FPCGMetadataAttributeBase::IsValidName(const FString& Name)
{
	for (int32 i = 0; i < Name.Len(); ++i)
	{
		if (!PCGMetadataAttributeBase::IsValidNameCharacter(Name[i]))
		{
			return false;
		}
	}

	return true;
}

bool FPCGMetadataAttributeBase::IsValidName(const FName& Name)
{
	// Early out on None
	return (Name == NAME_None) || IsValidName(Name.ToString());
}

bool FPCGMetadataAttributeBase::SanitizeName(FString& InOutName)
{
	bool bAnyCharactersSanitized = false;

	for (int32 i = 0; i < InOutName.Len(); ++i)
	{
		if (!PCGMetadataAttributeBase::IsValidNameCharacter(InOutName[i]))
		{
			InOutName[i] = '_';
			bAnyCharactersSanitized = true;
		}
	}

	return bAnyCharactersSanitized;
}
