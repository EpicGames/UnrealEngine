// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGMetadataAttributeTraits.h"

#include "Utils/PCGLogErrors.h"

void PCG::Private::LogNotLoadedObjectNotOnGameThread(const FSoftObjectPath& InObjectPath)
{
#if !NO_LOGGING
	PCGLog::LogErrorOnGraph(FText::Format(
		NSLOCTEXT("PCGMetadataAttributeTraits", "NotGameThread", "Tried to write '{0}' into an Object property, but it wasn't loaded and it is not on the game thread. We can't load objects outside of the game thread."),
		FText::FromString(InObjectPath.ToString())));
#endif // !NO_LOGGING
}
