// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGMetadataCommon.h"

#include "Serialization/ArchiveCountMem.h"
#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataCommon)

namespace PCG::Private
{
	class FNullArchive : public FArchive
	{
		/** Override FObjectPtr to do nothing*/
		virtual FArchive& operator<<(struct FObjectPtr& Value) override
		{
			return *this;
		}
	};

	/** Equivalent of the method in PropertyBag to convert the descriptor to a property. */
	FProperty* CreatePropertyFromDesc(const FPCGMetadataAttributeDesc& Desc, const FFieldVariant PropertyScope)
	{
		// Handle array and nested containers properties
		if (Desc.ContainerTypes.Num() > 0)
		{
			FProperty* Prop = nullptr; // The first created container will fill the return value, nested ones will fill the inner

			// Support for nested containers, i.e. : TArray<TArray<float>>
			FFieldVariant PropertyOwner = PropertyScope;
			FProperty** ValuePropertyPtr = &Prop;

			// Create the container list
			for (EPCGMetadataAttributeContainerTypes BagContainerType : Desc.ContainerTypes)
			{
				switch(BagContainerType)
				{
				case EPCGMetadataAttributeContainerTypes::Array:
					{
						// Create an array property as a container for the tail
						FArrayProperty* ArrayProperty = new FArrayProperty(PropertyOwner, Desc.Name);
						*ValuePropertyPtr = ArrayProperty;
						ValuePropertyPtr = &ArrayProperty->Inner;
						PropertyOwner = ArrayProperty;
						break;
					}
				case EPCGMetadataAttributeContainerTypes::Set:
					{
						// Create a set property as a container for the tail
						FSetProperty* SetProperty = new FSetProperty(PropertyOwner, Desc.Name);
						*ValuePropertyPtr = SetProperty;
						ValuePropertyPtr = &SetProperty->ElementProp;
						PropertyOwner = SetProperty;
						break;
					}
				case EPCGMetadataAttributeContainerTypes::Map:
					{
						// Create a map property as a container for the tail
						FMapProperty* MapProperty = new FMapProperty(PropertyOwner, Desc.Name);
						*ValuePropertyPtr = MapProperty;
						ValuePropertyPtr = &MapProperty->ValueProp;
						PropertyOwner = MapProperty;

						// Create the key property (container types are not supported for keys)
						FPCGMetadataAttributeDesc KeyDesc;
						KeyDesc.ValueType = Desc.KeyType;
						KeyDesc.ValueTypeObject = Desc.KeyTypeObject;
						KeyDesc.Name = Desc.Name;
						FProperty* KeyProp = CreatePropertyFromDesc(KeyDesc, PropertyOwner);
						if (!KeyProp)
						{
							delete MapProperty;
							return nullptr;
						}

						MapProperty->KeyProp = KeyProp;
						break;
					}
				default:
					ensureMsgf(false, TEXT("Unsupported container type %s"), *UEnum::GetValueAsString(BagContainerType));
					break;
				}
			}

			// Finally create the tail type
			FPCGMetadataAttributeDesc InnerDesc = Desc;
			InnerDesc.ContainerTypes.Reset();
			*ValuePropertyPtr = CreatePropertyFromDesc(InnerDesc, PropertyOwner);

			if (*ValuePropertyPtr == nullptr)
			{
				delete Prop;
				return nullptr;
			}
			else
			{
				return Prop;
			}
		}

		const UScriptStruct* ScriptStruct = nullptr;

		switch (Desc.ValueType)
		{
		case EPCGMetadataTypes::Boolean:
			{
				FBoolProperty* Prop = new FBoolProperty(PropertyScope, Desc.Name);
				Prop->SetBoolSize(sizeof(bool), true); // Enable native access (init the whole byte, rather than just first bit)
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPCGMetadataTypes::Byte:
			{
				FByteProperty* Prop = new FByteProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPCGMetadataTypes::Integer32:
			{
				FIntProperty* Prop = new FIntProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPCGMetadataTypes::Integer64:
			{
				FInt64Property* Prop = new FInt64Property(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPCGMetadataTypes::Float:
			{
				FFloatProperty* Prop = new FFloatProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPCGMetadataTypes::Double:
			{
				FDoubleProperty* Prop = new FDoubleProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPCGMetadataTypes::Name:
			{
				FNameProperty* Prop = new FNameProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPCGMetadataTypes::String:
			{
				FStrProperty* Prop = new FStrProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPCGMetadataTypes::Text:
			{
				FTextProperty* Prop = new FTextProperty(PropertyScope, Desc.Name);
				return Prop;
			}
		case EPCGMetadataTypes::Enum:
			if (const UEnum* Enum = Cast<UEnum>(Desc.ValueTypeObject))
			{
				FEnumProperty* Prop = new FEnumProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				FNumericProperty* UnderlyingProp = new FByteProperty(Prop, "UnderlyingType"); // HACK: Hardwire to byte property for now for BP compatibility
				UnderlyingProp->SetPropertyFlags(CPF_HasGetValueTypeHash | CPF_IsPlainOldData);
				Prop->SetEnum(const_cast<UEnum*>(Enum));
				Prop->AddCppProperty(UnderlyingProp);
				return Prop;
			}
			break;
		case EPCGMetadataTypes::SoftObjectPath:
			ScriptStruct = TBaseStructure<FSoftObjectPath>::Get();
			break;
		case EPCGMetadataTypes::SoftClassPath:
			ScriptStruct = TBaseStructure<FSoftClassPath>::Get();
			break;
		case EPCGMetadataTypes::Vector2:
			ScriptStruct = TBaseStructure<FVector2D>::Get();
			break;
		case EPCGMetadataTypes::Vector:
			ScriptStruct = TBaseStructure<FVector>::Get();
			break;
		case EPCGMetadataTypes::Vector4:
			ScriptStruct = TBaseStructure<FVector4>::Get();
			break;
		case EPCGMetadataTypes::Quaternion:
			ScriptStruct = TBaseStructure<FQuat>::Get();
			break;
		case EPCGMetadataTypes::Rotator:
			ScriptStruct = TBaseStructure<FRotator>::Get();
			break;
		case EPCGMetadataTypes::Transform:
			ScriptStruct = TBaseStructure<FTransform>::Get();
			break;
		case EPCGMetadataTypes::Struct:
			ScriptStruct = Cast<UScriptStruct>(Desc.ValueTypeObject);
			break;
		default:
			ensureMsgf(false, TEXT("Unhandled type %s"), *UEnum::GetValueAsString(Desc.ValueType));
		}

		if (ScriptStruct)
		{
			FStructProperty* Prop = new FStructProperty(PropertyScope, Desc.Name);
			// Struct Property requires the underlying struct to be non-const. It is done like that in the engine.
			Prop->Struct = const_cast<UScriptStruct*>(ScriptStruct);

			if (ScriptStruct->GetCppStructOps() && ScriptStruct->GetCppStructOps()->HasGetTypeHash())
			{
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
			}

			if (ScriptStruct->StructFlags & STRUCT_HasInstancedReference)
			{
				Prop->SetPropertyFlags(CPF_ContainsInstancedReference);
			}

			return Prop;
		}

		return nullptr;
	}

	/**
	 * Taken from FArrayProperty::CopyValuesInternal in order to leverage MemCpy when we have plain old data
	 * with the optimization twist for Object copies (also present in CopyValuesInternal)
	 * @todo_pcg: Refactorize it in Core to re-use it. Maybe add it to FScriptArrayHelper?
	 */
	void CopyArray(const FArrayProperty* InProperty, void* DestPtr, const void* SrcPtr, int32 Count)
	{
		const FProperty* Inner = InProperty ? InProperty->Inner : nullptr;

		if (!Inner || Count <= 0 || !DestPtr || !SrcPtr)
		{
			return;
		}

		size_t Size = Inner->GetElementSize();
		const uint8* SrcData = (uint8*)SrcPtr;
		uint8* DestData = (uint8*)DestPtr;
		if (!(Inner->PropertyFlags & CPF_IsPlainOldData))
		{
			// If we are in the closed body of a transaction, we have a special case for `FObjectProperty` that vastly
			// improves performance when copying arrays around (in the order of 4x faster). We **do not want** to do
			// this optimization with anything that derives from `FObjectProperty` though, so we **do not use** `IsA`
			// as that would all catch classes that inherited from `FObjectProperty`!
			if (AutoRTFM::IsClosed() && (Inner->GetClass() == FObjectProperty::StaticClass()))
			{
				#if UE_WITH_REMOTE_OBJECT_HANDLE
				// CopyCompleteValue will resolve handles in the RawPtr <- TObjectPtr case.
				// Since we can't abort while in an open scope, use a speculation scope to defer any
				// migrations until this scope ends.
				UE::RemoteExecutor::FSpeculationExecutionScope AvoidMigrationsInTheOpen;
				#endif

				// Since we've already written to each location that we are going to be copying over below, we know that
				// all of the record-writes that the transactional system would have inserted are no-ops. Also, each copy
				// of a `TObjectPtr` will go into the open to do mark the object as reachable, so by doing a single open
				// here we can avoid doing N calls to record-write and N calls to open for each object in the array. Big
				// saving on large arrays!
				UE_AUTORTFM_OPEN_NO_VALIDATION
				{
					// This optimization assumes that `FObjectProperty` is just backed by a `TObjectPtr` which when copying
					// does not do any heap allocations, just pulls over the pointer to the underlying `UObject` and then
					// marks it as reachable for incremental reachability analysis. Lets check that this is still the case
					// here so that if anyone ever changes that in future this line will shout at them!
					static_assert(std::is_same_v<decltype(FObjectProperty::GetDefaultPropertyValue()), TObjectPtr<UObject>>);

					for (int32 i = 0; i < Count; i++)
					{
						Inner->CopyCompleteValue(DestData + i * Size, SrcData + i * Size);
					}
				};
			}
			else
			{
				for (int32 i = 0; i < Count; i++)
				{
					Inner->CopyCompleteValue(DestData + i * Size, SrcData + i * Size);
				}
			}
		}
		else
		{
			FMemory::Memcpy(DestData, SrcData, Count * Size);
		}
	}
	
	TOptional<int32> ComputeHash(const FProperty* InProperty, const void* InValuePtr, const int32 InCount)
	{
		check(InProperty);
		
		// Empty array has a hash of 0
		if (InCount == 0)
		{
			return 0;
		}
		
		if (!InValuePtr)
		{
			return {};
		}

		const int32 ElementSize = InProperty->GetElementSize();
		int32 Result = InProperty->GetValueTypeHash(InValuePtr);
		for (int32 i = 1; i < InCount; i++)
		{
			Result = HashCombine(Result, InProperty->GetValueTypeHash(static_cast<const uint8*>(InValuePtr) + i * ElementSize));
		}
		
		return Result;
	}
	
	bool CompareArrays(const FProperty* InProperty, const void* LHSPtr, const int32 LHSCount, const void* RHSPtr, const int32 RHSCount)
	{
		check(InProperty);

		if (LHSCount != RHSCount)
		{
			return false;
		}
		
		// Empty arrays are equal (we know counts are equal, just test LHS)
		if (LHSCount == 0)
		{
			return true;
		}
		
		// If any pointer is invalid, can't compare.
		if (!LHSPtr || !RHSPtr)
		{
			return false;
		}
		
		const int32 ElementSize = InProperty->GetElementSize();
		for (int32 i = 0; i < LHSCount; i++)
		{
			const void* LHSValue = static_cast<const uint8*>(LHSPtr) + i * ElementSize;
			const void* RHSValue = static_cast<const uint8*>(RHSPtr) + i * ElementSize;
			
			if (!InProperty->Identical(LHSValue, RHSValue))
			{
				return false;
			}
		}
		
		return true;
	}
}

namespace PCGMetadataAttributeDesc
{
	bool IsObjectType(const EPCGMetadataTypes InType)
	{
		return InType == EPCGMetadataTypes::SoftObject || InType == EPCGMetadataTypes::Object 
			|| InType == EPCGMetadataTypes::SoftClass || InType == EPCGMetadataTypes::Class;
	}
	
	bool ShouldTypeRequireTypeObject(const EPCGMetadataTypes InType)
	{
		return InType == EPCGMetadataTypes::Struct || InType == EPCGMetadataTypes::Enum || IsObjectType(InType);
	}
}

namespace PCGMetadataDomainID
{
	const FPCGMetadataDomainID Default{EPCGMetadataDomainFlag::Default, -1, TEXT("Default")};
	const FPCGMetadataDomainID Elements{EPCGMetadataDomainFlag::Elements, -1, TEXT("Elements")};
	const FPCGMetadataDomainID Data{EPCGMetadataDomainFlag::Data, -1, TEXT("Data")};
	const FPCGMetadataDomainID Invalid{EPCGMetadataDomainFlag::Invalid, -1, TEXT("Invalid")};
}

bool FPCGMetadataAttributeDesc::IsSameType(const FPCGMetadataAttributeDesc& Other) const
{
	return ValueType == Other.ValueType &&
		ContainerTypes == Other.ContainerTypes &&
		ValueTypeObject == Other.ValueTypeObject &&
		KeyType == Other.KeyType &&
		KeyTypeObject == Other.KeyTypeObject;
}

bool FPCGMetadataAttributeDesc::IsCompatible(const FPCGMetadataAttributeDesc& Other) const
{
	if (ContainerTypes != Other.ContainerTypes)
	{
		return false;
	}

	auto ClassCompatible = [](EPCGMetadataTypes InType, const TObjectPtr<const UObject>& InTypeObject, EPCGMetadataTypes OtherType, const TObjectPtr<const UObject>& OtherTypeObject)
	{
		if (PCGMetadataAttributeDesc::IsObjectType(InType) && PCGMetadataAttributeDesc::IsObjectType(OtherType))
		{
			const UClass* InTypeClass = Cast<const UClass>(InTypeObject);
			const UClass* OtherTypeClass = Cast<const UClass>(OtherTypeObject);
			return InTypeClass && OtherTypeClass && InTypeClass->IsChildOf(OtherTypeClass);
		}
		else
		{
			return InType == OtherType && InTypeObject == OtherTypeObject;
		}
	};
	
	return ClassCompatible(ValueType, ValueTypeObject, Other.ValueType, Other.ValueTypeObject) 
		&& (!IsMap() || ClassCompatible(KeyType, KeyTypeObject, Other.KeyType, Other.KeyTypeObject));
}

bool FPCGMetadataAttributeDesc::IsValid() const
{
	auto ValidType = [](EPCGMetadataTypes InType, const TObjectPtr<const UObject>& InTypeObject)
	{
		if (InType >= EPCGMetadataTypes::Count)
		{
			return false;
		}
			
		return !PCGMetadataAttributeDesc::ShouldTypeRequireTypeObject(InType) || InTypeObject != nullptr;
	};
	
	if (ContainerTypes.Num() >= 2)
	{
		return false;
	}
	
	if (!ValidType(ValueType, ValueTypeObject))
	{
		return false;
	}
	
	if (!ContainerTypes.IsEmpty() && ContainerTypes[0] == EPCGMetadataAttributeContainerTypes::Map && !ValidType(KeyType, KeyTypeObject))
	{
		return false;
	}
	
	return true;
}

bool FPCGMetadataAttributeDesc::operator==(const FPCGMetadataAttributeDesc& Other) const
{
	return Name == Other.Name && IsSameType(Other);
}

void FPCGMetadataAttributeDesc::FixLegacyTypeId()
{
	auto FixType = [](EPCGMetadataTypes& InOutType, TObjectPtr<const UObject>& InOutTypeObject)
	{
		if (InOutType == EPCGMetadataTypes::Struct)
		{
			auto AssignType = [&InOutType, &InOutTypeObject](EPCGMetadataTypes NewType)
			{
				InOutType = NewType;
				InOutTypeObject = nullptr;
			};
	
			if (InOutTypeObject == TBaseStructure<FVector2D>::Get())
			{
				AssignType(EPCGMetadataTypes::Vector2);
			}
			else if (InOutTypeObject == TBaseStructure<FVector>::Get())
			{
				AssignType(EPCGMetadataTypes::Vector);
			}
			else if (InOutTypeObject == TBaseStructure<FVector4>::Get())
			{
				AssignType(EPCGMetadataTypes::Vector4);
			}
			else if (InOutTypeObject == TBaseStructure<FRotator>::Get())
			{
				AssignType(EPCGMetadataTypes::Rotator);
			}
			else if (InOutTypeObject == TBaseStructure<FQuat>::Get())
			{
				AssignType(EPCGMetadataTypes::Quaternion);
			}
			else if (InOutTypeObject == TBaseStructure<FTransform>::Get())
			{
				AssignType(EPCGMetadataTypes::Transform);
			}
			else if (InOutTypeObject == TBaseStructure<FSoftObjectPath>::Get())
			{
				AssignType(EPCGMetadataTypes::SoftObjectPath);
			}
			else if (InOutTypeObject == TBaseStructure<FSoftClassPath>::Get())
			{
				AssignType(EPCGMetadataTypes::SoftClassPath);
			}
		}
	};
	
	FixType(ValueType, ValueTypeObject);
	FixType(KeyType, KeyTypeObject);
}

bool FPCGMetadataAttributeDesc::IsSingleValue() const
{
	return ContainerTypes.IsEmpty() || ContainerTypes[0] == EPCGMetadataAttributeContainerTypes::None;
}

bool FPCGMetadataAttributeDesc::IsArray() const
{
	return !ContainerTypes.IsEmpty() && ContainerTypes[0] == EPCGMetadataAttributeContainerTypes::Array;
}

bool FPCGMetadataAttributeDesc::IsSet() const
{
	return !ContainerTypes.IsEmpty() && ContainerTypes[0] == EPCGMetadataAttributeContainerTypes::Set;
}

bool FPCGMetadataAttributeDesc::IsMap() const
{
	return !ContainerTypes.IsEmpty() && ContainerTypes[0] == EPCGMetadataAttributeContainerTypes::Map;
}

bool FPCGMetadataAttributeDesc::ContainsObject() const
{
	return PCGMetadataAttributeDesc::IsObjectType(ValueType) || (IsMap() && PCGMetadataAttributeDesc::IsObjectType(KeyType));
}

FString FPCGMetadataAttributeDesc::GetTypeString() const
{
	const UEnum* TypeEnum = StaticEnum<EPCGMetadataTypes>();
	check(TypeEnum);
	
	TStringBuilder<256> Builder;
	
	auto FillTypeStr = [TypeEnum, &Builder](EPCGMetadataTypes InType, const UObject* InTypeObject)
	{
		Builder += TypeEnum->GetNameStringByValue(static_cast<int64>(InType));

		if (PCGMetadataAttributeDesc::ShouldTypeRequireTypeObject(InType))
		{
			Builder += TEXT("<");
			Builder += InTypeObject ? InTypeObject->GetName() : TEXT("UNKNOWN");
			Builder += TEXT(">");
		}
	};
	
	if (IsSingleValue())
	{
		FillTypeStr(ValueType, ValueTypeObject);
	}
	else if (IsArray())
	{
		Builder += TEXT("Array<");
		FillTypeStr(ValueType, ValueTypeObject);
		Builder += TEXT(">");
	}
	else if (IsSet())
	{
		Builder += TEXT("Set<");
		FillTypeStr(ValueType, ValueTypeObject);
		Builder += TEXT(">");
	}
	else if (IsMap())
	{
		Builder += TEXT("Map<");
		FillTypeStr(KeyType, KeyTypeObject);
		Builder += TEXT(",");
		FillTypeStr(ValueType, ValueTypeObject);
		Builder += TEXT(">");
	}
	else
	{
		checkNoEntry();
	}
	
	return Builder.ToString();
}

FPCGMetadataAttributeDesc FPCGMetadataAttributeDesc::ConvertToArray() const
{
	FPCGMetadataAttributeDesc Result = *this;
	if (!IsArray())
	{
		Result.ContainerTypes = {EPCGMetadataAttributeContainerTypes::Array};
		Result.KeyType = EPCGMetadataTypes::Unknown;
		Result.KeyTypeObject = nullptr;
	}

	return Result;
}

FPCGMetadataAttributeDesc FPCGMetadataAttributeDesc::ConvertToSingleValue() const
{
	FPCGMetadataAttributeDesc Result = *this;
	if (!IsSingleValue())
	{
		Result.ContainerTypes.Empty();
		Result.KeyType = EPCGMetadataTypes::Unknown;
		Result.KeyTypeObject = nullptr;
	}

	return Result;
}

void FPCGMetadataAttributeDesc::ConvertObjectsToSoftPath()
{
	// Force objects/soft objects to be paths, as attributes do not support objects.
	auto FixDesc = [](EPCGMetadataTypes& InType, TObjectPtr<const UObject>& InTypeObject)
	{
		if (InType == EPCGMetadataTypes::Object || InType == EPCGMetadataTypes::SoftObject)
		{
			InType = EPCGMetadataTypes::SoftObjectPath;
			InTypeObject = nullptr;
		}
		else if (InType == EPCGMetadataTypes::Class || InType == EPCGMetadataTypes::SoftClass)
		{
			InType = EPCGMetadataTypes::SoftClassPath;
			InTypeObject = nullptr;
		}
	};
	
	FixDesc(ValueType, ValueTypeObject);
	FixDesc(KeyType, KeyTypeObject);
}

FPCGMetadataAttributeDesc FPCGMetadataAttributeDesc::CreateFromProperty(const FProperty* InProperty)
{
	if (!InProperty)
	{
		return {};
	}

	auto SetupProperty = [](const FProperty* InProperty, EPCGMetadataTypes& OutType, TObjectPtr<const UObject>& OutTypeObject) -> bool
	{
		if (!InProperty)
		{
			return false;
		}

		bool bValid = true;

		if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
		{
			if (NumericProperty->IsA<FDoubleProperty>())
			{
				OutType = EPCGMetadataTypes::Double;
			}
			else if (NumericProperty->IsA<FFloatProperty>())
			{
				OutType = EPCGMetadataTypes::Float;
			}
			else if (NumericProperty->IsA<FInt64Property>() || NumericProperty->IsA<FUInt64Property>())
			{
				OutType = EPCGMetadataTypes::Integer64;
			}
			else if (NumericProperty->IsA<FIntProperty>() || NumericProperty->IsA<FUInt32Property>())
			{
				OutType = EPCGMetadataTypes::Integer32;
			}
			else
			{
				// @todo_pcg: In case of int16/int8 it doesn't work.
				bValid = false;
			}
		}
		else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
		{
			// For Bool properties we have to be extra careful if they match packed properties, as it will not be readable as a bool exactly.
			// @todo_pcg: Support those down the line
			if (BoolProperty->IsNativeBool())
			{
				OutType = EPCGMetadataTypes::Boolean;
			}
			else
			{
				bValid = false;
			}
		}
		else if (const FStrProperty* StringProperty = CastField<FStrProperty>(InProperty))
		{
			OutType = EPCGMetadataTypes::String;
		}
		else if (const FNameProperty* NameProperty = CastField<FNameProperty>(InProperty))
		{
			OutType = EPCGMetadataTypes::Name;
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
		{
			OutType = EPCGMetadataTypes::Enum;
			OutTypeObject = EnumProperty->GetEnum();
		}
		else if (const FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(InProperty))
		{
			OutType = EPCGMetadataTypes::SoftClass;
			OutTypeObject = SoftClassProperty->MetaClass;
		}
		else if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(InProperty))
		{
			OutType = EPCGMetadataTypes::SoftObject;
			OutTypeObject = SoftObjectProperty->PropertyClass;
		}
		else if (const FClassProperty* ClassProperty = CastField<FClassProperty>(InProperty))
		{
			OutType = EPCGMetadataTypes::Class;
			OutTypeObject = ClassProperty->MetaClass;
		}
		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
		{
			OutType = EPCGMetadataTypes::Object;
			OutTypeObject = ObjectProperty->PropertyClass;
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			if (StructProperty->Struct == TBaseStructure<FVector>::Get())
			{
				OutType = EPCGMetadataTypes::Vector;
			}
			else if (StructProperty->Struct == TBaseStructure<FVector4>::Get())
			{
				OutType = EPCGMetadataTypes::Vector4;
			}
			else if (StructProperty->Struct == TBaseStructure<FQuat>::Get())
			{
				OutType = EPCGMetadataTypes::Quaternion;
			}
			else if (StructProperty->Struct == TBaseStructure<FTransform>::Get())
			{
				OutType = EPCGMetadataTypes::Transform;
			}
			else if (StructProperty->Struct == TBaseStructure<FRotator>::Get())
			{
				OutType = EPCGMetadataTypes::Rotator;
			}
			else if (StructProperty->Struct == TBaseStructure<FVector2D>::Get())
			{
				OutType = EPCGMetadataTypes::Vector2;
			}
			else if (StructProperty->Struct == TBaseStructure<FSoftObjectPath>::Get())
			{
				OutType = EPCGMetadataTypes::SoftObjectPath;
			}
			else if (StructProperty->Struct == TBaseStructure<FSoftClassPath>::Get())
			{
				OutType = EPCGMetadataTypes::SoftClassPath;
			}
			else
			{
				OutType = EPCGMetadataTypes::Struct;
				OutTypeObject = StructProperty->Struct;
			}
		}
		else
		{
			bValid = false;
		}

		return bValid;
	};

	FPCGMetadataAttributeDesc Result{};
	
	const UStruct* PropertyOuter = InProperty->GetOwnerStruct();
	const FName PropertyName = PropertyOuter ? *PropertyOuter->GetAuthoredNameForField(InProperty) : InProperty->GetFName();
	Result.Name = PropertyName;

	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
	{
		Result.ContainerTypes.Add(EPCGMetadataAttributeContainerTypes::Array);
		if (!SetupProperty(ArrayProperty->Inner, Result.ValueType, Result.ValueTypeObject))
		{
			return {};
		}
	}
	else if (const FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
	{
		Result.ContainerTypes.Add(EPCGMetadataAttributeContainerTypes::Set);
		if (!SetupProperty(SetProperty->ElementProp, Result.ValueType, Result.ValueTypeObject))
		{
			return {};
		}
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
	{
		Result.ContainerTypes.Add(EPCGMetadataAttributeContainerTypes::Map);
		if (!SetupProperty(MapProperty->ValueProp, Result.ValueType, Result.ValueTypeObject) || !SetupProperty(MapProperty->KeyProp, Result.KeyType, Result.KeyTypeObject))
		{
			return {};
		}
	}
	else if (InProperty)
	{
		if (!SetupProperty(InProperty, Result.ValueType, Result.ValueTypeObject))
		{
			return {};
		}
	}

	return Result;
}

int32 GetTypeHash(const FPCGMetadataAttributeDesc& Desc)
{
	int32 Hash = GetTypeHash(Desc.Name);
	Hash = HashCombine(Hash, GetTypeHash(Desc.ValueType));
	Hash = HashCombine(Hash, GetTypeHash(Desc.ContainerTypes));
	Hash = HashCombine(Hash, GetTypeHash(Desc.ValueTypeObject));
	Hash = HashCombine(Hash, GetTypeHash(Desc.KeyType));
	Hash = HashCombine(Hash, GetTypeHash(Desc.KeyTypeObject));
	return Hash;
}

/**
 * Create a new wrapper using the Attribute desc. This is expected to be stored in a SharedPtr to avoid having
 * to always recreate it.
 */
FPCGAttributeProperty::FPCGAttributeProperty(const FPCGMetadataAttributeDesc& InAttributeDesc)
{
	// Use an array property to be able to leverage FScriptArrayHelper. It will contain the property
	// that matches with the Descriptor.
	// Note: CreatePropertyFromDesc will allocate a raw pointer, and Property will become its owner. When Property
	// is destroyed, all allocated properties will also be deleted. It's all done internally.
	Property = MakeUnique<FArrayProperty>(FFieldVariant{nullptr}, NAME_None);
	FProperty* NewProperty = PCG::Private::CreatePropertyFromDesc(InAttributeDesc, Property.Get());

	if (!NewProperty)
	{
		Property.Reset();
		return;
	}

	Property->AddCppProperty(NewProperty);
	// We need to link the property so it fills the Property Flags and other members correctly.
	// We don't need an archive (that's how it is done in the FProperty::StaticLink), but FNullArchive is private so we copied it.
	PCG::Private::FNullArchive ArDummy;
	Property->Link(ArDummy);

	// Cache info if the data is plain old data.
	bIsPlainOldData = !!(NewProperty->PropertyFlags & CPF_IsPlainOldData);

	// Compress strings, soft paths and structs, if they are not in a set or map.
	bCompressData = (InAttributeDesc.IsSingleValue() || InAttributeDesc.IsArray()) &&
		(InAttributeDesc.ValueType == EPCGMetadataTypes::String
		|| InAttributeDesc.ValueType == EPCGMetadataTypes::Text
		|| InAttributeDesc.ValueType == EPCGMetadataTypes::SoftObjectPath
		|| InAttributeDesc.ValueType == EPCGMetadataTypes::SoftClassPath
		|| InAttributeDesc.ValueTypeObject == TBaseStructure<FSoftObjectPath>().Get()
		|| InAttributeDesc.ValueTypeObject == TBaseStructure<FSoftClassPath>().Get()
		|| InAttributeDesc.ValueType == EPCGMetadataTypes::Struct);

	InnerElementCachedSize = NewProperty->GetElementSize();
}

FPCGAttributeProperty::~FPCGAttributeProperty() = default;

void FPCGAttributeProperty::Copy(void* DestPtr, const void* SrcPtr, int32 Count) const
{
	PCG::Private::CopyArray(Property.Get(), DestPtr, SrcPtr, Count);
}

bool FPCGAttributeProperty::IsValid() const
{
	return Property.IsValid() && Property->Inner != nullptr;
}

const void* FPCGAttributeProperty::GetPtrInArray(const void* InArrayStart, int32 Index) const
{
	check(InnerElementCachedSize > 0);
	return static_cast<const uint8*>(InArrayStart) + Index * InnerElementCachedSize;
}