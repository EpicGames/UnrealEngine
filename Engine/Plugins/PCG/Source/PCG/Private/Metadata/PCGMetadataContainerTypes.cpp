// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGMetadataContainerTypes.h"

#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#define LOCTEXT_NAMESPACE "PCGMetadataContainerTypes"

PCG::FPCGAccessorBuffer::FPCGAccessorBuffer(const FPCGAccessorBuffer& Other)
{
	Internals = Other.Internals;
	Other.CopyMemoryTo(OwnMemory);
	bOwnMemory = !OwnMemory.IsEmpty();
}

PCG::FPCGAccessorBuffer& PCG::FPCGAccessorBuffer::operator=(const FPCGAccessorBuffer& Other)
{
	if (this == &Other)
	{
		// Nothing to do
		return *this;
	}

	// Need to make sure our own memory is clean.
	Reset();

	Internals = Other.Internals;

	Other.CopyMemoryTo(OwnMemory);
	bOwnMemory = !OwnMemory.IsEmpty();
	
	return *this;
}

PCG::FPCGAccessorBuffer::FPCGAccessorBuffer(FPCGAccessorBuffer&& Other)
{
	Other.MoveMemoryTo(OwnMemory);
	bOwnMemory = !OwnMemory.IsEmpty();
	Internals = MoveTemp(Other.Internals);
}

PCG::FPCGAccessorBuffer& PCG::FPCGAccessorBuffer::operator=(FPCGAccessorBuffer&& Other)
{
	if (this == &Other)
	{
		// Nothing to do
		return *this;
	}

	// Need to make sure our own memory is clean.
	Reset();
	
	Other.MoveMemoryTo(OwnMemory);
	bOwnMemory = !OwnMemory.IsEmpty();
	Internals = MoveTemp(Other.Internals);
	
	return *this;
}

PCG::FPCGAccessorBuffer::~FPCGAccessorBuffer()
{
	Reset();
}

void PCG::FPCGAccessorBuffer::Reset(int32 Slack)
{
	if (bOwnMemory)
	{
		check(Internals.ElementSize > 0);
		if (Internals.DestructFunc)
		{
			Internals.DestructFunc(OwnMemory.GetData(), OwnMemory.Num());
		}
		else if (Internals.ComplexDestructFunc)
		{
			Internals.ComplexDestructFunc(OwnMemory.GetData(), OwnMemory.Num());
		}
		
		OwnMemory.Empty(Slack, Internals.ElementSize, Internals.AlignmentSize);
	}
	
	bOwnMemory = false;
}

void* PCG::FPCGAccessorBuffer::GetOwnedMemoryPtr()
{
	return bOwnMemory ? OwnMemory.GetData() : nullptr;
}

int32 PCG::FPCGAccessorBuffer::Num() const
{
	return bOwnMemory ? OwnMemory.Num() : 0;
}

bool PCG::FPCGAccessorBuffer::IsOwningMemory() const
{
	return bOwnMemory;
}

const FPCGMetadataAttributeDesc& PCG::FPCGAccessorBuffer::GetUnderlyingDesc() const
{
	return Internals.UnderlyingDesc;
}

bool PCG::FPCGAccessorBuffer::CopyFromPointers(TConstArrayView<const void*> InValuesPtr, const FPCGMetadataAttributeDesc& InType)
{
	if (!InType.IsSameType(Internals.UnderlyingDesc) || !IsOwningMemory() || InValuesPtr.Num() != Num())
	{
		SetupAndAllocate(InValuesPtr.Num(), InType);
	}

	if (Num() != InValuesPtr.Num())
	{
		return false;
	}
	
	for (int32 i = 0; i < Num(); ++i)
	{
		uint8* WritePtr = static_cast<uint8*>(GetOwnedMemoryPtr()) + Internals.ElementSize * i;
		if (Internals.CopyFunc)
		{
			Internals.CopyFunc(WritePtr, InValuesPtr[i], 1);
		}
		else if (Internals.ComplexCopyFunc)
		{
			Internals.ComplexCopyFunc(WritePtr, InValuesPtr[i], 1);
		}
		else
		{
			FMemory::Memcpy(WritePtr, InValuesPtr[i], Internals.ElementSize);
		}
	}

	return true;
}

void PCG::FPCGAccessorBuffer::MoveMemoryTo(FScriptArray& Other, const FPCGMetadataAttributeDesc& InType)
{
	if (!Internals.UnderlyingDesc.IsSameType(InType))
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("MismatchedTypes", "Try to move memory from the setter to another array of the wrong type."));
		return;
	}

	MoveMemoryTo(Other);
}

void PCG::FPCGAccessorBuffer::MoveMemoryTo(FScriptArray& Other)
{
	if (bOwnMemory)
	{
		check(Internals.ElementSize > 0 && Internals.AlignmentSize > 0);
		Other.MoveAssign(OwnMemory, Internals.ElementSize, Internals.AlignmentSize);
	}
	
	bOwnMemory = false;
}

void PCG::FPCGAccessorBuffer::CopyMemoryTo(FScriptArray& Other) const
{
	if (!bOwnMemory)
	{
		return;
	}
	
	check(Internals.ElementSize > 0 && Internals.AlignmentSize > 0);
	
	// Other needs to be empty otherwise we're not sure how we can delete it.
	check(Other.IsEmpty());
	Other.Add(OwnMemory.Num(), Internals.ElementSize, Internals.AlignmentSize);
	if (Internals.ConstructFunc)
	{
		Internals.ConstructFunc(Other.GetData(), Other.Num());
	}
	
	if (Internals.CopyFunc)
	{
		Internals.CopyFunc(Other.GetData(), OwnMemory.GetData(), Other.Num());
	}
	else if (Internals.ComplexCopyFunc)
	{
		Internals.ComplexCopyFunc(Other.GetData(), OwnMemory.GetData(), Other.Num());
	}
	else
	{
		FMemory::Memcpy(Other.GetData(), OwnMemory.GetData(), Other.Num() * Internals.ElementSize);
	}
}

void PCG::FPCGAccessorBuffer::Allocate(int32 Count)
{
	check(Internals.ElementSize > 0 && Internals.AlignmentSize > 0);
	
	Reset(/*Slack=*/Count);
	OwnMemory.Add(Count, Internals.ElementSize, Internals.AlignmentSize);
	if (Internals.ConstructFunc)
	{
		Internals.ConstructFunc(OwnMemory.GetData(), OwnMemory.Num());
	}
	else if (Internals.ComplexConstructFunc)
	{
		Internals.ComplexConstructFunc(OwnMemory.GetData(), OwnMemory.Num());
	}
	
	bOwnMemory = true;
}

void PCG::FPCGAccessorBuffer::ResetInternals()
{
	Reset();

	Internals.UnderlyingDesc = FPCGMetadataAttributeDesc{};

	Internals.ElementSize = -1;
	Internals.AlignmentSize = -1;
	Internals.ConstructFunc = nullptr;
	Internals.DestructFunc = nullptr;
	Internals.CopyFunc = nullptr;

	Internals.UnderlyingProperty.Reset();
	Internals.ComplexConstructFunc.Reset();
	Internals.ComplexDestructFunc.Reset();
	Internals.ComplexCopyFunc.Reset();
}

void PCG::FPCGAccessorBuffer::SetupAndAllocate(int32 Count, const FPCGMetadataAttributeDesc& InType)
{
	ResetInternals();
	
	if (!InType.IsSingleValue() || !InType.IsValid())
	{
		return;
	}
	
	Internals.UnderlyingDesc = InType;
	
	// If we know the element size, use the accelerated functions
	const uint16 PCGType = static_cast<uint16>(InType.ValueType);
	if (PCG::Private::GetElementSize(PCGType) > 0)
	{
		Internals.ElementSize = PCG::Private::GetElementSize(PCGType);
		Internals.AlignmentSize = PCG::Private::GetAlignmentSize(PCGType);
		Internals.ConstructFunc = PCG::Private::GetConstructMemoryFunc(PCGType);
		Internals.DestructFunc = PCG::Private::GetDestroyMemoryFunc(PCGType);
		if (!PCG::Private::CanBeMemcpy(PCGType))
		{
			Internals.CopyFunc = PCG::Private::GetConstructibleTransformFunc(PCGType, PCGType);
		}
	}
	else
	{
		Internals.UnderlyingProperty = MakeShared<FPCGAttributeProperty>(Internals.UnderlyingDesc);
		Internals.ElementSize = Internals.UnderlyingProperty->GetProperty()->Inner->GetElementSize();
		Internals.AlignmentSize = Internals.UnderlyingProperty->GetProperty()->Inner->GetMinAlignment();
		Internals.ComplexConstructFunc = [Property = Internals.UnderlyingProperty->GetProperty()->Inner](void* DestPtr, int32 Count)
		{
			if (Property->HasAnyPropertyFlags(CPF_ZeroConstructor))
			{
				FMemory::Memzero(DestPtr, Count * Property->GetElementSize());
			}
			else
			{
				for (int32 i = 0; i < Count; ++i)
				{
					Property->InitializeValue(static_cast<uint8*>(DestPtr) + i * Property->GetElementSize());
				}
			}
		};
		
		Internals.ComplexDestructFunc = [Property = Internals.UnderlyingProperty->GetProperty()->Inner](void* DestPtr, int32 Count)
		{
			if (!Property->HasAnyPropertyFlags(CPF_NoDestructor))
			{
				for (int32 i = 0; i < Count; ++i)
				{
					Property->DestroyValue(static_cast<uint8*>(DestPtr) + i * Property->GetElementSize());
				}
			}
		};
		
		Internals.ComplexCopyFunc = [ArrayProperty = Internals.UnderlyingProperty->GetProperty()](void* DestPtr, const void* SrcPtr, int32 Count)
		{
			PCG::Private::CopyArray(ArrayProperty, DestPtr, SrcPtr, Count);
		};
	}
	
	Allocate(Count);
}

#undef LOCTEXT_NAMESPACE
