// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGMetadataDomain.h"

#include "PCGContext.h"
#include "PCGElement.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadata.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGPropertyHelpers.h"

#include "Algo/AnyOf.h"
#include "Async/ParallelFor.h"
#include "Serialization/ArchiveCrc32.h"

#define LOCTEXT_NAMESPACE "PCGMetadataDomain"

namespace PCGMetadata
{
	template<typename DataType>
	bool CreateAttributeFromPropertyHelper(FPCGMetadataDomain* Metadata, FName AttributeName, const DataType* DataPtr, const FProperty* InProperty)
	{
		if (!InProperty || !DataPtr || !Metadata)
		{
			return false;
		}

		auto CreateAttribute = [AttributeName, Metadata]<typename T>(T PropertyValue) -> bool
		{
			FPCGMetadataAttributeBase* BaseAttribute = Metadata->FindOrCreateAttribute<T>(AttributeName, PropertyValue, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false, /*bOverwriteIfTypeMismatch=*/true);
			
			return (BaseAttribute != nullptr);
		};

		return PCGPropertyHelpers::GetPropertyValueWithCallback(DataPtr, InProperty, CreateAttribute);
	}

	template<typename DataType>
	bool SetAttributeFromPropertyHelper(FPCGMetadataDomain* Metadata, FName AttributeName, PCGMetadataEntryKey& EntryKey, const DataType* DataPtr, const FProperty* InProperty, bool bCreate)
	{
		if (!InProperty || !DataPtr || !Metadata)
		{
			return false;
		}

		// Check if an attribute already exists or not if we ask to create a new one
		if (!bCreate && !Metadata->HasAttribute(AttributeName))
		{
			return false;
		}

		auto CreateAttributeAndSet = [AttributeName, Metadata, bCreate, &EntryKey]<typename PropertyType>(PropertyType PropertyValue) -> bool
		{
			FPCGMetadataAttributeBase* BaseAttribute = Metadata->GetMutableAttribute(AttributeName);

			if (!BaseAttribute && bCreate)
			{
				// Interpolation is disabled and no parent override.
				BaseAttribute = Metadata->CreateAttribute<PropertyType>(AttributeName, PropertyValue, false, false);
			}

			if (!BaseAttribute)
			{
				return false;
			}

			// @todo_pcg To be modified to use accessors.
			// Allow to set the value if both type matches or if we can construct AttributeType from PropertyType.
			return PCGMetadataAttribute::CallbackWithRightType(BaseAttribute->GetTypeId(), [&EntryKey, &PropertyValue, BaseAttribute, Metadata]<typename AttributeType>(AttributeType AttributeValue) -> bool
				{
					FPCGMetadataAttribute<AttributeType>* Attribute = static_cast<FPCGMetadataAttribute<AttributeType>*>(BaseAttribute);

					// Special cased because FSoftObjectPath currently has a deprecated constructor from FName which generates compile warnings.
					constexpr bool bAssigningNameToSoftObjectPath = std::is_same_v<AttributeType, FSoftObjectPath> && std::is_same_v<PropertyType, FName>;

					if constexpr (std::is_same_v<AttributeType, PropertyType>)
					{
						Metadata->InitializeOnSet(EntryKey);
						Attribute->SetValue(EntryKey, PropertyValue);
						return true;
					}
					else if constexpr (std::is_constructible_v<AttributeType, PropertyType> && !bAssigningNameToSoftObjectPath)
					{
						Metadata->InitializeOnSet(EntryKey);
						Attribute->SetValue(EntryKey, AttributeType(PropertyValue));
						return true;
					}
					else
					{
						return false;
					}
				});
		};

		return PCGPropertyHelpers::GetPropertyValueWithCallback(DataPtr, InProperty, CreateAttributeAndSet);
	}

	/** Utility structure to filter attributes when adding them. Must not be kept around as we hold a const ref to Params.*/
	struct FPCGMetadataAttributeNameFilter
	{
		FPCGMetadataAttributeNameFilter(const FPCGMetadataDomainInitializeParams& InParams)
			: Params(InParams)
		{
			if (Params.MatchOperator != EPCGStringMatchingOperator::Equal && Params.FilteredAttributes)
			{
				Algo::Transform(*Params.FilteredAttributes, NameStrings, [](const FName& InFilteredAttribute) {return InFilteredAttribute.ToString(); });
			}
		}

		/** Returns true if InName should be excluded. */
		bool operator()(const FName InName)
		{
			bool Result = false;

			if (Params.MatchOperator == EPCGStringMatchingOperator::Equal)
			{
				Result = Params.FilteredAttributes && Params.FilteredAttributes->Contains(InName);
			}
			else
			{
				const FString OtherAttributeString = InName.ToString();
				if (Params.MatchOperator == EPCGStringMatchingOperator::Substring)
				{
					Result = Algo::AnyOf(NameStrings, [&OtherAttributeString](const FString& InAttribute) { return OtherAttributeString.Contains(InAttribute); });
				}
				else if (Params.MatchOperator == EPCGStringMatchingOperator::Matches)
				{
					Result = Algo::AnyOf(NameStrings, [&OtherAttributeString](const FString& InAttribute) { return OtherAttributeString.MatchesWildcard(InAttribute); });
				}
				else
				{
					checkNoEntry();
					return false;
				}
			}

			return Params.FilterMode == EPCGMetadataFilterMode::ExcludeAttributes ? Result : !Result;
		}

	private:
		const FPCGMetadataDomainInitializeParams& Params;
		TArray<FString> NameStrings;
	};
}


//////////////////////////
/// FPCGMetadataDomain
//////////////////////////


void FPCGMetadataDomain::Serialize(FArchive& InArchive)
{
	LLM_SCOPE_BYTAG(PCG);
	
	InArchive.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	
	// To delta serialize against the default
	static const FPCGMetadataAttributeDesc DefaultDesc{};

	int32 NumAttributes = (InArchive.IsLoading() ? 0 : Attributes.Num());
	// We need to keep track of the max attribute Id, since it won't necessary be equal to the number of attributes + 1.
	int64 MaxAttributeId = -1;

	InArchive << NumAttributes;

	if (InArchive.IsLoading())
	{
		for (int32 AttributeIndex = 0; AttributeIndex < NumAttributes; ++AttributeIndex)
		{
			FName AttributeName = NAME_None;
			InArchive << AttributeName;
			
			FPCGMetadataAttributeBase* SerializedAttribute = nullptr;
			FPCGMetadataAttributeDesc AttributeDesc;
			AttributeDesc.Name = AttributeName;
		
			if (InArchive.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MergePCGMetadataAttributeBaseAndGeneric)
			{
				int32 AttributeTypeId = 0;
				InArchive << AttributeTypeId;
				
				AttributeDesc.ValueType = static_cast<EPCGMetadataTypes>(AttributeTypeId);
			}
			else
			{
				FPCGMetadataAttributeDesc::StaticStruct()->SerializeItem(InArchive, &AttributeDesc, /*Defaults=*/&DefaultDesc);
			}
			
			SerializedAttribute = AllocateAttributeFromDesc(AttributeDesc);
			check(SerializedAttribute);

			SerializedAttribute->Name = AttributeName;
			SerializedAttribute->Serialize(this, InArchive);
			Attributes.Add(AttributeName, SerializedAttribute);
			MaxAttributeId = FMath::Max(SerializedAttribute->AttributeId, MaxAttributeId);
		}
	}
	else
	{
		for (auto& [Name, Attribute] : Attributes)
		{
			InArchive << Name;
			
			FPCGMetadataAttributeDesc::StaticStruct()->SerializeItem(InArchive, &Attribute->CachedDesc, /*Defaults=*/&DefaultDesc);

			Attribute->Serialize(this, InArchive);
		}
	}

	InArchive << ParentKeys;

	// Finally, initialize non-serialized members
	if (InArchive.IsLoading())
	{
		// The next attribute id need to be bigger than the max attribute id of all attributes (or we could have collisions).
		// Therefore by construction, it should never be less than the number of attributes (but can be greater).
		NextAttributeId = MaxAttributeId + 1;
		check(NextAttributeId >= Attributes.Num());
		ItemKeyOffset = (Parent ? Parent->GetItemCountForChild() : 0);
	}
}

void FPCGMetadataDomain::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
{
	// Iterate over all attributes
	{
		PCG::TSharedScopeLock ReadLock(AttributeLock);
		for (const auto& [AttributeName, AttributePtr] : Attributes)
		{
			if (AttributePtr)
			{
				AttributePtr->GetResourceSizeEx(CumulativeResourceSize);
			}
		}
	}

	// Count entry keys/parent keys
	{
		PCG::TSharedScopeLock ReadLock(ItemLock);
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(ParentKeys.GetAllocatedSize());
	}
}

void FPCGMetadataDomain::AddToCrc(FArchiveCrc32& Ar, const UPCGData* Data, bool bFullDataCrc) const
{
	check(Data);

	TArray<const FPCGMetadataAttributeBase*> AllAttributes;

	{
		PCG::TSharedScopeLock ReadLock(AttributeLock);
		AllAttributes.Reserve(Attributes.Num());

		for (const TPair<FName, FPCGMetadataAttributeBase*>& Attribute : Attributes)
		{
			if (!ensure(Attribute.Value))
			{
				continue;
			}
			
			AllAttributes.Add(Attribute.Value);
		}
	}

	if (AllAttributes.IsEmpty())
	{
		return;
	}

	// Sort attributes so we have a consistent processing path
	Algo::Sort(AllAttributes, [this](const FPCGMetadataAttributeBase* A, const FPCGMetadataAttributeBase* B) { return A->Name.LexicalLess(B->Name); });

	// Create the keys only once, as they are the same for all the attributes.
	FPCGAttributePropertyInputSelector InputSource;
	Data->SetDomainFromDomainID(DomainID, InputSource);
	InputSource.SetAttributeName(AllAttributes[0]->Name);

	TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(Data, InputSource);
	if (!ensure(InputKeys.IsValid()))
	{
		return;
	}

	TArray<const PCGMetadataEntryKey*> EntryKeysPtr;
	TArray<PCGMetadataEntryKey> EntryKeys;

	// Then for each attribute, serialize the name and its values.
	for (const FPCGMetadataAttributeBase* Attribute : AllAttributes)
	{
		Ar << const_cast<FName&>(Attribute->Name);

		// To make sure we are not changing the CRC, for old values keep the old way of doing it (with slight optimization to pull non-trivial types
		// by pointer)
		if (Attribute->GetAttributeDesc().IsSingleValue() && Attribute->GetAttributeDesc().ValueType < EPCGMetadataTypes::EndLegacyTypes)
		{
			TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Attribute, this);
			if (!InputAccessor.IsValid())
			{
				continue;
			}

			auto Callback = [&InputAccessor, &InputKeys, &Ar]<typename T>(T)
			{
				// Pull non-trivial types as pointers.
				constexpr bool bUsePointers = !std::is_trivially_copyable_v<T>;
				using AttributeType = std::conditional_t<bUsePointers, const T*, T>;
				TArray<AttributeType> Values;
				Values.SetNumUninitialized(InputKeys->GetNum());

				InputAccessor->GetRange<AttributeType>(Values, 0, *InputKeys);

				for (const AttributeType& Value : Values)
				{
					// Add value to Crc
					if constexpr (bUsePointers)
					{
						PCG::Private::Serialize<const T&>(Ar, *Value);
					}
					else
					{
						PCG::Private::Serialize<const T&>(Ar, Value);
					}
				}
			};

			PCGMetadataAttribute::CallbackWithRightType(InputAccessor->GetUnderlyingType(), Callback);
		}
		else
		{
			// Pull the keys if they are not yet pulled
			if (EntryKeys.IsEmpty())
			{
				EntryKeysPtr.SetNumUninitialized(InputKeys->GetNum());
				if (!InputKeys->GetKeys<PCGMetadataEntryKey>(0, EntryKeysPtr))
				{
					continue;
				}
	
				EntryKeys.Reserve(EntryKeysPtr.Num());
				Algo::Transform(EntryKeysPtr, EntryKeys, [](const PCGMetadataEntryKey* EntryKey) { return EntryKey ? *EntryKey : PCGInvalidEntryKey; });
			}

			Attribute->SerializeValuesForEntryKeys(EntryKeys, Ar);
		}
	}
}

FPCGMetadataDomain::FPCGMetadataDomain(UPCGMetadata* InTopMetadata, FPCGMetadataDomainID InMetadataDomainID)
	: TopMetadata(InTopMetadata)
	, DomainID(InMetadataDomainID)
	, bSupportMultiEntries(InTopMetadata && InTopMetadata->MetadataDomainSupportsMultiEntries(DomainID))
{
	check(InTopMetadata);
}

FPCGMetadataDomain::~FPCGMetadataDomain()
{
	PCG::TUniqueScopeLock WriteLock(AttributeLock);
	for (TPair<FName, FPCGMetadataAttributeBase*>& AttributeEntry : Attributes)
	{
		delete AttributeEntry.Value;
		AttributeEntry.Value = nullptr;
	}
	Attributes.Reset();
}

void FPCGMetadataDomain::Initialize(const FPCGMetadataDomain* InParent)
{
	Initialize(FPCGMetadataDomainInitializeParams(InParent));
}

void FPCGMetadataDomain::Initialize(const FPCGMetadataDomainInitializeParams& InParams)
{
	if (Parent || Attributes.Num() != 0)
	{
		// Already initialized; note that while that might be construed as a warning, there are legit cases where this is correct
		return;
	}

	// If we don't have a top metadata (ill-formed), or we don't support parenting, force the copy
	if (!ensure(TopMetadata) || !TopMetadata->MetadataDomainSupportsParenting(DomainID))
	{
		// Make sure we have nothing in the attribute to copy
		if (InParams.OptionalEntriesToCopy)
		{
			FPCGMetadataDomainInitializeParams CopyParams = InParams;
			CopyParams.OptionalEntriesToCopy.Reset();
			InitializeAsCopy(CopyParams);
		}
		else
		{
			InitializeAsCopy(InParams);
		}

		return;
	}

	// Make sure that the parent of the top metadata is also set correctly
	if (TopMetadata->Parent == nullptr && InParams.Parent)
	{
		TopMetadata->Parent = InParams.Parent->TopMetadata;
	}

	Parent = ((InParams.Parent != this) ? InParams.Parent : nullptr);
	ItemKeyOffset = Parent ? Parent->GetItemCountForChild() : 0;

	// If we have been given an include list which is empty, then don't bother adding any attributes
	const bool bSkipAddingAttributesFromParent = (InParams.FilterMode == EPCGMetadataFilterMode::IncludeAttributes) && (!InParams.FilteredAttributes || InParams.FilteredAttributes->IsEmpty());
	if (!bSkipAddingAttributesFromParent)
	{
		AddAttributes(InParams);
	}
}

void FPCGMetadataDomain::InitializeAsCopy(const FPCGMetadataDomain* InMetadataToCopy)
{
	InitializeAsCopy(FPCGMetadataDomainInitializeParams(InMetadataToCopy));
}

void FPCGMetadataDomain::InitializeAsCopy(const FPCGMetadataDomainInitializeParams& InParams)
{
	if (!InParams.Parent)
	{
		return;
	}

	if (Parent || Attributes.Num() != 0)
	{
		UE_LOGF(LogPCG, Error, "Metadata has already been initialized or already contains attributes");
		return;
	}

	PCGMetadata::FPCGMetadataAttributeNameFilter ShouldSkipAttribute(InParams);

	// If we have a partial copy, it will flatten the metadata, so we don't need a parent.
	// Otherwise, we keep the parent hierarchy.
	const bool bPartialCopy = InParams.OptionalEntriesToCopy.IsSet();
	TArray<PCGMetadataEntryKey> NewEntryKeys;
	TArray<PCGMetadataValueKey> NewValueKeys;
	if (bPartialCopy)
	{
		const int32 Count = InParams.OptionalEntriesToCopy->Num();
		NewEntryKeys.SetNumUninitialized(Count);
		NewValueKeys.SetNumUninitialized(Count);
		ParentKeys.SetNumUninitialized(Count);
		for (int32 j = 0; j < Count; ++j)
		{
			NewEntryKeys[j] = PCGMetadataEntryKey(j);
			ParentKeys[j] = -1;
		}

		ItemKeyOffset = 0;
	}
	else
	{
		ParentKeys = InParams.Parent->ParentKeys;
		ItemKeyOffset = InParams.Parent->ItemKeyOffset;
		Parent = InParams.Parent->Parent;
	}

	// Copy attributes
	for (const TPair<FName, FPCGMetadataAttributeBase*>& OtherAttribute : InParams.Parent->Attributes)
	{
		if (!ShouldSkipAttribute(OtherAttribute.Key))
		{
			// Don't copy entries if we have a partial copy, we will set them all after.
			FPCGMetadataAttributeBase* Attribute = CopyAttribute(OtherAttribute.Value, OtherAttribute.Key, /*bKeepParent=*/false, /*bCopyEntries=*/!bPartialCopy, /*bCopyValues=*/true);

			if (bPartialCopy && OtherAttribute.Value && Attribute)
			{
				OtherAttribute.Value->GetValueKeys(*InParams.OptionalEntriesToCopy, NewValueKeys);
				Attribute->SetValuesFromValueKeys(NewEntryKeys, NewValueKeys);
			}
		}
	}
}

bool FPCGMetadataDomain::AddAttributes(const FPCGMetadataDomain* InOther)
{
	FPCGMetadataDomainInitializeParams Params{InOther};
	return AddAttributes(Params);
}

bool FPCGMetadataDomain::AddAttributes(const FPCGMetadataDomainInitializeParams& InParams)
{
	if (!InParams.Parent)
	{
		return false;
	}

	PCGMetadata::FPCGMetadataAttributeNameFilter ShouldSkipAttribute(InParams);

	bool bAttributeAdded = false;

	for (const TPair<FName, FPCGMetadataAttributeBase*>& OtherAttribute : InParams.Parent->Attributes)
	{
		if (ShouldSkipAttribute(OtherAttribute.Key) || !OtherAttribute.Value)
		{
			continue;
		}
		else if (HasAttribute(OtherAttribute.Key))
		{
			// If both the current attribute and the other attribute have the same type - nothing to do
			// If the current attribute can be broadcasted to the other but not the other way around - change the type
			// If none of this is true - do nothing
			const FPCGMetadataAttributeBase* Attribute = GetConstAttribute(OtherAttribute.Key);
			check(Attribute);
			
			const FPCGMetadataAttributeDesc& AttributeDesc =  Attribute->GetAttributeDesc();
			const FPCGMetadataAttributeDesc& OtherAttributeDesc =  OtherAttribute.Value->GetAttributeDesc();

			if (!AttributeDesc.IsSameType(OtherAttributeDesc) &&
				!PCG::Private::IsBroadcastable(OtherAttributeDesc, AttributeDesc) &&
				PCG::Private::IsBroadcastable(AttributeDesc, OtherAttributeDesc))
			{
				ChangeAttributeType(OtherAttribute.Key, OtherAttributeDesc);
			}
		}
		else if (CopyAttribute(OtherAttribute.Value, OtherAttribute.Key, /*bKeepParent=*/InParams.Parent == Parent, /*bCopyEntries=*/false, /*bCopyValues=*/false))
		{
			bAttributeAdded = true;
		}
	}

	return bAttributeAdded;
}

bool FPCGMetadataDomain::AddAttribute(const FPCGMetadataDomain* InOther, FName AttributeName)
{
	if (!InOther || !InOther->HasAttribute(AttributeName) || HasAttribute(AttributeName))
	{
		return false;
	}

	return CopyAttribute(InOther->GetConstAttribute(AttributeName), AttributeName, /*bKeepParent=*/InOther == Parent, /*bCopyEntries=*/false, /*bCopyValues=*/false) != nullptr;
}

void FPCGMetadataDomain::CopyAttributes(const FPCGMetadataDomain* InOther)
{
	if (!InOther || InOther == Parent)
	{
		return;
	}

	if (GetItemCountForChild() != InOther->GetItemCountForChild())
	{
		UE_LOGF(LogPCG, Error, "Mismatch in copy attributes since the entries do not match");
		return;
	}

	for (const TPair<FName, FPCGMetadataAttributeBase*> OtherAttribute : InOther->Attributes)
	{
		if (HasAttribute(OtherAttribute.Key))
		{
			continue;
		}
		else
		{
			CopyAttribute(OtherAttribute.Value, OtherAttribute.Key, /*bKeepParent=*/false, /*bCopyEntries=*/true, /*bCopyValues=*/true);
		}
	}
}

void FPCGMetadataDomain::CopyAttribute(const FPCGMetadataDomain* InOther, FName AttributeToCopy, FName NewAttributeName)
{
	if (!InOther)
	{
		return;
	}
	else if (HasAttribute(NewAttributeName) || !InOther->HasAttribute(AttributeToCopy))
	{
		return;
	}
	else if (InOther == Parent)
	{
		CopyExistingAttribute(AttributeToCopy, NewAttributeName);
		return;
	}

	if (GetItemCountForChild() != InOther->GetItemCountForChild())
	{
		UE_LOGF(LogPCG, Error, "Mismatch in copy attributes since the entries do not match");
		return;
	}

	CopyAttribute(InOther->GetConstAttribute(AttributeToCopy), NewAttributeName, /*bKeepParent=*/false, /*bCopyEntries=*/true, /*bCopyValues=*/true);
}

const FPCGMetadataDomain* FPCGMetadataDomain::GetRoot() const
{
	if (Parent)
	{
		return Parent->GetRoot();
	}
	else
	{
		return this;
	}
}

bool FPCGMetadataDomain::HasParent(const FPCGMetadataDomain* InTentativeParent) const
{
	if (!InTentativeParent)
	{
		return false;
	}

	const FPCGMetadataDomain* HierarchicalParent = Parent;
	while (HierarchicalParent && HierarchicalParent != InTentativeParent)
	{
		HierarchicalParent = HierarchicalParent->Parent;
	}

	return HierarchicalParent == InTentativeParent;
}

void FPCGMetadataDomain::FlattenImpl()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSubMetadata::FlattenImpl);

	const int32 NumEntries = GetItemCountForChild();

	{
		PCG::TUniqueScopeLock WriteLock(AttributeLock);
		for (auto& AttributePair : Attributes)
		{
			FPCGMetadataAttributeBase* Attribute = AttributePair.Value;
			check(Attribute);

			// For all stored entries (from the root), we need to make sure that entries that should have a concrete value have it
			// Optimization notes:
			// - we could skip entries that existed prior to attribute existence, etc.
			// - we could skip entries that have no parent, but that would require checking against the parent entries in the parent hierarchy
			for (int64 EntryKey = 0; EntryKey < NumEntries; ++EntryKey)
			{
				// Get value using value inheritance as expected
				PCGMetadataValueKey ValueKey = Attribute->GetValueKey(EntryKey);
				if (ValueKey != PCGDefaultValueKey)
				{
					// Set concrete non-default value
					Attribute->SetValueFromValueKey(EntryKey, ValueKey);
				}
			}

			// Finally, flatten values
			Attribute->Flatten();
		}
	}

	Parent = nullptr;
	ParentKeys.Reset();
	ParentKeys.Init(PCGInvalidEntryKey, NumEntries);
	ItemKeyOffset = 0;
}

bool FPCGMetadataDomain::FlattenAndCompress(const TArrayView<const PCGMetadataEntryKey>& InEntryKeysToKeep)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSubMetadata::FlattenAndCompress);

	// No keys or no parents, nothing to do
	int32 NumEntryToKeep = InEntryKeysToKeep.Num();
	if (!Attributes.IsEmpty())
	{
		TArrayView<const PCGMetadataEntryKey> EntryKeysToKeep = InEntryKeysToKeep;
		
		if (InEntryKeysToKeep.Num() > 1 && !SupportsMultiEntries())
		{
			EntryKeysToKeep = InEntryKeysToKeep.Left(1);
			UE_LOGF(LogPCG, Warning, "Tried to flatten and compress a domain '%ls', which doesn't support multiple entries, with %d entries to keep. Will only keep the first one.", *DomainID.DebugName.ToString(), InEntryKeysToKeep.Num());
		}

		{
			PCG::TUniqueScopeLock WriteLock(AttributeLock);
			for (auto& AttributePair : Attributes)
			{
				FPCGMetadataAttributeBase* Attribute = AttributePair.Value;
				check(Attribute);

				Attribute->FlattenAndCompress(EntryKeysToKeep);
			}
		}
	}
	else
	{
		NumEntryToKeep = 0;
	}

	Parent = nullptr;
	ParentKeys.Reset();
	ParentKeys.Init(PCGInvalidEntryKey, NumEntryToKeep);
	ItemKeyOffset = 0;

	return true;
}

void FPCGMetadataDomain::AddAttributeInternal(FName AttributeName, FPCGMetadataAttributeBase* Attribute)
{
	// This call assumes we have a write lock on the attribute map.
	Attributes.Add(AttributeName, Attribute);
}

void FPCGMetadataDomain::RemoveAttributeInternal(FName AttributeName)
{
	Attributes.Remove(AttributeName);
}

void FPCGMetadataDomain::SetLastCachedSelectorOnOwner(FName AttributeName)
{
	TopMetadata->SetLastCachedSelectorOnOwner(AttributeName, DomainID);
}

FPCGMetadataAttributeBase* FPCGMetadataDomain::AllocateAttributeFromDesc(const FPCGMetadataAttributeDesc& InDesc, const FPCGMetadataAttributeBase* InAttributeParent)
{
	// Single value and "old" types will be allocated as FPCGMetadataAttribute<T>, so we can still do `static_cast<FPCGMetadataAttribute<T>>(Attribute)`
	FPCGMetadataAttributeBase* Result = nullptr;
	if (InDesc.IsSingleValue())
	{
		Result = PCGMetadataAttribute::CallbackWithRightType(static_cast<uint16>(InDesc.ValueType), [this, &InDesc, InAttributeParent]<typename T>(T&&) -> FPCGMetadataAttributeBase*
		{
			// Making sure we do not change those types.
			static_assert(PCG::Private::TIsBasicType<T>::Value);
			return new FPCGMetadataAttribute<T>(typename FPCGMetadataAttribute<T>::FProtectedToken(), this, InDesc.Name, InAttributeParent, T{}, /*bInAllowsInterpolation=*/false);
		});
	}
	
	if (!Result)
	{
		if (InDesc.ContainsObject())
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("ValueTypeObjectIsUObject", "Tried to create an Object typed attribute ({0}: {1}). This is not supported, use SoftObjectPath/SoftClassPath"), FText::FromName(InDesc.Name), InDesc.GetTypeText()));
			return nullptr;
		}

		Result = new FPCGMetadataAttributeBase(InDesc, InAttributeParent, this, /*bInAllowsInterpolation=*/false);
	}
	
	if (Result && !Result->UnderlyingProperty)
	{
		// The attribute is invalid
		delete Result;
		Result = nullptr;
	}
	
	return Result;
}

FPCGMetadataAttributeBase* FPCGMetadataDomain::GetMutableAttribute(FName AttributeName)
{
	PCG::TSharedScopeLock ReadLock(AttributeLock);
	return GetMutableAttribute_Unsafe(AttributeName);
}
	
FPCGMetadataAttributeBase* FPCGMetadataDomain::GetMutableAttribute_Unsafe(FName AttributeName)
{
	FPCGMetadataAttributeBase* Attribute = nullptr;
	if (FPCGMetadataAttributeBase** FoundAttribute = Attributes.Find(AttributeName))
	{
		Attribute = *FoundAttribute;

		// Also when accessing an attribute, notify the PCG Data owner that the latest attribute manipulated is this one.
		SetLastCachedSelectorOnOwner(AttributeName);
	}

	return Attribute;
}

const FPCGMetadataAttributeBase* FPCGMetadataDomain::GetConstAttribute(FName AttributeName) const
{
	PCG::TSharedScopeLock ReadLock(AttributeLock);
	return GetConstAttribute_Unsafe(AttributeName);
}

const FPCGMetadataAttributeBase* FPCGMetadataDomain::GetConstAttribute_Unsafe(FName AttributeName) const
{
	const FPCGMetadataAttributeBase* Attribute = nullptr;
	if (const FPCGMetadataAttributeBase* const* FoundAttribute = Attributes.Find(AttributeName))
	{
		Attribute = *FoundAttribute;
	}
	
	return Attribute;
}

const FPCGMetadataAttributeBase* FPCGMetadataDomain::GetConstAttributeById(int32 InAttributeId) const
{
	const FPCGMetadataAttributeBase* Attribute = nullptr;

	{
		PCG::TSharedScopeLock ReadLock(AttributeLock);
		for (const auto& AttributePair : Attributes)
		{
			if (AttributePair.Value && AttributePair.Value->AttributeId == InAttributeId)
			{
				Attribute = AttributePair.Value;
				break;
			}
		}
	}

	return Attribute;
}

bool FPCGMetadataDomain::HasAttribute(FName AttributeName) const
{
	PCG::TSharedScopeLock ReadLock(AttributeLock);
	return Attributes.Contains(AttributeName);
}

bool FPCGMetadataDomain::HasCommonAttributes(const FPCGMetadataDomain* InMetadata) const
{
	if (!InMetadata)
	{
		return false;
	}

	bool bHasCommonAttribute = false;

	{
		PCG::TSharedScopeLock ReadLock(AttributeLock);
		for (const TPair<FName, FPCGMetadataAttributeBase*>& AttributePair : Attributes)
		{
			if (InMetadata->HasAttribute(AttributePair.Key))
			{
				bHasCommonAttribute = true;
				break;
			}
		}
	}

	return bHasCommonAttribute;
}

int32 FPCGMetadataDomain::GetAttributeCount() const
{
	PCG::TSharedScopeLock ReadLock(AttributeLock);
	return Attributes.Num();
}

void FPCGMetadataDomain::GetAttributes(TArray<FName>& AttributeNames, TArray<EPCGMetadataTypes>& AttributeTypes) const
{
	PCG::TSharedScopeLock ReadLock(AttributeLock);
	AttributeNames.Reserve(AttributeNames.Num() + Attributes.Num());
	AttributeTypes.Reserve(AttributeTypes.Num() + Attributes.Num());
	for (const TPair<FName, FPCGMetadataAttributeBase*>& Attribute : Attributes)
	{
		check(Attribute.Value && Attribute.Value->Name == Attribute.Key);
		AttributeNames.Add(Attribute.Key);

		// @todo_pcg: To be revisited with descriptors.
		if (Attribute.Value->GetTypeId() < static_cast<uint16>(EPCGMetadataTypes::Unknown))
		{
			AttributeTypes.Add(static_cast<EPCGMetadataTypes>(Attribute.Value->GetTypeId()));
		}
		else
		{
			AttributeTypes.Add(EPCGMetadataTypes::Unknown);
		}
	}
}

FName FPCGMetadataDomain::GetLatestAttributeNameOrNone() const
{
	FName LatestAttributeName = NAME_None;
	int64 MaxAttributeId = -1;
	
	{
		PCG::TSharedScopeLock ReadLock(AttributeLock);
		for (const TPair<FName, FPCGMetadataAttributeBase*>& It : Attributes)
		{
			if (It.Value && (It.Value->AttributeId > MaxAttributeId))
			{
				MaxAttributeId = It.Value->AttributeId;
				LatestAttributeName = It.Key;
			}
		}
	}

	return LatestAttributeName;
}

bool FPCGMetadataDomain::ParentHasAttribute(FName AttributeName) const
{
	return Parent && Parent->HasAttribute(AttributeName);
}

bool FPCGMetadataDomain::CreateAttributeFromProperty(FName AttributeName, const UObject* Object, const FProperty* InProperty)
{
	return PCGMetadata::CreateAttributeFromPropertyHelper<UObject>(this, AttributeName, Object, InProperty);
}

bool FPCGMetadataDomain::CreateAttributeFromDataProperty(FName AttributeName, const void* Data, const FProperty* InProperty)
{
	return PCGMetadata::CreateAttributeFromPropertyHelper<void>(this, AttributeName, Data, InProperty);
}

bool FPCGMetadataDomain::SetAttributeFromProperty(FName AttributeName, PCGMetadataEntryKey& EntryKey, const UObject* Object, const FProperty* InProperty, bool bCreate)
{
	return PCGMetadata::SetAttributeFromPropertyHelper<UObject>(this, AttributeName, EntryKey, Object, InProperty, bCreate);
}

bool FPCGMetadataDomain::SetAttributeFromDataProperty(FName AttributeName, PCGMetadataEntryKey& EntryKey, const void* Data, const FProperty* InProperty, bool bCreate)
{
	return PCGMetadata::SetAttributeFromPropertyHelper<void>(this, AttributeName, EntryKey, Data, InProperty, bCreate);
}

bool FPCGMetadataDomain::CopyExistingAttribute(FName AttributeToCopy, FName NewAttributeName, bool bKeepParent)
{
	return CopyAttribute(AttributeToCopy, NewAttributeName, bKeepParent, /*bCopyEntries=*/true, /*bCopyValues=*/true) != nullptr;
}

FPCGMetadataAttributeBase* FPCGMetadataDomain::CopyAttribute(FName AttributeToCopy, FName NewAttributeName, bool bKeepParent, bool bCopyEntries, bool bCopyValues)
{
	const FPCGMetadataAttributeBase* OriginalAttribute = nullptr;

	{
		PCG::TSharedScopeLock ReadLock(AttributeLock);
		if (FPCGMetadataAttributeBase** AttributeFound = Attributes.Find(AttributeToCopy))
		{
			OriginalAttribute = *AttributeFound;
		}
	}

	if (!OriginalAttribute && Parent)
	{
		OriginalAttribute = Parent->GetConstAttribute(AttributeToCopy);
	}

	if (!OriginalAttribute)
	{
		UE_LOGF(LogPCG, Warning, "Attribute %ls does not exist, therefore cannot be copied", *AttributeToCopy.ToString());
		return nullptr;
	}

	return CopyAttribute(OriginalAttribute, NewAttributeName, bKeepParent, bCopyEntries, bCopyValues);
}

FPCGMetadataAttributeBase* FPCGMetadataDomain::CopyAttribute(const FPCGMetadataAttributeBase* OriginalAttribute, FName NewAttributeName, bool bKeepParent, bool bCopyEntries, bool bCopyValues)
{
	check(OriginalAttribute);
	checkSlow(OriginalAttribute->GetMetadataDomain()->GetRoot() == GetRoot() || !bKeepParent);
	FPCGMetadataAttributeBase* NewAttribute = OriginalAttribute->Copy(NewAttributeName, this, bKeepParent, bCopyEntries, bCopyValues);

	if (NewAttribute)
	{
		PCG::TUniqueScopeLock WriteLock(AttributeLock);
		NewAttribute->AttributeId = NextAttributeId++;
		AddAttributeInternal(NewAttributeName, NewAttribute);

		// Also when creating an attribute, notify the PCG Data owner that the latest attribute manipulated is this one.
		SetLastCachedSelectorOnOwner(NewAttributeName);
	}

	return NewAttribute;
}

bool FPCGMetadataDomain::RenameAttribute(FName AttributeToRename, FName NewAttributeName)
{
	if (!FPCGMetadataAttributeBase::IsValidName(NewAttributeName))
	{
		UE_LOGF(LogPCG, Error, "New attribute name %ls is not valid", *NewAttributeName.ToString());
		return false;
	}

	bool bRenamed = false;
	{
		PCG::TUniqueScopeLock WriteLock(AttributeLock);
		if (FPCGMetadataAttributeBase** AttributeFound = Attributes.Find(AttributeToRename))
		{
			FPCGMetadataAttributeBase* Attribute = *AttributeFound;
			RemoveAttributeInternal(AttributeToRename);
			Attribute->Name = NewAttributeName;
			AddAttributeInternal(NewAttributeName, Attribute);

			// Also when renaming an attribute, notify the PCG Data owner that the latest attribute manipulated is this one.
			SetLastCachedSelectorOnOwner(NewAttributeName);

			bRenamed = true;
		}
	}

	if (!bRenamed)
	{
		UE_LOGF(LogPCG, Warning, "Attribute %ls does not exist and therefore cannot be renamed", *AttributeToRename.ToString());
	}

	return bRenamed;
}

void FPCGMetadataDomain::ClearAttribute(FName AttributeToClear)
{
	FPCGMetadataAttributeBase* Attribute = nullptr;

	{
		PCG::TSharedScopeLock ReadLock(AttributeLock);
		if (FPCGMetadataAttributeBase** AttributeFound = Attributes.Find(AttributeToClear))
		{
			Attribute = *AttributeFound;
		}
	}

	// If the attribute exists, then we can lose all the entries
	// If it doesn't but it exists in the parent hierarchy, then we must create a new attribute.
	if (Attribute)
	{
		Attribute->ClearEntries();
	}
}

void FPCGMetadataDomain::DeleteAttribute(FName AttributeToDelete)
{
	FPCGMetadataAttributeBase* Attribute = nullptr;

	// If it's a local attribute, then just delete it
	{
		PCG::TUniqueScopeLock WriteLock(AttributeLock);
		if (FPCGMetadataAttributeBase** AttributeFound = Attributes.Find(AttributeToDelete))
		{
			Attribute = *AttributeFound;
			RemoveAttributeInternal(AttributeToDelete);
		}
	}

	if (Attribute)
	{
		delete Attribute;
	}
	else
	{
		UE_LOGF(LogPCG, Verbose, "Attribute %ls does not exist and therefore cannot be deleted", *AttributeToDelete.ToString());
	}
}

bool FPCGMetadataDomain::ChangeAttributeType(FName AttributeName, int16 AttributeNewType)
{
	FPCGMetadataAttributeDesc NewDesc{};
	NewDesc.Name = AttributeName;
	NewDesc.ValueType = static_cast<EPCGMetadataTypes>(AttributeNewType);
	
	return ChangeAttributeType(AttributeName, NewDesc);
}

bool FPCGMetadataDomain::ChangeAttributeType(FName AttributeName, const FPCGMetadataAttributeDesc& AttributeNewDesc)
{
	FPCGMetadataAttributeBase* Attribute = GetMutableAttribute(AttributeName);

	if (!Attribute)
	{
		UE_LOGF(LogPCG, Error, "Attribute '%ls' does not exist and therefore cannot change its type", *AttributeName.ToString());
		return false;
	}

	if (Attribute->GetAttributeDesc().IsSameType(AttributeNewDesc))
	{
		// Nothing to do, attribute is already the type we want
		return true;
	}
	
	if (!Attribute->GetAttributeDesc().IsSingleValue() || !AttributeNewDesc.IsSingleValue())
	{
		// Not possible at the moment to transform not single value attributes.
		return false;
	}

	if (FPCGMetadataAttributeBase* NewAttribute = Attribute->CopyToAnotherType(static_cast<int16>(AttributeNewDesc.ValueType)))
	{
		NewAttribute->AttributeId = Attribute->AttributeId;

		{
			PCG::TUniqueScopeLock WriteLock(AttributeLock);
			RemoveAttributeInternal(AttributeName);
			AddAttributeInternal(AttributeName, NewAttribute);
		}

		delete Attribute;
		Attribute = nullptr;
	}

	return true;
}

int64 FPCGMetadataDomain::GetItemCountForChild() const
{
	PCG::TSharedScopeLock ReadLock(ItemLock);
	return ParentKeys.Num() + ItemKeyOffset;
}

int64 FPCGMetadataDomain::GetLocalItemCount() const
{
	PCG::TSharedScopeLock ReadLock(ItemLock);
	return ParentKeys.Num();
}

int64 FPCGMetadataDomain::AddEntry(int64 ParentEntry)
{
	PCG::TUniqueScopeLock WriteLock(ItemLock);
	if (bSupportMultiEntries)
	{
		return ParentKeys.Add(ParentEntry) + ItemKeyOffset;
	}
	else
	{
		if (ParentKeys.IsEmpty() && ItemKeyOffset == 0)
		{
			ParentKeys.Add(ParentEntry);
		}
		else
		{
			UE_LOGF(LogPCG, Warning, "Try to add an entry to a domain (%ls) that doesn't support multi entries. Will always return 0.", *DomainID.DebugName.ToString());
		}
		
		return 0;
	}
}

TArray<int64> FPCGMetadataDomain::AddEntries(TArrayView<const int64> ParentEntryKeys)
{
	if (ParentEntryKeys.IsEmpty())
	{
		return {};
	}
	
	TArray<int64> Result;
	Result.Reserve(ParentEntryKeys.Num());

	PCG::TUniqueScopeLock WriteLock(ItemLock);
	if (bSupportMultiEntries)
	{
		ParentKeys.Reserve(ParentKeys.Num() + ParentEntryKeys.Num());
		for (const int64 ParentEntry : ParentEntryKeys)
		{
			Result.Add(ParentKeys.Add(ParentEntry) + ItemKeyOffset);
		}
	}
	else
	{
		if (ParentEntryKeys.Num() > 1 || !ParentKeys.IsEmpty() || ItemKeyOffset != 0)
		{
			UE_LOGF(LogPCG, Warning, "Try to add multiple entries to a metadata domain that don't support it (%ls). Will always return 0.", *DomainID.DebugName.ToString());
		}
		
		if (ParentKeys.IsEmpty() && ItemKeyOffset == 0)
		{
			ParentKeys.Add(ParentEntryKeys[0]);
		}

		// The function expect to return the same number of keys, so return an array of 0;
		Result.SetNumZeroed(ParentEntryKeys.Num());
	}
	
	return Result;
}

void FPCGMetadataDomain::AddEntriesInPlace(TArrayView<int64*> ParentEntryKeys)
{
	if (ParentEntryKeys.IsEmpty())
	{
		return;
	}
	
	PCG::TUniqueScopeLock WriteLock(ItemLock);
	if (bSupportMultiEntries)
	{
		ParentKeys.Reserve(ParentKeys.Num() + ParentEntryKeys.Num());
		for (int64* ParentEntry : ParentEntryKeys)
		{
			*ParentEntry = ParentKeys.Add(*ParentEntry) + ItemKeyOffset;
		}
	}
	else
	{
		if (ParentEntryKeys.Num() > 1 || !ParentKeys.IsEmpty() || ItemKeyOffset != 0)
		{
			UE_LOGF(LogPCG, Warning, "Try to add multiple entries to a metadata domain that don't support it (%ls). Will always return 0.", *DomainID.DebugName.ToString());
		}
		
		if (ParentKeys.IsEmpty() && ItemKeyOffset == 0)
		{
			ParentKeys.Add(*ParentEntryKeys[0]);
		}

		// The function expect to return the same number of keys, so return an array of 0
		for (int64* ParentEntry : ParentEntryKeys)
		{
			*ParentEntry = 0;
		}
	}
}

int64 FPCGMetadataDomain::AddEntryPlaceholder()
{
	PCG::TSharedScopeLock ReadLock(ItemLock);
	if (bSupportMultiEntries)
	{
		return ParentKeys.Num() + DelayedEntriesIndex.IncrementExchange() + ItemKeyOffset;
	}
	else
	{
		const int64 DelayedEntry = DelayedEntriesIndex.IncrementExchange();
		if (DelayedEntry != 0 || !ParentKeys.IsEmpty() || ItemKeyOffset != 0)
		{
			UE_LOGF(LogPCG, Warning, "Try to add an entry to a domain (%ls) that doesn't support multi entries. Will always return 0.", *DomainID.DebugName.ToString());
		}
		
		return 0;
	}
}

void FPCGMetadataDomain::AddDelayedEntries(const TArray<TTuple<int64, int64>>& AllEntries)
{
	if (AllEntries.IsEmpty())
	{
		return;
	}
	
	PCG::TUniqueScopeLock WriteLock(ItemLock);
		
	if (bSupportMultiEntries)
	{
		ParentKeys.AddUninitialized(AllEntries.Num());
		for (const TTuple<int64, int64>& Entry : AllEntries)
		{
			int64 Index = Entry.Get<0>() - ItemKeyOffset;
			check(Index < ParentKeys.Num());
			ParentKeys[Index] = Entry.Get<1>();
		}
	}
	else
	{
		if (AllEntries.Num() > 1 || !ParentKeys.IsEmpty() || ItemKeyOffset != 0)
		{
			UE_LOGF(LogPCG, Warning, "Try to add multiple entries to a metadata domain that don't support it (%ls). Will always return 0.", *DomainID.DebugName.ToString());
		}

		if (ParentKeys.IsEmpty() && ItemKeyOffset == 0)
		{
			ParentKeys.Add(AllEntries[0].Get<1>());
		}
	}

	DelayedEntriesIndex.Exchange(0);
}

bool FPCGMetadataDomain::InitializeOnSet(PCGMetadataEntryKey& InOutKey, PCGMetadataEntryKey InParentKeyA, const FPCGMetadataDomain* InParentMetadataA, PCGMetadataEntryKey InParentKeyB, const FPCGMetadataDomain* InParentMetadataB)
{
	if (InOutKey == PCGInvalidEntryKey)
	{
		if (InParentKeyA != PCGInvalidEntryKey && Parent == InParentMetadataA)
		{
			InOutKey = AddEntry(InParentKeyA);
			return true;
		}
		else if (InParentKeyB != PCGInvalidEntryKey && Parent == InParentMetadataB)
		{
			InOutKey = AddEntry(InParentKeyB);
			return true;
		}
		else
		{
			InOutKey = AddEntry();
			return false;
		}
	}
	else if(InOutKey < ItemKeyOffset)
	{
		InOutKey = AddEntry(InOutKey);
		return false;
	}
	else
	{
		return false;
	}
}

PCGMetadataEntryKey FPCGMetadataDomain::GetParentKey(PCGMetadataEntryKey LocalItemKey) const
{
	if (LocalItemKey < ItemKeyOffset)
	{
		// Key is already in parent referential
		return LocalItemKey;
	}
	else
	{
		PCG::TSharedScopeLock ReadLock(ItemLock);
		if (LocalItemKey - ItemKeyOffset < ParentKeys.Num())
		{
			return ParentKeys[LocalItemKey - ItemKeyOffset];
		}
		else
		{
			UE_LOGF(LogPCG, Warning, "Invalid metadata key - check for entry key not properly initialized");
			return PCGInvalidEntryKey;
		}
	}
}

void FPCGMetadataDomain::GetParentKeys(TArrayView<PCGMetadataEntryKey> LocalItemKeys, const TBitArray<>* Mask) const
{
	GetParentKeysWithRange(PCGValueRangeHelpers::MakeValueRange(LocalItemKeys), Mask);
}

void FPCGMetadataDomain::GetParentKeysWithRange(TPCGValueRange<PCGMetadataEntryKey> LocalItemKeys, const TBitArray<>* Mask) const
{
	auto GetParentKey_Unsafe = [this](PCGMetadataEntryKey& LocalItemKey) -> void
	{
		if (LocalItemKey < ItemKeyOffset)
		{
			// Key is already in parent referential
			return;
		}
		else if (LocalItemKey - ItemKeyOffset < ParentKeys.Num())
		{
			LocalItemKey = ParentKeys[LocalItemKey - ItemKeyOffset];
		}
		else
		{
			UE_LOGF(LogPCG, Warning, "Invalid metadata key - check for entry key not properly initialized");
			LocalItemKey = PCGInvalidEntryKey;
		}
	};

	PCG::TSharedScopeLock ReadLock(ItemLock);
	if (Mask && ensure(LocalItemKeys.Num() == Mask->Num()))
	{
		for (TConstSetBitIterator<> It(*Mask); It; ++It)
		{
			GetParentKey_Unsafe(LocalItemKeys[It.GetIndex()]);
		}
	}
	else
	{
		for (PCGMetadataEntryKey& LocalItemKey : LocalItemKeys)
		{
			GetParentKey_Unsafe(LocalItemKey);
		}
	}
}

void FPCGMetadataDomain::MergeAttributes(PCGMetadataEntryKey InKeyA, const FPCGMetadataDomain* InMetadataA, PCGMetadataEntryKey InKeyB, const FPCGMetadataDomain* InMetadataB, PCGMetadataEntryKey& OutKey, EPCGMetadataOp Op)
{
	MergeAttributesSubset(InKeyA, InMetadataA, InMetadataA, InKeyB, InMetadataB, InMetadataB, OutKey, Op);
}

void FPCGMetadataDomain::MergeAttributesSubset(PCGMetadataEntryKey InKeyA, const FPCGMetadataDomain* InMetadataA, const FPCGMetadataDomain* InMetadataSubsetA, PCGMetadataEntryKey InKeyB, const FPCGMetadataDomain* InMetadataB, const FPCGMetadataDomain* InMetadataSubsetB, PCGMetadataEntryKey& OutKey, EPCGMetadataOp Op)
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMetadataDomain::MergeAttributesSubset);
	// Early out: nothing to do if both input metadata are null / points have no assigned metadata
	if (!InMetadataA && !InMetadataB)
	{
		return;
	}

	// For each attribute in the current metadata, query the values from point A & B, apply operation on the result and finally store in the out point.
	InitializeOnSet(OutKey, InKeyA, InMetadataA, InKeyB, InMetadataB);

	PCG::TSharedScopeLock ReadLock(AttributeLock);
	for(const TPair<FName, FPCGMetadataAttributeBase*>& AttributePair : Attributes)
	{
		const FName& AttributeName = AttributePair.Key;
		FPCGMetadataAttributeBase* Attribute = AttributePair.Value;

		// Get attribute from A
		const FPCGMetadataAttributeBase* AttributeA = nullptr;
		if (InMetadataA && ((InMetadataA == InMetadataSubsetA) || (InMetadataSubsetA && InMetadataSubsetA->HasAttribute(AttributeName))))
		{
			AttributeA = InMetadataA->GetConstAttribute(AttributeName);
		}

		if (AttributeA && !AttributeA->GetAttributeDesc().IsSameType(Attribute->GetAttributeDesc()))
		{
			UE_LOGF(LogPCG, Error, "Metadata type mismatch with attribute %ls", *AttributeName.ToString());
			AttributeA = nullptr;
		}

		// Get attribute from B
		const FPCGMetadataAttributeBase* AttributeB = nullptr;
		if (InMetadataB && ((InMetadataB == InMetadataSubsetB) || (InMetadataSubsetB && InMetadataSubsetB->HasAttribute(AttributeName))))
		{
			AttributeB = InMetadataB->GetConstAttribute(AttributeName);
		}

		if (AttributeB && !AttributeB->GetAttributeDesc().IsSameType(Attribute->GetAttributeDesc()))
		{
			UE_LOGF(LogPCG, Error, "Metadata type mismatch with attribute %ls", *AttributeName.ToString());
			AttributeB = nullptr;
		}

		if (AttributeA || AttributeB)
		{
			Attribute->SetValue(OutKey, AttributeA, InKeyA, AttributeB, InKeyB, Op);
		}
	}
}

void FPCGMetadataDomain::ResetWeightedAttributes(PCGMetadataEntryKey& OutKey)
{
	InitializeOnSet(OutKey);

	PCG::TSharedScopeLock ReadLock(AttributeLock);
	for(const TPair<FName, FPCGMetadataAttributeBase*>& AttributePair : Attributes)
	{
		FPCGMetadataAttributeBase* Attribute = AttributePair.Value;
		
		if (Attribute && Attribute->AllowsInterpolation())
		{
			Attribute->SetZeroValue(OutKey);
		}
	}
}

void FPCGMetadataDomain::AccumulateWeightedAttributes(PCGMetadataEntryKey InKey, const FPCGMetadataDomain* InMetadata, float Weight, bool bSetNonInterpolableAttributes, PCGMetadataEntryKey& OutKey)
{
	if (!InMetadata)
	{
		return;
	}

	bool bHasSetParent = InitializeOnSet(OutKey, InKey, InMetadata);

	const bool bShouldSetNonInterpolableAttributes = bSetNonInterpolableAttributes && !bHasSetParent;

	PCG::TSharedScopeLock ReadLock(AttributeLock);
	for(const TPair<FName, FPCGMetadataAttributeBase*>& AttributePair : Attributes)
	{
		const FName& AttributeName = AttributePair.Key;
		FPCGMetadataAttributeBase* Attribute = AttributePair.Value;

		if (const FPCGMetadataAttributeBase* OtherAttribute = InMetadata->GetConstAttribute(AttributeName))
		{
			if (!OtherAttribute->GetAttributeDesc().IsSameType(Attribute->GetAttributeDesc()))
			{
				UE_LOGF(LogPCG, Error, "Metadata type mismatch with attribute %ls", *AttributeName.ToString());
				continue;
			}

			if (Attribute->AllowsInterpolation())
			{
				Attribute->AccumulateValue(OutKey, OtherAttribute, InKey, Weight);
			}
			else if (bShouldSetNonInterpolableAttributes)
			{
				Attribute->SetValue(OutKey, OtherAttribute, InKey);
			}
		}
	}
}

void FPCGMetadataDomain::ComputeWeightedAttribute(PCGMetadataEntryKey& OutKey, const TArrayView<TPair<PCGMetadataEntryKey, float>>& InWeightedKeys, const FPCGMetadataDomain* InMetadata)
{
	if (!InMetadata || InWeightedKeys.IsEmpty())
	{
		return;
	}

	// Could ensure that InitializeOnSet returns false...
	PCG::TSharedScopeLock ReadLock(AttributeLock);
	for (const TPair<FName, FPCGMetadataAttributeBase*>& AttributePair : Attributes)
	{
		const FName& AttributeName = AttributePair.Key;
		FPCGMetadataAttributeBase* Attribute = AttributePair.Value;

		if (!Attribute->AllowsInterpolation())
		{
			continue;
		}

		if (const FPCGMetadataAttributeBase* OtherAttribute = InMetadata->GetConstAttribute(AttributeName))
		{
			if (!OtherAttribute->GetAttributeDesc().IsSameType(Attribute->GetAttributeDesc()))
			{
				UE_LOGF(LogPCG, Error, "Metadata type mismatch with attribute %ls", *AttributeName.ToString());
				continue;
			}

			Attribute->SetWeightedValue(OutKey, OtherAttribute, InWeightedKeys);
		}
	}
}

int64 FPCGMetadataDomain::GetItemKeyCountForParent() const
{
	return ItemKeyOffset;
}

void FPCGMetadataDomain::SetAttributes(PCGMetadataEntryKey InKey, const FPCGMetadataDomain* InMetadata, PCGMetadataEntryKey& OutKey)
{
	if (!InMetadata)
	{
		return;
	}

	if (InitializeOnSet(OutKey, InKey, InMetadata))
	{
		// Early out; we don't need to do anything else at this point
		return;
	}

	PCG::TSharedScopeLock ReadLock(AttributeLock);
	for(const TPair<FName, FPCGMetadataAttributeBase*>& AttributePair : Attributes)
	{
		const FName& AttributeName = AttributePair.Key;
		FPCGMetadataAttributeBase* Attribute = AttributePair.Value;

		if (const FPCGMetadataAttributeBase* OtherAttribute = InMetadata->GetConstAttribute(AttributeName))
		{
			if (!OtherAttribute->GetAttributeDesc().IsSameType(Attribute->GetAttributeDesc()))
			{
				UE_LOGF(LogPCG, Error, "Metadata type mismatch with attribute %ls", *AttributeName.ToString());
				continue;
			}

			Attribute->SetValue(OutKey, OtherAttribute, InKey);
		}
	}
}

void FPCGMetadataDomain::SetAttributes(const TArrayView<const PCGMetadataEntryKey>& InOriginalKeys, const FPCGMetadataDomain* InMetadata, const TArrayView<PCGMetadataEntryKey>* InOutOptionalKeys, FPCGContext* OptionalContext)
{
	if (!InMetadata || InMetadata->GetAttributeCount() == 0 || GetAttributeCount() == 0 || InOriginalKeys.IsEmpty())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSubMetadata::SetAttributes);

	check(!InOutOptionalKeys || (InOriginalKeys.Num() == InOutOptionalKeys->Num()));

	// There are a few things we can do to optimize here -
	// basically, we don't need to set attributes more than once for a given <in, out> pair
	TArray<PCGMetadataEntryKey, TInlineAllocator<256>> InKeys;
	TArray<PCGMetadataEntryKey, TInlineAllocator<256>> OutKeys;

	if (InOutOptionalKeys)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSubMetadata::SetAttributes::CreateDeduplicatedKeys);
		TMap<TPair<PCGMetadataEntryKey, PCGMetadataEntryKey>, int> PairMapping;

		for (int KeyIndex = 0; KeyIndex < InOriginalKeys.Num(); ++KeyIndex)
		{
			PCGMetadataEntryKey InKey = InOriginalKeys[KeyIndex];
			PCGMetadataEntryKey& OutKey = (*InOutOptionalKeys)[KeyIndex];

			if (int* MatchingPairIndex = PairMapping.Find(TPair<PCGMetadataEntryKey, PCGMetadataEntryKey>(InKey, OutKey)))
			{
				OutKey = *MatchingPairIndex;
			}
			else
			{
				int NewIndex = InKeys.Add(InKey);

				PairMapping.Emplace(TPair<PCGMetadataEntryKey, PCGMetadataEntryKey>(InKey, OutKey), NewIndex);
				OutKeys.Add(OutKey);
				OutKey = NewIndex;
			}
		}
	}
	else
	{
		InKeys.Append(InOriginalKeys);
		OutKeys.Init(PCGInvalidEntryKey, InOriginalKeys.Num());
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSubMetadata::SetAttributes::InitializeOnSet);

		for (int32 KeyIndex = 0; KeyIndex < InKeys.Num(); ++KeyIndex)
		{
			InitializeOnSet(OutKeys[KeyIndex], InKeys[KeyIndex], InMetadata);
		}
	}

	{
		PCG::TSharedScopeLock ReadLock(AttributeLock);
		int32 AttributeOffset = 0;
		const int32 DefaultAttributesPerDispatch = 64;
		int32 AttributesPerDispatch = OptionalContext ? DefaultAttributesPerDispatch : 1;
		if (OptionalContext && OptionalContext->AsyncState.NumAvailableTasks > 0)
		{
			AttributesPerDispatch = FMath::Min(OptionalContext->AsyncState.NumAvailableTasks, AttributesPerDispatch);
		}

		while (AttributeOffset < Attributes.Num())
		{
			TArray<FName> AttributeNames;
			TArray<EPCGMetadataTypes> AttributeTypes;
			GetAttributes(AttributeNames, AttributeTypes);

			const int32 AttributeCountInCurrentDispatch = FMath::Min(AttributesPerDispatch, Attributes.Num() - AttributeOffset);
			ParallelFor(AttributeCountInCurrentDispatch, [this, OptionalContext, AttributeOffset, InMetadata, &AttributeNames, &InKeys, &OutKeys](int32 WorkerIndex)
			{
				LLM_SCOPE_BYTAG(PCG);

				const FName AttributeName = AttributeNames[AttributeOffset + WorkerIndex];
				FPCGMetadataAttributeBase* Attribute = Attributes[AttributeName];

				if (const FPCGMetadataAttributeBase* OtherAttribute = InMetadata->GetConstAttribute(AttributeName))
				{
					if (!PCG::Private::IsBroadcastableOrConstructible(OtherAttribute->GetAttributeDesc(), Attribute->GetAttributeDesc()))
					{
						PCGE_LOG_C(Error, GraphAndLog, OptionalContext, FText::Format(NSLOCTEXT("PCGMetadata", "TypeMismatch", "Metadata type mismatch with attribute '{0}'"), FText::FromName(AttributeName)));
						return;
					}

					if (Attribute == OtherAttribute)
					{
						TArray<PCGMetadataValueKey> ValueKeys;
						Attribute->GetValueKeys(TArrayView<const PCGMetadataEntryKey>(InKeys), ValueKeys);
						Attribute->SetValuesFromValueKeys(OutKeys, ValueKeys);
					}
					else
					{
						// Create accessors
						TUniquePtr<const IPCGAttributeAccessor> OtherAttributeAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OtherAttribute, InMetadata);
						TUniquePtr<IPCGAttributeAccessor> ThisAttributeAccessor = PCGAttributeAccessorHelpers::CreateAccessor(Attribute, this);

						TConstArrayView<PCGMetadataEntryKey> InKeysView(InKeys);
						FPCGAttributeAccessorKeysEntries OtherAttributeKeys(InKeysView);
						
						TArrayView<PCGMetadataEntryKey> OutKeysView(OutKeys);
						FPCGAttributeAccessorKeysEntries ThisAttributeKeys(OutKeysView);

						if (!OtherAttributeAccessor || !ThisAttributeAccessor)
						{
							return;
						}
						
						OtherAttributeAccessor->CopyTo(OtherAttributeKeys, *ThisAttributeAccessor, ThisAttributeKeys, /*Index=*/0, /*Count=*/InKeysView.Num(), EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
					}
				}
			});

			AttributeOffset += AttributeCountInCurrentDispatch;
		}
	}

	if (InOutOptionalKeys)
	{
		// Finally, copy back the actual out keys to the original out keys
		for (PCGMetadataEntryKey& OutKey : *InOutOptionalKeys)
		{
			OutKey = OutKeys[OutKey];
		}
	}
}

void FPCGMetadataDomain::SetAttributes(const TArrayView<const PCGMetadataEntryKey>& InKeys, const FPCGMetadataDomain* InMetadata, const TArrayView<PCGMetadataEntryKey>& OutKeys, FPCGContext* OptionalContext)
{
	SetAttributes(InKeys, InMetadata, &OutKeys, OptionalContext);
}

FPCGMetadataAttributeBase* FPCGMetadataDomain::CreateAttribute(const FPCGMetadataAttributeDesc& AttributeDesc, bool bAllowsInterpolation,bool bOverrideParent)
{
	return CreateAttributeInternal(AttributeDesc, bAllowsInterpolation, bOverrideParent, [](FPCGMetadataAttributeBase*){});
}

FPCGMetadataAttributeBase* FPCGMetadataDomain::CreateAttributeInternal(const FPCGMetadataAttributeDesc& AttributeDesc, bool bAllowsInterpolation, bool bOverrideParent, TFunctionRef<void(FPCGMetadataAttributeBase*)> SetupDefaultValue, bool bAlreadyLocked)
{
	const FName AttributeName(AttributeDesc.Name);
	if (!FPCGMetadataAttributeBase::IsValidName(AttributeName))
	{
		UE_LOGF(LogPCG, Error, "Attribute name '%ls' is invalid", *AttributeName.ToString());
		return nullptr;
	}

	const FPCGMetadataAttributeBase* ParentAttribute = nullptr;

	if (bOverrideParent && Parent)
	{
		ParentAttribute = Parent->GetConstAttribute(AttributeName);
	}
	
	if (ParentAttribute && (!ParentAttribute->GetAttributeDesc().IsSameType(AttributeDesc)))
	{
		// Can't parent if the types do not match
		ParentAttribute = nullptr;
	}

	FPCGMetadataAttributeBase* NewAttribute = AllocateAttributeFromDesc(AttributeDesc, ParentAttribute);
	if (!NewAttribute)
	{
		return nullptr;
	}
	
	NewAttribute->bAllowsInterpolation = bAllowsInterpolation;
	SetupDefaultValue(NewAttribute);

	{
		PCG::TUniqueScopeLock WriteLock(AttributeLock, !bAlreadyLocked);

		if (FPCGMetadataAttributeBase** ExistingAttribute = Attributes.Find(AttributeName))
		{
			delete NewAttribute;
			if (!(*ExistingAttribute)->GetAttributeDesc().IsSameType(AttributeDesc))
			{
				UE_LOGF(LogPCG, Error, "Attribute %ls already exists but is not the right type. Abort.", *AttributeName.ToString());
				return nullptr;
			}
			else
			{
				UE_LOGF(LogPCG, Warning, "Attribute %ls already exists", *AttributeName.ToString());
				NewAttribute = *ExistingAttribute;
			}
		}
		else
		{
			NewAttribute->AttributeId = NextAttributeId++;
			AddAttributeInternal(AttributeName, NewAttribute);

			// Also when creating an attribute, notify the PCG Data owner that the latest attribute manipulated is this one.
			SetLastCachedSelectorOnOwner(AttributeName);
		}
	}

	return NewAttribute;
}

FPCGMetadataAttributeBase* FPCGMetadataDomain::FindOrCreateAttributeInternal(const FPCGMetadataAttributeDesc& AttributeDesc, bool bAllowsInterpolation, bool bOverrideParent, bool bOverwriteIfTypeMismatch, TFunctionRef<void(FPCGMetadataAttributeBase*)> SetupDefaultValue)
{
	const FName AttributeName = AttributeDesc.Name;
	
	{
		PCG::TSharedScopeLock ReadLock(AttributeLock);
		if (FPCGMetadataAttributeBase* Attribute = GetMutableAttribute_Unsafe(AttributeName); Attribute && Attribute->GetAttributeDesc().IsSameType(AttributeDesc))
		{
			return Attribute;
		}
	}

	PCG::TUniqueScopeLock WriteLock(AttributeLock);
	if (FPCGMetadataAttributeBase* Attribute = GetMutableAttribute_Unsafe(AttributeName); Attribute && Attribute->GetAttributeDesc().IsSameType(AttributeDesc))
	{
		return Attribute;
	}

	// If an attribute with this name exists here, there is a type mismatch.
	if (FPCGMetadataAttributeBase** FoundAttribute = Attributes.Find(AttributeName))
	{
		if (bOverwriteIfTypeMismatch)
		{
			delete *FoundAttribute;
			RemoveAttributeInternal(AttributeName);
		}
		else
		{
			return nullptr;
		}
	}

	// A new attribute will be created.
	return CreateAttributeInternal(AttributeDesc, bAllowsInterpolation, bOverrideParent, SetupDefaultValue, /*bAlreadyLocked=*/true);
}

const FPCGMetadataDomain* FPCGMetadataDomainHandle::GetConstDomain() const
{
	return DomainHandle.Pin().Get();
}

FPCGMetadataDomain* FPCGMetadataDomainHandle::GetMutableDomain()
{
	return !bConst ? DomainHandle.Pin().Get() : nullptr;
}

DEFINE_FUNCTION(UPCGMetadataDomainLibrary::execCreateAttributeFromValue)
{
	P_GET_STRUCT_REF(FPCGMetadataDomainHandle, DomainHandle)
	P_GET_PROPERTY(FNameProperty, Name)
	
	// Read wildcard value input.
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;
	Stack.StepCompiledIn<FProperty>(nullptr);
	
	const FProperty* TargetProperty = Stack.MostRecentProperty;
	void* TargetPtr = Stack.MostRecentPropertyContainer;
	
	P_GET_PROPERTY(FBoolProperty, bAllowsInterpolation)

	P_FINISH;

	P_NATIVE_BEGIN;
	
	*static_cast<bool*>(RESULT_PARAM) = false;
	
	FPCGMetadataDomain* Domain = DomainHandle.GetMutableDomain();
	if (!Domain)
	{
		UPCGBlueprintHelpers::ThrowBlueprintException(LOCTEXT("InvalidDomain", "Invalid domain or const."));
		return;
	}
	
	check(Domain->GetTopMetadata());
	UPCGData* OutData = Cast<UPCGData>(Domain->GetTopMetadata()->GetOuter());
	if (!OutData)
	{
		UPCGBlueprintHelpers::ThrowBlueprintException(LOCTEXT("InvalidData", "Outer of the metadata is not a valid data."));
		return;
	}
	
	if (Domain->HasAttribute(Name))
	{
		UPCGBlueprintHelpers::ThrowBlueprintException(LOCTEXT("ExistingAttribute", "Attribute already exists."));
		return;
	}
	
	// Try with old accessors first then new accessors
	TUniquePtr<const IPCGAttributeAccessor> PropertyAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(TargetProperty, /*bUseGenericAccessors=*/false);
	if (!PropertyAccessor)
	{
		PropertyAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(TargetProperty, /*bUseGenericAccessors=*/true);
	}

	if (!PropertyAccessor)
	{
		UPCGBlueprintHelpers::ThrowBlueprintException(LOCTEXT("InvalidValue", "Provided value is not supported by PCG Metadata."));
		return;
	}
	
	const void* PropertyPtrs[1] = {TargetPtr};
	FPCGAttributeAccessorKeysGenericPtrs PropertyKeys(PropertyPtrs);
	
	FPCGAttributePropertyInputSelector AttributeSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Name);
	OutData->SetDomainFromDomainID(Domain->GetDomainID(), AttributeSelector);

	PCGAttributeAccessorHelpers::FPCGCreateAccessorWithAttributeCreationParams Params =
	{
		.InData = OutData,
		.InSelector = &AttributeSelector,
		.InMatchingAccessor = PropertyAccessor.Get(),
		.InMatchingKeysForDefaultValue = &PropertyKeys
	};
	
	if (!PCGAttributeAccessorHelpers::CreateAccessorWithAttributeCreation(Params))
	{
		UPCGBlueprintHelpers::ThrowBlueprintException(FText::Format(LOCTEXT("FailToCreateAttribute", "Failed to create attribute {0} of type {1}"), FText::FromName(Name), FText::FromString(PropertyAccessor->GetUnderlyingDesc().GetTypeString())));
	}
	else
	{
		*static_cast<bool*>(RESULT_PARAM) = true;
	}

	P_NATIVE_END;
}

#undef LOCTEXT_NAMESPACE
