// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGMetadataPartitionCommon.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataCommon.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Algo/IndexOf.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "PCGMetadataPartitionCommon"

namespace PCGMetadataPartitionCommon
{
	using IndexPartition = TArray<TArray<int32>>;

	// @todo_pcg: Evaluate the bit array for removal as a contingency slow path
	template <typename T>
	constexpr bool IsBitArray = std::is_same_v<T, TBitArray<>>;

	// @todo_pcg: To be removed or replaced by internal procedural selection if/when the algorithm is validated
	static TAutoConsoleVariable<bool> CVarPCGMetadataPartitionMixedRadixHash(
		TEXT("pcg.MetadataPartition.UseMixedRadixHash"),
		true,
		TEXT("Enables the mixed radix hash method for partitioning attributes."));

	static TAutoConsoleVariable<bool> CVarPCGMetadataPartitionValidateOverflow(
		TEXT("pcg.MetadataPartition.ValidateOverflow"),
		true,
		TEXT("Validates that the composite key space does not overflow uint64 before using the mixed radix algorithm. "
			"Falls back to bitwise intersection on overflow. Disabling may cause silent partition collisions if the product "
			"of unique-value counts across all attributes exceeds uint64 (2^64)."));

	/**
	* Partition a given attribute, by first partitioning all value keys that point to the same value
	* and then for each unique value key, list of index in the keys that match for this value.
	*/
	template <typename PartitionType>
	TArray<PartitionType> AttributePartition(const FPCGMetadataAttributeBase* InAttribute, const IPCGAttributeAccessorKeys& InKeys, FPCGContext* InOptionalContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataPartitionCommon::AttributePartition);
		check(InAttribute);

		const int32 NumberOfEntries = InKeys.GetNum();

		if (NumberOfEntries <= 0)
		{
			return {};
		}

		TArray<const PCGMetadataEntryKey*, TInlineAllocator<256>> TempEntriesPtr;
		TempEntriesPtr.SetNum(NumberOfEntries);
		if (!InKeys.GetKeys(0, TArrayView(TempEntriesPtr.GetData(), NumberOfEntries)))
		{
			return {};
		}

		TArray<PCGMetadataEntryKey, TInlineAllocator<256>> TempEntries;
		TempEntries.Reserve(NumberOfEntries);
		Algo::Transform(TempEntriesPtr, TempEntries, [](const PCGMetadataEntryKey* EntryKey) { return *EntryKey; });

		TArray<PartitionType> PartitionedData = InAttribute->AttributePartition<PartitionType>(TempEntries);
		if (PartitionedData.IsEmpty())
		{
			return {};
		}

		// Since we partition on the value array, it is not guaranteed that the values appears in the same order than the entries.
		// So sort the final array using the first index as a sort criteria. Empty partitions will be at the beginning too.
		if constexpr (IsBitArray<PartitionType>)
		{
			Algo::Sort(PartitionedData, [](const TBitArray<>& LHS, const TBitArray<>& RHS) -> bool
			{
				const int32 FirstBitSetLHS = LHS.Find(true);
				if (FirstBitSetLHS == INDEX_NONE)
				{
					return true;
				}

				const int32 FirstBitSetRHS = RHS.Find(true);
				if (FirstBitSetRHS == INDEX_NONE)
				{
					return false;
				}

				return FirstBitSetLHS < FirstBitSetRHS;
			});
		}
		else
		{
			PartitionedData.Sort([](const PartitionType& LHS, const PartitionType& RHS) -> bool
			{
				if (LHS.IsEmpty())
				{
					return true;
				}
				else if (RHS.IsEmpty())
				{
					return false;
				}
				else
				{
					return LHS[0] < RHS[0];
				}
			});
		}

		return PartitionedData;
	}

	/**
	* Partition a given accessor that iterate on all values, find the identical ones,
	* and then for each unique value, list of index in the keys that match for this value.
	*/
	template <typename PartitionType, typename T>
	TArray<PartitionType> ValuePartition(const IPCGAttributeAccessor& InAccessor, const IPCGAttributeAccessorKeys& InKeys, FPCGContext* InOptionalContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataPartitionCommon::ValuePartition);
		TArray<T> UniqueValues;
		TArray<PartitionType> PartitionedData;

		PCGMetadataElementCommon::ApplyOnAccessor<T>(InKeys, InAccessor, [&PartitionedData, &UniqueValues, NumberOfEntries = InKeys.GetNum()](const T& InValue, int32 InIndex)
		{
			// TODO: Might want to upgrade to something better since it can be quadratic and grow quickly.
			int32 UniqueValueIndex = UniqueValues.IndexOfByPredicate([&InValue](const T& OtherValue)
			{
				// For consistency with the attribute part, use MetadataTraits::Equal
				return PCG::Private::MetadataTraits<T>::Equal(InValue, OtherValue);
			});

			if (UniqueValueIndex == INDEX_NONE)
			{
				UniqueValueIndex = UniqueValues.Add(InValue);
				PartitionType& Partition = PartitionedData.Emplace_GetRef();
				if constexpr (IsBitArray<PartitionType>)
				{
					Partition.Init(false, NumberOfEntries);
				}
			}

			if constexpr (IsBitArray<PartitionType>)
			{
				PartitionedData[UniqueValueIndex][InIndex] = true;
			}
			else
			{
				PartitionedData[UniqueValueIndex].Add(InIndex);
			}
		});

		return PartitionedData;
	}

	/**
	* Dispatch the partition according to the data and selector.
	*/
	template <typename PartitionType>
	TArray<PartitionType> AttributeGenericPartition(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataPartitionCommon::AttributeGenericPartition::SingleSelector);
		if (!InData)
		{
			return {};
		}

		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InData, InSelector);
		if (!Keys.IsValid())
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidKeys", "Could not create keys for the input data with selector {0}"), InSelector.GetDisplayText()), InOptionalContext);
			return {};
		}

		// Implementation note:
		// We'll use the attribute partition only for compressed types here (+ needs to be basic attribute only)
		// because otherwise we can run into issues where keeping track of the breadth of values is not great.
		bool bUseAttributePartition = false;
		const FPCGMetadataAttributeBase* Attribute = nullptr;

		if (InSelector.IsBasicAttribute())
		{
			const UPCGMetadata* Metadata = InData->ConstMetadata();
			if (!Metadata)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidMetadata", "Input data does not have metadata, while requesting an attribute {0}"), InSelector.GetDisplayText()), InOptionalContext);
				return {};
			}

			Attribute = Metadata->GetConstAttribute(InSelector.GetName());
			if (!Attribute)
			{
				if (!bSilenceMissingAttributeErrors)
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidAttribute", "Attribute {0} not found"), InSelector.GetDisplayText()), InOptionalContext);
				}

				return {};
			}

			bUseAttributePartition = Attribute->UsesValueKeys();
			if (!bUseAttributePartition)
			{
				// Check if the underlying type is supported for the ValuePartition, if so use it, otherwise use the normal attribute partition
				// that will do a pure equality check.
				if (!Attribute->GetAttributeDesc().IsSingleValue() || !PCGMetadataAttribute::CallbackWithRightType(static_cast<uint16>(Attribute->GetAttributeDesc().ValueType), [](auto) -> bool { return true; }))
				{
					bUseAttributePartition = true;
				}
			}
		}

		if (bUseAttributePartition)
		{
			check(Attribute);
			return AttributePartition<PartitionType>(Attribute, *Keys, InOptionalContext);
		}
		else
		{
			TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InData, InSelector);
			if (!Accessor.IsValid())
			{
				if (!bSilenceMissingAttributeErrors)
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidAccessor", "Attribute {0} not found"), InSelector.GetDisplayText()), InOptionalContext);
				}

				return {};
			}

			auto Operation = [&Accessor, &Keys, &InSelector, InOptionalContext](auto Dummy) -> TArray<PartitionType>
			{
				// Rotators don't have a hash, convert them to Quat
				using AttributeType = std::conditional_t<std::is_same_v<decltype(Dummy), FRotator>, FQuat, decltype(Dummy)>;

				// Can't partition on a transform.
				if constexpr (std::is_same_v<AttributeType, FTransform>)
				{
					PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidType", "Attribute {0} is a transform, partition on transforms is not supported"), InSelector.GetDisplayText()), InOptionalContext);
					return {};
				}
				else
				{
					return ValuePartition<PartitionType, AttributeType>(*Accessor, *Keys, InOptionalContext);
				}
			};

			return PCGMetadataAttribute::CallbackWithRightType(Accessor->GetUnderlyingType(), Operation);
		}
	}

	/**
	* Dispatch the partition according to the data and selector.
	*/
	TArray<TArray<int32>> AttributeGenericPartition(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		return AttributeGenericPartition<TArray<int32>>(InData, InSelector, InOptionalContext, bSilenceMissingAttributeErrors);
	}

	/** Combine per-attribute partitions into a final partition using mixed radix composite key encoding. */
	IndexPartition MixedRadixCompositeKeyPartition(const TArray<IndexPartition>& PerAttributePartitions, const int32 NumElements, const int32 NumAttributes)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataPartitionCommon::AttributeGenericPartition::MultiSelector::CompositeKeyGrouping);
		IndexPartition FinalPartition;

		TArray<int32> PartitionCounts;
		PartitionCounts.SetNum(NumAttributes);

		TArray<TArray<int32>> PointToPartitionIndex;
		PointToPartitionIndex.SetNum(NumAttributes);

		// Build reverse mapping (point -> partition index) and collect partition counts for composite key encoding
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataPartitionCommon::AttributeGenericPartition::MultiSelector::ReverseMapping);
			for (int32 AttributeIndex = 0; AttributeIndex < NumAttributes; ++AttributeIndex)
			{
				const IndexPartition& AttributePartition = PerAttributePartitions[AttributeIndex];
				PartitionCounts[AttributeIndex] = AttributePartition.Num();

				TArray<int32>& PointToPartitionMap = PointToPartitionIndex[AttributeIndex];
				// By default, this is set to -1, because 0 is a valid partition index.
				PointToPartitionMap.Init(INDEX_NONE, NumElements);

				for (int32 PartitionIndex = 0; PartitionIndex < AttributePartition.Num(); ++PartitionIndex)
				{
					for (const int32 PointIndex : AttributePartition[PartitionIndex])
					{
						PointToPartitionMap[PointIndex] = PartitionIndex;
					}
				}
			}
		}

		// Maps the composite key (see below) to the index in FinalPartition with one entry per unique attribute combination.
		TMap<uint64, int32> CompositeKeyToPartition;

		for (int32 PointIndex = 0; PointIndex < NumElements; ++PointIndex)
		{
			// https://en.wikipedia.org/wiki/Composite_key
			uint64 CompositeKey = 0;

			// Use mixed radix encoding to create unique keys, scaling multiplicatively on the partition counts.
			// https://en.wikipedia.org/wiki/Mixed_radix
			uint64 Multiplier = 1;
			for (int32 AttributeIndex = 0; AttributeIndex < NumAttributes; ++AttributeIndex)
			{
				CompositeKey += Multiplier * PointToPartitionIndex[AttributeIndex][PointIndex];
				Multiplier *= PartitionCounts[AttributeIndex];
			}

			// Look up or create the final partition for this key.
			const int32* FinalIndex = CompositeKeyToPartition.Find(CompositeKey);
			if (!FinalIndex)
			{
				FinalIndex = &CompositeKeyToPartition.Add(CompositeKey, FinalPartition.Num());
				FinalPartition.Emplace();
			}

			FinalPartition[*FinalIndex].Add(PointIndex);
		}

		return FinalPartition;
	}

	/** Combine per-attribute partitions into a final partition using bitwise intersection. */
	IndexPartition BitwiseIntersectionPartition(const TArray<IndexPartition>& PerAttributePartitions, int32 NumElements)
	{
		using BitPartition = TArray<TBitArray<>>;

		TArray<BitPartition> BitPartitions;
		const int32 NumAttributes = PerAttributePartitions.Num();
		BitPartitions.SetNum(NumAttributes);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataPartitionCommon::AttributeGenericPartition::MultiSelector::PartitionOnBitArray);
			for (int32 AttributeIndex = 0; AttributeIndex < NumAttributes; ++AttributeIndex)
			{
				BitPartition& Partition = BitPartitions[AttributeIndex];
				Partition.SetNum(PerAttributePartitions[AttributeIndex].Num());
				for (int32 PartitionIndex = 0; PartitionIndex < PerAttributePartitions[AttributeIndex].Num(); ++PartitionIndex)
				{
					Partition[PartitionIndex].Init(false, NumElements);
					for (const int32 ElementIndex : PerAttributePartitions[AttributeIndex][PartitionIndex])
					{
						Partition[PartitionIndex][ElementIndex] = true;
					}
				}
			}
		}

		BitPartition IterativePartition = BitPartitions[0]; // TODO: This can be optimized to filter down in pairs in parallel - O(log N) - instead of serial

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataPartitionCommon::AttributeGenericPartition::Intersection);
			// Intersect all the BitArray partitions
			for (int32 PartitionIndex = 1; PartitionIndex < BitPartitions.Num(); ++PartitionIndex)
			{
				BitPartition CurrentBitPartition = IterativePartition;
				BitPartition& NextBitPartition = BitPartitions[PartitionIndex];

				IterativePartition.Empty();

				for (const TBitArray<>& FirstBitArray : CurrentBitPartition)
				{
					for (const TBitArray<>& SecondBitArray : NextBitPartition)
					{
						TBitArray<> Result = TBitArray<>::BitwiseAND(FirstBitArray, SecondBitArray, EBitwiseOperatorFlags::MaxSize);
						// Only capture if non-empty. Discard empty BitArrays.
						if (TConstSetBitIterator(Result))
						{
							IterativePartition.Emplace(std::move(Result));
						}
					}
				}
			}
		}

		IndexPartition FinalPartition;

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataPartitionCommon::AttributeGenericPartition::MultiSelector::ConversionToIndices);
			// Convert back into indices
			for (const TBitArray<>& BitArray : IterativePartition)
			{
				TArray<int>& Indices = FinalPartition.Emplace_GetRef();
				for (TConstSetBitIterator It(BitArray); It; ++It)
				{
					Indices.Emplace(It.GetIndex());
				}
			}
		}

		return FinalPartition;
	}

	/**
	 * Partition on multiple attributes by first partitioning on the attributes independently. Then build a reverse
	 * mapping from each point to its per-attribute partition index. Finally, group points by their composite key
	 * (a mixed radix computation of partition indices) in a single pass. O(N*M).
	 *
	 * Implementer's Note: A previous slow path algorithm for bitwise intersection remains as a fallback in the event of
	 * an overflow (uint64) or if bypassed explicitly via CVar.
	 *
	 * Multi-Partition Example:
	 * Pt  A  B  C                         Partition on A->[0,1],[2,3,4]
	 *  0  a  a  a                         Partition on B->[0],[1,2],[3,4]
	 *  1  a  b  a                         Partition on C->[0,1],[2],[3,4]
	 *  2  b  b  b                         Composite keys: 0=(0,0,0), 1=(0,1,0), 2=(1,1,1), 3=(1,2,1), 4=(1,2,1)
	 *  3  b  c  c                         Final Partition (A&B&C)->[0],[1],[2],[3,4]
	 *  4  b  c  c
	 */
	TArray<TArray<int32>> AttributeGenericPartition(const UPCGData* InData, const TArrayView<const FPCGAttributePropertySelector>& InSelectorArrayView, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataPartitionCommon::AttributeGenericPartition::MultiSelector);

		// Small optimization to partition on a single attribute
		if (InSelectorArrayView.Num() == 1)
		{
			return AttributeGenericPartition<TArray<int32>>(InData, InSelectorArrayView[0], InOptionalContext, bSilenceMissingAttributeErrors);
		}

		if (!InData || InSelectorArrayView.IsEmpty() || !InData->ConstMetadata())
		{
			return {};
		}

		// Get the element count from the number of keys which should work for spatial points and attribute sets
		const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InData, InSelectorArrayView[0]);
		if (!Keys.IsValid())
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidKeys", "Could not create keys for the input data with selector {0}"), InSelectorArrayView[0].GetDisplayText()), InOptionalContext);
			return {};
		}

		const int32 NumElements = Keys->GetNum();
		const int32 NumAttributes = InSelectorArrayView.Num();

		TArray<IndexPartition> PerAttributePartitions;
		PerAttributePartitions.SetNum(NumAttributes);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataPartitionCommon::AttributeGenericPartition::MultiSelector::PerAttributePartition);

			// Partition per attribute
			for (int32 I = 0; I < NumAttributes; ++I)
			{
				PerAttributePartitions[I] = AttributeGenericPartition<TArray<int32>>(InData, InSelectorArrayView[I], InOptionalContext, bSilenceMissingAttributeErrors);
				// A partition could be empty if the selector is bad, which would result in a 0 value multiplier later.
				if (PerAttributePartitions[I].IsEmpty())
				{
					if (NumElements > 0 && !bSilenceMissingAttributeErrors)
					{
						PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("AttributeResultsInZeroPartitions", "Attribute resulted in zero partitions. Check selector {0}"), InSelectorArrayView[I].GetDisplayText()), InOptionalContext);
					}

					return {};
				}
			}
		}

		bool bUseMixedRadixHash = CVarPCGMetadataPartitionMixedRadixHash.GetValueOnAnyThread();

		if (CVarPCGMetadataPartitionValidateOverflow.GetValueOnAnyThread())
		{
			// Overflow check: the composite key space is the product of all partition counts. With uint64, this supports
			// a very large range, i.e. 6 attributes with 1000 unique values each or 20 attributes with up to 9 unique
			// values each. But, as a nuclear option, we can check and bail out early.
			// Validated with 1M points across 1000 unique partitions with no issues.
			uint64 Multiplier = 1;
			for (int32 AttributeIndex = 0; AttributeIndex < NumAttributes; ++AttributeIndex)
			{
				const int32 NumPartitions = PerAttributePartitions[AttributeIndex].Num();
				if (Multiplier > (std::numeric_limits<uint64>::max() / FMath::Max(NumPartitions, 1)))
				{
					PCGLog::LogErrorOnGraph(LOCTEXT("AttributePartitionOverflow", "Overflow in partitioning too many unique attribute partitions. Falling back to slower algorithm."), InOptionalContext);
					bUseMixedRadixHash = false;
					break;
				}

				Multiplier *= NumPartitions;
			}
		}

		// Select algorithm: mixed radix composite key (fast path) or bitwise intersection (slow path)
		if (bUseMixedRadixHash)
		{
			return MixedRadixCompositeKeyPartition(PerAttributePartitions, NumElements, NumAttributes);
		}
		else
		{
			return BitwiseIntersectionPartition(PerAttributePartitions, NumElements);
		}
	}

	/**
	* Do a partition on the given point data for the selector
	*/
	TArray<UPCGData*> AttributePointPartition(const UPCGBasePointData* InData, const TArrayView<const FPCGAttributePropertySelector>& InSelectorArrayView, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		TArray<TArray<int32>> Partition = AttributeGenericPartition(InData, InSelectorArrayView, InOptionalContext, bSilenceMissingAttributeErrors);
		if (Partition.IsEmpty())
		{
			return {};
		}

		TArray<UPCGData*> PartitionedData;
		PartitionedData.Reserve(Partition.Num());

		for (TArray<int32>& Indices : Partition)
		{
			if (Indices.IsEmpty())
			{
				continue;
			}

			UPCGBasePointData* CurrentPointData = FPCGContext::NewPointData_AnyThread(InOptionalContext);
			PartitionedData.Add(CurrentPointData);

			FPCGInitializeFromDataParams InitializeFromDataParams(InData);
			InitializeFromDataParams.bInheritSpatialData = false;

			CurrentPointData->InitializeFromDataWithParams(InitializeFromDataParams);

			UPCGBasePointData::SetPoints(InData, CurrentPointData, Indices, /*bCopyAll=*/false);
		}

		return PartitionedData;
	}

	UPCGData* RemoveDuplicatesPoint(const UPCGBasePointData* InData, const TArrayView<const FPCGAttributePropertySelector>& InSelectorArrayView, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		TArray<TArray<int32>> Partition = AttributeGenericPartition(InData, InSelectorArrayView, InOptionalContext, bSilenceMissingAttributeErrors);
		if (Partition.IsEmpty())
		{
			return nullptr;
		}

		UPCGBasePointData* OutputPointData = nullptr;
		TArray<int32> IndicesToCopy;
		IndicesToCopy.Reserve(Partition.Num());

		for (TArray<int32>& Indices : Partition)
		{
			if (Indices.IsEmpty())
			{
				continue;
			}

			IndicesToCopy.Add(Indices[0]);
		}

		if (IndicesToCopy.IsEmpty())
		{
			return nullptr;
		}

		OutputPointData = FPCGContext::NewPointData_AnyThread(InOptionalContext);

		FPCGInitializeFromDataParams InitializeFromDataParams(InData);
		InitializeFromDataParams.bInheritSpatialData = false;

		OutputPointData->InitializeFromDataWithParams(InitializeFromDataParams);
		OutputPointData->SetNumPoints(IndicesToCopy.Num());
		
		UPCGBasePointData::SetPoints(InData, OutputPointData, IndicesToCopy, /*bCopyAll=*/false);

		return OutputPointData;
	}

	TArray<UPCGData*> AttributeParamSpatialPartition(const UPCGData* InData, const TArrayView<const FPCGAttributePropertySelector>& InSelectorArray, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		if (!InData)
		{
			return {};
		}

		if (!InData->IsA<UPCGSpatialData>() && !InData->IsA<UPCGParamData>())
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidDataType", "Input data is not an attribute set nor a spatial data. Operation not supported."), InOptionalContext);
			return {};
		}

		const TArray<TArray<int32>> Partition = AttributeGenericPartition(InData, InSelectorArray, InOptionalContext, bSilenceMissingAttributeErrors);

		if (Partition.IsEmpty())
		{
			return {};
		}

		const UPCGSpatialData* InSpatialData = Cast<const UPCGSpatialData>(InData);

		TArray<UPCGData*> PartitionedData;
		PartitionedData.Reserve(Partition.Num());

		TArray<FName> AttributeNames;
		TArray<EPCGMetadataTypes> AttributeTypes;
		const UPCGMetadata* OriginalMetadata = InData->ConstMetadata();
		OriginalMetadata->GetAttributes(AttributeNames, AttributeTypes);

		for (const TArray<int32>& Indices : Partition)
		{
			if (Indices.IsEmpty())
			{
				continue;
			}

			UPCGMetadata* NewMetadata = nullptr;

			if (InSpatialData)
			{
				UPCGSpatialData* NewData = FPCGContext::NewObject_AnyThread<UPCGSpatialData>(InOptionalContext, GetTransientPackage(), InSpatialData->GetClass());
				NewData->InitializeFromData(InSpatialData);
				NewMetadata = NewData->Metadata;
				PartitionedData.Add(NewData);
			}
			else
			{
				UPCGParamData* NewData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InOptionalContext);
				PCGMetadataHelpers::InitializeMetadataWithDataDomainCopyAndElementsNoCopy(InData, NewData);
				NewMetadata = NewData->Metadata;
				PartitionedData.Add(NewData);
			}

			TArray<PCGMetadataEntryKey> EntryKeys;
			EntryKeys.Reserve(Indices.Num());
			for (int32 i = 0; i < Indices.Num(); ++i)
			{
				EntryKeys.Add(NewMetadata->AddEntry());
			}

			for (const FName AttributeName : AttributeNames)
			{
				const FPCGMetadataAttributeBase* OriginalAttribute = OriginalMetadata->GetConstAttribute(AttributeName);
				FPCGMetadataAttributeBase* NewAttribute = NewMetadata->GetMutableAttribute(AttributeName);
				check(OriginalAttribute && NewAttribute);

				for (int32 i = 0; i < Indices.Num(); ++i)
				{
					NewAttribute->SetValue(EntryKeys[i], OriginalAttribute, Indices[i]);
				}
			}
		}

		return PartitionedData;
	}

	UPCGData* RemoveDuplicatesParamSpatial(const UPCGData* InData, const TArrayView<const FPCGAttributePropertySelector>& InSelectorArray, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		if (!InData)
		{
			return nullptr;
		}

		if (!InData->IsA<UPCGSpatialData>() && !InData->IsA<UPCGParamData>())
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidDataType", "Input data is not an attribute set nor a spatial data. Operation not supported."), InOptionalContext);
			return nullptr;
		}

		const TArray<TArray<int32>> Partition = AttributeGenericPartition(InData, InSelectorArray, InOptionalContext, bSilenceMissingAttributeErrors);

		if (Partition.IsEmpty())
		{
			return nullptr;
		}

		const UPCGSpatialData* InSpatialData = Cast<const UPCGSpatialData>(InData);

		UPCGData* OutputData = nullptr;
		UPCGMetadata* OutMetadata = nullptr;

		TArray<FName> AttributeNames;
		TArray<EPCGMetadataTypes> AttributeTypes;
		const UPCGMetadata* OriginalMetadata = InData->ConstMetadata();
		OriginalMetadata->GetAttributes(AttributeNames, AttributeTypes);

		for (const TArray<int32>& Indices : Partition)
		{
			if (Indices.IsEmpty())
			{
				continue;
			}

			if (!OutputData)
			{
				if (InSpatialData)
				{
					UPCGSpatialData* NewData = FPCGContext::NewObject_AnyThread<UPCGSpatialData>(InOptionalContext, GetTransientPackage(), InSpatialData->GetClass());
					NewData->InitializeFromData(InSpatialData);
					OutputData = NewData;
					OutMetadata = NewData->Metadata;
				}
				else
				{
					UPCGParamData* NewData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InOptionalContext);
					PCGMetadataHelpers::InitializeMetadataWithDataDomainCopyAndElementsNoCopy(InData, NewData);
					OutputData = NewData;
					OutMetadata = NewData->Metadata;
				}
			}

			check(OutMetadata);

			const PCGMetadataEntryKey CurrentEntryKey = OutMetadata->AddEntry();

			for (const FName AttributeName : AttributeNames)
			{
				const FPCGMetadataAttributeBase* OriginalAttribute = OriginalMetadata->GetConstAttribute(AttributeName);
				FPCGMetadataAttributeBase* NewAttribute = OutMetadata->GetMutableAttribute(AttributeName);
				check(OriginalAttribute && NewAttribute);

				NewAttribute->SetValue(CurrentEntryKey, OriginalAttribute, Indices[0]);
			}
		}

		return OutputData;
	}

	TArray<UPCGData*> AttributePartition(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		return AttributePartition(InData, TArrayView<const FPCGAttributePropertySelector>(&InSelector, 1), InOptionalContext, bSilenceMissingAttributeErrors);
	}

	TArray<UPCGData*> AttributePartition(const UPCGData* InData, const TArrayView<const FPCGAttributePropertySelector>& InSelectorArrayView, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		if (const UPCGBasePointData* InPointData = Cast<UPCGBasePointData>(InData))
		{
			return AttributePointPartition(InPointData, InSelectorArrayView, InOptionalContext, bSilenceMissingAttributeErrors);
		}
		else
		{
			return AttributeParamSpatialPartition(InData, InSelectorArrayView, InOptionalContext, bSilenceMissingAttributeErrors);
		}
	}

	UPCGData* RemoveDuplicates(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		return RemoveDuplicates(InData, TArrayView<const FPCGAttributePropertySelector>(&InSelector, 1), InOptionalContext, bSilenceMissingAttributeErrors);
	}

	UPCGData* RemoveDuplicates(const UPCGData* InData, const TArrayView<const FPCGAttributePropertySelector>& InSelectorArrayView, FPCGContext* InOptionalContext, bool bSilenceMissingAttributeErrors)
	{
		if (const UPCGBasePointData* InPointData = Cast<UPCGBasePointData>(InData))
		{
			return RemoveDuplicatesPoint(InPointData, InSelectorArrayView, InOptionalContext, bSilenceMissingAttributeErrors);
		}
		else
		{
			return RemoveDuplicatesParamSpatial(InData, InSelectorArrayView, InOptionalContext, bSilenceMissingAttributeErrors);
		}
	}
}

#undef LOCTEXT_NAMESPACE