// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGMetadataTypesConstantStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMetadataTypesConstantStruct)

FString FPCGMetadataTypesConstantStruct::ToString() const
{
	switch (Type)
	{
	case EPCGMetadataTypes::Integer32:
		return FString::Printf(TEXT("%d"), Int32Value);
	case EPCGMetadataTypes::Float:
		return FString::Printf(TEXT("%g"), FloatValue);
	case EPCGMetadataTypes::Integer64:
		return FString::Printf(TEXT("%lld"), IntValue);
	case EPCGMetadataTypes::Double:
		return FString::Printf(TEXT("%g"), DoubleValue);
	case EPCGMetadataTypes::String:
		return FString::Printf(TEXT("\"%s\""), *StringValue);
	case EPCGMetadataTypes::Name:
		return FString::Printf(TEXT("N(\"%s\")"), *NameValue.ToString());
	case EPCGMetadataTypes::Vector2:
		return FString::Printf(TEXT("V(%g, %g)"), Vector2Value.X, Vector2Value.Y);
	case EPCGMetadataTypes::Vector:
		return FString::Printf(TEXT("V(%g, %g, %g)"), VectorValue.X, VectorValue.Y, VectorValue.Z);
	case EPCGMetadataTypes::Vector4:
		return FString::Printf(TEXT("V(%g, %g, %g, %g)"), Vector4Value.X, Vector4Value.Y, Vector4Value.Z, Vector4Value.W);
	case EPCGMetadataTypes::Rotator:
		return FString::Printf(TEXT("R(%g, %g, %g)"), RotatorValue.Roll, RotatorValue.Pitch, RotatorValue.Yaw);
	case EPCGMetadataTypes::Quaternion:
		return FString::Printf(TEXT("Q(%g, %g, %g, %g)"), QuatValue.X, QuatValue.Y, QuatValue.Z, QuatValue.W);
	case EPCGMetadataTypes::Transform:
		return FString::Printf(TEXT("Transform"));
	case EPCGMetadataTypes::Boolean:
		return FString::Printf(TEXT("%s"), (BoolValue ? TEXT("True") : TEXT("False")));
	case EPCGMetadataTypes::SoftObjectPath:
		return FString::Printf(TEXT("%s"), *SoftObjectPathValue.ToString());
	case EPCGMetadataTypes::SoftClassPath:
		return FString::Printf(TEXT("%s"), *SoftClassPathValue.ToString());
	default:
		break;
	}

	return FString();
}

FString FPCGMetadataTypesConstantStruct::TypeToString() const
{
	const UEnum* EnumPtr = StaticEnum<EPCGMetadataTypes>();
	return EnumPtr ? EnumPtr->GetNameStringByValue(static_cast<int64>(Type)) : FString{};
}

#if WITH_EDITOR
void FPCGMetadataTypesConstantStruct::OnPostLoad()
{
	// We used to represent soft object/class paths as strings, but now we support them natively.
	if (Type == EPCGMetadataTypes::String)
	{
		if (StringMode_DEPRECATED == EPCGMetadataTypesConstantStructStringMode::SoftObjectPath)
		{
			Type = EPCGMetadataTypes::SoftObjectPath;
		}
		if (StringMode_DEPRECATED == EPCGMetadataTypesConstantStructStringMode::SoftClassPath)
		{
			Type = EPCGMetadataTypes::SoftClassPath;
		}
	}
}
#endif
