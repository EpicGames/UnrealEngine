// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGObjectPropertyOverride.h"

#include "PCGContext.h"
#include "PCGData.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGContainerAccessor.h"

#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGObjectPropertyOverride)

#define LOCTEXT_NAMESPACE "PCGObjectPropertyOverride"

namespace PCGObjectPropertyOverrideHelpers
{
	FPCGPinProperties CreateObjectPropertiesOverridePin(FName Label, const FText& Tooltip)
	{
		FPCGPinProperties ObjectOverridePinProperties(Label, EPCGDataType::Param, /*bAllowMultipleConnections=*/false, /*bAllowMultipleData=*/false, Tooltip);
		ObjectOverridePinProperties.SetAdvancedPin();
		return ObjectOverridePinProperties;
	}

	void ApplyOverrides(const TArray<FPCGObjectPropertyOverrideDescription>& InObjectPropertyOverrideDescriptions, const TArray<TPair<UObject*, int32>>& TargetObjectToOverrideIndices, FName OverridesPinLabel, int32 InInputDataIndex, FPCGContext* Context, bool bPropagateEditChangeEvent)
	{
		if (!Context || TargetObjectToOverrideIndices.IsEmpty() || OverridesPinLabel == NAME_None || InInputDataIndex == INDEX_NONE)
		{
			return;
		}

		const TArray<FPCGTaggedData> OverridesTaggedData = Context->InputData.GetInputsByPin(OverridesPinLabel);
		if (OverridesTaggedData.IsEmpty())
		{
			return;
		}
		
		if (!OverridesTaggedData.IsValidIndex(InInputDataIndex))
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("InconsistentDataCount", "The data provided on pin '{0}' does not have a consistent size with the input index '{1}'. Will use the first one."), FText::FromName(OverridesPinLabel), FText::AsNumber(InInputDataIndex)), Context);
			InInputDataIndex = 0;
		}

		const UPCGData* OverrideData = OverridesTaggedData[InInputDataIndex].Data;
		
		FApplyOverrideParams Params =
		{
			.ObjectPropertyOverrideDescriptions = InObjectPropertyOverrideDescriptions,
			.TargetObjectToOverrideIndices = TargetObjectToOverrideIndices,
			.OverridesData = MakeConstArrayView(&OverrideData, 1),
			.OptionalContext = Context,
			.bPropagateEditChangeEvent = bPropagateEditChangeEvent
		};

		return ApplyOverrides(Params);
	}

	void ApplyOverrides(const FApplyOverrideParams& InParams)
	{
		if (InParams.TargetObjectToOverrideIndices.IsEmpty() || InParams.OverridesData.IsEmpty())
		{
			return;
		}

		if (InParams.OverridesData.Num() == 0)
		{
			return;
		}
		else if (InParams.OverridesData.Num() > 1)
		{
			if (!InParams.bOneDataSetPerElement)
			{
				PCGLog::LogWarningOnGraph(LOCTEXT("TooManyData", "When not having a Data set per element it is expected to pass just a single override data. Will use the first one"), InParams.OptionalContext);
			}
			else if (InParams.OverridesData.Num() != InParams.TargetObjectToOverrideIndices.Num())
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ExpectedSameDataCount", "When having one data set per element, it is expected to have the same number of override data than elements, we expected {0} and got {1} data."), InParams.TargetObjectToOverrideIndices.Num(), InParams.OverridesData.Num()), InParams.OptionalContext);
				return;
			}
		}

		for (const TPair<UObject*, int32>& TargetObjectAndIndex : InParams.TargetObjectToOverrideIndices)
		{
			UObject* TargetObject = TargetObjectAndIndex.Key;
			int32 InputKeyIndex = TargetObjectAndIndex.Value;

			if (!IsValid(TargetObject))
			{
				continue;
			}

			const UPCGData* OverrideData = InParams.bOneDataSetPerElement ? InParams.OverridesData[InputKeyIndex % InParams.OverridesData.Num()] : InParams.OverridesData[0];

			FPCGObjectOverrides<UObject> ObjectOverrides(TargetObject);

			FPCGObjectOverrides<UObject>::FInitializeParams Params =
			{
				.OverrideDescriptions = InParams.ObjectPropertyOverrideDescriptions,
				.TemplateObject = TargetObject,
				.SourceData = OverrideData,
				.OptionalContext = InParams.OptionalContext,
				.bInPropagateEditChangeEvent = InParams.bPropagateEditChangeEvent,
				.bAllowedToUseAllSourceDataForContainers = InParams.bOneDataSetPerElement
			};
			
			ObjectOverrides.Initialize(Params);
			if (!ObjectOverrides.Apply(InputKeyIndex)) // Use the First Entry of the param data for override (similar to what is done in Parameter Overrides in FPCGContext)
			{
				PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ApplyOverrideFailed", "Failed to apply property overrides to object '%s'."), FText::FromName(TargetObject->GetClass()->GetFName())), InParams.OptionalContext);
			}
		}
	}

	void ApplyOverridesFromParams(const TArray<FPCGObjectPropertyOverrideDescription>& InObjectPropertyOverrideDescriptions, UObject* TargetObject, FName OverridesPinLabel, FPCGContext* Context, bool bPropagateEditChangeEvent)
	{
		ApplyOverrides(InObjectPropertyOverrideDescriptions,
			{ TPair<UObject*, int32>(TargetObject, /*Entry index=*/0) },
			OverridesPinLabel,
			/*InInputDataIndex=*/0,
			Context,
			bPropagateEditChangeEvent);
	}
}

void FPCGObjectSingleOverride::Initialize(const FPCGAttributePropertySelector& InputSelector, const FString& OutputProperty, const UStruct* TemplateClass, const UPCGData* SourceData, FPCGContext* Context, bool bComputePropertyEditChain)
{
	FInitializeParams Params =
	{
		.InputSelector = &InputSelector,
		.OutputProperty = MakeStringView(OutputProperty),
		.TemplateClass = TemplateClass,
		.SourceData = SourceData,
		.OptionalContext = Context,
		.bComputePropertyEditChain = bComputePropertyEditChain
	};

	Initialize(Params);
}

void FPCGObjectSingleOverride::Initialize(const FInitializeParams& InParams)
{
	if (!InParams.InputSelector)
	{
		return;
	}
	
	InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(InParams.SourceData, *InParams.InputSelector);
	ObjectOverrideInputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InParams.SourceData, *InParams.InputSelector);

	if (!ObjectOverrideInputAccessor.IsValid())
	{
		PCGLog::LogWarningOnGraph(FText::Format(NSLOCTEXT("PCGObjectPropertyOverride", "OverrideInputInvalid", "ObjectOverride for input '{0}' does not exist or is unsupported. Check the attribute or property selection."), InParams.InputSelector->GetDisplayText()), InParams.OptionalContext);
		return;
	}

	const FPCGAttributePropertySelector OutputSelector = FPCGAttributePropertySelector::CreateSelectorFromString(InParams.OutputProperty);
	// TODO: Move implementation into a new helper: PCGAttributeAccessorHelpers::CreatePropertyAccessor(const FPCGAttributePropertySelector& Selector, UStruct* Class)
	const TArray<FString>& ExtraNames = OutputSelector.GetExtraNames();

	TArray<FName> PropertyNames;
	PropertyNames.Reserve(ExtraNames.Num() + 1);
	PropertyNames.Add(OutputSelector.GetAttributeName());
	for (const FString& Name : ExtraNames)
	{
		PropertyNames.Add(FName(Name));
	}

	TArray<const FProperty*> PropertyChain;
	PCGAttributeAccessorHelpers::GetPropertyChain(PropertyNames, InParams.TemplateClass, PropertyChain);
	if (!PropertyChain.IsEmpty())
	{
		TArray<const FStructProperty*> EncounteredStructProps;
		bWillNeedLoading = PropertyChain.Last()->ContainsObjectReference(EncounteredStructProps, EPropertyObjectReferenceType::Strong);

		int32 NumContainers = 0;
		for (const FProperty* Property : PropertyChain)
		{
			if (ensure(Property) && Property->IsA<FArrayProperty>())
			{
				NumContainers++;
			}

#if WITH_EDITOR
			if (InParams.bComputePropertyEditChain)
			{
				EditPropertyChain.AddTail(const_cast<FProperty*>(Property));
			}
#endif // WITH_EDITOR
		}

		if (NumContainers > 1)
		{
			// Unsupported
			ObjectOverrideInputAccessor.Reset();

			PCGLog::LogErrorOnGraph(FText::Format(NSLOCTEXT("PCGObjectPropertyOverride", "OverrideOutputTooManyContainers", "ObjectOverride for output '{0}' is within multiple containers. Unsupported."), OutputSelector.GetDisplayText()), InParams.OptionalContext);
			return;
		}
		
		// To support old flows, use the old accessors to access the attribute, only in single value and if the descriptors don't match.
		const FPCGMetadataAttributeDesc PropertyDesc = FPCGMetadataAttributeDesc::CreateFromProperty(PropertyChain.Last());
		const bool bDescriptorsMatch = PropertyDesc.IsSameType(ObjectOverrideInputAccessor->GetUnderlyingDesc());
		const bool bUseGenericAccessors = PropertyDesc.IsValid() && (!ObjectOverrideInputAccessor->GetUnderlyingDesc().IsSingleValue() || bDescriptorsMatch);
		
		if (PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(PropertyChain.Last(), bUseGenericAccessors))
		{
			ObjectOverrideOutputAccessor = NumContainers == 1 ? PCGContainerAccessorHelpers::MakeContainerAccessor(PropertyChain, bUseGenericAccessors) : PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(MoveTemp(PropertyChain), bUseGenericAccessors);
			bIsContainer = NumContainers == 1;
		}
	}

	if (!ObjectOverrideOutputAccessor.IsValid())
	{
		ObjectOverrideInputAccessor.Reset();
		PCGLog::LogWarningOnGraph(FText::Format(NSLOCTEXT("PCGObjectPropertyOverride", "OverrideOutputInvalid", "ObjectOverride for object property '{0}' is invalid or unsupported. Check the attribute or property selection."), FText::FromStringView(InParams.OutputProperty)), InParams.OptionalContext);
		return;
	}
	
	// We support converting single value to array
	FPCGMetadataAttributeDesc InputDesc = ObjectOverrideInputAccessor->GetUnderlyingDesc();
	const FPCGMetadataAttributeDesc& OutputDesc = ObjectOverrideOutputAccessor->GetUnderlyingDesc();
	if (OutputDesc.IsArray() && InputDesc.IsSingleValue())
	{
		InputDesc.ContainerTypes = OutputDesc.ContainerTypes;
	}

	if (!PCG::Private::IsBroadcastableOrConstructible(InputDesc, OutputDesc))
	{
		PCGLog::LogWarningOnGraph(
			FText::Format(
				NSLOCTEXT("PCGObjectPropertyOverride", "TypesIncompatible", "ObjectOverride cannot set input '{0}' to output '{1}'. Cannot convert type '{2}' to type '{3}'. Will be skipped."),
				InParams.InputSelector->GetDisplayText(),
				FText::FromStringView(InParams.OutputProperty),
				ObjectOverrideInputAccessor->GetUnderlyingDesc().GetTypeText(),
				ObjectOverrideOutputAccessor->GetUnderlyingDesc().GetTypeText()),
			InParams.OptionalContext);

		ObjectOverrideInputAccessor.Reset();
		ObjectOverrideOutputAccessor.Reset();
		return;
	}
	
	ObjectOverrideInputSelector = *InParams.InputSelector;
	bAllowedToUseAllSourceDataForContainers = InParams.bAllowedToUseAllSourceDataForContainers;
}

bool FPCGObjectSingleOverride::IsValid() const
{
	return InputKeys.IsValid() && ObjectOverrideInputAccessor.IsValid() && ObjectOverrideOutputAccessor.IsValid();
}

void FPCGObjectSingleOverride::PreApply(UObject* TargetObject)
{
#if WITH_EDITOR
	if (TargetObject && !EditPropertyChain.IsEmpty())
	{
		TargetObject->PreEditChange(EditPropertyChain);
	}
#endif // WITH_EDITOR
}

void FPCGObjectSingleOverride::PostApply(UObject* TargetObject)
{
#if WITH_EDITOR
	if (TargetObject && !EditPropertyChain.IsEmpty())
	{
		FPropertyChangedChainEvent ChangedEvent{EditPropertyChain, FPropertyChangedEvent{EditPropertyChain.GetHead()->GetValue(), EPropertyChangeType::ValueSet}};
		TargetObject->PostEditChangeChainProperty(ChangedEvent);
	}
#endif // WITH_EDITOR
}

bool FPCGObjectSingleOverride::Apply(int32 InputKeyIndex, IPCGAttributeAccessorKeys& OutputKey)
{
	if (!IsValid())
	{
		return false;
	}

	check(ObjectOverrideInputAccessor.IsValid());
	check(ObjectOverrideOutputAccessor.IsValid());
	
	if (bAllowedToUseAllSourceDataForContainers && bIsContainer)
	{
		return ObjectOverrideInputAccessor->CopyTo(*InputKeys, *ObjectOverrideOutputAccessor, OutputKey, InputKeyIndex, /*OutputIndex=*/0, /*Count=*/InputKeys->GetNum(), EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible | EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer);
	}
	else
	{
		return ObjectOverrideInputAccessor->CopyTo(*InputKeys, *ObjectOverrideOutputAccessor, OutputKey, InputKeyIndex, /*OutputIndex=*/0, /*Count=*/1, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible | EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer);
	}
}

void FPCGObjectSingleOverride::GatherAllOverridesToLoad(TArray<FSoftObjectPath>& OutObjectsToLoad) const
{
	if (!bWillNeedLoading || !IsValid())
	{
		return;
	}

	PCGMetadataElementCommon::ApplyOnAccessor<FSoftObjectPath>(*InputKeys, *ObjectOverrideInputAccessor, [&OutObjectsToLoad](FSoftObjectPath&& Value, int32 Index)
	{
		// Only keep valid paths
		if (!Value.IsNull())
		{
			OutObjectsToLoad.AddUnique(std::move(Value));
		}
	});
}

#undef LOCTEXT_NAMESPACE
