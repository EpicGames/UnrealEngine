// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGAssetExporter.h"

#include "PCGModule.h"

#include "AssetRegistry/AssetData.h"
#include "Misc/Base64.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAssetExporter)

TSubclassOf<UPCGDataAsset> UPCGAssetExporter::BP_GetAssetType_Implementation() const
{
	return UPCGDataAsset::StaticClass();
}

TSubclassOf<UPCGDataAsset> UPCGAssetExporter::GetAssetType() const
{
	return BP_GetAssetType();
}

bool UPCGAssetExporter::BP_ExportToAsset_Implementation(UPCGDataAsset* Asset)
{
	// TODO in BP derived exporter classes
	return false;
}

UPackage* UPCGAssetExporter::Update(const FAssetData& PCGAsset)
{
#if WITH_EDITOR
	SerializeMetadataFromAsset(PCGAsset);
	if (UPackage* Package = UpdateAsset(PCGAsset))
	{
		SerializeMetadataToAsset(PCGAsset);
		Package->MarkPackageDirty();
		return Package;
	}
	else
	{
		UE_LOGF(LogPCG, Warning, "Updating the '%ls' PCG asset failed.", *PCGAsset.AssetName.ToString())
		return nullptr;
	}
#else
	UE_LOGF(LogPCG, Error, "PCG Asset update is not supported in non-editor builds.");
	return nullptr;
#endif
}

UPackage* UPCGAssetExporter::UpdateAsset(const FAssetData& PCGAsset)
{
	// Default implementation will load the asset and call the BP update method
	UPCGDataAsset* LoadedAsset = Cast<UPCGDataAsset>(PCGAsset.GetAsset());

	if (LoadedAsset && BP_ExportToAsset(LoadedAsset))
	{
		return LoadedAsset->GetPackage();
	}
	else
	{
		return nullptr;
	}
}

bool UPCGAssetExporter::Export(const FString& PackageName, UPCGDataAsset* Asset)
{
#if WITH_EDITOR
	if (Asset)
	{
		Asset->ExporterClass = GetClass();
		if (ExportAsset(PackageName, Asset))
		{
			DuplicateAndReOuterData(Asset);
			SerializeMetadataToAsset(Asset);
			Asset->MarkPackageDirty();
			return true;
		}
		else
		{
			return false;
		}
	}
	else
	{
		return false;
	}
#else
	UE_LOGF(LogPCG, Error, "PCG Asset export should not be used in non-editor builds.");
	return false;
#endif
}

bool UPCGAssetExporter::ExportAsset(const FString& PackageName, UPCGDataAsset* Asset)
{
	return BP_ExportToAsset(Asset);
}

void UPCGAssetExporter::SerializeMetadataFromAsset(const FAssetData& PCGAsset)
{
#if WITH_EDITOR
	const FString ExporterMetadata = PCGAsset.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, ExporterMetadata));
	
	TArray<uint8> PayloadData;
	if (FBase64::Decode(ExporterMetadata, PayloadData))
	{
		FMemoryReader PayloadAr(PayloadData, /*bIsPersistent=*/true);

		// Read custom versions
		FCustomVersionContainer CustomVersions;
		CustomVersions.Serialize(PayloadAr);
		PayloadAr.SetCustomVersions(CustomVersions);

		SerializeMetadata(PayloadAr);
	}
#endif // WITH_EDITOR
}

void UPCGAssetExporter::SerializeMetadataToAsset(const FAssetData& PCGAsset)
{
#if WITH_EDITOR
	// Asset is expected to be in memory at this point, but this might trigger loading otherwise
	UPCGDataAsset* Asset = Cast<UPCGDataAsset>(PCGAsset.GetAsset());

	if (!Asset)
	{
		return;
	}

	TArray<uint8> PayloadData;
	FMemoryWriter PayloadAr(PayloadData, /*bIsPersistent=*/true);

	SerializeMetadata(PayloadAr);

	// Serialize custom versions
	TArray<uint8> HeaderData;
	FMemoryWriter HeaderAr(HeaderData);
	FCustomVersionContainer CustomVersions = PayloadAr.GetCustomVersions();
	CustomVersions.Serialize(HeaderAr);

	TArray<uint8> SerializedMetadata = MoveTemp(HeaderData);
	SerializedMetadata.Append(PayloadData);

	Asset->ExporterMetadata = FBase64::Encode(SerializedMetadata);
#endif // WITH_EDITOR
}

void UPCGAssetExporter::DuplicateAndReOuterData(UPCGDataAsset* Asset)
{
	if (!Asset)
	{
		return;
	}

	// Implementation note: we can't easily update the contents of data even through visit data
	// because it doesn't allow changing its parts.
	for (FPCGTaggedData& TaggedData : Asset->Data.TaggedData)
	{
		if (TaggedData.Data && TaggedData.Data->GetOuter() == GetTransientPackage())
		{
			UPCGData* DuplicatedData = TaggedData.Data->DuplicateData(nullptr);
			DuplicatedData->Rename(nullptr, Asset);

			DuplicatedData->VisitDataNetwork([Asset](const UPCGData* InData)
			{
				if (InData && InData->GetOuter() == Asset)
				{
					const_cast<UPCGData*>(InData)->Flatten();
				}
			});

			TaggedData.Data = DuplicatedData;
		}
	}
}
