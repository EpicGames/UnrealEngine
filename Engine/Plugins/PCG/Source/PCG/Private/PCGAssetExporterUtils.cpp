// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGAssetExporterUtils.h"

#include "PCGContext.h"
#include "PCGModule.h"

#if WITH_EDITOR
#include "CoreGlobals.h"
#include "ContentBrowserModule.h"
#include "FileHelpers.h"
#include "IContentBrowserSingleton.h"
#include "AssetRegistry/AssetRegistryModule.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAssetExporterUtils)

#define LOCTEXT_NAMESPACE "PCGAssetExporterUtils"

namespace PCGAssetExporterUtils
{
	static const FText NonEditorError = LOCTEXT("CannotExportInNonEditor", "PCG Asset Exporter Utils cannot be used in non-editor builds.");
}

UPackage* UPCGAssetExporterUtils::CreateAsset(UPCGAssetExporter* Exporter, FPCGAssetExporterParameters Parameters)
{
	return CreateAsset(Exporter, Parameters, nullptr);
}

UPackage* UPCGAssetExporterUtils::CreateAsset(UPCGAssetExporter* Exporter, const FPCGAssetExporterParameters& Parameters, FPCGContext* Context)
{
#if WITH_EDITOR
	if (!Exporter || !Exporter->GetAssetType())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("MissingExporter", "Unable to create asset without an exporter, or exporter is not setup properly."), Context);
		return nullptr;
	}

	const bool bExporterNeedsRooting = !Exporter->IsRooted();
	if (bExporterNeedsRooting)
	{
		Exporter->AddToRoot();
	}

	UPackage* ResultPackage = CreateAsset(Exporter->GetAssetType(),
		Parameters,
		[Exporter](const FString& PackagePath, UObject* Asset) -> bool
		{
			return Exporter->Export(PackagePath, CastChecked<UPCGDataAsset>(Asset));
		},
		Context);

	if (bExporterNeedsRooting)
	{
		Exporter->RemoveFromRoot();
	}

	return ResultPackage;
#else
	PCGLog::LogErrorOnGraph(PCGAssetExporterUtils::NonEditorError, Context);
	return nullptr;
#endif // WITH_EDITOR
}

UPackage* UPCGAssetExporterUtils::CreateAsset(const UClass* AssetClass, const FPCGAssetExporterParameters& Parameters, TFunctionRef<bool(const FString&, UObject*)> ExportFunc, FPCGContext* Context)
{
#if WITH_EDITOR
	if (!AssetClass)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("MissingAssetClass", "Unable to create asset without a class."), Context);
		return nullptr;
	}
	
	FString AssetName = Parameters.AssetName;
	FString AssetPath = Parameters.AssetPath;
	FString PackageName = FPaths::Combine(AssetPath, AssetName);

	if (Parameters.bOpenSaveDialog)
	{
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DefaultPath = AssetPath;
		SaveAssetDialogConfig.DefaultAssetName = AssetName;
		SaveAssetDialogConfig.AssetClassNames.Add(AssetClass->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = NSLOCTEXT("PCGAssetExporter", "SaveAssetToFileDialogTitle", "Save PCG Asset");

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
		if (!SaveObjectPath.IsEmpty())
		{
			AssetName = FPackageName::ObjectPathToObjectName(SaveObjectPath);
			AssetPath = FString(); // not going to be reused
			PackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		}
		else
		{
			return nullptr;
		}
	}
	else
	{
		// Perform some validation on the package name, so we can prevent crashes downstream when trying to create or save the package.
		FText Reason;
		if (!FPackageName::IsValidObjectPath(PackageName, &Reason))
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidPackagePath", "Invalid package path '{0}': {1}."), { FText::FromString(PackageName), Reason }), Context);
			return nullptr;
		}
	}

	UPackage* Package = FPackageName::DoesPackageExist(PackageName) ? LoadPackage(nullptr, *PackageName, LOAD_None) : FindPackage(nullptr, *PackageName);

	UObject* Asset = nullptr;
	bool NewAssetCreated = false;

	if (Package)
	{
		UObject* Object = FindObjectFast<UObject>(Package, *AssetName);
		if (Object && Object->GetClass() != AssetClass)
		{
			Object->SetFlags(RF_Transient);
			Object->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
			NewAssetCreated = true;
		}
		else
		{
			Asset = Object;
		}
	}
	else
	{
		Package = CreatePackage(*PackageName);

		if (Package)
		{
			NewAssetCreated = true;
		}
		else
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("UnableToCreatePackage", "Unable to create the package with name '{0}'."), FText::FromString(PackageName)), Context);
			return nullptr;
		}
	}

	if (!Asset)
	{
		const EObjectFlags Flags = RF_Public | RF_Standalone | RF_Transactional;
		Asset = NewObject<UObject>(Package, AssetClass, FName(*AssetName), Flags);
	}

	if (Asset)
	{
		if (NewAssetCreated)
		{
			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(Asset);
		}

		if (ExportFunc(PackageName, Asset))
		{
			// Make sure everybody knows we changed the asset
			if (!NewAssetCreated)
			{
				FCoreUObjectDelegates::BroadcastOnObjectModified(Asset);
			}
		}
		else
		{
			return nullptr;
		}
	}

	// Save the file
	if (Package && Parameters.bSaveOnExportEnded)
	{
		// Set GIsRunningUnattendedScript to 'true' if it isn't already set while running commandlets
		TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, GIsRunningUnattendedScript || IsRunningCommandlet());
		FEditorFileUtils::PromptForCheckoutAndSave({ Package }, /*bCheckDirty=*/false, /*bPromptToSave=*/false);
	}

	return Package;
#else
	PCGLog::LogErrorOnGraph(PCGAssetExporterUtils::NonEditorError, Context);
	return nullptr;
#endif
}

void UPCGAssetExporterUtils::UpdateAssets(const TArray<FAssetData>& PCGAssets, FPCGAssetExporterParameters InParameters)
{
	UpdateAssets(PCGAssets, InParameters, nullptr);
}

void UPCGAssetExporterUtils::UpdateAssets(const TArray<FAssetData>& PCGAssets, const FPCGAssetExporterParameters& InParameters, FPCGContext* Context)
{
#if WITH_EDITOR
	TArray<UPackage*> PackagesToSave;
	FPCGAssetExporterParameters Parameters = InParameters;

	// Never open a dialog when updating pre-existing assets
	Parameters.bOpenSaveDialog = false;

	for (const FAssetData& PCGAsset : PCGAssets)
	{
		TSubclassOf<UPCGAssetExporter> ExporterSubclass;

		const FString ExporterClassString = PCGAsset.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, ExporterClass));
		TSoftClassPtr<> ExporterClass(ExporterClassString);
		if (ExporterClass.IsValid() && ExporterClass->IsChildOf<UPCGAssetExporter>())
		{
			ExporterSubclass = ExporterClass.Get();
		}
		else
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("InvalidExporterOnAsset", "Unable to update asset '{0}' because exporter isn't valid."), FText::FromName(PCGAsset.AssetName)), Context);
			continue;
		}

		UPCGAssetExporter* Exporter = NewObject<UPCGAssetExporter>(GetTransientPackage(), ExporterSubclass);

		if (!Exporter)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("UnableToCreateExporter", "Unable to create exporter for asset '{0}' during update process."), FText::FromName(PCGAsset.AssetName)), Context);
			continue;
		}

		Exporter->AddToRoot();

		if (UPackage* Package = Exporter->Update(PCGAsset))
		{
			FCoreUObjectDelegates::BroadcastOnObjectModified(PCGAsset.GetAsset());
			PackagesToSave.Add(Package);
		}

		Exporter->RemoveFromRoot();
	}

	// Save the file(s)
	if (!PackagesToSave.IsEmpty() && Parameters.bSaveOnExportEnded)
	{
		// Set GIsRunningUnattendedScript to 'true' if it isn't already set while running commandlets
		TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, GIsRunningUnattendedScript || IsRunningCommandlet());
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, /*bCheckDirty=*/false, /*bPromptToSave=*/false);
	}
#else
	PCGLog::LogErrorOnGraph(PCGAssetExporterUtils::NonEditorError, Context);
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
