// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGSubsystem.h"

#include "EngineDefines.h" // For UE_ENABLE_DEBUG_DRAWING

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCommon)

namespace PCGLog
{
	FString GetComponentOwnerName(const UPCGComponent* InComponent, bool bUseLabel, FString InDefaultName)
	{
		if (InComponent && InComponent->GetOwner())
		{
			if (IsRunningCommandlet())
			{
				return InComponent->GetOwner()->GetPathName();
			}
			else if(bUseLabel)
			{
				return InComponent->GetOwner()->GetActorNameOrLabel();
			}
			else
			{
				return InComponent->GetOwner()->GetName();
			}
		}

		return InDefaultName;
	}

	FString GetExecutionSourceName(const IPCGGraphExecutionSource* InExecutionSource, bool bUseLabel, FString InDefaultSource)
	{
		const UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource);
		if (PCGComponent)
		{
			return GetComponentOwnerName(PCGComponent, bUseLabel, InDefaultSource);
		}
		else if (InExecutionSource)
		{
			return InExecutionSource->GetExecutionState().GetDebugName();
		}
		else
		{
			return InDefaultSource;
		}
	}
}

namespace PCGFeatureSwitches
{
	TAutoConsoleVariable<bool> CVarCheckSamplerMemory{
		TEXT("pcg.CheckSamplerMemory"),
		true,
		TEXT("Checks expected memory size consumption prior to performing sampling operations")
	};

	TAutoConsoleVariable<float> CVarSamplerMemoryThreshold{
		TEXT("pcg.SamplerMemoryThreshold"),
		0.8f,
		TEXT("Normalized threshold of remaining physical memory required to abort sampling operation."),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			if (InVariable->GetFloat() < 0.f || InVariable->GetFloat() > 1.0)
			{
				InVariable->SetWithCurrentPriority(FMath::Clamp(InVariable->GetFloat(), 0.f, 1.f));
			}
		})
	};

	namespace Helpers
	{
		uint64 GetAvailableMemoryForSamplers()
		{
			const FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
			// Also uses AvailableVirtual because the system might have plenty of physical memory but still be limited by virtual memory available in some cases.
			// (i.e. per-process quota, paging file size lower than actual memory available, etc.).
			return CVarSamplerMemoryThreshold.GetValueOnAnyThread() * FMath::Min(MemoryStats.AvailablePhysical, MemoryStats.AvailableVirtual);
		}
	}
}

namespace PCGSystemSwitches
{
#if WITH_EDITOR
	TAutoConsoleVariable<bool> CVarPausePCGExecution(
		TEXT("pcg.PauseExecution"),
		false,
		TEXT("Pauses all execution of PCG but does not cancel tasks."));

	TAutoConsoleVariable<bool> CVarGlobalDisableRefresh(
		TEXT("pcg.GlobalDisableRefresh"),
		false,
		TEXT("Disable refresh for all PCG Components."));

	TAutoConsoleVariable<bool> CVarDirtyLoadAsPreviewOnLoad(
		TEXT("pcg.DirtyLoadAsPreviewOnLoad"),
		false,
		TEXT("Enables dirtying on load for load as preview components.\nTurning off this option will require to force generate or apply a change before this component is regenerated."));

	TAutoConsoleVariable<bool> CVarForceDynamicGraphDispatch(
		TEXT("pcg.GraphCompilation.ForceDynamicDispatch"),
		false,
		TEXT("Forces all subgraph executions to be dynamic. Performance warning, also requires to flush all compiled graphs."));
#endif

#if UE_ENABLE_DEBUG_DRAWING
	TAutoConsoleVariable<bool> CVarPCGDebugDrawGeneratedCells(
		TEXT("pcg.GraphExecution.DebugDrawGeneratedCells"),
		false,
		TEXT("Draws debug boxes around any cell that is executing, colored by grid level, and in PIE/standalone labels grid size and coords."));
#endif // UE_ENABLE_DEBUG_DRAWING

#if !UE_BUILD_SHIPPING
	TAutoConsoleVariable<bool> CVarFuzzGPUMemory(
		TEXT("pcg.GPU.FuzzMemory"),
		false,
		TEXT("Initializes GPU buffers with random numbers which helps to reproduce undefined behaviour from unitialized memory."));
#endif //!UE_BUILD_SHIPPING

	TAutoConsoleVariable<bool> CVarPassGPUDataThroughGridLinks(
		TEXT("pcg.Graph.GPU.PassGPUDataThroughGridLinks"),
		true,
		TEXT("Whether proxies for GPU data are cached in per pin output data and passed through grid links. If false data is read back to CPU."));

	// Deprecated section.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TAutoConsoleVariable<bool> CVarReleaseTransientResourcesEarly(
		TEXT("pcg.ReleaseTransientResourcesEarly"),
		true,
		TEXT("DEPRECATED: This feature is always enabled now."),
		ECVF_ReadOnly);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

namespace PCGHiGenGrid
{
	bool IsValidGridSize(uint32 InGridSize)
	{
		return InGridSize > UninitializedGridSize() && InGridSize < UnboundedGridSize();
	}

	bool IsValidGrid(EPCGHiGenGrid InGrid)
	{
		// Check the bitmask value is within range
		return InGrid >= EPCGHiGenGrid::GridMin && static_cast<uint32>(InGrid) < 2 * static_cast<uint32>(EPCGHiGenGrid::GridMax);
	}

	bool IsValidGridOrUninitialized(EPCGHiGenGrid InGrid)
	{
		return IsValidGrid(InGrid) || InGrid == EPCGHiGenGrid::Uninitialized;
	}

	bool IsValidGridOrUninitialized(uint32 InGridSize)
	{
		return IsValidGridSize(InGridSize) || InGridSize == PCGHiGenGrid::UninitializedGridSize();
	}

	uint32 GridToGridSize(EPCGHiGenGrid InGrid)
	{
		const uint32 GridAsUint = static_cast<uint32>(InGrid);
		ensure(FMath::IsPowerOfTwo(GridAsUint));
		// TODO: support other units
		return IsValidGrid(InGrid) ? (GridAsUint * 100) : UninitializedGridSize();
	}

	EPCGHiGenGrid GridSizeToGrid(uint32 InGridSize)
	{
		if (InGridSize == UnboundedGridSize())
		{
			return EPCGHiGenGrid::Unbounded;
		}
		else if (InGridSize <= GridToGridSize(EPCGHiGenGrid::GridMin))
		{
			return EPCGHiGenGrid::GridMin;
		}
		else if (InGridSize >= GridToGridSize(EPCGHiGenGrid::GridMax))
		{
			return EPCGHiGenGrid::GridMax;
		}

		// Grid size is in cm, but EPCGHiGenGrid is in meters so we convert the units before rounding.
		const uint32 RoundedGridSizeMeters = FMath::RoundUpToPowerOfTwo(InGridSize / 100);
		return static_cast<EPCGHiGenGrid>(RoundedGridSizeMeters);
	}

	EPCGHiGenGrid ScaledGridSizeToGrid(uint32 InScaledGridSize, double InGridScale)
	{
		if (InScaledGridSize == UnboundedGridSize())
		{
			return EPCGHiGenGrid::Unbounded;
		}
		else if (InScaledGridSize == UninitializedGridSize())
		{
			return EPCGHiGenGrid::Uninitialized;
		}
		else if (InGridScale <= UE_KINDA_SMALL_NUMBER)
		{
			ensure(false);
			return EPCGHiGenGrid::Uninitialized;
		}

		const uint32 UnscaledGridSize = static_cast<uint32>(FMath::RoundToInt(static_cast<double>(InScaledGridSize) / InGridScale));

		return GridSizeToGrid(UnscaledGridSize);
	}

	uint32 UninitializedGridSize()
	{
		return static_cast<uint32>(EPCGHiGenGrid::Uninitialized);
	}

	uint32 UnboundedGridSize()
	{
		return static_cast<uint32>(EPCGHiGenGrid::Unbounded);
	}

	double SanitizeGridSizeMultiplier(double InGridSizeMultiplier)
	{
		// The smallest grid scaled by the multiplier have an integral size.
		const double SmallestGridSize = static_cast<double>(GridToGridSize(EPCGHiGenGrid::GridMin));
		// Arbitrary choice. We could reduce this if we hear the requirement to have smaller grid cells.
		const double SmallestScaledGridSize = 100.0;
		return FMath::Max(SmallestScaledGridSize, FMath::RoundToDouble(InGridSizeMultiplier * SmallestGridSize)) / SmallestGridSize;
	}
}

void FPCGRuntimeGenerationRadii::PostSerialize(const FArchive& Ar)
{
	ComputeHash();
}

double FPCGRuntimeGenerationRadii::GetGenerationRadiusFromGrid(EPCGHiGenGrid Grid) const
{
	const int32 QualityLevel = UPCGSubsystem::GetPCGQualityLevel();

	switch (Grid)
	{
		case EPCGHiGenGrid::Grid4: return Radius400.GetValue(QualityLevel);
		case EPCGHiGenGrid::Grid8: return Radius800.GetValue(QualityLevel);
		case EPCGHiGenGrid::Grid16: return Radius1600.GetValue(QualityLevel);
		case EPCGHiGenGrid::Grid32: return Radius3200.GetValue(QualityLevel);
		case EPCGHiGenGrid::Grid64: return Radius6400.GetValue(QualityLevel);
		case EPCGHiGenGrid::Grid128: return Radius12800.GetValue(QualityLevel);
		case EPCGHiGenGrid::Grid256: return Radius25600.GetValue(QualityLevel);
		case EPCGHiGenGrid::Grid512: return Radius51200.GetValue(QualityLevel);
		case EPCGHiGenGrid::Grid1024: return Radius102400.GetValue(QualityLevel);
		case EPCGHiGenGrid::Grid2048: return Radius204800.GetValue(QualityLevel);
		case EPCGHiGenGrid::Grid4096: return Radius204800.GetValue(QualityLevel) * (1 << 1);
		case EPCGHiGenGrid::Grid8192: return Radius204800.GetValue(QualityLevel) * (1 << 2);
		case EPCGHiGenGrid::Grid16384: return Radius204800.GetValue(QualityLevel) * (1 << 3);
		case EPCGHiGenGrid::Grid32768: return Radius204800.GetValue(QualityLevel) * (1 << 4);
		case EPCGHiGenGrid::Grid65536: return Radius204800.GetValue(QualityLevel) * (1 << 5);
		case EPCGHiGenGrid::Grid131072: return Radius204800.GetValue(QualityLevel) * (1 << 6);
		case EPCGHiGenGrid::Grid262144: return Radius204800.GetValue(QualityLevel) * (1 << 7);
		case EPCGHiGenGrid::Grid524288: return Radius204800.GetValue(QualityLevel) * (1 << 8);
		case EPCGHiGenGrid::Grid1048576: return Radius204800.GetValue(QualityLevel) * (1 << 9);
		case EPCGHiGenGrid::Grid2097152: return Radius204800.GetValue(QualityLevel) * (1 << 10);
		case EPCGHiGenGrid::Grid4194304: return Radius204800.GetValue(QualityLevel) * (1 << 11);
		case EPCGHiGenGrid::Unbounded: return RadiusUnbounded.GetValue(QualityLevel);
	}

	ensure(false);
	return 0;
}

double FPCGRuntimeGenerationRadii::GetCleanupRadiusFromGrid(EPCGHiGenGrid Grid) const
{
	return GetGenerationRadiusFromGrid(Grid) * CleanupRadiusScalar.GetValue(UPCGSubsystem::GetPCGQualityLevel());
}

bool FPCGRuntimeGenerationRadii::operator==(const FPCGRuntimeGenerationRadii& Other) const
{
	auto ArePerQualityFloatsEqual = [](const FPerQualityLevelFloat& A, const FPerQualityLevelFloat& B)
	{
		if (A.GetDefault() != B.GetDefault())
		{
			return false;
		}

		const TMap<int32, float>& QualityValuesA = A.PerQuality;
		const TMap<int32, float>& QualityValuesB = B.PerQuality;

		if (QualityValuesA.Num() != QualityValuesB.Num())
		{
			return false;
		}
		
		for (const auto& [QualityLevel, ValueA] : QualityValuesA)
		{
			const float* ValueB = QualityValuesB.Find(QualityLevel);

			if (!ValueB || !FMath::IsNearlyEqual(ValueA, *ValueB))
			{
				return false;
			}
		}

		return true;
	};

	return ArePerQualityFloatsEqual(RadiusUnbounded, Other.RadiusUnbounded)
		&& ArePerQualityFloatsEqual(Radius400, Other.Radius400)
		&& ArePerQualityFloatsEqual(Radius800, Other.Radius800)
		&& ArePerQualityFloatsEqual(Radius1600, Other.Radius1600)
		&& ArePerQualityFloatsEqual(Radius3200, Other.Radius3200)
		&& ArePerQualityFloatsEqual(Radius6400, Other.Radius6400)
		&& ArePerQualityFloatsEqual(Radius12800, Other.Radius12800)
		&& ArePerQualityFloatsEqual(Radius25600, Other.Radius25600)
		&& ArePerQualityFloatsEqual(Radius51200, Other.Radius51200)
		&& ArePerQualityFloatsEqual(Radius102400, Other.Radius102400)
		&& ArePerQualityFloatsEqual(Radius204800, Other.Radius204800)
		&& ArePerQualityFloatsEqual(CleanupRadiusScalar, Other.CleanupRadiusScalar);
}

#if WITH_EDITOR
void FPCGRuntimeGenerationRadii::ApplyDeprecation()
{
	if (GenerationRadius != 0.0)
	{
		RadiusUnbounded = GenerationRadius;
		GenerationRadius = 0.0;
	}

	if (GenerationRadius400 != 0.0)
	{
		Radius400 = GenerationRadius400;
		GenerationRadius400 = 0.0;
	}

	if (GenerationRadius800 != 0.0)
	{
		Radius800 = GenerationRadius800;
		GenerationRadius800 = 0.0;
	}

	if (GenerationRadius1600 != 0.0)
	{
		Radius1600 = GenerationRadius1600;
		GenerationRadius1600 = 0.0;
	}

	if (GenerationRadius3200 != 0.0)
	{
		Radius3200 = GenerationRadius3200;
		GenerationRadius3200 = 0.0;
	}

	if (GenerationRadius6400 != 0.0)
	{
		Radius6400 = GenerationRadius6400;
		GenerationRadius6400 = 0.0;
	}

	if (GenerationRadius12800 != 0.0)
	{
		Radius12800 = GenerationRadius12800;
		GenerationRadius12800 = 0.0;
	}

	if (GenerationRadius25600 != 0.0)
	{
		Radius25600 = GenerationRadius25600;
		GenerationRadius25600 = 0.0;
	}

	if (GenerationRadius51200 != 0.0)
	{
		Radius51200 = GenerationRadius51200;
		GenerationRadius51200 = 0.0;
	}

	if (GenerationRadius102400 != 0.0)
	{
		Radius102400 = GenerationRadius102400;
		GenerationRadius102400 = 0.0;
	}

	if (GenerationRadius204800 != 0.0)
	{
		Radius204800 = GenerationRadius204800;
		GenerationRadius204800 = 0.0;
	}

	if (CleanupRadiusMultiplier != 0.0)
	{
		CleanupRadiusScalar = CleanupRadiusMultiplier;
		CleanupRadiusMultiplier = 0.0;
	}
}
#endif

void FPCGRuntimeGenerationRadii::ComputeHash()
{
	Hash = 0;

	auto HashQualityFloat = [](const FPerQualityLevelFloat& InQualityFloat)
	{
		uint32 Hash = 0;

		TArray<int32, TInlineAllocator<4>> QualityLevels;
		InQualityFloat.PerQuality.GetKeys(QualityLevels);

		// Sort quality levels so that the hash will be stable.
		QualityLevels.Sort();

		for (const int32 QualityLevel : QualityLevels)
		{
			const float Value = InQualityFloat.GetValue(QualityLevel);

			Hash = HashCombine(Hash, HashCombine(QualityLevel, Value));
		}

		Hash = HashCombine(Hash, InQualityFloat.GetDefault());

		return Hash;
	};

	Hash = HashCombine(Hash, HashQualityFloat(RadiusUnbounded));
	Hash = HashCombine(Hash, HashQualityFloat(Radius400));
	Hash = HashCombine(Hash, HashQualityFloat(Radius800));
	Hash = HashCombine(Hash, HashQualityFloat(Radius1600));
	Hash = HashCombine(Hash, HashQualityFloat(Radius3200));
	Hash = HashCombine(Hash, HashQualityFloat(Radius6400));
	Hash = HashCombine(Hash, HashQualityFloat(Radius12800));
	Hash = HashCombine(Hash, HashQualityFloat(Radius25600));
	Hash = HashCombine(Hash, HashQualityFloat(Radius51200));
	Hash = HashCombine(Hash, HashQualityFloat(Radius102400));
	Hash = HashCombine(Hash, HashQualityFloat(Radius204800));
	Hash = HashCombine(Hash, HashQualityFloat(CleanupRadiusScalar));
}

uint32 GetTypeHash(const FPCGRuntimeGenerationRadii& InGenerationRadii)
{
	return InGenerationRadii.Hash;
}

namespace PCGDelegates
{
#if WITH_EDITOR
	FOnInstanceLayoutChanged OnInstancedPropertyBagLayoutChanged;
#endif
} // PCGDelegates

namespace PCGPinIdHelpers
{
	FPCGPinId NodeIdAndPinIndexToPinId(FPCGTaskId NodeId, uint64 PinIndex)
	{
		// Construct a unique ID from node index and pin index.
		ensure(PinIndex < PCGPinIdHelpers::MaxOutputPins);
		return (NodeId * PCGPinIdHelpers::PinActiveBitmaskSize) + (PinIndex % PCGPinIdHelpers::PinActiveBitmaskSize);
	}

	FPCGPinId NodeIdToPinId(FPCGTaskId NodeId)
	{
		// Use (max pins - 1) to make a special pin ID that is not associated to a specific node pin.
		return (NodeId * PCGPinIdHelpers::PinActiveBitmaskSize) + PCGPinIdHelpers::MaxOutputPins;
	}

	FPCGPinId OffsetNodeIdInPinId(FPCGPinId PinId, uint64 NodeIDOffset)
	{
		return NodeIDOffset * PCGPinIdHelpers::PinActiveBitmaskSize + PinId;
	}

	FPCGTaskId GetNodeIdFromPinId(FPCGPinId PinId)
	{
		return PinId / PCGPinIdHelpers::PinActiveBitmaskSize;
	}

	uint64 GetPinIndexFromPinId(FPCGPinId PinId)
	{
		return PinId % PCGPinIdHelpers::PinActiveBitmaskSize;
	}
}

namespace PCGPointCustomPropertyNames
{
	bool IsCustomPropertyName(FName Name)
	{
		return Name == PCGPointCustomPropertyNames::ExtentsName ||
			Name == PCGPointCustomPropertyNames::LocalCenterName ||
			Name == PCGPointCustomPropertyNames::PositionName ||
			Name == PCGPointCustomPropertyNames::RotationName ||
			Name == PCGPointCustomPropertyNames::ScaleName ||
			Name == PCGPointCustomPropertyNames::LocalSizeName ||
			Name == PCGPointCustomPropertyNames::ScaledLocalSizeName;
	}

	const FName ExtentsName = TEXT("Extents");
	const FName LocalCenterName = TEXT("LocalCenter");
	const FName PositionName = TEXT("Position");
	const FName RotationName = TEXT("Rotation");
	const FName ScaleName = TEXT("Scale");
	const FName LocalSizeName = TEXT("LocalSize");
	const FName ScaledLocalSizeName = TEXT("ScaledLocalSize");
}

namespace PCGAttributeNameConstants
{
	const FLazyName ActorReferenceAttribute = "ActorReference";
	const FLazyName ComponentReferenceAttribute = "ComponentReference";
}

namespace PCG::Private
{
	const FString UserParameterTagData = TEXT("PCGUserParametersTagData");
}

namespace PCGPinConstants
{
	const FName DefaultInputLabel = TEXT("In");
	const FName DefaultOutputLabel = TEXT("Out");
	const FName DefaultParamsLabel = TEXT("Overrides");
	UE_DEPRECATED(5.6, "Please use 'DefaultExecutionDependencyLabel' instead.")
	const FName DefaultDependencyOnlyLabel = TEXT("Dependency Only");
	const FName DefaultExecutionDependencyLabel = TEXT("Execution Dependency");

	const FName DefaultInFilterLabel = TEXT("InsideFilter");
	const FName DefaultOutFilterLabel = TEXT("OutsideFilter");

	namespace Private
	{
		const FName OldDefaultParamsLabel = TEXT("Params");
	}

	namespace Icons
	{
		const FName LoopPinIcon = TEXT("GraphEditor.Macro.Loop_16x");
		const FName FeedbackPinIcon = TEXT("GraphEditor.GetSequenceBinding");
	}

#if WITH_EDITOR
	namespace Tooltips
	{
		const FText ExecutionDependencyTooltip = NSLOCTEXT("PCGCommon", "ExecutionDependencyPinTooltip", "Data passed to this pin will be used to order execution but will otherwise not contribute to the results of this node.");
	}
#endif // WITH_EDITOR
} // namespace PCGPinConstants

namespace PCGNodeConstants::Icons
{
	const FName CompactNodeConvert = TEXT("PCG.Node.Compact.Convert");
	const FName CompactNodeFilter = TEXT("PCG.Node.Compact.Filter");
}

// Metadata used by PCG
namespace PCGObjectMetadata
{
	const FLazyName Overridable = TEXT("PCG_Overridable");
	const FLazyName OverridableCPUAndGPU = TEXT("PCG_OverridableCPUAndGPU");
	const FLazyName OverridableCPUAndGPUWithReadback = TEXT("PCG_OverridableCPUAndGPUWithReadback");
	const FLazyName NotOverridable = TEXT("PCG_NotOverridable");
	const FLazyName OverridableChildProperties = TEXT("PCG_OverridableChildProperties");
	const FLazyName OverrideAliases = TEXT("PCG_OverrideAliases");
	const FLazyName DiscardPropertySelection = TEXT("PCG_DiscardPropertySelection");
	const FLazyName DiscardExtraSelection = TEXT("PCG_DiscardExtraSelection");
	const FLazyName EnumMetadataDomain = TEXT("PCG_MetadataDomain");
	const FLazyName PropertyReadOnly = TEXT("PCG_PropertyReadOnly");
	const FLazyName DataTypeIdentifierSupportsComposition = TEXT("PCG_SupportsComposition");
	const FLazyName DataTypeIdentifierFilter = TEXT("PCG_TypeFilter");
	const FLazyName DataTypeDisplayName = TEXT("PCG_DataTypeDisplayName");
}

namespace PCGQualityHelpers
{
	const FName PinLabelDefault = TEXT("Default");
	const FName PinLabelLow = TEXT("Low");
	const FName PinLabelMedium = TEXT("Medium");
	const FName PinLabelHigh = TEXT("High");
	const FName PinLabelEpic = TEXT("Epic");
	const FName PinLabelCinematic = TEXT("Cinematic");

	FName GetQualityPinLabel()
	{
		const int32 QualityLevel = UPCGSubsystem::GetPCGQualityLevel();
		FName SelectedPinLabel = NAME_None;

		switch (QualityLevel)
		{
			case 0: // Low Quality
				SelectedPinLabel = PinLabelLow;
				break;
			case 1: // Medium Quality
				SelectedPinLabel = PinLabelMedium;
				break;
			case 2: // High Quality
				SelectedPinLabel = PinLabelHigh;
				break;
			case 3: // Epic Quality
				SelectedPinLabel = PinLabelEpic;
				break;
			case 4: // Cinematic Quality
				SelectedPinLabel = PinLabelCinematic;
				break;
			default: // Default to Low Quality if we don't have a valid quality level
				SelectedPinLabel = PinLabelDefault;
		}

		return SelectedPinLabel;
	}
}

FPCGStackHandle::FPCGStackHandle(TSharedPtr<const FPCGStackContext> InStackContext, int32 InStackIndex)
	: StackContext(InStackContext)
	, StackIndex(InStackIndex)
{
}

bool FPCGStackHandle::IsValid() const
{
	return StackContext && StackIndex >= 0 && StackIndex < StackContext->GetNumStacks();
}

const FPCGStack* FPCGStackHandle::GetStack() const
{
	return IsValid() ? StackContext->GetStack(StackIndex) : nullptr;
}

