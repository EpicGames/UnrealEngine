// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGComponent.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGEngineSettings.h"
#include "PCGGraph.h"
#include "PCGInputOutputSettings.h"
#include "PCGManagedResource.h"
#include "PCGPin.h"
#include "PCGSubgraph.h"
#include "PCGSubsystem.h"
#include "Data/PCGIntersectionData.h"
#include "Data/PCGLandscapeData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGUnionData.h"
#include "Graph/PCGGraphPerExecutionCache.h"
#include "Graph/PCGStackContext.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGBlueprintHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "RuntimeGen/PCGRuntimeGenScheduler.h"
#include "RuntimeGen/GenSources/PCGGenSourceBase.h"
#include "RuntimeGen/SchedulingPolicies/PCGSchedulingPolicyBase.h"
#include "RuntimeGen/SchedulingPolicies/PCGSchedulingPolicyDistanceAndDirection.h"
#include "Utils/PCGGeneratedResourcesLogging.h"
#include "Utils/PCGGraphExecutionLogging.h"

#include "CoreGlobals.h"
#include "LandscapeProxy.h"
#include "Algo/AnyOf.h"
#include "Algo/IndexOf.h"
#include "Algo/Transform.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Interfaces/ITargetPlatform.h"
#include "Kismet/GameplayStatics.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGComponent)

#if WITH_EDITOR
#include "Editor.h"
#include "EditorActorFolders.h"
#include "ScopedTransaction.h"
#include "Editor/EditorEngine.h"
#include "Editor/Transactor.h"
#include "Misc/ITransactionObjectAnnotation.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectGlobals.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#endif

#define LOCTEXT_NAMESPACE "UPCGComponent"

namespace PCGComponent
{
	template <typename DelegateType, typename ...Args>
	static void BroadcastDynamicDelegate(const DelegateType& Delegate, UPCGComponent* PCGComponent, Args&& ...InArgs)
	{
#if WITH_EDITOR
		const TGuardValue ScriptExecutionGuard(GAllowActorScriptExecutionInEditor, true);
#endif // WITH_EDITOR
		Delegate.Broadcast(PCGComponent, std::forward<Args>(InArgs)...);
	}

	static TAutoConsoleVariable<bool> CVarDisableStealingForFlattenInPostProcessGraph(
		TEXT("pcg.Advanced.DisableStealingForFlattenInPostProcessGraph"),
		false,
		TEXT("At the end of execution, generated output data is flattened in place if it can be stolen, but it can be disabled if it has unwanted side effects."));

	static TAutoConsoleVariable<bool> CVarConvertToPointDataInPostProcessGraph(
		TEXT("pcg.Advanced.ConvertToPointDataInPostProcessGraph"),
		false,
		TEXT("At the end of execution, generated output data will be converted from UPCGPointArrayData to UPCGPointData."));

	static TAutoConsoleVariable<bool> CVarDeletePreviewResourcesWhenUnloading(
		TEXT("pcg.Advanced.DeletePreviewResourcesWhenUnloading"),
		true,
		TEXT("Delete generated preview resources (actors) when PCG Component gets unloaded."));

#if WITH_EDITOR
	static TAutoConsoleVariable<bool> CVarEnableLocalRefreshForPCGComponentDependencies(
		TEXT("pcg.EnableLocalRefreshForPCGComponentDependencies"),
		true,
		TEXT("When a partitioned component depends on another partitioned component, only react to local changes. Can be disabled if it has unwanted side effects."));
#endif // WITH_EDITOR

	[[nodiscard]] double RoundTrackingPriority(double InTrackingPriority)
	{
		// Round to 4 decimals (10^4)
		constexpr double Factor = 10000;
		return FMath::RoundToDouble(InTrackingPriority * Factor) / Factor;
	}

	EPCGHiGenGrid GetUnscaledGenerationGrid(uint32 ScaledGridSize, const UPCGGraph* Graph)
	{
		const double GridScale = Graph ? Graph->GetGridSizeMultiplier() : 1.0;
		return PCGHiGenGrid::ScaledGridSizeToGrid(ScaledGridSize, GridScale);
	}
}

UPCGComponent::UPCGComponent(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	SetFlags(RF_Transactional);
	ExecutionState.Component = this;
	GraphInstance = InObjectInitializer.CreateDefaultSubobject<UPCGGraphInstance>(this, TEXT("PCGGraphInstance"));
	SchedulingPolicyClass = UPCGSchedulingPolicyDistanceAndDirection::StaticClass();

#if WITH_EDITOR
	// If we are in Editor, and we are a BP template (no owner), we will mark this component to force a generate when added to world.
	if (!PCGHelpers::IsRuntimeOrPIE() && !GetOwner() && !HasAnyFlags(RF_ClassDefaultObject))
	{
		bForceGenerateOnBPAddedToWorld = true;
	}
#endif // WITH_EDITOR
}

bool UPCGComponent::CanPartition() const
{
	return Cast<APCGPartitionActor>(GetOwner()) == nullptr;
}

bool UPCGComponent::IsPartitioned() const
{
	return bIsComponentPartitioned && CanPartition();
}

void UPCGComponent::SetIsPartitioned(bool bIsNowPartitioned)
{
	if (bIsNowPartitioned == bIsComponentPartitioned)
	{
		return;
	}
	
	// Allow BP class to set the flag. BP class won't have an owner. They won't get registered anyway.
	if (!GetOwner())
	{
		bIsComponentPartitioned = bIsNowPartitioned;
		return;
	}

	bool bDoActorMapping = bGenerated || PCGHelpers::IsRuntimeOrPIE();

	if (UPCGSubsystem* Subsystem = GetSubsystem())
	{
		if (bGenerated)
		{
			CleanupLocalImmediate(/*bRemoveComponents=*/true);
		}

		// Update the component on the subsystem
		bIsComponentPartitioned = bIsNowPartitioned;
		Subsystem->RegisterOrUpdateExecutionSource(this, bDoActorMapping);
	}
	else
	{
		bIsComponentPartitioned = false;
	}
}

bool UPCGComponent::Use2DGrid() const
{
	if (UPCGGraph* PCGGraph = GetGraph())
	{
		return PCGGraph->Use2DGrid();
	}

	return GetDefault<UPCGGraph>()->Use2DGrid();
}

FPCGGridDescriptor UPCGComponent::GetGridDescriptor(uint32 GridSize) const
{
	return GetGridDescriptorInternal(GridSize, /*bRuntimeHashUpdate=*/false);
}

FPCGGridDescriptor UPCGComponent::GetGridDescriptorInternal(uint32 GridSize, bool bRuntimeHashUpdate) const
{
	// Return owner descriptor in case of Partition Actors
	if (APCGPartitionActor* PartitionActorOwner = Cast<APCGPartitionActor>(GetOwner()))
	{
		const FPCGGridDescriptor GridDescriptor = PartitionActorOwner->GetGridDescriptor();
		// If this is a local component, we only serve grid descriptors of the same grid size.
		check(GridSize == GridDescriptor.GetGridSize());

		return GridDescriptor;
	}

	FPCGGridDescriptor PCGGridDescriptor = FPCGGridDescriptor()
		.SetGridSize(GridSize)
		.SetIs2DGrid(Use2DGrid())
		.SetIsRuntime(IsManagedByRuntimeGenSystem());

	// Avoid setting this for runtime components. They do not support being assigned a DataLayers and/or HLOD Layer
	// They also do not need a RuntimeGridDescriptorHash which exists only because of DataLayers/HLODLayers
	if (!IsManagedByRuntimeGenSystem())
	{
#if WITH_EDITORONLY_DATA
		// Only return the RuntimeGridDescriptorHash for PIE Worlds and not when we are updating the Runtime Hash
		if (GetWorld() && GetWorld()->IsPlayInEditor() && !bRuntimeHashUpdate)
		{
			PCGGridDescriptor.SetRuntimeHash(RuntimeGridDescriptorHash);
		}
		else
		{
			PCGGridDescriptor.SetDataLayerAssets(GetOwner()->GetDataLayerAssets());
			PCGGridDescriptor.SetHLODLayer(GetOwner()->GetHLODLayer());
		}
#else
		PCGGridDescriptor.SetRuntimeHash(RuntimeGridDescriptorHash);
#endif
	}

	return PCGGridDescriptor;
}

void UPCGComponent::SetGraph_Implementation(UPCGGraphInterface* InGraph)
{
	SetGraphInterfaceLocal(InGraph);
}

UPCGGraph* UPCGComponent::GetGraph() const
{
	return (GraphInstance ? GraphInstance->GetGraph() : nullptr);
}

void UPCGComponent::SetGraphLocal(UPCGGraphInterface* InGraph)
{
	SetGraphInterfaceLocal(InGraph);
}

void UPCGComponent::SetGraphInterfaceLocal(UPCGGraphInterface* InGraphInterface)
{
	if (ensure(GraphInstance))
	{
		GraphInstance->SetGraph(InGraphInterface);
		RefreshAfterGraphChanged(GraphInstance, EPCGChangeType::Structural | EPCGChangeType::GenerationGrid);
	}
}

void UPCGComponent::AddToManagedResources(UPCGManagedResource* InResource)
{
	FPCGManagedResourceContainerHelper ContainerHelper(this);
	ContainerHelper.AddManagedResource(InResource);
}

void UPCGComponent::OnManagedResourceAdded(UPCGManagedResource* InResource)
{
	if (InResource)
	{
		if (!ensure(InResource->GetOuter() == this))
		{
			UPCGBlueprintHelpers::ThrowBlueprintException(LOCTEXT("ResourceNotOutered", "Managed resources need to be outered to their PCG component."));
		}
	}
}

void UPCGComponent::AddComponentsToManagedResources(const TArray<UActorComponent*>& InComponents)
{
	if (InComponents.IsEmpty())
	{
		return;
	}

	for (UActorComponent* Component : InComponents)
	{
		if (Component && !Component->ComponentHasTag(PCGHelpers::DefaultPCGTag))
		{
			Component->Modify();
			Component->ComponentTags.Add(PCGHelpers::DefaultPCGTag);
		}
	}

	FPCGManagedResourceContainerHelper ContainerHelper(this);
	UE::TScopeLock ResourcesLock(GeneratedResourcesLock);
	ensure(ContainerHelper.AreResourcesAccessible());

	UPCGManagedComponentDefaultList* DefaultList = nullptr;
	for (const TObjectPtr<UPCGManagedResource>& ManagedResource : ContainerHelper.GetConstManagedResourcesNoLock())
	{
		if (UPCGManagedComponentDefaultList* ExistingList = Cast<UPCGManagedComponentDefaultList>(ManagedResource))
		{
			DefaultList = ExistingList;
			break;
		}
	}

	if (!DefaultList)
	{
		DefaultList = NewObject<UPCGManagedComponentDefaultList>(this);
		ContainerHelper.AddManagedResourceNoLock(DefaultList);
	}

	check(DefaultList);
	// Implementation note: we call the AddGeneratedComponentsFromBP method to make sure that if this is done from BP, the construction method is properly updated

	TArray<TSoftObjectPtr<UActorComponent>> Components;
	Algo::Transform(InComponents, Components, [](UActorComponent* Component) { return TSoftObjectPtr<UActorComponent>(Component); });

	DefaultList->AddGeneratedComponentsFromBP(Components);
}

void UPCGComponent::AddActorsToManagedResources(const TArray<AActor*>& InActors)
{
	TArray<TSoftObjectPtr<AActor>> ValidActors;

	for (AActor* Actor : InActors)
	{
		if (Actor)
		{
			if (!Actor->Tags.Contains(PCGHelpers::DefaultPCGActorTag))
			{
				Actor->Modify();
				Actor->Tags.Add(PCGHelpers::DefaultPCGActorTag);
			}

			ValidActors.AddUnique(Actor);
		}
	}

	if (ValidActors.IsEmpty())
	{
		return;
	}

	UPCGManagedActors* ManagedResource = NewObject<UPCGManagedActors>(this);
	ManagedResource->GetMutableGeneratedActors() = std::move(ValidActors);

	AddToManagedResources(ManagedResource);
}

bool UPCGComponent::AreManagedResourcesAccessible() const
{
	const FPCGManagedResourceConstContainerHelper ContainerHelper(this);
	return ContainerHelper.AreResourcesAccessible();
}

void UPCGComponent::ForEachManagedResource(TFunctionRef<void(UPCGManagedResource*)> InFunction)
{
	FPCGManagedResourceContainerHelper ContainerHelper(this);
	ContainerHelper.ForEachManagedResource(InFunction);
}

void UPCGComponent::ForEachConstManagedResource(TFunctionRef<void(const UPCGManagedResource*)> InFunction) const
{
	const FPCGManagedResourceConstContainerHelper ContainerHelper(this);
	ContainerHelper.ForEachConstManagedResource(InFunction);
}

bool UPCGComponent::IsAnyObjectManagedByResource(const TArrayView<const UObject*> InObjects) const
{
	const FPCGManagedResourceConstContainerHelper ContainerHelper(this);
	return ContainerHelper.IsAnyObjectManagedByResource(InObjects);
}

bool UPCGComponent::ShouldGenerate(bool bForce, EPCGComponentGenerationTrigger RequestedGenerationTrigger) const
{
	if (!bActivated || !GetGraph() || !GetSubsystem())
	{
		return false;
	}

	if (IsManagedByRuntimeGenSystem())
	{
		// If we're runtime generated, turn down other requests.
		const bool bShouldGenerate = RequestedGenerationTrigger == EPCGComponentGenerationTrigger::GenerateAtRuntime;
		if (!bShouldGenerate)
		{
			UE_LOGF(LogPCG, Warning, "Generation request with trigger %d denied as this component is managed by the runtime generation scheduler.", (int)RequestedGenerationTrigger);
		}

		return bShouldGenerate;
	}

#if WITH_EDITOR
	// Always run Generate if we are in editor and partitioned since the original component doesn't know the state of the local one.
	if (IsPartitioned() && !PCGHelpers::IsRuntimeOrPIE())
	{
		return true;
	}
#endif

	// Always generate if procedural ISMs are being used, because the instance data is not persistent, and is currently lost regularly when the GPU Scene is flushed.
	if (bProceduralInstancesInUse)
	{
		return true;
	}

	// A request is invalid only if it was requested "GenerateOnLoad", but it is "GenerateOnDemand"
	// Meaning that all "GenerateOnDemand" requests are always valid, and "GenerateOnLoad" request is only valid if we want a "GenerateOnLoad" trigger.
	const bool bValidRequest = !(RequestedGenerationTrigger == EPCGComponentGenerationTrigger::GenerateOnLoad && GenerationTrigger == EPCGComponentGenerationTrigger::GenerateOnDemand);
	
	// Consider the component Generated only if it is not currently cleaning up
	const bool bConsiderGenerated = bGenerated && !IsCleaningUp();

	return ((!bConsiderGenerated && bValidRequest) ||
#if WITH_EDITOR
			bDirtyGenerated || 
#endif
			bForce);
}

void UPCGComponent::SetPropertiesFromOriginal(const UPCGComponent* Original)
{
	check(Original);

	EPCGComponentInput NewInputType = Original->InputType;

	// If we're inheriting properties from another component that would have targeted a "special" actor
	// then we must make sure we update the InputType appropriately
	if (NewInputType == EPCGComponentInput::Actor)
	{
		if(Cast<ALandscapeProxy>(Original->GetOwner()) != nullptr && Cast<ALandscapeProxy>(GetOwner()) == nullptr)
		{
			NewInputType = EPCGComponentInput::Landscape;
		}
	}

	if (!ensure(GraphInstance))
	{
		return;
	}

	// Make sure the tags match
	ComponentTags = Original->ComponentTags;

	const bool bGraphInstanceIsDifferent = !GraphInstance->IsEquivalent(Original->GraphInstance);

#if WITH_EDITOR
	const bool bHasDirtyInput = InputType != NewInputType;
	bool bIsDirty = bHasDirtyInput || bGraphInstanceIsDifferent;
#endif // WITH_EDITOR

	InputType = NewInputType;
	Seed = Original->Seed;
	GenerationTrigger = Original->GenerationTrigger;
	bOverrideGenerationRadii = Original->bOverrideGenerationRadii;
	GenerationRadii = Original->GenerationRadii;
	bActivated = Original->bActivated;

	// Make sure the Graph instance on the local component is the same than the graph instance on the original component
	// (Same graph and same parameter overrides)
	UPCGGraphInterface* OriginalGraph = Original->GraphInstance ? Original->GraphInstance->Graph : nullptr;
	if (OriginalGraph != GraphInstance->Graph)
	{
		GraphInstance->SetGraph(OriginalGraph);
	}

	if (bGraphInstanceIsDifferent && OriginalGraph)
	{
		GraphInstance->CopyParameterOverrides(Original->GraphInstance);
	}

	SchedulingPolicyClass = Original->SchedulingPolicyClass;
	RefreshSchedulingPolicy();

	if (SchedulingPolicy && ensure(Original->SchedulingPolicy) && !SchedulingPolicy->IsEquivalent(Original->SchedulingPolicy))
	{
		UEngine::CopyPropertiesForUnrelatedObjects(Original->SchedulingPolicy, SchedulingPolicy);

#if WITH_EDITOR
		bIsDirty = true;
#endif
	}

#if WITH_EDITOR
	// Note that while we dirty here, we won't trigger a refresh since we don't have the required context
	if (bIsDirty)
	{
		Modify(!IsInPreviewMode());
		DirtyGenerated(bHasDirtyInput ? EPCGComponentDirtyFlag::Input : EPCGComponentDirtyFlag::None);
	}
#endif
}

void UPCGComponent::Generate()
{
	if (IsGenerating())
	{
		return;
	}

	GenerateLocal(/*bForce=*/false);
}

void UPCGComponent::Generate_Implementation(bool bForce)
{
	GenerateLocal(bForce);
}

void UPCGComponent::GenerateLocal(bool bForce)
{
	GenerateLocalGetTaskId(bForce);
}

void UPCGComponent::GenerateLocal(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, uint32 Grid, const TArray<FPCGTaskId>& Dependencies)
{
	GenerateInternal(bForce, Grid, RequestedGenerationTrigger, Dependencies);
}

// Deprecated 5.8
void UPCGComponent::GenerateLocal(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, EPCGHiGenGrid Grid, const TArray<FPCGTaskId>& Dependencies)
{
	GenerateInternal(bForce, PCGHiGenGrid::GridToGridSize(Grid), RequestedGenerationTrigger, Dependencies);
}

FPCGTaskId UPCGComponent::GenerateLocalGetTaskId(bool bForce)
{
	return GenerateInternal(bForce, PCGHiGenGrid::UninitializedGridSize(), EPCGComponentGenerationTrigger::GenerateOnDemand, {});
}

FPCGTaskId UPCGComponent::GenerateLocalGetTaskId(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, uint32 Grid)
{
	return GenerateInternal(bForce, Grid, RequestedGenerationTrigger, {});
}

FPCGTaskId UPCGComponent::GenerateLocalGetTaskId(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, uint32 Grid, const TArray<FPCGTaskId>& Dependencies)
{
	return GenerateInternal(bForce, Grid, RequestedGenerationTrigger, Dependencies);
}

// Deprecated 5.8
FPCGTaskId UPCGComponent::GenerateLocalGetTaskId(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, EPCGHiGenGrid Grid)
{
	return GenerateInternal(bForce, PCGHiGenGrid::GridToGridSize(Grid), RequestedGenerationTrigger, {});
}

// Deprecated 5.8
FPCGTaskId UPCGComponent::GenerateLocalGetTaskId(EPCGComponentGenerationTrigger RequestedGenerationTrigger, bool bForce, EPCGHiGenGrid Grid, const TArray<FPCGTaskId>& Dependencies)
{
	return GenerateInternal(bForce, PCGHiGenGrid::GridToGridSize(Grid), RequestedGenerationTrigger, Dependencies);
}

FPCGTaskId UPCGComponent::GenerateInternal(bool bForce, uint32 Grid, EPCGComponentGenerationTrigger RequestedGenerationTrigger, const TArray<FPCGTaskId>& Dependencies)
{
	if (IsGenerating() || !GetSubsystem() || !ShouldGenerate(bForce, RequestedGenerationTrigger))
	{
		return InvalidPCGTaskId;
	}

	Modify(!IsInPreviewMode());

#if WITH_EDITOR
	LocalChangedBounds = FBox(EForceInit::ForceInit);
#endif // WITH_EDITOR

	// Clear prior to generation.
	bProceduralInstancesInUse = false;

	CurrentGenerationTask = GetSubsystem()->ScheduleComponent(this, Grid, bForce, Dependencies);

	if (CurrentGenerationTask != InvalidPCGTaskId)
	{
		ClearPerPinGeneratedOutput();

#if WITH_EDITOR
		PCG_EXECUTION_CACHE_VALIDATION_CREATE_SCOPE(this);
		// Notify Subsystem first
		GetSubsystem()->OnPCGGraphStartGenerating(this);
#endif // WITH_EDITOR
		// Notify Delegate next
		OnPCGGraphStartGeneratingDelegate.Broadcast(this);

		PCGComponent::BroadcastDynamicDelegate(OnPCGGraphStartGeneratingExternal, this);
	}

	return CurrentGenerationTask;
}

FPCGTaskId UPCGComponent::CreateGenerateTask(bool bForce, const TArray<FPCGTaskId>& Dependencies)
{
	if (IsGenerating())
	{
		return InvalidPCGTaskId;
	}

#if WITH_EDITOR
	// TODO: Have a better way to know when we need to generate a new seed.
	//if (bForce && bGenerated && !bDirtyGenerated)
	//{
	//	++Seed;
	//}
#endif

	// Keep track of all the dependencies
	TArray<FPCGTaskId> AdditionalDependencies;
	const TArray<FPCGTaskId>* AllDependencies = &Dependencies;

	if (bGenerated)
	{
		CleanupLocal(/*bRemoveComponents=*/false, Dependencies);
	}

	if (IsCleaningUp())
	{
		AdditionalDependencies.Reserve(Dependencies.Num() + 1);
		AdditionalDependencies.Append(Dependencies);
		AdditionalDependencies.Add(CurrentCleanupTask);
		AllDependencies = &AdditionalDependencies;
	}

	const FBox NewBounds = GetGridBounds();
	if (!NewBounds.IsValid)
	{
		OnProcessGraphAborted();
		return InvalidPCGTaskId;
	}

#if WITH_EDITOR
	// No need for lock since it is not executed in parallel.
	ChangeTracker.ResetCurrentExecution();
#endif // WITH_EDITOR

	return GetSubsystem()->ScheduleGraph(this, *AllDependencies);
}

void UPCGComponent::PostProcessGraph(const FBox& InNewBounds, bool bInGenerated, FPCGContext* Context)
{
	PCGGraphExecutionLogging::LogPostProcessGraph(this);

	LastGeneratedBounds = InNewBounds;

	const bool bHadGeneratedOutputBefore = GeneratedGraphOutput.TaggedData.Num() > 0;

	CleanupUnusedManagedResources();

	ClearGraphGeneratedOutput();

#if WITH_EDITOR
	ResetIgnoredChangeOrigins(/*bLogIfAnyPresent=*/true);
#endif

	UPCGSubsystem* Subsystem = GetSubsystem();

	if (bInGenerated)
	{
		bGenerated = true;
#if WITH_EDITOR
		bWasGeneratedThisSession = true;
#endif

		CurrentGenerationTask = InvalidPCGTaskId;

		// After a successful generation, we also want to call PostGenerateFunctions
		// if we have any. We also need a context.

		if (Context)
		{
			// TODO: should we filter based on supported serialized types here?
			for (const FPCGTaggedData& TaggedData : Context->InputData.TaggedData)
			{
				// Proxies should never get cached on the component output. These can hold onto large chunks of video memory and are not
				// directly serializable.
				if (ensure(TaggedData.Data && TaggedData.Data->CanBeSerialized()))
				{
					// Visit the generated data; if the outer is the transient package or this component for all data, then
					// we don't need to duplicate the data and can change the outer & flatten the data without any additional copies.
					// Caveat: since that data could be in the cache currently, we shouldn't reouter it to a component that's not in the persistent level, otherwise when the
					// sublevel is unloaded, it could leak references.
					// We also need to duplicate the data if the data is used multiple times.
					bool bDataCanBeStolen = !TaggedData.bIsUsedMultipleTimes && !PCGComponent::CVarDisableStealingForFlattenInPostProcessGraph.GetValueOnAnyThread();
#if WITH_EDITOR
					bDataCanBeStolen &= (GetOwner() && GetOwner()->GetWorld() && (GetOwner()->GetLevel() == GetOwner()->GetWorld()->PersistentLevel));
#endif

					if (bDataCanBeStolen)
					{
						TaggedData.Data->VisitDataNetwork([this, &bDataCanBeStolen](const UPCGData* InData)
						{
							if (InData && InData->GetOuter() != GetTransientPackage() && InData->GetOuter() != this)
							{
								bDataCanBeStolen = false;
							}
						});
					}

					FPCGTaggedData OutputTaggedData = TaggedData;

					if (!bDataCanBeStolen)
					{
						if (UPCGData* DuplicatedData = TaggedData.Data->DuplicateData(Context))
						{
							OutputTaggedData.Data = DuplicatedData;
						}
						else
						{
							// Duplication failed, don't keep that data
							UE_LOGF(LogPCG, Warning, "Failed data duplication in the PostProcessGraph - will be missing from the generated output data.");
							continue;
						}
					}
					// TODO: instead of doing this (moving to transient then moving back), it might be better to flow down a 'bMarkDirty' in the Flatten call.
					else if(IsInPreviewMode())
					{
						// As a safety procedure, we're going to move all the data back to the transient package prior to doing the flatten,
						// since it will dirty the source package otherwise
						OutputTaggedData.Data->VisitDataNetwork([this](const UPCGData* InData)
						{
							if (InData)
							{
								const_cast<UPCGData*>(InData)->Rename(nullptr, GetTransientPackage(), IsInPreviewMode() ? REN_DoNotDirty : REN_None);
							}
						});
					}

					// Flatten data
					OutputTaggedData.Data->VisitDataNetwork([](const UPCGData* InData)
					{
						if (InData)
						{
							const_cast<UPCGData*>(InData)->Flatten();
						}
					});

					// Reouter data
					OutputTaggedData.Data->VisitDataNetwork([this](const UPCGData* InData)
					{
						if (InData)
						{
							InData->MarkUsage(EPCGDataUsage::ComponentOutputData);

							const_cast<UPCGData*>(InData)->Rename(nullptr, this, IsInPreviewMode() ? REN_DoNotDirty : REN_None);
						}
					});

					if (PCGComponent::CVarConvertToPointDataInPostProcessGraph.GetValueOnAnyThread())
					{
						// To be backward compatible with licensee code that might rely on this output being strongly typed to UPCGPointData
						// we convert UPCGPointArrayData back to UPCGPointData. This should be a temporary solution and licensees can convert their code to use UPCGBasePointData so
						// that it is compatible with both point data classes and disable that CVar.
						if (const UPCGPointArrayData* PointArrayData = Cast<UPCGPointArrayData>(OutputTaggedData.Data.Get()))
						{
							OutputTaggedData.Data = PointArrayData->ToPointData(Context);

							UE_LOGF(LogPCG, Warning, "UPCGPointArrayData was converted to UPCGPointData. Please update your code to support UPCGBasePointData if needed and set pcg.Advanced.ConvertToPointDataInPostProcessGraph to 0");
						}
					}

					// Finally add to the generated output collection
					GeneratedGraphOutput.TaggedData.Add(OutputTaggedData);
				}
			}

			// If the original component is partitioned, local components have to forward
			// their inputs, so that they can be gathered by the original component.
			// We don't have the info on the original component here, so forward for all
			// components.
			Context->OutputData = Context->InputData;
		}

#if WITH_EDITOR
		// Reset this flag to avoid re-generating on further refreshes.
		bForceGenerateOnBPAddedToWorld = false;

		bDirtyGenerated = false;

		bool bEnableLocalRefreshForPCGComponentDependencies = PCGComponent::CVarEnableLocalRefreshForPCGComponentDependencies.GetValueOnAnyThread();

		if (bEnableLocalRefreshForPCGComponentDependencies)
		{
			// We want to use only the local bounds if the original component was not executed.
			const UPCGGraph* Graph = GraphInstance ? GraphInstance->GetGraph() : nullptr;
			
			if (!ensure(Graph))
			{
				// Should never happen, but previous behavior if we were in this state would be to go through with the total bounds.
				bEnableLocalRefreshForPCGComponentDependencies = false;
			}
			else
			{
				const bool bOriginalExecuted = !IsPartitioned() || !Graph || (Graph->IsHierarchicalGenerationEnabled() && Graph->GetDefaultGridSize() == PCGHiGenGrid::UnboundedGridSize());
				if (IsLocalComponent() || bOriginalExecuted)
				{
					LocalChangedBounds = GetTotalBounds();
				}

				if (UPCGComponent* OriginalComponent = GetOriginalComponent(); OriginalComponent != this)
				{
					PCG::TScopeLock Lock(OriginalComponent->LocalChangedBoundsLock);
					OriginalComponent->LocalChangedBounds += LocalChangedBounds;
				}
			}
		}

		// Notify Subsystem first, if it exists
		if (Subsystem)
		{
			Subsystem->OnPCGGraphGenerated(this, bEnableLocalRefreshForPCGComponentDependencies ? LocalChangedBounds : TOptional<FBox>{});
		}
#endif

		// Notify Delegate next
		OnPCGGraphGeneratedDelegate.Broadcast(this);

#if WITH_EDITOR
		UpdateDynamicTracking();
#endif // WITH_EDITOR

		if (IsValidChecked(this) && Context)
		{
			CallPostGenerateFunctions(Context);
		}

		// If Generate function made the component invalid for whatever reason, we re-check if this is valid.
		if (IsValidChecked(this))
		{
			PCGComponent::BroadcastDynamicDelegate(OnPCGGraphGeneratedExternal, this);
		}
	}

	// Trigger notification - will be used by other tracking mechanisms
#if WITH_EDITOR
	const bool bHasGeneratedOutputAfter = GeneratedGraphOutput.TaggedData.Num() > 0;

	if (IsValidChecked(this) && (bHasGeneratedOutputAfter || bHadGeneratedOutputBefore))
	{
		FProperty* GeneratedOutputProperty = FindFProperty<FProperty>(UPCGComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UPCGComponent, GeneratedGraphOutput));
		check(GeneratedOutputProperty);
		FPropertyChangedEvent GeneratedOutputChangedEvent(GeneratedOutputProperty, EPropertyChangeType::ValueSet);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, GeneratedOutputChangedEvent);
	}

	StopGenerationInProgress();
#endif

#if PCG_PROFILING_ENABLED
	if (Subsystem)
	{
		Subsystem->OnPCGSourceGenerationDone(IsValidChecked(this) ? this : nullptr, EPCGGenerationStatus::Completed);
	}
#endif

#if WITH_EDITOR
	LocalChangedBounds = FBox(EForceInit::ForceInit);
#endif // WITH_EDITOR
}

void UPCGComponent::CallPostGenerateFunctions(FPCGContext* Context) const
{
	check(Context);

	if (AActor* Owner = GetOwner())
	{
		for (const FName& FunctionName : PostGenerateFunctionNames)
		{
			if (UFunction* PostGenerateFunc = Owner->GetClass()->FindFunctionByName(FunctionName))
			{
				// Validate that the function take the right number of arguments
				if (PostGenerateFunc->NumParms != 1)
				{
					UE_LOGF(LogPCG, Error, "[UPCGComponent] PostGenerateFunction \"%ls\" from actor \"%ls\" doesn't have exactly 1 parameter. Will skip the call.", *FunctionName.ToString(), *Owner->GetFName().ToString());
					continue;
				}

				bool bIsValid = false;
				TFieldIterator<FProperty> PropIterator(PostGenerateFunc);
				while (PropIterator)
				{
					if (!!(PropIterator->PropertyFlags & CPF_Parm))
					{
						if (FStructProperty* Property = CastField<FStructProperty>(*PropIterator))
						{
							if (Property->Struct == FPCGDataCollection::StaticStruct())
							{
								bIsValid = true;
								break;
							}
						}
					}

					++PropIterator;
				}

				if (bIsValid)
				{
					Owner->ProcessEvent(PostGenerateFunc, &Context->InputData);
				}
				else
				{
					UE_LOGF(LogPCG, Error, "[UPCGComponent] PostGenerateFunction \"%ls\" from actor \"%ls\" parameter type is not PCGDataCollection. Will skip the call.", *FunctionName.ToString(), *Owner->GetFName().ToString());
				}
			}
			else
			{
				UE_LOGF(LogPCG, Error, "[UPCGComponent] PostGenerateFunction \"%ls\" was not found in the component owner \"%ls\".", *FunctionName.ToString(), *Owner->GetFName().ToString());
			}
		}
	}
}

void UPCGComponent::PostCleanupGraph(bool bRemoveComponents)
{
	CurrentCleanupTask = InvalidPCGTaskId;
	
	if (!bRemoveComponents)
	{
		// If we didn't remove components, it's a shallow cleanup before generating, so early out here.
		return;
	}

	bGenerated = false;
	const bool bHadGeneratedGraphOutput = GeneratedGraphOutput.TaggedData.Num() > 0;

	ClearGraphGeneratedOutput();
	ClearPerPinGeneratedOutput();

#if WITH_EDITOR
	ChangeTracker.DynamicallyTrackedKeysToSettings.Reset();

	// Notify Subsystem first
	if (UPCGSubsystem* Subsystem = GetSubsystem())
	{
		Subsystem->OnPCGGraphCleaned(this);
	}
#endif

	// Notify Delegate next
	OnPCGGraphCleanedDelegate.Broadcast(this);

#if WITH_EDITOR
	bDirtyGenerated = false;

	if (bHadGeneratedGraphOutput)
	{
		FProperty* GeneratedOutputProperty = FindFProperty<FProperty>(UPCGComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UPCGComponent, GeneratedGraphOutput));
		check(GeneratedOutputProperty);
		FPropertyChangedEvent GeneratedOutputChangedEvent(GeneratedOutputProperty, EPropertyChangeType::ValueSet);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, GeneratedOutputChangedEvent);
	}
#endif

	PCGComponent::BroadcastDynamicDelegate(OnPCGGraphCleanedExternal, this);
}

void UPCGComponent::OnProcessGraphAborted(bool bQuiet, bool bCleanupUnusedResources)
{
	if (!bQuiet)
	{
		UE_LOGF(LogPCG, Warning, "Process Graph was called but aborted, check for errors in log if you expected a result.");
	}

#if WITH_EDITOR
	// On abort, there may be ignores still registered, silently remove these.
	ResetIgnoredChangeOrigins(/*bLogIfAnyPresent=*/false);

	LocalChangedBounds = FBox(EForceInit::ForceInit);
#endif

	if (bCleanupUnusedResources)
	{
		CleanupUnusedManagedResources();
	}

	CurrentGenerationTask = InvalidPCGTaskId;
	CurrentCleanupTask = InvalidPCGTaskId; // this is needed to support cancellation

#if PCG_PROFILING_ENABLED
	UPCGSubsystem* Subsystem = GetSubsystem();
#endif

#if WITH_EDITOR
	CurrentRefreshTask = InvalidPCGTaskId;
	// Implementation note: while it may seem logical to clear the bDirtyGenerated flag here, 
	// the component is still considered dirty if we aborted processing, hence it should stay this way.

	StopGenerationInProgress();

	// Notify Subsystem first
	if (Subsystem)
	{
		Subsystem->OnPCGGraphCancelled(this);
	}
#endif

	// Notify Delegate next
	OnPCGGraphCancelledDelegate.Broadcast(this);

#if PCG_PROFILING_ENABLED
	if (Subsystem)
	{
		Subsystem->OnPCGSourceGenerationDone(this, EPCGGenerationStatus::Aborted);
	}
#endif

	PCGComponent::BroadcastDynamicDelegate(OnPCGGraphCancelledExternal, this);
}

void UPCGComponent::Cleanup()
{
	if (IsManagedByRuntimeGenSystem())
	{
		UE_LOGF(LogPCG, Warning, "Cleanup request denied as this component is managed by the runtime generation scheduler.");
		return;
	}

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("PCGCleanup", "Clean up PCG component"));
#endif

	CleanupLocal(/*bRemoveComponents=*/true);
}

void UPCGComponent::Cleanup_Implementation(bool bRemoveComponents)
{
	CleanupLocal(bRemoveComponents);
}

void UPCGComponent::PurgeUnlinkedResources(const AActor* InActor)
{
	const AActor* ThisActor = InActor;
	if (!ThisActor)
	{
		return;
	}

	TSet<TSoftObjectPtr<AActor>> ActorsToDelete;

	TArray<AActor*>AttachedActors;
	TArray<UActorComponent*> ActorComponentList;

	ThisActor->GetAttachedActors(AttachedActors);

	for (AActor* Actor : AttachedActors)
	{
		if (Actor && Actor->ActorHasTag(PCGHelpers::DefaultPCGActorTag))
		{
			ActorsToDelete.Add(Actor);
		}
	};

	// Cleanup any actor components with tag and not managed by any other components
	ThisActor->ForEachComponent(/*bIncludeFromChildActors=*/true, [&ActorComponentList](UActorComponent* ActorComponent)
	{
		if (ActorComponent && ActorComponent->ComponentHasTag(PCGHelpers::DefaultPCGTag))
		{
			ActorComponentList.Add(ActorComponent);
		}
	});
	
	bool bCanPurge = true;

	ThisActor->ForEachComponent<UPCGComponent>(/*bIncludeFromChildActors=*/true, [&ActorComponentList, &ActorsToDelete, &bCanPurge](UPCGComponent* Component)
	{
		if(!bCanPurge)
		{
			return;
		}

		UE::TScopeLock ResourcesLock(Component->GeneratedResourcesLock);
		FPCGManagedResourceContainerHelper ContainerHelper(Component);
		if(ensure(ContainerHelper.AreResourcesAccessible()))
		{
			for (UPCGManagedResource* ManagedResource : ContainerHelper.GetConstManagedResourcesNoLock())
			{
				if (UPCGManagedComponent* ManagedComponent = Cast<UPCGManagedComponent>(ManagedResource))
				{
					ActorComponentList.RemoveSwap(ManagedComponent->GeneratedComponent.Get());
				}
				else if (UPCGManagedActors* ManagedActors = Cast<UPCGManagedActors>(ManagedResource))
				{
					for (const TSoftObjectPtr<AActor>& GeneratedActor : ManagedActors->GetConstGeneratedActors())
					{
						ActorsToDelete.Remove(GeneratedActor);
					}
				}
			}
		}
		else
		{
			bCanPurge = false;
		}
	});

	if (!bCanPurge)
	{
		UE_LOGF(LogPCG, Error, "[UPCGComponent::PurgeUnlinkedResources] Coouldd not purge resources because some were inacessible.");
		return;
	}

	for (UActorComponent* ActorComponent : ActorComponentList)
	{
		ActorComponent->DestroyComponent();
	}

	if (UWorld* World = InActor->GetWorld())
	{
		UPCGActorHelpers::DeleteActors(World, ActorsToDelete.Array());
	}
}

void UPCGComponent::CleanupLocalDeleteAllGeneratedObjects(const TArray<FPCGTaskId>& Dependencies)
{
	UPCGSubsystem* Subsystem = GetSubsystem();
	if (!Subsystem)
	{
		return;
	}
	
	TArray<FPCGTaskId> TaskIds;
	
	auto SchedulePurge = [this, Subsystem, &TaskIds, &Dependencies](UPCGComponent* Component)
	{
		FPCGTaskId TaskId;
		TWeakObjectPtr<UPCGComponent> ScheduledComponent(Component);

		TaskId = Subsystem->ScheduleGeneric([this, ScheduledComponent]()
			{
				if (UPCGComponent* Component = ScheduledComponent.Get())
				{
					if (IsValid(Component))
					{
						Component->PurgeUnlinkedResources(this->GetOwner());
					}
				}

				return true;
			},
			this, Dependencies);

		if (TaskId != InvalidPCGTaskId)
		{
			TaskIds.Add(TaskId);
		}
	};
	
	SchedulePurge(this);

	if (IsPartitioned())
	{
		Subsystem->ForAllRegisteredLocalComponents(this, SchedulePurge);
	}

	CleanupLocal(/*bRemoveComponents=*/true, TaskIds);
}

void UPCGComponent::CleanupLocal(bool bRemoveComponents)
{
	CleanupLocal(bRemoveComponents, TArray<FPCGTaskId>());
}

FPCGTaskId UPCGComponent::CleanupLocal(bool bRemoveComponents, const TArray<FPCGTaskId>& Dependencies)
{
	UPCGSubsystem* Subsystem = GetSubsystem();

	if (!Subsystem || IsCleaningUp())
	{
		return InvalidPCGTaskId;
	}

	const FPCGManagedResourceConstContainerHelper ContainerHelper(this);
	const bool bNeedsLocalCleanup = bGenerated || IsGenerating() || !ContainerHelper.IsEmpty();
	if (!bNeedsLocalCleanup && Subsystem->GetPCGComponentPartitionActorMappings(this).IsEmpty())
	{
		return InvalidPCGTaskId;
	}

	PCGGeneratedResourcesLogging::LogCleanupLocal(this, bRemoveComponents);

	Modify(!IsInPreviewMode() && bNeedsLocalCleanup);

#if WITH_EDITOR
	ExtraCapture.ResetCapturedMessages();
#endif

	CurrentCleanupTask = Subsystem->ScheduleCleanup(this, bRemoveComponents, Dependencies);
	return CurrentCleanupTask;
}

void UPCGComponent::CancelGeneration()
{
	if (CurrentGenerationTask != InvalidPCGTaskId && GetSubsystem())
	{
		GetSubsystem()->CancelGeneration(this);
	}
}

void UPCGComponent::NotifyPropertiesChangedFromBlueprint()
{
#if WITH_EDITOR
	DirtyGenerated(EPCGComponentDirtyFlag::Actor);
	Refresh();
#endif
}

AActor* UPCGComponent::ClearPCGLink(UClass* TemplateActorClass)
{
	FPCGMoveResourceParams Params
	{
		.TemplateTargetClass = TemplateActorClass
	};

	return ClearPCGLink(Params);
}

AActor* UPCGComponent::ClearPCGLink(const FPCGMoveResourceParams& InParams)
{
	if (!bGenerated || !GetOwner() || !GetWorld())
	{
		return nullptr;
	}

	FPCGMoveResourceParams Params = InParams; // copied so we can affect the actor

	// TODO: Perhaps remove this part if we want to do it in the PCG Graph.
	if (IsGenerating() || IsCleaningUp())
	{
		return nullptr;
	}

	// Create target actor if needed.
	UWorld* World = GetWorld();
	AActor* NewActor = nullptr;

	if (!Params.Target)
	{
		FActorSpawnParameters ActorSpawnParams;
		UClass* SpawnClass = Params.GetTemplateClass<AActor>() ? Params.GetTemplateClass<AActor>().Get() : AActor::StaticClass();

		const FName DefaultName = Params.DefaultName == NAME_None ? TEXT("PCGStamp") : Params.DefaultName;

		ActorSpawnParams.Name = DefaultName;
		ActorSpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
		ActorSpawnParams.OverrideLevel = GetOwner()->GetLevel();

		UPCGActorHelpers::FSpawnDefaultActorParams SpawnDefaultActorParams(World, SpawnClass, GetOwner()->GetTransform(), ActorSpawnParams);
	
		if (Params.bAttachToParent)
		{
			SpawnDefaultActorParams.Parent = GetOwner();
		}

	#if WITH_EDITOR
		SpawnDefaultActorParams.DataLayerInstances = GetOwner()->GetDataLayerInstances();
		SpawnDefaultActorParams.HLODLayer = GetOwner()->GetHLODLayer();
	#endif

		// First create a new actor that will be the new owner of all the resources
		NewActor = UPCGActorHelpers::SpawnDefaultActor(SpawnDefaultActorParams);
	#if WITH_EDITOR
		if (NewActor)
		{
			FActorLabelUtilities::SetActorLabelUnique(NewActor, GetOwner()->GetActorLabel() + TEXT("_") + DefaultName.ToString());
			NewActor->SetFolderPath(GetOwner()->GetFolderPath());
		}
	#endif

		// Affect newly created actor as the target actor for the subsequent resource move.
		Params.Target = NewActor;
	}

	// Then move all resources linked to this component to this actor
	bool bHasMovedResources = MoveResourcesToNewActor(Params, /*bCreateChild=*/false);

	// And finally, if we are partitioned, we need to do the same for all PCGActors, in Editor only.
	if (IsPartitioned())
	{
#if WITH_EDITOR
		if (UPCGSubsystem* Subsystem = GetSubsystem())
		{
			Subsystem->ClearPCGLink(this, LastGeneratedBounds, Params);
		}
#endif // WITH_EDITOR
	}
	else
	{
		if (bHasMovedResources)
		{
#if WITH_EDITOR
			DirtyGenerated(EPCGComponentDirtyFlag::Input, /*bDispatchToLocalComponents=*/true);
#endif
		}
		else if(NewActor)
		{
			NewActor->Destroy();
			NewActor = nullptr;
		}
	}

#if WITH_EDITOR
	// If there is an associated generated folder from this actor, rename it according to the stamp name
	if (World && NewActor)
	{
		FString GeneratedFolderPath;
		PCGHelpers::GetGeneratedActorsFolderPath(GetOwner(), GeneratedFolderPath);
		
		FString GeneratedStampFolder;
		PCGHelpers::GetGeneratedActorsFolderPath(NewActor, GeneratedStampFolder);

		if (!GeneratedFolderPath.IsEmpty() && !GeneratedStampFolder.IsEmpty())
		{
			FFolder GeneratedFolder(FFolder::GetWorldRootFolder(World).GetRootObject(), *GeneratedFolderPath);
			FFolder StampFolder(FFolder::GetWorldRootFolder(World).GetRootObject(), *GeneratedStampFolder);

			const bool bGeneratedFolderExists = GeneratedFolder.IsValid() && FActorFolders::Get().ContainsFolder(*World, GeneratedFolder);
			const bool bStampFolderExists = FActorFolders::Get().ContainsFolder(*World, StampFolder);

			// TODO: improve behavior when target stamp folder would exist
			if (bGeneratedFolderExists && !bStampFolderExists)
			{
				FActorFolders::Get().RenameFolderInWorld(*World, GeneratedFolder, StampFolder);
			}
		}
	}
#endif

	PCGComponent::BroadcastDynamicDelegate(OnPCGLinkClearedExternal, this, NewActor);

	return NewActor;
}

EPCGHiGenGrid UPCGComponent::GetUnscaledGenerationGrid() const
{
	return PCGComponent::GetUnscaledGenerationGrid(GetGenerationGridSize(), GetGraph());
}

void UPCGComponent::StoreOutputDataForPin(const FString& InResourceKey, const FPCGDataCollection& InData)
{
	UE::TWriteScopeLock ScopedWriteLock(PerPinGeneratedOutputLock);

	InData.MarkUsage(EPCGDataUsage::ComponentPerPinOutputData);

	FPCGDataCollection* FoundExistingData = PerPinGeneratedOutput.Find(InResourceKey);

	// For all existing data items, clear their usage which may release transient resources if the data is not present in the new data collection.
	if (FoundExistingData)
	{
		for (const FPCGTaggedData& ExistingData : FoundExistingData->TaggedData)
		{
			if (ExistingData.Data && !InData.TaggedData.FindByPredicate([&ExistingData](const FPCGTaggedData& NewData) { return ExistingData.Data == NewData.Data; }))
			{
				ExistingData.Data->ClearUsage(EPCGDataUsage::ComponentPerPinOutputData);
			}
		}

		*FoundExistingData = InData;
	}
	else
	{
		PerPinGeneratedOutput.Add(InResourceKey, InData);
	}
}

const FPCGDataCollection* UPCGComponent::RetrieveOutputDataForPin(const FString& InResourceKey)
{
	UE::TReadScopeLock ScopedReadLock(PerPinGeneratedOutputLock);
	return PerPinGeneratedOutput.Find(InResourceKey);
}

void UPCGComponent::ClearPerPinGeneratedOutput()
{
	UE::TWriteScopeLock ScopedWriteLock(PerPinGeneratedOutputLock);

	for (TPair<FString, FPCGDataCollection>& Entry : PerPinGeneratedOutput)
	{
		Entry.Value.ClearUsage(EPCGDataUsage::ComponentPerPinOutputData);
	}

	PerPinGeneratedOutput.Reset();
}

void UPCGComponent::SetSchedulingPolicyClass(TSubclassOf<UPCGSchedulingPolicyBase> InSchedulingPolicyClass) 
{
	SchedulingPolicyClass = InSchedulingPolicyClass;
	RefreshSchedulingPolicy();
}

const FPCGRuntimeGenerationRadii& UPCGComponent::GetGenerationRadii() const
{
	return (bOverrideGenerationRadii || !GetGraph()) ? GenerationRadii : GetGraph()->GenerationRadii;
}

double UPCGComponent::GetGenerationRadiusFromGrid(uint32 Grid) const
{
	const double Multiplier = FMath::Max(PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationRadiusMultiplier.GetValueOnAnyThread(), 0.0f);

	const UPCGGraph* Graph = GetGraph();
	ensure(Graph);

	if (bOverrideGenerationRadii)
	{
		const EPCGHiGenGrid UnscaledGrid = PCGComponent::GetUnscaledGenerationGrid(Grid, GetGraph());
		return Multiplier * GenerationRadii.GetGenerationRadiusFromGrid(UnscaledGrid);
	}
	else if (Graph)
	{
		return Multiplier * Graph->GetGridGenerationRadiusFromGrid(Grid);
	}

	return 0;
}

double UPCGComponent::GetCleanupRadiusFromGrid(uint32 Grid) const
{
	const double Multiplier = FMath::Max(PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationRadiusMultiplier.GetValueOnAnyThread(), 0.0f);

	const UPCGGraph* Graph = GetGraph();
	ensure(Graph);

	if (bOverrideGenerationRadii)
	{
		const EPCGHiGenGrid UnscaledGrid = PCGComponent::GetUnscaledGenerationGrid(Grid, GetGraph());
		return Multiplier * GenerationRadii.GetCleanupRadiusFromGrid(UnscaledGrid);
	}
	else if (Graph)
	{
		return Multiplier * Graph->GetGridCleanupRadiusFromGrid(Grid);
	}

	return 0;
}

bool UPCGComponent::UseActorComponentlessGeneration() const
{
	if (bOverrideUseActorComponentlessGeneration)
	{
		return bUseActorComponentlessGeneration;
	}

	return GetGraph() && GetGraph()->UseActorComponentlessGeneration();
}

bool UPCGComponent::MoveResourcesToNewActor(const FPCGMoveResourceParams& Params, bool bCreateChild)
{
	// Don't move resources if we are generating or cleaning up
	if (IsGenerating() || IsCleaningUp())
	{
		return false;
	}

	const AActor* Owner = GetOwner();
	if (!Owner)
	{
		UE_LOGF(LogPCG, Error, "[UPCGComponent::MoveResourcesToNewActor] Owner is null, child actor not created.");
		return false;
	}

	AActor* TargetActor = Params.GetTarget<AActor>();
	check(TargetActor);
	AActor* ParentActor = TargetActor;
	AActor* CreatedChildActor = nullptr;
	
	bool bHasMovedResources = false;

	Modify(!IsInPreviewMode());

#if WITH_EDITOR
	FName FolderPath;
#endif

	if (bCreateChild)
	{
		FActorSpawnParameters ActorSpawnParams;
		const FString DefaultName(TEXT("PCGStampChild"));
		ActorSpawnParams.Name = *DefaultName;
		ActorSpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
		ActorSpawnParams.OverrideLevel = Owner->GetLevel();

		UPCGActorHelpers::FSpawnDefaultActorParams SpawnDefaultActorParams(GetWorld(), ParentActor->GetClass(), Owner->GetTransform(), ActorSpawnParams);

#if WITH_EDITOR
		SpawnDefaultActorParams.DataLayerInstances = Owner->GetDataLayerInstances();
		SpawnDefaultActorParams.HLODLayer = Owner->GetHLODLayer();
#endif

		CreatedChildActor = UPCGActorHelpers::SpawnDefaultActor(SpawnDefaultActorParams);
#if WITH_EDITOR

		if (CreatedChildActor)
		{
			FActorLabelUtilities::SetActorLabelUnique(CreatedChildActor, DefaultName);
		}

		if (TargetActor->GetFolderPath() != NAME_None)
		{
			FolderPath = FName(*(ParentActor->GetFolderPath().ToString() / ParentActor->GetActorLabel()));
		}
		else
		{
			FolderPath = FName(*ParentActor->GetActorLabel());
		}
#endif
	}

	// Create a new move param set so we can retarget to the newly created actor.
	FPCGMoveResourceParams PotentiallyRetargetedParams = Params;
	PotentiallyRetargetedParams.Target = CreatedChildActor ? CreatedChildActor : ParentActor;
	PotentiallyRetargetedParams.ExpectedPreviousOwner = Owner;

#if WITH_EDITOR
	// Trying to move all resources for now. Perhaps in the future we won't want that.
	TSet<TSoftObjectPtr<AActor>> MovedActors;

	// Do not rely on CVar to load actors when Clearing PCG Link
	FPCGManagedActorLoadingScope Scope(/*bInShouldLoadActors=*/true);
#endif
	{
		UE::TScopeLock ResourcesLock(GeneratedResourcesLock);
		FPCGManagedResourceContainerHelper ContainerHelper(this);
		ensure(ContainerHelper.AreResourcesAccessible());
#if WITH_EDITOR
		Scope.AddResources(this, ContainerHelper.GetConstManagedResourcesNoLock());
#endif
		TArray<TSoftObjectPtr<AActor>> TentativeMovedGeneratedActors;
		const TArray<TObjectPtr<UPCGManagedResource>>& ManagedResources = ContainerHelper.GetConstManagedResourcesNoLock();

		for(int ResourceIndex = ManagedResources.Num() - 1; ResourceIndex >=0; --ResourceIndex)
		{
			TObjectPtr<UPCGManagedResource> GeneratedResource = ManagedResources[ResourceIndex];
			TentativeMovedGeneratedActors.Reset();

			if (GeneratedResource)
			{
#if WITH_EDITOR
				if (UPCGManagedActors* ManagedActors = Cast<UPCGManagedActors>(GeneratedResource))
				{
					TentativeMovedGeneratedActors = ManagedActors->GetConstGeneratedActors();
				}
#endif
				if (GeneratedResource->MoveResource(PotentiallyRetargetedParams))
				{
					TSet<TSoftObjectPtr<AActor>> Dummy;
					GeneratedResource->ReleaseIfUnused(Dummy);
					bHasMovedResources = true;

#if WITH_EDITOR
					if (!TentativeMovedGeneratedActors.IsEmpty())
					{
						MovedActors.Append(TentativeMovedGeneratedActors);
					}
#endif

					ContainerHelper.RemoveManagedResourceAtNoLock(ResourceIndex);
				}
			}
			else
			{
				UE_LOGF(LogPCG, Error, "[UPCGComponent::MoveResourcesToNewActor] Null generated resource encountered on actor \"%ls\" and will be skipped.", *Owner->GetFName().ToString());
			}
		}
	}

	// Move generated output if the params indicate to do so.
	if (Params.TargetDataCollection)
	{
		for(int DataIndex = GeneratedGraphOutput.TaggedData.Num() - 1; DataIndex >= 0; --DataIndex)
		{
			FPCGTaggedData OutputData = GeneratedGraphOutput.TaggedData[DataIndex];

			if (Params.Accepts(OutputData))
			{
				check(OutputData.Data);
				// Remove it from the generated graph output (incl. cache)
				OutputData.Data->ClearUsage(EPCGDataUsage::ComponentOutputData);
				GeneratedGraphOutput.TaggedData.RemoveAt(DataIndex);
				Params.TargetDataCollection->TaggedData.Add(OutputData);

				bHasMovedResources = true;
			}
		}
	}

	if (CreatedChildActor)
	{
		// No resources moved destroy stamp actor
		if (!bHasMovedResources)
		{
			GetWorld()->DestroyActor(CreatedChildActor);
			return false;
		}

		USceneComponent* RootComponent = CreatedChildActor->GetRootComponent();
		check(RootComponent);

		TArray<AActor*> AttachedActors;
		CreatedChildActor->GetAttachedActors(AttachedActors);

		// Set Folder Path if actor is not going to be deleted or if we have moved actors
		const bool bDestroyActor = RootComponent->GetNumChildrenComponents() == 0 && AttachedActors.IsEmpty();

#if WITH_EDITOR
		const bool bSetFolderPath = !MovedActors.IsEmpty() || !bDestroyActor;
		if (bSetFolderPath)
		{
			CreatedChildActor->SetFolderPath(FolderPath);
		}

		// Move Generated Actors
		if (!MovedActors.IsEmpty())
		{
			FString OutGeneratedFolderStr;
			PCGHelpers::GetGeneratedActorsFolderPath(CreatedChildActor, OutGeneratedFolderStr);
			FName GeneratedFolder(*OutGeneratedFolderStr);

			for (const TSoftObjectPtr<AActor>& MovedActorPtr : MovedActors)
			{
				if (AActor* MovedActor = MovedActorPtr.Get())
				{
					MovedActor->SetFolderPath(GeneratedFolder);
				}
			}
		}
#endif

		// No moved components destroy stamp actor
		if (bDestroyActor)
		{
			GetWorld()->DestroyActor(CreatedChildActor);
			return false;
		}
	}

	return bHasMovedResources;
}

void UPCGComponent::CleanupLocalImmediate(bool bRemoveComponents, bool bCleanupLocalComponents)
{
	FPCGManagedResourceContainerHelper ContainerHelper(this);
	PCGGeneratedResourcesLogging::LogCleanupLocalImmediate(this, bRemoveComponents, ContainerHelper.GetManagedResourcesCopy());

	UPCGSubsystem* Subsystem = GetSubsystem();

	// Cleanup Local should work even if we don't have any subsytem. In cook (or in other places), if Cleanup is necessary, we need to make sure to 
	// cleanup the managed resources on the component even if we don't have a subsystem. If we don't have a subsystem, assume we are unbounded.
	bool bHasUnbounded = true;

	if (Subsystem)
	{
		PCGHiGenGrid::FSizeArray GridSizes;
		ensure(PCGHelpers::GetGenerationGridSizes(GetGraph(), Subsystem->GetPCGWorldActor(), GridSizes, bHasUnbounded));
	}

	// Cancels generation of this component if there is an ongoing generation in progress.
	CancelGeneration();

	ContainerHelper.Cleanup(bRemoveComponents);

	PostCleanupGraph(bRemoveComponents);

	// If the component is partitioned, we will forward the calls to its local components.
	if (Subsystem && bCleanupLocalComponents && IsPartitioned())
	{
		Subsystem->CleanupLocalComponentsImmediate(this, bRemoveComponents);
	}

	PCGGeneratedResourcesLogging::LogCleanupLocalImmediateFinished(this, ContainerHelper.GetManagedResourcesCopy());
}

FPCGTaskId UPCGComponent::CreateCleanupTask(bool bRemoveComponents, const TArray<FPCGTaskId>& Dependencies)
{
	if (GetSubsystem() && GetSubsystem()->IsGraphCacheDebuggingEnabled())
	{
		UE_LOGF(LogPCG, Log, "[%ls] --- CLEANUP COMPONENT ---", GetOwner() ? *GetOwner()->GetName() : TEXT("MissingComponent"));
	}

	FPCGManagedResourceContainerHelper ContainerHelper(this);
	if ((!bGenerated && ContainerHelper.IsEmpty() && !IsGenerating()) || IsCleaningUp())
	{
		return InvalidPCGTaskId;
	}

	PCGGeneratedResourcesLogging::LogCreateCleanupTask(this, bRemoveComponents);

	// Keep track of all the dependencies
	TArray<FPCGTaskId> AdditionalDependencies;
	const TArray<FPCGTaskId>* AllDependencies = &Dependencies;

	if (IsGenerating())
	{
		AdditionalDependencies.Reserve(Dependencies.Num() + 1);
		AdditionalDependencies.Append(Dependencies);
		AdditionalDependencies.Add(CurrentGenerationTask);
		AllDependencies = &AdditionalDependencies;
	}

	TSharedPtr<FPCGManagedResourceContainerHelper::FCleanupTask> ContainerCleanup = ContainerHelper.CreateCleanupTask();

	auto AbortCleanup = [ContainerCleanup]()
	{
		ContainerCleanup->Abort();
		return true;
	};

	auto CleanupTask = [ContainerCleanup, bRemoveComponents]()
	{
		return ContainerCleanup->Execute(bRemoveComponents);
	};

	return GetSubsystem()->ScheduleGeneric(CleanupTask, AbortCleanup, this, *AllDependencies);
}

void UPCGComponent::CleanupUnusedManagedResources()
{
	FPCGManagedResourceContainerHelper ContainerHelper(this);
	ContainerHelper.CleanupUnusedManagedResources();
}

void UPCGComponent::ClearGraphGeneratedOutput(bool bClearLoadedPreviewData)
{
	GeneratedGraphOutput.ClearUsage(EPCGDataUsage::ComponentOutputData);

#if WITH_EDITOR
	// Do not re-outer loaded preview data if bClearLoadedPreviewData is false
	auto IsLoadedPreviewData = [this, bClearLoadedPreviewData](const UPCGData* InData)
	{
		return !bClearLoadedPreviewData && LoadedPreviewGeneratedGraphOutput.TaggedData.ContainsByPredicate([InData](const FPCGTaggedData & TaggedData) { return TaggedData.Data == InData; });
	};
#endif

	for (FPCGTaggedData& GeneratedData : GeneratedGraphOutput.TaggedData)
	{
		if ( GeneratedData.Data 
			&& GeneratedData.Data->GetOuter() == this 
#if WITH_EDITOR
			&& !IsLoadedPreviewData(GeneratedData.Data.Get())
#endif
			)
		{
			const_cast<UPCGData*>(GeneratedData.Data.Get())->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		}
	}

#if WITH_EDITOR
	if (bClearLoadedPreviewData)
	{
		LoadedPreviewGeneratedGraphOutput.Reset();
	}
#endif

	GeneratedGraphOutput.Reset();
}

void UPCGComponent::BeginPlay()
{
	LLM_SCOPE_BYTAG(PCG);
	Super::BeginPlay();

#if WITH_EDITOR
	// Disable auto-refreshing on preview actors until we have something more robust on the execution side.
	if (GetOwner() && GetOwner()->bIsEditorPreviewActor)
	{
		return;
	}
#endif

	// Register itself to the PCGSubsystem, to map the component to all its corresponding PartitionActors if it is partition among other things.
	if (GetSubsystem())
	{
		ensure(!bGenerated || LastGeneratedBounds.IsValid);
		GetSubsystem()->RegisterOrUpdateExecutionSource(this);
	}

	// Procedural instances are never persisted so always require generation.
	const bool bAlreadyGenerated = bGenerated && !bProceduralInstancesInUse;

	// Either this is the original component and it is non-null or this is a local component and we need the original component to be loaded to trigger a generation
	UPCGComponent* OriginalComponent = GetOriginalComponent();

	if (OriginalComponent && bActivated && !bAlreadyGenerated && GenerationTrigger == EPCGComponentGenerationTrigger::GenerateOnLoad)
	{
		GenerateInternal(/*bForce=*/false, PCGHiGenGrid::UninitializedGridSize(), EPCGComponentGenerationTrigger::GenerateOnLoad, {});
	}
}

void UPCGComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	// General comment: we shouldn't usually be cleaning up resources in the EndPlay call.
	// There might be an exception currently with unregistering of Local components (UE-215065) which needs fixing.
	
	// Always try to unregister itself, if it doesn't exist, it will early out. 
	// Just making sure that we don't left some resources registered while dead.
	if (UPCGSubsystem* Subsystem = GetSubsystem())
	{
		Subsystem->CancelGeneration(this);
		Subsystem->UnregisterExecutionSource(this);
	}

	FPCGManagedResourceContainerHelper ContainerHelper(this);
	ContainerHelper.EndPlay();

	Super::EndPlay(EndPlayReason);
}

void UPCGComponent::OnUnregister()
{
#if WITH_EDITOR
	if (UPCGSubsystem* Subsystem = GetSubsystem())
	{
		// Track if Component got unregistered through unloading (not property changes / blueprint construction)
		bUnregisteredThroughLoading = Subsystem->IsComponentBeingUnloaded(this);

		if (!PCGHelpers::IsRuntimeOrPIE())
		{
			Subsystem->CancelGeneration(this);
		}

		// Always unregister here if we are being unloaded (we are not being re-registered)
		if (bUnregisteredThroughLoading)
		{
			Subsystem->UnregisterExecutionSource(this, /*bForce=*/true);
		}
	}

	// We shouldn't cleanup resources in OnUnregister in most persistent cases.
	// This specific case is to handle World Partition unloading of actors where our actor might have a PCG Component which generated some Preview Resources.
	// In this case we do want to release those resources and the actors they might hold on to. If we don't those actors become orphaned and we might end up with duplicates when reloading the component.
	TSet<TSoftObjectPtr<AActor>> ActorsToDelete;
	{
		FPCGManagedResourceContainerHelper ContainerHelper(this);
		UE::TScopeLock ResourcesLock(GeneratedResourcesLock);
		if (ensure(ContainerHelper.AreResourcesAccessible()))
		{
			const bool bCleanupPersistentResources = bUnregisteredThroughLoading && PCGComponent::CVarDeletePreviewResourcesWhenUnloading.GetValueOnAnyThread() && CurrentEditingMode == EPCGEditorDirtyMode::Preview;

			for (UPCGManagedResource* ManagedResource : ContainerHelper.GetConstManagedResourcesNoLock())
			{
				if (ManagedResource)
				{
					// Make sure resource is indeed a transient/preview resource
					const bool bReleasePersistent = bCleanupPersistentResources &&
						!ManagedResource->IsMarkedTransientOnLoad() &&
						!ContainerHelper.GetConstLoadedPreviewResourcesNoLock().Contains(ManagedResource);

					if (bReleasePersistent || ManagedResource->ReleaseOnTeardown())
					{
						ManagedResource->Release(/*bHardRelease=*/true, ActorsToDelete);
					}
				}
			}
		}
	}

	if (!ActorsToDelete.IsEmpty())
	{
		// Dispatch delete of actors to next tick
		::ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakWorld = TWeakObjectPtr<UWorld>(GetWorld()), InActorsToDelete = MoveTemp(ActorsToDelete)]()
		{
			LLM_SCOPE_BYTAG(PCG);

			if (UWorld* World = WeakWorld.Get())
			{
				UPCGActorHelpers::DeleteActors(World, InActorsToDelete.Array());
			}
		});
	}
#endif // WITH_EDITOR

	Super::OnUnregister();
}

void UPCGComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGComponent::OnComponentDestroyed);
#if WITH_EDITOR
	// BeginDestroy is not called immediately when a component is destroyed. Therefore callbacks are not cleaned
	// until GC is ran, and can stack up with BP reconstruction scripts. Force the removal of callbacks here. If the component
	// is dead, we don't want to react to callbacks anyway.
	if (GraphInstance)
	{
		GraphInstance->OnGraphChangedDelegate.RemoveAll(this);
		GraphInstance->TeardownCallbacks();
	}
#endif

	// Bookkeeping local components that might be deleted by the user.
	// Making sure that the corresponding partition actor doesn't keep a dangling references
	if (APCGPartitionActor* PAOwner = Cast<APCGPartitionActor>(GetOwner()))
	{
		PAOwner->RemoveLocalComponent(this);
	}

#if WITH_EDITOR
	// Don't do the unregister in OnUnregister, because we have flows where the component gets Unregistered/Registered without getting destroyed.
	if (UPCGSubsystem* Subsystem = GetSubsystem())
	{
		if (!PCGHelpers::IsRuntimeOrPIE())
		{
			Subsystem->UnregisterExecutionSource(this);
		}

		if (IsCreatedByConstructionScript())
		{
			Subsystem->SetConstructionScriptSourceComponent(this);
		}
	}
#endif // WITH_EDITOR

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

#if WITH_EDITOR
void UPCGComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UPCGComponent* This = CastChecked<UPCGComponent>(InThis);

	This->ExecutionInspection.AddReferencedObjects(Collector);

	Super::AddReferencedObjects(This, Collector);
}
#endif // WITH_EDITOR

void UPCGComponent::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYTAG(PCG);

#if WITH_EDITOR
	TArray<TObjectPtr<UPCGManagedResource>> GeneratedResourcesCopy;
	FPCGDataCollection GeneratedGraphOutputCopy;

	FPCGManagedResourceContainerHelper ContainerHelper(this);
	if (!Ar.IsLoading() && CurrentEditingMode == EPCGEditorDirtyMode::Preview)
	{
		ContainerHelper.BeginPreviewSave(GeneratedResourcesCopy);

		GeneratedGraphOutputCopy = GeneratedGraphOutput;
		GeneratedGraphOutput = LoadedPreviewGeneratedGraphOutput;
	}

	// When duplicating for PIE, we need to Update the RuntimeGridDescriptorHash before duplication for unsaved changes
	if (Ar.IsSaving() && (Ar.GetPortFlags() & PPF_DuplicateForPIE))
	{
		UpdateRuntimeGridDescriptorHash();
	}
#endif // WITH_EDITOR

	Ar.UsingCustomVersion(FPCGCustomVersion::GUID);

	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (!Ar.IsCooking() && !Ar.IsLoadingFromCookedPackage())
	{
		int32 DataVersion = FPCGCustomVersion::LatestVersion;
		if (Ar.IsLoading())
		{
			DataVersion = Ar.CustomVer(FPCGCustomVersion::GUID);

			if (DataVersion < FPCGCustomVersion::SupportPartitionedComponentsInNonPartitionedLevels)
			{
				bDisableIsComponentPartitionedOnLoad = true;
			}
		}

		if (DataVersion >= FPCGCustomVersion::DynamicTrackingKeysSerializedInComponent)
		{
			ChangeTracker.Serialize(Ar);
		}
	}
#endif // WITH_EDITORONLY_DATA


#if WITH_EDITOR
	if (!Ar.IsLoading() && CurrentEditingMode == EPCGEditorDirtyMode::Preview)
	{
		ContainerHelper.EndPreviewSave(GeneratedResourcesCopy);
		GeneratedGraphOutput = GeneratedGraphOutputCopy;
	}
#endif // WITH_EDITOR
}

void UPCGComponent::PostLoad()
{
	LLM_SCOPE_BYTAG(PCG);
	Super::PostLoad();

#if WITH_EDITOR

	FPCGManagedResourceContainerHelper ContainerHelper(this);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!GeneratedResources_DEPRECATED.IsEmpty())
	{
		ContainerHelper.SetManagedResources(MoveTemp(GeneratedResources_DEPRECATED));
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Debug resources are transient and will generate null entries on save, clean them up here
	ContainerHelper.RemoveManagedResourceNoLock(nullptr);

	// Force dirty to be false on load. We should never refresh on load.
	bDirtyGenerated = false;

	// We can never be generated if we have no graph
	if (!GetGraph())
	{
		bGenerated = false;
	} // We can never be generated if we are a CDO
	else if (IsTemplate())
	{
		// It is possible to create a Blueprint from an Actor containing a generated PCG Component.
		// Eventually this shouldn't be possible but for data that has already been saved, we fix it up here.
		bGenerated = false;
		GeneratedGraphOutput.Reset();
	}

	// Components marked as partitioned in non-WP worlds from BEFORE partitioning was supported in non-WP worlds can leak resources. To fix this, we can just unset 'bIsComponentPartitioned'.
	if (bDisableIsComponentPartitionedOnLoad)
	{
		const UWorld* World = GetOwner() ? GetOwner()->GetWorld() : nullptr;

		if (IsPartitioned() && !IsManagedByRuntimeGenSystem() && World && World->GetWorldPartition() == nullptr)
		{
			bIsComponentPartitioned = false;
		}

		bDisableIsComponentPartitionedOnLoad = false;
	}

	/** Deprecation code, should be removed once generated data has been updated */
	if (GetOwner() && bGenerated && ContainerHelper.IsEmpty())
	{
		TArray<UInstancedStaticMeshComponent*> ISMCs;
		GetOwner()->GetComponents(ISMCs);

		for (UInstancedStaticMeshComponent* ISMC : ISMCs)
		{
			if (ISMC->ComponentTags.Contains(GetFName()))
			{
				UPCGManagedISMComponent* ManagedComponent = NewObject<UPCGManagedISMComponent>(this);
				ManagedComponent->GeneratedComponent = ISMC;
				ContainerHelper.AddManagedResourceNoLock(ManagedComponent);
			}
		}

		if (GeneratedActors_DEPRECATED.Num() > 0)
		{
			UPCGManagedActors* ManagedActors = NewObject<UPCGManagedActors>(this);
			ManagedActors->GetMutableGeneratedActors() = GeneratedActors_DEPRECATED.Array();
			ContainerHelper.AddManagedResourceNoLock(ManagedActors);
			GeneratedActors_DEPRECATED.Reset();
		}
	}

	if (Graph_DEPRECATED)
	{
		GraphInstance->SetGraph(Graph_DEPRECATED);
		Graph_DEPRECATED = nullptr;
	}
	
	GenerationRadii.ApplyDeprecation();

	SetupCallbacksOnCreation();

	// Always set the editing mode to Preview when we're in GenerateAtRuntime mode
	CurrentEditingMode = IsManagedByRuntimeGenSystem() ? EPCGEditorDirtyMode::Preview : SerializedEditingMode;

	if (CurrentEditingMode == EPCGEditorDirtyMode::Preview)
	{
		// Make sure we update the transient state if we have been forced into Preview mode by runtime generation.
		if (CurrentEditingMode != SerializedEditingMode)
		{
			PreviousEditingMode = SerializedEditingMode;
			ChangeTransientState(CurrentEditingMode);
		}

		bGenerated = false;
	}
	else if (CurrentEditingMode == EPCGEditorDirtyMode::LoadAsPreview && !PCGHelpers::IsRuntimeOrPIE() && bGenerated)
	{
		CurrentEditingMode = EPCGEditorDirtyMode::Preview;
		MarkResourcesAsTransientOnLoad();
		LoadedPreviewGeneratedGraphOutput = GeneratedGraphOutput;
		bDirtyGenerated = PCGSystemSwitches::CVarDirtyLoadAsPreviewOnLoad.GetValueOnAnyThread();
	}

	// @todo_pcg: Migrate FPCGInteractiveToolDataContainer and runtime generation scheduling data entries into SourceDataContainer her
#endif

	if (!IsValid(SchedulingPolicy))
	{
		RefreshSchedulingPolicy();
	}
	else
	{
		// Note: Should not be transactional *unless* we are - this flag is already included in RF_PropagateToSubobjects.
		// Clearing it here in case it was incorrectly set on a previous save. This is needed to resolve some undo issues.
		SchedulingPolicy->ClearFlags(RF_Transactional);
		const EObjectFlags Flags = GetMaskedFlags(RF_PropagateToSubObjects);
		SchedulingPolicy->SetFlags(Flags);
#if WITH_EDITOR
		SchedulingPolicy->SetShouldDisplayProperties(IsManagedByRuntimeGenSystem());
#endif
	}

	// keep track if component was already generated when it got loaded
	bGeneratedOffline = bGenerated;
}

#if WITH_EDITOR
void UPCGComponent::SetupCallbacksOnCreation()
{
	UpdateTrackingCache();

	if (GraphInstance)
	{
		// We might have already connected in PostInitProperties
		// To be sure, remove it and re-add it.
		GraphInstance->OnGraphChangedDelegate.RemoveAll(this);
		GraphInstance->OnGraphChangedDelegate.AddUObject(this, &UPCGComponent::OnGraphChanged);
	}
}

bool UPCGComponent::CanEditChange(const FProperty* InProperty) const
{
	// Can't change anything if the component is local
	return !bIsComponentLocal && Super::CanEditChange(InProperty);
}
#endif

void UPCGComponent::BeginDestroy()
{
	GeneratedGraphOutput.Reset();
	PerPinGeneratedOutput.Reset();

#if WITH_EDITOR
	// BeginDestroy will end up calling OnComponentDestroyed and in this specific case we don't want to rename our Outputs to transient package.
	// Known case where GeneratedGraphOutput can be non empty here is the editor exit purge.
	if (GraphInstance)
	{
		GraphInstance->OnGraphChangedDelegate.RemoveAll(this);
	}

	// For the special case where a component is part of a reconstruction script (from a BP),
	// but gets destroyed immediately, we need to force the unregistering. 
	if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
	{
		PCGSubsystem->UnregisterExecutionSource(this, /*bForce=*/true);
	}
#endif // WITH_EDITOR

	Super::BeginDestroy();
}

bool UPCGComponent::IsEditorOnly() const
{
#if WITH_EDITOR
	if (FPCGModule::IsEditorOnlyGeneration())
	{
		return true;
	}
#endif

	return Super::IsEditorOnly() || (GraphInstance && GraphInstance->IsEditorOnly());
}

bool UPCGComponent::NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const
{
	if (!Super::NeedsLoadForTargetPlatform(TargetPlatform))
	{
		return false;
	}

	const UPCGGraph* PCGGraph = GetGraph();
	return !PCGGraph || PCGGraph->NeedsLoadForTargetPlatform(TargetPlatform);
}

void UPCGComponent::PostInitProperties()
{
	LLM_SCOPE_BYTAG(PCG);

#if WITH_EDITOR
	GraphInstance->OnGraphChangedDelegate.AddUObject(this, &UPCGComponent::OnGraphChanged);
#endif // WITH_EDITOR

	Super::PostInitProperties();
}

void UPCGComponent::OnRegister()
{
	LLM_SCOPE_BYTAG(PCG);
	Super::OnRegister();

#if WITH_EDITOR
	bUnregisteredThroughLoading = false;

	// Disable auto-refreshing on preview actors until we have something more robust on the execution side.
	if (GetOwner() && GetOwner()->bIsEditorPreviewActor)
	{
		return;
	}

	// We can't register to the subsystem in OnRegister if we are at runtime because
	// the landscape can be not loaded yet.
	// It will be done in BeginPlay at runtime.
	if (!PCGHelpers::IsRuntimeOrPIE() && GetSubsystem())
	{
		if (UWorld* World = GetWorld())
		{
			ensure(!bGenerated || LastGeneratedBounds.IsValid);
			GetSubsystem()->RegisterOrUpdateExecutionSource(this, bGenerated);
		}
	}
#endif //WITH_EDITOR
}

TStructOnScope<FActorComponentInstanceData> UPCGComponent::GetComponentInstanceData() const
{
	TStructOnScope<FActorComponentInstanceData> InstanceData = MakeStructOnScope<FActorComponentInstanceData, FPCGComponentInstanceData>(this);
	return InstanceData;
}

void UPCGComponent::OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType)
{
	RefreshAfterGraphChanged(InGraph, ChangeType);
}

void UPCGComponent::RefreshAfterGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType)
{
	if (InGraph != GraphInstance)
	{
		return;
	}

	const bool bIsTrivial = (ChangeType & ~(EPCGChangeType::Cosmetic | EPCGChangeType::GraphCustomization)) == EPCGChangeType::None;
	if (bIsTrivial)
	{
		// If it is a cosmetic change (or no change), nothing to do
		return;
	}

	const bool bHasGraph = (InGraph && InGraph->GetGraph());

	const bool bIsStructural = ((ChangeType & (EPCGChangeType::Edge | EPCGChangeType::Structural)) != EPCGChangeType::None);
	const bool bDirtyInputs = bIsStructural || ((ChangeType & EPCGChangeType::Input) != EPCGChangeType::None);

#if WITH_EDITOR
	// In editor, since we've changed the graph, we might have changed the tracked actor tags as well
	if (!PCGHelpers::IsRuntimeOrPIE())
	{
		if (UPCGSubsystem* Subsystem = GetSubsystem())
		{
			// Don't update the tracking if nothing changed for the tracking.
			TArray<FPCGSelectionKey> ChangedKeys;
			if (UpdateTrackingCache(&ChangedKeys))
			{
				Subsystem->UpdateTracking(this, &ChangedKeys);
			}
		}

		DirtyGenerated(bDirtyInputs ? (EPCGComponentDirtyFlag::Actor | EPCGComponentDirtyFlag::Landscape) : EPCGComponentDirtyFlag::None);

		// If there is no graph, we should still refresh if we are runtime-managed, since the RuntimeGenScheduler will need to flush its resources.
		if (bHasGraph || IsManagedByRuntimeGenSystem())
		{
			Refresh(ChangeType);
		}
		else
		{
			// With no graph, we clean up
			CleanupLocal(/*bRemoveComponents=*/true);
		}

		ExecutionState.GetInspection().ClearInspectionData();

		return;
	}
#endif

	if (IsManagedByRuntimeGenSystem())
	{
		if (UPCGSubsystem* Subsystem = GetSubsystem())
		{
			Subsystem->RefreshRuntimeGenExecutionSource(this, ChangeType);
		}
	}
	else
	{
		// Otherwise, if we are in PIE or runtime, force generate if we have a graph (and were generated). Or cleanup if we have no graph
		if (bHasGraph && bGenerated)
		{
			GenerateLocal(/*bForce=*/true);
		}
		else if (!bHasGraph)
		{
			CleanupLocal(/*bRemoveComponents=*/true);
		}
	}
}

#if WITH_EDITOR
void UPCGComponent::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange)
	{
		return;
	}

	const FName PropName = PropertyAboutToChange->GetFName();

	if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, GenerationTrigger))
	{
		if (IsManagedByRuntimeGenSystem())
		{
			if (UPCGSubsystem* Subsystem = GetSubsystem())
			{
				// When toggling off of GenerateAtRuntime, we should flush the RuntimeGenScheduler state for this component.
				Subsystem->RefreshRuntimeGenExecutionSource(this, EPCGChangeType::GenerationGrid);
			}

			// Reset to the the editing mode we were in before entering GenerateAtRuntime mode.
			SetEditingMode(PreviousEditingMode, SerializedEditingMode);
		}
	}
	else if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, bIsComponentPartitioned))
	{
		if (IsManagedByRuntimeGenSystem())
		{
			if (UPCGSubsystem* Subsystem = GetSubsystem())
			{
				// When toggling IsPartitioned, we should proactively flush the RuntimeGenScheduler.
				Subsystem->RefreshRuntimeGenExecutionSource(this, EPCGChangeType::GenerationGrid);
			}
		}
	}
}

void UPCGComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	LLM_SCOPE_BYTAG(PCG);

	if (!PropertyChangedEvent.Property || !IsValidChecked(this))
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);
		return;
	}

	const FName PropName = PropertyChangedEvent.Property->GetFName();

	bool bTransientPropertyChangedThatDoesNotRequireARefresh = false;

	// Implementation note:
	// Since the current editing mode is a transient variable, if we do not do this transition here before going in the Super call,
	//  we can end up in a situation where BP actors are reconstructed (... this component included ...) which makes this fall into the !IsValid case just after
	if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, CurrentEditingMode))
	{
		// When affecting the editing mode from the user's point of view, we need to change both the current & serialized values
		SetEditingMode(CurrentEditingMode, CurrentEditingMode);
		ChangeTransientState(CurrentEditingMode);
		bTransientPropertyChangedThatDoesNotRequireARefresh = true;
	}

	bool bWasDirtyGenerated = bDirtyGenerated;
	bDirtyGenerated = true;

	Super::PostEditChangeProperty(PropertyChangedEvent);

	// BP actors will early out here as construction script will have created a new component.
	if (!IsValidChecked(this))
	{
		return;
	}

	// Restore dirty flag for non BP cases. BP components will always regenerate for now.
	bDirtyGenerated = bWasDirtyGenerated;

	const FName MemberName = PropertyChangedEvent.MemberProperty->GetFName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPCGComponent, bOverrideGenerationRadii))
	{
		// RuntimeGen will automatically pick up any changes to generation radii, we don't need to do any work here.
		return;
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UPCGComponent, GenerationRadii))
	{
		// RuntimeGen will automatically pick up any changes to generation radii, we don't need to do any work here.
		GenerationRadii.ComputeHash();
		return;
	}

	// Important note: all property changes already go through the OnObjectPropertyChanged, and will be dirtied here.
	// So where only a Refresh is needed, it goes through the "capture all" else case.
	if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, bIsComponentPartitioned))
	{
		if (CanPartition())
		{
			// At this point, bIsComponentPartitioned is already set with the new value.
			// But we need to do some cleanup before
			// So keep this new value, and take its negation for the cleanup.
			bool bIsNowPartitioned = bIsComponentPartitioned;
			bIsComponentPartitioned = !bIsComponentPartitioned;

			// SetIsPartitioned cleans up before, so keep track if we were generated or not.
			bool bWasGenerated = bGenerated;
			SetIsPartitioned(bIsNowPartitioned);

			// And finally, re-generate if we were generated and activated
			// Delay to next frame so that the Component unregister doesn't cancel this 
			//  - Only affects non BP PCG Components because those get invalidated / handled by ConstructionScript
			if (bWasGenerated && bActivated)
			{
				if (UPCGSubsystem* Subsystem = GetSubsystem())
				{
					Subsystem->ScheduleGeneric([this]()
					{
						GenerateLocal(/*bForce=*/false);
						return true;
					}, this, {});
				}
			}
		}
	}
	else if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, GraphInstance))
	{
		// If anything happens on the graph instance, it will be handled there.
	}
	else if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, InputType))
	{
		if (UPCGSubsystem* Subsystem = GetSubsystem())
		{
			TArray<FPCGSelectionKey> ChangedKeys;
			if (UpdateTrackingCache(&ChangedKeys))
			{
				Subsystem->UpdateTracking(this, &ChangedKeys);
			}
		}

		DirtyGenerated(EPCGComponentDirtyFlag::Input);
		Refresh();
	}
	else if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, bParseActorComponents))
	{
		DirtyGenerated(EPCGComponentDirtyFlag::Input);
		Refresh();
	}
	else if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, Seed))
	{
		DirtyGenerated();
		Refresh();
	}
	else if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, TrackingPriority))
	{
		TrackingPriority = PCGComponent::RoundTrackingPriority(TrackingPriority);

		DirtyGenerated();
		Refresh();
	}
	else if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, SchedulingPolicyClass))
	{
		// We don't need to refresh the component here because this does not effect generation behavior, only scheduling behavior.
		RefreshSchedulingPolicy();
	} 
	else if (PropName == GET_MEMBER_NAME_CHECKED(UPCGComponent, GenerationTrigger))
	{
		// Implementation detail - if we get rid of the scheduling policy here, we'll trigger an editor error, so just create the policy if needed
		if (!SchedulingPolicy)
		{
			RefreshSchedulingPolicy();
		}
		else
		{
			// We should only display scheduling policy parameters when in runtime generation mode.
			SchedulingPolicy->SetShouldDisplayProperties(IsManagedByRuntimeGenSystem());
		}

		if (IsManagedByRuntimeGenSystem())
		{
			// If we have been set to GenerateAtRuntime, we should cleanup any artifacts.
			CleanupLocalImmediate(/*bRemoveComponents=*/true, /*bCleanupLocalComponents=*/true);

			PreviousEditingMode = CurrentEditingMode;

			SetEditingMode(/*InEditingMode=*/EPCGEditorDirtyMode::Preview, SerializedEditingMode);
			ChangeTransientState(CurrentEditingMode);
		}
		else
		{
			Refresh();
		}
	}
	// General properties that don't affect behavior
	else if(!bTransientPropertyChangedThatDoesNotRequireARefresh)
	{
		Refresh();
	}
}

void UPCGComponent::PostEditImport()
{
	LLM_SCOPE_BYTAG(PCG);
	Super::PostEditImport();

	SetupCallbacksOnCreation();
}

void UPCGComponent::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Update RuntimeGridDescriptorHash on Save (Actor might have changed DataLayers and we need to update)
	if (!ObjectSaveContext.IsCooking())
	{
		UpdateRuntimeGridDescriptorHash();
	}
}

void UPCGComponent::UpdateRuntimeGridDescriptorHash()
{
	// No need to maintain RuntimeGridDescriptorHash for PCGComponents owned by Partition Actors
	if (!IsTemplate() && GetOwner() && !GetOwner()->IsA<APCGPartitionActor>())
	{
		FPCGGridDescriptor GridDescriptor = GetGridDescriptorInternal(0, /*bRuntimeHashUpdate=*/true);
		RuntimeGridDescriptorHash = GridDescriptor.GetRuntimeHash();
	}
}

bool UPCGComponent::Modify(const bool bAlwaysMarkDirty)
{
	const bool bResult = Super::Modify(bAlwaysMarkDirty);

	// UActorComponent::Modify forwards BP construction-script components to the owner, which would skip recording the component.
	// Implementation Note: SaveObject is idempotent, so this should be okay.
	SaveToTransactionBuffer(this, /*bMarkDirty=*/false);

	return bResult;
}

TSharedPtr<ITransactionObjectAnnotation> UPCGComponent::FactoryTransactionAnnotation(const ETransactionAnnotationCreationMode InCreationMode) const
{
	return InCreationMode == ETransactionAnnotationCreationMode::DefaultInstance ? PCG::Transaction::CreateAnnotation() : PCG::Transaction::CreateAnnotation(*this);
}

void UPCGComponent::PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation)
{
	PCG::Transaction::RestoreFromAnnotation(*this, TransactionAnnotation);

	// UObject::PostEditUndo(annotation) calls UObject::PostEditUndo() qualified (no virtual dispatch). Invoke PCGComponent's directly.
	UPCGComponent::PostEditUndo();
}

void UPCGComponent::PreEditUndo()
{
	Super::PreEditUndo();

	// Here we will keep a copy of flags that we require to keep through the undo
	// so we can have a consistent state
	LastGeneratedBoundsPriorToUndo = LastGeneratedBounds;
	
	// Capture CurrentEditingMode before it potentially changes due to undo/redo so we can compare against in PostEditUndo.
	// This only applies to non-BP PCGComponents because PreEditUndo/PostEditUndo will not be called in BP paths.
	LastEditingModePriorToUndo = CurrentEditingMode;

	if (IsManagedByRuntimeGenSystem())
	{
		// Always flush the runtime gen state before undo/redo to avoid leaking resources or locking grid cells from future generation.
		if (UPCGSubsystem* Subsystem = GetSubsystem())
		{
			Subsystem->RefreshRuntimeGenExecutionSource(this, EPCGChangeType::GenerationGrid);
		}
	}

	if (bGenerated)
	{
		// Cleanup so managed resources are cleaned in all cases
		CleanupLocalImmediate(/*bRemoveComponents=*/true);
		// Put back generated flag to its original value so it is captured properly
		bGenerated = true;
	}
}

void UPCGComponent::PostEditUndo()
{
	LastGeneratedBounds = LastGeneratedBoundsPriorToUndo;

	// Handle undo/redo of CurrentEditingMode
	if (LastEditingModePriorToUndo != CurrentEditingMode)
	{
		ChangeTransientState(CurrentEditingMode);
		LastEditingModePriorToUndo = CurrentEditingMode;
	}

	UpdateTrackingCache();
	DirtyGenerated(EPCGComponentDirtyFlag::All);

	if (UPCGSubsystem* Subsystem = GetSubsystem())
	{
		Subsystem->UpdateTracking(this);
	}

	if (bGenerated)
	{
		// Cancel existing means a refresh will always be scheduled even if another refresh was pending. If an undo
		// operation removes the component, a valid refresh task ID is set but the refresh task itself will fail
		// and leave the valid task ID hanging on the component. Forcing here means if we later retrieve this state
		// from the undo/redo buffer, the refresh will be forced which will reset the state.
		Refresh(EPCGChangeType::Structural | EPCGChangeType::GenerationGrid, /*bCancelExistingRefresh=*/true);
	}

	Super::PostEditUndo();
}

bool UPCGComponent::UpdateTrackingCache(TArray<FPCGSelectionKey>* OptionalChangedKeys)
{
	// Without an owner, it probably means we are in a BP template, so no need to update the tracking cache.
	// Same for local components, as we will use the cache of the original component.
	if (!GetOwner() || IsLocalComponent())
	{
		return false;
	}

	// Store in a temporary map to detect key changes.
	FPCGSelectionKeyToSettingsMap NewTrackedKeysToSettings;

	const int32 PreviousKeyCount = ChangeTracker.StaticallyTrackedKeysToSettings.Num();
	
	bool bHasChanged = false;

	if (UPCGGraph* PCGGraph = GetGraph())
	{
		// @todo_pcg: to remove usage
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Also add a key for the landscape, with settings null and always culled, if we should track the landscape
		if (ShouldTrackLandscape())
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			FPCGSelectionKey LandscapeKey = FPCGSelectionKey(ALandscapeProxy::StaticClass());
			NewTrackedKeysToSettings.FindOrAdd(LandscapeKey).Emplace(/*Settings*/nullptr, /*bIsCulled*/true);
		}

		bHasChanged = PCGGraph->UpdateTrackedSelectionKeys(ChangeTracker.StaticallyTrackedKeysToSettings, NewTrackedKeysToSettings, OptionalChangedKeys);
	}
	else
	{
		// Handle case were we no longer have a graph
		bHasChanged = ChangeTracker.StaticallyTrackedKeysToSettings.Num() > 0;
		if (OptionalChangedKeys)
		{
			OptionalChangedKeys->Reserve(ChangeTracker.StaticallyTrackedKeysToSettings.Num());
			for (const TPair<FPCGSelectionKey, TArray<FPCGSettingsAndCulling>>& It : ChangeTracker.StaticallyTrackedKeysToSettings)
			{
				OptionalChangedKeys->Add(It.Key);
			}
		}
	}

	ChangeTracker.StaticallyTrackedKeysToSettings = MoveTemp(NewTrackedKeysToSettings);

	return bHasChanged;
}

void UPCGComponent::DirtyGenerated(EPCGComponentDirtyFlag DirtyFlag, const bool bDispatchToLocalComponents)
{
	if (GetSubsystem() && GetSubsystem()->IsGraphCacheDebuggingEnabled())
	{
		UE_LOGF(LogPCG, Log, "[%ls] --- DIRTY GENERATED ---", *GetOwner()->GetName());
	}

	bDirtyGenerated = true;

	// Dirty data as a waterfall from basic values
	if (!!(DirtyFlag & EPCGComponentDirtyFlag::Actor))
	{
		CachedActorData = nullptr;
		// Since landscape data is related on the bounds of the current actor, when we dirty the actor data, we need to dirty the landscape data as well
		CachedLandscapeData = nullptr;
		CachedLandscapeHeightData = nullptr;
		CachedInputData = nullptr;
		CachedPCGData = nullptr;
	}

	if (!!(DirtyFlag & EPCGComponentDirtyFlag::Landscape))
	{
		CachedLandscapeData = nullptr;
		CachedLandscapeHeightData = nullptr;
		if (InputType == EPCGComponentInput::Landscape)
		{
			CachedInputData = nullptr;
			CachedPCGData = nullptr;
		}
	}

	if (!!(DirtyFlag & EPCGComponentDirtyFlag::Input))
	{
		CachedInputData = nullptr;
		CachedPCGData = nullptr;
	}

	if (!!(DirtyFlag & EPCGComponentDirtyFlag::Data))
	{
		CachedPCGData = nullptr;
	}
	
	// For partitioned graph, we must forward the call to the partition actor, if we need to
	// TODO: Don't forward for None for now, as it could break some stuff
	if (bActivated && IsPartitioned() && bDispatchToLocalComponents)
	{
		if (GetSubsystem())
		{
			GetSubsystem()->DirtyGraph(this, LastGeneratedBounds, DirtyFlag);
		}
	}
}

void UPCGComponent::ResetLastGeneratedBounds()
{
	LastGeneratedBounds = FBox(EForceInit::ForceInit);
}

// UE_DEPRECATED(5.6)
bool UPCGComponent::IsInspecting() const
{
	return ExecutionInspection.IsInspecting();
}

// UE_DEPRECATED(5.6)
void UPCGComponent::EnableInspection()
{
	ExecutionInspection.EnableInspection();
}

// UE_DEPRECATED(5.6)
void UPCGComponent::DisableInspection()
{
	ExecutionInspection.DisableInspection();
};

// UE_DEPRECATED(5.6)
void UPCGComponent::NotifyNodeExecuted(const UPCGNode* InNode, const FPCGStack* InStack, const PCGUtils::FCallTime* InTimer, bool bNodeUsedCache)
{
	ExecutionInspection.NotifyNodeExecuted(InNode, InStack, InTimer, bNodeUsedCache);
}

// UE_DEPRECATED(5.6)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
TMap<TObjectKey<const UPCGNode>, TSet<UPCGComponent::NodeExecutedNotificationData>> UPCGComponent::GetExecutedNodeStacks() const
{
	TMap<TObjectKey<const UPCGNode>, TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData>> ExecutedNodeStacks = ExecutionInspection.GetExecutedNodeStacks();
	TMap<TObjectKey<const UPCGNode>, TSet<UPCGComponent::NodeExecutedNotificationData>> DeprecatedExecutedNodeStacks;
	for(auto& Pair : ExecutedNodeStacks)
	{
		TSet<UPCGComponent::NodeExecutedNotificationData>& DeprecatedNotifications = DeprecatedExecutedNodeStacks.Add(Pair.Key);
		Algo::Transform(Pair.Value, DeprecatedNotifications, [](const FPCGGraphExecutionInspection::FNodeExecutedNotificationData& Notification)
		{
			return UPCGComponent::NodeExecutedNotificationData(Notification.Stack, Notification.Timer);
		});
	}

	return DeprecatedExecutedNodeStacks;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// UE_DEPRECATED(5.6)
uint64 UPCGComponent::GetNodeInactivePinMask(const UPCGNode* InNode, const FPCGStack& Stack) const
{
	return ExecutionInspection.GetNodeInactivePinMask(InNode, Stack);
}

// UE_DEPRECATED(5.6)
void UPCGComponent::NotifyNodeDynamicInactivePins(const UPCGNode* InNode, const FPCGStack* InStack, uint64 InactivePinBitmask) const
{
	ExecutionInspection.NotifyNodeDynamicInactivePins(InNode, InStack, InactivePinBitmask);
}

// UE_DEPRECATED(5.6)
bool UPCGComponent::WasNodeExecuted(const UPCGNode* InNode, const FPCGStack& Stack) const
{
	return ExecutionInspection.WasNodeExecuted(InNode, Stack);
}

// UE_DEPRECATED(5.6)
void UPCGComponent::StoreInspectionData(const FPCGStack* InStack, const UPCGNode* InNode, const PCGUtils::FCallTime* InTimer, const FPCGDataCollection& InInputData, const FPCGDataCollection& InOutputData, bool bUsedCache)
{
	ExecutionInspection.StoreInspectionData(InStack, InNode, InTimer, InInputData, InOutputData, bUsedCache);
}

// UE_DEPRECATED(5.6)
const FPCGDataCollection* UPCGComponent::GetInspectionData(const FPCGStack& InStack) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return ExecutionInspection.GetInspectionData(InStack);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

// UE_DEPRECATED(5.6)
void UPCGComponent::ClearInspectionData(bool bClearPerNodeExecutionData)
{
	ExecutionInspection.ClearInspectionData(bClearPerNodeExecutionData);
}

// UE_DEPRECATED(5.6)
bool UPCGComponent::HasNodeProducedData(const UPCGNode* InNode, const FPCGStack& Stack) const
{
	return ExecutionInspection.HasNodeProducedData(InNode, Stack);
}

void UPCGComponent::Refresh(EPCGChangeType ChangeType, bool bCancelExistingRefresh)
{
	// Disable auto-refreshing on preview actors until we have something more robust on the execution side.
	if (GetOwner() && GetOwner()->bIsEditorPreviewActor)
	{
		return;
	}

	// Runtime component refreshes should go through the runtime scheduler.
	if (IsManagedByRuntimeGenSystem())
	{
		if (UPCGSubsystem* Subsystem = GetSubsystem())
		{
			Subsystem->RefreshRuntimeGenExecutionSource(this, ChangeType);
		}

		return;
	}

	// If the component is tagged as not to regenerate in the editor, only exceptional cases should trigger a refresh
	// namely: the component is deactivated.
	// Note that the component changing its IsPartitioned state is already covered in the PostEditChangeProperty
	// Note that even if this is force refresh/structural change, we will NOT refresh
	if (!bRegenerateInEditor && bActivated)
	{
		// We still need to trigger component registration event otherwise further generations will fail if this is moved.
		// Note that we pass in false here to remove everything when moving a partitioned graph because we would otherwise need to do a reversible stamp to support this
		if (UPCGSubsystem* Subsystem = GetSubsystem())
		{
			Subsystem->RegisterOrUpdateExecutionSource(this, bGenerated);
		}

		return;
	}

	// If refresh is disabled, just exit
	if (PCGSystemSwitches::CVarGlobalDisableRefresh.GetValueOnAnyThread() || IsRunningCommandlet())
	{
		return;
	}

	// Discard any refresh if have already one scheduled.
	if (UPCGSubsystem* Subsystem = GetSubsystem())
	{
		// Cancel an already existing generation if either the change is structural in nature (which requires a recompilation, so a full-rescheduling)
		// or if the generation is already started
		const bool bGenerationWasInProgress = IsGenerationInProgress();
		const bool bStructural = !!(ChangeType & EPCGChangeType::Structural);
		bool bNeedToCancelCurrentTasks = (CurrentGenerationTask != InvalidPCGTaskId && (bStructural || bGenerationWasInProgress));

		// Cancel an already existing refresh if caller allows this
		if (bCancelExistingRefresh && CurrentRefreshTask != InvalidPCGTaskId)
		{
			bNeedToCancelCurrentTasks = true;

			CurrentRefreshTask = InvalidPCGTaskId;
		}

		const bool bScheduleRefresh = CurrentRefreshTask == InvalidPCGTaskId && CurrentCleanupTask == InvalidPCGTaskId;

		if (bNeedToCancelCurrentTasks)
		{
			Subsystem->CancelGeneration(this, /*bCleanupManagedResources=*/!bScheduleRefresh);
		}

		// Calling a new refresh here might not be sufficient; if the current component was generating but was not previously generated,
		// then the bGenerated flag will be false, which will prevent a subsequent update here
		if (bScheduleRefresh)
		{
			CurrentRefreshTask = Subsystem->ScheduleRefresh(this, bGenerationWasInProgress);
		}
	}
}

void UPCGComponent::StartGenerationInProgress()
{
	// Implementation detail:
	// Since the original component is not guaranteed to run the FetchInput element, local components are "allowed" to mark generation in progress on their original component.
	// However, the PostProcessGraph on the original component will be guaranteed to be called at the end of the execution so we do not need this mechanism in that case.
	bGenerationInProgress = true;

	if (IsLocalComponent())
	{
		if (UPCGComponent* OriginalComponent = CastChecked<APCGPartitionActor>(GetOwner())->GetOriginalComponent(this))
		{
			OriginalComponent->bGenerationInProgress = true;
		}
	}
}

void UPCGComponent::StopGenerationInProgress()
{
	bGenerationInProgress = false;
}

bool UPCGComponent::IsGenerationInProgress()
{
	return bGenerationInProgress;
}

bool UPCGComponent::ShouldGenerateBPPCGAddedToWorld() const
{
	if (PCGHelpers::IsRuntimeOrPIE())
	{
		return false;
	}
	else
	{
		// Generate on drop can be disabled by global settings or locally by not having "GenerateOnLoad" as a generation trigger (and Generate on Drop option to false).
		if (const UPCGEngineSettings* Settings = GetDefault<UPCGEngineSettings>())
		{
			return Settings->bGenerateOnDrop && bForceGenerateOnBPAddedToWorld &&
				(GenerationTrigger == EPCGComponentGenerationTrigger::GenerateOnLoad || (GenerationTrigger == EPCGComponentGenerationTrigger::GenerateOnDemand && bGenerateOnDropWhenTriggerOnDemand));
		}
		else
		{
			return false;
		}
	}
}

// deprecated
bool UPCGComponent::IsObjectTracked(const UObject* InObject, bool& bOutIsCulled) const
{
	check(InObject);

	if (!GetOwner())
	{
		return false;
	}


	// We should always track the owner of the component, without culling
	if (GetOwner() == InObject)
	{
		bOutIsCulled = false;
		return true;
	}

	// @todo_pcg: to remove usage
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// If we track the landscape using legacy methods and it is a landscape, it should be tracked as culled
	if (InObject && InObject->IsA<ALandscapeProxy>() && ShouldTrackLandscape())
	{
		bOutIsCulled = true;
		return true;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const FPCGSelectionKey ObjectKey = FPCGSelectionKey::CreateFromObjectPtr(InObject);

	auto CheckMap = [this, &ObjectKey, &bOutIsCulled](const FPCGSelectionKeyToSettingsMap& InMap) -> bool
	{
		for (const auto& It : InMap)
		{
			bool bFound = It.Key.IsMatching(ObjectKey, this);

			if (bFound)
			{
				bOutIsCulled = PCGSettings::IsKeyCulled(It.Value);
				return true;
			}
		}

		return false;
	};

	return CheckMap(ChangeTracker.StaticallyTrackedKeysToSettings) || CheckMap(ChangeTracker.DynamicallyTrackedKeysToSettings);
}

void UPCGComponent::OnRefresh(bool bForceRefresh)
{
	check(!IsManagedByRuntimeGenSystem());

	// Mark the refresh task invalid to allow re-triggering refreshes
	CurrentRefreshTask = InvalidPCGTaskId;

	// Before doing a refresh, update the component to the subsystem if we are partitioned
	// Only redo the mapping if we are generated
	UPCGSubsystem* Subsystem = GetSubsystem();
	const bool bWasGenerated = bGenerated;
	const bool bWasGeneratedOrGenerating = bWasGenerated || bForceRefresh || IsGenerating();

	const FPCGManagedResourceConstContainerHelper ContainerHelper(this);

	// If we are partitioned but we have resources, we need to force a cleanup
	if (IsPartitioned() && !ContainerHelper.IsEmpty())
	{
		CleanupLocalImmediate(/*bRemoveComponents=*/true);
	}

	if (Subsystem)
	{
		Subsystem->RegisterOrUpdateExecutionSource(this, /*bDoActorMapping=*/ bWasGeneratedOrGenerating);
	}

	// Following a change in some properties or in some spatial information related to this component,
	// We need to regenerate/cleanup the graph, depending of the state in the editor.
	if (!bActivated)
	{
		CleanupLocalImmediate(/*bRemoveComponents=*/true, /*bCleanupLocalComponents=*/true);
		// Retain our generated state when going inactive, and mark bDirtyGenerated so that the component will re-generate upon re-activation (if necessary).
		bGenerated = bWasGenerated;

		bDirtyGenerated = bWasGenerated;
	}
	else
	{
		// If we just cleaned up resources, call back generate. Only do this for original component, which will then trigger
		// generation of local components. Also, for BPs, we ask if we should generate, to support generate on added to world.
		if ((bWasGeneratedOrGenerating || ShouldGenerateBPPCGAddedToWorld()) && !IsLocalComponent() && (!bGenerated || bRegenerateInEditor))
		{
			GenerateLocal(/*bForce=*/false);
		}
	}
}
#endif // WITH_EDITOR

// The Actor Data Cache is a double buffered cache
// - When executing a graph we assign a CurrentGenerationTask to the Component.
// - When calling GetActorPCGData(), we will first try and find the Data inside the Execution Cache using the CurrentGenerationTask id
// - This guarantees that once that execution cache is primed, we will always return the same Data for the whole execution
// - If the Execution Cache doesn't contain the Data, we will first check the Component local cache to see if it is still valid (Can get invalidated by UPCGComponent::DirtyGenerated call or in some case if the landscape cache is dirty)
// - If the local Component Cache is valid, we will store the data in the Execution Cache for the following calls and return the data
// - If the local Component Cache isn't valid, we will create the cache Data, store the data in the Execution Cache and in the Component local cache
// - If CurrentGenerationTask is InvalidTaskId then only the Local Component Cache will be used
//
// This applies to GetActorPCGData/GetLandscapePCGData/GetLandscapeHeightData/GetInputPCGData/GetPCGData
UPCGData* UPCGComponent::GetPCGData() const
{
	// @todo_pcg: before removing bValidateWritable=false validate first that this method will always be called after a ExecutePreGraph call
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::PCGData,
		[this]()
		{
			if (!CachedPCGData)
			{
				CachedPCGData = CreatePCGData();
			}
			return CachedPCGData;
		}, /*bValidateWritable=*/false);
}

UPCGData* UPCGComponent::GetInputPCGData() const
{
	// @todo_pcg: before removing bValidateWritable=false validate first that this method will always be called after a ExecutePreGraph call
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::InputData,
		[this]()
		{
			if (!CachedInputData)
			{
				CachedInputData = CreateInputPCGData();
			}
			return CachedInputData;
		}, /*bValidateWritable=*/false);
}

UPCGData* UPCGComponent::GetActorPCGData() const
{
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::SelfData,
		[this]()
		{
			if (!CachedActorData || IsLandscapeCachedDataDirty(CachedActorData))
			{
				CachedActorData = CreateActorPCGData();
			}
			return CachedActorData;
		});
}

UPCGData* UPCGComponent::GetLandscapePCGData() const
{
	// @todo_pcg: before removing bValidateWritable=false validate first that this method will always be called after a ExecutePreGraph call
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::LandscapeData,
		[this]()
		{
			if (!CachedLandscapeData || IsLandscapeCachedDataDirty(CachedLandscapeData))
			{
				CachedLandscapeData = CreateLandscapePCGData(/*bHeightOnly=*/false);
			}
			return CachedLandscapeData;
		}, /*bValidateWritable=*/false);
}

UPCGData* UPCGComponent::GetLandscapeHeightPCGData() const
{
	// @todo_pcg: before removing bValidateWritable=false validate first that this method will always be called after a ExecutePreGraph call
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::LandscapeHeightData,
		[this]()
		{
			if (!CachedLandscapeHeightData || IsLandscapeCachedDataDirty(CachedLandscapeHeightData))
			{
				CachedLandscapeHeightData = CreateLandscapePCGData(/*bHeightOnly=*/true);
			}
			return CachedLandscapeHeightData;
		}, /*bValidateWritable=*/false);
}

UPCGData* UPCGComponent::GetOriginalActorPCGData() const
{
	APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(GetOwner());
	if (!PartitionActor)
	{
		return GetActorPCGData();
	}

	// Special case where we forward to original so do our own writable validation
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::OriginalSelfData,
		[this, PartitionActor]()
		{
			UPCGData* Data = nullptr;
			if (UPCGComponent* OriginalComponent = PartitionActor->GetOriginalComponent(this))
			{
				Data = OriginalComponent->GetActorPCGData();
				if (!Data)
				{
					PCG_EXECUTION_CACHE_VALIDATION_CHECK(this);
					Data = OriginalComponent->CreateActorPCGData();
				}
			}
			return Data;
		}, /*bValidateWritable=*/false);
}

UPCGComponent* UPCGComponent::GetOriginalComponent() const
{
	if (!IsLocalComponent())
	{
		return const_cast<UPCGComponent*>(this);
	}

	const APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(GetOwner());
	return ensure(PartitionActor) ? PartitionActor->GetOriginalComponent(this) : const_cast<UPCGComponent*>(this);
}

const UPCGComponent* UPCGComponent::GetConstOriginalComponent() const
{
	if (!IsLocalComponent())
	{
		return this;
	}

	const APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(GetOwner());
	return ensure(PartitionActor) ? PartitionActor->GetOriginalComponent(this) : this;
}

bool UPCGComponent::DoesGridDependOnWorldStreaming(uint32 InGridSize) const
{
	return SchedulingPolicy && SchedulingPolicy->DoesGridDependOnWorldStreaming(InGridSize);
}

UPCGData* UPCGComponent::CreateActorPCGData() const
{
	return CreateActorPCGData(GetOwner(), bParseActorComponents);
}

UPCGData* UPCGComponent::CreateActorPCGData(AActor* Actor, bool bParseActor) const
{
	return CreateActorPCGData(Actor, this, bParseActor);
}

UPCGData* UPCGComponent::CreateActorPCGData(AActor* Actor, const UPCGComponent* Component, bool bParseActor)
{
	FPCGDataCollection Collection = CreateActorPCGDataCollection(Actor, Component, EPCGDataType::Any, bParseActor);
	if (Collection.TaggedData.Num() > 1)
	{
		UPCGUnionData* Union = NewObject<UPCGUnionData>();
		for (const FPCGTaggedData& TaggedData : Collection.TaggedData)
		{
			Union->AddData(CastChecked<const UPCGSpatialData>(TaggedData.Data));
		}

		return Union;
	}
	else if(Collection.TaggedData.Num() == 1)
	{
		return const_cast<UPCGData*>(Collection.TaggedData[0].Data.Get());
	}
	else
	{
		return nullptr;
	}
}

FPCGDataCollection UPCGComponent::CreateActorPCGDataCollection(AActor* Actor, const UPCGComponent* Component, EPCGDataType InDataFilter, bool bParseActor, bool* bOutOptionalSanitizedTagAttributeName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGComponent::CreateActorPCGData);

	FPCGGetDataFunctionRegistryParams Params;
	Params.Source = Component;
	Params.bParseActor = bParseActor;
	Params.DataTypeFilter = InDataFilter;

	FPCGGetDataFunctionRegistryOutput Output;
	FPCGModule::ConstGetDataFunctionRegistry().GetDataFromActor(/*Context=*/nullptr, Params, Actor, Output);

	return Output.Collection;
}

void UPCGComponent::RefreshSchedulingPolicy()
{
	if (SchedulingPolicy && (!SchedulingPolicyClass || SchedulingPolicy->GetClass() != SchedulingPolicyClass || !IsManagedByRuntimeGenSystem()))
	{
		// Only delete it if we are the owner, it's for deprecation where local components had hard ref on original policy.
		if (IsValid(SchedulingPolicy) && SchedulingPolicy->GetOuter() == this)
		{
#if WITH_EDITOR
			// We are renaming to a new outer on an object that may still be loading. Since we are destroying this object
			// pass REN_AllowPackageLinkerMismatch to allow the linker to remain on the object so we don't have to force a load 
			// to complete before the rename 
			SchedulingPolicy->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
#endif

			SchedulingPolicy->MarkAsGarbage();
		}

		SchedulingPolicy = nullptr;
	}

	// We should never create the scheduling policy when not in runtime generation mode.
	if (SchedulingPolicyClass && !SchedulingPolicy && IsManagedByRuntimeGenSystem() && !IsLocalComponent())
	{
		const EObjectFlags Flags = GetMaskedFlags(RF_PropagateToSubObjects);
		SchedulingPolicy = NewObject<UPCGSchedulingPolicyBase>(this, SchedulingPolicyClass, NAME_None, Flags);
	}

#if WITH_EDITOR
	if (SchedulingPolicy)
	{
		SchedulingPolicy->SetShouldDisplayProperties(IsManagedByRuntimeGenSystem());
	}
#endif
}

void UPCGComponent::ExecutePreGraph(FPCGContext* InContext)
{
#if WITH_EDITOR
	StartGenerationInProgress();
#endif

	ensureMsgf(GetGenerationTaskId() != InvalidPCGTaskId, TEXT("Component was Scheduled for generation without having its CurrentGenerationTask assigned"));

	// Call getters which will create the data and cache it
	GetActorPCGData();
	GetOriginalActorPCGData();
	GetGridBounds();
	GetOriginalGridBounds();
	GetLocalSpaceBounds();
	GetOriginalLocalSpaceBounds();
	GetTotalBounds();
}

UPCGData* UPCGComponent::CreatePCGData() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGComponent::CreatePCGData);
	return GetInputPCGData();
}

UPCGData* UPCGComponent::CreateLandscapePCGData(bool bHeightOnly) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGComponent::CreateLandscapePCGData);
	AActor* Actor = GetOwner();

	if (!Actor)
	{
		return nullptr;
	}

	UPCGData* ActorData = GetActorPCGData();

	if (ALandscapeProxy* Landscape = Cast<ALandscapeProxy>(Actor))
	{
		return ActorData;
	}

	const UPCGSpatialData* ActorSpatialData = Cast<const UPCGSpatialData>(ActorData);

	FBox ActorBounds;

	if (ActorSpatialData)
	{
		ActorBounds = ActorSpatialData->GetBounds();
	}
	else
	{
		FVector Origin;
		FVector Extent;
		Actor->GetActorBounds(/*bOnlyCollidingComponents=*/false, Origin, Extent);
		ActorBounds = FBox::BuildAABB(Origin, Extent);
	}

	TArray<TWeakObjectPtr<ALandscapeProxy>> Landscapes = PCGHelpers::GetLandscapeProxies(Actor->GetWorld(), ActorBounds);

	if (Landscapes.IsEmpty())
	{
		// No landscape found
		return nullptr;
	}

	FBox LandscapeBounds(EForceInit::ForceInit);

	for (TWeakObjectPtr<ALandscapeProxy> Landscape : Landscapes)
	{
		if (Landscape.IsValid())
		{
			LandscapeBounds += GetGridBounds(Landscape.Get());
		}
	}

	// TODO: we're creating separate landscape data instances here so we can do some tweaks on it (such as storing the right target actor) but this probably should change
	UPCGLandscapeData* LandscapeData = NewObject<UPCGLandscapeData>();
	const UPCGGraph* PCGGraph = GetGraph();

	FPCGLandscapeDataProps LandscapeDataProps;
	LandscapeDataProps.bGetHeightOnly = bHeightOnly;
	LandscapeDataProps.bGetLayerWeights = (PCGGraph && PCGGraph->bLandscapeUsesMetadata);
	LandscapeDataProps.bSampleVirtualTextures = false;

	LandscapeData->Initialize(Landscapes, LandscapeBounds, LandscapeDataProps);

	return LandscapeData;
}

UPCGData* UPCGComponent::CreateInputPCGData() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGComponent::CreateInputPCGData);
	AActor* Actor = GetOwner();
	check(Actor);

	// Construct proper input based on input type
	if (InputType == EPCGComponentInput::Actor)
	{
		return GetActorPCGData();
	}
	else if (InputType == EPCGComponentInput::Landscape)
	{
		UPCGData* ActorData = GetActorPCGData();

		const UPCGSpatialData* ActorSpatialData = Cast<const UPCGSpatialData>(ActorData);

		if (!ActorSpatialData)
		{
			// TODO ? support non-spatial data on landscape?
			return nullptr;
		}

		const UPCGSpatialData* LandscapeData = Cast<const UPCGSpatialData>(GetLandscapePCGData());

		if (!LandscapeData)
		{
			return nullptr;
		}

		if (LandscapeData == ActorSpatialData)
		{
			return ActorData;
		}

		// Decide whether to intersect or project
		// Currently, it makes sense to intersect only for volumes;
		// Note that we don't currently check for a volume object but only on dimension
		// so intersections (such as volume X partition actor) get picked up properly
		if (ActorSpatialData->GetDimension() >= 3)
		{
			return LandscapeData->IntersectWith(nullptr, ActorSpatialData);
		}
		else
		{
			return ActorSpatialData->ProjectOn(nullptr, LandscapeData);
		}
	}
	else
	{
		// In this case, the input data will be provided in some other form,
		// Most likely to be stored in the PCG data grid.
		return nullptr;
	}
}

bool UPCGComponent::IsLandscapeCachedDataDirty(const UPCGData* Data) const
{
	bool IsCacheDirty = false;

	if (const UPCGLandscapeData* CachedData = Cast<UPCGLandscapeData>(Data))
	{
		const UPCGGraph* PCGGraph = GetGraph();
		IsCacheDirty = PCGGraph && (CachedData->IsUsingMetadata() != PCGGraph->bLandscapeUsesMetadata);
	}

	return IsCacheDirty;
}

FBox UPCGComponent::GetGridBounds() const
{
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::Bounds,
		[this]()
		{
			return PCGHelpers::GetGridBounds(GetOwner(), this);
		});
}

FBox UPCGComponent::GetOriginalGridBounds() const
{
	const APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(GetOwner());
	if (!PartitionActor)
	{
		return GetGridBounds();
	}

	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::OriginalBounds,
		[this, PartitionActor]()
		{
			const UPCGComponent* OriginalComponent = PartitionActor->GetOriginalComponent(this);
			const UPCGComponent* ValidComponent = OriginalComponent ? OriginalComponent : this;
			return PCGHelpers::GetGridBounds(ValidComponent->GetOwner(), ValidComponent);
		});
}

FBox UPCGComponent::GetLocalSpaceBounds() const
{
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::LocalSpaceBounds,
		[this]()
		{
			return PCGHelpers::GetActorLocalBounds(GetOwner());
		});
}

FBox UPCGComponent::GetOriginalLocalSpaceBounds() const
{
	const APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(GetOwner());
	if (!PartitionActor)
	{
		return GetLocalSpaceBounds();
	}

	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::OriginalLocalSpaceBounds,
		[this, PartitionActor]()
		{
			const UPCGComponent* OriginalComponent = PartitionActor->GetOriginalComponent(this);
			const UPCGComponent* ValidComponent = OriginalComponent ? OriginalComponent : this;
			return PCGHelpers::GetActorLocalBounds(ValidComponent->GetOwner());
		});
}

FBox UPCGComponent::GetTotalBounds() const
{
	/* Total bounds are cached at the beginning of execution through UPCGComponent::ExecutePreGraph for tracking purposes of the previously generated resources. */
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::TotalBounds,
		[this]()
		{
			return PCGHelpers::GetActorBounds(GetOwner(), /*bIgnorePCGCreatedComponents=*/false);
		});
}

FBox UPCGComponent::GetGridBounds(const AActor* Actor) const
{
	return PCGHelpers::GetGridBounds(Actor, this);
}

UPCGSubsystem* UPCGComponent::GetSubsystem() const
{
	return GetOwner() ? UPCGSubsystem::GetInstance(GetOwner()->GetWorld()) : nullptr;
}

#if WITH_EDITOR
void UPCGComponent::ApplyToEachSettings(const FPCGSelectionKey& InKey, const TFunctionRef<void(const FPCGSelectionKey&, const FPCGSettingsAndCulling&)> InCallback) const
{
	ChangeTracker.ForEachSettingTrackingKey(InKey, InCallback);
}

TArray<FPCGSelectionKey> UPCGComponent::GatherTrackingKeys() const
{
	return ChangeTracker.GatherTrackingKeys();
}

bool UPCGComponent::IsKeyTrackedAndCulled(const FPCGSelectionKey& Key, bool& bOutIsCulled) const
{
	return ChangeTracker.IsKeyTrackedAndCulled(Key, bOutIsCulled);
}

// deprecated
bool UPCGComponent::ShouldTrackLandscape() const
{
	const UPCGGraph* PCGGraph = GetGraph();
	
	// We should track the landscape if the landscape pins are connected, or if the input type is Landscape and we are using the Input pin.
	const bool bUseLandscapePin = PCGGraph &&
		(PCGGraph->GetInputNode()->IsOutputPinConnected(PCGInputOutputConstants::DefaultLandscapeLabel) ||
		PCGGraph->GetInputNode()->IsOutputPinConnected(PCGInputOutputConstants::DefaultLandscapeHeightLabel));


	const bool bHasLandscapeHasInput = PCGGraph && InputType == EPCGComponentInput::Landscape 
		&& Algo::AnyOf(PCGGraph->GetInputNode()->GetOutputPins(), [](const UPCGPin* InPin) { return InPin && InPin->IsConnected(); });

	return bUseLandscapePin || bHasLandscapeHasInput;
}

void UPCGComponent::RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling)
{
	ChangeTracker.RegisterDynamicTracking(InSettings, InDynamicKeysAndCulling);
}

void UPCGComponent::RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings)
{
	ChangeTracker.RegisterDynamicTracking(InKeysToSettings);
}

void UPCGComponent::UpdateDynamicTracking()
{
	ChangeTracker.UpdateDynamicTracking(this);
}

void UPCGComponent::StartIgnoringChangeOriginDuringGeneration(const UObject* InChangeOriginToIgnore)
{
	StartIgnoringChangeOriginsDuringGeneration(TArrayView<const UObject*>(&InChangeOriginToIgnore, 1));
}

void UPCGComponent::StartIgnoringChangeOriginsDuringGeneration(const TArrayView<const UObject*> InChangeOriginsToIgnore)
{
	UE::TWriteScopeLock Lock(IgnoredChangeOriginsLock);
	for (const UObject* InChangeOriginToIgnore : InChangeOriginsToIgnore)
	{
		if (int32* FoundCounter = IgnoredChangeOriginsToCounters.Find(InChangeOriginToIgnore))
		{
			int32& Counter = *FoundCounter; // Put in local variable to evade SA warnings.
			ensure(Counter >= 0);

			Counter = FMath::Max(0, Counter) + 1;
		}
		else
		{
			IgnoredChangeOriginsToCounters.Add(InChangeOriginToIgnore, 1);
		}
	}
}

void UPCGComponent::StopIgnoringChangeOriginDuringGeneration(const UObject* InChangeOriginToIgnore)
{
	StopIgnoringChangeOriginsDuringGeneration(TArrayView<const UObject*>(&InChangeOriginToIgnore, 1));
}

void UPCGComponent::StopIgnoringChangeOriginsDuringGeneration(const TArrayView<const UObject*> InChangeOriginsToIgnore)
{
	UE::TWriteScopeLock Lock(IgnoredChangeOriginsLock);

	for (const UObject* InChangeOriginToIgnore : InChangeOriginsToIgnore)
	{
		int32* FoundCounter = IgnoredChangeOriginsToCounters.Find(InChangeOriginToIgnore);
		if (ensure(FoundCounter))
		{
			int32& Counter = *FoundCounter; // Put in local variable to evade SA warnings.
			ensure(Counter > 0);

			if (--Counter <= 0)
			{
				IgnoredChangeOriginsToCounters.Remove(InChangeOriginToIgnore);
			}
		}
	}
}

bool UPCGComponent::IsIgnoringChangeOrigin(const UObject* InChangeOrigin) const 
{
	const UObject* Dummy = nullptr;
	return IsIgnoringAnyChangeOrigins(TArrayView<const UObject*>(&InChangeOrigin, 1), Dummy);
}

bool UPCGComponent::IsIgnoringAnyChangeOrigins(const TArrayView<const UObject*> InChangeOrigins, const UObject*& OutFirstObjectFound) const
{
	EPCGIgnoreChangeOriginReason Reason;
	return IsIgnoringAnyChangeOrigins(InChangeOrigins, OutFirstObjectFound, Reason);
}

bool UPCGComponent::IsIgnoringAnyChangeOrigins(const TArrayView<const UObject*> InChangeOrigins, const UObject*& OutFirstObjectFound, EPCGIgnoreChangeOriginReason& OutReason) const
{
	OutReason = EPCGIgnoreChangeOriginReason::None;

	if ((bIgnoreLandscapeTracking || (GetGraph() && GetGraph()->bIgnoreLandscapeTracking)))
	{
		const int32 Index = Algo::IndexOfByPredicate(InChangeOrigins, [](const UObject* ChangeOrigin) { return Cast<ALandscapeProxy>(ChangeOrigin); });
		if (Index != INDEX_NONE)
		{
			OutFirstObjectFound = InChangeOrigins[Index];
			OutReason = bIgnoreLandscapeTracking ? EPCGIgnoreChangeOriginReason::ComponentDisabledLandscapeTracking : EPCGIgnoreChangeOriginReason::GraphDisabledLandscapeTracking;
			return true;
		}
	}

	{
		UE::TReadScopeLock Lock(IgnoredChangeOriginsLock);
		for (const UObject* ChangeOrigin : InChangeOrigins)
		{
			const int32* Counter = IgnoredChangeOriginsToCounters.Find(ChangeOrigin);
			if (Counter && ensure(*Counter > 0))
			{
				OutFirstObjectFound = ChangeOrigin;
				OutReason = EPCGIgnoreChangeOriginReason::GenerationIgnoredObject;
				return true;
			}
		}
	}

	if (UPCGGraph* Graph = GetGraph())
	{
		for (UClass* ChangeTrackingIgnoredClass : Graph->GetChangeTrackingIgnoredClasses())
		{
			for (const UObject* ChangeOrigin : InChangeOrigins)
			{
				if(ChangeOrigin && ChangeOrigin->IsA(ChangeTrackingIgnoredClass))
				{
					OutFirstObjectFound = ChangeOrigin;
					OutReason = EPCGIgnoreChangeOriginReason::GraphIgnoredClass;
					return true;
				}
			}
		}
	}
	
	return false;
}

void UPCGComponent::ResetIgnoredChangeOrigins(bool bLogIfAnyPresent)
{
	UE::TWriteScopeLock Lock(IgnoredChangeOriginsLock);

	if (bLogIfAnyPresent && !IgnoredChangeOriginsToCounters.IsEmpty())
	{
		UE_LOG(LogPCG, Warning, TEXT("[%s/%s] ResetIgnoredChangeOrigins: IgnoredChangeOrigins should be empty but %d found, purged."),
			GetOwner() ? *GetOwner()->GetName() : TEXT("MISSINGACTOR"),
			*GetName(),
			IgnoredChangeOriginsToCounters.Num());
	}

	IgnoredChangeOriginsToCounters.Reset();
}
#endif // WITH_EDITOR

void UPCGComponent::SetManagedResources(const TArray<TObjectPtr<UPCGManagedResource>>& Resources)
{
	FPCGManagedResourceContainerHelper ContainerHelper(this);
	ContainerHelper.SetManagedResources(Resources);
}

void UPCGComponent::GetManagedResources(TArray<TObjectPtr<UPCGManagedResource>>& Resources) const
{
	const FPCGManagedResourceConstContainerHelper ContainerHelper(this);
	Resources =	ContainerHelper.GetManagedResourcesCopy();
}

void UPCGComponent::SetEditingMode(EPCGEditorDirtyMode InEditingMode, EPCGEditorDirtyMode InSerializedEditingMode)
{
	CurrentEditingMode = InEditingMode;
	SerializedEditingMode = InSerializedEditingMode;
}

#if WITH_EDITOR
bool UPCGComponent::DeletePreviewResources()
{
	FPCGManagedResourceContainerHelper ContainerHelper(this);
	return ContainerHelper.DeletePreviewResources();
}

void UPCGComponent::MarkResourcesAsTransientOnLoad()
{
	FPCGManagedResourceContainerHelper ContainerHelper(this);
	ContainerHelper.MarkResourcesAsTransientOnLoad();

	// Force duplicate owner for PIE so that transient changes are reflected in game
	if (AActor* Owner = GetOwner())
	{
		Owner->bForceDuplicateInPIE = true;
	}
}

void UPCGComponent::ChangeTransientState(EPCGEditorDirtyMode NewEditingMode)
{
	// Changing the transient state can and will play with packages and those changes are not meant to be tracked by the transaction buffer.
	// Instead, UPCGComponent will react to CurrentEditingMode changes between PreEditUndo/PostEditUndo and call ChangeTransientState accordingly to reflect the change to CurrentEditingMode.
	TGuardValue<ITransaction*> SuppressTransaction(GUndo, nullptr);

	FPCGManagedResourceContainerHelper ContainerHelper(this);
	bool bShouldMarkDirty = ContainerHelper.ChangeTransientState(NewEditingMode);

	if (NewEditingMode != EPCGEditorDirtyMode::Preview)
	{
		bShouldMarkDirty |= (LoadedPreviewGeneratedGraphOutput != GeneratedGraphOutput);
		LoadedPreviewGeneratedGraphOutput.Reset();
	}

	if (IsLocalComponent())
	{
		if (NewEditingMode == EPCGEditorDirtyMode::Preview)
		{
			bShouldMarkDirty = true;
			MarkPackageDirty();
		}

		if (NewEditingMode == EPCGEditorDirtyMode::Preview)
		{
			SetFlags(RF_Transient);
		}
		else
		{
			ClearFlags(RF_Transient);
		}

		ForEachObjectWithOuter(this, [NewEditingMode](UObject* Object)
		{
			if (NewEditingMode == EPCGEditorDirtyMode::Preview)
			{
				Object->SetFlags(RF_Transient);
			}
			else
			{
				Object->ClearFlags(RF_Transient);
			}
		});

		if (NewEditingMode != EPCGEditorDirtyMode::Preview)
		{
			bShouldMarkDirty = true;
			MarkPackageDirty();
		}
	}
	else if (bShouldMarkDirty)
	{
		MarkPackageDirty();
	}

	// Un-transient PAs if needed and propagate the call
	if (IsPartitioned())
	{
		if (UPCGSubsystem* Subsystem = GetSubsystem())
		{
			Subsystem->PropagateEditingModeToLocalComponents(this, NewEditingMode);
		}
	}
}

bool UPCGComponent::GetStackContext(FPCGStackContext& OutStackContext) const
{
	UPCGSubsystem* Subsystem = GetSubsystem();
	if (Subsystem && Subsystem->GetStackContext(this, OutStackContext))
	{
		FPCGStack ComponentStack;
		ComponentStack.PushFrame(this);
		OutStackContext.PrependParentStack(&ComponentStack);
		return true;
	}

	return false;
}

TArray<TSoftObjectPtr<AActor>> UPCGComponent::GetManagedActorPaths(AActor* InActor)
{
	TSet<TSoftObjectPtr<AActor>> ManagedActorPaths;
	InActor->ForEachComponent<UPCGComponent>(/*bIncludeFromChildActors=*/true, [&ManagedActorPaths](UPCGComponent* Component)
	{
		FPCGManagedResourceContainerHelper ContainerHelper(Component);
		PCG::TScopeLock ResourceLock(Component->GeneratedResourcesLock);
		if (ensure(ContainerHelper.AreResourcesAccessible()))
		{
			for (const UPCGManagedResource* ManagedResource : ContainerHelper.GetConstManagedResourcesNoLock())
			{
				if (const UPCGManagedActors* ManagedActors = Cast<UPCGManagedActors>(ManagedResource))
				{
					ManagedActorPaths.Append(ManagedActors->GetConstGeneratedActors());
				}
			}
		}
	});

	return ManagedActorPaths.Array();
}

#endif // WITH_EDITOR

FPCGComponentInstanceData::FPCGComponentInstanceData(const UPCGComponent* InSourceComponent)
	: FActorComponentInstanceData(InSourceComponent)
	, SourceComponent(InSourceComponent)
#if WITH_EDITOR
	, CurrentEditingMode(InSourceComponent->CurrentEditingMode)
	, PreviousEditingMode(InSourceComponent->PreviousEditingMode)
	, bForceGenerateOnBPAddedToWorld(InSourceComponent->bForceGenerateOnBPAddedToWorld)
	, bGeneratedOffline(InSourceComponent->bGeneratedOffline)
#endif
{
#if WITH_EDITOR
	auto CopyFromTrackingKeys = [](const FPCGSelectionKeyToSettingsMap& InKeysToSettingsMap, TArray<FPCGTrackedKeyInstanceData>& OutTrackingKeys)
	{
		for (const auto& KeyValuePair : InKeysToSettingsMap)
		{
			for (const auto& SettingsAndCullingPair : KeyValuePair.Value)
			{
				OutTrackingKeys.Add(FPCGTrackedKeyInstanceData { KeyValuePair.Key, SettingsAndCullingPair.Key, SettingsAndCullingPair.Value });
			}
		}
	};
	
	CopyFromTrackingKeys(InSourceComponent->ChangeTracker.DynamicallyTrackedKeysToSettings, DynamicallyTrackedKeysToSettings);
	CopyFromTrackingKeys(InSourceComponent->ChangeTracker.StaticallyTrackedKeysToSettings, StaticallyTrackedKeysToSettings);
#endif
}

bool FPCGComponentInstanceData::ContainsData() const
{
	return true;
}

void FPCGComponentInstanceData::ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase)
{
	Super::ApplyToComponent(Component, CacheApplyPhase);

	if (CacheApplyPhase == ECacheApplyPhase::PostUserConstructionScript)
	{
		UPCGComponent* PCGComponent = CastChecked<UPCGComponent>(Component);

		// IMPORTANT NOTE:
		// ConstructionSourceComponent: The previous instance of the PCGComponent (not related to annotation, will be the OldComponent in the OnObjectsReplaced call)
		// We will transfer/copy properties/resources from this component that flow forward only (aren't undo/redoable)
		//
		// SourceComponent: The previous instance (same as ConstructionSourceComponent on a regular edit but an annotated Component when in a Undo/Redo operation)
		// We will transfer/copy properties from this component that should reflect undo redo (annotation)
		const UPCGComponent* ConstructionSourceComponent = SourceComponent;
#if WITH_EDITOR
		if (UPCGSubsystem* Subsystem = PCGComponent->GetSubsystem())
		{
			UPCGComponent* FoundConstructionSourceComponent = nullptr;
			if(Subsystem->RemoveAndCopyConstructionScriptSourceComponent(Component->GetOwner(), Component->GetFName(), FoundConstructionSourceComponent))
			{
				ConstructionSourceComponent = FoundConstructionSourceComponent;
			}
		}

		// Any members:
		// 
		// - UPROPERTY() with no specifiers
		// - Transient UPROPERTY
		// - non UPROPERTY
		// 
		// are NOT copied over when re-running the construction script and need to be handled by the instance data so we reapply them here.
		PCGComponent->bForceGenerateOnBPAddedToWorld = bForceGenerateOnBPAddedToWorld;
		PCGComponent->PreviousEditingMode = PreviousEditingMode;	
		PCGComponent->CurrentEditingMode = CurrentEditingMode;
		PCGComponent->bGeneratedOffline = bGeneratedOffline;

		auto CopyToTrackingKeys = [](const TArray<FPCGTrackedKeyInstanceData>& InTrackingKeys, FPCGSelectionKeyToSettingsMap& OutKeysToSettingsMap)
		{
			OutKeysToSettingsMap.Empty();
			for(const FPCGTrackedKeyInstanceData& InstanceData : InTrackingKeys)
			{
				OutKeysToSettingsMap.FindOrAdd(InstanceData.SelectionKey).Add({ InstanceData.Settings , InstanceData.bValue });
			}
		};

		CopyToTrackingKeys(StaticallyTrackedKeysToSettings, PCGComponent->ChangeTracker.StaticallyTrackedKeysToSettings);
		CopyToTrackingKeys(DynamicallyTrackedKeysToSettings, PCGComponent->ChangeTracker.DynamicallyTrackedKeysToSettings);

#endif

		if (ConstructionSourceComponent)
		{
			const FPCGManagedResourceConstContainerHelper ConstructionContainerHelper(ConstructionSourceComponent);

			// Critical: LastGeneratedBounds
			PCGComponent->LastGeneratedBounds = ConstructionSourceComponent->LastGeneratedBounds;

			// Duplicate generated resources + retarget them
#if WITH_EDITOR
			TMap<TObjectPtr<UPCGManagedResource>, TObjectPtr<UPCGManagedResource>> GeneratedResourceMapping;
#endif
			TArray<TObjectPtr<UPCGManagedResource>> DuplicatedResources;
			for (const TObjectPtr<UPCGManagedResource>& Resource : ConstructionContainerHelper.GetConstManagedResourcesNoLock())
			{
				if (Resource)
				{
					UPCGManagedResource* DuplicatedResource = CastChecked<UPCGManagedResource>(StaticDuplicateObject(Resource, PCGComponent, FName()));
					DuplicatedResource->PostApplyToComponent();
					DuplicatedResources.Add(DuplicatedResource);

#if WITH_EDITOR
					GeneratedResourceMapping.Add(Resource, DuplicatedResource);
#endif
				}
			}

			if (DuplicatedResources.Num() > 0)
			{
				PCGComponent->SetManagedResources(MoveTemp(DuplicatedResources));
			}

#if WITH_EDITOR
			// bDirtyGenerated is transient.
			PCGComponent->bDirtyGenerated = ConstructionSourceComponent->bDirtyGenerated;

			TArray<TObjectPtr<UPCGManagedResource>> DuplicateLoadedPreviewResources;
			for (const TObjectPtr<UPCGManagedResource>& Resource : ConstructionContainerHelper.GetConstLoadedPreviewResourcesNoLock())
			{
				if (Resource)
				{
					UPCGManagedResource* DuplicatedResource = nullptr;
					if (TObjectPtr<UPCGManagedResource>* FoundDuplicate = GeneratedResourceMapping.Find(Resource))
					{
						DuplicatedResource = *FoundDuplicate;
					}
					else
					{
						DuplicatedResource = CastChecked<UPCGManagedResource>(StaticDuplicateObject(Resource, PCGComponent, FName()));
						DuplicatedResource->PostApplyToComponent();
					}
					DuplicateLoadedPreviewResources.Add(DuplicatedResource);
				}
			}

			if (DuplicateLoadedPreviewResources.Num() > 0)
			{
				FPCGManagedResourceContainerHelper ContainerHelper(PCGComponent);
				ContainerHelper.SetLoadedPreviewResourcesNoLock(MoveTemp(DuplicateLoadedPreviewResources));
			}

			PCGComponent->LoadedPreviewGeneratedGraphOutput = ConstructionSourceComponent->LoadedPreviewGeneratedGraphOutput;

			for (FPCGTaggedData& LoadedPreviewTaggedData : PCGComponent->LoadedPreviewGeneratedGraphOutput.TaggedData)
			{
				if (LoadedPreviewTaggedData.Data)
				{
					LoadedPreviewTaggedData.Data = CastChecked<UPCGData>(StaticDuplicateObject(LoadedPreviewTaggedData.Data.Get(), PCGComponent));
				}
			}

			PCGComponent->bWasGeneratedThisSession = ConstructionSourceComponent->bWasGeneratedThisSession;

			PCGComponent->ExecutionInspection.InspectionCounter = ConstructionSourceComponent->ExecutionInspection.InspectionCounter;
			PCGComponent->TrackingPriority = PCGComponent::RoundTrackingPriority(PCGComponent->TrackingPriority);
#endif
		}

#if WITH_EDITOR
		// Reconnect callbacks
		if (PCGComponent->GraphInstance)
		{
			PCGComponent->GraphInstance->SetupCallbacks();
			PCGComponent->GraphInstance->OnGraphChangedDelegate.RemoveAll(PCGComponent);
			PCGComponent->GraphInstance->OnGraphChangedDelegate.AddUObject(PCGComponent, &UPCGComponent::OnGraphChanged);
		}
#endif // WITH_EDITOR

		bool bDoActorMapping = PCGComponent->bGenerated || PCGHelpers::IsRuntimeOrPIE();

		// If the generation mode or the policy class is changed, we won't receive a PostEditChange event, because this is a BP. Since it will not trigger a policy update, do it here.
		PCGComponent->RefreshSchedulingPolicy();

		// Also remap
		UPCGSubsystem* Subsystem = PCGComponent->GetSubsystem();
		if (Subsystem && ConstructionSourceComponent)
		{
			Subsystem->RemapExecutionSource(ConstructionSourceComponent, PCGComponent, bDoActorMapping);
		}

#if WITH_EDITOR
		// Disconnect callbacks on source.
		if (ConstructionSourceComponent)
		{
			if (ConstructionSourceComponent->GraphInstance)
			{
				ConstructionSourceComponent->GraphInstance->TeardownCallbacks();
			}
		}

		// Make sure to apply latest editing mode
		PCGComponent->ChangeTransientState(PCGComponent->CurrentEditingMode);

		// Finally, start a delayed refresh task (if there is not one already), in editor only
		// It is important to be delayed, because we cannot spawn Partition Actors within this scope,
		// because we are in a construction script.
		// Note that we only do this if we are not currently loading
		if (!ConstructionSourceComponent || !ConstructionSourceComponent->HasAllFlags(RF_WasLoaded))
		{
			PCGComponent->Refresh();
		}
#endif // WITH_EDITOR

		// Copy over the source data container
		if (ConstructionSourceComponent)
		{
			PCGComponent->SourceDataContainer = ConstructionSourceComponent->SourceDataContainer;
		}
	}
}

void FPCGComponentInstanceData::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(SourceComponent);
}

#undef LOCTEXT_NAMESPACE
