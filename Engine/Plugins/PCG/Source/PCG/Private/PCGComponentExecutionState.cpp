// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGComponentExecutionState.h"

#include "PCGComponent.h"
#include "PCGSubsystem.h"
#include "Grid/PCGPartitionActor.h"

#include "LandscapeProxy.h"
#include "GameFramework/Actor.h"

UPCGData* FPCGComponentExecutionState::GetSelfData() const
{
	check(Component);
	return Component->GetActorPCGData();
}

int32 FPCGComponentExecutionState::GetSeed() const
{
	check(Component);
	return Component->Seed;
}

FString FPCGComponentExecutionState::GetDebugName() const
{
	check(Component && Component->GetOwner());
	return Component->GetOwner()->GetName();
}

FTransform FPCGComponentExecutionState::GetTransform() const
{
	check(Component && Component->GetOwner());
	return Component->GetOwner()->GetTransform();
}

FTransform FPCGComponentExecutionState::GetOriginalTransform() const
{
	if (IPCGGraphExecutionSource* OriginalSource = GetOriginalSource())
	{
		return OriginalSource->GetExecutionState().GetTransform();
	}
	else
	{
		return GetTransform();
	}
}

UWorld* FPCGComponentExecutionState::GetWorld() const
{
	check(Component);
	return Component->GetWorld();
}

UObject* FPCGComponentExecutionState::GetTarget() const 
{
	check(Component);
	return Component->GetOwner();
}

bool FPCGComponentExecutionState::IsActive() const
{
	check(Component);
	return Component->bActivated;
}

bool FPCGComponentExecutionState::HasAuthority() const
{
	check(Component && Component->GetOwner());
	return Component->GetOwner()->HasAuthority();
}

FBox FPCGComponentExecutionState::GetBounds() const
{
	check(Component);
	return Component->GetGridBounds();
}

FBox FPCGComponentExecutionState::GetOriginalBounds() const
{
	check(Component);
	return Component->GetOriginalGridBounds();
}

FBox FPCGComponentExecutionState::GetLocalSpaceBounds() const
{
	check(Component);
	return Component->GetLocalSpaceBounds();
}

FBox FPCGComponentExecutionState::GetOriginalLocalSpaceBounds() const
{
	check(Component);
	return Component->GetOriginalLocalSpaceBounds();
}

FBox FPCGComponentExecutionState::GetTotalBounds() const
{
	check(Component);
	return Component->GetTotalBounds();
}

bool FPCGComponentExecutionState::Use2DGrid() const
{
	check(Component);
	return Component->Use2DGrid();
}

UPCGGraph* FPCGComponentExecutionState::GetGraph() const
{
	check(Component);
	return Component->GetGraph();
}

UPCGGraphInstance* FPCGComponentExecutionState::GetGraphInstance() const
{
	check(Component);
	return Component->GetGraphInstance();
}

void FPCGComponentExecutionState::OnGraphExecutionAborted(bool bQuiet, bool bCleanupUnusedResources)
{
	check(Component);
	Component->OnProcessGraphAborted(bQuiet, bCleanupUnusedResources);
}

void FPCGComponentExecutionState::Cancel()
{
	check(Component);
	Component->CancelGeneration();
}

bool FPCGComponentExecutionState::IsGenerating() const
{
	check(Component);
	return Component->IsGenerating();
}

bool FPCGComponentExecutionState::IsGenerated() const
{
	check(Component);
	return Component->bGenerated;
}

FBox FPCGComponentExecutionState::GetGeneratedBounds() const
{
	check(Component);
	return Component->GetLastGeneratedBounds();
}

void FPCGComponentExecutionState::ExecutePreGraph(FPCGContext* InContext)
{
	check(Component);
	Component->ExecutePreGraph(InContext);
}

bool FPCGComponentExecutionState::IsManagedByRuntimeGenSystem() const
{
	check(Component);
	return Component->IsManagedByRuntimeGenSystem();
}

const UPCGSchedulingPolicyBase* FPCGComponentExecutionState::GetRuntimeGenSchedulingPolicy() const
{
	check(Component);
	return Component->GetRuntimeGenSchedulingPolicy();
}

const FPCGRuntimeGenerationRadii& FPCGComponentExecutionState::GetGenerationRadii() const
{
	check(Component);
	return Component->GetGenerationRadii();
}

double FPCGComponentExecutionState::GetGenerationRadiusFromGrid(uint32 InGrid) const
{
	check(Component);
	return Component->GetGenerationRadiusFromGrid(InGrid);
}

double FPCGComponentExecutionState::GetCleanupRadiusFromGrid(uint32 InGrid) const
{
	check(Component);
	return Component->GetCleanupRadiusFromGrid(InGrid);
}

IPCGGraphExecutionSource* FPCGComponentExecutionState::GetOriginalSource() const
{
	check(Component);
	return Component->GetOriginalComponent();
}

IPCGGraphExecutionSource* FPCGComponentExecutionState::GetLocalSource(const FPCGGridDescriptor& GridDescriptor, const FIntVector& CellCoords) const
{
	check(Component);

	if (UPCGSubsystem* Subsystem = Cast<UPCGSubsystem>(GetSubsystem()))
	{
		return Subsystem->GetLocalComponent(GridDescriptor, CellCoords, CastChecked<UPCGComponent>(GetOriginalSource()));
	}

	return nullptr;
}

bool FPCGComponentExecutionState::IsPartitioned() const
{
	check(Component);
	return Component->IsPartitioned();
}

bool FPCGComponentExecutionState::IsLocalSource() const
{
	check(Component);
	return Component->IsLocalComponent();
}

FPCGGridDescriptor FPCGComponentExecutionState::GetGridDescriptor(uint32 InGridSize) const
{
	check(Component);
	return Component->GetGridDescriptor(InGridSize);
}

uint32 FPCGComponentExecutionState::GetGenerationGridSize() const
{
	check(Component);
	return (IsPartitioned() || IsLocalSource()) ? Component->GetGenerationGridSize() : PCGHiGenGrid::UninitializedGridSize();
}

FIntVector FPCGComponentExecutionState::GetGenerationGridCoords() const
{
	check(Component);

	if (APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(Component->GetOwner()))
	{
		return PartitionActor->GetGridCoord();
	}

	return FIntVector::ZeroValue;
}

bool FPCGComponentExecutionState::IsInPreviewMode() const
{
	check(Component);
	return Component->IsInPreviewMode();
}

FPCGTaskId FPCGComponentExecutionState::GetGenerationTaskId() const
{
	check(Component);
	return Component->GetGenerationTaskId();
}

void FPCGComponentExecutionState::StoreOutputDataForPin(const FString& InResourceKey, const FPCGDataCollection& InData) const
{
	check(Component);
	Component->StoreOutputDataForPin(InResourceKey, InData);
}

const FPCGDataCollection* FPCGComponentExecutionState::RetrieveOutputDataForPin(const FString& InResourceKey) const
{
	check(Component);
	return Component->RetrieveOutputDataForPin(InResourceKey);
}

void FPCGComponentExecutionState::OnManagedResourceAdded(UPCGManagedResource* InResource)
{
	check(Component);
	Component->OnManagedResourceAdded(InResource);
}

FPCGManagedResourceContainer* FPCGComponentExecutionState::GetManagedResourceContainer()
{
	check(Component);
	return &Component->ManagedResourceContainer;
}

FTransactionallySafeCriticalSection* FPCGComponentExecutionState::GetManagedResourceContainerLock()
{
	check(Component);
	return &Component->GeneratedResourcesLock;
}

FPCGTaskId FPCGComponentExecutionState::Generate(const FGenerateParams& InGenerateParams)
{
	check(Component);

	const uint32 Grid = InGenerateParams.bGenerateLocalSources ? PCGHiGenGrid::UninitializedGridSize() : GetGenerationGridSize();
	const EPCGComponentGenerationTrigger GenTrigger = IsManagedByRuntimeGenSystem() ? EPCGComponentGenerationTrigger::GenerateAtRuntime : EPCGComponentGenerationTrigger::GenerateOnDemand;
	return Component->GenerateLocalGetTaskId(GenTrigger, InGenerateParams.bEvenIfAlreadyGenerated, Grid, InGenerateParams.Dependencies);
}

FPCGSourceDataContainer* FPCGComponentExecutionState::GetSourceDataContainer()
{
	check(Component);
	return &Component->SourceDataContainer;
}

const FPCGSourceDataContainer* FPCGComponentExecutionState::GetSourceDataContainer() const
{
	check(Component);
	return &Component->SourceDataContainer;
}

// UE_DEPRECATED(5.8)
FPCGTaskId FPCGComponentExecutionState::GenerateLocalGetTaskId(EPCGHiGenGrid Grid)
{
	FGenerateParams GenerateParams;
	GenerateParams.bEvenIfAlreadyGenerated = true;
	GenerateParams.bGenerateLocalSources = Grid == EPCGHiGenGrid::Uninitialized;

	return Generate(GenerateParams);
}

FPCGTaskId FPCGComponentExecutionState::Cleanup(const FCleanupParams& InCleanupParams)
{
	check(Component);

	if (InCleanupParams.bImmediate)
	{
		Component->CleanupLocalImmediate(InCleanupParams.bReleaseManagedResources, InCleanupParams.bCleanupLocalSources);
		return InvalidPCGTaskId;
	}

	// Partitioned components will always cleanup their LCs, so warn if that option is toggled off
	if (!InCleanupParams.bCleanupLocalSources && IsPartitioned())
	{
		UE_LOGF(LogPCG, Warning, "Tried to schedule cleanup on a partitioned original execution source with bCleanupLocalSources=false, but that is not supported. Local components will be cleaned up.");
		// doesn't need to return, can still clean up the OC.
	}

	return Component->CleanupLocal(InCleanupParams.bReleaseManagedResources, InCleanupParams.Dependencies);
}

#if PCG_PROFILING_ENABLED
const PCGUtils::FExtraCapture& FPCGComponentExecutionState::GetExtraCapture() const
{
	check(Component);
	return Component->ExtraCapture;
}

PCGUtils::FExtraCapture& FPCGComponentExecutionState::GetExtraCapture()
{
	check(Component);
	return Component->ExtraCapture;
}

const FPCGGraphExecutionInspection& FPCGComponentExecutionState::GetInspection() const
{
	check(Component);
	return Component->ExecutionInspection;
}

FPCGGraphExecutionInspection& FPCGComponentExecutionState::GetInspection()
{
	check(Component);
	return Component->ExecutionInspection;
}
#endif // PCG_PROFILING_ENABLED

#if WITH_EDITOR
void FPCGComponentExecutionState::RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling)
{
	check(Component);
	Component->RegisterDynamicTracking(InSettings, InDynamicKeysAndCulling);
}

void FPCGComponentExecutionState::RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings)
{
	check(Component);
	Component->RegisterDynamicTracking(InKeysToSettings);
}

TArray<FPCGSelectionKey> FPCGComponentExecutionState::GatherTrackingKeys() const
{
	check(Component);
	return Component->GatherTrackingKeys();
}

bool FPCGComponentExecutionState::IsKeyTrackedAndCulled(const FPCGSelectionKey& Key, bool& bOutIsCulled) const
{
	check(Component);
	return Component->IsKeyTrackedAndCulled(Key, bOutIsCulled);
}

void FPCGComponentExecutionState::ForEachSettingTrackingKey(const FPCGSelectionKey& InKey, const TFunctionRef<void(const FPCGSelectionKey&, const FPCGSettingsAndCulling&)> InCallback) const
{
	check(Component);
	Component->ApplyToEachSettings(InKey, InCallback);
}

void FPCGComponentExecutionState::AddLocalSourceDynamicTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceDynamicTrackingKeys)
{
	check(Component);
	Component->ChangeTracker.AddLocalSourceDynamicTrackingKeys(LocalSourceDynamicTrackingKeys);
}

void FPCGComponentExecutionState::AddLocalSourceCurrentTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceCurrentExecutionTrackingKeys, const TSet<const UPCGSettings*>& LocalSourceCurrentExecutionTrackingSettings)
{
	check(Component);
	Component->ChangeTracker.AddLocalSourceCurrentTrackingKeys(LocalSourceCurrentExecutionTrackingKeys, LocalSourceCurrentExecutionTrackingSettings);
}

bool FPCGComponentExecutionState::IsRefreshInProgress() const
{
	check(Component);
	return Component->IsRefreshInProgress();
}

bool FPCGComponentExecutionState::WasGeneratedThisSession() const
{
	check(Component);
	return Component->WasGeneratedThisSession();
}

FPCGDynamicTrackingPriority FPCGComponentExecutionState::GetDynamicTrackingPriority() const
{
	check(Component);

	// Hash [0, 1] 
	double Fraction = static_cast<double>(GetTypeHash(Component->GetPathName())) / std::numeric_limits<uint32>::max();
	// Divide Fraction by 2^14 so that max value most significant decimal is the 5th (1.0 / 16384 -> 0.00006103515625)
	// TrackingPriority is rounded to 4 decimals so that Fraction (Hash) doesn't interfere with it (or almost never, in which case difference in value would be minimal).
	Fraction /= 16384;

	// So that leading integer and fractional user part stay the same add or subtract the hash based fraction based on sign
	return Component->TrackingPriority >= 0 ? (double)Component->TrackingPriority + Fraction : (double)Component->TrackingPriority - Fraction;
}

#endif // WITH_EDITOR
