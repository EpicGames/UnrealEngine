// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGContext.h"
#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "PCGSubsystem.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPointArrayData.h"
#include "Editor/IPCGEditorModule.h"
#include "Graph/PCGGraphExecutor.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGContainerAccessor.h"

#include "GameFramework/Actor.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGContext)

#define LOCTEXT_NAMESPACE "PCGContext"

TAutoConsoleVariable<bool> CVarPCGEnablePointArrayData(
	TEXT("pcg.EnablePointArrayData"),
	true,
	TEXT("Whether to use the new UPCGPointArrayData when calling FPCGContext::CreatePointData_AnyThread"));

namespace PCGContextHelpers
{
	thread_local bool bIsInitializingSettings = false;
}

FPCGContext::~FPCGContext()
{
	// Release the collection data before removing from GC gathering list from the executor to make sure there's no gap in coverage here
	InputData.Reset();
	OutputData.Reset();

	if (TSharedPtr<FPCGGraphExecutor> Executor = GraphExecutor.Pin())
	{
		Executor->RemoveReleasedContextForGC(this);
	}

	// This should not happen, might be a missing call to FPCGContext::Release
	// Clear Handle context pointer so that we don't double delete the FPCGContext
	ensureMsgf(!Handle, TEXT("FPCGContext deleted without first setting the Handle to nullptr, probably a delete that should be replaced by a FPCGContext::Release call"));
	if (Handle)
	{
		Handle->Context = nullptr;
	}
}

FPCGContextHandle::~FPCGContextHandle()
{
	if (Context)
	{
		delete Context;
	}
}

void FPCGContext::InitFromParams(const FPCGInitializeElementParams& InParams)
{
	ExecutionSource = InParams.ExecutionSource;
	InputData = *InParams.InputData;
	Node = InParams.Node;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SourceComponent = Cast<UPCGComponent>(ExecutionSource.Get());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UPCGBasePointData* FPCGContext::NewPointData_AnyThread(FPCGContext* Context)
{
	if (CVarPCGEnablePointArrayData.GetValueOnAnyThread())
	{
		return NewObject_AnyThread<UPCGPointArrayData>(Context);
	}
	else
	{
		return NewObject_AnyThread<UPCGPointData>(Context);
	}
}

TSubclassOf<UPCGBasePointData> FPCGContext::GetDefaultPointDataClass()
{
	return CVarPCGEnablePointArrayData.GetValueOnAnyThread() ? UPCGPointArrayData::StaticClass()
		: UPCGPointData::StaticClass();
}

FPCGTaskId FPCGContext::GetGraphExecutionTaskId() const
{
	const FPCGStack* StackPtr = GetStack();
	return ensure(StackPtr) ? StackPtr->GetGraphExecutionTaskId() : InvalidPCGTaskId;
}

FString FPCGContext::GetTaskName() const
{
	if (Node)
	{
		const FName NodeName = ((Node->NodeTitle != NAME_None) ? Node->NodeTitle : Node->GetFName());

		const UPCGSettings* Settings = GetInputSettings<UPCGSettings>();
		const FString NodeAdditionalInformation = Settings ? Settings->GetAdditionalTitleInformation() : FString();

		if (NodeAdditionalInformation.IsEmpty() || NodeAdditionalInformation == NodeName)
		{
			return NodeName.ToString();
		}
		else
		{
			return FString::Printf(TEXT("%s (%s)"), *NodeName.ToString(), *NodeAdditionalInformation);
		}
	}
	else
	{
		return TEXT("Anonymous task");
	}
}

int FPCGContext::GetSeed() const
{
	if (const UPCGSettings* Settings = GetInputSettings<UPCGSettings>())
	{
		return Settings->GetSeed(ExecutionSource.Get());
	}
	else if (ExecutionSource.IsValid())
	{
		return ExecutionSource->GetExecutionState().GetSeed();
	}
	else
	{
		return PCGValueConstants::DefaultSeed;
	}
}

FString FPCGContext::GetComponentName() const
{
	return GetExecutionSourceName();
}

FString FPCGContext::GetExecutionSourceName() const
{
	return PCGLog::GetExecutionSourceName(ExecutionSource.Get(), /*bUseLabel=*/false, /*InDefaultName=*/TEXT("Non-PCG Component"));
}

AActor* FPCGContext::GetTargetActor(const UPCGSpatialData* InSpatialData) const
{
	if (InSpatialData && InSpatialData->TargetActor.Get())
	{
		return InSpatialData->TargetActor.Get();
	}
	else
	{
		return GetTypedExecutionTarget<AActor>();
	}
}

UObject* FPCGContext::GetExecutionTarget() const
{
	if (IPCGGraphExecutionSource* ExecutionSourcePtr = ExecutionSource.Get())
	{
		return ExecutionSourcePtr->GetExecutionState().GetTarget();
	}
	else
	{
		return nullptr;
	}
}

const UPCGSettingsInterface* FPCGContext::GetInputSettingsInterface() const
{
	if (Node)
	{
		return InputData.GetSettingsInterface(Node->GetSettingsInterface());
	}
	else
	{
		return InputData.GetSettingsInterface();
	}
}

bool FPCGContext::IsInitializingSettings()
{
	return PCGContextHelpers::bIsInitializingSettings;
}

void FPCGContext::InitializeSettings(bool bSkipPostLoad)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGContext::InitializeSettings);

	if (SettingsWithOverride)
	{
		return;
	}

	// Thread local
	PCGContextHelpers::bIsInitializingSettings = true;
	ON_SCOPE_EXIT
	{
		PCGContextHelpers::bIsInitializingSettings = false;
	};

	if (const UPCGSettings* NodeSettings = GetOriginalSettings<UPCGSettings>())
	{
		// Don't apply overrides if the original settings are disabled
		if (!NodeSettings->bEnabled)
		{
			return;
		}

		// Only duplicate the settings if we have overriable params and we have at least one param pin connected.
		const TArray<FPCGSettingsOverridableParam>& OverridableParams = NodeSettings->OverridableParams();
		if (!OverridableParams.IsEmpty())
		{
			bool bHasParamConnected = !InputData.GetParamsByPin(PCGPinConstants::DefaultParamsLabel).IsEmpty();

			int32 Index = 0;
			while (!bHasParamConnected && Index < OverridableParams.Num())
			{
				bHasParamConnected |= !InputData.GetParamsByPin(OverridableParams[Index++].Label).IsEmpty();
			}

			if (bHasParamConnected)
			{
				// If we have an override and there is a hard ref override, we need to make sure that all objects
				// are loaded. If not, we need to schedule the task on the main thread.
				if (NodeSettings->HasAnyOverridableHardReferences())
				{
					for (const FPCGSettingsOverridableParam& Param : OverridableParams)
					{
						// If the param is not a hard ref, ignore.
						if (!Param.IsHardReferenceOverride())
						{
							continue;
						}

						PCGAttributeAccessorHelpers::AccessorParamResult AccessorResult{};
						TUniquePtr<const IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateConstAccessorForOverrideParamWithResult(InputData, Param, &AccessorResult);

						// If the accessor failed to be created, ignore
						if (!AttributeAccessor.IsValid())
						{
							continue;
						}

						// If we have a container on the param, we have to get all the paths
						TArray<FSoftObjectPath> ObjectPaths;
						if (ensure(Param.NumContainers <= 1))
						{
							if (Param.NumContainers == 1 && ensure(AccessorResult.Keys.IsValid()))
							{
								ObjectPaths.SetNum(AccessorResult.Keys->GetNum());
								if (!AttributeAccessor->GetRange<FSoftObjectPath>(ObjectPaths, 0, *AccessorResult.Keys, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible))
								{
									continue;
								}
							}
							else
							{
								if (!PCGAttributeAccessorHelpers::GetOverrideParamValue(*AttributeAccessor, ObjectPaths.Emplace_GetRef()))
								{
									continue;
								}
							}

							FGCScopeGuard GCScope;
							for (const FSoftObjectPath& ObjectPath : ObjectPaths)
							{
								if (!ObjectPath.ResolveObject())
								{
									// We have an override value that is not loaded. We need to schedule the task on the main thread.
									RequestToRunOnMainThread();
									break;
								}
							}
						}

						if (CanExecuteOnlyOnMainThread())
						{
							break;
						}
					}
				}
			
				FObjectDuplicationParameters DuplicateParams(const_cast<UPCGSettings*>(NodeSettings), GetTransientPackage());
				DuplicateParams.bSkipPostLoad = bSkipPostLoad;
				DuplicateParams.ApplyFlags = RF_Transient;
				// Remove RF_WasLoaded and RF_LoadCompleted because those objects aren't loaded they are duplicated.
				// When this happens outside of the GameThread the duplicated objects would end up with the Async flags + the Loaded flags which
				// can trigger an ensure in VerifyObjectLoadFlagsWhenFinishedLoading (in !WITH_EDITOR dev/debug builds).
				// We also remove the RF_Transactional flag as those duplicated objects are execution only objects and can't/should never be transacted.
				DuplicateParams.FlagMask &= ~(RF_WasLoaded | RF_LoadCompleted | RF_Transactional);
			
				TMap<UObject*, UObject*> CreatedObjects;
				DuplicateParams.CreatedObjects = &CreatedObjects;
				
				{
					FGCScopeGuard Scope;
					SettingsWithOverride = Cast<UPCGSettings>(StaticDuplicateObjectEx(DuplicateParams));
				}

				const bool bIsInGameThread = IsInGameThread();
				for (auto& KeyValuePair : CreatedObjects)
				{
					if (KeyValuePair.Value)
					{
						if (!bIsInGameThread)
						{
							// Outside of GameThread we need to clear the Async flags on newly duplicated objects
							AsyncObjects.Add(KeyValuePair.Value);
						}

						if (bSkipPostLoad)
						{
							// We are not calling PostLoad so remove the NeedPostLoad flags here
							KeyValuePair.Value->ClearFlags(EObjectFlags::RF_NeedPostLoad | EObjectFlags::RF_NeedPostLoadSubobjects);
						}
					}
				}
				
				// Force seed copy to prevent issue due to delta serialization vs. Seed being initialized in the constructor only for new nodes
				SettingsWithOverride->Seed = NodeSettings->Seed;
				SettingsWithOverride->OriginalSettings = NodeSettings;

				// If anything needs to be done by the Settings object after its been duplicated for override it should be done in here, outside of the gamethread that might include code that would be normally done in PostLoad
				SettingsWithOverride->OnOverrideSettingsDuplicated(bSkipPostLoad);
			}
		}
	}
}

void FPCGContext::OverrideSettings()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGContext::OverrideSettings);

	// If we have to be on the main thread check it there. It is necessary for overrides of hard references that are not yet loaded. We can reset the flag afterwards.
	check(!bMustRunOnMainThread || IsInGameThread());
	bMustRunOnMainThread = false;

	// Use original settings to avoid recomputing OverridableParams() everytime
	const UPCGSettings* OriginalSettings = GetOriginalSettings<UPCGSettings>();

	if (!SettingsWithOverride || !OriginalSettings)
	{
		return;
	}

	for (const FPCGSettingsOverridableParam& Param : OriginalSettings->OverridableParams())
	{
		if (!ensure(!Param.Properties.IsEmpty()))
		{
			PCGE_LOG_C(Error, GraphAndLog, this, FText::Format(LOCTEXT("ParamPropertyIsEmpty", "Override pin '{0}' has no property set, we can't override it."), FText::FromName(Param.Label)));
			continue;
		}

		// Verification that container is valid and we have the right class.
		void* Container = nullptr;

		if (!Param.PropertyClass)
		{
			continue;
		}

		if (!SettingsWithOverride->GetClass()->IsChildOf(Param.PropertyClass))
		{
			UObject* ObjectPtr = GetExternalContainerForOverridableParam(Param);

			if (ObjectPtr && ObjectPtr->IsA(Param.Properties[0]->GetOwnerClass()))
			{
				Container = ObjectPtr;
			}
			else if (!ObjectPtr)
			{
				FStructView StructView = GetExternalStructContainerForOverridableParam(Param);
				if (StructView.IsValid() && StructView.GetScriptStruct() == Param.Properties[0]->GetOwnerStruct())
				{
					Container = StructView.GetMemory();
				}
				else if (!StructView.IsValid())
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					// It's the responsability of the callee to make sure we have a valid memory space to read from.
					Container = GetUnsafeExternalContainerForOverridableParam(Param);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
		}
		else
		{
			if (!SettingsWithOverride->IsA(Param.Properties[0]->GetOwnerClass()))
			{
				continue;
			}

			Container = SettingsWithOverride.Get();
		}

		if (!Container)
		{
			continue;
		}

		PCGAttributeAccessorHelpers::AccessorParamResult AccessorResult{};
		TUniquePtr<const IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateConstAccessorForOverrideParamWithResult(InputData, Param, &AccessorResult);

		const FName AttributeName = AccessorResult.AttributeName;

		// Attribute doesn't exist
		if (!AttributeAccessor)
		{
			// Throw a warning if the pin was connected, but accessor failed
			if (AccessorResult.bPinConnected)
			{
				PCGE_LOG_C(Warning, GraphAndLog, this, FText::Format(LOCTEXT("AttributeNotFoundOnConnectedPin", "Override pin '{0}' is connected, but attribute '{1}' was not found."), FText::FromName(Param.Label), FText::FromName(AttributeName)));
			}

			continue;
		}

		// If aliases were used, throw a warning to ask the user to update its graph
		if (AccessorResult.bUsedAliases)
		{
			PCGE_LOG_C(Warning, GraphAndLog, this, FText::Format(LOCTEXT("OverrideWithAlias", "Attribute '{0}' was not found, but one of its deprecated aliases ('{1}') was. Please update the name to the new value."), FText::FromName(AttributeName), FText::FromName(AccessorResult.AliasUsed)));
		}

		// Throw warnings if we have multiple data on override (multiple attribute sets or multi entry attribute set)
		if (AccessorResult.bHasMultipleAttributeSetsOnOverridePin || AccessorResult.bHasMultipleDataInAttributeSet)
		{
			const FText OverridePinText = AccessorResult.bPinConnected ? FText::Format(LOCTEXT("OverridePinText", "override pin '{0}'"), FText::FromName(Param.Label)) : LOCTEXT("GlobalOverridePinText", "global override pin");

			if (AccessorResult.bHasMultipleAttributeSetsOnOverridePin)
			{
				PCGE_LOG_C(Warning, GraphAndLog, this, FText::Format(LOCTEXT("HasMultipleAttributeSetsOnOverridePin", "Multiple attribute sets were found on the {0}. We will use the first one."), OverridePinText));
			}

			if (AccessorResult.bHasMultipleDataInAttributeSet && Param.NumContainers == 0)
			{
				PCGE_LOG_C(Warning, GraphAndLog, this, FText::Format(LOCTEXT("HasMultipleDataInAttributeSet", "Multi entry attribute set was found on the {0}. We will only use the first entry to override."), OverridePinText));
			}
		}
		
		if (Param.NumContainers > 1)
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("TooManyContainers", "Property {0} is within multiple containers (i.e. array of array) which is not supported."), FText::FromName(Param.Label)), this);
			continue;
		}
		
		FPCGAttributeAccessorKeysSingleObjectPtr PropertyObjectKey(Container);

		// To support old flows, use the old accessors to access the attribute, only in single value and if the descriptors don't match.
		const FPCGMetadataAttributeDesc PropertyDesc = FPCGMetadataAttributeDesc::CreateFromProperty(Param.Properties.Last());
		const bool bDescriptorsMatch = PropertyDesc.IsSameType(AttributeAccessor->GetUnderlyingDesc());
		const bool bUseGenericAccessors = PropertyDesc.IsValid() && (!AttributeAccessor->GetUnderlyingDesc().IsSingleValue() || bDescriptorsMatch);
		
		TUniquePtr<IPCGAttributeAccessor> PropertyAccessor;
		
		if (PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(Param.Properties.Last(), bUseGenericAccessors))
		{
			PropertyAccessor = Param.NumContainers == 1
				? PCGContainerAccessorHelpers::MakeContainerAccessor(Param.Properties, bUseGenericAccessors)
				: PCGAttributeAccessorHelpers::CreatePropertyChainAccessor(TArray<const FProperty*>(Param.Properties), bUseGenericAccessors);
		}
		
		if (!PropertyAccessor.IsValid() || (Param.NumContainers == 1 && !AccessorResult.Keys))
		{
			continue;
		}
		
		FPCGAttributeAccessorKeysEntries FirstEntry(PCGFirstEntryKey);
		// If we are converting single value attribute to array attribute we need to use the keys, otherwise it is the first entry and a count of 1
		const bool bSingleValueToArrayConversion = AttributeAccessor->GetUnderlyingDesc().IsSingleValue() && Param.NumContainers == 1 && AccessorResult.Keys;
		const int32 Count = bSingleValueToArrayConversion ? AccessorResult.Keys->GetNum() : 1;
		const IPCGAttributeAccessorKeys& AttributeKeys = bSingleValueToArrayConversion ? *AccessorResult.Keys : FirstEntry;
		
		const bool bParamOverridden = AttributeAccessor->CopyTo(AttributeKeys, *PropertyAccessor, PropertyObjectKey, /*Index=*/ 0, /*Count=*/ Count, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible | EPCGAttributeAccessorFlags::AllowRangeOfValuesIntoSingleContainer);
		if (!bParamOverridden)
		{
			PCGE_LOG_C(Warning, GraphAndLog, this, FText::Format(LOCTEXT("ConversionFailed", "Parameter '{0}' ({1}) cannot be converted from attribute '{2}' ({3})"),
				FText::FromName(Param.Label), 
				PropertyAccessor->GetUnderlyingDesc().GetTypeText(),
				FText::FromName(AttributeName),
				AttributeAccessor->GetUnderlyingDesc().GetTypeText()));
		}
		else
		{
			OverriddenParams.Add(&Param);
		}
	}

	// Make sure CacheCrc is up to date
	SettingsWithOverride->CacheCrc();
}

bool FPCGContext::IsValueOverriden(const TArrayView<const FName>& PropertyNameChain) const
{
	if (PropertyNameChain.IsEmpty())
	{
		return false;
	}

	const FPCGSettingsOverridableParam* const* OverrideParam = OverriddenParams.FindByPredicate([&PropertyNameChain](const FPCGSettingsOverridableParam* ParamToCheck)
	{
		return ParamToCheck && ParamToCheck->PropertiesNames == PropertyNameChain;
	});

	return OverrideParam != nullptr;
}

bool FPCGContext::IsValueOverriden(const FName PropertyName) const
{
	const FPCGSettingsOverridableParam* const* OverriddenParam = OverriddenParams.FindByPredicate([PropertyName](const FPCGSettingsOverridableParam* ParamToCheck)
	{
		return ParamToCheck && !ParamToCheck->PropertiesNames.IsEmpty() && ParamToCheck->PropertiesNames[0] == PropertyName;
	});

	return OverriddenParam != nullptr;
}

#if WITH_EDITOR
void FPCGContext::LogVisual(ELogVerbosity::Type InVerbosity, const FText& InMessage) const
{
	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		const FPCGStack* StackPtr = GetStack();
		FPCGStack StackWithNode = StackPtr ? FPCGStack(*StackPtr) : FPCGStack();
		StackWithNode.PushFrame(Node);
		PCGEditorModule->GetNodeVisualLogsMutable().Log(StackWithNode, InVerbosity, InMessage);
	}
}

bool FPCGContext::HasVisualLogs() const
{
	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		const FPCGStack* StackPtr = GetStack();
		FPCGStack StackWithNode = StackPtr ? FPCGStack(*StackPtr) : FPCGStack();
		StackWithNode.PushFrame(Node);
		return PCGEditorModule->GetNodeVisualLogs().HasLogs(StackWithNode);
	}

	return false;
}

bool FPCGContext::IsExecutingGraphInspected() const
{
	const FPCGStack* StackPtr = GetStack();
	const UPCGGraph* Graph = StackPtr ? StackPtr->GetGraphForCurrentFrame() : nullptr;

	return Graph && Graph->IsSameInspectedStack(*StackPtr);
}

AActor* FPCGContext::GetDebugDisplayActor() const
{
	if (AActor* Actor = GetTypedExecutionTarget<AActor>())
	{
		return Actor;
	}
	else if (IPCGGraphExecutionSource* ExecutionSourcePtr = ExecutionSource.Get())
	{
		if (UPCGSubsystem* PCGSubsystem = Cast<UPCGSubsystem>(ExecutionSource->GetExecutionState().GetSubsystem()))
		{
			return PCGSubsystem->GetOrCreateDebugActor();
		}
	}

	return nullptr;
}
#endif // WITH_EDITOR

void FPCGContext::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	InputData.AddReferences(Collector);
	OutputData.AddReferences(Collector);

	for (TPair<FPCGDataCollection, FPCGDataCollection>& CachedIOResult : CachedInputToOutputInternalResults)
	{
		CachedIOResult.Key.AddReferences(Collector);
		CachedIOResult.Value.AddReferences(Collector);
	}

	if (SettingsWithOverride)
	{
		Collector.AddReferencedObject(SettingsWithOverride);
	}

	AddExtraStructReferencedObjects(Collector);
}

void FPCGContext::Release(FPCGContext* InContext)
{
	if (!InContext)
	{
		return;
	}

	// Adds this to the tracked contexts so we ensure that the data isn't GC'ed too early for the context destructor to behave properly.
	if(TSharedPtr<FPCGGraphExecutor> Executor = InContext->GraphExecutor.Pin())
	{
		Executor->AddReleasedContextForGC(InContext);
	}

	if (!InContext->Handle)
	{
		delete InContext;
	}
	else
	{
		// Release the Handle SharedPtr (at this point only captured and pinned Handles can keep the Context alive)
		// The Context will get deleted when the handle gets released
		InContext->Handle = nullptr;
	}
}

FPCGTaskId FPCGContext::ScheduleGraph(const FPCGScheduleGraphParams& InParams)
{
	if (TSharedPtr<FPCGGraphExecutor> Executor = GraphExecutor.Pin(); Executor && InParams.ExecutionSource)
	{
		return Executor->ScheduleGraph(InParams);
	}
	else
	{
		return InvalidPCGTaskId;
	}
}

FPCGTaskId FPCGContext::ScheduleGeneric(const FPCGScheduleGenericParams& InParams)
{
	if (TSharedPtr<FPCGGraphExecutor> Executor = GraphExecutor.Pin())
	{
		return Executor->ScheduleGeneric(InParams);
	}
	else
	{
		return InvalidPCGTaskId;
	}
}

bool FPCGContext::GetOutputData(FPCGTaskId InTaskId, FPCGDataCollection& OutData)
{
	if (TSharedPtr<FPCGGraphExecutor> Executor = GraphExecutor.Pin())
	{
		return Executor->GetOutputData(InTaskId, OutData);
	}
	else
	{
		return false;
	}
}

void FPCGContext::ClearOutputData(FPCGTaskId InTaskId)
{
	if (TSharedPtr<FPCGGraphExecutor> Executor = GraphExecutor.Pin())
	{
		Executor->ClearOutputData(InTaskId);
	}
}

void FPCGContext::InitializeGraphExecutor(FPCGContext* InContext)
{
	check(InContext);
	InContext->GraphExecutor = GraphExecutor;
}

bool FPCGContext::IsCacheEnabled()
{
	IPCGGraphExecutionSource* Source = ExecutionSource.IsValid() ? ExecutionSource.Get() : nullptr;
	if (Source && !Source->GetExecutionState().IsCacheEnabled())
	{
		return false;
	}

	TSharedPtr<FPCGGraphExecutor> Executor = GraphExecutor.Pin();
	return Executor && Executor->GetCache().IsEnabled();
}

void FPCGContext::StoreInCache(const FPCGStoreInCacheParams& Params, const FPCGDataCollection& InOutput)
{
	if (TSharedPtr<FPCGGraphExecutor> Executor = GraphExecutor.Pin())
	{
		Executor->GetCache().StoreInCache(Params, InOutput);
	}
}

bool FPCGContext::GetFromCache(const FPCGGetFromCacheParams& Params, FPCGDataCollection& OutOutput) const
{
	if (TSharedPtr<FPCGGraphExecutor> Executor = GraphExecutor.Pin())
	{
		return Executor->GetCache().GetFromCache(Params, OutOutput);
	}
	else
	{
		return false;
	}
}


namespace PCGContext
{
	thread_local FPCGContext* ThisThreadContext = nullptr;
}

FPCGContext* FPCGContext::GetContextForThread()
{
	return PCGContext::ThisThreadContext;
}

FPCGSourceDataContainer* FPCGContext::GetSourceDataContainer()
{
	IPCGGraphExecutionSource* Source = ExecutionSource.IsValid() ? ExecutionSource.Get() : nullptr;
	return Source ? Source->GetExecutionState().GetSourceDataContainer() : nullptr;
}

const FPCGSourceDataContainer* FPCGContext::GetSourceDataContainer() const
{
	const IPCGGraphExecutionSource* Source = ExecutionSource.IsValid() ? ExecutionSource.Get() : nullptr;
	return Source ? Source->GetExecutionState().GetSourceDataContainer() : nullptr;
}

FPCGContextScope::FPCGContextScope(FPCGContext* InContext)
{
	PreviousContext = PCGContext::ThisThreadContext;
	ThisContext = InContext;
	PCGContext::ThisThreadContext = InContext;
}

FPCGContextScope::~FPCGContextScope()
{
	checkf(PCGContext::ThisThreadContext == ThisContext, TEXT("Multiple scope are destroyed in the wrong order"));
	PCGContext::ThisThreadContext = PreviousContext;
}

#undef LOCTEXT_NAMESPACE
