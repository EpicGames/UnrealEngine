// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FPCGCustomVersion::GUID(0x2763920D, 0x0F784B39, 0x986E4BB3, 0xA88D666D);

// Register the custom version with core
FCustomVersionRegistration GRegisterPCGCustomVersion(FPCGCustomVersion::GUID, FPCGCustomVersion::LatestVersion, TEXT("PCGBaseVer"));
