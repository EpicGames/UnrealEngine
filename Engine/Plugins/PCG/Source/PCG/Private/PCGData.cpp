// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGData.h"

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGNode.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGSpatialData.h"
#include "Data/PCGUnionData.h"

#include "HAL/IConsoleManager.h"
#include "Serialization/ArchiveCrc32.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGData)

namespace PCGDataConstants
{
	const FName DataDomainName = "Data";
	const FName DefaultDomainName = "Default";
}

//#define PCG_DATA_USAGE_LOGGING

static TAutoConsoleVariable<bool> CVarPCGFullDataCrcInCommandlets(
	TEXT("pcg.FullDataCrcInCommandlets"),
	true,
	TEXT("Always compute full data Crcs in Commandlets (to improve reusability)."));

UPCGData::UPCGData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		InitUID();
	}
}

void UPCGData::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	
	if (Metadata)
	{
		Metadata->GetResourceSizeEx(CumulativeResourceSize);
	}
}

FPCGCrc UPCGData::GetOrComputeCrc(bool bFullDataCrc) const
{
#if WITH_EDITOR
	const bool bComputeFullDataCrc = (bFullDataCrc || (CVarPCGFullDataCrcInCommandlets.GetValueOnAnyThread() && IsRunningCommandlet())) && SupportsFullDataCrc();
#else
	const bool bComputeFullDataCrc = bFullDataCrc && SupportsFullDataCrc();
#endif

	// If we already valid Crc and it is a full data crc or we aren't being requested one, return the current Crc
	if (Crc.IsValid() && (bIsFullDataCrc || !bComputeFullDataCrc))
	{
		return Crc;
	}

	Crc = ComputeCrc(bComputeFullDataCrc);
	bIsFullDataCrc = bComputeFullDataCrc;

	return Crc;
}

void UPCGData::MarkUsage(EPCGDataUsage InUsage) const
{
#ifdef PCG_DATA_USAGE_LOGGING
	const UEnum* PCGDataUsageEnum = StaticEnum<EPCGDataUsage>();
	UE_LOGF(LogPCG, Warning, "%ls: Marked usage flag %ls, flags %d -> %d",
		*GetName(),
		*PCGDataUsageEnum->GetNameStringByValue(static_cast<__underlying_type(EPCGDataUsage)>(InUsage)),
		Usage.load(),
		Usage.load() | static_cast<__underlying_type(EPCGDataUsage)>(InUsage));
#endif

	Usage |= static_cast<__underlying_type(EPCGDataUsage)>(InUsage);
}

void UPCGData::ClearUsage(EPCGDataUsage InUsage) const
{
#ifdef PCG_DATA_USAGE_LOGGING
	const UEnum* PCGDataUsageEnum = StaticEnum<EPCGDataUsage>();
	UE_LOGF(LogPCG, Warning, "%ls: Unmarked usage flag %ls, flags %d -> %d",
		*GetName(),
		*PCGDataUsageEnum->GetNameStringByValue(static_cast<__underlying_type(EPCGDataUsage)>(InUsage)),
		Usage.load(),
		Usage.load() & ~static_cast<__underlying_type(EPCGDataUsage)>(InUsage));
#endif

	Usage &= ~static_cast<__underlying_type(EPCGDataUsage)>(InUsage);

	if (Usage == static_cast<__underlying_type(EPCGDataUsage)>(EPCGDataUsage::None) && CollectionRefCount == 0)
	{
#ifdef PCG_DATA_USAGE_LOGGING
		UE_LOGF(LogPCG, Warning, "%ls: !! ReleaseTransientResources after usage unmarked !!", *GetName());
#endif

		VisitDataNetwork([](const UPCGData* InData)
		{
			if (InData)
			{
				const_cast<UPCGData*>(InData)->ReleaseTransientResources(TEXT("Data unused"));
			}
		});
	}
}

bool UPCGData::HasUsage(EPCGDataUsage InUsage) const
{
	return !!(Usage & static_cast<__underlying_type(EPCGDataUsage)>(InUsage));
}

void UPCGData::IncCollectionRefCount() const
{
	++CollectionRefCount;
}

void UPCGData::DecCollectionRefCount() const
{
	--CollectionRefCount;

	ensure(CollectionRefCount >= 0);

	if (Usage == static_cast<__underlying_type(EPCGDataUsage)>(EPCGDataUsage::None) && CollectionRefCount == 0)
	{
#ifdef PCG_DATA_USAGE_LOGGING
		UE_LOGF(LogPCG, Warning, "%ls: !! ReleaseTransientResources after usage unmarked !!", *GetName());
#endif

		VisitDataNetwork([](const UPCGData* InData)
		{
			if (InData)
			{
				const_cast<UPCGData*>(InData)->ReleaseTransientResources(TEXT("Data unused"));
			}
		});
	}
}

bool UPCGData::IsSupportedMetadataDomainID(const FPCGMetadataDomainID& InDomainID) const
{
	return InDomainID == PCGMetadataDomainID::Default || GetAllSupportedMetadataDomainIDs().Contains(InDomainID);
}

FPCGMetadataDomainID UPCGData::GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const
{
	const FName DomainName = InSelector.GetDomainName();
	if (DomainName == NAME_None || DomainName == PCGDataConstants::DefaultDomainName)
	{
		return PCGMetadataDomainID::Default;
	}
	else if (DomainName == PCGDataConstants::DataDomainName)
	{
		return PCGMetadataDomainID::Data;
	}
	else
	{
		return FPCGMetadataDomainID(EPCGMetadataDomainFlag::Invalid);
	}
}

bool UPCGData::SetDomainFromDomainID(const FPCGMetadataDomainID& InDomainID, FPCGAttributePropertySelector& InOutSelector) const
{
	if (InDomainID == PCGMetadataDomainID::Data)
	{
		InOutSelector.SetDomainName(PCGDataConstants::DataDomainName, /*bResetExtraNames=*/false);
		return true;
	}
	else if (InDomainID == PCGMetadataDomainID::Default)
	{
		InOutSelector.SetDomainName(NAME_None, /*bResetExtraNames=*/false);
		return true;
	}
	else
	{
		return false;
	}
}

bool UPCGData::MetadataDomainSupportsMultiEntries(const FPCGMetadataDomainID& InDomainID) const
{
	// By default, data domain does not support multi entries
	return InDomainID.IsValid() && InDomainID != PCGMetadataDomainID::Data;
}

bool UPCGData::MetadataDomainSupportsParenting(const FPCGMetadataDomainID& InDomainID) const
{
	// By default parenting is supported
	return true;
}

FPCGCrc UPCGData::ComputeCrc(bool bFullDataCrc) const
{
	FArchiveCrc32 Ar;
	AddToCrc(Ar, bFullDataCrc);

	return FPCGCrc(Ar.GetCrc());
}

void UPCGData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	// Add "last attribute" which can affect downstream nodes.
	FPCGAttributePropertyInputSelector LastSelector = GetCachedLastSelector();
	LastSelector.AddToCrc(Ar);
}

void UPCGData::VisitDataNetwork(TFunctionRef<void(const UPCGData*)> Action) const
{
	Action(this);
}

void UPCGData::AddUIDToCrc(FArchiveCrc32& Ar) const
{
	// Fallback implementation uses UID to ensure every object returns a different Crc.
	uint64 UIDValue = UID;
	Ar << UIDValue;
}

void UPCGData::InitUID()
{
	UID = ++UIDCounter;
}

void UPCGData::PostEditImport()
{
	Super::PostEditImport();
	InitUID();
}

void UPCGData::Flatten()
{
	if (UPCGMetadata* ThisMetadata = MutableMetadata())
	{
		ThisMetadata->FlattenImpl();
	}
}

UPCGData* UPCGData::DuplicateData(bool bInitializeMetadata) const
{
	//@todo_pcg: Not sure StaticDuplicateObject is thread safe, most UPCGDAta subclasses override DuplicateData
	ensure(IsInGameThread());
	return Cast<UPCGData>(StaticDuplicateObject(this, GetTransientPackage()));
}

UPCGData* UPCGData::DuplicateData(FPCGContext* Context, bool bInitializeMetadata) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return DuplicateData(bInitializeMetadata);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FPCGTaggedData::operator==(const FPCGTaggedData& Other) const
{
	return Data == Other.Data &&
		Tags.Num() == Other.Tags.Num() &&
		Tags.Includes(Other.Tags) &&
		Pin == Other.Pin;
}

bool FPCGTaggedData::operator!=(const FPCGTaggedData& Other) const
{
	return !operator==(Other);
}

FPCGCrc FPCGTaggedData::ComputeCrc(bool bFullDataCrc) const
{
	FArchiveCrc32 Ar;

	Ar << const_cast<FName&>(Pin);

	if (Data)
	{
		const FPCGCrc DataCrc = Data->GetOrComputeCrc(bFullDataCrc);
		uint32 CrcValue = DataCrc.GetValue();
		Ar << CrcValue;
	}

	// TODO: Ensuring tags are sorted could prevent CRC change.
	Ar << const_cast<TSet<FString>&>(Tags);

	return FPCGCrc(Ar.GetCrc());
}

TArray<FPCGTaggedData> FPCGDataCollection::GetInputs() const
{
	return GetAllSpatialInputs();
}

TArray<FPCGTaggedData> FPCGDataCollection::GetAllSpatialInputs() const
{
	return TaggedData.FilterByPredicate([](const FPCGTaggedData& Data) {
		return Cast<UPCGSpatialData>(Data.Data) != nullptr;
		});
}

TArray<FPCGTaggedData> FPCGDataCollection::GetInputsByPin(const FName& InPinLabel) const
{
	return TaggedData.FilterByPredicate([&InPinLabel](const FPCGTaggedData& Data) {
		if (!ensure(Data.Data))
		{
			return false;
		}

		return Data.Pin == InPinLabel;
		});
}

TArray<FPCGTaggedData> FPCGDataCollection::GetSpatialInputsByPin(const FName& InPinLabel) const
{
	return TaggedData.FilterByPredicate([&InPinLabel](const FPCGTaggedData& Data) {
		if (!ensure(Data.Data))
		{
			return false;
		}

		return Data.Pin == InPinLabel && Data.Data.IsA<UPCGSpatialData>();
		});
}

int32 FPCGDataCollection::GetInputCountByPin(const FName& InPinLabel) const
{
	int32 Count = 0;
	for (const FPCGTaggedData& Data : TaggedData)
	{
		if (Data.Pin == InPinLabel)
		{
			++Count;
		}
	}
	return Count;
}

int32 FPCGDataCollection::GetSpatialInputCountByPin(const FName& InPinLabel) const
{
	int32 Count = 0;
	for (const FPCGTaggedData& Data : TaggedData)
	{
		if (Data.Pin == InPinLabel && Data.Data && Data.Data.IsA<UPCGSpatialData>())
		{
			++Count;
		}
	}
	return Count;
}

const UPCGSpatialData* FPCGDataCollection::GetSpatialUnionOfInputsByPin(const FName& InPinLabel, bool& bOutUnionDataCreated) const
{
	check(IsInGameThread());
	return GetSpatialUnionOfInputsByPin(nullptr, InPinLabel, bOutUnionDataCreated);
}

const UPCGSpatialData* FPCGDataCollection::GetSpatialUnionOfInputsByPin(FPCGContext* InContext, const FName& InPinLabel, bool& bOutUnionDataCreated) const
{
	TArray<FPCGTaggedData> SpatialDataOnPin = TaggedData.FilterByPredicate([&InPinLabel](const FPCGTaggedData& Data) {
		return Data.Pin == InPinLabel && Data.Data && Data.Data->IsA<UPCGSpatialData>();
	});

	const UPCGSpatialData* Result = nullptr;
	UPCGUnionData* Union = nullptr;
	for (const FPCGTaggedData& Data : SpatialDataOnPin)
	{
		const UPCGSpatialData* SpatialInput = Data.Data ? Cast<const UPCGSpatialData>(Data.Data) : nullptr;
		if (!ensure(SpatialInput))
		{
			continue;
		}

		if (!Result)
		{
			// First valid data
			Result = SpatialInput;
		}
		else
		{
			if (!Union)
			{
				// Second valid data - set up union
				Union = FPCGContext::NewObject_AnyThread<UPCGUnionData>(InContext);
				Union->Initialize(Result, SpatialInput);

				// Make result union
				Result = Union;
			}
			else
			{
				// Nth valid data
				Union->AddData(SpatialInput);
			}
		}
	}

	// Indicate whether we created a union data
	bOutUnionDataCreated = !!Union;

	return Result;
}

TArray<FPCGTaggedData> FPCGDataCollection::GetTaggedInputs(const FString& InTag) const
{
	return GetTaggedTypedInputs<UPCGSpatialData>(InTag);
}

TArray<FPCGTaggedData> FPCGDataCollection::GetAllSettings() const
{
	return TaggedData.FilterByPredicate([](const FPCGTaggedData& Data) {
		return Cast<UPCGSettings>(Data.Data) != nullptr;
		});
}

TArray<FPCGTaggedData> FPCGDataCollection::GetAllParams() const
{
	return TaggedData.FilterByPredicate([](const FPCGTaggedData& Data)
	{
		return Cast<UPCGParamData>(Data.Data) || (Data.Data && Data.Data->GetUnderlyingDataType() == EPCGDataType::Param);
	});
}

TArray<FPCGTaggedData> FPCGDataCollection::GetParamsByPin(const FName& InPinLabel) const
{
	return TaggedData.FilterByPredicate([&InPinLabel](const FPCGTaggedData& Data)
	{
		return Data.Pin == InPinLabel && (Cast<UPCGParamData>(Data.Data) || (Data.Data && Data.Data->GetUnderlyingDataType() == EPCGDataType::Param));
	});
}

TArray<FPCGTaggedData> FPCGDataCollection::GetTaggedParams(const FString& InTag) const
{
	return TaggedData.FilterByPredicate([&InTag](const FPCGTaggedData& Data)
	{
		return Data.Tags.Contains(InTag) && (Cast<UPCGParamData>(Data.Data) || (Data.Data && Data.Data->GetUnderlyingDataType() == EPCGDataType::Param));
	});
}

UPCGParamData* FPCGDataCollection::GetParamsWithDeprecation(const UPCGNode* Node) const
{
	// First try with the param pin
	if (UPCGParamData* ParamData = GetFirstParamsOnParamsPin())
	{
		return ParamData;
	}

	// If there is nothing on the params pin, temporarily supports reading param from the input pin, to avoid breaking graphs with nodes that doesn't have a param pin
	// Log a warning too
	if (Node)
	{
		for (const UPCGPin* InputPin : Node->GetInputPins())
		{
			// To avoid matching the individual parameters pins added by the override system, we'll ignore
			// any input pin that only accepts params.
			if (InputPin->Properties.AllowedTypes == EPCGDataType::Param)
			{
				continue;
			}

			for (const FPCGTaggedData& TaggedDatum : GetParamsByPin(InputPin->Properties.Label))
			{
				if (const UPCGParamData* Params = Cast<UPCGParamData>(TaggedDatum.Data))
				{
					UE_LOGF(LogPCG, Warning, "[%ls] Found an Attribute Set data on an input pin that should not accept attributes. Make sure to re-wire it to the Overrides pin if it is used for overrides.", *Node->GetNodeTitle(EPCGNodeTitleType::ListView).ToString());
					return const_cast<UPCGParamData*>(Params);
				}
			}
		}
	}

	return nullptr;
}

UPCGParamData* FPCGDataCollection::GetFirstParamsOnParamsPin() const
{
	TArray<FPCGTaggedData> ParamsOnDefaultPin = GetParamsByPin(PCGPinConstants::DefaultParamsLabel);
	return (ParamsOnDefaultPin.IsEmpty() ? nullptr : const_cast<UPCGParamData*>(Cast<UPCGParamData>(ParamsOnDefaultPin[0].Data)));
}

void FPCGDataCollection::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
{
	for (const FPCGTaggedData& Data : TaggedData)
	{
		if (Data.Data)
		{
			Data.Data->VisitDataNetwork([&CumulativeResourceSize](const UPCGData* Data) {
				// Cast away const-ness. The extended mode of GetResourceSizeEx accounts memory for all objects outer'd
				// to this object, and that calls GetObjectsWithOuter which is non-const. We don't use this extended mode
				// and we need to be able to operate on const objects.
				const_cast<UPCGData*>(Data)->GetResourceSizeEx(CumulativeResourceSize);
			});
		}
	}
}

const UPCGSettings* FPCGDataCollection::GetSettings(const UPCGSettings* InDefaultSettings) const
{
	if (!InDefaultSettings)
	{
		return GetSettings<UPCGSettings>();
	}
	else
	{
		const FPCGTaggedData* MatchingData = TaggedData.FindByPredicate([InDefaultSettings](const FPCGTaggedData& Data) {
			return Data.Data &&
				(Data.Data->GetClass() == InDefaultSettings->GetClass() ||
					Data.Data->GetClass()->IsChildOf(InDefaultSettings->GetClass()));
			});

		return MatchingData ? Cast<const UPCGSettings>(MatchingData->Data) : InDefaultSettings;
	}
}

const UPCGSettingsInterface* FPCGDataCollection::GetSettingsInterface() const
{
	return GetSettings<UPCGSettingsInterface>();
}

const UPCGSettingsInterface* FPCGDataCollection::GetSettingsInterface(const UPCGSettingsInterface* InDefaultSettingsInterface) const
{
	if (!InDefaultSettingsInterface || InDefaultSettingsInterface->GetSettings() == nullptr)
	{
		return GetSettingsInterface();
	}
	else
	{
		const FPCGTaggedData* MatchingData = TaggedData.FindByPredicate([InDefaultSettingsInterface](const FPCGTaggedData& Data) {
			if (const UPCGSettingsInterface* DataSettingsInterface = Cast<UPCGSettingsInterface>(Data.Data))
			{
				// Compare settings classes
				return DataSettingsInterface->GetSettings()->GetClass() == InDefaultSettingsInterface->GetSettings()->GetClass() ||
					DataSettingsInterface->GetSettings()->GetClass()->IsChildOf(InDefaultSettingsInterface->GetSettings()->GetClass());
			}

			return false;
		});

		return MatchingData ? Cast<const UPCGSettingsInterface>(MatchingData->Data) : InDefaultSettingsInterface;
	}
}

bool FPCGDataCollection::operator==(const FPCGDataCollection& Other) const
{
	if (bCancelExecution != Other.bCancelExecution || TaggedData.Num() != Other.TaggedData.Num())
	{
		return false;
	}

	// TODO: Once we make the arguments order irrelevant, then this should be updated
	for (int32 DataIndex = 0; DataIndex < TaggedData.Num(); ++DataIndex)
	{
		if (TaggedData[DataIndex] != Other.TaggedData[DataIndex])
		{
			return false;
		}
	}
	
	return true;
}

bool FPCGDataCollection::operator!=(const FPCGDataCollection& Other) const
{
	return !operator==(Other);
}

FPCGDataCollection& FPCGDataCollection::operator+=(const FPCGDataCollection& Other)
{
	AddData(Other.TaggedData, Other.DataCrcs);
	return *this;
}

void FPCGDataCollection::AddReferences(FReferenceCollector& Collector)
{
	for (FPCGTaggedData& Data : TaggedData)
	{
		if (Data.Data)
		{
			Collector.AddReferencedObject(Data.Data.Data);
		}
	}
}

void FPCGDataCollection::ComputeCrcs(bool bFullDataCrc)
{
	DataCrcs.SetNumUninitialized(TaggedData.Num(), EAllowShrinking::No);

	for (int I = 0; I < TaggedData.Num(); ++I)
	{
		DataCrcs[I] = TaggedData[I].ComputeCrc(bFullDataCrc);
	}
}

void FPCGDataCollection::AddData(const FPCGTaggedData& InData, const FPCGCrc& InDataCrc)
{
	TaggedData.Add(InData);
	DataCrcs.Add(InDataCrc);
}

void FPCGDataCollection::AddData(const TConstArrayView<FPCGTaggedData>& InData, const TConstArrayView<FPCGCrc>& InDataCrcs)
{
	TaggedData.Append(InData);
	DataCrcs.Append(InDataCrcs);
}

void FPCGDataCollection::AddDataForPin(const TConstArrayView<FPCGTaggedData>& InData, const TConstArrayView<FPCGCrc>& InDataCrcs, uint32 InputPinLabelCrc)
{
	TaggedData.Append(InData);
	DataCrcs.Append(InDataCrcs);

	// Add input pin label to Crc to uniquely identify inputs per-pin, or use a placeholder for symmetry.
	// Note that the cached data Crc will already contain the output pin (calculated in element post execute).
	for (int I = (DataCrcs.Num() - InDataCrcs.Num()); I < DataCrcs.Num(); ++I)
	{
		DataCrcs[I].Combine(InputPinLabelCrc);
	}
}

void FPCGDataCollection::Reset()
{
	// Implementation note: We are assuming that there is no need to remove the data from the rootset here.
	TaggedData.Reset();
	DataCrcs.Reset();
	bCancelExecution = false;
}

int32 FPCGDataCollection::StripEmptyPointData()
{
	return TaggedData.RemoveAll([](const FPCGTaggedData& Data)
	{
		const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(Data.Data);
		return PointData && PointData->GetNumPoints() == 0;
	});
}

TArray<UPCGData*> UPCGDataFunctionLibrary::GetInputsByPredicate(const FPCGDataCollection& InCollection, TArray<FPCGTaggedData>& OutTaggedData, TFunctionRef<bool(const FPCGTaggedData&)> InPredicate)
{
	TArray<UPCGData*> Inputs;
	OutTaggedData.Reset();

	for (const FPCGTaggedData& TaggedData : InCollection.TaggedData)
	{
		if (InPredicate(TaggedData))
		{
			Inputs.Add(const_cast<UPCGData*>(TaggedData.Data.Get()));
			OutTaggedData.Add(TaggedData);
		}
	}

	return Inputs;
}

TArray<UPCGData*> UPCGDataFunctionLibrary::GetTypedInputs(const FPCGDataCollection& InCollection, TArray<FPCGTaggedData>& OutTaggedData, TSubclassOf<UPCGData> InDataTypeClass)
{
	return GetInputsByPredicate(InCollection, OutTaggedData, [InDataTypeClass](const FPCGTaggedData& TaggedData)
	{
		return TaggedData.Data && (!InDataTypeClass || TaggedData.Data->IsA(InDataTypeClass));
	});
}

TArray<UPCGData*> UPCGDataFunctionLibrary::GetTypedInputsByPin(const FPCGDataCollection& InCollection, const FPCGPinProperties& InPin, TArray<FPCGTaggedData>& OutTaggedData, TSubclassOf<UPCGData> InDataTypeClass)
{
	return GetTypedInputsByPinLabel(InCollection, InPin.Label, OutTaggedData, InDataTypeClass);
}

TArray<UPCGData*> UPCGDataFunctionLibrary::GetTypedInputsByPinLabel(const FPCGDataCollection& InCollection, FName InPinLabel, TArray<FPCGTaggedData>& OutTaggedData, TSubclassOf<UPCGData> InDataTypeClass)
{
	return GetInputsByPredicate(InCollection, OutTaggedData, [&InPinLabel, InDataTypeClass](const FPCGTaggedData& TaggedData)
	{
		return TaggedData.Pin == InPinLabel && TaggedData.Data && (!InDataTypeClass || TaggedData.Data->IsA(InDataTypeClass));
	});
}

TArray<UPCGData*> UPCGDataFunctionLibrary::GetTypedInputsByTag(const FPCGDataCollection& InCollection, const FString& InTag, TArray<FPCGTaggedData>& OutTaggedData, TSubclassOf<UPCGData> InDataTypeClass)
{
	return GetInputsByPredicate(InCollection, OutTaggedData, [&InTag, InDataTypeClass](const FPCGTaggedData& TaggedData)
	{
		return TaggedData.Tags.Contains(InTag) && TaggedData.Data && (!InDataTypeClass || TaggedData.Data->IsA(InDataTypeClass));
	});
}

void UPCGDataFunctionLibrary::AddToCollection(FPCGDataCollection& InCollection, const UPCGData* InData, FName InPinLabel, TArray<FString> InTags)
{
	if (InData)
	{
		FPCGTaggedData& TaggedData = InCollection.TaggedData.Emplace_GetRef();
		TaggedData.Data = InData;
		TaggedData.Pin = InPinLabel;
		TaggedData.Tags.Append(InTags);
	}
}

// DEPRECATED
TArray<FPCGTaggedData> UPCGDataFunctionLibrary::GetInputs(const FPCGDataCollection& InCollection)
{
	return InCollection.GetAllSpatialInputs();
}

TArray<FPCGTaggedData> UPCGDataFunctionLibrary::GetInputsByPinLabel(const FPCGDataCollection& InCollection, const FName InPinLabel)
{
	return InCollection.GetInputsByPin(InPinLabel);
}

TArray<FPCGTaggedData> UPCGDataFunctionLibrary::GetInputsByTag(const FPCGDataCollection& InCollection, const FString& InTag)
{
	return InCollection.GetTaggedInputs(InTag);
}

TArray<FPCGTaggedData> UPCGDataFunctionLibrary::GetParams(const FPCGDataCollection& InCollection)
{
	return InCollection.GetAllParams();
}

TArray<FPCGTaggedData> UPCGDataFunctionLibrary::GetParamsByPinLabel(const FPCGDataCollection& InCollection, const FName InPinLabel)
{
	return InCollection.GetParamsByPin(InPinLabel);
}

TArray<FPCGTaggedData> UPCGDataFunctionLibrary::GetParamsByTag(const FPCGDataCollection& InCollection, const FString& InTag)
{
	return InCollection.GetTaggedParams(InTag);
}

TArray<FPCGTaggedData> UPCGDataFunctionLibrary::GetAllSettings(const FPCGDataCollection& InCollection)
{
	return InCollection.GetAllSettings();
}
