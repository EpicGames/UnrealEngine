// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDataAsset.h"

#include "UObject/ObjectSaveContext.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#endif // WITH_EDITOR

void UPCGDataAsset::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
	
#if WITH_EDITOR
	CachedPins.CachedPins = GetPins();
	CachedPins.bHasCachedPins = true;
#endif // WITH_EDITOR
}

TArray<FPCGPinProperties> UPCGDataAsset::GetPins() const
{
	TArray<FPCGPinProperties> Pins;

	for (const FPCGTaggedData& TaggedData : Data.TaggedData)
	{
		if (!TaggedData.Data)
		{
			continue;
		}

		FPCGPinProperties* MatchingPin = Pins.FindByPredicate([&TaggedData](const FPCGPinProperties& PinProperty) { return PinProperty.Label == TaggedData.Pin; });

		if (!MatchingPin)
		{
			Pins.Emplace(TaggedData.Pin, TaggedData.Data->GetDataTypeId());
		}
		else
		{
			MatchingPin->AllowedTypes |= TaggedData.Data->GetDataTypeId();
		}
	}

	return Pins;
}

PCGDataAsset::FGetAssetDataOutput UPCGDataAsset::GetAssetRegistryData(const TSoftObjectPtr<UPCGDataAsset>& InSoftAsset, const bool bShouldLoadAsset)
{
	if (InSoftAsset.IsNull())
	{
		return {};
	}
	
	if (UPCGDataAsset* Asset = bShouldLoadAsset ? InSoftAsset.LoadSynchronous() : InSoftAsset.Get())
	{
		return PCGDataAsset::FGetAssetDataOutput
		{
			.Asset = Asset,
			.Name = Asset->Name,
			.CachedPins = {.CachedPins=Asset->GetPins(), .bHasCachedPins=true},
#if WITH_EDITOR
			.Description = Asset->Description,
			.Color = Asset->Color
#endif // WITH_EDITOR
		};
	}
	
#if WITH_EDITOR
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	const FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(InSoftAsset.ToSoftObjectPath());
	return GetAssetRegistryData(AssetData);
#else // !WITH_EDITOR
	return {};
#endif // WITH_EDITOR
}

#if WITH_EDITOR
PCGDataAsset::FGetAssetDataOutput UPCGDataAsset::GetAssetRegistryData(const FAssetData& AssetData)
{
	if (const UClass* AssetClass = AssetData.GetClass(); AssetClass && AssetClass->IsChildOf(UPCGDataAsset::StaticClass()))
	{
		PCGDataAsset::FGetAssetDataOutput Result =
		{
			.Asset = nullptr,
			.Name = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, Name)),
			.CachedPins = {},
			.Description = AssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, Description)),
			.Color = {},
		};

		FString TempString;
		AssetData.GetTagValue<FString>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, CachedPins), TempString);
		if (!TempString.IsEmpty())
		{
			FPCGDataAssetCachedPins::StaticStruct()->ImportText(*TempString, &Result.CachedPins, nullptr, 0, nullptr, FString{});
		}

		// Since Color was not an AssetRegistrySearchable before, we'll rely on the bHasCachedPins for deprecation
		if (Result.CachedPins.bHasCachedPins)
		{
			// Initialize it with the default value.
			Result.Color.Emplace(FLinearColor::White);

			TempString.Reset();
			AssetData.GetTagValue<FString>(GET_MEMBER_NAME_CHECKED(UPCGDataAsset, Color), TempString);
			if (!TempString.IsEmpty())
			{
				TBaseStructure<FLinearColor>::Get()->ImportText(*TempString, Result.Color.GetPtrOrNull(), nullptr, 0, nullptr, FString{});
			}
		}

		return Result;
	}

	return {};
}
#endif // WITH_EDITOR