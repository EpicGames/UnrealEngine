// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDataVisualizationRegistry.h"

#include "Data/PCGSpatialData.h"

#if WITH_EDITOR
void FPCGDataVisualizationRegistry::RegisterPCGDataVisualization(const TSubclassOf<UPCGData> DataClass, TUniquePtr<const IPCGDataVisualization> Visualization)
{
	// TODO: We can support multiple visualizations in the future, but special care will be needed to avoid multiple types competing to own the ALV visualization.
	if (ensureMsgf(!ExternalRegistry.Find(DataClass), TEXT("Cannot register multiple data visualizations of the same type.")))
	{
		ExternalRegistry.Add(DataClass, std::move(Visualization));
	}
}

void FPCGDataVisualizationRegistry::UnregisterPCGDataVisualization(const TSubclassOf<UPCGData> DataClass)
{
	ExternalRegistry.Remove(DataClass);
}

const IPCGDataVisualization* FPCGDataVisualizationRegistry::GetDataVisualization(const TSubclassOf<UPCGData> PCGDataTypeClass) const
{
	TSubclassOf<UPCGData> CurrentClass = PCGDataTypeClass;
	const TSubclassOf<UPCGData> UPCGDataSuperClass = UPCGData::StaticClass()->GetSuperClass();

	// Iterate up the class hierarchy to find the first data class with a visualization implementation.
	while (CurrentClass && CurrentClass != UPCGDataSuperClass)
	{
		// Always try external implementations first, so the user can author their own visualizations for built-in PCG types if they desire.
		if (const TUniquePtr<const IPCGDataVisualization>* VisualizationPtr = ExternalRegistry.Find(CurrentClass))
		{
			return VisualizationPtr->Get();
		}

		if (const TUniquePtr<const IPCGDataVisualization>* VisualizationPtr = InternalRegistry.Find(CurrentClass))
		{
			return VisualizationPtr->Get();
		}

		CurrentClass = CurrentClass->GetSuperClass();

		// Skip SpatialDataWithPointCache since it will never have a visualization implementation.
		if (CurrentClass == UPCGSpatialDataWithPointCache::StaticClass())
		{
			CurrentClass = CurrentClass->GetSuperClass();
		}
	}

	return nullptr;
}
#endif