// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDebug.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDebug)

namespace PCGDebugVisConstants
{
	extern const FSoftObjectPath DefaultPointMesh = FSoftObjectPath(TEXT("/PCG/DebugObjects/PCG_Cube.PCG_Cube"));
	const FSoftObjectPath MaterialForDefaultPointMesh = FSoftObjectPath(TEXT("Material'/PCG/DebugObjects/PCG_DebugMaterial.PCG_DebugMaterial'"));
}

FPCGDebugVisualizationSettings::FPCGDebugVisualizationSettings()
{
	PointMesh = PCGDebugVisConstants::DefaultPointMesh;
}

TSoftObjectPtr<UMaterialInterface> FPCGDebugVisualizationSettings::GetMaterial() const
{
	if (MaterialOverride.IsNull() && PointMesh.ToSoftObjectPath() == PCGDebugVisConstants::DefaultPointMesh)
	{
		return TSoftObjectPtr<UMaterialInterface>(PCGDebugVisConstants::MaterialForDefaultPointMesh);
	}
	else
	{
		return MaterialOverride;
	}
}
