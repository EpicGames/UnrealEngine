// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDebugDrawComponent.h"

#include "PCGModule.h"

#if WITH_EDITOR
#include "Editor.h"
#endif // WITH_EDITOR

#include "TimerManager.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDebugDrawComponent)

#define LOCTEXT_NAMESPACE "PCGDebugVisuals"

void UPCGDebugDrawComponent::AddDebugString(const FDebugRenderSceneProxy::FText3d& TextIn3D)
{
	SceneProxy->Texts.Add(TextIn3D);
}

void UPCGDebugDrawComponent::AddDebugString(const FString& InString, const FVector& InLocation, const FLinearColor& InColor)
{
	SceneProxy->Texts.Emplace(InString, InLocation, InColor);
}

void UPCGDebugDrawComponent::AddDebugStrings(const TArrayView<FDebugRenderSceneProxy::FText3d> TextIn3DArray)
{
	SceneProxy->Texts.Append(TextIn3DArray);
}

void UPCGDebugDrawComponent::StartTimer(const float DurationInMilliseconds)
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->GetTimerManager()->SetTimer(TimerHandle, FTimerDelegate::CreateUObject(this, &UPCGDebugDrawComponent::OnTimerElapsed), DurationInMilliseconds, /*InbLoop=*/true);
	}
	else
#endif //WITH_EDITOR
	{
		UWorld* World = GetWorld();
		if (ensure(World))
		{
			World->GetTimerManager().SetTimer(TimerHandle, FTimerDelegate::CreateUObject(this, &UPCGDebugDrawComponent::OnTimerElapsed), DurationInMilliseconds, /*InbLoop=*/true);
		}
	}

	if (TimerHandle.IsValid())
	{
		DebugDrawDelegateHelper.InitDelegateHelper(SceneProxy);
	}
}

void UPCGDebugDrawComponent::ClearTimer()
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(TimerHandle);
	}
	else
#endif //WITH_EDITOR
	{
		UWorld* World = GetWorld();
		if (ensure(World))
		{
			World->GetTimerManager().ClearTimer(TimerHandle);
		}
	}
}

FDebugRenderSceneProxy* UPCGDebugDrawComponent::CreateDebugSceneProxy()
{
	SceneProxy = new FDebugRenderSceneProxy(this);
	SceneProxy->ViewFlagName = PCGEngineShowFlags::Debug;
	return SceneProxy;
}

void UPCGDebugDrawComponent::OnUnregister()
{
	ClearTimer();
	Super::OnUnregister();
}

void UPCGDebugDrawComponent::OnTimerElapsed()
{
	ClearTimer();
	DestroyComponent();
}

#undef LOCTEXT_NAMESPACE
