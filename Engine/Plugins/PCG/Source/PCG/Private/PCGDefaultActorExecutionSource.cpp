// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDefaultActorExecutionSource.h"

#include "PCGSubsystem.h"
#include "Data/PCGUnionData.h"
#include "Graph/PCGGraphPerExecutionCache.h"
#include "Helpers/PCGHelpers.h"

#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDefaultActorExecutionSource)

FPCGDefaultActorExecutionState::FPCGDefaultActorExecutionState(UPCGDefaultActorExecutionSource* InSource)
	: FPCGDefaultWorldObjectExecutionState(InSource)
{
}

FString FPCGDefaultActorExecutionState::GetDebugName() const
{
	if (AActor* Actor = GetActor())
	{
		return FString(TEXT("PCGDefaultActorExecutionSource - "))
#if WITH_EDITOR
			+ Actor->GetActorLabel();
#else
			+ Actor->GetName();
#endif
	}
	else
	{
		return FPCGDefaultWorldObjectExecutionState::GetDebugName();
	}
}


UPCGData* FPCGDefaultActorExecutionState::GetSelfDataInternal() const
{
	AActor* Actor = GetActor();
	if (!Actor)
	{
		return FPCGDefaultWorldObjectExecutionState::GetSelfData();
	}
	
	FPCGGetDataFunctionRegistryParams Params;
	Params.Source = GetSource();
	Params.bParseActor = true;
	Params.DataTypeFilter = EPCGDataType::Spatial;

	FPCGGetDataFunctionRegistryOutput Output;
	FPCGModule::ConstGetDataFunctionRegistry().GetDataFromActor(/*Context=*/nullptr, Params, Actor, Output);

	if (Output.Collection.TaggedData.Num() > 1)
	{
		UPCGUnionData* Union = NewObject<UPCGUnionData>();
		for (const FPCGTaggedData& TaggedData : Output.Collection.TaggedData)
		{
			Union->AddData(CastChecked<const UPCGSpatialData>(TaggedData.Data));
		}

		return Union;
	}
	else if (Output.Collection.TaggedData.Num() == 1)
	{
		return const_cast<UPCGData*>(Output.Collection.TaggedData[0].Data.Get());
	}

	return nullptr;
}

UPCGData* FPCGDefaultActorExecutionState::GetSelfData() const
{
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(GetTypedSource(), PCGPerExecutionCacheGuids::SelfData, [this]() { return GetSelfDataInternal(); });
}

void FPCGDefaultActorExecutionState::ExecutePreGraph(FPCGContext* Context)
{
	GetSelfData();
	GetBounds();
	GetLocalSpaceBounds();
}

FTransform FPCGDefaultActorExecutionState::GetTransform() const
{
	if (AActor* Actor = GetActor())
	{
		return Actor->GetTransform();
	}
	else
	{
		return FPCGDefaultWorldObjectExecutionState::GetTransform();
	}
}

UObject* FPCGDefaultActorExecutionState::GetTarget() const
{
	return GetActor();
}

bool FPCGDefaultActorExecutionState::HasAuthority() const
{
	if (AActor* Actor = GetActor())
	{
		return Actor->HasAuthority();
	}
	else
	{
		return FPCGDefaultWorldObjectExecutionState::HasAuthority();
	}
}

FBox FPCGDefaultActorExecutionState::GetBoundsInternal() const
{
	if (AActor* Actor = GetActor())
	{
		return PCGHelpers::GetActorBounds(Actor);
	}
	else
	{
		return FPCGDefaultWorldObjectExecutionState::GetBounds();
	}
}

FBox FPCGDefaultActorExecutionState::GetBounds() const
{
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(GetTypedSource(), PCGPerExecutionCacheGuids::Bounds, [this]() { return GetBoundsInternal(); });
}

FBox FPCGDefaultActorExecutionState::GetLocalSpaceBoundsInternal() const
{
	if (AActor* Actor = GetActor())
	{
		return PCGHelpers::GetActorLocalBounds(Actor);
	}
	else
	{
		return FPCGDefaultWorldObjectExecutionState::GetLocalSpaceBounds();
	}
}

FBox FPCGDefaultActorExecutionState::GetLocalSpaceBounds() const
{
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(GetTypedSource(), PCGPerExecutionCacheGuids::LocalSpaceBounds, [this]() { return GetLocalSpaceBoundsInternal(); });
}

UPCGDefaultActorExecutionSource::UPCGDefaultActorExecutionSource()
{
	// Inefficient but more foolproof - we'll replace the previously created state from the base class by our version.
	State = MakeUnique<FPCGDefaultActorExecutionState>(this);
}

void UPCGDefaultActorExecutionSource::Initialize(const ParamsType& InParams)
{
	UPCGDefaultExecutionSource::Initialize(InParams);
	SetActor(InParams.Actor);
}

void UPCGDefaultActorExecutionSource::SetActor(AActor* InActor)
{
	SetWorldObject(InActor);
}

AActor* UPCGDefaultActorExecutionSource::GetActor()
{
	return Cast<AActor>(GetWorldObject());
}