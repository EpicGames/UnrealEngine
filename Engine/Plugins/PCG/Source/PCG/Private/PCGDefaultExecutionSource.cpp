// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDefaultExecutionSource.h"

#include "PCGSubgraph.h"
#include "Subsystems/IPCGBaseSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDefaultExecutionSource)

UPCGData* FPCGDefaultExecutionState::GetSelfData() const
{
	return nullptr;
}

int32 FPCGDefaultExecutionState::GetSeed() const
{
	check(Source);
	return Source->Seed;
}

FString FPCGDefaultExecutionState::GetDebugName() const
{
	return TEXT("PCGDefaultExecutionSource");
}

FTransform FPCGDefaultExecutionState::GetTransform() const
{
	return FTransform::Identity;
}

UWorld* FPCGDefaultExecutionState::GetWorld() const
{
	return nullptr;
}

bool FPCGDefaultExecutionState::HasAuthority() const
{
	return false;
}

FBox FPCGDefaultExecutionState::GetBounds() const
{
	return FBox(-FVector::One(), FVector::One());
}

UPCGGraph* FPCGDefaultExecutionState::GetGraph() const
{
	check(Source);
	return Source->GetGraph();
}

UPCGGraphInstance* FPCGDefaultExecutionState::GetGraphInstance() const
{
	check(Source);
	return Source->GetGraphInstance();
}

void FPCGDefaultExecutionState::OnGraphExecutionAborted(bool bQuiet, bool bCleanupUnusedResources)
{
	Source->CurrentGenerationTask = InvalidPCGTaskId;
#if PCG_PROFILING_ENABLED
	if (IPCGBaseSubsystem* Subsystem = GetSubsystem())
	{
		Subsystem->OnPCGSourceGenerationDone(Source, EPCGGenerationStatus::Aborted);
	}
#endif
}

void FPCGDefaultExecutionState::Cancel()
{
	if (Source->CurrentGenerationTask == InvalidPCGTaskId)
	{
		return;
	}

	if (IPCGBaseSubsystem* Subsystem = GetSubsystem())
	{
		Subsystem->CancelGeneration(Source);
	}
}

bool FPCGDefaultExecutionState::IsGenerating() const
{
	check(Source);
	return Source->CurrentGenerationTask != InvalidPCGTaskId;
}

IPCGGraphExecutionSource* FPCGDefaultExecutionState::GetOriginalSource() const
{
	// Partitioned generation not supported.
	return Source;
}

#if PCG_PROFILING_ENABLED
const PCGUtils::FExtraCapture& FPCGDefaultExecutionState::GetExtraCapture() const
{
	check(Source);
	return Source->ExtraCapture;
}

PCGUtils::FExtraCapture& FPCGDefaultExecutionState::GetExtraCapture()
{
	check(Source);
	return Source->ExtraCapture;
}

const FPCGGraphExecutionInspection& FPCGDefaultExecutionState::GetInspection() const
{
	check(Source);
	return Source->Inspection;
}

FPCGGraphExecutionInspection& FPCGDefaultExecutionState::GetInspection()
{
	check(Source);
	return Source->Inspection;
}
#endif // PCG_PROFILING_ENABLED

void FPCGDefaultExecutionState::OnPostProcess(const FPCGDataCollection& InDataCollection)
{
	PostProcessCallback.ExecuteIfBound(InDataCollection);
}

void UPCGDefaultExecutionSource::SetGraphInterface(UPCGGraphInterface* InGraphInterface)
{
	if (InGraphInterface == GraphInterface)
	{
		return;
	}

#if WITH_EDITOR
	if (GraphInterface)
	{
		GraphInterface->OnGraphChangedDelegate.RemoveAll(this);
	}
#endif // WITH_EDITOR
	
	GraphInterface = InGraphInterface;

#if WITH_EDITOR
	if (GraphInterface)
	{
		GraphInterface->OnGraphChangedDelegate.AddUObject(this, &UPCGDefaultExecutionSource::OnGraphChanged);
	}
#endif // WITH_EDITOR
}

UPCGGraphInstance* UPCGDefaultExecutionSource::GetGraphInstance()
{
	return Cast<UPCGGraphInstance>(GraphInterface);
}

UPCGGraph* UPCGDefaultExecutionSource::GetGraph()
{
	return GraphInterface ? GraphInterface->GetMutablePCGGraph() : nullptr;
}

#if WITH_EDITOR
void UPCGDefaultExecutionSource::OnGraphChanged(UPCGGraphInterface* InGraphInterface, EPCGChangeType ChangeType)
{
	if (InGraphInterface != GraphInterface)
	{
		return;
	}

	if (ChangeType == EPCGChangeType::Cosmetic ||
		ChangeType == EPCGChangeType::GraphCustomization ||
		ChangeType == EPCGChangeType::None)
	{
		// If it is a cosmetic change (or no change), nothing to do
		return;
	}

	if (!InGraphInterface || !InGraphInterface->GetGraph())
	{
		return;
	}

	Generate();
}
#endif // WITH_EDITOR

void UPCGDefaultExecutionSource::Generate()
{
	if (CurrentGenerationTask != InvalidPCGTaskId)
	{
		return;
	}

	if (IPCGBaseSubsystem* Subsystem = State->GetSubsystem())
	{
		FPCGScheduleGraphParams GraphParams{
			/*InGraph=*/ GetGraph(),
			/*InExecutionSource=*/ this,
			/*InPreGraphElement=*/ nullptr,
			/*InInputElement=*/ MakeShared<FPCGInputForwardingElement>(FPCGDataCollection{}),
			/*InExternalDependencies*/ {},
			/*InFromStack*/ nullptr,
			/*bInAllowHierarchicalGeneration*/ false};
		
		const FPCGTaskId GenerationTaskId = Subsystem->ScheduleGraph(GraphParams);
		if (GenerationTaskId != InvalidPCGTaskId)
		{
			FPCGScheduleGenericParams Params{
				/*InOperation=*/ [WeakSubsystem = TWeakInterfacePtr(Subsystem)](FPCGContext* InContext) -> bool
				{
					// Source was deleted
					UPCGDefaultExecutionSource* This = Cast<UPCGDefaultExecutionSource>(InContext->ExecutionSource.Get());
					if (!This)
					{
						return true;
					}

					This->State->OnPostProcess(InContext->InputData);

					This->CurrentGenerationTask = InvalidPCGTaskId;
#if PCG_PROFILING_ENABLED
					if (WeakSubsystem.IsValid())
					{
						WeakSubsystem->OnPCGSourceGenerationDone(This, EPCGGenerationStatus::Completed);
					}
#endif // PCG_PROFILING_ENABLED
					
					return true;
				},
				/*InExecutionSource=*/this,
				/*InExecutionDependencies=*/{},
				// Implementation note: here we're taking the generation task as a data dependency to have data passed in the callback, because we'll need it for the post process.
				/*InDataDependencies=*/{GenerationTaskId},
				/*bSupportBasePointDataInput=*/true
			};
			
			CurrentGenerationTask = Subsystem->ScheduleGeneric(Params);
		}
#if PCG_PROFILING_ENABLED
		else
		{
			Subsystem->OnPCGSourceGenerationDone(this, EPCGGenerationStatus::Aborted);
		}
#endif // PCG_PROFILING_ENABLED
	}
}

void UPCGDefaultExecutionSource::Sunset()
{
#if WITH_EDITOR
	if (IsValid(GraphInterface))
	{
		check(GraphInterface);
		GraphInterface->OnGraphChangedDelegate.RemoveAll(this);
	}
#endif // WITH_EDITOR
}

UPCGDefaultExecutionSource::UPCGDefaultExecutionSource()
{
	State = MakeUnique<FPCGDefaultExecutionState>(this);
}

void UPCGDefaultExecutionSource::BeginDestroy()
{
	Sunset();

	Super::BeginDestroy();
}

void UPCGDefaultExecutionSource::Initialize(const ParamsType& InParams)
{
	SetGraphInterface(InParams.GraphInterface);
	SetSeed(InParams.Seed);
	State->SetPostProcessCallback(InParams.PostProcessCallback);
}

void UPCGDefaultExecutionSource::SetSeed(int32 InSeed)
{
	Seed = InSeed;
}

void UPCGDefaultExecutionSource::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UPCGDefaultExecutionSource* This = CastChecked<UPCGDefaultExecutionSource>(InThis);
	
#if WITH_EDITOR
	This->Inspection.AddReferencedObjects(Collector);
#endif // WITH_EDITOR

	Super::AddReferencedObjects(This, Collector);
}
