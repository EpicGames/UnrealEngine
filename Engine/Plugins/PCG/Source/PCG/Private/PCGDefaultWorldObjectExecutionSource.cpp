// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDefaultWorldObjectExecutionSource.h"

#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDefaultWorldObjectExecutionSource)

#if WITH_EDITOR
static TAutoConsoleVariable<bool> CVarLevelAssetGraphsUseTransactions(
	TEXT("pcg.LevelAsset.UseTransactions"),
	true,
	TEXT("Enables the use of transactions in level asset graphs."));
#endif

FPCGDefaultWorldObjectExecutionState::FPCGDefaultWorldObjectExecutionState()
{
#if WITH_EDITOR
	bUseTransactions = CVarLevelAssetGraphsUseTransactions.GetValueOnAnyThread();
#endif
}

FPCGDefaultWorldObjectExecutionState::FPCGDefaultWorldObjectExecutionState(UPCGDefaultWorldObjectExecutionSource* InSource)
	: FPCGDefaultExecutionState(InSource)
{
#if WITH_EDITOR
	bUseTransactions = CVarLevelAssetGraphsUseTransactions.GetValueOnAnyThread();
#endif
}

FString FPCGDefaultWorldObjectExecutionState::GetDebugName() const
{
	return FString(TEXT("PCGDefaultWorldObjectExecutionSource - ")) + (GetWorldObject() ? GetWorldObject()->GetName() : TEXT("Unknown object"));
}

UObject* FPCGDefaultWorldObjectExecutionState::GetTarget() const
{
	return GetWorldObject();
}

UWorld* FPCGDefaultWorldObjectExecutionState::GetWorld() const
{
	if (UObject* Object = GetWorldObject())
	{
		return Object->GetWorld();
	}
	else
	{
		return FPCGDefaultExecutionState::GetWorld();
	}
}

UPCGDefaultWorldObjectExecutionSource::UPCGDefaultWorldObjectExecutionSource()
{
	// Inefficient but more foolproof - we'll replace the previously created state from the base class by our version.
	State = MakeUnique<FPCGDefaultWorldObjectExecutionState>(this);
}

void UPCGDefaultWorldObjectExecutionSource::Initialize(const FPCGDefaultWorldObjectExecutionSourceParams& InParams)
{
	UPCGDefaultExecutionSource::Initialize(InParams);
	SetWorldObject(InParams.WorldObject);
}