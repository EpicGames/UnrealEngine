// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGDynamicPins.h"

#include "PCGNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDynamicPins)

TArray<FName> FPCGDynamicPinContainer::GetPinLabels() const
{
	TArray<FName> Labels;
	Labels.Reserve(PinProperties.Num());

	for (const FPCGPinProperties& Props : PinProperties)
	{
		Labels.Add(Props.Label);
	}

	return Labels;
}

#if WITH_EDITOR
void FPCGDynamicPinContainer::AddPin(FPCGPinProperties&& Properties)
{
	if (Properties.Label == NAME_None)
	{
		Properties.Label = FName(TEXT("NewPin"));
	}

	Properties.Label = MakeUniqueLabel(Properties.Label);

	PinProperties.Emplace(MoveTemp(Properties));
}

EPCGChangeType FPCGDynamicPinContainer::RemovePin(const int32 Index, UPCGNode* Node, const EPCGPinDirection Direction)
{
	check(Node && Index >= 0 && Index < PinProperties.Num());

	// Get the live pin from the node and break its edges
	UPCGPin* Pin = (Direction == EPCGPinDirection::Input) ? Node->GetInputPin(PinProperties[Index].Label) : Node->GetOutputPin(PinProperties[Index].Label);
	check(Pin);

	const bool bEdgeRemoved = Pin->BreakAllEdges();

	// Give the removed pin a placeholder label so it won't collide during UpdatePins.
	const FName PlaceholderLabel = MakeUniqueLabel(NAME_Error, /*ExcludeIndex=*/Index);
	if (Direction == EPCGPinDirection::Input)
	{
		Node->RenameInputPin(Pin->Properties.Label, PlaceholderLabel, /*bBroadcastUpdate=*/false);
	}
	else
	{
		Node->RenameOutputPin(Pin->Properties.Label, PlaceholderLabel, /*bBroadcastUpdate=*/false);
	}

	PinProperties.RemoveAt(Index);

	return EPCGChangeType::Node | EPCGChangeType::Settings | (bEdgeRemoved ? EPCGChangeType::Edge : EPCGChangeType::None);
}

EPCGChangeType FPCGDynamicPinContainer::RenamePin(const int32 Index, FName NewLabel, UPCGNode* Node, const EPCGPinDirection Direction)
{
	check(Node && Index >= 0 && Index < PinProperties.Num());

	const FName OldLabel = PinProperties[Index].Label;
	if (OldLabel == NewLabel)
	{
		return EPCGChangeType::None;
	}

	NewLabel = MakeUniqueLabel(NewLabel, Index);

	PinProperties[Index].Label = NewLabel;

	if (Direction == EPCGPinDirection::Input)
	{
		Node->RenameInputPin(OldLabel, NewLabel, /*bBroadcastUpdate=*/false);
	}
	else
	{
		Node->RenameOutputPin(OldLabel, NewLabel, /*bBroadcastUpdate=*/false);
	}

	return EPCGChangeType::Node | EPCGChangeType::Settings;
}

FName FPCGDynamicPinContainer::MakeUniqueLabel(FName BaseName, int32 ExcludeIndex) const
{
	auto LabelExists = [this, ExcludeIndex](const FName Label)
	{
		const int32 Index = FindPinIndex(Label);
		return Index != INDEX_NONE && Index != ExcludeIndex;
	};

	if (!LabelExists(BaseName))
	{
		return BaseName;
	}

	const FString BaseLabel = BaseName.ToString();
	uint32 Count = 1;
	FName Candidate = BaseName;
	while (LabelExists(Candidate))
	{
		Candidate = FName(FString::Printf(TEXT("%s%d"), *BaseLabel, Count++));
	}

	return Candidate;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> IPCGDynamicPinsProvider::GetCombinedPinProperties(const EPCGPinDirection Direction) const
{
	TArray<FPCGPinProperties> Combined = GetStaticPinProperties(Direction);

	if (const FPCGDynamicPinContainer* Container = GetDynamicPinContainer(Direction))
	{
		Combined.Append(Container->PinProperties);
	}

	return Combined;
}

TArray<FName> IPCGDynamicPinsProvider::GetAllDynamicPinLabels(const EPCGPinDirection Direction) const
{
	if (const FPCGDynamicPinContainer* Container = GetDynamicPinContainer(Direction))
	{
		return Container->GetPinLabels();
	}

	return {};
}
