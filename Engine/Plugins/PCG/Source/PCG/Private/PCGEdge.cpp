// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEdge.h"
#include "PCGPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEdge)

UPCGEdge::UPCGEdge(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetFlags(RF_Transactional);
}

void UPCGEdge::PostLoad()
{
	Super::PostLoad();
	SetFlags(RF_Transactional);
}

bool UPCGEdge::IsValid() const
{
	return InputPin.Get() && OutputPin.Get();
}

UPCGPin* UPCGEdge::GetOtherPin(const UPCGPin* Pin)
{
	check(Pin == InputPin || Pin == OutputPin);
	return Pin == InputPin ? OutputPin : InputPin;
}

const UPCGPin* UPCGEdge::GetOtherPin(const UPCGPin* Pin) const
{
	check(Pin == InputPin || Pin == OutputPin);
	return Pin == InputPin ? OutputPin : InputPin;
}

const UPCGNode* UPCGEdge::GetInputNode() const
{
	return InputPin ? InputPin->Node : nullptr;
}

const UPCGNode* UPCGEdge::GetOutputNode() const
{
	return OutputPin ? OutputPin->Node : nullptr;
}

FName UPCGEdge::GetInputPinLabel() const
{
	return InputPin ? InputPin->Properties.Label : NAME_None;
}

FName UPCGEdge::GetOutputPinLabel() const
{
	return OutputPin ? OutputPin->Properties.Label : NAME_None;
}
