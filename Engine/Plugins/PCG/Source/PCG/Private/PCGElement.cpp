// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGElement.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGSubsystem.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGTextureData.h"
#include "Graph/DataOverride/PCGDataOverride.h"
#include "Graph/DataOverride/PCGDataOverrideHelpers.h"
#include "Graph/PCGGraphCache.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"

#if WITH_EDITOR
#include "PCGDataVisualization.h"
#endif

#include "DrawDebugHelpers.h"
#include "EngineDefines.h" // For UE_ENABLE_DEBUG_DRAWING
#include "HAL/IConsoleManager.h"
#include "Utils/PCGExtraCapture.h"

#define LOCTEXT_NAMESPACE "PCGElement"

static TAutoConsoleVariable<bool> CVarPCGValidatePinAndTypes(
	TEXT("pcg.debug.ValidateDataPinsAndTypes"),
	true,
	TEXT("Controls whether we validate that output data from nodes are assigned to valid pins and have proper types for their pins."));

static TAutoConsoleVariable<bool> CVarPCGValidatePointMetadata(
	TEXT("pcg.debug.ValidatePointMetadata"),
	true,
	TEXT("Controls whether we validate that the metadata entry keys on the output point data are consistent"));

static TAutoConsoleVariable<bool> CVarPCGAllowPerDataCaching(
	TEXT("pcg.AllowPerDataCaching"),
	true,
	TEXT("Controls whether we test & split down inputs to check caching per input on primary loop nodes."));

static TAutoConsoleVariable<bool> CVarPCGShouldVerifyIfOutputsAreUsedMultipleTimes(
	TEXT("pcg.ShouldVerifyIfOutputsAreUsedMultipleTimes"),
	true,
	TEXT("Add small computation at the end of each node to detect if the data is used multiple times. Necessary for data stealing."));

static TAutoConsoleVariable<bool> CVarPCGEnablePointArrayDataToPointDataConversionWarnings(
	TEXT("pcg.EnablePointArrayDataToPointDataConversionWarnings"),
	false,
	TEXT("Warn about input conversions from PointArrayData to PointData so that code that needs to be updated is identified."));

#if WITH_EDITOR
#define PCG_ELEMENT_EXECUTION_BREAKPOINT() \
	if (Context && Context->GetInputSettingsInterface() && Context->GetInputSettingsInterface()->bBreakDebugger && Context->IsExecutingGraphInspected()) \
	{ \
		UE_DEBUG_BREAK(); \
	}
#else
#define PCG_ELEMENT_EXECUTION_BREAKPOINT()
#endif

namespace PCGElementHelpers
{
	bool SplitDataPerPrimaryPin(const UPCGSettings* Settings, const FPCGDataCollection& Collection, EPCGElementExecutionLoopMode Mode, TArray<FPCGDataCollection>& OutPrimaryCollections, FPCGDataCollection& OutCommonCollection)
	{
		check(Settings);
		OutPrimaryCollections.Reset();
		OutCommonCollection.TaggedData.Reset();

		// Early out
		if (Mode == EPCGElementExecutionLoopMode::NotALoop)
		{
			return false;
		}

		TArray<FPCGPinProperties> RequiredPins = Settings->InputPinProperties();
		RequiredPins.RemoveAll([](const FPCGPinProperties& Props) { return !Props.IsRequiredPin(); });

		// Early out if the single pin criteria is not met. Normally this is an implementer issue
		if (Mode == EPCGElementExecutionLoopMode::SinglePrimaryPin && RequiredPins.Num() != 1)
		{
			return false;
		}

		TArray<FName, TInlineAllocator<4>> RequiredPinLabels;
		Algo::Transform(RequiredPins, RequiredPinLabels, [](const FPCGPinProperties& Props) { return Props.Label; });

		TArray<FPCGDataCollection, TInlineAllocator<4>> DataPerRequiredPin;
		DataPerRequiredPin.SetNum(RequiredPinLabels.Num());

		for (int32 DataIndex = 0; DataIndex < Collection.TaggedData.Num(); ++DataIndex)
		{
			const FPCGTaggedData& TaggedData = Collection.TaggedData[DataIndex];
			int32 RequiredPinIndex = RequiredPinLabels.IndexOfByKey(TaggedData.Pin);

			if (RequiredPinIndex == INDEX_NONE)
			{
				OutCommonCollection.TaggedData.Add(TaggedData);
			}
			else
			{
				DataPerRequiredPin[RequiredPinIndex].TaggedData.Add(TaggedData);
			}
		}

		if (DataPerRequiredPin.IsEmpty())
		{
			return true;
		}

		// Broadcast to final primary collections
		if (Mode == EPCGElementExecutionLoopMode::SinglePrimaryPin)
		{
			check(DataPerRequiredPin.Num() == 1);
			OutPrimaryCollections.Reserve(DataPerRequiredPin[0].TaggedData.Num());

			for(int32 DataIndex = 0; DataIndex < DataPerRequiredPin[0].TaggedData.Num(); ++DataIndex)
			{
				FPCGDataCollection& OutPrimaryCollection = OutPrimaryCollections.Emplace_GetRef();
				OutPrimaryCollection.TaggedData.Add(DataPerRequiredPin[0].TaggedData[DataIndex]);
			}
		}
		else if (Mode == EPCGElementExecutionLoopMode::MatchingPrimaryPins)
		{
			const int32 NumberOfData = DataPerRequiredPin[0].TaggedData.Num();

			// Validate matching number of entries
			for (int32 RequiredPinIndex = 1; RequiredPinIndex < DataPerRequiredPin.Num(); ++RequiredPinIndex)
			{
				if (DataPerRequiredPin[RequiredPinIndex].TaggedData.Num() != NumberOfData)
				{
					return false;
				}
			}

			OutPrimaryCollections.SetNum(NumberOfData);

			for (int32 DataIndex = 0; DataIndex < NumberOfData; ++DataIndex)
			{
				for (int32 RequiredPinIndex = 0; RequiredPinIndex < DataPerRequiredPin.Num(); ++RequiredPinIndex)
				{
					OutPrimaryCollections[DataIndex].TaggedData.Add(DataPerRequiredPin[RequiredPinIndex].TaggedData[DataIndex]);
				}
			}
		}
		else if (Mode == EPCGElementExecutionLoopMode::PrimaryPinAndBroadcastablePins)
		{
			// Idea - behaves like the matching primary pins, but we'll move out the data from pins that have a single data instead.
			int32 NumberOfData = DataPerRequiredPin[0].TaggedData.Num();

			// In this case, we'll just validate that the values match, and that the other required pins don't have extraneous data too.
			for (int32 RequiredPinIndex = 1; RequiredPinIndex < DataPerRequiredPin.Num(); ++RequiredPinIndex)
			{
				const int32 DataCountOnPin = DataPerRequiredPin[RequiredPinIndex].TaggedData.Num();
				if (DataCountOnPin != 1 && DataCountOnPin != NumberOfData)
				{
					return false;
				}
			}

			OutPrimaryCollections.SetNum(NumberOfData);

			for (int32 RequiredPinIndex = 0; RequiredPinIndex < DataPerRequiredPin.Num(); ++RequiredPinIndex)
			{
				if (DataPerRequiredPin[RequiredPinIndex].TaggedData.Num() == NumberOfData)
				{
					for (int32 DataIndex = 0; DataIndex < NumberOfData; ++DataIndex)
					{
						OutPrimaryCollections[DataIndex].TaggedData.Add(DataPerRequiredPin[RequiredPinIndex].TaggedData[DataIndex]);
					}
				}
				else
				{
					ensure(DataPerRequiredPin[RequiredPinIndex].TaggedData.Num() == 1);
					OutCommonCollection.TaggedData.Append(DataPerRequiredPin[RequiredPinIndex].TaggedData);
				}
			}
		}
		/*else if (Mode == EPCGElementExecutionLoopMode::CartesianPins)
		{
			int32 NumberOfCollections = 1;
			for (int32 RequiredPinIndex = 0; RequiredPinIndex < DataPerRequiredPin.Num(); ++RequiredPinIndex)
			{
				NumberOfCollections *= DataPerRequiredPin[RequiredPinIndex].TaggedData.Num();
			}

			OutPrimaryCollections.SetNum(NumberOfCollections);

			for (int32 RequiredPinIndex = 0; RequiredPinIndex < DataPerRequiredPin.Num(); ++RequiredPinIndex)
			{
				const FPCGDataCollection& PinData = DataPerRequiredPin[RequiredPinIndex];

				for (int32 OutCollectionIndex = 0; OutCollectionIndex < OutPrimaryCollections.Num(); ++OutCollectionIndex)
				{
					OutPrimaryCollections[OutCollectionIndex].TaggedData.Add(PinData.TaggedData[OutCollectionIndex % PinData.TaggedData.Num()]);
				}
			}
		}*/
		else
		{
			// Invalid mode
			return false;
		}

		return true;
	}
}

bool IPCGElement::Execute(FPCGContext* Context) const
{
	// Note that this might be re-executed even if in the "Done" state because the task might have dynamic dependencies.
	check(Context && Context->AsyncState.NumAvailableTasks != 0 && Context->CurrentPhase <= EPCGExecutionPhase::Done);
	check(Context->AsyncState.bIsRunningOnMainThread || !CanExecuteOnlyOnMainThread(Context));

	FPCGContextScope ContextScope(Context);

	PCGUtils::FScopedCallOutputDevice OutputDevice;

	while (Context->CurrentPhase != EPCGExecutionPhase::Done)
	{
		PCGUtils::FScopedCall ScopedCall(this, Context, OutputDevice);
		bool bExecutionPostponed = false;

		switch (Context->CurrentPhase)
		{
			case EPCGExecutionPhase::NotExecuted:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(EPCGExecutionPhase::NotExecuted);
				PCG_ELEMENT_EXECUTION_BREAKPOINT();

				if (!PreExecute(Context))
				{
					bExecutionPostponed = true;
				}

				break;
			}

			case EPCGExecutionPhase::PrepareData:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(EPCGExecutionPhase::PrepareData);
				PCG_ELEMENT_EXECUTION_BREAKPOINT();

				if (PrepareData(Context))
				{
					Context->CurrentPhase = EPCGExecutionPhase::Execute;
				}
				else
				{
					bExecutionPostponed = true;
				}
				break;
			}

			case EPCGExecutionPhase::Execute:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(EPCGExecutionPhase::Execute);
				PCG_ELEMENT_EXECUTION_BREAKPOINT();

#if UE_ENABLE_DEBUG_DRAWING 
				if (PCGSystemSwitches::CVarPCGDebugDrawGeneratedCells.GetValueOnAnyThread())
				{
					PCGHelpers::DebugDrawGenerationVolume(Context);
				}
#endif // UE_ENABLE_DEBUG_DRAWING

				if (ExecuteInternal(Context))
				{
					Context->CurrentPhase = EPCGExecutionPhase::PostExecute;
				}
				else
				{
					bExecutionPostponed = true;
				}
				break;
			}

			case EPCGExecutionPhase::PostExecute:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(EPCGExecutionPhase::PostExecute);
				PCG_ELEMENT_EXECUTION_BREAKPOINT();

				PostExecute(Context);
				break;
			}

			default: // should not happen
			{
				check(0);
				break;
			}
		}

		if (bExecutionPostponed || 
			Context->AsyncState.ShouldStop() ||
			(!Context->AsyncState.bIsRunningOnMainThread && CanExecuteOnlyOnMainThread(Context))) // phase change might require access to main thread
		{
			break;
		}
	}

	return Context->CurrentPhase == EPCGExecutionPhase::Done;
}

bool IPCGElement::PreExecute(FPCGContext* Context) const
{
	check(Context);
	// Check for early outs (task cancelled + node disabled)
	// Early out to stop execution
	if (Context->InputData.bCancelExecution || (!Context->ExecutionSource.GetWeakObjectPtr().IsExplicitlyNull() && !Context->ExecutionSource.IsValid()))
	{
		Context->OutputData.bCancelExecution = true;

		if (IsCancellable())
		{
			// Skip task completely
			Context->CurrentPhase = EPCGExecutionPhase::Done;
			return true;
		}
	}

	// Prepare to move to prepare data phase
	Context->CurrentPhase = EPCGExecutionPhase::PrepareData;

	const UPCGSettingsInterface* SettingsInterface = Context->GetInputSettingsInterface();

	if (!SettingsInterface)
	{
		return true;
	}

	if (!SettingsInterface->bEnabled)
	{
		//Pass-through - no execution
		DisabledPassThroughData(Context);
		Context->CurrentPhase = EPCGExecutionPhase::PostExecute;
		return true;
	}

	const UPCGSettings* OriginalSettings = Context->GetOriginalSettings<UPCGSettings>();

	// If we were supposed to execute on GPU and end up here, then GPU compilation failed. Pass through.
	if (OriginalSettings && OriginalSettings->ShouldExecuteOnGPU())
	{
		DisabledPassThroughData(Context);
		Context->CurrentPhase = EPCGExecutionPhase::PostExecute;
		return true;
	}

	if (!ReadbackGPUDataForOverrides(Context))
	{
		Context->CurrentPhase = EPCGExecutionPhase::NotExecuted;
		return false;
	}

	// Will override the settings if there is any override.
	Context->OverrideSettings();

	const UPCGSettings* Settings = Context->GetInputSettings<UPCGSettings>();

	if (CVarPCGAllowPerDataCaching.GetValueOnAnyThread())
	{
		// Default implementation when the entries in a primary loop can be processed independently, e.g. they can appear in the cache separately
		// Implementation note: this supposes that the current node has only ONE required pin, and not multiple.
		// For more complex cases (such as multiple required pins, whether cartesian or matching), the implementation should use common code instead to streamline this process -
		// both when getting the results from the cache but also when writing them
		if (ExecutionLoopMode(Settings) != EPCGElementExecutionLoopMode::NotALoop && IsCacheableInstance(Settings))
		{
			PreExecutePrimaryLoopElement(Context, Settings);
		}
	}

	return true;
}

void IPCGElement::PreExecutePrimaryLoopElement(FPCGContext* Context, const UPCGSettings* Settings) const
{
	check(Context);

	if (!Settings)
	{
		return;
	}

	// No need to split data if we're not using the cache.
	if (!Context->IsCacheEnabled())
	{
		return;
	}

	// Mark inputs in the order they're presented so we can appropriately find the relation from output to input after the execution
	// TODO: this is not sufficient to do a proper mapping from output to input when we have a cartesian loop
	for (int32 DataIndex = 0; DataIndex < Context->InputData.TaggedData.Num(); ++DataIndex)
	{
		Context->InputData.TaggedData[DataIndex].OriginalIndex = DataIndex;
	}

	TArray<FPCGDataCollection> PrimaryDataCollections;
	FPCGDataCollection OtherData;
	if (!PCGElementHelpers::SplitDataPerPrimaryPin(Settings, Context->InputData, ExecutionLoopMode(Settings), PrimaryDataCollections, OtherData))
	{
		return;
	}

	// Implementation note: if there is a single primary data collection, then there's no point checking in the cache again, since that has already been done.
	if (PrimaryDataCollections.Num() <= 1)
	{
		return;
	}

	const bool bShouldComputeFullOutputDataCrc = ShouldComputeFullOutputDataCrc(Context);

	// Check against the cache if subcollections of one data from the primary data collection + the other data is found already in the cache.
	// If so, we can remove the matching input data - note that this is somewhat trivial in the single pin & matching pin cases, but in general is not going to work for cartesian cases.
	for (int32 PrimaryDataIndex = PrimaryDataCollections.Num() - 1; PrimaryDataIndex >= 0; --PrimaryDataIndex)
	{
		const FPCGDataCollection& PrimaryDataCollection = PrimaryDataCollections[PrimaryDataIndex];
		FPCGDataCollection SubCollection = PrimaryDataCollection;
		SubCollection.TaggedData.Append(OtherData.TaggedData);

		SubCollection.TaggedData.Sort([](const FPCGTaggedData& A, const FPCGTaggedData& B) { return A.OriginalIndex < B.OriginalIndex; });
		SubCollection.ComputeCrcs(bShouldComputeFullOutputDataCrc);

		FPCGGetFromCacheParams CacheParams = { .Node = Context->Node, .Element = this, .ExecutionSource = Context->ExecutionSource.Get() };
		GetDependenciesCrc(FPCGGetDependenciesCrcParams(&SubCollection, Settings, Context->ExecutionSource.Get()), CacheParams.Crc);

		FPCGDataCollection SubCollectionOutput;
		if (Context->GetFromCache(CacheParams, SubCollectionOutput))
		{
			// Found a match in the cache, add it to the output, and remove the matching inputs.
			// Note that in the input part we'll take only the data present in the PrimaryDataCollection here, as we will reintroduce only those then.
			// IMPLEMENTATION NOTE: the order is important here; if we can't guarantee the hint index, then we should just do a remove!
			TPair<FPCGDataCollection, FPCGDataCollection>& InputToOutputResults = Context->CachedInputToOutputInternalResults.Emplace_GetRef();
			InputToOutputResults.Key = PrimaryDataCollection;
			InputToOutputResults.Value = SubCollectionOutput;

			Context->InputData.TaggedData.RemoveAll([&PrimaryDataCollection](const FPCGTaggedData& Data) { return PrimaryDataCollection.TaggedData.Contains(Data); });
		}
	}

	//
	// TODO : if there are no inputs left, then we could skip the execute phase
	// 
}

bool IPCGElement::PrepareData(FPCGContext* Context) const
{
	if (!ConvertInputsIfNeeded(Context))
	{
		return false;
	}
	
	return PrepareDataInternal(Context);
}

bool IPCGElement::ConvertInputsIfNeeded(FPCGContext* Context) const
{
	if (Context->bConvertInputsIfNeededDone)
	{
		return true;
	}

	if (Context->InputData.TaggedData.IsEmpty())
	{
		Context->bConvertInputsIfNeededDone = true;
		return true;
	}

	if (!SupportsGPUResidentData(Context))
	{
		// Multi-stage GPU readback. The stages are sequential rather than parallel because the graph executor does not support mixing bIsPaused polling with
		// DynamicDependencies in the same task. The texture readback must be scheduled on the game thread because it can affect the texture source.

		// Stage 1: proxy readbacks (inline poll).
		{
			bool bHasPendingProxyReadbacks = false;

			for (FPCGTaggedData& TaggedData : Context->InputData.TaggedData)
			{
				if (const UPCGProxyForGPUData* DataWithGPUSupport = Cast<UPCGProxyForGPUData>(TaggedData.Data))
				{
#if WITH_EDITOR
					if (Context->Node && Context->GetStack() && Context->ExecutionSource.IsValid())
					{
						Context->ExecutionSource->GetExecutionState().GetInspection().NotifyGPUToCPUReadback(Context->Node, Context->GetStack());
					}
#endif

					UPCGProxyForGPUData::FReadbackResult Result = DataWithGPUSupport->GetCPUData(Context);
					if (Result.bComplete)
					{
						ensure(Result.TaggedData.Data);
						TaggedData.Data = Result.TaggedData.Data;
						Context->bInputDataModified = true;
					}
					else
					{
						bHasPendingProxyReadbacks = true;
					}
				}
			}

			if (bHasPendingProxyReadbacks)
			{
				Context->bIsPaused = true;

				// Unpause on the next game-thread tick so the scheduler re-enters this element and polls again.
				ExecuteOnGameThread(UE_SOURCE_LOCATION, [ContextHandle = Context->GetOrCreateHandle()]()
				{
					if (TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin())
					{
						if (FPCGContext* ContextPtr = SharedHandle->GetContext())
						{
							ContextPtr->bIsPaused = false;
						}
					}
				});

				return false;
			}
		}

		// Stage 2: texture readbacks (GT-only generic task, sleep via DynamicDependencies).
		{
			bool bHasPendingTextureReadbacks = false;
			for (const FPCGTaggedData& TaggedData : Context->InputData.TaggedData)
			{
				if (const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(TaggedData.Data))
				{
					if (!TextureData->IsReadbackComplete())
					{
						bHasPendingTextureReadbacks = true;
						break;
					}
				}
			}

			if (bHasPendingTextureReadbacks)
			{
				auto DoTextureReadbacks = [ContextHandle = Context->GetOrCreateHandle()](FPCGContext*)
				{
					TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin();
					FPCGContext* ContextPtr = SharedHandle ? SharedHandle->GetContext() : nullptr;
					if (!ContextPtr)
					{
						return true;
					}

					bool bAllComplete = true;
					for (const FPCGTaggedData& TaggedData : ContextPtr->InputData.TaggedData)
					{
						if (const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(TaggedData.Data))
						{
							if (!TextureData->IsReadbackComplete())
							{
#if WITH_EDITOR
								if (ContextPtr->Node && ContextPtr->GetStack() && ContextPtr->ExecutionSource.IsValid())
								{
									ContextPtr->ExecutionSource->GetExecutionState().GetInspection().NotifyGPUToCPUReadback(ContextPtr->Node, ContextPtr->GetStack());
								}
#endif

								if (!TextureData->RequestCPUReadback())
								{
									bAllComplete = false;
								}
							}
						}
					}

					return bAllComplete;
				};

				FPCGScheduleGenericParams Params(DoTextureReadbacks, Context->ExecutionSource.Get());
				Params.bCanExecuteOnlyOnMainThread = true;

				const FPCGTaskId TextureReadbackTaskId = Context->ScheduleGeneric(Params);
				if (ensure(TextureReadbackTaskId != InvalidPCGTaskId))
				{
					Context->DynamicDependencies.Add(TextureReadbackTaskId);
					return false;
				}
			}
		}
	}

	if (!SupportsBasePointDataInputs(Context))
	{
		bool bDataWasConverted = false;

		for (FPCGTaggedData& Data : Context->InputData.TaggedData)
		{
			if (const UPCGBasePointData* BasePointData = Cast<UPCGBasePointData>(Data.Data))
			{
				// No doesn't support anything else than UPCGPointData
				if (!BasePointData->IsA<UPCGPointData>())
				{
					Data.Data = BasePointData->ToPointData(Context);
					bDataWasConverted = true;
				}
			}
		}

		if (bDataWasConverted && CVarPCGEnablePointArrayDataToPointDataConversionWarnings.GetValueOnAnyThread())
		{
			const UPCGSettings* Settings = Context->GetOriginalSettings<UPCGSettings>();
			const FText SettingsName = Settings ? FText::FromString(Settings->GetName()) : LOCTEXT("UnknownSettings", "Unknown");
			PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("UnsupportedPointArrayData", "ToPointData was called on inputs of node '{0}'. Consider implementing support for UPCGBasePointData (IPCGElement::SupportsBasePointDataInputs)."), SettingsName));
		}
	}

	if (EnumHasAnyFlags(GetDataOverridePhase(), EPCGDataOverridePhase::PrepareData))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::ConvertInputsIfNeeded::ApplyDataOverrides);

		const bool bHasNodeLevelOverrides = PCG::DataOverride::HasNodeLevelDataOverrides(Context);
		bool bHasAnyOverrides = bHasNodeLevelOverrides;

		// Need to duplicate the data in pre-application phase to avoid stomping data shared by other elements.
		// @todo_pcg: Consider data stealing if possible.
		for (FPCGTaggedData& TaggedData : Context->InputData.TaggedData)
		{
			// If we have node overrides, all data will be copied. If not, check for pin level overrides.
			if (TaggedData.Data && (bHasNodeLevelOverrides || PCG::DataOverride::HasPinLevelDataOverrides(Context, TaggedData)))
			{
				TaggedData.Data = TaggedData.Data->DuplicateData(Context);
				bHasAnyOverrides = true;
			}
		}

		if (bHasAnyOverrides)
		{
			PCG::DataOverride::ApplyDataOverrides(Context, Context->InputData.TaggedData);

#if WITH_EDITOR
			// Notify inspection that input data overrides were applied (pre-execute).
			// @todo_pcg: Track orphaned overrides
			if (Context->Node && Context->GetStack() && Context->ExecutionSource.IsValid())
			{
				Context->ExecutionSource->GetExecutionState().GetInspection().NotifyDataOverridesApplied(Context->Node, Context->GetStack());
			}
#endif
		}
	}

	Context->bConvertInputsIfNeededDone = true;
	return true;
}

bool IPCGElement::PrepareDataInternal(FPCGContext* Context) const
{
	return true;
}

void IPCGElement::PostExecute(FPCGContext* Context) const
{
	const UPCGSettings* Settings = Context->GetInputSettings<UPCGSettings>();

	// Allow sub class to do some processing here
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("IPCGElement::PostExecute::PostExecuteInternal (%s)"), Settings ? *Settings->GetName() : TEXT("")));
		PostExecuteInternal(Context);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::PostExecute::CleanupAndValidateOutput);
		// Cleanup and validate output
		CleanupAndValidateOutput(Context);
	}

	if (CVarPCGAllowPerDataCaching.GetValueOnAnyThread())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::PostExecute::PostExecutePrimaryLoopElement);
		if (!Context->OutputData.bCancelExecution && ExecutionLoopMode(Settings) != EPCGElementExecutionLoopMode::NotALoop && IsCacheableInstance(Settings))
		{
			PostExecutePrimaryLoopElement(Context, Settings);
		}
	}

	const UPCGSettingsInterface* SettingsInterface = Context->GetInputSettingsInterface();

	// Data overrides must be applied before the Crc
	if (SettingsInterface
		&& SettingsInterface->bEnabled // Ignore overrides if node is disabled.
		&& !Context->OutputData.TaggedData.IsEmpty()
		&& EnumHasAnyFlags(GetDataOverridePhase(), EPCGDataOverridePhase::PostExecute)
		&& PCG::DataOverride::HasAnyDataOverrides(Context, Context->OutputData.TaggedData))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::PostExecute::ApplyDataOverrides);
		PCG::DataOverride::ApplyDataOverrides(Context, Context->OutputData.TaggedData);

#if WITH_EDITOR
		// Notify inspection that output data overrides were applied (post-execute).
		// @todo_pcg: Track orphaned overrides
		if (Context->Node && Context->GetStack() && Context->ExecutionSource.IsValid())
		{
			Context->ExecutionSource->GetExecutionState().GetInspection().NotifyDataOverridesApplied(Context->Node, Context->GetStack());
		}
#endif
	}

	if (Context->IsCacheEnabled())
	{
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::PostExecute::CRC);

			// Some nodes benefit from computing an actual CRC from the data. This can halt the propagation of change/executions through the graph. For
			// data like landscapes we will never have a full accurate data crc for it so we'll tend to assume changed which triggers downstream
			// execution. Performing a detailed CRC of output data can detect real change in the data and halt the cascade of execution.
			const bool bShouldComputeFullOutputDataCrc = ShouldComputeFullOutputDataCrc(Context);

			// Compute Crc from output data which will include output pin labels.
			Context->OutputData.ComputeCrcs(bShouldComputeFullOutputDataCrc);
		}

#if WITH_EDITOR
		const bool bHasErrorsOrWarnings = Context->Node && Context->HasVisualLogs();
#else
		const bool bHasErrorsOrWarnings = false;
#endif

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::PostExecute::StoreInCache);
			// Store result in cache
			// TODO - There is a potential mismatch here between using the Settings (incl. overrides) and the input settings interface (pre-overrides), as done in the graph executor.
			// TODO - The dependencies CRC here should always be valid except in the indirection case, which we should normalize to allow caching (tested here, otherwise it can ensure in the graph cache)
			if (!Context->OutputData.bCancelExecution && !bHasErrorsOrWarnings && Context->DependenciesCrc.IsValid() && IsCacheableInstance(Settings))
			{
				bool bCacheable = true;

				// GPU proxies are never cached. Caching them would hold on to precious graphics memory.
				for (const FPCGTaggedData& TaggedData : Context->OutputData.TaggedData)
				{
					if (TaggedData.Data && !TaggedData.Data->IsCacheable())
					{
						bCacheable = false;
						break;
					}
				}

				if (bCacheable)
				{
					FPCGStoreInCacheParams Params = { .Element = this, .Crc = Context->DependenciesCrc };
					Context->StoreInCache(Params, Context->OutputData);
				}
			}
		}
	}

	// Analyze if the output data is used multiple times, if the element requires it.
	if (ShouldVerifyIfOutputsAreUsedMultipleTimes(Settings))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::PostExecute::ShouldVerifyIfOutputsAreUsedMultipleTimes);
		
		TSet<const UPCGData*> InputData;
		TMap<const UPCGData*, int> OutputTaggedDataMap;
		OutputTaggedDataMap.Reserve(Context->OutputData.TaggedData.Num());
		for (FPCGTaggedData& OutputData : Context->OutputData.TaggedData)
		{
			OutputTaggedDataMap.FindOrAdd(OutputData.Data, 0) += 1;
		}

		for (const FPCGTaggedData& InputTaggedData: Context->InputData.TaggedData)
		{
			InputData.Add(InputTaggedData.Data);
		}
		
		for (FPCGTaggedData& OutputData : Context->OutputData.TaggedData)
		{
			// Enforce that pinless data is always used multiple times, or if the debug mode is enabled, or was detected multiple times in the output.
			if (OutputTaggedDataMap[OutputData.Data] > 1 || OutputData.bPinlessData || (Settings && Settings->CanBeDebugged() && Settings->bDebug))
			{
				OutputData.bIsUsedMultipleTimes = true;
				continue;
			}
			
			// For data that are marked to be used multiple times, they are potentially not used multiple times if they are not passthrough
			// (hence if they are not in the input). So set them back to false in that case. It will be set to true again by the executor
			// if it is actually used in multiple places.
			if (OutputData.bIsUsedMultipleTimes && !InputData.Contains(OutputData.Data))
			{
#if !UE_BUILD_SHIPPING
				OutputData.OriginatingNode = Context->Node;
#endif //!UE_BUILD_SHIPPING
				OutputData.bIsUsedMultipleTimes = false;
			}
		}
	}
	
#if WITH_EDITOR
	// Register the element to the component indicating the element has run and can have dynamic tracked keys.
	if (Settings && Settings->CanDynamicallyTrackKeys() && Context->ExecutionSource.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IPCGElement::PostExecute::RegisterDynamicTracking);
		Context->ExecutionSource->GetExecutionState().RegisterDynamicTracking(Settings, {});
	}
#endif // WITH_EDITOR

	Context->CurrentPhase = EPCGExecutionPhase::Done;
}

void IPCGElement::PostExecutePrimaryLoopElement(FPCGContext* Context, const UPCGSettings* Settings) const
{
	check(Context);

	if (!Settings || !Context->IsCacheEnabled())
	{
		return;
	}

#if WITH_EDITOR
	const bool bHasErrorsOrWarnings = Context->Node && Context->HasVisualLogs();
#else
	const bool bHasErrorsOrWarnings = false;
#endif

	// Store individual results in the cache; here we will try to match the remaining hint indices from the input data with the ones given at the output.
	TArray<FPCGDataCollection> PrimaryDataCollections;
	FPCGDataCollection OtherData;
	if (!Context->OutputData.bCancelExecution && !bHasErrorsOrWarnings && PCGElementHelpers::SplitDataPerPrimaryPin(Settings, Context->InputData, ExecutionLoopMode(Settings), PrimaryDataCollections, OtherData))
	{
		const bool bShouldComputeFullOutputDataCrc = ShouldComputeFullOutputDataCrc(Context);

		for (const FPCGDataCollection& PrimaryDataCollection : PrimaryDataCollections)
		{
			if (PrimaryDataCollection.TaggedData.IsEmpty())
			{
				continue;
			}
			
			const int32 OriginalIndex = PrimaryDataCollection.TaggedData[0].OriginalIndex;

			if (OriginalIndex == INDEX_NONE)
			{
				continue;
			}

			FPCGDataCollection SubCollectionOutput;
			SubCollectionOutput.TaggedData = Context->OutputData.TaggedData.FilterByPredicate([OriginalIndex](const FPCGTaggedData& TaggedData) { return TaggedData.OriginalIndex == OriginalIndex; });

			bool bCacheable = true;

			for (const FPCGTaggedData& TaggedData : SubCollectionOutput.TaggedData)
			{
				if (TaggedData.Data && !TaggedData.Data->IsCacheable())
				{
					bCacheable = false;
					break;
				}
			}

			if (bCacheable)
			{
				FPCGDataCollection SubCollection = PrimaryDataCollection;
				SubCollection.TaggedData.Append(OtherData.TaggedData);

				SubCollection.TaggedData.Sort([](const FPCGTaggedData& A, const FPCGTaggedData& B) { return A.OriginalIndex < B.OriginalIndex; });
				SubCollection.ComputeCrcs(bShouldComputeFullOutputDataCrc);

				FPCGCrc DependenciesCrc;
				GetDependenciesCrc(FPCGGetDependenciesCrcParams(&SubCollection, Settings, Context->ExecutionSource.Get()), DependenciesCrc);

				SubCollectionOutput.ComputeCrcs(bShouldComputeFullOutputDataCrc);

				FPCGStoreInCacheParams Params = { .Element = this, .Crc = DependenciesCrc };
				Context->StoreInCache(Params, SubCollectionOutput);
			}
		}
	}

	// Put back cached results and set aside input (needed for inspection) if any
	if (!Context->CachedInputToOutputInternalResults.IsEmpty())
	{
		// Push cached results back to the final output data, from the last to the first, at the right place.
		for (int CachedCollectionIndex = Context->CachedInputToOutputInternalResults.Num() - 1; CachedCollectionIndex >= 0; --CachedCollectionIndex)
		{
			// Since we've assigned original indices up front, we should be able to respect this here.
			const FPCGDataCollection& CachedInputData = Context->CachedInputToOutputInternalResults[CachedCollectionIndex].Key;
			const int32 CacheInputOriginalIndex = (CachedInputData.TaggedData.IsEmpty() ? INDEX_NONE : CachedInputData.TaggedData[0].OriginalIndex);

			for (const FPCGTaggedData& CachedInput : CachedInputData.TaggedData)
			{
				const int32 InsertInputIndex = CacheInputOriginalIndex != INDEX_NONE ? Context->InputData.TaggedData.IndexOfByPredicate([InputIndex = CachedInput.OriginalIndex](const FPCGTaggedData& TaggedData) { return TaggedData.OriginalIndex > InputIndex; }) : INDEX_NONE;

				if (InsertInputIndex != INDEX_NONE)
				{
					Context->InputData.TaggedData.Insert(CachedInput, InsertInputIndex);
				}
				else
				{
					Context->InputData.TaggedData.Add(CachedInput);
				}
			}
			
			// Note: we're modifying the context data, but it will not be reused after
			FPCGDataCollection& CachedOutputData = Context->CachedInputToOutputInternalResults[CachedCollectionIndex].Value;

			// Since this data comes from the cache, we can't rely on its original index; however, it should be used as-if it was keyed on the input original index.
			const int32 CacheOutputOriginalIndex = CacheInputOriginalIndex;

			// Apply matching original index so that further iterations place the data at the right indices.
			for (FPCGTaggedData& TaggedData : CachedOutputData.TaggedData)
			{
				TaggedData.OriginalIndex = CacheOutputOriginalIndex;
			}

			const int32 InsertOutputIndex = CacheOutputOriginalIndex != INDEX_NONE ? Context->OutputData.TaggedData.IndexOfByPredicate([CacheOutputOriginalIndex](const FPCGTaggedData& TaggedData) { return TaggedData.OriginalIndex > CacheOutputOriginalIndex; }) : INDEX_NONE;

			if (InsertOutputIndex != INDEX_NONE)
			{
				Context->OutputData.TaggedData.Insert(CachedOutputData.TaggedData, InsertOutputIndex);
			}
			else
			{
				Context->OutputData.TaggedData.Append(CachedOutputData.TaggedData);
			}
		}
	}
}

void IPCGElement::Abort(FPCGContext* Context) const
{
	AbortInternal(Context);
}

void IPCGElement::DisabledPassThroughData(FPCGContext* Context) const
{
	check(Context);

	const UPCGSettings* Settings = Context->GetInputSettings<UPCGSettings>();
	check(Settings);

	if (!Context->Node)
	{
		// Full pass-through if we don't have a node
		Context->OutputData = Context->InputData;
		return;
	}

	if (Context->Node->GetInputPins().Num() == 0 || Context->Node->GetOutputPins().Num() == 0)
	{
		// No input pins or not output pins, return nothing
		return;
	}

	const UPCGPin* PassThroughInputPin = Context->Node->GetPassThroughInputPin();
	const UPCGPin* PassThroughOutputPin = Context->Node->GetPassThroughOutputPin();
	if (!PassThroughInputPin || !PassThroughOutputPin)
	{
		// No pin to grab pass through data from or to pass data to.
		return;
	}

	const FPCGDataTypeIdentifier OutputType = PassThroughOutputPin->GetCurrentTypesID();

	// Grab data from pass-through pin, push it all to output pin
	Context->OutputData.TaggedData = Context->InputData.GetInputsByPin(PassThroughInputPin->Properties.Label);
	for (FPCGTaggedData& Data : Context->OutputData.TaggedData)
	{
		Data.Pin = PassThroughOutputPin->Properties.Label;
	}

	// Pass through input data if both it and the output are params, or if the output type supports it (e.g. if we have a incoming
	// surface connected to an input pin of type Any, do not pass the surface through to an output pin of type Point).
	auto InputDataShouldPassThrough = [OutputType](const FPCGTaggedData& InData)
	{
		const FPCGDataTypeIdentifier InputType = InData.Data ? FPCGDataTypeIdentifier{InData.Data->GetDataTypeId()} : FPCGDataTypeIdentifier{};
		const bool bInputTypeNotWiderThanOutputType = !InputType.IsWider(OutputType);

		return bInputTypeNotWiderThanOutputType && (InputType != EPCGDataType::Param || OutputType == EPCGDataType::Param);
	};

	// Now remove any non-params edges, and if only one edge should come through, remove the others
	if (Settings->OnlyPassThroughOneEdgeWhenDisabled())
	{
		// Find first incoming non-params data that is coming through the pass through pin
		TArray<FPCGTaggedData> InputsOnFirstPin = Context->InputData.GetInputsByPin(PassThroughInputPin->Properties.Label);
		const int FirstNonParamsDataIndex = InputsOnFirstPin.IndexOfByPredicate(InputDataShouldPassThrough);

		if (FirstNonParamsDataIndex != INDEX_NONE)
		{
			// Remove everything except the data we found above
			for (int Index = Context->OutputData.TaggedData.Num() - 1; Index >= 0; --Index)
			{
				if (Index != FirstNonParamsDataIndex)
				{
					Context->OutputData.TaggedData.RemoveAt(Index);
				}
			}
		}
		else
		{
			// No data found to return
			Context->OutputData.TaggedData.Empty();
		}
	}
	else
	{
		// Remove any incoming non-params data that is coming through the pass through pin
		TArray<FPCGTaggedData> InputsOnFirstPin = Context->InputData.GetInputsByPin(PassThroughInputPin->Properties.Label);
		for (int Index = InputsOnFirstPin.Num() - 1; Index >= 0; --Index)
		{
			const FPCGTaggedData& Data = InputsOnFirstPin[Index];

			if (!InputDataShouldPassThrough(Data))
			{
				Context->OutputData.TaggedData.RemoveAt(Index);
			}
		}
	}
}

FPCGContext* IPCGElement::CreateContext()
{
	return new FPCGContext();
}

#if WITH_EDITOR
bool IPCGElement::DebugDisplay(FPCGContext* Context) const
{
	// Check Debug flag.
	const UPCGSettingsInterface* SettingsInterface = Context->GetInputSettingsInterface();
	if (!SettingsInterface || !SettingsInterface->bDebug)
	{
		return true;
	}

	// If graph is being inspected, only display Debug if the execution source is being inspected, or in the HiGen case also display if
	// this component is a parent of an inspected component (because this data is available to child components).

	// If the graph is not being inspected, or the current component is being inspected, then we know we should display
	// debug, if not then we do further checks.
	IPCGGraphExecutionSource* ExecutionSource = Context->ExecutionSource.Get();
	UPCGGraph* Graph = ExecutionSource ? ExecutionSource->GetExecutionState().GetGraph() : nullptr;
	if (Graph && Graph->IsInspecting() && !ExecutionSource->GetExecutionState().GetInspection().IsInspecting() && Graph->DebugFlagAppliesToIndividualComponents())
	{
		// If we're no doing HiGen, or if the current component is not a local component (and therefore will not have children),
		// then do not display debug.
		if (!Graph->IsHierarchicalGenerationEnabled())
		{
			return true;
		}

		// If a child of this component is being inspected (a local component on smaller grid and overlapping) then we still show debug,
		// because this data is available to that child for use.
		if (UPCGSubsystem* Subsystem = Cast<UPCGSubsystem>(ExecutionSource->GetExecutionState().GetSubsystem()))
		{
			const uint32 ThisGenerationGridSize = ExecutionSource->GetExecutionState().GetGenerationGridSize();

			bool bFoundInspectedChild = false;

			// @todo_pcg: support execution source
			if(UPCGComponent* OriginalComponent = Cast<UPCGComponent>(ExecutionSource))
			{
				Subsystem->ForAllOverlappingComponentsInHierarchy(OriginalComponent, [ThisGenerationGridSize, &bFoundInspectedChild](UPCGComponent* InLocalComponent)
				{
					if (InLocalComponent->GetExecutionState().GetGenerationGridSize() < ThisGenerationGridSize && InLocalComponent->GetExecutionState().GetInspection().IsInspecting())
					{
						bFoundInspectedChild = true;
					}
				});
			}

			// If no inspected child component then don't display debug.
			if (!bFoundInspectedChild)
			{
				return true;
			}
		}
	}

	// In the case of a node with multiple output pins, we will select only the inputs from the first non-empty pin.
	const UPCGPin* FirstOutPin = Context->Node ? Context->Node->GetFirstConnectedOutputPin() : nullptr;

	bool bHasPendingReadbacks = false;

	TArray<FPCGTaggedData> DataToDebug;
	DataToDebug.Reserve(Context->OutputData.TaggedData.Num());

	// If there are any proxies in the input data, request readback to CPU.
	for (const FPCGTaggedData& Output : Context->OutputData.TaggedData)
	{
		// Skip output if we're filtering on the first pin or the the data is null.
		if (!Output.Data || (FirstOutPin && FirstOutPin->Properties.Label != Output.Pin))
		{
			continue;
		}

		if (const UPCGProxyForGPUData* DataWithGPUSupport = Cast<UPCGProxyForGPUData>(Output.Data))
		{
			// Poll until readback is done.
			UPCGProxyForGPUData::FReadbackResult Result = DataWithGPUSupport->GetCPUData(Context);

			if (Result.bComplete)
			{
				// Readback done. Queue up data for debug (unless we've found pending readbacks, in which case we'll run again later).
				if (ensure(Result.TaggedData.Data) && !bHasPendingReadbacks)
				{
					FPCGTaggedData TaggedData = Output;
					TaggedData.Data = Result.TaggedData.Data;
					TaggedData.Tags.Append(Result.TaggedData.Tags);

					DataToDebug.Add(TaggedData);
				}
			}
			else
			{
				bHasPendingReadbacks = true;
			}
		}
		else if (const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(Output.Data))
		{
			if (!TextureData->IsReadbackComplete())
			{
				if (Context->Node && Context->GetStack() && Context->ExecutionSource.IsValid())
				{
					Context->ExecutionSource->GetExecutionState().GetInspection().NotifyGPUToCPUReadback(Context->Node, Context->GetStack());
				}

				// Schedule a game-thread-only task to trigger the readback and poll until complete. The readback must run on the game thread because touching the texture source on a worker thread crashes.
				auto DoReadback = [WeakTextureData = MakeWeakObjectPtr(TextureData)](FPCGContext*)
				{
					const UPCGTexture2DSingleBaseData* TexData = WeakTextureData.Get();
					return !TexData || TexData->RequestCPUReadback();
				};

				FPCGScheduleGenericParams Params(DoReadback, Context->ExecutionSource.Get());
				Params.bCanExecuteOnlyOnMainThread = true;

				FPCGTaskId ReadbackTaskId = Context->ScheduleGeneric(Params);
				if (ReadbackTaskId != InvalidPCGTaskId)
				{
					Context->DynamicDependencies.Add(ReadbackTaskId);
				}

				bHasPendingReadbacks = true;
			}
			else
			{
				DataToDebug.Add(Output);
			}
		}
		else
		{
			DataToDebug.Add(Output);
		}
	}

	if (bHasPendingReadbacks)
	{
		// Signal that data was not ready, will run again later.
		return false;
	}

	const FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetConstPCGDataVisualizationRegistry();

	for (const FPCGTaggedData& Data : DataToDebug)
	{
		if (const IPCGDataVisualization* DataVis = DataVisRegistry.GetDataVisualization(Data.Data->GetClass()))
		{
			DataVis->ExecuteDebugDisplay(Context, SettingsInterface, Data.Data, Context->GetDebugDisplayActor());
		}
	}

	return true;
}
#endif // WITH_EDITOR

void IPCGElement::CleanupAndValidateOutput(FPCGContext* Context) const
{
	check(Context);
	const UPCGSettingsInterface* SettingsInterface = Context->GetInputSettingsInterface();
	const UPCGSettings* Settings = SettingsInterface ? SettingsInterface->GetSettings() : nullptr;

	// Implementation note - disabled passthrough nodes can happen only in subgraphs/ spawn actor nodes
	// which will behave properly when disabled. 
	if (Settings && !IsPassthrough(Settings))
	{
		// Cleanup any residual labels if the node isn't supposed to produce them
		// TODO: this is a bit of a crutch, could be refactored out if we review the way we push tagged data
		TArray<FPCGPinProperties> OutputPinProperties = Settings->AllOutputPinProperties();
		if(OutputPinProperties.Num() == 1)
		{
			for (FPCGTaggedData& TaggedData : Context->OutputData.TaggedData)
			{
				if (!TaggedData.bPinlessData)
				{
					TaggedData.Pin = OutputPinProperties[0].Label;
				}
			}
		}

		// Validate all out data for errors in labels
#if WITH_EDITOR
		const bool bNeedsPinsAndTypesValidation = CVarPCGValidatePinAndTypes.GetValueOnAnyThread();
		const bool bNeedsMetadataValidation =  CVarPCGValidatePointMetadata.GetValueOnAnyThread();

		if (SettingsInterface->bEnabled && (bNeedsPinsAndTypesValidation || bNeedsMetadataValidation))
		{
			if (bNeedsPinsAndTypesValidation)
			{
				// remove null outputs
				Context->OutputData.TaggedData.RemoveAll([this, Context](const FPCGTaggedData& TaggedData) {

					if (TaggedData.Data == nullptr)
					{
						PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("NullPinOutputData", "Invalid output(s) generated for pin '{0}'"), FText::FromName(TaggedData.Pin)));
						return true;
					}

					return false;
					});
			}

			for (FPCGTaggedData& TaggedData : Context->OutputData.TaggedData)
			{
				if (bNeedsPinsAndTypesValidation)
				{
					const int32 MatchIndex = OutputPinProperties.IndexOfByPredicate([&TaggedData](const FPCGPinProperties& InProp) { return TaggedData.Pin == InProp.Label; });
					if (MatchIndex == INDEX_NONE)
					{
						// Only display an error if we expected this data to have a pin.
						if (!TaggedData.bPinlessData)
						{
							PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("OutputCannotBeRouted", "Output data generated for non-existent output pin '{0}'"), FText::FromName(TaggedData.Pin)));
						}
					}
					else if (ensure(TaggedData.Data))
					{
						// Try to get dynamic current pin types, otherwise settle for static types
						const UPCGPin* OutputPin = Context->Node ? Context->Node->GetOutputPin(OutputPinProperties[MatchIndex].Label) : nullptr;
						const FPCGDataTypeIdentifier PinTypes = OutputPin ? OutputPin->GetCurrentTypesID() : OutputPinProperties[MatchIndex].AllowedTypes;

						FPCGDataTypeIdentifier DataType = TaggedData.Data->GetUnderlyingDataTypeId();
						const bool bTypesOverlap = !!(PinTypes & DataType);
						const bool bTypeIsSubset = !DataType.IsWider(PinTypes);
						// TODO: Temporary fix for Settings directly from InputData (ie. from elements with code and not PCG nodes)
						if ((!bTypesOverlap || !bTypeIsSubset) && DataType != EPCGDataType::Settings)
						{
							PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("OutputIncompatibleType", "Output data generated for pin '{0}' does not have a compatible type: '{1}'. Consider using more specific/narrower input pin types, or more general/wider output pin types."), FText::FromName(TaggedData.Pin), DataType.ToDisplayText()));
						}
					}
				}

				if (bNeedsMetadataValidation)
				{
					if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(TaggedData.Data))
					{
						const int32 MaxMetadataEntry = PointData->Metadata ? PointData->Metadata->GetItemCountForChild() : 0;

						bool bHasError = false;

						const TConstPCGValueRange<int64> MetadataEntryRange = PointData->GetConstMetadataEntryValueRange();

						for(int32 PointIndex = 0; PointIndex < MetadataEntryRange.Num() && !bHasError; ++PointIndex)
						{
							bHasError |= (MetadataEntryRange[PointIndex] >= MaxMetadataEntry);
						}

						if (bHasError)
						{
							PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("OutputMissingPointMetadata", "Output generated for pin '{0}' does not have valid point metadata"), FText::FromName(TaggedData.Pin)));
						}
					}

					if (const UPCGMetadata* Metadata = TaggedData.Data.Get() ? TaggedData.Data->ConstMetadata() : nullptr)
					{
						const bool bMetadataHasParent = (Metadata->GetParent() != nullptr);

						TArray<FPCGMetadataDomainID> DomainIDs = TaggedData.Data->GetAllSupportedMetadataDomainIDs();
						for (FPCGMetadataDomainID DomainID : DomainIDs)
						{
							if (const FPCGMetadataDomain* Domain = Metadata->GetConstMetadataDomain(DomainID))
							{
								const bool bDomainHasParent = (Domain->GetParent() != nullptr);

								if (bDomainHasParent && !bMetadataHasParent)
								{
									PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("InvalidDomainParenting", "Invalid parenting on the domain '{0}'."), FText::FromName(DomainID.DebugName)));
								}
							}
						}
					}
				}
			}
		}
#endif
	}
}

FPCGContext* IPCGElement::Initialize(const FPCGDataCollection& InputData, TWeakObjectPtr<UPCGComponent> SourceComponent, const UPCGNode* Node)
{
	return nullptr;
}

FPCGContext* IPCGElement::Initialize(const FPCGInitializeElementParams& InParams)
{
	check(InParams.InputData);
	// For backward compatibility (call the old signature)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPCGContext* Context = Initialize(*InParams.InputData, Cast<UPCGComponent>(InParams.ExecutionSource.GetObject()), InParams.Node);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	if (!Context)
	{
		Context = CreateContext();
		Context->InitFromParams(InParams);
	}
	else
	{
		Context->ExecutionSource = InParams.ExecutionSource;

		const UPCGSettings* Settings = InParams.Node ? InParams.Node->GetSettings() : nullptr;
		const FString SettingsName = Settings ? Settings->GetName() : TEXT("Unknown");
		UE_LOGF(LogPCG, Warning, "This node '%ls' implements a deprecated version of Initialize. Please implement version with FPCGInitializeElementParams parameter.", *SettingsName);
	}

	return Context;
}

bool IPCGElement::IsCacheable(const UPCGSettings* Settings) const
{
	return !Settings || !Settings->ShouldExecuteOnGPU();
}

bool IPCGElement::IsCacheableInstance(const UPCGSettingsInterface* InSettingsInterface) const
{
	if (InSettingsInterface)
	{
		if (!InSettingsInterface->bEnabled)
		{
			return false;
		}
		else
		{
			return IsCacheable(InSettingsInterface->GetSettings());
		}
	}
	else
	{
		return false;
	}
}

bool IPCGElement::ShouldVerifyIfOutputsAreUsedMultipleTimes(const UPCGSettings* InSettings) const
{
	return CVarPCGShouldVerifyIfOutputsAreUsedMultipleTimes.GetValueOnAnyThread();
}

void IPCGElement::GetDependenciesCrc(const FPCGDataCollection& InInput, const UPCGSettings* InSettings, UPCGComponent* InComponent, FPCGCrc& OutCrc) const
{
	// do nothing
}

void IPCGElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("IPCGElement::GetDependenciesCrc (%s)"), InParams.Settings ? *InParams.Settings->GetName() : TEXT("")));
	
	check(InParams.InputData);
	FPCGCrc CopyCrc(OutCrc);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GetDependenciesCrc(*InParams.InputData, InParams.Settings, Cast<UPCGComponent>(InParams.ExecutionSource), OutCrc);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Call to deprecated method didn't yield a different Crc so we calculate it here
	if (!OutCrc.IsValid() || CopyCrc == OutCrc)
	{
		// Start from a random prime.
		FPCGCrc Crc(1000003);

		// The cached data CRCs are computed in FPCGGraphExecutor::BuildTaskInput and incorporate data CRC, tags, output pin label and input pin label.
		for (const FPCGCrc& DataCrc : InParams.InputData->DataCrcs)
		{
			Crc.Combine(DataCrc);
		}

		if (InParams.Settings)
		{
			FPCGCrc SettingsCrc = InParams.Settings->GetSettingsCrc();
			if (ensure(SettingsCrc.IsValid()))
			{
				Crc.Combine(SettingsCrc);
			}
		}

		if (InParams.ExecutionSource && (!InParams.Settings || InParams.Settings->UseSeed()))
		{
			Crc.Combine(InParams.ExecutionSource->GetExecutionState().GetSeed());
		}

		const FPCGSourceDataContainer* DataContainer = InParams.ExecutionSource ? InParams.ExecutionSource->GetExecutionState().GetSourceDataContainer() : nullptr;
		if (DataContainer && !DataContainer->IsEmpty())
		{
			// @todo_pcg: This ties a non-empty data container to every element's Crc. If the data within the container is changed,
			// every element's Crc will be invalidated. To be addressed in a secondary pass.
			Crc.Combine(DataContainer->GetDirtyGeneration());
		}

		OutCrc = Crc;
	}
	else
	{
		const FString SettingsName = InParams.Settings ? InParams.Settings->GetName() : TEXT("Unknown");
		UE_LOGF(LogPCG, Warning, "This node '%ls' implements a deprecated version of GetDependenciesCrc. Please implement version with FPCGGetDependenciesCrcParams parameter.", *SettingsName);
	}
}

bool IPCGElement::BuildCacheParams(IPCGGraphCache* Cache, const UPCGNode* Node, const FPCGDataCollection& Input, IPCGGraphExecutionSource* ExecutionSource, FPCGGetFromCacheParams& OutCacheParams) const
{
	if (!Cache || !Cache->IsEnabled())
	{
		return false;
	}

	const UPCGSettingsInterface* SettingsInterface = Input.GetSettingsInterface(Node ? Node->GetSettingsInterface() : nullptr);
	const UPCGSettings* Settings = SettingsInterface ? SettingsInterface->GetSettings() : nullptr;
	const bool bCacheable = IsCacheableInstance(SettingsInterface);

	OutCacheParams = { .Node = Node, .Element = this, .ExecutionSource = ExecutionSource };

	if (Settings && bCacheable)
	{
		GetDependenciesCrc(FPCGGetDependenciesCrcParams(&Input, Settings, ExecutionSource), OutCacheParams.Crc);
		return true;
	}
	else
	{
		return false;
	}
}

EPCGCachingStatus IPCGElement::RetrieveResultsFromCache(IPCGGraphCache* Cache, const FPCGGetFromCacheParams& InCacheParams, FPCGDataCollection& Output) const
{
	check(Cache && Cache->IsEnabled() && InCacheParams.Crc.IsValid());

	if (Cache->GetFromCache(InCacheParams, Output))
	{
		return EPCGCachingStatus::Cached;
	}

	return EPCGCachingStatus::NotInCache;
}

EPCGCachingStatus IPCGElement::RetrieveResultsFromCache(IPCGGraphCache* Cache, const UPCGNode* Node, const FPCGDataCollection& Input, IPCGGraphExecutionSource* ExecutionSource, FPCGDataCollection& Output, FPCGCrc* OutCrc) const
{
	if (!Cache || !Cache->IsEnabled())
	{
		return EPCGCachingStatus::NotInCache;
	}

	const UPCGSettingsInterface* SettingsInterface = Input.GetSettingsInterface(Node ? Node->GetSettingsInterface() : nullptr);
	const UPCGSettings* Settings = SettingsInterface ? SettingsInterface->GetSettings() : nullptr;
	const bool bCacheable = IsCacheableInstance(SettingsInterface);

	FPCGGetFromCacheParams Params = { .Node = Node, .Element = this, .ExecutionSource = ExecutionSource };

	if (Settings && bCacheable)
	{
		GetDependenciesCrc(FPCGGetDependenciesCrcParams(&Input, Settings, ExecutionSource), Params.Crc);

		if (OutCrc)
		{
			*OutCrc = Params.Crc;
		}
	}

	if (Params.Crc.IsValid() && Cache->GetFromCache(Params, Output))
	{
		return EPCGCachingStatus::Cached;
	}
	else
	{
		return EPCGCachingStatus::NotInCache;
	}
}

bool IPCGElement::ReadbackGPUDataForOverrides(FPCGContext* Context) const
{
	check(Context);
	
	if (Context->InputData.TaggedData.IsEmpty())
	{
		return true;
	}

	const UPCGSettings* OriginalSettings = Context->GetOriginalSettings<UPCGSettings>();

	if (!OriginalSettings)
	{
		return true;
	}

	const TArray<FPCGSettingsOverridableParam>& OverridableParams = OriginalSettings->OverridableParams();

	if (OverridableParams.IsEmpty())
	{
		return true;
	}

	bool bHasPendingReadbacks = false;

	// If there are any proxies in the input data, request readback to CPU.
	for (FPCGTaggedData& TaggedData : Context->InputData.TaggedData)
	{
		const UPCGProxyForGPUData* DataWithGPUSupport = Cast<UPCGProxyForGPUData>(TaggedData.Data);

		if (DataWithGPUSupport && OverridableParams.ContainsByPredicate([&TaggedData](const FPCGSettingsOverridableParam& InOverridableParam) { return InOverridableParam.Label == TaggedData.Pin; }))
		{
#if WITH_EDITOR
			if (Context->Node && Context->GetStack() && Context->ExecutionSource.IsValid())
			{
				Context->ExecutionSource->GetExecutionState().GetInspection().NotifyGPUToCPUReadback(Context->Node, Context->GetStack());
			}
#endif

			// Poll until readback is done.
			UPCGProxyForGPUData::FReadbackResult Result = DataWithGPUSupport->GetCPUData(Context);

			if (Result.bComplete)
			{
				// Readback done, replace the proxy with the result data.
				ensure(Result.TaggedData.Data);
				TaggedData.Data = Result.TaggedData.Data;
				TaggedData.Tags.Append(Result.TaggedData.Tags);

				// Will update referenced data objects.
				Context->bInputDataModified = true;
			}
			else
			{
				bHasPendingReadbacks = true;
			}
		}
	}

	if (bHasPendingReadbacks)
	{
		Context->bIsPaused = true;

		// Not ready to execute and unlikely to be in the very short term, sleep until next frame.
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [ContextHandle = Context->GetOrCreateHandle()]()
		{
			if (TSharedPtr<FPCGContextHandle> SharedHandle = ContextHandle.Pin())
			{
				if (FPCGContext* ContextPtr = SharedHandle->GetContext())
				{
					ContextPtr->bIsPaused = false;
				}
			}
		});

		return false;
	}

	return true;
}

#undef PCG_ELEMENT_EXECUTION_BREAKPOINT
#undef LOCTEXT_NAMESPACE
