// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGEngineSettings.h"

#include "HAL/IConsoleManager.h"
#include "Misc/ConfigUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEngineSettings)

#define LOCTEXT_NAMESPACE "PCGEngineSettings"

FName UPCGEngineSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UPCGEngineSettings::GetSectionText() const
{
	return LOCTEXT("PCGEngineSettingsName", "PCG");
}
#endif // WITH_EDITOR

void UPCGEngineSettings::PostInitProperties()
{
#if WITH_EDITOR
	if (IsTemplate())
	{
		// We want the .ini file to have precedence over the CVar constructor, so we apply the ini to the CVar before following the regular UDeveloperSettingsBackedByCVars flow
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/PCG.PCGEngineSettings"), *GEngineIni, ECVF_SetByProjectSetting);
	}
#endif // WITH_EDITOR

	Super::PostInitProperties();
}

#undef LOCTEXT_NAMESPACE
