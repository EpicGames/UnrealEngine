// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGGraph.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGEdge.h"
#include "PCGInputOutputSettings.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSubgraph.h"
#include "PCGSubsystem.h"
#include "Data/PCGUserParametersData.h"
#include "Editor/IPCGEditorModule.h"
#include "Elements/PCGHiGenGridSize.h"
#include "Elements/PCGUserParameterGet.h"
#include "Elements/ControlFlow/PCGQualityBranch.h"
#include "Elements/ControlFlow/PCGQualitySelect.h"
#include "Graph/PCGGraphCompilationData.h"
#include "Graph/PCGGraphCompiler.h"
#include "Graph/PCGGraphExecutor.h"
#include "Helpers/PCGPropertyHelpers.h"

#include "AssetRegistry/AssetData.h"
#include "Interfaces/ITargetPlatform.h"
#include "Logging/StructuredLog.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/ObjectEditorOptionalSupport.h"

#if WITH_EDITOR
#include "CoreGlobals.h"
#include "Editor.h"
#include "Engine/AssetManager.h"
#include "Dialogs/Dialogs.h"
#include "EdGraph/EdGraphPin.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectSaveContext.h"
#else
#include "UObject/Package.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGraph)

#define LOCTEXT_NAMESPACE "PCGGraph"

namespace PCGGraph
{
	static TAutoConsoleVariable<bool> CVarFixInvalidEdgesOnPostLoad(
		TEXT("pcg.Graph.FixInvalidEdgesOnPostLoad"),
		true,
		TEXT("Validates all edges are connected to valid pins/nodes and removes any invalid edges"));

	static TAutoConsoleVariable<bool> CVarEnableComputeGraphInstancePooling(
		TEXT("pcg.GPU.ComputeGraphInstancePooling"),
		false,
		TEXT("Caches compute graph instances rather than creating each instance and its data providers from scratch each time."));

	static TAutoConsoleVariable<bool> CVarEnableWarningOnPinDestructiveChangesInPostLoad(
		TEXT("pcg.Graph.EnableWarningOnPinDestructiveChangesInPostLoad"),
		false,
		TEXT("Output warnings when destructive changes happens on PostLoad in UpdatePins."));

	static TAutoConsoleVariable<bool> CVarRecompileGraphOnChange(
		TEXT("pcg.Graph.CompileTasksToDetectChanges"),
		true,
		TEXT("Recompile graph after edits to detect if compiled tasks changed, to avoid unnecessary graph execution"));

	static TAutoConsoleVariable<bool> CVarEnableActorComponentlessGeneration(
		TEXT("pcg.RuntimeGeneration.EnableActorComponentlessGeneration"),
		false,
		TEXT("Allows executing runtime PCG graphs without partition actors and local PCG components."));
}

namespace PCGGraphUtils
{
	/** Returns true if the two descriptors are valid and compatible */
	bool ArePropertiesCompatible(const FPropertyBagPropertyDesc* InSourcePropertyDesc, const FPropertyBagPropertyDesc* InTargetPropertyDesc)
	{
		return InSourcePropertyDesc && InTargetPropertyDesc && InSourcePropertyDesc->CompatibleType(*InTargetPropertyDesc);
	}

	/** Checks if the value for a source property in a source struct has the same value that the target property in the target struct. */
	bool ArePropertiesIdentical(const FPropertyBagPropertyDesc* InSourcePropertyDesc, const FInstancedPropertyBag& InSourceInstance, const FPropertyBagPropertyDesc* InTargetPropertyDesc, const FInstancedPropertyBag& InTargetInstance)
	{
		if (!InSourceInstance.IsValid() || !InTargetInstance.IsValid() || !InSourcePropertyDesc || !InSourcePropertyDesc->CachedProperty || !InTargetPropertyDesc || !InTargetPropertyDesc->CachedProperty)
		{
			return false;
		}

		if (!InSourcePropertyDesc->CompatibleType(*InTargetPropertyDesc))
		{
			return false;
		}

		const uint8* SourceValueAddress = InSourceInstance.GetValue().GetMemory() + InSourcePropertyDesc->CachedProperty->GetOffset_ForInternal();
		const uint8* TargetValueAddress = InTargetInstance.GetValue().GetMemory() + InTargetPropertyDesc->CachedProperty->GetOffset_ForInternal();

		return InSourcePropertyDesc->CachedProperty->Identical(SourceValueAddress, TargetValueAddress);
	}

	/** Copy the value for a source property in a source struct to the target property in the target struct. */
	void CopyPropertyValue(const FPropertyBagPropertyDesc* InSourcePropertyDesc, const FInstancedPropertyBag& InSourceInstance, const FPropertyBagPropertyDesc* InTargetPropertyDesc, FInstancedPropertyBag& InTargetInstance)
	{
		if (!InSourceInstance.IsValid() || !InTargetInstance.IsValid() || !InSourcePropertyDesc || !InSourcePropertyDesc->CachedProperty || !InTargetPropertyDesc || !InTargetPropertyDesc->CachedProperty)
		{
			return;
		}

		// Can't copy if they are not compatible.
		if (!InSourcePropertyDesc->CompatibleType(*InTargetPropertyDesc))
		{
			return;
		}

		const uint8* SourceValueAddress = InSourceInstance.GetValue().GetMemory() + InSourcePropertyDesc->CachedProperty->GetOffset_ForInternal();
		uint8* TargetValueAddress = InTargetInstance.GetMutableValue().GetMemory() + InTargetPropertyDesc->CachedProperty->GetOffset_ForInternal();

		InSourcePropertyDesc->CachedProperty->CopyCompleteValue(TargetValueAddress, SourceValueAddress);
	}

#if WITH_EDITOR
	/** Get the value for a source property as string to be used with Export and Import text. */
	bool GetDefaultPropertyValueForEditor(const FPropertyBagPropertyDesc* InSourcePropertyDesc, const FInstancedPropertyBag& InSourceInstance, FString& OutValueString)
	{
		if (!InSourceInstance.IsValid() || !InSourcePropertyDesc || !InSourcePropertyDesc->CachedProperty)
		{
			return false;
		}

		InSourcePropertyDesc->CachedProperty->ExportText_InContainer(0, OutValueString, InSourceInstance.GetValue().GetMemory(), InSourceInstance.GetValue().GetMemory(), nullptr, PPF_None);
		return true;
	}
#endif // WITH_EDITOR

	EPCGChangeType NotifyTouchedNodes(const TSet<UPCGNode*>& InTouchedNodes, EPCGChangeType InChangeType)
	{
		EPCGChangeType FinalChangeType = EPCGChangeType::None;

		// Build a final list of all touched nodes, so we can broadcast the change once below.
		TSet<UPCGNode*> FinalTouchedNodes = InTouchedNodes;

		for (UPCGNode* TouchedNode : InTouchedNodes)
		{
			if (TouchedNode)
			{
				const EPCGChangeType NodeChangeType = InChangeType | TouchedNode->PropagateDynamicPinTypes(FinalTouchedNodes);

				FinalChangeType |= NodeChangeType;
			}
		}

		// Do change notifications for the final set.
#if WITH_EDITOR
		for (UPCGNode* TouchedNode : FinalTouchedNodes)
		{
			check(TouchedNode);
			TouchedNode->OnNodeChangedDelegate.Broadcast(TouchedNode, EPCGChangeType::Node | InChangeType);
		}
#endif

		return FinalChangeType;
	}
}

/****************************
* UPCGGraphInterface
****************************/

FText UPCGGraphInterface::GetDisplayName() const
{
#if WITH_EDITOR
	TOptional<FText> OverrideTitle = GetTitleOverride();
	if (OverrideTitle.IsSet())
	{
		return OverrideTitle.GetValue();
	}
#endif

	return FText::FromString(FName::NameToDisplayString(GetName(), /*bIsBool=*/false));
}

#if WITH_EDITOR
bool UPCGGraphInterface::IsStandaloneGraph() const
{
	return GetGraphUsage() != EPCGGraphUsage::Standard;
}

EPCGGraphUsage UPCGGraphInterface::GetGraphUsage() const
{
	return GetGraph() ? GetGraph()->GraphUsageContext : EPCGGraphUsage::Standard;
}

UObject* UPCGGraphInterface::GetUserParameterHierarchyRoot() const
{
	return nullptr;
}

bool UPCGGraphInterface::IsStandaloneGraphAsset(const FAssetData& InAssetData)
{
	return GetGraphUsage(InAssetData) != EPCGGraphUsage::Standard;
}

EPCGGraphUsage UPCGGraphInterface::GetGraphUsage(const FAssetData& InAssetData)
{
	if (UPCGGraphInterface* GraphInterface = Cast<UPCGGraphInterface>(InAssetData.FastGetAsset(/*bLoad=*/false)))
	{
		return GraphInterface->GetGraphUsage();
	}
	else if (InAssetData.IsInstanceOf<UPCGGraph>())
	{
		bool bTagValue = false;
		FString TagStringValue;
		// Two cases here:
		// older data -> use bIsStandaloneGraph, if true -> Asset usage.
		// newer data -> use GraphUsageContext.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (InAssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(UPCGGraph, bIsStandaloneGraph), bTagValue) && bTagValue)
		{
			return EPCGGraphUsage::Asset;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		else if (InAssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(UPCGGraph, GraphUsageContext), TagStringValue))
		{
			int64 UsageValue = StaticEnum<EPCGGraphUsage>()->GetValueByNameString(TagStringValue);
			return UsageValue == INDEX_NONE ? EPCGGraphUsage::Standard : static_cast<EPCGGraphUsage>(UsageValue);
		}
	}
	else if (InAssetData.IsInstanceOf<UPCGGraphInstance>())
	{
		FSoftObjectPath GraphPath = FSoftObjectPath(InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, Graph)));
		UAssetManager& AssetManager = UAssetManager::Get();
		FAssetData GraphAssetData;
		if (AssetManager.GetAssetDataForPath(GraphPath, GraphAssetData))
		{
			return GetGraphUsage(GraphAssetData);
		}
	}

	return EPCGGraphUsage::Standard;
}
#endif

EPropertyBagResult UPCGGraphInterface::SetGraphParameter(const FName PropertyName, const uint64 Value, const UEnum* Enum)
{
	FInstancedPropertyBag* UserParameters = GetMutableUserParametersStruct();
	check(UserParameters);

	const EPropertyBagResult Result = FPCGGraphParameterExtension::SetGraphParameter(*UserParameters, PropertyName, Value, Enum);
	if (Result == EPropertyBagResult::Success)
	{
		OnGraphParametersChanged(EPCGGraphParameterEvent::ValueModifiedLocally, PropertyName);
	}
	return Result;
}

#if WITH_EDITOR
EPropertyBagAlterationResult UPCGGraphInterface::RenameUserParameter(const FName CurrentName, const FName NewName)
{
	EPropertyBagAlterationResult Result = EPropertyBagAlterationResult::SourcePropertyNotFound;
	if (FInstancedPropertyBag* UserParametersStruct = GetMutableUserParametersStruct())
	{
		if (UserParametersStruct->FindPropertyDescByName(CurrentName))
		{
			Modify();
			Result = UserParametersStruct->RenameProperty(CurrentName, NewName);

			if (Result == EPropertyBagAlterationResult::Success)
			{
				OnGraphParametersChanged(EPCGGraphParameterEvent::PropertyRenamed, NewName);
			}
		}
	}

	return Result;
}
#endif // WITH_EDITOR

bool UPCGGraphInterface::UpdateArrayGraphParameter(const FName PropertyName, TFunctionRef<bool(FPropertyBagArrayRef& PropertyBagArrayRef)> Callback)
{
	FInstancedPropertyBag* UserParameters = GetMutableUserParametersStruct();
	check(UserParameters);

	TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> Result = UserParameters->GetMutableArrayRef(PropertyName);

	if (!Result.HasError() && Result.HasValue() && Callback(Result.GetValue()))
	{
		OnGraphParametersChanged(EPCGGraphParameterEvent::ValueModifiedLocally, PropertyName);
		return true;
	}
	else
	{
		return false;
	}
}

bool UPCGGraphInterface::UpdateSetGraphParameter(const FName PropertyName, TFunctionRef<bool(FPropertyBagSetRef& PropertyBagSetRef)> Callback)
{
	FInstancedPropertyBag* UserParameters = GetMutableUserParametersStruct();
	check(UserParameters);

	TValueOrError<FPropertyBagSetRef, EPropertyBagResult> Result = UserParameters->GetMutableSetRef(PropertyName);

	if (!Result.HasError() && Result.HasValue() && Callback(Result.GetValue()))
	{
		OnGraphParametersChanged(EPCGGraphParameterEvent::ValueModifiedLocally, PropertyName);
		return true;
	}
	else
	{
		return false;
	}
}

bool UPCGGraphInterface::IsInstance() const
{
	return this != GetGraph();
}

bool UPCGGraphInterface::IsEquivalent(const UPCGGraphInterface* Other) const
{
	if (this == Other)
	{
		return true;
	}

	const UPCGGraph* OtherGraph = Other ? Other->GetGraph() : nullptr;
	const UPCGGraph* ThisGraph = GetGraph();

	if (ThisGraph != OtherGraph)
	{
		return false;
	}
	else if (!ThisGraph && !OtherGraph)
	{
		return true;
	}

	const FInstancedPropertyBag* OtherParameters = Other->GetUserParametersStruct();
	const FInstancedPropertyBag* ThisParameters = this->GetUserParametersStruct();
	check(OtherParameters && ThisParameters);

	if (ThisParameters->GetNumPropertiesInBag() != OtherParameters->GetNumPropertiesInBag())
	{
		return false;
	}

	const UPropertyBag* OtherPropertyBag = OtherParameters->GetPropertyBagStruct();
	const UPropertyBag* ThisPropertyBag = ThisParameters->GetPropertyBagStruct();

	if (!ThisPropertyBag || !OtherPropertyBag)
	{
		return ThisPropertyBag == OtherPropertyBag;
	}

	// TODO: Be more resitant to different layout.
	// For now we are only comparing structs that must have the same layout.
	TConstArrayView<FPropertyBagPropertyDesc> OtherParametersDescs = OtherPropertyBag->GetPropertyDescs();
	TConstArrayView<FPropertyBagPropertyDesc> ThisParametersDescs = ThisPropertyBag->GetPropertyDescs();
	check(OtherParametersDescs.Num() == ThisParametersDescs.Num());

	// TODO: Hashing might be more efficient.
	for (int32 i = 0; i < ThisParametersDescs.Num(); ++i)
	{
		const FPropertyBagPropertyDesc& ThisParametersDesc = ThisParametersDescs[i];
		const FPropertyBagPropertyDesc& OtherParametersDesc = OtherParametersDescs[i];

		if (!PCGGraphUtils::ArePropertiesCompatible(&ThisParametersDesc, &OtherParametersDesc))
		{
			return false;
		}

		if (!PCGGraphUtils::ArePropertiesIdentical(&ThisParametersDesc, *ThisParameters, &OtherParametersDesc, *OtherParameters))
		{
			return false;
		}
	}

	return true;
}

UPCGUserParametersData* UPCGGraphInterface::ConstructUserParametersData(FPCGContext* InContext) const
{
	const FInstancedPropertyBag* UserParameters = GetUserParametersStruct();
	if (!UserParameters)
	{
		return nullptr;
	}

	UPCGUserParametersData* UserParamData = FPCGContext::NewObject_AnyThread<UPCGUserParametersData>(InContext);
	UserParamData->UserParameters = UserParameters->GetValue();
	return UserParamData;
}

#if WITH_EDITOR
bool UPCGGraphInterface::VerifyAndUpdateIfGraphParameterValueChanged(const FPropertyChangedChainEvent& PropertyChangedEvent)
{
	const FInstancedPropertyBag* UserParameters = GetUserParametersStruct();
	const UPropertyBag* PropertyBag = UserParameters ? UserParameters->GetPropertyBagStruct() : nullptr;

	if (!PropertyBag)
	{
		return false;
	}

	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* PropertyNode = PropertyChangedEvent.PropertyChain.GetHead();
	
	while (PropertyNode)
	{
		if (PropertyNode->GetValue() && PropertyNode->GetValue()->GetOwnerStruct() == PropertyBag)
		{
			break;
		}

		PropertyNode = PropertyNode->GetNextNode();
	}

	if (PropertyNode && PropertyNode->GetValue())
	{
		OnGraphParametersChanged(EPCGGraphParameterEvent::ValueModifiedLocally, PropertyNode->GetValue()->GetFName());
		return true;
	}
	else
	{
		return false;
	}
}

bool UPCGGraphInterface::VerifyIfGraphCustomizationChanged(const FPropertyChangedChainEvent& PropertyChangedEvent)
{
	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* PropertyNode = PropertyChangedEvent.PropertyChain.GetHead();
	while (PropertyNode)
	{
		if (PropertyNode->GetValue() && PropertyNode->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGGraph, GraphCustomization))
		{
			return true;
		}

		PropertyNode = PropertyNode->GetNextNode();
	}

	return false;
}
#endif // WITH_EDITOR

EPCGChangeType UPCGGraphInterface::GetChangeTypeForGraphParameterChange(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName)
{
	// If the parameter had its order changed in the struct, was just added or was removed but was not used in the graph, it is not a change that requires a refresh, so we go with Cosmetic change type.
	if (InChangeType == EPCGGraphParameterEvent::PropertyMoved
		|| InChangeType == EPCGGraphParameterEvent::Added
		|| InChangeType == EPCGGraphParameterEvent::RemovedUnused
		|| InChangeType == EPCGGraphParameterEvent::CategoryChanged)
	{
		return EPCGChangeType::Cosmetic;
	}

	// If it is not linked to a single property, or it was removed and used, we need to refresh, so we go with Settings change type.
	if (InChangedPropertyName == NAME_None || InChangeType == EPCGGraphParameterEvent::RemovedUsed)
	{
		return EPCGChangeType::Settings;
	}

	const UPCGGraph* Graph = GetGraph();
	const FInstancedPropertyBag* UserParameters = GetUserParametersStruct();
	if (!ensure(Graph && UserParameters))
	{
		// Should never happen, but if there is no graph nor user parameters, there is nothing to do.
		return EPCGChangeType::None;
	}

	// Finally if anything change on a property that has an impact for the graph, look for GetUserParameters nodes for this property, to only refresh if the property is used.
	// TODO: add tracking for user parameters from subgraphs
	/*for (const UPCGNode* Node : Graph->GetNodes())
	{
		if (!Node)
		{
			continue;
		}

		if (UPCGUserParameterGetSettings* Settings = Cast<UPCGUserParameterGetSettings>(Node->GetSettings()))
		{
			if (Settings->PropertyName == InChangedPropertyName)
			{
				return EPCGChangeType::Settings;
			}
		}
	}

	// At this point, we didn't find any node that use our property, so no refresh needed.
	return EPCGChangeType::Cosmetic;*/
	return EPCGChangeType::Settings;
}

/****************************
* UPCGGraph
****************************/

UPCGGraph::UPCGGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.DoNotCreateNestedDefaultSubobject({ TEXT("DefaultInputNode"), TEXT("DefaultNodeSettings") }).DoNotCreateNestedDefaultSubobject({ TEXT("DefaultOutputNode"), TEXT("DefaultNodeSettings") }))
{
	InputNode = ObjectInitializer.CreateDefaultSubobject<UPCGNode>(this, TEXT("DefaultInputNode"));
	InputNode->SetFlags(RF_Transactional);

	// Since pins would be allocated after initializing the input/output nodes, we must make sure to allocate them using the object initializer
	int NumAllocatedPins = 1;
	auto PinAllocator = [&ObjectInitializer, &NumAllocatedPins](UPCGNode* Node)
	{
		FName DefaultPinName = TEXT("DefaultPin");
		DefaultPinName.SetNumber(NumAllocatedPins++);
		return ObjectInitializer.CreateDefaultSubobject<UPCGPin>(Node, DefaultPinName);
	};

	UPCGGraphInputOutputSettings* InputSettings = ObjectInitializer.CreateDefaultSubobject<UPCGGraphInputOutputSettings>(this, TEXT("DefaultInputNodeSettings"));
	InputSettings->SetInput(true);
	InputNode->SetSettingsInterface(InputSettings, /*bUpdatePins=*/false);

	// Only allocate default pins if this is the default object
	InputNode->CreateDefaultPins(PinAllocator);
	
	OutputNode = ObjectInitializer.CreateDefaultSubobject<UPCGNode>(this, TEXT("DefaultOutputNode"));
	OutputNode->SetFlags(RF_Transactional);

	UPCGGraphInputOutputSettings* OutputSettings = ObjectInitializer.CreateDefaultSubobject<UPCGGraphInputOutputSettings>(this, TEXT("DefaultOutputNodeSettings"));
	OutputSettings->SetInput(false);
	OutputNode->SetSettingsInterface(OutputSettings, /*bUpdatePins=*/false);

	// Only allocate default pins if this is the default object
	OutputNode->CreateDefaultPins(PinAllocator);
	
#if WITH_EDITOR
	OutputNode->PositionX = 200;
#endif

#if WITH_EDITOR
	InputNode->OnNodeChangedDelegate.AddUObject(this, &UPCGGraph::OnNodeChanged);
	OutputNode->OnNodeChangedDelegate.AddUObject(this, &UPCGGraph::OnNodeChanged);
#endif

	// Note: default connection from input to output
	// should be added when creating from scratch,
	// but not when using a blueprint construct script.
	//InputNode->ConnectTo(OutputNode);
	//OutputNode->ConnectFrom(InputNode);

	// Force the user parameters to have an empty property bag. It is necessary to catch the first
	// add property into the undo/redo history.
	UserParameters.MigrateToNewBagStruct(UPropertyBag::GetOrCreateFromDescs({}));
}

void UPCGGraph::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	UE_LOG_CONTEXT("PCGGraph", FSoftObjectPath(this));
	
	// TODO: review this once the data has been updated, it's unwanted weight going forward
	// Deprecation
	InputNode->ConditionalPostLoad();

	if (!Cast<UPCGGraphInputOutputSettings>(InputNode->GetSettings()))
	{
		InputNode->SetSettingsInterface(NewObject<UPCGGraphInputOutputSettings>(this, TEXT("DefaultInputNodeSettings")));
	}

	Cast<UPCGGraphInputOutputSettings>(InputNode->GetSettings())->SetInput(true);

	OutputNode->ConditionalPostLoad();

	if (!Cast<UPCGGraphInputOutputSettings>(OutputNode->GetSettings()))
	{
		OutputNode->SetSettingsInterface(NewObject<UPCGGraphInputOutputSettings>(this, TEXT("DefaultOutputNodeSettings")));
	}

	Cast<UPCGGraphInputOutputSettings>(OutputNode->GetSettings())->SetInput(false);

	// Ensure that all nodes are loaded (& updated their deprecated data)
	// If a node is null (can happen if an asset was saved with a node that don't exist in the current session), we don't want to crash. So remove the faulty node and warn the user.
	// Keep track if that ever happen to force an edge cleanup
	bool bHasInvalidNode = false;

	for (int32 i = Nodes.Num() - 1; i >= 0; --i)
	{
		if (!Nodes[i])
		{
			UE_LOGF(LogPCG, Error, "Graph %ls has a node that doesn't exist anymore. Check if you are missing a plugin or if you saved an asset with an old settings that was removed/renamed.", *GetPathName());
			bHasInvalidNode = true;
			Nodes.RemoveAtSwap(i);
		}
		else
		{
			Nodes[i]->ConditionalPostLoad();
		}
	}

	// Also do this for ExtraNodes
	for (int32 i = ExtraEditorNodes.Num() - 1; i >= 0; --i)
	{
		if (!ExtraEditorNodes[i])
		{
			UE_LOGF(LogPCG, Error, "Graph %ls has an extra non-PCG node that doesn't exist anymore. Check if you are missing a plugin or if you saved an asset with an old settings that was removed/renamed.", *GetPathName());
			ExtraEditorNodes.RemoveAtSwap(i);
		}
		else
		{
			ExtraEditorNodes[i]->ConditionalPostLoad();

			// And convert Comment nodes to comment node data.
			if (const UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(ExtraEditorNodes[i]))
			{
				CommentNodes.Emplace_GetRef().InitializeFromCommentNode(*CommentNode);
				ExtraEditorNodes.RemoveAtSwap(i);
			}
		}
	}

	// Create a copy to iterate through the nodes while more might be added
	TArray<UPCGNode*> NodesCopy(Nodes);
	for (UPCGNode* Node : NodesCopy)
	{
		Node->ApplyStructuralDeprecation();
	}

	// Finally, apply deprecation that changes edges/rebinds
	ForEachNode([](UPCGNode* InNode) { InNode->ApplyDeprecationBeforeUpdatePins(); return true; });

	// Update pins on all nodes
	bool bQuiet = !PCGGraph::CVarEnableWarningOnPinDestructiveChangesInPostLoad.GetValueOnAnyThread();
	ForEachNode([bQuiet](UPCGNode* InNode) { InNode->UpdatePins(bQuiet); return true; });

	// Finally, apply deprecation that changes edges/rebinds
	ForEachNode([](UPCGNode* InNode) { InNode->ApplyDeprecation(); return true; });

	InputNode->OnNodeChangedDelegate.AddUObject(this, &UPCGGraph::OnNodeChanged);
	OutputNode->OnNodeChangedDelegate.AddUObject(this, &UPCGGraph::OnNodeChanged);

	// Also, try to remove all nodes that are invalid (meaning that the settings are null)
	// We remove it at the end, to let the nodes that have null settings to clean up their pins and edges.
	for (int32 i = Nodes.Num() - 1; i >= 0; --i)
	{
		if (!Nodes[i]->GetSettings())
		{
			Nodes.RemoveAtSwap(i);
		}
	}

	// Nodes is an array of TObjectPtr so we need this trick to convert to a UPCGNode*.
	OnNodesAdded(static_cast<TArray<UPCGNode*>>(MutableView(Nodes)), /*bNotify=*/false);

	if (bHasInvalidNode || PCGGraph::CVarFixInvalidEdgesOnPostLoad.GetValueOnAnyThread())
	{
		FixInvalidEdges();
	}

	SetupEditorData();
	
	int32 CustomVersion = GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (CustomVersion < FFortniteMainBranchObjectVersion::PCG_PropertyBagHierarchySupport)
	{
		if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
		{
			if (UserParameterHierarchyRoot)
			{
				if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
				{
					PCGEditorModule->TransferPropertyBagMetadataIntoHierarchy(this);
				}
			}
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (HiGenExponential != 0)
	{
		HiGenGridSizeMultiplier = static_cast<double>(1 << HiGenExponential);

		// Also propagate multiplier to generation radii, because this was the behavior we had in 5.7
		auto UpdateGenerationRadius = [Multiplier = HiGenGridSizeMultiplier](double& Radius)
		{
			Radius = FMath::Min(Radius * Multiplier, PCGHiGenGrid::UnboundedGridSize() * FPCGRuntimeGenerationRadii::DefaultGenerationRadiusMultiplier);
		};

		UpdateGenerationRadius(GenerationRadii.GenerationRadius);
		UpdateGenerationRadius(GenerationRadii.GenerationRadius400);
		UpdateGenerationRadius(GenerationRadii.GenerationRadius800);
		UpdateGenerationRadius(GenerationRadii.GenerationRadius1600);
		UpdateGenerationRadius(GenerationRadii.GenerationRadius3200);
		UpdateGenerationRadius(GenerationRadii.GenerationRadius6400);
		UpdateGenerationRadius(GenerationRadii.GenerationRadius12800);
		UpdateGenerationRadius(GenerationRadii.GenerationRadius25600);
		UpdateGenerationRadius(GenerationRadii.GenerationRadius51200);
		UpdateGenerationRadius(GenerationRadii.GenerationRadius102400);
		UpdateGenerationRadius(GenerationRadii.GenerationRadius204800);

		HiGenExponential = 0;
	}

	GenerationRadii.ApplyDeprecation();

	if (bIsEditorOnly)
	{
		ShouldCook = false;
		bIsEditorOnly = false;
	}

	if (bIsStandaloneGraph)
	{
		if (GraphUsageContext == EPCGGraphUsage::Standard)
		{
			GraphUsageContext = EPCGGraphUsage::Asset;
		}

		bIsStandaloneGraph = false;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

void UPCGGraph::PostInitProperties()
{
	Super::PostInitProperties();
#if WITH_EDITOR
	SetupEditorData();
#endif
}

void UPCGGraph::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
#if WITH_EDITOR
	SetupEditorData();
#endif
}

void UPCGGraph::PostEditImport()
{
	Super::PostEditImport();
#if WITH_EDITOR
	SetupEditorData();
#endif
}

bool UPCGGraph::IsEditorOnly() const
{
#if WITH_EDITOR
	if (FPCGModule::IsEditorOnlyGeneration())
	{
		return true;
	}
#endif

	// Embedded subgraphs inherit the decision from their parent.
	if (const UPCGGraph* EmbeddedParentGraph = GetEmbeddedParentGraph())
	{
		return EmbeddedParentGraph->IsEditorOnly();
	}

	bool bIsCurrentlyEditorOnly = (Super::IsEditorOnly() || !AnyPlatformShouldCook());

	if (!bIsCurrentlyEditorOnly)
	{
		auto IsSubgraphEditorOnly = [&bIsCurrentlyEditorOnly](UPCGNode* Node)
		{
			if (const UPCGBaseSubgraphNode* SubgraphNode = Cast<UPCGBaseSubgraphNode>(Node))
			{
				if (const UPCGGraph* Subgraph = SubgraphNode->GetSubgraph())
				{
					if (Subgraph->IsEditorOnly_Internal())
					{
						bIsCurrentlyEditorOnly = true;
						return false;
					}
				}
			}

			return true;
		};

		ForEachNodeRecursively(IsSubgraphEditorOnly);
	}

	return bIsCurrentlyEditorOnly;
}

void UPCGGraph::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYTAG(PCG);
	
	Super::Serialize(Ar);
	
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}

bool UPCGGraph::IsEditorOnly_Internal() const
{
	if (const UPCGGraph* EmbeddedParentGraph = GetEmbeddedParentGraph())
	{
		return EmbeddedParentGraph->IsEditorOnly_Internal();
	}

	return Super::IsEditorOnly() || !AnyPlatformShouldCook();
}

bool UPCGGraph::NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const
{
	if (!Super::NeedsLoadForTargetPlatform(TargetPlatform))
	{
		return false;
	}

	return ShouldCookForPlatform(TargetPlatform);
}

bool UPCGGraph::ShouldCookForPlatform(const ITargetPlatform* TargetPlatform) const
{
	if (const UPCGGraph* EmbeddedParentGraph = GetEmbeddedParentGraph())
	{
		return EmbeddedParentGraph->ShouldCookForPlatform(TargetPlatform);
	}

#if WITH_EDITOR || WITH_PREVIEW_PPX_DATA
	return ShouldCook.GetValueForPlatform(TargetPlatform ? FName(*TargetPlatform->IniPlatformName()) : NAME_None);
#else
	return ShouldCook.GetValue();
#endif
}

bool UPCGGraph::Contains(const UPCGGraph* InGraph) const
{
	bool bContains = false;
	auto ContainsSelectedGraph = [InGraph, &bContains](UPCGNode* Node)
	{
		if (UPCGBaseSubgraphNode* SubgraphNode = Cast<UPCGBaseSubgraphNode>(Node))
		{
			if (InGraph == SubgraphNode->GetSubgraph())
			{
				bContains = true;
				return false; // stop execution
			}
		}

		return true;
	};

	ForEachNodeRecursively(ContainsSelectedGraph);
	return bContains;
}

UPCGNode* UPCGGraph::FindNodeWithSettings(const UPCGSettingsInterface* InSettings, bool bRecursive) const
{
	UPCGNode* NodeFound = nullptr;

	auto FindNode = [&NodeFound, InSettings](UPCGNode* InNode)
	{
		if (InNode && InNode->GetSettingsInterface() == InSettings)
		{
			NodeFound = InNode;
			return false; // stop execution
		}
		else
		{
			return true;
		}
	};

	if (bRecursive)
	{
		ForEachNodeRecursively(FindNode);
	}
	else
	{
		ForEachNode(FindNode);
	}

	return NodeFound;
}

TArray<UPCGNode*> UPCGGraph::FindNodesWithSettings(const TSubclassOf<UPCGSettingsInterface> InSettingsClass, bool bRecursive) const
{
	TArray<UPCGNode*> Result;

	auto FindNode = [&Result, InSettingsClass](UPCGNode* InNode)
	{
		if (InNode && InNode->GetSettingsInterface() && InNode->GetSettingsInterface()->GetClass() == InSettingsClass)
		{
			Result.AddUnique(InNode);
		}

		// Always return true so we never stop execution
		return true;
	};

	if (bRecursive)
	{
		ForEachNodeRecursively(FindNode);
	}
	else
	{
		ForEachNode(FindNode);
	}

	return Result;
}

UPCGNode* UPCGGraph::FindNodeByTitleName(FName NodeTitle, bool bRecursive, const TSubclassOf<const UPCGSettings> OptionalClass) const
{
	UPCGNode* NodeFound = nullptr;

	auto FindNode = [&NodeFound, NodeTitle, OptionalClass](UPCGNode* InNode)
	{
		const UPCGSettings* Settings = InNode ? InNode->GetSettings() : nullptr;
		if (Settings && (!OptionalClass || Settings->IsA(OptionalClass)) && InNode->NodeTitle == NodeTitle)
		{
			NodeFound = InNode;
			return false; // stop execution
		}

		return true;
	};

	if (bRecursive)
	{
		ForEachNodeRecursively(FindNode);
	}
	else
	{
		ForEachNode(FindNode);
	}

	return NodeFound;
}

#if WITH_EDITOR

void UPCGGraph::SetupEditorData()
{
	if (HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad) == false)
	{
		if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
		{
			if (UserParameterHierarchyRoot == nullptr)
			{
				UserParameterHierarchyRoot = PCGEditorModule->CreatePropertyBagHierarchyRootObject(this);
			}
			if (UserParameterHierarchyRoot)
			{
				PCGEditorModule->ConnectPropertyBagHierarchyRootObjectModifyDelegate(this);
			}
		}
	}
}

FPCGAssetCachedPins UPCGGraph::BuildCachedPins() const
{
	FPCGAssetCachedPins Pins;
	Pins.bHasCachedPins = true;
	Pins.InputPins = DefaultInputPinProperties();
	Pins.OutputPins = DefaultOutputPinProperties();

	return Pins;
}

void UPCGGraph::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		PCGEditorModule->OnGraphPreSave(this, ObjectSaveContext);
	}

	// Cache pins for asset retrieval
	if(IsAsset())
	{
		CachedPins = BuildCachedPins();
	}

	// Prepare cook data
	if (ObjectSaveContext.IsCooking() && ShouldCookForPlatform(ObjectSaveContext.GetTargetPlatform()) && !IsEditorOnly())
	{
		FPCGGraphCompiler GraphCompiler(/*bIsCooking=*/true);

		// Compile graph for all grid sizes in preparation for cooking.
		if (IsHierarchicalGenerationEnabled())
		{
			bool bHasUnbounded = true;
			PCGHiGenGrid::FSizeArray GridSizes;
			GetGridSizes(GridSizes, bHasUnbounded);

			for (uint32 GridSize : GridSizes)
			{
				FPCGStackContext StackContext;
				GraphCompiler.GetCompiledTasks(this, GridSize, StackContext);
			}

			if (bHasUnbounded)
			{
				FPCGStackContext StackContext;
				GraphCompiler.GetCompiledTasks(this, PCGHiGenGrid::UnboundedGridSize(), StackContext);
			}
		}

		// Always cook unitialized grid tasks which are used if component is not partitioned.
		{
			FPCGStackContext StackContext;
			GraphCompiler.GetCompiledTasks(this, PCGHiGenGrid::UninitializedGridSize(), StackContext);
		}

		// Move compiled results into cooked results.
		FPCGGraphCompilerCache& Cache = GraphCompiler.GetCache();
		TMap<uint32, TArray<FPCGGraphTask>>* CompiledTasks = Cache.TopGraphToTaskMap.Find(this);
		TMap<uint32, FPCGStackContext>* CompiledStackContexts = Cache.TopGraphToStackContextMap.Find(this);
		TMap<uint32, TArray<TObjectPtr<UPCGComputeGraph>>>* CompiledComputeGraphs = Cache.TopGraphToComputeGraphMap.Find(this);

		if (CookedCompilationData != nullptr)
		{
			// Reset and reuse the instance for determinism during multi-platform cooks,
			// to avoid the sub-object name being different for each platform.
			CookedCompilationData->Tasks.Reset();
			CookedCompilationData->StackContexts.Reset();
			CookedCompilationData->ComputeGraphs.Reset();
		}
		else
		{
			CookedCompilationData = NewObject<UPCGGraphCompilationData>(this);
		}

		if (ensure(CompiledTasks))
		{
			CookedCompilationData->Tasks.Reserve(CompiledTasks->Num());

			for (TPair<uint32, TArray<FPCGGraphTask>>& Pair : *CompiledTasks)
			{
				for (FPCGGraphTask& GraphTask : Pair.Value)
				{
					GraphTask.PrepareForCook();
				}

				CookedCompilationData->Tasks.Emplace(Pair.Key, FPCGGraphTasks(std::move(Pair.Value)));
			}
		}

		if (ensure(CompiledStackContexts))
		{
			CookedCompilationData->StackContexts.Reserve(CompiledStackContexts->Num());

			for (TPair<uint32, FPCGStackContext>& Pair : *CompiledStackContexts)
			{
				CookedCompilationData->StackContexts.Emplace(Pair.Key, std::move(Pair.Value));
			}
		}

		// Note: We don't have an ensure on the CompiledComputeGraphs like the other compiled data since graphs that do not
		// produce compute graphs will never create an entry in this mapping.
		if (CompiledComputeGraphs)
		{
			CookedCompilationData->ComputeGraphs.Reserve(CompiledComputeGraphs->Num());

			for (TPair<uint32, TArray<TObjectPtr<UPCGComputeGraph>>>& Pair : *CompiledComputeGraphs)
			{
				CookedCompilationData->ComputeGraphs.Emplace(Pair.Key, FPCGComputeGraphs(std::move(Pair.Value)));
			}
		}
	}
}
#endif

#if WITH_EDITOR
void UPCGGraph::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UPCGPin::StaticClass()));
}
#endif

void UPCGGraph::BeginDestroy()
{
#if WITH_EDITOR
	// Nodes is an array of TObjectPtr so we need this trick to convert to a UPCGNode*.
	// We don't need to notify that nodes were removed when the graph dies.
	OnNodesRemoved(static_cast<TArray<UPCGNode*>>(MutableView(Nodes)), /*bNotify=*/false);

	if (OutputNode)
	{
		OutputNode->OnNodeChangedDelegate.RemoveAll(this);
	}

	if (InputNode)
	{
		InputNode->OnNodeChangedDelegate.RemoveAll(this);
	}

	// BeginDestroy can happen when the module is already trying to unload, so make sure we only notify if the module is actually loaded.
	if (FPCGModule::IsPCGModuleLoaded())
	{
		FPCGModule::GetPCGModuleChecked().NotifyGraphChanged(this, EPCGChangeType::Structural | EPCGChangeType::GenerationGrid);
	}

	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		if (UserParameterHierarchyRoot)
		{
			PCGEditorModule->GraphIsBeingDestroyed(this);
		}
	}

#endif

	Super::BeginDestroy();
}

void UPCGGraph::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UPCGGraph* This = CastChecked<UPCGGraph>(InThis);

	for (TPair<FComputeGraphInstanceKey, TArray<TSharedPtr<FComputeGraphInstance>>>& Entry : This->AllComputeGraphInstances)
	{
		for (TSharedPtr<FComputeGraphInstance> Instance : Entry.Value)
		{
			Collector.AddPropertyReferences(FComputeGraphInstance::StaticStruct(), Instance.Get());
		}
	}

#if WITH_EDITOR
	Collector.AddReferencedObject(This->PCGEditorGraph, This);
#endif

	Super::AddReferencedObjects(InThis, Collector);
}

bool UPCGGraph::ContainsEmbeddedSubgraph(UPCGGraph* InEmbeddedSubgraph) const
{
	return EmbeddedSubgraphs.Contains(InEmbeddedSubgraph);
}

void UPCGGraph::DuplicateEmbeddedSubgraphs(const TArray<UPCGGraph*>& InEmbeddedSubgraphs, TMap<UPCGGraph*, UPCGGraph*>& OutDuplicatedGraphs)
{
	check(OutDuplicatedGraphs.IsEmpty());

	// Find graphs to duplicate recursively
	TSet<UPCGGraph*> EmbeddedSubgraphsToCopy;
	for (UPCGGraph* EmbeddedSubgraphToCopy : InEmbeddedSubgraphs)
	{
		bool bAlreadySet = false;
		EmbeddedSubgraphsToCopy.Add(EmbeddedSubgraphToCopy, &bAlreadySet);
		if (!bAlreadySet)
		{
			EmbeddedSubgraphToCopy->ForEachNodeRecursively([&EmbeddedSubgraphsToCopy](UPCGNode* InNode)
			{
				if (UPCGBaseSubgraphSettings* RecursiveSubgraphSettings = Cast<UPCGBaseSubgraphSettings>(InNode->GetSettings()))
				{
					if (UPCGGraph* RecursiveSubgraph = RecursiveSubgraphSettings->GetSubgraph(); RecursiveSubgraph && RecursiveSubgraph->IsEmbeddedSubgraph())
					{
						EmbeddedSubgraphsToCopy.Add(RecursiveSubgraph);
					}
				}

				return true;
			});
		}
	}

	if (EmbeddedSubgraphsToCopy.IsEmpty())
	{
		return;
	}

	Modify();

	// Duplicate graphs
	for (UPCGGraph* EmbeddedSubgraphToCopy : EmbeddedSubgraphsToCopy)
	{
		check(EmbeddedSubgraphToCopy->IsEmbeddedSubgraph());

		UPCGGraph* DuplicatedEmbeddedSubgraph = DuplicateObject<UPCGGraph>(EmbeddedSubgraphToCopy, this);
		EmbeddedSubgraphs.Add(DuplicatedEmbeddedSubgraph);

		check(DuplicatedEmbeddedSubgraph->GetEmbeddedParentGraph() == this);

		OutDuplicatedGraphs.Add(EmbeddedSubgraphToCopy, DuplicatedEmbeddedSubgraph);
	}

	// Replace internal references
	for (auto It : OutDuplicatedGraphs)
	{
		FArchiveReplaceObjectRef<UPCGGraph>(It.Value, OutDuplicatedGraphs);
	}
}

EPCGHiGenGrid UPCGGraph::GetDefaultGrid() const 
{ 
	if (UPCGGraph* EmbeddedParentGraph = GetEmbeddedParentGraph())
	{
		return EmbeddedParentGraph->GetDefaultGrid();
	}

	ensure(IsHierarchicalGenerationEnabled()); 
	return HiGenGridSize; 
}

uint32 UPCGGraph::GetDefaultGridSize() const
{
	if (UPCGGraph* EmbeddedParentGraph = GetEmbeddedParentGraph())
	{
		return EmbeddedParentGraph->GetDefaultGridSize();
	}

	if (IsHierarchicalGenerationEnabled() && PCGHiGenGrid::IsValidGrid(HiGenGridSize))
	{
		return FMath::RoundToInt(PCGHiGenGrid::GridToGridSize(HiGenGridSize) * GetGridSizeMultiplier());
	}
	else
	{
		return PCGHiGenGrid::UnboundedGridSize();
	}
}

bool UPCGGraph::IsHierarchicalGenerationEnabled() const 
{ 
	if (UPCGGraph* EmbeddedParentGraph = GetEmbeddedParentGraph())
	{
		return EmbeddedParentGraph->IsHierarchicalGenerationEnabled();
	}

	return bUseHierarchicalGeneration; 
}

bool UPCGGraph::Use2DGrid() const 
{ 
	if (UPCGGraph* EmbeddedParentGraph = GetEmbeddedParentGraph())
	{
		return EmbeddedParentGraph->Use2DGrid();
	}

	return bUse2DGrid; 
}

TSharedPtr<FComputeGraphInstance> UPCGGraph::RetrieveComputeGraphInstanceFromPool(const FComputeGraphInstanceKey& InKey, bool& bOutNewInstance) const
{
	check(IsInGameThread());

	TSharedPtr<FComputeGraphInstance> Instance = nullptr;

	if (PCGGraph::CVarEnableComputeGraphInstancePooling.GetValueOnGameThread())
	{
		TArray<TSharedPtr<FComputeGraphInstance>>& Instances = AvailableComputeGraphInstances.FindOrAdd(InKey);

		if (!Instances.IsEmpty())
		{
			Instance = Instances.Pop(EAllowShrinking::No);
			bOutNewInstance = false;
		}
		else
		{
			// If we didn't find an instance, create a new one.
			Instance = MakeShared<FComputeGraphInstance>();
			bOutNewInstance = true;

			AllComputeGraphInstances.FindOrAdd(InKey).Add(Instance);
		}
	}
	else
	{
		Instance = MakeShared<FComputeGraphInstance>();
		bOutNewInstance = true;
	}
	
	return Instance;
}

void UPCGGraph::ReturnComputeGraphInstanceToPool(const FComputeGraphInstanceKey& InKey, TSharedPtr<FComputeGraphInstance> InInstance) const
{
	check(IsInGameThread());

	if (PCGGraph::CVarEnableComputeGraphInstancePooling.GetValueOnGameThread())
	{
		TArray<TSharedPtr<FComputeGraphInstance>>* AllInstances = AllComputeGraphInstances.Find(InKey);

		// An instance can only be considered valid if it still exists in the pool of all instances. If it's no longer in the pool of all instances, then it
		// must have been flushed and should now be considered stale.
		const bool bValidInstance = AllInstances && AllInstances->Contains(InInstance);

		if (bValidInstance)
		{
			TArray<TSharedPtr<FComputeGraphInstance>>& Instances = AvailableComputeGraphInstances.FindOrAdd(InKey);
			Instances.Add(InInstance);
		}
	}
}

UPCGNode* UPCGGraph::AddNodeOfType(TSubclassOf<class UPCGSettings> InSettingsClass, UPCGSettings*& OutDefaultNodeSettings)
{
	UPCGSettings* Settings = NewObject<UPCGSettings>(GetTransientPackage(), InSettingsClass, NAME_None, RF_Transactional);

	if (!Settings)
	{
		return nullptr;
	}

	UPCGNode* Node = AddNode(Settings);

	if (Node)
	{
		Settings->Rename(nullptr, Node, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
	}

	OutDefaultNodeSettings = Settings;
	return Node;
}

UPCGNode* UPCGGraph::AddNode(UPCGSettingsInterface* InSettingsInterface)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGGraph::AddNode);
	if (!InSettingsInterface || !InSettingsInterface->GetSettings())
	{
		return nullptr;
	}

	UPCGNode* Node = InSettingsInterface->GetSettings()->CreateNode();

	if (Node)
	{
		Node->SetFlags(RF_Transactional);

		Modify();

		// Assign settings to node & reparent
		Node->SetSettingsInterface(InSettingsInterface);

		// Reparent node to this graph
		Node->Rename(nullptr, this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

#if WITH_EDITOR
		const FName DefaultNodeName = InSettingsInterface->GetSettings()->GetDefaultNodeName();
		if (DefaultNodeName != NAME_None)
		{
			const FName NodeName = MakeUniqueObjectName(this, UPCGNode::StaticClass(), DefaultNodeName);
			// Flags added because default flags favor tick/interactive, not load-time renaming.
			Node->Rename(*NodeName.ToString(), nullptr, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}

		// Give the chance to the graph/settings to update the node title.
		Node->SetNodeTitle(NAME_None);
#endif

		Nodes.Add(Node);
		OnNodeAdded(Node);
	}

	return Node;
}

UPCGNode* UPCGGraph::AddNodeInstance(UPCGSettings* InSettings)
{
	if (!InSettings)
	{
		return nullptr;
	}

	UPCGSettingsInstance* SettingsInstance = NewObject<UPCGSettingsInstance>();
	SettingsInstance->SetSettings(InSettings);

	UPCGNode* Node = AddNode(SettingsInstance);

	if (Node)
	{
		SettingsInstance->Rename(nullptr, Node, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		SettingsInstance->SetFlags(RF_Transactional);
	}

	return Node;
}

UPCGNode* UPCGGraph::AddNodeCopy(const UPCGSettings* InSettings, UPCGSettings*& DefaultNodeSettings)
{
	if (!InSettings)
	{
		return nullptr;
	}

	UPCGSettings* SettingsCopy = DuplicateObject(InSettings, nullptr);
	UPCGNode* NewNode = AddNode(SettingsCopy);

	if (SettingsCopy)
	{
		SettingsCopy->Rename(nullptr, NewNode, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
	}

	DefaultNodeSettings = SettingsCopy;
	return NewNode;
}

void UPCGGraph::OnNodeAdded(UPCGNode* InNode, bool bNotify)
{
	OnNodesAdded(MakeArrayView<UPCGNode*>(&InNode, 1), bNotify);
}

void UPCGGraph::OnNodesAdded(TArrayView<UPCGNode*> InNodes, bool bNotify)
{
#if WITH_EDITOR
	EPCGChangeType ChangeType = EPCGChangeType::Structural;

	for (UPCGNode* Node : InNodes)
	{
		if (Node)
		{
			Node->OnNodeChangedDelegate.AddUObject(this, &UPCGGraph::OnNodeChanged);

			if (Node->GetSettings() && Node->GetSettings()->IsA<UPCGHiGenGridSizeSettings>())
			{
				ChangeType |= EPCGChangeType::GenerationGrid;
			}
		}
	}

	if (bNotify)
	{
		NotifyGraphStructureChanged(ChangeType);
	}
#endif
}

void UPCGGraph::OnNodeRemoved(UPCGNode* InNode, bool bNotify)
{
	OnNodesRemoved(MakeArrayView<UPCGNode*>(&InNode, 1), bNotify);
}

void UPCGGraph::OnNodesRemoved(TArrayView<UPCGNode*> InNodes, bool bNotify)
{
#if WITH_EDITOR
	bool bAnyGridSizeNodes = false;

	for (UPCGNode* Node : InNodes)
	{
		if (Node)
		{
			Node->OnNodeChangedDelegate.RemoveAll(this);

			bAnyGridSizeNodes |= !!Cast<UPCGHiGenGridSizeSettings>(Node->GetSettings());
		}
	}

	if (bNotify)
	{
		NotifyGraphStructureChanged(bAnyGridSizeNodes ? (EPCGChangeType::Structural | EPCGChangeType::GenerationGrid) : EPCGChangeType::Structural);
	}
#endif
}

UPCGNode* UPCGGraph::AddEdge(UPCGNode* From, const FName& FromPinLabel, UPCGNode* To, const FName& ToPinLabel)
{
	AddLabeledEdge(From, FromPinLabel, To, ToPinLabel);
	return To;
}

bool UPCGGraph::AddLabeledEdge(UPCGNode* From, const FName& FromPinLabel, UPCGNode* To, const FName& ToPinLabel)
{
	if (!From || !To)
	{
		UE_LOGF(LogPCG, Error, "Invalid edge nodes");
		return false;
	}

	UPCGPin* FromPin = From->GetOutputPin(FromPinLabel);

	if (!FromPin)
	{
		UE_LOGF(LogPCG, Error, "From node %ls does not have the %ls label", *From->GetName(), *FromPinLabel.ToString());
		return false;
	}

	UPCGPin* ToPin = To->GetInputPin(ToPinLabel);

	if (!ToPin)
	{
		UE_LOGF(LogPCG, Error, "To node %ls does not have the %ls label", *To->GetName(), *ToPinLabel.ToString());
		return false;
	}

#if WITH_EDITOR
	DisableNotificationsForEditor();
#endif

	TSet<UPCGNode*> TouchedNodes;

	// Create edge
	FromPin->AddEdgeTo(ToPin, &TouchedNodes);

	bool bToPinBrokeOtherEdges = false;

	// Add an edge to a pin that doesn't allow multiple connections requires to do some cleanup
	if (!ToPin->AllowsMultipleConnections())
	{
		bToPinBrokeOtherEdges = ToPin->BreakAllIncompatibleEdges(&TouchedNodes);
	}

	const EPCGChangeType ChangeType = PCGGraphUtils::NotifyTouchedNodes(TouchedNodes, EPCGChangeType::Structural) | EPCGChangeType::Edge;

#if WITH_EDITOR
	// After all nodes are notified, re-enable graph notifications and send graph change notification.
	EnableNotificationsForEditor();

	NotifyGraphStructureChanged(ChangeType);
#endif

	return bToPinBrokeOtherEdges;
}

TArray<FPCGPinProperties> UPCGGraph::DefaultInputPinProperties() const
{
	return InputNode ? CastChecked<UPCGGraphInputOutputSettings>(InputNode->GetSettings())->OutputPinProperties() : TArray<FPCGPinProperties>();
}

TArray<FPCGPinProperties> UPCGGraph::DefaultOutputPinProperties() const
{
	return OutputNode ? CastChecked<UPCGGraphInputOutputSettings>(OutputNode->GetSettings())->InputPinProperties() : TArray<FPCGPinProperties>();
}

TArray<UPCGEdge*> UPCGGraph::GetAllEdges() const
{
	TArray<UPCGEdge*> AllEdges;

	auto GatherInputEdgesFromNode = [&AllEdges](const UPCGNode* Node) 
	{
		for (const UPCGPin* InPin : Node->InputPins)
		{
			AllEdges.Append(InPin->Edges);
		}
	};

	// Since we gather edge from their Output side (The input side of the nodes) we have to skip InputNode
	GatherInputEdgesFromNode(OutputNode);
	for (const UPCGNode* Node : Nodes)
	{
		GatherInputEdgesFromNode(Node);
	}
	return AllEdges;
}

TObjectPtr<UPCGNode> UPCGGraph::ReconstructNewNode(const UPCGNode* InNode)
{
	UPCGSettings* NewSettings = nullptr;
	TObjectPtr<UPCGNode> NewNode = AddNodeCopy(InNode->GetSettings(), NewSettings);

#if WITH_EDITOR
	InNode->TransferEditorProperties(NewNode);
#endif // WITH_EDITOR

	return NewNode;
}

bool UPCGGraph::Contains(UPCGNode* Node) const
{
	return Node == InputNode || Node == OutputNode || Nodes.Contains(Node);
}

void UPCGGraph::AddNode(UPCGNode* InNode)
{
	AddNodes_Internal(MakeArrayView<UPCGNode*>(&InNode, 1));
}

void UPCGGraph::AddNodes(TArray<UPCGNode*>& InNodes)
{
	AddNodes_Internal(InNodes);
}

void UPCGGraph::AddNodes_Internal(TArrayView<UPCGNode*> InNodes)
{
	if (InNodes.IsEmpty())
	{
		return;
	}

	Modify();

	for (UPCGNode* Node : InNodes)
	{
		check(Node);
		Node->Rename(nullptr, this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

#if WITH_EDITOR
		const FName DefaultNodeName = Node->GetSettings()->GetDefaultNodeName();
		if (DefaultNodeName != NAME_None)
		{
			FName NodeName = MakeUniqueObjectName(this, UPCGNode::StaticClass(), DefaultNodeName);
			Node->Rename(*NodeName.ToString(), nullptr, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
#endif

		Nodes.Add(Node);
	}

	OnNodesAdded(InNodes);
}

void UPCGGraph::RemoveNode(UPCGNode* InNode)
{
	RemoveNodes_Internal(MakeArrayView<UPCGNode*>(&InNode, 1));
}

void UPCGGraph::RemoveNodes(TArray<UPCGNode*>& InNodes)
{
	RemoveNodes_Internal(InNodes);
}

void UPCGGraph::RemoveNodes_Internal(TArrayView<UPCGNode*> InNodes)
{
	if (InNodes.IsEmpty())
	{
		return;
	}

	Modify();

#if WITH_EDITOR
	DisableNotificationsForEditor();
#endif

	TSet<UPCGNode*> TouchedNodes;

	for (UPCGNode* Node : InNodes)
	{
		check(Node);

		for (UPCGPin* InputPin : Node->InputPins)
		{
			InputPin->BreakAllEdges(&TouchedNodes);
		}

		for (UPCGPin* OutputPin : Node->OutputPins)
		{
			OutputPin->BreakAllEdges(&TouchedNodes);
		}

		// We're about to remove InNode, so don't bother triggering updates
		TouchedNodes.Remove(Node);

		// Add the node to the transaction, to make sure we reconnect everything correctly on Undo/Redo
		Node->Modify();

		// Remove the node from the graph inners.
		Node->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);

		Nodes.Remove(Node);
	}

#if WITH_EDITOR
	EnableNotificationsForEditor();
#endif

	PCGGraphUtils::NotifyTouchedNodes(TouchedNodes, EPCGChangeType::Structural);

	OnNodesRemoved(InNodes);
}

bool UPCGGraph::RemoveEdge(UPCGNode* From, const FName& FromLabel, UPCGNode* To, const FName& ToLabel)
{
	if (!From || !To)
	{
		UE_LOGF(LogPCG, Error, "Invalid from/to node in RemoveEdge");
		return false;
	}

#if WITH_EDITOR
	DisableNotificationsForEditor();
#endif

	UPCGPin* OutPin = From->GetOutputPin(FromLabel);
	UPCGPin* InPin = To->GetInputPin(ToLabel);

	TSet<UPCGNode*> TouchedNodes;
	if (OutPin)
	{
		OutPin->BreakEdgeTo(InPin, &TouchedNodes);
	}

	const EPCGChangeType ChangeType = PCGGraphUtils::NotifyTouchedNodes(TouchedNodes, EPCGChangeType::Structural) | EPCGChangeType::Edge;

#if WITH_EDITOR
	// After all nodes are notified, re-enable graph notifications and send graph change notification.
	EnableNotificationsForEditor();

	if (TouchedNodes.Num() > 0)
	{
		NotifyGraphStructureChanged(ChangeType);
	}
#endif

	return TouchedNodes.Num() > 0;
}

bool UPCGGraph::ForEachNode(TFunctionRef<bool(UPCGNode*)> Action) const
{
	if (!Action(InputNode) ||
		!Action(OutputNode))
	{
		return false;
	}

	for (UPCGNode* Node : Nodes)
	{
		if (!Action(Node))
		{
			return false;
		}
	}

	return true;
}

bool UPCGGraph::ForEachNodeRecursively(TFunctionRef<bool(UPCGNode*)> Action) const
{
	TSet<const UPCGGraph*> VisitedGraphs;
	return ForEachNodeRecursively_Internal(Action, VisitedGraphs);
}

bool UPCGGraph::ForEachNodeRecursively_Internal(TFunctionRef<bool(UPCGNode*)> Action, TSet<const UPCGGraph*>& VisitedGraphs) const
{
	check(!VisitedGraphs.Contains(this));
	VisitedGraphs.Add(this);

	auto RecursiveCall = [&Action, &VisitedGraphs](UPCGNode* Node) -> bool
	{
		if (!Action(Node))
		{
			return false;
		}

		if (UPCGBaseSubgraphNode* SubgraphNode = Cast<UPCGBaseSubgraphNode>(Node))
		{
			if (const UPCGGraph* Subgraph = SubgraphNode->GetSubgraph())
			{
				if (!VisitedGraphs.Contains(Subgraph))
				{
					return Subgraph->ForEachNodeRecursively_Internal(Action, VisitedGraphs);
				}
			}
		}

		return true;
	};

	return ForEachNode(RecursiveCall);
}

bool UPCGGraph::RemoveInboundEdges(UPCGNode* InNode, const FName& InboundLabel)
{
	check(InNode);
	TSet<UPCGNode*> TouchedNodes;

#if WITH_EDITOR
	DisableNotificationsForEditor();
#endif

	if (UPCGPin* InputPin = InNode->GetInputPin(InboundLabel))
	{
		InputPin->BreakAllEdges(&TouchedNodes);
	}

	const EPCGChangeType ChangeType = PCGGraphUtils::NotifyTouchedNodes(TouchedNodes, EPCGChangeType::Structural);

#if WITH_EDITOR
	// After all nodes are notified, re-enable graph notifications and send graph change notification.
	EnableNotificationsForEditor();

	if (TouchedNodes.Num() > 0)
	{
		NotifyGraphStructureChanged(ChangeType);
	}
#endif

	return TouchedNodes.Num() > 0;
}

bool UPCGGraph::RemoveOutboundEdges(UPCGNode* InNode, const FName& OutboundLabel)
{
	check(InNode);
	// Make a list of downstream nodes which may need pin updates when the edges change
	TSet<UPCGNode*> TouchedNodes;

#if WITH_EDITOR
	DisableNotificationsForEditor();
#endif

	if (UPCGPin* OutputPin = InNode->GetOutputPin(OutboundLabel))
	{
		OutputPin->BreakAllEdges(&TouchedNodes);
	}

	const EPCGChangeType ChangeType = PCGGraphUtils::NotifyTouchedNodes(TouchedNodes, EPCGChangeType::Structural);

#if WITH_EDITOR
	// After all nodes are notified, re-enable graph notifications and send graph change notification.
	EnableNotificationsForEditor();

	if (TouchedNodes.Num() > 0)
	{
		NotifyGraphStructureChanged(ChangeType);
	}
#endif

	return TouchedNodes.Num() > 0;
}

#if WITH_EDITOR
void UPCGGraph::ForceNotificationForEditor(EPCGChangeType ChangeType)
{
	// Queue up the delayed change
	NotifyGraphChanged(ChangeType);

	if (bUserPausedNotificationsInGraphEditor)
	{
		EnableNotificationsForEditor();
		DisableNotificationsForEditor();
	}
}

void UPCGGraph::PreNodeUndo(UPCGNode* InPCGNode)
{
	if (InPCGNode)
	{
		InPCGNode->OnNodeChangedDelegate.RemoveAll(this);
	}
}

void UPCGGraph::PostNodeUndo(UPCGNode* InPCGNode)
{
	if (InPCGNode)
	{
		InPCGNode->OnNodeChangedDelegate.AddUObject(this, &UPCGGraph::OnNodeChanged);
	}
}
#endif

void UPCGGraph::GetGridSizes(PCGHiGenGrid::FSizeArray& OutGridSizes, bool& bOutHasUnbounded) const
{
	PCG::TScopeLock Lock(CachedGridInfoLock);

	if (!CachedGridInfo)
	{
		CacheGridSizesInternalNoLock();
	}

	OutGridSizes = CachedGridInfo->GridSizes;
	bOutHasUnbounded = CachedGridInfo->bHasUnbounded;
}

void UPCGGraph::CacheGridSizesInternalNoLock() const
{
	FGridInfo GridInfo;

	GridInfo.bHasUnbounded = HiGenGridSize == EPCGHiGenGrid::Unbounded;

	const uint32 GraphDefaultGridSize = GetDefaultGridSize();
	if (!IsHierarchicalGenerationEnabled())
	{
		if (PCGHiGenGrid::IsValidGridSize(GetDefaultGridSize()))
		{
			GridInfo.GridSizes.AddUnique(GraphDefaultGridSize);
		}
	}
	else
	{
		bool bHasUninitialized = false;
		for (const UPCGNode* Node : Nodes)
		{
			const uint32 GridSize = GetNodeGenerationGridSize(Node, GraphDefaultGridSize);
			if (PCGHiGenGrid::IsValidGridSize(GridSize))
			{
				GridInfo.GridSizes.AddUnique(GridSize);
			}
			else if (GridSize == PCGHiGenGrid::UnboundedGridSize())
			{
				GridInfo.bHasUnbounded = true;
			}
			else if (GridSize == PCGHiGenGrid::UninitializedGridSize())
			{
				// Outside nodes will not have a concrete grid set
				bHasUninitialized = true;
			}
		}

		if (bHasUninitialized)
		{
			// Nodes outside grid ranges will execute at graph default
			GridInfo.GridSizes.AddUnique(GraphDefaultGridSize);
		}

		// Descending
		GridInfo.GridSizes.Sort([](const uint32& A, const uint32& B) { return A > B; });
	}

	CachedGridInfo = MoveTemp(GridInfo);
}

void UPCGGraph::GetParentGridSizes(const uint32 InChildGridSize, PCGHiGenGrid::FSizeArray& OutParentGridSizes) const
{
	if (!PCGHiGenGrid::IsValidGridSize(InChildGridSize))
	{
		// Grid size is 0 or Unbounded or some other invalid value, and will not have any parent grids.
		return;
	}

	PCG::TScopeLock Lock(CachedGridInfoLock);
	
	if (const PCGHiGenGrid::FSizeArray* FoundGridSizeInfo = ChildGridSizeToParentGridSizes.Find(InChildGridSize))
	{
		OutParentGridSizes = *FoundGridSizeInfo;
		return;
	}

	// No higen means no parent dependencies.
	if (IsHierarchicalGenerationEnabled())
	{
		// Use in parent grid calculations to cache results. Note this map is specific to InChildGridSize 
		const uint32 DefaultGridSize = GetDefaultGridSize();

		for (const UPCGNode* Node : Nodes)
		{
			if (Node)
			{
				PCGHiGenGrid::FSizeArray NodeAllGridSizes = CalculateNodeGridSizesRecursiveNoLock(Node, DefaultGridSize);

				if (!NodeAllGridSizes.IsEmpty() && NodeAllGridSizes.Last() == InChildGridSize)
				{
					for (uint32 GridSize : NodeAllGridSizes)
					{
						OutParentGridSizes.AddUnique(GridSize);
					}
				}
			}
		}

		OutParentGridSizes.Remove(InChildGridSize);

		// Always output in descending order.
		OutParentGridSizes.Sort([](const uint32& A, const uint32& B) { return A > B; });
	}

	ChildGridSizeToParentGridSizes.Add(InChildGridSize, OutParentGridSizes);
}

double UPCGGraph::GetGridGenerationRadiusFromGrid(uint32 Grid) const
{
	const EPCGHiGenGrid UnscaledGrid = PCGHiGenGrid::ScaledGridSizeToGrid(Grid, GetGridSizeMultiplier());
	return GenerationRadii.GetGenerationRadiusFromGrid(UnscaledGrid);
}

double UPCGGraph::GetGridCleanupRadiusFromGrid(uint32 Grid) const
{
	const EPCGHiGenGrid UnscaledGrid = PCGHiGenGrid::ScaledGridSizeToGrid(Grid, GetGridSizeMultiplier());
	return GenerationRadii.GetCleanupRadiusFromGrid(UnscaledGrid);
}

bool UPCGGraph::UseActorComponentlessGeneration() const
{
	return bUseActorComponentlessGeneration && PCGGraph::CVarEnableActorComponentlessGeneration.GetValueOnAnyThread();
}

#if WITH_EDITOR
void UPCGGraph::DisableNotificationsForEditor()
{
	check(GraphChangeNotificationsDisableCounter >= 0);
	++GraphChangeNotificationsDisableCounter;
}

void UPCGGraph::EnableNotificationsForEditor()
{
	check(GraphChangeNotificationsDisableCounter > 0);
	--GraphChangeNotificationsDisableCounter;

	if (GraphChangeNotificationsDisableCounter == 0 && bDelayedChangeNotification)
	{
		NotifyGraphChanged(DelayedChangeType);
		bDelayedChangeNotification = false;
		DelayedChangeType = EPCGChangeType::None;
	}
}

void UPCGGraph::ToggleUserPausedNotificationsForEditor()
{
	if (bUserPausedNotificationsInGraphEditor)
	{
		EnableNotificationsForEditor();
	}
	else
	{
		DisableNotificationsForEditor();
	}

	bUserPausedNotificationsInGraphEditor = !bUserPausedNotificationsInGraphEditor;
}

void UPCGGraph::SetExtraEditorNodes(const TArray<TObjectPtr<const UObject>>& InNodes)
{
	ExtraEditorNodes.Empty();

	for (const UObject* Node : InNodes)
	{
		ExtraEditorNodes.Add(DuplicateObject(Node, this));
	}
}

void UPCGGraph::RemoveCommentNode(const FGuid& InNodeGUID)
{
	CommentNodes.RemoveAll([InNodeGUID](const FPCGGraphCommentNodeData& NodeData) { return InNodeGUID == NodeData.GUID; });
}

void UPCGGraph::RemoveExtraEditorNode(const UObject* InNode)
{
	ExtraEditorNodes.Remove(const_cast<UObject*>(InNode));
}

void UPCGGraph::EnableInspection(const FPCGStack& InInspectedStack)
{
	PCG::TUniqueScopeLock WriteLock(InspectionStateLock);
	bIsInspecting = true;
	InspectedStack = InInspectedStack;
}

void UPCGGraph::DisableInspection()
{
	PCG::TUniqueScopeLock WriteLock(InspectionStateLock);
	bIsInspecting = false;
	InspectedStack = FPCGStack();
}

FPCGStack UPCGGraph::GetInspectedStack() const
{
	PCG::TSharedScopeLock ReadLock(InspectionStateLock);
	return InspectedStack;
}

bool UPCGGraph::IsSameInspectedStack(const FPCGStack& InOtherStack) const
{
	PCG::TSharedScopeLock ReadLock(InspectionStateLock);
	return bIsInspecting && InspectedStack == InOtherStack;
}

bool UPCGGraph::PrimeGraphCompilationCache()
{
	UPCGSubsystem* Subsystem = UPCGSubsystem::GetActiveEditorInstance();
	FPCGGraphCompiler* GraphCompiler = Subsystem ? Subsystem->GetGraphCompiler() : nullptr;

	if (!GraphCompiler)
	{
		return false;
	}

	FPCGStackContext StackContext;
	GraphCompiler->GetCompiledTasks(this, PCGHiGenGrid::UninitializedGridSize(), StackContext, /*bIsTopGraph=*/true);

	UE_LOGF(LogPCG, Verbose, "UPCGGraph::PrimeGraphCompilationCache '%ls' %u", *this->GetName(), PCGHiGenGrid::UninitializedGridSize());

	return true;
}

bool UPCGGraph::Recompile()
{
	UPCGSubsystem* Subsystem = UPCGSubsystem::GetActiveEditorInstance();
	FPCGGraphCompiler* GraphCompiler = Subsystem ? Subsystem->GetGraphCompiler() : nullptr;

	if (!GraphCompiler)
	{
		return true;
	}

	const bool bChanged = GraphCompiler->Recompile(this, PCGHiGenGrid::UninitializedGridSize(), /*bIsTopGraph=*/true);

	UE_LOGF(LogPCG, Verbose, "UPCGGraph::Recompile '%ls' grid: %u changed: %d", *this->GetName(), PCGHiGenGrid::UninitializedGridSize(), bChanged ? 1 : 0);

	return bChanged;
}

void UPCGGraph::OnPCGQualityLevelChanged()
{
	// No need to broadcast a refresh here since relevant execution sources will be refreshed from UPCGSubsystem::OnPCGQualityLevelChanged()

	// If we have a quality branch/select node we need to broadcast a cosmetic change to update the pin visualization.
	for (UPCGNode* Node : Nodes)
	{
		if (!Node)
		{
			continue;
		}

		const UPCGSettings* Settings = Node->GetSettings();

		if (Settings && (Settings->IsA<UPCGQualityBranchSettings>() || Settings->IsA<UPCGQualitySelectSettings>()))
		{
			Node->OnNodeChangedDelegate.Broadcast(Node, EPCGChangeType::Cosmetic);
		}
	}
}

const TArray<TSubclassOf<UObject>>& UPCGGraph::GetChangeTrackingIgnoredClasses() const
{
	if (UPCGGraph* EmbeddedParentGraph = GetEmbeddedParentGraph())
	{
		return EmbeddedParentGraph->GetChangeTrackingIgnoredClasses();
	}

	return ChangeTrackingIgnoredClasses;
}

FPCGSelectionKeyToSettingsMap UPCGGraph::GetTrackedSelectionKeysToSettings() const
{
	FPCGSelectionKeyToSettingsMap TagsToSettings;
	TArray<TObjectPtr<const UPCGGraph>> VisitedGraphs;

	GetTrackedSelectionKeysToSettings(TagsToSettings, VisitedGraphs);
	return TagsToSettings;
}

void UPCGGraph::GetTrackedSelectionKeysToSettings(FPCGSelectionKeyToSettingsMap& OutTagsToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGGraph::GetTrackedSelectionKeysToSettings);
	
	if (OutVisitedGraphs.Contains(this))
	{
		return;
	}

	OutVisitedGraphs.Emplace(this);

	for (const UPCGNode* Node : Nodes)
	{
		const UPCGSettings* Settings = Node ? Node->GetSettings() : nullptr;
		// Don't track for disabled nodes.
		if (Settings && Settings->bEnabled)
		{
			Settings->GetStaticTrackedKeys(OutTagsToSettings, OutVisitedGraphs);
		}
	}

	// Make sure that all Self/Original keys are marked as not-cull, since the component will always intersect with its owner/original
	// We need to loop because we can have multiple keys that have Filter to self or original.
	for (auto& It : OutTagsToSettings)
	{
		const FPCGSelectionKey& Key = It.Key;
		TArray<FPCGSettingsAndCulling>& SettingsAndCullingArray = It.Value;
		if (Key.ActorFilter == EPCGActorFilter::Self || Key.ActorFilter == EPCGActorFilter::Original)
		{
			for (FPCGSettingsAndCulling& SettingsAndCullingPair : SettingsAndCullingArray)
			{
				SettingsAndCullingPair.Value = false;
			}
		}
	}
}

bool UPCGGraph::UpdateTrackedSelectionKeys(FPCGSelectionKeyToSettingsMap& InExistingKeys, FPCGSelectionKeyToSettingsMap& InOutUpdatedKeys, TArray<FPCGSelectionKey>* OptionalChangedKeys) const
{
	int32 FoundKeys = 0;

	InOutUpdatedKeys.Append(GetTrackedSelectionKeysToSettings());

	// A tag should be culled, if only all the settings that track this tag should cull.
	// Note that is only impact the fact that we track (or not) this tag.
	// If a setting is marked as "should cull", it will only be dirtied (at least by default), if the actor with the
	// given tag intersect with the component.
	for (const TPair<FPCGSelectionKey, TArray<FPCGSettingsAndCulling>>& It : InOutUpdatedKeys)
	{
		const FPCGSelectionKey& Key = It.Key;

		// Should cull only if all the settings requires a cull.
		const bool bShouldCull = PCGSettings::IsKeyCulled(It.Value);
		const TArray<FPCGSettingsAndCulling>* OldSettingsAndCulling = InExistingKeys.Find(It.Key);
		const bool OldCulling = OldSettingsAndCulling && PCGSettings::IsKeyCulled(*OldSettingsAndCulling);
		const bool bNewKeyOrCullChanged = !OldSettingsAndCulling || (OldCulling != bShouldCull);

		InExistingKeys.Remove(Key);

		if (!bNewKeyOrCullChanged)
		{
			++FoundKeys;
		}
		else if (OptionalChangedKeys)
		{
			OptionalChangedKeys->Add(Key);
		}
	}

	// At the end, we also have keys that were tracked but no more, so add them at the list of tracked keys
	if (OptionalChangedKeys)
	{
		OptionalChangedKeys->Reserve(OptionalChangedKeys->Num() + InExistingKeys.Num());

		for (const TPair<FPCGSelectionKey, TArray<FPCGSettingsAndCulling>>& It : InExistingKeys)
		{
			OptionalChangedKeys->Add(It.Key);
		}
	}
	
	return InOutUpdatedKeys.Num() != FoundKeys || !InExistingKeys.IsEmpty();
}

void UPCGGraph::NotifyGraphStructureChanged(EPCGChangeType ChangeType, bool bForce)
{
	bool bExecutionAffected = true;

	if (PCGGraph::CVarRecompileGraphOnChange.GetValueOnAnyThread())
	{
		// If settings were not changed, we can gate the change notification based on whether compiled graph output changed. This compilation check
		// does not support settings changes.
		if (!bForce && !(ChangeType & EPCGChangeType::Settings))
		{
			bExecutionAffected = Recompile();
		}

		if (!bExecutionAffected)
		{
			// Remove a range of change types related to graph editing which may prevent unnecessary execution. Note settings changes always recompile (checked above).
			ChangeType &= ~(EPCGChangeType::Structural | EPCGChangeType::GenerationGrid | EPCGChangeType::Node | EPCGChangeType::Edge);

			// Positively flagging as cosmetic is required because downstream things specifically test for this currently.
			ChangeType |= EPCGChangeType::Cosmetic;
		}
	}

	NotifyGraphChanged(ChangeType);
}

void UPCGGraph::NotifyGraphChanged(EPCGChangeType ChangeType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGGraph::NotifyGraphChanged);

	const bool bNonCosmeticChange = (ChangeType & ~(EPCGChangeType::Cosmetic | EPCGChangeType::GraphCustomization)) != EPCGChangeType::None;

	if (bNonCosmeticChange)
	{
		// Graph settings, nodes, graph structure can all change the higen grid sizes.
		{
			PCG::TUniqueScopeLock WriteLock(NodeToGridSizeLock);
			NodeToGridSize.Reset();
			NodeToAllGridSizes.Reset();
		}

		{
			PCG::TScopeLock Lock(CachedGridInfoLock);
			CachedGridInfo.Reset();
			ChildGridSizeToParentGridSizes.Reset();
		}

		// Any non-trivial change to graph may change compute graphs.
		check(IsInGameThread());
		AvailableComputeGraphInstances.Empty(AvailableComputeGraphInstances.Num());
		AllComputeGraphInstances.Empty(AllComputeGraphInstances.Num());
	}

	if (GraphChangeNotificationsDisableCounter > 0)
	{
		bDelayedChangeNotification = true;
		DelayedChangeType |= ChangeType;
		return;
	}

	// Skip recursive cases which can happen either through direct recursivity (A -> A) or indirectly (A -> B -> A)
	if (bIsNotifying)
	{
		return;
	}

	bIsNotifying = true;

	// Notify the subsystem/compiler cache before so it gets recompiled properly
	const bool bNotifySubsystem = ((ChangeType & (EPCGChangeType::Structural | EPCGChangeType::Edge)) != EPCGChangeType::None);
	if (bNotifySubsystem)
	{
		FPCGModule::GetPCGModuleChecked().NotifyGraphChanged(this, ChangeType);
	}

	if (bNonCosmeticChange)
	{
		// Also notify other systems that this graph changed, only if the owner is not a PCG Component nor PCG Subgraph.
		// They already have their own system to trigger a refresh.
		const UObject* Outer = GetOuter();
		if (!Outer || !(Outer->IsA<UPCGComponent>() || Outer->IsA<UPCGSubgraphSettings>()))
		{
			FPropertyChangedEvent EmptyEvent{ nullptr };
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, EmptyEvent);
		}
	}

	// Notify Outer Graph if we are a embedded subgraph
	if (UPCGGraph* Parent = GetEmbeddedParentGraph())
	{
		Parent->NotifyGraphChanged(ChangeType);
	}

	OnGraphChangedDelegate.Broadcast(this, ChangeType);

	bIsNotifying = false;
}

void UPCGGraph::NotifyGraphParametersChanged(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName)
{
	if (bIsNotifying)
	{
		return;
	}

	bIsNotifying = true;
	OnGraphParametersChangedDelegate.Broadcast(this, InChangeType, InChangedPropertyName);
	bIsNotifying = false;

	NotifyGraphChanged(GetChangeTypeForGraphParameterChange(InChangeType, InChangedPropertyName));
}

void UPCGGraph::OnNodeChanged(UPCGNode* InNode, EPCGChangeType ChangeType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGGraph::OnNodeChanged);

	if (!!(ChangeType & EPCGChangeType::Structural))
	{
		// Update node to grid size map for grid size changes.
		if (Cast<UPCGHiGenGridSizeSettings>(InNode->GetSettings()))
		{
			ChangeType |= EPCGChangeType::GenerationGrid;

			PCG::TUniqueScopeLock WriteLock(NodeToGridSizeLock);
			NodeToGridSize.Reset();
			NodeToAllGridSizes.Reset();
			ChildGridSizeToParentGridSizes.Reset();
		}
	}

	if ((ChangeType & ~EPCGChangeType::Cosmetic) != EPCGChangeType::None)
	{
		NotifyGraphStructureChanged(ChangeType);
	}
}

void UPCGGraph::OnUserParameterHierarchyModified()
{
	OnGraphParametersChanged(EPCGGraphParameterEvent::CategoryChanged, GET_MEMBER_NAME_CHECKED(UPCGGraph, UserParameters));
}

void UPCGGraph::PreEditChange(FProperty* InProperty)
{
	Super::PreEditChange(InProperty);

	if (!InProperty)
	{
		return;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGGraph, UserParameters))
	{
		// We need to keep track of the previous property bag, to detect if a property was added/removed/renamed/moved/modified...
		PreviousPropertyBag = UserParameters.GetPropertyBagStruct();
	}
}

void UPCGGraph::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (!PropertyChangedEvent.Property)
	{
		return;
	}
	
	const FProperty* EditedProperty = PropertyChangedEvent.Property;
	const FName EditedPropertyName = EditedProperty->GetFName();
	
	const FEditPropertyChain::TDoubleLinkedListNode* HeadPropertyNode = PropertyChangedEvent.PropertyChain.GetHead();
	const FProperty* HeadProperty = HeadPropertyNode ? HeadPropertyNode->GetValue() : nullptr;

	if (!HeadProperty)
	{
		return;
	}

	const FName HeadPropertyName = HeadProperty->GetFName();
	
	bool bHiGenSettingsChanged = false;

	if (HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, bLandscapeUsesMetadata) ||
		HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, bHasDefaultConstructedInputs))
	{
		NotifyGraphChanged(EPCGChangeType::Input);
	}
	else if (HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, UserParameters))
	{
		// If the edited property is not the User Parameter itself, it's a parameter inside
		if (EditedProperty != HeadProperty)
		{
			VerifyAndUpdateIfGraphParameterValueChanged(PropertyChangedEvent);
		}
		// If it's the same, it was likely triggered by ApplyChangesToPropertyDescs, which means a structural change to the parameters
		else
		{
			EPCGGraphParameterEvent ChangeType = EPCGGraphParameterEvent::None;
			const int32 NumberOfUserParametersPreEdit = PreviousPropertyBag ? PreviousPropertyBag->GetPropertyDescs().Num() : 0;
			const int32 NumberOfUserParametersPostEdit = UserParameters.GetNumPropertiesInBag();
			FName ChangedPropertyName = NAME_None;

			if (NumberOfUserParametersPostEdit > NumberOfUserParametersPreEdit)
			{
				ChangeType = EPCGGraphParameterEvent::Added;
			}
			else if (NumberOfUserParametersPostEdit < NumberOfUserParametersPreEdit)
			{
				// Removed, but not knowing if it is used or not yet.
				ChangeType = EPCGGraphParameterEvent::RemovedUnused;
			}
			else if (PreviousPropertyBag) // && NumberOfUserParametersPostEdit == NumberOfUserParametersPreEdit
			{
				for (int32 i = 0; i < NumberOfUserParametersPostEdit; ++i)
				{
					const FPropertyBagPropertyDesc& PreDesc = PreviousPropertyBag->GetPropertyDescs()[i];
					const FPropertyBagPropertyDesc& PostDesc = UserParameters.GetPropertyBagStruct()->GetPropertyDescs()[i];

					// Not Same ID -> Moved
					if (PreDesc.ID != PostDesc.ID)
					{
						ChangeType = EPCGGraphParameterEvent::PropertyMoved;
						break;
					}
					// Same ID but different name -> Renamed
					else if (PreDesc.Name != PostDesc.Name)
					{
						ChangeType = EPCGGraphParameterEvent::PropertyRenamed;
						ChangedPropertyName = PostDesc.Name;
						break;
					}
					// Same name but different type -> Type modified
					else if (!PostDesc.CompatibleType(PreDesc))
					{
						ChangeType = EPCGGraphParameterEvent::PropertyTypeModified;
						ChangedPropertyName = PostDesc.Name;
						break;
					}
					// Then, check if the metadata has changed.
					if (PreDesc.MetaData != PostDesc.MetaData)
					{
						ChangeType = EPCGGraphParameterEvent::MetadataModified;
						ChangedPropertyName = PostDesc.Name;
						break;
					}
				}
			}

			if (ChangeType != EPCGGraphParameterEvent::None)
			{
				OnGraphParametersChanged(ChangeType, ChangedPropertyName);
			}
		}
	}
	else if (HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, HiGenGridSize)
		|| HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, bUseHierarchicalGeneration)
		|| HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, bUse2DGrid)
		|| HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, bUseActorComponentlessGeneration))
	{
		bHiGenSettingsChanged = true;
	}
	else if (HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, HiGenGridSizeMultiplier))
	{
		HiGenGridSizeMultiplier = PCGHiGenGrid::SanitizeGridSizeMultiplier(HiGenGridSizeMultiplier);

		bHiGenSettingsChanged = true;
	}
	else if (HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraphInterface, Title)
		|| HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraphInterface, bOverrideTitle)
		|| HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraphInterface, Color)
		|| HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraphInterface, bOverrideColor)
		|| HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, GraphUsageContext)
		|| HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, Category)
		|| HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, Description))
	{
		NotifyGraphChanged(EPCGChangeType::Cosmetic | EPCGChangeType::GraphCustomization);
	}
	else if (HeadPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraph, GenerationRadii))
	{
		GenerationRadii.ComputeHash();
	}
	else if (VerifyIfGraphCustomizationChanged(PropertyChangedEvent))
	{
		NotifyGraphChanged(EPCGChangeType::GraphCustomization);
	}

	if (bHiGenSettingsChanged)
	{
		// The higen settings change the structure of the graph (presence or absence of links between grid levels).
		NotifyGraphChanged(EPCGChangeType::Structural | EPCGChangeType::GenerationGrid);
	}

	PreviousPropertyBag = nullptr;
}

void UPCGGraph::PostEditUndo()
{
	// If we have parameters, they might have changed with this Undo/Redo. So trigger the same mechanism as PostLoad or add multiple properties to update the graph instances that would depend on it.
	if (UserParameters.GetNumPropertiesInBag() > 0)
	{
		OnGraphParametersChanged(EPCGGraphParameterEvent::UndoRedo, NAME_None);
	}
}

UPCGGraph* UPCGGraph::AddNewEmbeddedSubgraph()
{
	check(SupportsEmbeddedSubgraphs());

	Modify();
	UPCGGraph* EmbeddedSubgraph = NewObject<UPCGGraph>(this, GetEmbeddedSubgraphClass(), NAME_None, RF_Transactional);
	EmbeddedSubgraph->bOverrideTitle = true;
	EmbeddedSubgraph->Title = FText::FromName(EmbeddedSubgraph->GetFName());
	EmbeddedSubgraphs.Add(EmbeddedSubgraph);
	NotifyGraphChanged(EPCGChangeType::Structural | EPCGChangeType::GraphCustomization);
	return EmbeddedSubgraph;
}

void UPCGGraph::DeleteEmbeddedSubgraph(UPCGGraph* InEmbeddedSubgraph)
{
	check(SupportsEmbeddedSubgraphs());

	if (int32 Index = EmbeddedSubgraphs.Find(InEmbeddedSubgraph); Index != INDEX_NONE)
	{
		Modify();
		EmbeddedSubgraphs.RemoveAt(Index);

		InEmbeddedSubgraph->Modify();
		InEmbeddedSubgraph->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
		InEmbeddedSubgraph->MarkAsGarbage();
		NotifyGraphChanged(EPCGChangeType::Structural | EPCGChangeType::GraphCustomization);
	}
}

void UPCGGraph::FixInvalidEdges()
{
	auto ValidatePins = [this](const TArray<TObjectPtr<UPCGPin>>& Pins, bool bPinsAreInputs)
	{
		for (UPCGPin* Pin : Pins)
		{
			if (!Pin)
			{
				continue;
			}

			for (int32 i = Pin->Edges.Num() - 1; i >= 0; --i)
			{
				UPCGPin* OtherPin = Pin->Edges[i] ? (bPinsAreInputs ? Pin->Edges[i]->InputPin : Pin->Edges[i]->OutputPin) : nullptr;
				UPCGNode* ConnectedNode = OtherPin ? OtherPin->Node.Get() : nullptr;

				// Remove trivially invalid edges.
				if (!ensure(OtherPin && OtherPin->Node))
				{
					UE_LOGF(LogPCG, Error, "Removed edge to a missing pin or pin that has no node, from graph '%ls'.", *GetFName().ToString());

					Pin->Edges.RemoveAt(i);
				}
				else if (!ConnectedNode || (GetInputNode() != ConnectedNode && GetOutputNode() != ConnectedNode && !Nodes.Contains(ConnectedNode)))
				{
					// Remove edges to nodes that are not present in the graph.
					UE_LOGF(LogPCG, Error, "Removed edge to a node '%ls' that is not registered in graph '%ls'.",
						ConnectedNode ? *ConnectedNode->GetFName().ToString() : TEXT("NULL"),
						*GetFName().ToString());

					Pin->Edges.RemoveAt(i);
				}
			}
		}
	};

	ForEachNode([&ValidatePins](UPCGNode* InNode)
	{
		ValidatePins(InNode->GetInputPins(), /*bPinsAreInputs=*/true);
		ValidatePins(InNode->GetOutputPins(), /*bPinsAreInputs=*/false);
		return true;
	});
}

bool UPCGGraph::UserParametersCanRemoveProperty(FGuid InPropertyID, FName InPropertyName)
{
	// Check if the property has some getters in the graph
	for (const UPCGNode* Node : Nodes)
	{
		if (!Node)
		{
			continue;
		}

		if (const UPCGUserParameterGetSettings* Settings = Cast<UPCGUserParameterGetSettings>(Node->GetSettings()))
		{
			if (Settings->PropertyGuid == InPropertyID)
			{
				// We found a getter. Ask the user if he is OK with that
				FText RemoveCheckMessage = FText::Format(LOCTEXT("UserParametersRemoveCheck", "Property {0} is in use in the graph. Are you sure you want to remove it?"), FText::FromName(InPropertyName));
				FSuppressableWarningDialog::FSetupInfo Info(RemoveCheckMessage, LOCTEXT("UserParametersRemoveCheck_Message", "Remove property"), "UserParametersRemove");
				Info.ConfirmText = FCoreTexts::Get().Yes;
				Info.CancelText = FCoreTexts::Get().No;
				FSuppressableWarningDialog AddLevelWarning(Info);
				if (AddLevelWarning.ShowModal() == FSuppressableWarningDialog::Cancel)
				{
					return false;
				}
			}
		}
	}

	return true;
}

bool UPCGGraph::UserParametersIsPinTypeAccepted(FEdGraphPinType InPinType, bool bIsChild)
{
	// Text and interface not supported
	return InPinType.PinCategory != TEXT("text") && InPinType.PinCategory != TEXT("interface");
}
#endif // WITH_EDITOR

void UPCGGraph::OnGraphParametersChanged(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName)
{
	if (InChangeType == EPCGGraphParameterEvent::RemovedUsed
		|| InChangeType == EPCGGraphParameterEvent::RemovedUnused
		|| InChangeType == EPCGGraphParameterEvent::PropertyRenamed
		|| InChangeType == EPCGGraphParameterEvent::PropertyTypeModified)
	{
		// Look for all the Get Parameter nodes and make sure to delete all nodes that doesn't exist anymore
		TArray<UPCGNode*> NodesToRemove;

		for (UPCGNode* Node : Nodes)
		{
			if (!Node)
			{
				continue;
			}

			if (UPCGUserParameterGetSettings* Settings = Cast<UPCGUserParameterGetSettings>(Node->GetSettings()))
			{
				const FPropertyBagPropertyDesc* PropertyDesc = UserParameters.FindPropertyDescByID(Settings->PropertyGuid);
				if (!PropertyDesc)
				{
					NodesToRemove.Add(Node);
				}
				else if (Settings->PropertyName != PropertyDesc->Name)
				{
					const FName OldName = Settings->PropertyName;
					Settings->UpdatePropertyName(PropertyDesc->Name);
					Node->NodeTitle = PropertyDesc->Name;
					// We make sure to keep the edges connected, by renaming the pin label
					Node->RenameOutputPin(OldName, PropertyDesc->Name);
				}

				if (InChangeType == EPCGGraphParameterEvent::PropertyTypeModified)
				{
					// Also force a refresh on the pin output for the subtype.
					Node->UpdatePins();
				}
			}
		}

		if (!NodesToRemove.IsEmpty())
		{
			Modify();

			for (UPCGNode* Node : NodesToRemove)
			{
				RemoveNode(Node);
			}
		}
	}

#if WITH_EDITOR
	NotifyGraphParametersChanged(InChangeType, InChangedPropertyName);
#endif // WITH_EDITOR
}

void UPCGGraph::UpdateUserParametersStruct(TFunctionRef<void(FInstancedPropertyBag&)> Callback)
{
	Callback(UserParameters);
	// Since anything could have changed, trigger a refresh like a post load (to compare what changed)
	OnGraphParametersChanged(EPCGGraphParameterEvent::GraphPostLoad, NAME_None);
}

FInstancedPropertyBag* UPCGGraph::GetMutableUserParametersStruct()
{
	return &UserParameters;
}

uint32 UPCGGraph::GetNodeGenerationGridSize(const UPCGNode* InNode, uint32 InDefaultGridSize) const
{
	{
		PCG::TSharedScopeLock ReadLock(NodeToGridSizeLock);
		if (const uint32* CachedGridSize = NodeToGridSize.Find(InNode))
		{
			return *CachedGridSize;
		}
	}

	{
		PCG::TUniqueScopeLock WriteLock(NodeToGridSizeLock);
		return CalculateNodeGridSizeRecursive_Unsafe(InNode, InDefaultGridSize);
	}
}

bool UPCGGraph::AnyPlatformShouldCook() const
{
#if WITH_EDITOR || WITH_PREVIEW_PPX_DATA
	if (ShouldCook.Default)
	{
		return true;
	}

	for (const auto& Pair : ShouldCook.PerPlatform)
	{
		if (Pair.Value)
		{
			return true;
		}
	}

	return false;
#else
	return ShouldCook.GetValue();
#endif
}

uint32 UPCGGraph::CalculateNodeGridSizeRecursive_Unsafe(const UPCGNode* InNode, uint32 InDefaultGridSize) const
{
	if (const uint32* CachedGridSize = NodeToGridSize.Find(InNode))
	{
		return *CachedGridSize;
	}

	uint32 GridSize = InDefaultGridSize;

	const UPCGHiGenGridSizeSettings* GridSizeSettings = Cast<UPCGHiGenGridSizeSettings>(InNode->GetSettings());
	if (GridSizeSettings && GridSizeSettings->bEnabled)
	{
		GridSize = FMath::Min(GridSize, GridSizeSettings->GetScaledGridSize());
	}
	else
	{
		// Grid size for a node is the minimum of the grid sizes of connected upstream nodes.
		for (const UPCGPin* Pin : InNode->GetInputPins())
		{
			if (Pin)
			{
				for (const UPCGEdge* Edge : Pin->Edges)
				{
					const UPCGPin* OtherPin = Edge ? Edge->InputPin.Get() : nullptr;
					if (OtherPin && OtherPin->Node.Get())
					{
						const uint32 InputGridSize = CalculateNodeGridSizeRecursive_Unsafe(OtherPin->Node, InDefaultGridSize);
						if (PCGHiGenGrid::IsValidGridSize(InputGridSize))
						{
							GridSize = FMath::Min(GridSize, InputGridSize);
						}
					}
				}
			}
		}
	}

	if (GridSize != PCGHiGenGrid::UninitializedGridSize())
	{
		NodeToGridSize.Add(InNode, GridSize);
	}

	return GridSize;
}

PCGHiGenGrid::FSizeArray UPCGGraph::CalculateNodeGridSizesRecursiveNoLock(const UPCGNode* InNode, uint32 InDefaultGridSize) const
{
	if (!InNode)
	{
		return {};
	}

	if (const PCGHiGenGrid::FSizeArray* CachedGridSizes = NodeToAllGridSizes.Find(InNode))
	{
		return *CachedGridSizes;
	}

	PCGHiGenGrid::FSizeArray AllGridSizes;

	const uint32 NodeGridSize = CalculateNodeGridSizeRecursive_Unsafe(InNode, InDefaultGridSize);
	AllGridSizes.AddUnique(NodeGridSize);

	for (const UPCGPin* Pin : InNode->GetInputPins())
	{
		if (Pin)
		{
			for (const UPCGEdge* Edge : Pin->Edges)
			{
				const UPCGPin* OtherPin = Edge ? Edge->InputPin.Get() : nullptr;
				if (OtherPin && OtherPin->Node.Get())
				{
					PCGHiGenGrid::FSizeArray InputGridSizes = CalculateNodeGridSizesRecursiveNoLock(OtherPin->Node.Get(), InDefaultGridSize);

					for (uint32 GridSize : InputGridSizes)
					{
						AllGridSizes.AddUnique(GridSize);
					}
				}
			}
		}
	}

	// Descending order.
	AllGridSizes.Sort([](const uint32& A, const uint32& B) { return A > B; });

	NodeToAllGridSizes.Add(InNode, AllGridSizes);

	return AllGridSizes;
}

EPropertyBagAlterationResult UPCGGraph::AddUserParameters(const TArray<FPropertyBagPropertyDesc>& InDescs, const UPCGGraph* InOptionalOriginalGraph)
{
	switch (const EPropertyBagAlterationResult Result = UserParameters.AddProperties(InDescs))
	{
		case EPropertyBagAlterationResult::Success:
			break;
		default: // Any error, return the result.
			return Result;
	}

	if (InOptionalOriginalGraph)
	{
		if (const FInstancedPropertyBag* OriginalPropertyBag = InOptionalOriginalGraph->GetUserParametersStruct())
		{
			UserParameters.CopyMatchingValuesByID(*OriginalPropertyBag);
		}
	}

	OnGraphParametersChanged(EPCGGraphParameterEvent::MultiplePropertiesAdded, NAME_None);

	return EPropertyBagAlterationResult::Success;
}

#if WITH_EDITOR
void UPCGGraph::SetHiddenFlagInputNode(bool bHidden)
{
	if (InputNode && InputNode->bHidden != bHidden)
	{
		InputNode->Modify();
		InputNode->bHidden = bHidden;
	}
}

void UPCGGraph::SetHiddenFlagOutputNode(bool bHidden)
{
	if (OutputNode && OutputNode->bHidden != bHidden)
	{
		OutputNode->Modify();
		OutputNode->bHidden = bHidden;
	}
}
#endif // WITH_EDITOR

/****************************
* UPCGGraphInstance
****************************/

void UPCGGraphInstance::PostLoad()
{
	Super::PostLoad();

	if (Graph)
	{
		Graph->ConditionalPostLoad();
	}

	RefreshParameters(EPCGGraphParameterEvent::GraphPostLoad);

#if WITH_EDITOR
	SetupCallbacks();
#endif // WITH_EDITOR
}

bool UPCGGraphInstance::NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const
{
	if (!Super::NeedsLoadForTargetPlatform(TargetPlatform))
	{
		return false;
	}

	const UPCGGraph* PCGGraph = GetGraph();
	return !PCGGraph || PCGGraph->NeedsLoadForTargetPlatform(TargetPlatform);
}

void UPCGGraphInstance::BeginDestroy()
{
#if WITH_EDITOR
	TeardownCallbacks();
#endif // WITH_EDITOR

	Super::BeginDestroy();
}

void UPCGGraphInstance::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

#if WITH_EDITOR
	SetupCallbacks();
#endif // WITH_EDITOR
}

void UPCGGraphInstance::PostEditImport()
{
	Super::PostEditImport();

#if WITH_EDITOR
	SetupCallbacks();
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UPCGGraphInstance::PreEditChange(FProperty* InProperty)
{
	Super::PreEditChange(InProperty);

	if (!InProperty)
	{
		return;
	}

	// We need to be careful and only capture `Graph` if it is our graph and not a graph parameter called `Graph`!
	if (InProperty->GetOwnerClass() == UPCGGraphInstance::StaticClass() && InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, Graph))
	{
		PreGraphCache = Graph;
		TeardownCallbacks();
	}
}

void UPCGGraphInstance::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (!PropertyChangedEvent.Property)
	{
		return;
	}

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	// We need to be careful and only capture `Graph` if it is our graph and not a graph parameter called `Graph`!
	if (PropertyChangedEvent.Property->GetOwnerClass() == UPCGGraphInstance::StaticClass() && PropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, Graph))
	{
		// If the new graph hierarchy has this graph in it, return to the previous value.
		if (Graph && !CanGraphInterfaceBeSet(Graph))
		{
			UE_LOGF(LogPCG, Error, "Attempting to assign %ls would cause infinite recursion in the graph instance hierarchy, this is not allowed.", *Graph->GetPathName());
			Graph = PreGraphCache.Get();
		}

		SetupCallbacks();

		// No need to refresh if it is the same graph, but we need to refresh if we have no graph anymore, but the pre graph was valid (but isn't anymore like in a Force Delete Asset)
		if (Graph != PreGraphCache || (!Graph && !PreGraphCache.IsExplicitlyNull()))
		{
			OnGraphParametersChanged(Graph, EPCGGraphParameterEvent::GraphChanged, NAME_None);
		}

		// Reset them there to avoid any side effect if Pre/Post are called multiple times for the same transaction.
		PreGraphCache = nullptr;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FPCGOverrideInstancedPropertyBag, PropertiesIDsOverridden))
	{
		// A change on the overriden properties can come from multiple places (like ImportText), and we don't know what changed or caused the change
		// so treat it as a Undo/Redo.
		OnGraphParametersChanged(Graph, EPCGGraphParameterEvent::UndoRedo, NAME_None);
	}
	else if (VerifyIfGraphCustomizationChanged(PropertyChangedEvent))
	{
		OnGraphChangedDelegate.Broadcast(this, EPCGChangeType::GraphCustomization);
	}
	else if (VerifyAndUpdateIfGraphParameterValueChanged(PropertyChangedEvent))
	{
		// Handled in the function
	}
	else
	{
		// For other changes, push a cosmetic change
		OnGraphChangedDelegate.Broadcast(this, EPCGChangeType::Cosmetic);
	}
}

void UPCGGraphInstance::PreEditUndo()
{
	Super::PreEditUndo();

	TeardownCallbacks();

	PreGraphCache = Graph;
}

void UPCGGraphInstance::PostEditUndo()
{
	Super::PostEditUndo();

	SetupCallbacks();

	// Since we don't know what happened, we need to notify any changes
	NotifyGraphParametersChanged(EPCGGraphParameterEvent::GraphChanged, NAME_None);
}

void UPCGGraphInstance::OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType)
{
	if (InGraph == Graph)
	{
		if (ChangeType != EPCGChangeType::Cosmetic)
		{
			// Also notify other systems that this graph changed, only if the owner is not a PCG Component nor PCG Subgraph.
			// They already have their own system to trigger a refresh.
			const UObject* Outer = GetOuter();
			if (!Outer || !(Outer->IsA<UPCGComponent>() || Outer->IsA<UPCGSubgraphSettings>()))
			{
				FPropertyChangedEvent EmptyEvent{ nullptr };
				FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, EmptyEvent);
			}
		}

		OnGraphChangedDelegate.Broadcast(this, ChangeType);
	}
}

bool UPCGGraphInstance::CanEditChange(const FProperty* InProperty) const
{
	UPCGComponent* Component = Cast<UPCGComponent>(GetOuter());
	AActor* ComponentOwner = Component ? Component->GetOwner() : nullptr;
		
	if(ComponentOwner && ComponentOwner->IsInLevelInstance() && !ComponentOwner->IsInEditLevelInstance())
	{
		return false;
	}

	// Graph can only be changed if it is not in a local PCGComponent
	if (InProperty && InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, Graph))
	{
		if (Component)
		{
			return !Component->IsLocalComponent();
		}
	}

	return true;
}

void UPCGGraphInstance::TeardownCallbacks()
{
	if (FPCGContext::IsInitializingSettings())
	{
		return;
	}

	if (Graph)
	{
		Graph->OnGraphChangedDelegate.RemoveAll(this);
		Graph->OnGraphParametersChangedDelegate.RemoveAll(this);
	}
}

TOptional<FPCGGraphToolData> UPCGGraphInstance::GetGraphToolData() const
{
	TOptional<FPCGGraphToolData> GraphToolData = GetGraph() ? GetGraph()->GetGraphToolData() : TOptional<FPCGGraphToolData>();

	if (GraphToolData.IsSet())
	{
		GraphToolData->ApplyOverrides(ToolDataOverrides);
	}
	
	return GraphToolData;
}

void UPCGGraphInstance::SetupCallbacks()
{
	if (FPCGContext::IsInitializingSettings())
	{
		return;
	}

	if (Graph && !Graph->OnGraphChangedDelegate.IsBoundToObject(this))
	{
		Graph->OnGraphChangedDelegate.AddUObject(this, &UPCGGraphInstance::OnGraphChanged);
		Graph->OnGraphParametersChangedDelegate.AddUObject(this, &UPCGGraphInstance::OnGraphParametersChanged);
	}
}
#endif

void UPCGGraphInstance::SetGraph(UPCGGraphInterface* InGraph)
{
	if (InGraph && !CanGraphInterfaceBeSet(InGraph))
	{
		UE_LOGF(LogPCG, Error, "Attempting to assign %ls would cause infinite recursion in the graph instance hierarchy, this is not allowed.", *InGraph->GetPathName());
		return;
	}

	if (InGraph == Graph)
	{
		// Nothing to do
		return;
	}

	Modify();

#if WITH_EDITOR
	TeardownCallbacks();
#endif // WITH_EDITOR

	Graph = InGraph;

#if WITH_EDITOR
	SetupCallbacks();
#endif // WITH_EDITOR

#if WITH_EDITOR
	OnGraphParametersChanged(Graph, EPCGGraphParameterEvent::GraphChanged, NAME_None);
#else
	// TODO: We need to revisit this, because it won't update any child graph that has this instance as their graph.
	// Making the hotswap of graph instances within graph instances not working as intended in non-editor builds.
	// Perhaps that should not be possible? At least it is mitigated in the GetUserParameter node, that will take the first valid layout.
	RefreshParameters(EPCGGraphParameterEvent::GraphChanged, NAME_None);
#endif // WITH_EDITOR
}

TObjectPtr<UPCGGraphInterface> UPCGGraphInstance::CreateInstance(UObject* InOwner, UPCGGraphInterface* InGraph)
{
	if (!InOwner || !InGraph)
	{
		return nullptr;
	}

	TObjectPtr<UPCGGraphInstance> GraphInstance = NewObject<UPCGGraphInstance>(InOwner, MakeUniqueObjectName(InOwner, UPCGGraphInstance::StaticClass(), InGraph->GetFName()), RF_Transactional | RF_Public);
	GraphInstance->SetGraph(InGraph);

	return GraphInstance;
}

#if WITH_EDITOR
void UPCGGraphInstance::NotifyGraphParametersChanged(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName)
{
	OnGraphParametersChangedDelegate.Broadcast(this, InChangeType, InChangedPropertyName);

	// Also propagates the changes
	OnGraphChanged(Graph, GetChangeTypeForGraphParameterChange(InChangeType, InChangedPropertyName));
}

TOptional<FText> UPCGGraphInstance::GetTitleOverride() const
{
	return (!bOverrideTitle && Graph) ? Graph->GetTitleOverride() : UPCGGraphInterface::GetTitleOverride();
}

TOptional<FLinearColor> UPCGGraphInstance::GetColorOverride() const
{
	return (!bOverrideColor && Graph) ? Graph->GetColorOverride() : UPCGGraphInterface::GetColorOverride();
}

UObject* UPCGGraphInstance::GetUserParameterHierarchyRoot() const
{
	return Graph ? Graph->GetUserParameterHierarchyRoot() : nullptr;
}
#endif // WITH_EDITOR

void UPCGGraphInstance::OnGraphParametersChanged(UPCGGraphInterface* InGraph, EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName)
{
	if (InGraph != Graph && InGraph != this)
	{
		return;
	}

	EPCGGraphParameterEvent ChangeType = InChangeType;
	if (InGraph == Graph && InChangeType == EPCGGraphParameterEvent::ValueModifiedLocally)
	{
		// If we receive a "ValueModifiedLocally" and it was on our Graph, we transform it to "ValueModifiedByParent"
		ChangeType = EPCGGraphParameterEvent::ValueModifiedByParent;
	}

	RefreshParameters(ChangeType, InChangedPropertyName);
#if WITH_EDITOR
	NotifyGraphParametersChanged(ChangeType, InChangedPropertyName);
#endif // WITH_EDITOR
}

void UPCGGraphInstance::OnGraphParametersChanged(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName)
{
	OnGraphParametersChanged(this, InChangeType, InChangedPropertyName);
}

FInstancedPropertyBag* UPCGGraphInstance::GetMutableUserParametersStruct()
{
	return &ParametersOverrides.Parameters;
}

void UPCGGraphInstance::RefreshParameters(EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName)
{
	if (!Graph)
	{
		if (ParametersOverrides.IsValid())
		{
			Modify();
		}

		ParametersOverrides.Reset();
	}
	else
	{
		const FInstancedPropertyBag* ParentUserParameters = Graph->GetUserParametersStruct();

		// Refresh can modify nothing, but we still need to keep a snapshot of this object state, if it ever change.
		// Don't mark it dirty by default, only if something changed.
		Modify(/*bAlwaysMarkDirty=*/false);

		if (ParametersOverrides.RefreshParameters(ParentUserParameters, InChangeType, InChangedPropertyName))
		{
			MarkPackageDirty();
		}
	}
}

void UPCGGraphInstance::UpdatePropertyOverride(const FProperty* InProperty, bool bMarkAsOverridden)
{
	if (!Graph || !InProperty)
	{
		return;
	}

	Modify();

	const FInstancedPropertyBag* ParentUserParameters = Graph->GetUserParametersStruct();
	if (ParametersOverrides.UpdatePropertyOverride(InProperty, bMarkAsOverridden, ParentUserParameters))
	{
#if WITH_EDITOR
		// If it is true, it means that the value has changed, so propagate the changes, in Editor
		NotifyGraphParametersChanged(EPCGGraphParameterEvent::ValueModifiedLocally, InProperty->GetFName());
#endif // WITH_EDITOR
	}
}

void UPCGGraphInstance::CopyParameterOverrides(UPCGGraphInterface* InGraph)
{
	// Can't copy if they do not have the same base graph
	if (!InGraph || GetGraph() != InGraph->GetGraph())
	{
		return;
	}

	// Inherit from the instance overrides or reset them.
	if (const UPCGGraphInstance* OtherGraphInstance = Cast<UPCGGraphInstance>(InGraph))
	{
		ParametersOverrides.PropertiesIDsOverridden = OtherGraphInstance->ParametersOverrides.PropertiesIDsOverridden;
	}
	else
	{
		ParametersOverrides.PropertiesIDsOverridden.Reset();
	}

	// And copy everything, to be sure we are up to date.
	ParametersOverrides.Parameters.CopyMatchingValuesByID(*InGraph->GetUserParametersStruct());
}

void UPCGGraphInstance::ResetPropertyToDefault(const FProperty* InProperty)
{
	if (!IsPropertyOverridden(InProperty))
	{
		return;
	}

	Modify();

	bool bValueChanged = ParametersOverrides.ResetPropertyToDefault(InProperty, Graph->GetUserParametersStruct());

#if WITH_EDITOR
	if (bValueChanged)
	{
		NotifyGraphParametersChanged(EPCGGraphParameterEvent::ValueModifiedLocally, InProperty->GetFName());
	}
#endif // WITH_EDITOR
}

bool UPCGGraphInstance::IsPropertyOverriddenAndNotDefault(const FProperty* InProperty) const
{
	return Graph ? ParametersOverrides.IsPropertyOverriddenAndNotDefault(InProperty, Graph->GetUserParametersStruct()) : false;
}


#if WITH_EDITOR
FString UPCGGraphInstance::GetDefaultPropertyValueForEditor(const FProperty* InProperty, bool& bIsDifferent) const
{
	bIsDifferent = false;
	
	if (!IsPropertyOverridden(InProperty))
	{
		return {};
	}
	
	return ParametersOverrides.GetDefaultPropertyValueForEditor(InProperty, Graph->GetUserParametersStruct(), bIsDifferent);
}

FString UPCGGraphInstance::ExportOverriddenPropertyIdsChangeForEditor(const FProperty* InProperty, bool bMarkAsOverridden, bool& bIsDifferent) const
{
	TSet<FGuid> CopyOfPropertyIDsOverridden;
	bIsDifferent = false;
	
	if (const FPropertyBagPropertyDesc* PropertyDesc = ParametersOverrides.Parameters.FindPropertyDescByName(InProperty->GetFName()))
	{
		if (bMarkAsOverridden && !ParametersOverrides.PropertiesIDsOverridden.Contains(PropertyDesc->ID))
		{
			CopyOfPropertyIDsOverridden = ParametersOverrides.PropertiesIDsOverridden;
			CopyOfPropertyIDsOverridden.Add(PropertyDesc->ID);
			bIsDifferent = true;
		}
		else if (!bMarkAsOverridden && ParametersOverrides.PropertiesIDsOverridden.Contains(PropertyDesc->ID))
		{
			CopyOfPropertyIDsOverridden = ParametersOverrides.PropertiesIDsOverridden;
			CopyOfPropertyIDsOverridden.Remove(PropertyDesc->ID);
			bIsDifferent = true;
		}
	}

	if (bIsDifferent)
	{
		const FProperty* PropertyIDsOverriddenProperty = FPCGOverrideInstancedPropertyBag::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FPCGOverrideInstancedPropertyBag, PropertiesIDsOverridden));
		check(PropertyIDsOverriddenProperty);
		FString Result{};
		PropertyIDsOverriddenProperty->ExportTextItem_Direct(Result, &CopyOfPropertyIDsOverridden, nullptr, nullptr, PPF_None);
		return Result;
	}
	else
	{
		return FString{};
	}
}
#endif // WITH_EDITOR

bool UPCGGraphInstance::IsGraphParameterOverridden(const FName PropertyName) const
{
	return (ParametersOverrides.Parameters.FindPropertyDescByName(PropertyName) != nullptr);
}

bool UPCGGraphInstance::CanGraphInterfaceBeSet(const UPCGGraphInterface* GraphInterface) const
{
	if (GraphInterface == this)
	{
		return false;
	}

	const UPCGGraphInstance* GraphInstance = Cast<const UPCGGraphInstance>(GraphInterface);
	// Can always set a normal graph (or null graph)
	if (!GraphInstance)
	{
		return true;
	}
	
	return CanGraphInterfaceBeSet(GraphInstance->Graph);
}

bool FPCGOverrideInstancedPropertyBag::RefreshParameters(const FInstancedPropertyBag* ParentUserParameters, EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName)
{
	check(ParentUserParameters);

	bool bWasModified = false;

	if (!ParentUserParameters->IsValid())
	{
		Reset();
		return true;
	}

	switch (InChangeType)
	{
	case EPCGGraphParameterEvent::GraphChanged:
	{
		// We should always copy the parents parameters and reset overrides when the graph changes. Even if it is the same struct, values might be different.
		bWasModified = true;
		// Copy the parent parameters and reset overriddes
		Parameters = *ParentUserParameters;
		PropertiesIDsOverridden.Reset();
#if WITH_EDITOR
		// Notify the UI to force refresh
		PCGDelegates::OnInstancedPropertyBagLayoutChanged.Broadcast(Parameters);
#endif // WITH_EDITOR
		break;
	}
	case EPCGGraphParameterEvent::Added: // fall-through
	case EPCGGraphParameterEvent::RemovedUnused: // fall-through
	case EPCGGraphParameterEvent::RemovedUsed: // fall-through
	case EPCGGraphParameterEvent::PropertyRenamed: // fall-through
	case EPCGGraphParameterEvent::PropertyMoved: // fall-through
	case EPCGGraphParameterEvent::CategoryChanged: // fall-through
	case EPCGGraphParameterEvent::PropertyTypeModified:
	{
		bWasModified = true;
		const FPropertyBagPropertyDesc* ThisPropertyDesc = Parameters.FindPropertyDescByName(InChangedPropertyName);

		if (ThisPropertyDesc)
		{
			UpdatePropertyOverride(ThisPropertyDesc->CachedProperty, false, ParentUserParameters);
		}

		MigrateToNewBagInstance(*ParentUserParameters);
#if WITH_EDITOR
		// Notify the UI to force refresh
		PCGDelegates::OnInstancedPropertyBagLayoutChanged.Broadcast(Parameters);
#endif // WITH_EDITOR
		break;
	}
	case EPCGGraphParameterEvent::MetadataModified:
	{
		MigrateToNewBagInstance(*ParentUserParameters);
#if WITH_EDITOR
		// Notify the UI to force refresh
		PCGDelegates::OnInstancedPropertyBagLayoutChanged.Broadcast(Parameters);
#endif // WITH_EDITOR
		break;
	}
	case EPCGGraphParameterEvent::ValueModifiedByParent:
	{
		const FPropertyBagPropertyDesc* OriginalPropertyDesc = ParentUserParameters->FindPropertyDescByName(InChangedPropertyName);
		const FPropertyBagPropertyDesc* ThisPropertyDesc = Parameters.FindPropertyDescByName(InChangedPropertyName);

		check(InChangedPropertyName != NAME_None);
		check(OriginalPropertyDesc);

		if (!PCGGraphUtils::ArePropertiesCompatible(OriginalPropertyDesc, ThisPropertyDesc))
		{
			bWasModified = true;
			MigrateToNewBagInstance(*ParentUserParameters);
		}
		else if (!IsPropertyOverridden(ThisPropertyDesc->CachedProperty))
		{
			// Only update the value if the property is not overriden.
			bWasModified = true;
			PCGGraphUtils::CopyPropertyValue(OriginalPropertyDesc, *ParentUserParameters, ThisPropertyDesc, Parameters);
		}
		break;
	}
	case EPCGGraphParameterEvent::ValueModifiedLocally:
	{
		const FPropertyBagPropertyDesc* OriginalPropertyDesc = ParentUserParameters->FindPropertyDescByName(InChangedPropertyName);
		const FPropertyBagPropertyDesc* ThisPropertyDesc = Parameters.FindPropertyDescByName(InChangedPropertyName);

		check(InChangedPropertyName != NAME_None);
		check(OriginalPropertyDesc);

		if (!PCGGraphUtils::ArePropertiesCompatible(OriginalPropertyDesc, ThisPropertyDesc))
		{
			bWasModified = true;
			MigrateToNewBagInstance(*ParentUserParameters);
		}
		else
		{
			// Force the value to be overridden, if it is not equal to the value and it was changed from the outside
			if (!PCGGraphUtils::ArePropertiesIdentical(OriginalPropertyDesc, *ParentUserParameters, ThisPropertyDesc, Parameters))
			{
				bWasModified = true;
				UpdatePropertyOverride(ThisPropertyDesc->CachedProperty, true, ParentUserParameters);
			}
		}
		break;
	}
	// Do the same thing in case of post load, multiple properties added and undo/redo.
	// We have 3 enums to avoid puzzling someone that wonders why we would call GraphPostLoad when we add multiple properties or undo/redo.
	case EPCGGraphParameterEvent::GraphPostLoad: // fall-through
	case EPCGGraphParameterEvent::MultiplePropertiesAdded: // fall-through
	case EPCGGraphParameterEvent::UndoRedo:
	{
		// Check if the property struct mismatch. If so, do the migration
		if (Parameters.GetPropertyBagStruct() != ParentUserParameters->GetPropertyBagStruct())
		{
			bWasModified = true;
			MigrateToNewBagInstance(*ParentUserParameters);
		}

		if (Parameters.GetPropertyBagStruct() == nullptr)
		{
			return bWasModified;
		}

		// And then overwrite all non-overridden values
		for (const FPropertyBagPropertyDesc& ThisPropertyDesc : Parameters.GetPropertyBagStruct()->GetPropertyDescs())
		{
			if (!IsPropertyOverridden(ThisPropertyDesc.CachedProperty))
			{
				const FPropertyBagPropertyDesc* OriginalPropertyDesc = ParentUserParameters->FindPropertyDescByID(ThisPropertyDesc.ID);

				if (!PCGGraphUtils::ArePropertiesIdentical(OriginalPropertyDesc, *ParentUserParameters, &ThisPropertyDesc, Parameters))
				{
					bWasModified = true;
					PCGGraphUtils::CopyPropertyValue(OriginalPropertyDesc, *ParentUserParameters, &ThisPropertyDesc, Parameters);
				}
			}
		}
		break;
	}
	}

	return bWasModified;
}

bool FPCGOverrideInstancedPropertyBag::UpdatePropertyOverride(const FProperty* InProperty, bool bMarkAsOverridden, const FInstancedPropertyBag* ParentUserParameters)
{
	if (!InProperty)
	{
		return false;
	}

	if (const FPropertyBagPropertyDesc* PropertyDesc = Parameters.FindPropertyDescByName(InProperty->GetFName()))
	{
		if (bMarkAsOverridden)
		{
			PropertiesIDsOverridden.Add(PropertyDesc->ID);
		}
		else
		{
			PropertiesIDsOverridden.Remove(PropertyDesc->ID);
		}
	}

	// Reset the value if it is not marked overridden anymore.
	if (!bMarkAsOverridden)
	{
		return ResetPropertyToDefault(InProperty, ParentUserParameters);
	}

	return false;
}

bool FPCGOverrideInstancedPropertyBag::ResetPropertyToDefault(const FProperty* InProperty, const FInstancedPropertyBag* ParentUserParameters)
{
	check(ParentUserParameters);

	const FPropertyBagPropertyDesc* OriginalPropertyDesc = ParentUserParameters->FindPropertyDescByName(InProperty->GetFName());
	const FPropertyBagPropertyDesc* ThisPropertyDesc = Parameters.FindPropertyDescByName(InProperty->GetFName());

	if (OriginalPropertyDesc && ThisPropertyDesc)
	{
		if (!PCGGraphUtils::ArePropertiesIdentical(OriginalPropertyDesc, *ParentUserParameters, ThisPropertyDesc, Parameters))
		{
			PCGGraphUtils::CopyPropertyValue(OriginalPropertyDesc, *ParentUserParameters, ThisPropertyDesc, Parameters);
			return true;
		}
	}

	return false;
}

#if WITH_EDITOR
FString FPCGOverrideInstancedPropertyBag::GetDefaultPropertyValueForEditor(const FProperty* InProperty, const FInstancedPropertyBag* ParentUserParameters, bool& bIsDifferent) const
{
	bIsDifferent = false;
	check(ParentUserParameters);

	const FPropertyBagPropertyDesc* OriginalPropertyDesc = ParentUserParameters->FindPropertyDescByName(InProperty->GetFName());
	const FPropertyBagPropertyDesc* ThisPropertyDesc = Parameters.FindPropertyDescByName(InProperty->GetFName());

	if (OriginalPropertyDesc && ThisPropertyDesc)
	{
		if (!PCGGraphUtils::ArePropertiesIdentical(OriginalPropertyDesc, *ParentUserParameters, ThisPropertyDesc, Parameters))
		{
			FString OutValueString;
			if (PCGGraphUtils::GetDefaultPropertyValueForEditor(OriginalPropertyDesc, *ParentUserParameters, OutValueString))
			{
				bIsDifferent = true;
				return OutValueString;
			}
		}
	}

	return {};
}
#endif // WITH_EDITOR

bool FPCGOverrideInstancedPropertyBag::IsPropertyOverridden(const FProperty* InProperty) const
{
	if (!InProperty)
	{
		return false;
	}

	const FPropertyBagPropertyDesc* PropertyDesc = Parameters.FindPropertyDescByName(InProperty->GetFName());
	return PropertyDesc && PropertiesIDsOverridden.Contains(PropertyDesc->ID);
}

bool FPCGOverrideInstancedPropertyBag::IsPropertyOverriddenAndNotDefault(const FProperty* InProperty, const FInstancedPropertyBag* ParentUserParameters) const
{
	check(ParentUserParameters);

	const FPropertyBagPropertyDesc* OriginalPropertyDesc = ParentUserParameters->FindPropertyDescByName(InProperty->GetFName());
	const FPropertyBagPropertyDesc* ThisPropertyDesc = Parameters.FindPropertyDescByName(InProperty->GetFName());

	if (OriginalPropertyDesc && ThisPropertyDesc && PropertiesIDsOverridden.Contains(ThisPropertyDesc->ID))
	{
		return !PCGGraphUtils::ArePropertiesIdentical(OriginalPropertyDesc, *ParentUserParameters, ThisPropertyDesc, Parameters);
	}
	else
	{
		return false;
	}
}

void FPCGOverrideInstancedPropertyBag::Reset()
{
	Parameters.Reset();
	PropertiesIDsOverridden.Reset();
}

void FPCGOverrideInstancedPropertyBag::MigrateToNewBagInstance(const FInstancedPropertyBag& NewBagInstance)
{
	// Keeping a map between id and types. We will remove override for property that changed types.
	TMap<FGuid, FPropertyBagPropertyDesc> IdToDescMap;
	if (Parameters.GetPropertyBagStruct())
	{
		for (const FPropertyBagPropertyDesc& PropertyDesc : Parameters.GetPropertyBagStruct()->GetPropertyDescs())
		{
			IdToDescMap.Emplace(PropertyDesc.ID, PropertyDesc);
		}
	}

	Parameters.MigrateToNewBagInstance(NewBagInstance);

	if (NewBagInstance.GetPropertyBagStruct() == nullptr)
	{
		return;
	}

	// Remove overridden parameters that are not in the bag anymore, or have changed type
	TArray<FGuid> OverriddenParametersCopy = PropertiesIDsOverridden.Array();
	for (const FGuid PropertyId: OverriddenParametersCopy)
	{
		const FPropertyBagPropertyDesc* NewPropertyDesc = NewBagInstance.FindPropertyDescByID(PropertyId);
		const FPropertyBagPropertyDesc* OldPropertyDesc = IdToDescMap.Find(PropertyId);

		const bool bTypeHasChanged = NewPropertyDesc && OldPropertyDesc && (NewPropertyDesc->ValueType != OldPropertyDesc->ValueType || NewPropertyDesc->ValueTypeObject != OldPropertyDesc->ValueTypeObject);

		if (!NewPropertyDesc || bTypeHasChanged)
		{
			PropertiesIDsOverridden.Remove(PropertyId);
			continue;
		}
	}
}

bool UPCGGraphInstance::GraphAssetFilter(const FAssetData& AssetData) const
{
#if WITH_EDITOR
	if (UPCGSubgraphSettings* SubgraphSettings = Cast<UPCGSubgraphSettings>(GetOuter()))
	{
		return SubgraphSettings->SubgraphAssetFilter(AssetData);
	}
	// TODO : add filtering on PCG components?
	else
	{
		return false;
	}
#else
	return false;
#endif
}

#undef LOCTEXT_NAMESPACE
