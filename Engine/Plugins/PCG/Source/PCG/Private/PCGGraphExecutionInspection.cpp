// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGGraphExecutionInspection.h"

#include "PCGProfilingLog.h"

#if PCG_PROFILING_ENABLED

#if WITH_EDITOR

#include "PCGInputOutputSettings.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "PCGSubgraph.h"

void FPCGInspectionData::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (Data)
	{
		Data->AddReferences(Collector);
	}
}

FString FPCGInspectionData::GetReferencerName() const
{
	return TEXT("FPCGInspectionData");
}

bool FPCGGraphExecutionInspection::IsInspecting() const
{
	return InspectionCounter > 0;
}

void FPCGGraphExecutionInspection::EnableInspection()
{
	if (!ensure(InspectionCounter >= 0))
	{
		InspectionCounter = 0;
	}

	InspectionCounter++;
}

void FPCGGraphExecutionInspection::DisableInspection()
{
	if (ensure(InspectionCounter > 0))
	{
		InspectionCounter--;
	}

	if (InspectionCounter == 0)
	{
		ClearInspectionData(/*bClearPerNodeExecutionData=*/false);
	}
};

#endif // WITH_EDITOR

void FPCGGraphExecutionInspection::NotifyNodeExecuted(const UPCGNode* InNode, const FPCGStack* InStack, const PCGUtils::FCallTime* InTimer, bool bNodeUsedCache)
{
	if (!ensure(InStack && InNode))
	{
		return;
	}

	// Reset timer information if taken from cache to provide good info in the profiling window
	PCGUtils::FCallTime Timer;
	if (InTimer && !bNodeUsedCache)
	{
		Timer = *InTimer;
	}

	PCG::TUniqueScopeLock WriteLock(NodeToStacksInWhichNodeExecutedLock);
	NodeToStacksInWhichNodeExecuted.FindOrAdd(InNode).Add(FNodeExecutedNotificationData(*InStack, MoveTemp(Timer)));
	++ExecutedStacksGeneration;
}

TMap<TObjectKey<const UPCGNode>, TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData>> FPCGGraphExecutionInspection::GetExecutedNodeStacks() const
{
	PCG::TSharedScopeLock ReadLock(NodeToStacksInWhichNodeExecutedLock);
	return NodeToStacksInWhichNodeExecuted;
}

TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData> FPCGGraphExecutionInspection::GetExecutedNodeStacks(const UPCGNode* InNode) const
{
	TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData> ExecutedStacks;
	PCG::TSharedScopeLock ReadLock(NodeToStacksInWhichNodeExecutedLock);
	if (const TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData>* FoundStacks = NodeToStacksInWhichNodeExecuted.Find(InNode))
	{
		ExecutedStacks = *FoundStacks;
	}

	return ExecutedStacks;
}

void FPCGGraphExecutionInspection::ForEachExecutedNodeStack(TFunctionRef<bool(const UPCGNode*, const TSet<FNodeExecutedNotificationData>&)> Callback) const
{
	PCG::TSharedScopeLock ReadLock(NodeToStacksInWhichNodeExecutedLock);
	for (const auto& [NodeKey, NotificationDataSet] : NodeToStacksInWhichNodeExecuted)
	{
		if (const UPCGNode* Node = NodeKey.ResolveObjectPtr())
		{
			if (!Callback(Node, NotificationDataSet))
			{
				break;
			}
		}
	}
}

void FPCGGraphExecutionInspection::ForEachExecutedNodeStack(const UPCGNode* InNode, TFunctionRef<void(const TSet<FNodeExecutedNotificationData>&)> Callback) const
{
	PCG::TSharedScopeLock ReadLock(NodeToStacksInWhichNodeExecutedLock);
	if (const TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData>* FoundStacks = NodeToStacksInWhichNodeExecuted.Find(InNode))
	{
		Callback(*FoundStacks);
	}
}

uint64 FPCGGraphExecutionInspection::GetExecutedStacksGeneration() const
{
	PCG::TSharedScopeLock ReadLock(NodeToStacksInWhichNodeExecutedLock);
	return ExecutedStacksGeneration;
}

void FPCGGraphExecutionInspection::ClearExecutedNodeData()
{
	PCG::TUniqueScopeLock WriteLock(NodeToStacksInWhichNodeExecutedLock);
	if (!NodeToStacksInWhichNodeExecuted.IsEmpty())
	{
		NodeToStacksInWhichNodeExecuted.Reset();
		++ExecutedStacksGeneration;
	}
}

#if WITH_EDITOR

uint64 FPCGGraphExecutionInspection::GetNodeInactivePinMask(const UPCGNode* InNode, const FPCGStack& Stack) const
{
	PCG::TSharedScopeLock ReadLock(NodeToStackToInactivePinMaskLock);

	if (const TMap<const FPCGStack, uint64>* StackToMask = NodeToStackToInactivePinMask.Find(InNode))
	{
		if (const uint64* Mask = StackToMask->Find(Stack))
		{
			return *Mask;
		}
	}

	return 0;
}

void FPCGGraphExecutionInspection::NotifyNodeDynamicInactivePins(const UPCGNode* InNode, const FPCGStack* InStack, uint64 InactivePinBitmask) const
{
	if (!ensure(InStack && InNode))
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(NodeToStackToInactivePinMaskLock);
	TMap<const FPCGStack, uint64>& StackToInactivePinMask = NodeToStackToInactivePinMask.FindOrAdd(InNode);
	StackToInactivePinMask.FindOrAdd(*InStack) = InactivePinBitmask;
}

bool FPCGGraphExecutionInspection::WasNodeExecuted(const UPCGNode* InNode, const FPCGStack& Stack) const
{
	PCG::TSharedScopeLock ReadLock(NodeToStacksInWhichNodeExecutedLock);
	const TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData>* FoundNotifications = NodeToStacksInWhichNodeExecuted.Find(InNode);

	// Since the operator== & hash functions don't rely on the timer, we can just build a stub from the stack.
	FPCGGraphExecutionInspection::FNodeExecutedNotificationData NotificationStub(Stack, PCGUtils::FCallTime());
	return FoundNotifications && FoundNotifications->Contains(NotificationStub);
}

#endif // WITH_EDITOR

void FPCGGraphExecutionInspection::StoreInspectionData(const FPCGStack* InStack, const UPCGNode* InNode, const PCGUtils::FCallTime* InTimer, const FPCGDataCollection& InInputData, const FPCGDataCollection& InOutputData, bool bUsedCache)
{
	if (!InNode || !ensure(InStack))
	{
		return;
	}

	// Notify component that this task executed. Useful for editor visualization, and populates per-node timing info in build when profiling is enabled.
#if !WITH_EDITOR
	if (PCGProfilingLog::IsEnabled())
#endif
	{
		NotifyNodeExecuted(InNode, InStack, InTimer, bUsedCache);
	}

#if WITH_EDITOR
	if (!InOutputData.TaggedData.IsEmpty())
	{
		PCG::TUniqueScopeLock WriteLock(NodeToStacksThatProducedDataLock);

		NodeToStacksThatProducedData.FindOrAdd(InNode).Add(*InStack);
	}
	else
	{
		PCG::TUniqueScopeLock WriteLock(NodeToStacksThatProducedDataLock);

		if (TSet<FPCGStack>* Stacks = NodeToStacksThatProducedData.Find(InNode))
		{
			Stacks->Remove(*InStack);
		}
	}

	// Retain inspection data for nodes that are actively being viewport-edited or persistently marked for it.
	if (IsInspecting() || (InNode->GetSettingsInterface() && (InNode->GetSettingsInterface()->IsTemporaryManualEditingEnabled() || InNode->GetSettingsInterface()->IsMarkedForManualEditing())))
	{
		InInputData.MarkUsage(EPCGDataUsage::ComponentInspectionData);
		InOutputData.MarkUsage(EPCGDataUsage::ComponentInspectionData);

		auto StorePinInspectionDataFromNode = [](const FPCGStack* InStack, const TArray<TObjectPtr<UPCGPin>>& InPins, const FPCGDataCollection& InData, TMap<FPCGStack, TSharedPtr<FPCGDataCollection>>& InOutInspectionCache)
		{
			for (const UPCGPin* Pin : InPins)
			{
				FPCGStack Stack = *InStack;

				// Append the Node and Pin to the current Stack to uniquely identify each DataCollection
				TArray<FPCGStackFrame>& StackFrames = Stack.GetStackFramesMutable();
				StackFrames.Emplace(Pin);

				FPCGDataCollection PinDataCollection;
				InData.GetInputsAndCrcsByPin(Pin->Properties.Label, PinDataCollection.TaggedData, PinDataCollection.DataCrcs);

				// Implementation note: since static subgraphs actually are visited twice and the second time the input doesn't match the input pins, we don't clear the data.
				if (!PinDataCollection.TaggedData.IsEmpty())
				{
					if (TSharedPtr<FPCGDataCollection>* CollectionInCache = InOutInspectionCache.Find(Stack))
					{
						*(*CollectionInCache) += PinDataCollection;
					}
					else
					{
						InOutInspectionCache.Add(Stack, MakeShared<FPCGDataCollection>(MoveTemp(PinDataCollection)));
					}
				}
			}
		};

		auto StorePinInspectionData = [InStack, InNode, &StorePinInspectionDataFromNode](const TArray<TObjectPtr<UPCGPin>>& InPins, const FPCGDataCollection& InData, TMap<FPCGStack, TSharedPtr<FPCGDataCollection>>& InOutInspectionCache)
		{
			FPCGStack Stack = *InStack;

			// Append the Node (here) and Pin (in call) to the current Stack to uniquely identify each DataCollection
			TArray<FPCGStackFrame>& StackFrames = Stack.GetStackFramesMutable();
			StackFrames.Reserve(StackFrames.Num() + 2);
			StackFrames.Emplace(InNode);

			StorePinInspectionDataFromNode(&Stack, InPins, InData, InOutInspectionCache);
		};

		PCG::TUniqueScopeLock WriteLock(InspectionCacheLock);

		// Special case: if we have a static (embedded) subgraph, then the actual data inputs (not params) of the subgraph will be on the input node.
		// Considering we don't allow inspection on input pins of the input node, then we can move that data up the chain.
		if (InNode->GetSettings()->IsA<UPCGGraphInputOutputSettings>() && InStack->GetStackFrames().Num() > 2)
		{
			// We're expecting the last frame to be the graph
			// Then, if the graph was statically dispatched, it will be the subgraph node.
			// In the case of a dynamic subgraph or loop, it will be the loop index instead.
			FPCGStack StackToSubgraphNode = *InStack;
			TArray<FPCGStackFrame>& StackFrames = StackToSubgraphNode.GetStackFramesMutable();
			StackFrames.Pop();

			if (const UPCGSubgraphNode* Node = StackFrames.Last().GetObject_AnyThread<UPCGSubgraphNode>())
			{
				StorePinInspectionDataFromNode(&StackToSubgraphNode, Node->GetInputPins(), InInputData, InspectionCache);
			}
		}

		StorePinInspectionData(InNode->GetInputPins(), InInputData, InspectionCache);
		StorePinInspectionData(InNode->GetOutputPins(), InOutputData, InspectionCache);
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR

bool FPCGGraphExecutionInspection::InspectData(const FPCGStack& InStack, TFunctionRef<void(const FPCGDataCollection&)> InspectFunc) const
{
	PCG::TSharedScopeLock ReadLock(InspectionCacheLock);
	if (const TSharedPtr<FPCGDataCollection>* DataPtr = InspectionCache.Find(InStack))
	{
		if(const FPCGDataCollection* Data = DataPtr->Get())
		{
			InspectFunc(*Data);
			return true;
		}
	}

	return false;
}

TSharedPtr<FPCGInspectionData> FPCGGraphExecutionInspection::GetInspectionDataPtr(const FPCGStack& InStack) const
{
	TSharedPtr<FPCGInspectionData> InspectionDataPtr;

	PCG::TSharedScopeLock ReadLock(InspectionCacheLock);
	if (const TSharedPtr<FPCGDataCollection>* DataPtr = InspectionCache.Find(InStack); DataPtr && DataPtr->IsValid())
	{
		// Only allocate FPCGInspectionData if we found some data to return to avoid creating FGCObjects for nothing
		InspectionDataPtr = MakeShared<FPCGInspectionData>();
		InspectionDataPtr->Data = *DataPtr;
	}

	return InspectionDataPtr;
}

void FPCGGraphExecutionInspection::ClearInspectionData(bool bClearPerNodeExecutionData)
{
	{
		PCG::TUniqueScopeLock WriteLock(InspectionCacheLock);

		for (TPair<FPCGStack, TSharedPtr<FPCGDataCollection>>& Entry : InspectionCache)
		{
			Entry.Value->ClearUsage(EPCGDataUsage::ComponentInspectionData);
		}

		InspectionCache.Reset();
	}

	if (bClearPerNodeExecutionData)
	{
		{
			PCG::TUniqueScopeLock WriteLock(NodeToStacksThatProducedDataLock);
			NodeToStacksThatProducedData.Reset();
		}

		ClearExecutedNodeData();

		{
			PCG::TUniqueScopeLock WriteLock(NodeToStackToInactivePinMaskLock);
			NodeToStackToInactivePinMask.Reset();
		}

		{
			PCG::TUniqueScopeLock WriteLock(NodeToStacksTriggeringGPUTransfersLock);
			NodeToStacksTriggeringGPUUploads.Reset();
			NodeToStacksTriggeringGPUReadbacks.Reset();
		}

		{
			PCG::TUniqueScopeLock WriteLock(NodeToStacksWithOverridesAppliedLock);
			NodeToStacksWithOverridesApplied.Reset();
		}

		{
			PCG::TUniqueScopeLock WriteLock(ResolvedDeltasLock);
			ResolvedDeltas.Reset();
		}
	}
}

bool FPCGGraphExecutionInspection::HasNodeProducedData(const UPCGNode* InNode, const FPCGStack& Stack) const
{
	PCG::TSharedScopeLock ReadLock(NodeToStacksThatProducedDataLock);

	const TSet<FPCGStack>* StacksThatProducedData = NodeToStacksThatProducedData.Find(InNode);

	return StacksThatProducedData && StacksThatProducedData->Contains(Stack);
}

void FPCGGraphExecutionInspection::NotifyGPUToCPUReadback(const UPCGNode* InNode, const FPCGStack* InStack) const
{
	if (!ensure(InNode) || !ensure(InStack))
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(NodeToStacksTriggeringGPUTransfersLock);
	NodeToStacksTriggeringGPUReadbacks.FindOrAdd(InNode).Add(*InStack);
}

void FPCGGraphExecutionInspection::NotifyCPUToGPUUpload(const UPCGNode* InNode, const FPCGStack* InStack) const
{
	if (!ensure(InNode) || !ensure(InStack))
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(NodeToStacksTriggeringGPUTransfersLock);
	NodeToStacksTriggeringGPUUploads.FindOrAdd(InNode).Add(*InStack);
}

bool FPCGGraphExecutionInspection::DidNodeTriggerGPUToCPUReadback(const UPCGNode* InNode, const FPCGStack& Stack) const
{
	PCG::TSharedScopeLock ReadLock(NodeToStacksTriggeringGPUTransfersLock);
	const TSet<FPCGStack>* FoundStacks = NodeToStacksTriggeringGPUReadbacks.Find(InNode);
	return FoundStacks && FoundStacks->Contains(Stack);
}

bool FPCGGraphExecutionInspection::DidNodeTriggerCPUToGPUUpload(const UPCGNode* InNode, const FPCGStack& Stack) const
{
	PCG::TSharedScopeLock ReadLock(NodeToStacksTriggeringGPUTransfersLock);
	const TSet<FPCGStack>* FoundStacks = NodeToStacksTriggeringGPUUploads.Find(InNode);
	return FoundStacks && FoundStacks->Contains(Stack);
}

void FPCGGraphExecutionInspection::NotifyDataOverridesApplied(const UPCGNode* InNode, const FPCGStack* InStack) const
{
	if (!ensure(InNode) || !ensure(InStack))
	{
		return;
	}

	PCG::TUniqueScopeLock WriteLock(NodeToStacksWithOverridesAppliedLock);
	NodeToStacksWithOverridesApplied.FindOrAdd(InNode).Add(*InStack);
}

bool FPCGGraphExecutionInspection::NodeAppliedDataOverrides(const UPCGNode* InNode, const FPCGStack& Stack) const
{
	PCG::TSharedScopeLock ReadLock(NodeToStacksWithOverridesAppliedLock);
	const TSet<FPCGStack>* FoundStacks = NodeToStacksWithOverridesApplied.Find(InNode);
	return FoundStacks && FoundStacks->Contains(Stack);
}

void FPCGGraphExecutionInspection::NotifyDeltasResolved(const FPCGSourceDataStorageKey& InStorageKey, TArray<FPCGDeltaKey>&& InResolvedDeltas) const
{
	PCG::TUniqueScopeLock WriteLock(ResolvedDeltasLock);
	ResolvedDeltas.Add(InStorageKey, MoveTemp(InResolvedDeltas));
}

bool FPCGGraphExecutionInspection::IsDeltaResolved(const FPCGSourceDataStorageKey& InStorageKey, const FPCGDeltaKey& InDeltaKey) const
{
	PCG::TSharedScopeLock ReadLock(ResolvedDeltasLock);
	const TArray<FPCGDeltaKey>* Resolved = ResolvedDeltas.Find(InStorageKey);
	return Resolved && Resolved->Contains(InDeltaKey);
}

void FPCGGraphExecutionInspection::AddReferencedObjects(FReferenceCollector& Collector)
{
	PCG::TSharedScopeLock ReadLock(InspectionCacheLock);
	for (auto& Pair : InspectionCache)
	{
		Pair.Value->AddReferences(Collector);
	}
}

#endif // WITH_EDITOR

#endif // PCG_PROFILING_ENABLED
