// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGGraphExecutionRegistry.h"
#include "PCGModule.h"

int32 FPCGGraphExecutionRegistry::NextHandle = 0;

void IPCGGraphExecutionSourceProvider::BroadcastExecutionSourcesChanged()
{
	if (FPCGModule* PCGModule = FPCGModule::GetPCGModule())
	{
		PCGModule->GetGraphExecutionRegistry().BroadcastGraphExecutionSourcesChanged();
	}
}

TArray<FPCGGraphExecutionSourceDescriptor> FPCGGraphExecutionRegistry::GatherExecutionSources() const
{
	TArray<FPCGGraphExecutionSourceDescriptor> ExecutionSources;
	for (TSharedRef<IPCGGraphExecutionSourceProvider> Provider : RegisteredExecutionSourceProviders)
	{
		ExecutionSources.Append(Provider->GatherExecutionSources());
	}

	return ExecutionSources;
}

void FPCGGraphExecutionRegistry::Shutdown()
{
	RegisteredExecutionSourceProviders.Empty();
}

FPCGGraphExecutionSourceProviderHandle FPCGGraphExecutionRegistry::RegisterExecutionSourceProvider(TSharedRef<IPCGGraphExecutionSourceProvider> Provider)
{
	check(Provider->Handle == INDEX_NONE);
	const FPCGGraphExecutionSourceProviderHandle ProviderHandle = NextHandle++;
	Provider->Handle = ProviderHandle;
	RegisteredExecutionSourceProviders.Add(MoveTemp(Provider));
	return ProviderHandle;
}

void FPCGGraphExecutionRegistry::UnregisterExecutionSourceProvider(FPCGGraphExecutionSourceProviderHandle Handle)
{
	const int32 ProviderIndex = RegisteredExecutionSourceProviders.IndexOfByPredicate([Handle](const TSharedRef<IPCGGraphExecutionSourceProvider>& Provider) { return Provider->Handle == Handle; });
	if (ProviderIndex != INDEX_NONE)
	{
		RegisteredExecutionSourceProviders[ProviderIndex]->Handle = INDEX_NONE;
		RegisteredExecutionSourceProviders.RemoveAtSwap(ProviderIndex);
	}
}