// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGGraphExecutionStateInterface.h"

#include "PCGGraphExecutionInspection.h"
#include "PCGManagedResourceContainer.h"
#include "PCGModule.h"
#include "Editor/IPCGEditorModule.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Subsystems/PCGEngineSubsystem.h"
#include "Subsystems/PCGSubsystem.h"

#if WITH_EDITOR
#include "Misc/ITransactionObjectAnnotation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGraphExecutionStateInterface)

void IPCGGraphExecutionState::AddToManagedResources(UPCGManagedResource* InResource)
{
	ensure(false);
	UE_LOGF(LogPCG, Error, "Use FPCGManagedResourceContainerHelper with GetManagedResourceContainer instead.");
}

IPCGBaseSubsystem* IPCGGraphExecutionState::GetSubsystem() const
{
	IPCGBaseSubsystem* Subsystem = nullptr;
	
	if (UWorld* World = GetWorld())
	{
		Subsystem = UPCGSubsystem::GetInstance(World);
	}
	else
	{
		Subsystem = UPCGEngineSubsystem::Get();
	}

	return Subsystem;
}

FPCGGridDescriptor IPCGGraphExecutionState::GetGridDescriptor(uint32 InGridSize) const
{
	return FPCGGridDescriptor()
		.SetGridSize(InGridSize)
		.SetIs2DGrid(Use2DGrid())
		.SetIsRuntime(IsManagedByRuntimeGenSystem());
}

const FPCGRuntimeGenerationRadii& IPCGGraphExecutionState::GetGenerationRadii() const
{
	static const FPCGRuntimeGenerationRadii DefaultGenerationRadii;
	return DefaultGenerationRadii;
}

#if WITH_EDITOR

FPCGDynamicTrackingPriority IPCGGraphExecutionState::GetDynamicTrackingPriority() const
{
	ensureMsgf(false, TEXT("Please implement IPCGGraphExecutionState::GetDynamicTrackingPriority"));
	return FPCGDynamicTrackingPriority();
}

#elif !UE_BUILD_SHIPPING

// Deprecation (5.8)
const PCGUtils::FExtraCapture& IPCGGraphExecutionState::GetExtraCapture() const
{
	ensureMsgf(false, TEXT("PCG: GetExtraCapture() must now be overridden outside WITH_EDITOR when !UE_BUILD_SHIPPING as it is required for in-build profiling."));
	static PCGUtils::FExtraCapture FallbackCaptureObject;
	return FallbackCaptureObject;
}

// Deprecation (5.8)
PCGUtils::FExtraCapture& IPCGGraphExecutionState::GetExtraCapture()
{
	ensureMsgf(false, TEXT("PCG: GetExtraCapture() must now be overridden outside WITH_EDITOR when !UE_BUILD_SHIPPING as it is required for in-build profiling."));
	static PCGUtils::FExtraCapture FallbackCaptureObject;
	return FallbackCaptureObject;
}

// Deprecation (5.8)
const FPCGGraphExecutionInspection& IPCGGraphExecutionState::GetInspection() const
{
	ensureMsgf(false, TEXT("PCG: GetInspection() must now be overridden outside WITH_EDITOR when !UE_BUILD_SHIPPING as it is required for in-build profiling."));
	static FPCGGraphExecutionInspection FallbackInspectionObject;
	return FallbackInspectionObject;
}

// Deprecation (5.8)
FPCGGraphExecutionInspection& IPCGGraphExecutionState::GetInspection()
{
	ensureMsgf(false, TEXT("PCG: GetInspection() must now be overridden outside WITH_EDITOR when !UE_BUILD_SHIPPING as it is required for in-build profiling."));
	static FPCGGraphExecutionInspection FallbackInspectionObject;
	return FallbackInspectionObject;
}

#endif

//////////////////////////////////////////////////////////////////////

UPCGGraphExecutionSource::UPCGGraphExecutionSource(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//////////////////////////////////////////////////////////////////////

FPCGSoftGraphExecutionSource::FPCGSoftGraphExecutionSource(const TSoftObjectPtr<UObject>& InSoftObjectPtr)
	: SoftObjectPtr(InSoftObjectPtr)
{
}

FPCGSoftGraphExecutionSource::FPCGSoftGraphExecutionSource(IPCGGraphExecutionSource* InSource)
	: SoftObjectPtr(Cast<UObject>(InSource))
{
}

FPCGSoftGraphExecutionSource::FPCGSoftGraphExecutionSource(const IPCGGraphExecutionSource* InSource)
	: FPCGSoftGraphExecutionSource(const_cast<IPCGGraphExecutionSource*>(InSource))
{
}

FPCGSoftGraphExecutionSource::FPCGSoftGraphExecutionSource(const FPCGSoftGraphExecutionSource& Other)
{
	SoftObjectPtr = Other.SoftObjectPtr;
}

FPCGSoftGraphExecutionSource::FPCGSoftGraphExecutionSource(FPCGSoftGraphExecutionSource&& Other)
{
	SoftObjectPtr = std::move(Other.SoftObjectPtr);
}

FPCGSoftGraphExecutionSource& FPCGSoftGraphExecutionSource::operator=(const FPCGSoftGraphExecutionSource& Other)
{
	Reset();

	SoftObjectPtr = Other.SoftObjectPtr;
	return *this;
}

FPCGSoftGraphExecutionSource& FPCGSoftGraphExecutionSource::operator=(FPCGSoftGraphExecutionSource&& Other)
{
	Reset();

	SoftObjectPtr = std::move(Other.SoftObjectPtr);
	return *this;
}

FPCGSoftGraphExecutionSource& FPCGSoftGraphExecutionSource::operator=(const IPCGGraphExecutionSource* InSource)
{
	Reset();

	*this = FPCGSoftGraphExecutionSource(InSource);
	return *this;
}

IPCGGraphExecutionSource* FPCGSoftGraphExecutionSource::Get() const
{
	if (!CachedWeakPtr.IsValid())
	{
		FGCScopeGuard ScopeGuard;
		
		UObject* Object = SoftObjectPtr.Get();
		if (!Object || !Object->Implements<UPCGGraphExecutionSource>())
		{
			return nullptr;
		}

		IPCGGraphExecutionSource* InterfacePtr = CastChecked<IPCGGraphExecutionSource>(Object);
		
		{
			PCG::TScopeLock ScopeLock(Lock);
			if (!CachedWeakPtr.IsValid())
			{
				CachedWeakPtr = CastChecked<IPCGGraphExecutionSource>(Object);
			}
		}
	}

	return CachedWeakPtr.Get();
}

UObject* FPCGSoftGraphExecutionSource::GetObject() const
{
	return SoftObjectPtr.Get();
}

void FPCGSoftGraphExecutionSource::Reset()
{
	SoftObjectPtr = nullptr;
	if (CachedWeakPtr.IsValid())
	{
		PCG::TScopeLock ScopeLock(Lock);
		CachedWeakPtr.Reset();
	}
}

bool FPCGSoftGraphExecutionSource::operator==(const FPCGSoftGraphExecutionSource& Other) const
{
	return Other.SoftObjectPtr == SoftObjectPtr;
}

int32 GetTypeHash(const FPCGSoftGraphExecutionSource& This)
{
	return GetTypeHash(This.SoftObjectPtr);
}

#if WITH_EDITOR
namespace PCG::Transaction
{
	// Carries an FPCGSourceDataContainer snapshot through the transaction buffer.
	// Property diffing can't capture FSharedStruct payloads since they're shared-pointer references to live state.
	class FPCGExecutionSourceTransactionAnnotation : public ITransactionObjectAnnotation
	{
	public:
		FPCGExecutionSourceTransactionAnnotation() = default;
		explicit FPCGExecutionSourceTransactionAnnotation(const FPCGSourceDataContainer& InSnapshot) : Snapshot(InSnapshot) {}

		virtual void AddReferencedObjects(FReferenceCollector& Collector) override
		{
			Snapshot.AddStructReferencedObjects(Collector);
		}

		virtual void Serialize(FArchive& Ar) override
		{
			Snapshot.Serialize(Ar);
		}

		const FPCGSourceDataContainer& GetSnapshot() const { return Snapshot; }

	private:
		FPCGSourceDataContainer Snapshot;
	};

	TSharedPtr<ITransactionObjectAnnotation> CreateAnnotation()
	{
		return MakeShared<FPCGExecutionSourceTransactionAnnotation>();
	}

	TSharedPtr<ITransactionObjectAnnotation> CreateAnnotation(const IPCGGraphExecutionSource& Source)
	{
		const FPCGSourceDataContainer* Container = Source.GetExecutionState().GetSourceDataContainer();
		if (!Container)
		{
			return nullptr;
		}

		return MakeShared<FPCGExecutionSourceTransactionAnnotation>(*Container);
	}

	void RestoreFromAnnotation(IPCGGraphExecutionSource& Source, const TSharedPtr<ITransactionObjectAnnotation>& Annotation)
	{
		const TSharedPtr<FPCGExecutionSourceTransactionAnnotation> PCGAnnotation = StaticCastSharedPtr<FPCGExecutionSourceTransactionAnnotation>(Annotation);
		if (!PCGAnnotation)
		{
			return;
		}

		if (FPCGSourceDataContainer* Container = Source.GetExecutionState().GetSourceDataContainer())
		{
			*Container = PCGAnnotation->GetSnapshot();
		}
	}
} // namespace PCG::Transaction
#endif // WITH_EDITOR
