// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGInputOutputSettings.h"

#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGPin.h"
#include "Graph/PCGStackContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGInputOutputSettings)

#define LOCTEXT_NAMESPACE "PCGInputOutputElement"

namespace PCGInputOutputPrivate
{
	struct FLabelAndTooltip
	{
		FLabelAndTooltip(FName InLabel, FText InTooltip = FText::GetEmpty())
			: Label(InLabel), Tooltip(InTooltip)
		{
		}

		FName Label;
		FText Tooltip;
	};

	TArray<FLabelAndTooltip>& GetStaticInLabels()
	{
		static TArray<FLabelAndTooltip> StaticInLabels =
		{
			{
				PCGPinConstants::DefaultInputLabel,
				LOCTEXT("InputOutputInPinTooltip", "Default input pin.")
			}
		};

		return StaticInLabels;
	}

	TArray<FLabelAndTooltip>& GetStaticOutLabels()
	{
		static TArray<FLabelAndTooltip> StaticOutLabels =
		{
			{
				PCGPinConstants::DefaultOutputLabel
			}
		};

		return StaticOutLabels;
	}
}

bool FPCGInputOutputElement::ExecuteInternal(FPCGContext* Context) const
{
	// Essentially a pass-through element
	Context->OutputData = Context->InputData;
	return true;
}

bool FPCGInputOutputElement::SupportsGPUResidentData(FPCGContext* InContext) const
{
	check(InContext);

	const UPCGGraphInputOutputSettings* Settings = InContext->GetOriginalSettings<UPCGGraphInputOutputSettings>();
	const FPCGStack* Stack = InContext->GetStack();
	if (!Settings || !ensure(Stack))
	{
		return false;
	}

	if (Settings->IsInput())
	{
		return true;
	}
	else
	{
		// The output node of subgraphs can return GPU data proxies, but the top graph output must read back.
		return !Stack->IsCurrentFrameInRootGraph();
	}
}

void UPCGGraphInputOutputSettings::PostLoad()
{
	Super::PostLoad();

	if (!PinLabels_DEPRECATED.IsEmpty())
	{
		for (const FName& PinLabel : PinLabels_DEPRECATED)
		{
			Pins.Emplace(PinLabel);
		}

		PinLabels_DEPRECATED.Reset();
	}

	if (!CustomPins_DEPRECATED.IsEmpty())
	{
		Pins.Append(CustomPins_DEPRECATED);
		CustomPins_DEPRECATED.Empty();
	}
}

void UPCGGraphInputOutputSettings::SetInput(bool bInIsInput)
{
	bIsInput = bInIsInput;

	if (!bHasAddedDefaultPin)
	{
		bHasAddedDefaultPin = true;

		if (Pins.IsEmpty())
		{
			const EPCGDataType DefaultPinDataType = bIsInput ? EPCGDataType::Spatial : EPCGDataType::Any;
			const TArray<PCGInputOutputPrivate::FLabelAndTooltip>& StaticLabels = bIsInput ? PCGInputOutputPrivate::GetStaticInLabels() : PCGInputOutputPrivate::GetStaticOutLabels();

			Algo::Transform(StaticLabels, Pins, [DefaultPinDataType](const PCGInputOutputPrivate::FLabelAndTooltip& InLabelAndTooltip) {
				FPCGPinProperties Res = FPCGPinProperties(InLabelAndTooltip.Label, DefaultPinDataType, /*bMultiConnections=*/true, /*bMultiData=*/true, InLabelAndTooltip.Tooltip);
				Res.SetAdvancedPin();
				return Res;
			});
		}
	}
}

#if WITH_EDITOR
void UPCGGraphInputOutputSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	if (DataVersion < FPCGCustomVersion::UpdateInputOutputNodesDefaults)
	{
		// At this point we should already have the default in/out, done in the SetInput, so we don't need to consider the default in/out here.
		check(Pins.Num() > 0);

		// Add default pins to the node if and only if any of its default pins were connected.
		TArray<FPCGPinProperties> DeprecatedPropertiesToAdd;
		TArray<TObjectPtr<UPCGPin>>& PreviousPins = (bIsInput ? OutputPins : InputPins);
		// Basically, any previous pins that's connected and also not part of the current Pins array is an old pin
		for (UPCGPin* PreviousPin : PreviousPins)
		{
			if (!PreviousPin || !PreviousPin->IsConnected())
			{
				continue;
			}

			// If the previous pin isn't found, just copy directly
			if (!Pins.FindByPredicate([PreviousPin](const FPCGPinProperties& PinProperty) { return PinProperty.Label == PreviousPin->Properties.Label; }))
			{
				DeprecatedPropertiesToAdd.Add(PreviousPin->Properties);
			}
		}

		if (!DeprecatedPropertiesToAdd.IsEmpty())
		{
			Pins.Insert(MoveTemp(DeprecatedPropertiesToAdd), 1);
		}
	}
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGraphInputOutputSettings::DefaultPinProperties(bool bInvisiblePin) const
{
	// It is important for serialization that this is not modified, or it could break existing graphs.
	TArray<FPCGPinProperties> PinProperties;
	const EPCGDataType DefaultPinDataType = bIsInput ? EPCGDataType::Spatial : EPCGDataType::Any;
	const TArray<PCGInputOutputPrivate::FLabelAndTooltip>& StaticLabels = bIsInput ? PCGInputOutputPrivate::GetStaticInLabels() : PCGInputOutputPrivate::GetStaticOutLabels();

	Algo::Transform(StaticLabels, PinProperties, [bInvisiblePin, DefaultPinDataType](const PCGInputOutputPrivate::FLabelAndTooltip& InLabelAndTooltip) {
		FPCGPinProperties Res = FPCGPinProperties(InLabelAndTooltip.Label, DefaultPinDataType, /*bMultiConnections=*/true, /*bMultiData=*/true, InLabelAndTooltip.Tooltip);
		Res.bInvisiblePin = bInvisiblePin;
		Res.SetAdvancedPin();
		return Res;
	});

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGraphInputOutputSettings::DefaultInputPinProperties() const
{
	return DefaultPinProperties(/*bInvisiblePin=*/bIsInput);
}

TArray<FPCGPinProperties> UPCGGraphInputOutputSettings::DefaultOutputPinProperties() const
{
	return DefaultPinProperties(/*bInvisiblePin=*/!bIsInput);
}

const FPCGPinProperties& UPCGGraphInputOutputSettings::AddPin(const FPCGPinProperties& NewPinProperties)
{
	Modify();
	int32 Index = Pins.Add(NewPinProperties);
	FixPinProperties();
	return Pins[Index];
}

#if WITH_EDITOR
void UPCGGraphInputOutputSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		// Label has changed if we have modified "CustomPins" array
		bool bLabelChanged = PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGGraphInputOutputSettings, Pins);

		if (!bLabelChanged)
		{
			// Or might has changed if an element of the array "CustomPin" was modified.
			bLabelChanged = PropertyChangedEvent.MemberProperty &&
				PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGGraphInputOutputSettings, Pins) &&
				PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, Label);
		}

		if (bLabelChanged)
		{
			FixPinProperties();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

EPCGChangeType UPCGGraphInputOutputSettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (bIsInput)
	{
		// If pins were removed or the required-ness of a pin has changed, this needs to be a structural change.
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraphInputOutputSettings, Pins) ||
			(MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraphInputOutputSettings, Pins) &&
			PropertyName == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, PinStatus)))
		{
			ChangeType |= EPCGChangeType::Structural;
		}
	}

	// If a label changed, we need to recompile, as the tasks inputs are gathered by pin label.
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPCGGraphInputOutputSettings, Pins) &&
		PropertyName == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, Label))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGraphInputOutputSettings::InputPinProperties() const
{
	if (bIsInput)
	{
		TArray<FPCGPinProperties> InvisiblePins = Pins;
		for (FPCGPinProperties& Pin : InvisiblePins)
		{
			Pin.bInvisiblePin = true;
		}

		return InvisiblePins;
	}
	else
	{
		return Pins;
	}
}

TArray<FPCGPinProperties> UPCGGraphInputOutputSettings::OutputPinProperties() const
{
	if (bIsInput)
	{
		return Pins;
	}
	else
	{
		TArray<FPCGPinProperties> InvisiblePins = Pins;
		for (FPCGPinProperties& Pin : InvisiblePins)
		{
			Pin.bInvisiblePin = true;
		}

		return InvisiblePins;
	}
}

void UPCGGraphInputOutputSettings::FixPinProperties()
{
	// No need to fix if we have no pins
	if (Pins.IsEmpty())
	{
		return;
	}

	TSet<FName> AllLabels;
	bool bWasModified = false;

	for (FPCGPinProperties& PinProperties : Pins)
	{
		// Avoid "None" pin label
		if (PinProperties.Label == NAME_None)
		{
			if (!bWasModified)
			{
				bWasModified = true;
				Modify();
			}

			PinProperties.Label = PCGInputOutputConstants::DefaultNewCustomPinName;
		}

		uint32 Count = 1;
		FString OriginalLabel = PinProperties.Label.ToString();

		while (AllLabels.Contains(PinProperties.Label))
		{
			if (!bWasModified)
			{
				bWasModified = true;
				Modify();
			}

			PinProperties.Label = FName(FString::Printf(TEXT("%s%d"), *OriginalLabel, Count++));
		}

		AllLabels.Emplace(PinProperties.Label);
	}
}

#undef LOCTEXT_NAMESPACE
