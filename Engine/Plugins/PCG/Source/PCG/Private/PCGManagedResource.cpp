// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGManagedResource.h"

#include "PCGComponent.h"
#include "PCGModule.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Algo/NoneOf.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Engine/Level.h"
#include "UObject/Package.h"
#include "Utils/PCGGeneratedResourcesLogging.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGManagedResource)

static TAutoConsoleVariable<bool> CVarForceReleaseResourcesOnGenerate(
	TEXT("pcg.ForceReleaseResourcesOnGenerate"),
	false,
	TEXT("Purges all tracked generated resources on generate"));

static TAutoConsoleVariable<bool> CVarLoadManagedActorsForCleanup(
	TEXT("pcg.LoadManagedActorsForCleanup"),
	true,
	TEXT("When enabled unloaded actors will get loaded before clean up"));

bool FPCGMoveResourceParams::Accepts(const UObject* InObject) const
{
	if (!InObject) // Never accept nullptrs
	{
		return false;
	}
	else if (RequiredTags.IsEmpty() && ExcludedTags.IsEmpty()) // Always accept if there are no requirements
	{
		return true;
	}
	else if (const AActor* Actor = Cast<AActor>(InObject))
	{
		auto TestActorTags = [Actor](FName Tag) { return Actor->Tags.Contains(Tag); };

		return (RequiredTags.IsEmpty() || Algo::AllOf(RequiredTags, TestActorTags)) &&
			(ExcludedTags.IsEmpty() || Algo::NoneOf(ExcludedTags, TestActorTags));
	}
	else if (const UActorComponent* Component = Cast<UActorComponent>(InObject))
	{
		auto TestComponentTags = [Component](FName Tag) { return Component->ComponentHasTag(Tag); };

		return (RequiredTags.IsEmpty() || Algo::AllOf(RequiredTags, TestComponentTags)) &&
			(ExcludedTags.IsEmpty() || Algo::NoneOf(ExcludedTags, TestComponentTags));
	}
	else
	{
		// Unknown object type, assume we accept it.
		return true;
	}
}

bool FPCGMoveResourceParams::Accepts(const FPCGTaggedData& InData) const
{
	if (!InData.Data)
	{
		return false;
	}
	else if (RequiredTags.IsEmpty() && ExcludedTags.IsEmpty())
	{
		return true;
	}
	else
	{
		auto TestDataTags = [&InData](FName Tag) { return InData.Tags.Contains(Tag.ToString()); };

		return (RequiredTags.IsEmpty() || Algo::AllOf(RequiredTags, TestDataTags)) &&
			(ExcludedTags.IsEmpty() || Algo::NoneOf(ExcludedTags, TestDataTags));
	}
}

void UPCGManagedResource::PostApplyToComponent()
{
	// Nothing - apply to component should already properly remap most of everything we need to do
	// In the case of actors, this means we'll keep the references to the actors as-is
	// In the cas of components, the remapping will not be needed either since they won't be affected this way.
}

// By default, if it is not a hard release, we mark the resource unused.
bool UPCGManagedResource::Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& /*OutActorsToDelete*/)
{
	bIsMarkedUnused = true;
	return bHardRelease;
}

bool UPCGManagedResource::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	if (bIsMarkedUnused)
	{
		Release(true, OutActorsToDelete);
		return true;
	}

	return false;
}

bool UPCGManagedResource::CanBeUsed() const
{
#if WITH_EDITOR
	return !bMarkedTransientOnLoad;
#else
	return true;
#endif
}

void UPCGManagedResource::MarkAsUsed()
{
	ensure(CanBeUsed()); 
	bIsMarkedUnused = false;
	Crc = FPCGCrc();
}

bool UPCGManagedResource::DebugForcePurgeAllResourcesOnGenerate()
{
	return CVarForceReleaseResourcesOnGenerate.GetValueOnAnyThread();
}

#if WITH_EDITOR
void UPCGManagedResource::ChangeTransientState(EPCGEditorDirtyMode NewEditingMode)
{
	// We expect that our resource isn't bMarkedTransientOnLoad when setting the editor mode to preview because this would mean potential data loss
	ensure(!bMarkedTransientOnLoad || NewEditingMode != EPCGEditorDirtyMode::Preview);

	// Any change in the transient state resets the transient state that was set on load, regardless of the bNowTransient flag
	bMarkedTransientOnLoad = false;

	bIsPreview = (NewEditingMode == EPCGEditorDirtyMode::Preview);
}
#endif // WITH_EDITOR

bool UPCGManagedResource::MoveResourceToNewActor(AActor* NewActor, const AActor* ExpectedPreviousOwner)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return MoveResourceToNewActor(NewActor);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UPCGManagedResource::MoveResource(const FPCGMoveResourceParams& Params)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return MoveResourceToNewActor(Params.GetTarget<AActor>(), Params.GetExpectedPreviousOwner<AActor>());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FPCGManagedActorLoadingScope::FPCGManagedActorLoadingScope()
{
#if WITH_EDITOR
	bShouldLoadActors = ensure(!IsGarbageCollecting()) && CVarLoadManagedActorsForCleanup.GetValueOnAnyThread();
#endif
}

FPCGManagedActorLoadingScope::FPCGManagedActorLoadingScope(bool bInShouldLoadActors)
{
#if WITH_EDITOR
	bShouldLoadActors = ensure(!IsGarbageCollecting()) && bInShouldLoadActors;
#endif
}

void FPCGManagedActorLoadingScope::AddResources(IPCGGraphExecutionSource* InExecutionSource, const TArray<UPCGManagedResource*>& InManagedResources)
{
#if WITH_EDITOR
	if (!bShouldLoadActors)
	{
		return;
	}

	if (!InExecutionSource || !InExecutionSource->GetExecutionState().GetWorld() || InExecutionSource->GetExecutionState().GetWorld()->IsGameWorld())
	{
		return;
	}

	// @todo_pcg: check that this is valid way of getting WP here
	UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(InExecutionSource->GetExecutionState().GetWorld());
	if (!WorldPartition)
	{
		return;
	}

	for (UPCGManagedResource* ManagedResource : InManagedResources)
	{
		if (UPCGManagedActors* ManagedActors = Cast<UPCGManagedActors>(ManagedResource))
		{
			for (const TSoftObjectPtr<AActor>& ManagedActor : ManagedActors->GetConstGeneratedActors())
			{
				if (const FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstanceByPath(ManagedActor.ToSoftObjectPath()))
				{
					LoadedActors.Add(FWorldPartitionReference(WorldPartition, ActorDescInstance->GetGuid()));
				}
			}
		}
	}
#endif // WITH_EDITOR 
}


void UPCGManagedActors::PostEditImport()
{
	// In this case, the managed actors won't be copied along the actor/component,
	// So we just have to "forget" the actors, leaving the ownership to the original actor only.
	Super::PostEditImport();
	GeneratedActorsArray.Reset();
}

void UPCGManagedActors::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!GeneratedActors.IsEmpty())
	{
		GeneratedActorsArray = GeneratedActors.Array();
		GeneratedActors.Reset();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

bool UPCGManagedActors::Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGManagedActors::Release);

	const bool bShouldDeleteActors = bHardRelease || !bSupportsReset;

	if (!Super::Release(bShouldDeleteActors, OutActorsToDelete))
	{
		PCGGeneratedResourcesLogging::LogManagedActorsRelease(this, GeneratedActorsArray, bHardRelease, /*bOnlyMarkedForCleanup=*/true);

		// Mark actors as potentially-to-be-cleaned-up
		for (TSoftObjectPtr<AActor> GeneratedActor : GeneratedActorsArray)
		{
			if (GeneratedActor.IsValid())
			{
				GeneratedActor->Tags.Add(PCGHelpers::MarkedForCleanupPCGTag);
			}
		}

		return false;
	}

#if WITH_EDITOR
	if (bMarkedTransientOnLoad)
	{
		// Here, instead of adding the actors to be deleted (which has the side effect of potentially emptying the package, which leads to its deletion, we will hide the actors instead
		for (TSoftObjectPtr<AActor> GeneratedActor : GeneratedActorsArray)
		{
			// Hide actor if it is loaded
			if (AActor* Actor = GeneratedActor.Get())
			{
				Actor->SetIsTemporarilyHiddenInEditor(true);
				Actor->SetHidden(true);
				Actor->SetActorEnableCollision(false);
				Actor->bIgnoreInPIE = true;
			}
		}
	}
	else
#endif // WITH_EDITOR
	{
		OutActorsToDelete.Append(GeneratedActorsArray);
	}

	PCGGeneratedResourcesLogging::LogManagedActorsRelease(this, GeneratedActorsArray, bHardRelease, /*bOnlyMarkedForCleanup=*/false);

	// Cleanup recursively
	TInlineComponentArray<UPCGComponent*, 1> ComponentsToCleanup;

	for (TSoftObjectPtr<AActor> GeneratedActor : GeneratedActorsArray)
	{
		if (GeneratedActor.IsValid())
		{
			GeneratedActor.Get()->GetComponents(ComponentsToCleanup);

			for (UPCGComponent* Component : ComponentsToCleanup)
			{
				// It is more complicated to handled a non-immediate cleanup when doing it recursively in the managed actors.
				// Do it all immediate then.
				Component->CleanupLocalImmediate(/*bRemoveComponents=*/bHardRelease);
			}

			ComponentsToCleanup.Reset();
		}
	}

#if WITH_EDITOR
	if (!bMarkedTransientOnLoad)
#endif
	{
		GeneratedActorsArray.Reset();
	}
	
	return true;
}

bool UPCGManagedActors::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	return Super::ReleaseIfUnused(OutActorsToDelete) || GeneratedActorsArray.IsEmpty();
}

bool UPCGManagedActors::MoveResourceToNewActor(AActor* NewActor)
{
	FPCGMoveResourceParams Params
	{
		.Target = NewActor
	};

	return MoveResource(Params);
}

bool UPCGManagedActors::MoveResource(const FPCGMoveResourceParams& Params)
{
	AActor* TargetActor = Params.GetTarget<AActor>();
	check(TargetActor);
	bool bMovedResource = false;

	for(int ActorIndex = GeneratedActorsArray.Num() - 1; ActorIndex >=0; --ActorIndex)
	{
		TSoftObjectPtr<AActor> Actor = GeneratedActorsArray[ActorIndex];

		if (!Actor.IsValid() || !Params.Accepts(Actor.Get()))
		{
			continue;
		}

		bMovedResource = true;

		const bool bWasAttached = (Actor->GetAttachParentActor() != nullptr);

		if (bWasAttached)
		{
			Actor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
			Actor->SetOwner(nullptr);
			Actor->AttachToActor(TargetActor, FAttachmentTransformRules::KeepWorldTransform);
		}

		GeneratedActorsArray.RemoveAtSwap(ActorIndex);
	}

	return bMovedResource;
}

void UPCGManagedActors::MarkAsUsed()
{
	Super::MarkAsUsed();
	// Technically we don't ever have to "use" a preexisting managed actor resource, but this is to be consistent with the other implementations
	ensure(0);
}

void UPCGManagedActors::MarkAsReused()
{
	Super::MarkAsReused();

	for (TSoftObjectPtr<AActor> GeneratedActor : GeneratedActorsArray)
	{
		if (GeneratedActor.IsValid())
		{
			GeneratedActor->Tags.Remove(PCGHelpers::MarkedForCleanupPCGTag);
		}
	}
}

bool UPCGManagedActors::IsManaging(const UObject* InObject) const
{
	if (!InObject || !InObject->IsA<AActor>())
	{
		return false;
	}

	return Algo::AnyOf(GeneratedActorsArray, [InObject](const TSoftObjectPtr<AActor>& SoftGeneratedActor)
		{
			const AActor* GeneratedActor = SoftGeneratedActor.Get();
			return GeneratedActor && GeneratedActor == InObject;
		});
}

#if WITH_EDITOR
void UPCGManagedActors::ChangeTransientState(EPCGEditorDirtyMode NewEditingMode)
{
	const bool bNowTransient = (NewEditingMode == EPCGEditorDirtyMode::Preview);

	for (TSoftObjectPtr<AActor> GeneratedActorPtr : GeneratedActorsArray)
	{
		// Make sure to load if needed because we need to affect the actors regardless of the current WP state
		if (AActor* GeneratedActor = GeneratedActorPtr.Get())
		{
			const bool bWasTransient = GeneratedActor->HasAnyFlags(RF_Transient);

			if (bNowTransient != bWasTransient)
			{
				UPackage* CurrentPackage = GeneratedActor->GetExternalPackage();
				if (bNowTransient)
				{
					GeneratedActor->SetFlags(RF_Transient);
					if (CurrentPackage)
					{
						CurrentPackage->SetDirtyFlag(true);

						// Disable external packaging first because AActor::SetPackageExternal will early out if package is already external.
						// In this case we want to change the external package so we remove the previous one first.
						GeneratedActor->SetPackageExternal(/*bExternal=*/false, /*bShouldDirty=*/false);

						UPackage* PreviewPackage = UPCGActorHelpers::CreatePreviewPackage(GeneratedActor->GetLevel(), GeneratedActor->GetName());
						ensure(PreviewPackage);

						// Use the preview package.
						GeneratedActor->SetPackageExternal(/*bExternal=*/true, /*bShouldDirty=*/false, PreviewPackage);
					}
				}
				else
				{
					GeneratedActor->ClearFlags(RF_Transient);
					if (CurrentPackage)
					{
						// Disable external packaging first because AActor::SetPackageExternal will early out if package is already external.
						// In this case we want to change the external package so we remove the previous one first.
						GeneratedActor->SetPackageExternal(/*bExternal=*/false, /*bShouldDirty=*/false);

						// Use the default external package for this actor.
						GeneratedActor->SetPackageExternal(/*bExternal=*/true, /*bShouldDirty=*/false);
					}
				}
			}

			// If the actor had PCG components, propagate this downward
			{
				TInlineComponentArray<UPCGComponent*, 4> PCGComponents;
				GeneratedActor->GetComponents(PCGComponents);

				for (UPCGComponent* PCGComponent : PCGComponents)
				{
					PCGComponent->SetEditingMode(/*CurrentEditingMode=*/NewEditingMode, /*SerializedEditingMode=*/NewEditingMode);
					PCGComponent->ChangeTransientState(NewEditingMode);
				}
			}

			if (bNowTransient != bWasTransient)
			{
				if (!bNowTransient)
				{
					ForEachObjectWithOuter(GeneratedActor, [bNowTransient](UObject* Object)
					{
						if (bNowTransient)
						{
							Object->SetFlags(RF_Transient);
						}
						else
						{
							Object->ClearFlags(RF_Transient);
						}
					});
				}
			}
		}
	}

	Super::ChangeTransientState(NewEditingMode);
}
#endif // WITH_EDITOR

void UPCGManagedComponentBase::PostEditImport()
{
	Super::PostEditImport();

	// Rehook components from the original to the locally duplicated components
	UPCGComponent* OwningComponent = Cast<UPCGComponent>(GetOuter());
	AActor* Actor = OwningComponent ? OwningComponent->GetOwner() : nullptr;

	if (!Actor)
	{
		// Somewhat irrelevant case, if we don't have an actor or a component, there's not a lot we can do.
		ForgetComponents();
	}
	else if(GetComponentsCount() > 0)
	{
		TInlineComponentArray<UActorComponent*, 64> ActorComponents;
		Actor->GetComponents(ActorComponents);

		TArrayView<TSoftObjectPtr<UActorComponent>> GeneratedComponents = GetComponentsArray();
		for (int ComponentIndex = GeneratedComponents.Num() - 1; ComponentIndex >= 0; --ComponentIndex)
		{
			TSoftObjectPtr<UActorComponent> GeneratedComponent = GeneratedComponents[ComponentIndex];

			// Do not erase if we haven't yet imported the properties (value is explicitely null)
			if (GeneratedComponent.IsNull())
			{
				continue;
			}

			// If Generated component is part of the current actor components, we keep it
			if (UActorComponent* GeneratedComponentPtr = GeneratedComponent.Get(); GeneratedComponentPtr && ActorComponents.Contains(GeneratedComponentPtr))
			{
				continue;
			}

			// Forget components that are not owned by this actor
			ForgetComponent(ComponentIndex);
		}
	}
}

#if WITH_EDITOR
void UPCGManagedComponentBase::HideComponents()
{
	const int32 ComponentCount = GetComponentsCount();
	for (int32 ComponentIndex = 0; ComponentIndex < ComponentCount; ++ComponentIndex)
	{
		HideComponent(ComponentIndex);
	}
}

void UPCGManagedComponentBase::HideComponent(int32 ComponentIndex)
{
	// Default implementation to be backward compatible
	HideComponent();
}

void UPCGManagedComponent::HideComponent()
{
	if (GeneratedComponent.IsValid())
	{
		GeneratedComponent->UnregisterComponent();

		// If transient on load, hide for PIE
		if (bMarkedTransientOnLoad)
		{
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(GeneratedComponent.Get()))
			{
				SceneComponent->bHiddenInGame = true;
			}
		}
	}
}

void UPCGManagedComponentList::HideComponent(int32 ComponentIndex)
{
	if (GeneratedComponents[ComponentIndex].IsValid())
	{
		GeneratedComponents[ComponentIndex]->UnregisterComponent();

		// If transient on load, hide for PIE
		if (bMarkedTransientOnLoad)
		{
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(GeneratedComponents[ComponentIndex].Get()))
			{
				SceneComponent->bHiddenInGame = true;
			}
		}
	}
}
#endif // WITH_EDITOR

void UPCGManagedComponentBase::ForgetComponents()
{
	const int32 ComponentCount = GetComponentsCount();
	for (int32 ComponentIndex = ComponentCount - 1; ComponentIndex >= 0; --ComponentIndex)
	{
		ForgetComponent(ComponentIndex);
	}
}

void UPCGManagedComponentBase::ForgetComponent(int32 ComponentIndex)
{
	// Default implementation to be backward compatible
	ForgetComponent();
}

void UPCGManagedComponentList::ForgetComponent(int32 ComponentIndex)
{
	GeneratedComponents.RemoveAtSwap(ComponentIndex);
}

void UPCGManagedComponentBase::ResetComponents()
{
	const int32 ComponentCount = GetComponentsCount();
	for (int32 ComponentIndex = 0; ComponentIndex < ComponentCount; ++ComponentIndex)
	{
		ResetComponent(ComponentIndex);
	}
}

void UPCGManagedComponentBase::ResetComponent(int32 ComponentIndex)
{
	// Default implementation to be backward compatible
	ResetComponent();
}

bool UPCGManagedComponent::Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& /*OutActorsToDelete*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGManagedComponent::Release);

	const bool bSupportsComponentReset = SupportsComponentReset();
	bool bDeleteComponent = bHardRelease || !bSupportsComponentReset;

	if (GeneratedComponent.IsValid())
	{
#if WITH_EDITOR
		if (bMarkedTransientOnLoad)
		{
			PCGGeneratedResourcesLogging::LogManagedComponentHidden(this);

			HideComponent();
			bIsMarkedUnused = true;
		}
		else
#endif // WITH_EDITOR
		{
			if (bDeleteComponent)
			{
				PCGGeneratedResourcesLogging::LogManagedResourceHardRelease(this);

				GeneratedComponent->DestroyComponent();
				ForgetComponent();
			}
			else
			{
				PCGGeneratedResourcesLogging::LogManagedResourceSoftRelease(this);

				// We can only mark it unused if we can reset the component.
				bIsMarkedUnused = true;
				GeneratedComponent->ComponentTags.Add(PCGHelpers::MarkedForCleanupPCGTag);
			}
		}
	}
	else
	{
		PCGGeneratedResourcesLogging::LogManagedComponentDeleteNull(this);

		// Dead component reference - clear it out.
		bDeleteComponent = true;
	}

	return bDeleteComponent;
}

bool UPCGManagedComponentList::Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& /*OutActorsToDelete*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGManagedComponentList::Release);

	const bool bSupportsComponentReset = SupportsComponentReset();
	bool bDeleteComponent = bHardRelease || !bSupportsComponentReset;

	// Start by removing all dead components from the array
	for (int32 ComponentIndex = GeneratedComponents.Num() - 1; ComponentIndex >= 0; --ComponentIndex)
	{
		if (!GeneratedComponents[ComponentIndex].IsValid())
		{
			GeneratedComponents.RemoveAtSwap(ComponentIndex);
		}
	}

	// Nothing left - this resource can be released
	if (GeneratedComponents.IsEmpty())
	{
		return true;
	}

#if WITH_EDITOR
	if (bMarkedTransientOnLoad)
	{
		PCGGeneratedResourcesLogging::LogManagedComponentHidden(this);
		HideComponents();
		bIsMarkedUnused = true;
	}
	else
#endif
	{
		if (bDeleteComponent)
		{
			PCGGeneratedResourcesLogging::LogManagedResourceHardRelease(this);
			for (TSoftObjectPtr<UActorComponent> GeneratedComponent : GeneratedComponents)
			{
				GeneratedComponent->DestroyComponent();
			}

			ForgetComponents();
		}
		else
		{
			PCGGeneratedResourcesLogging::LogManagedResourceSoftRelease(this);
			bIsMarkedUnused = true;

			for (TSoftObjectPtr<UActorComponent> GeneratedComponent : GeneratedComponents)
			{
				GeneratedComponent->ComponentTags.Add(PCGHelpers::MarkedForCleanupPCGTag);
			}
		}
	}

	return bDeleteComponent;
}

bool UPCGManagedComponent::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	return Super::ReleaseIfUnused(OutActorsToDelete) || !GeneratedComponent.IsValid();
}

bool UPCGManagedComponentList::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	if (Super::ReleaseIfUnused(OutActorsToDelete))
	{
		return true;
	}

	// Start by removing all dead components from the array
	for (int32 ComponentIndex = GeneratedComponents.Num() - 1; ComponentIndex >= 0; --ComponentIndex)
	{
		if (!GeneratedComponents[ComponentIndex].IsValid())
		{
			GeneratedComponents.RemoveAtSwap(ComponentIndex);
		}
	}

	// Nothing left - this resource can be released
	return GeneratedComponents.IsEmpty();
}

bool UPCGManagedComponentBase::MoveResourceToNewActor(AActor* NewActor, const AActor* ExpectedPreviousOwner)
{
	FPCGMoveResourceParams Params
	{
		.Target = NewActor,
		.ExpectedPreviousOwner = ExpectedPreviousOwner
	};

	return MoveResource(Params);
}

bool UPCGManagedComponentBase::MoveResource(const FPCGMoveResourceParams& Params)
{
	AActor* TargetActor = Params.GetTarget<AActor>();
	check(TargetActor);

	bool bMovedResources = false;

	TArrayView<TSoftObjectPtr<UActorComponent>> GeneratedComponents = GetComponentsArray();
	for (int32 ComponentIndex = GeneratedComponents.Num() - 1; ComponentIndex >= 0; --ComponentIndex)
	{
		TSoftObjectPtr<UActorComponent> GeneratedComponent = GeneratedComponents[ComponentIndex];

		if (!GeneratedComponent.IsValid() || !Params.Accepts(GeneratedComponent.Get()))
		{
			continue;
		}

		TObjectPtr<AActor> OldOwner = GeneratedComponent->GetOwner();
		check(OldOwner);

		// Prevent moving of components on external (or spawned) actors, but still transfer resource
		if (!Params.ExpectedPreviousOwner || OldOwner == Params.ExpectedPreviousOwner)
		{
			bool bDetached = false;
			bool bAttached = false;

			GeneratedComponent->UnregisterComponent();

			// Need to change owner first to avoid that the PCG Component will react to this component changes.
			GeneratedComponent->Rename(nullptr, TargetActor);

			// Check if it is a scene component, and if so, use its method to attach/detach to root component
			if (TObjectPtr<USceneComponent> GeneratedSceneComponent = Cast<USceneComponent>(GeneratedComponent.Get()))
			{
				GeneratedSceneComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
				bDetached = true;
				bAttached = GeneratedSceneComponent->AttachToComponent(TargetActor->GetRootComponent(), FAttachmentTransformRules::KeepWorldTransform);
			}

			// Otherwise use the default one.
			if (!bAttached)
			{
				if (!bDetached)
				{
					OldOwner->RemoveInstanceComponent(GeneratedComponent.Get());
				}

				TargetActor->AddInstanceComponent(GeneratedComponent.Get());
			}

			GeneratedComponent->RegisterComponent();
		}

		ForgetComponent(ComponentIndex);
		bMovedResources = true;
	}

	return bMovedResources;
}

bool UPCGManagedComponentBase::IsManaging(const UObject* InObject) const
{
	if (!InObject || !InObject->IsA<UActorComponent>())
	{
		return false;
	}

	// Const-cast because GetComponentsArray is not const.
	return Algo::AnyOf(const_cast<UPCGManagedComponentBase*>(this)->GetComponentsArray(), [InObject](const TSoftObjectPtr<UActorComponent>& SoftActorComponent)
		{
			const UActorComponent* ActorComponent = SoftActorComponent.Get();
			return ActorComponent && ActorComponent == InObject;
		});
}

void UPCGManagedComponentBase::MarkAsUsed()
{
	if (!bIsMarkedUnused)
	{
		return;
	}

	Super::MarkAsUsed();

	// Can't reuse a resource if we can't reset it. Make sure we never take this path in this case.
	check(SupportsComponentReset());

	ResetComponents();

	TArrayView<TSoftObjectPtr<UActorComponent>> GeneratedComponents = GetComponentsArray();
	for (TSoftObjectPtr<UActorComponent> GeneratedComponent : GeneratedComponents)
	{
		if (GeneratedComponent.Get())
		{
			// Remove all non-default tags, including the "marked for cleanup" tag
			GeneratedComponent->ComponentTags.Reset();
			GeneratedComponent->ComponentTags.Add(PCGHelpers::DefaultPCGTag);
		}
	}
}

void UPCGManagedComponentBase::MarkAsReused()
{
	Super::MarkAsReused();

	TArrayView<TSoftObjectPtr<UActorComponent>> GeneratedComponents = GetComponentsArray();
	for (TSoftObjectPtr<UActorComponent> GeneratedComponent : GeneratedComponents)
	{
		if (GeneratedComponent.Get())
		{
			GeneratedComponent->ComponentTags.Remove(PCGHelpers::MarkedForCleanupPCGTag);
		}
	}
}

void UPCGManagedComponentBase::SetupGeneratedComponentFromBP(TSoftObjectPtr<UActorComponent> InGeneratedComponent)
{
	// Components that are created from blueprint are automatically tagged as "created by construction script",
	// regardless of whether that is true. This makes sure that the flags on the component are correct and considered an instance component
	// and will then be properly serialized and managed by PCG.
	if (UActorComponent* Component = InGeneratedComponent.Get())
	{
		if (AActor* ComponentOwner = Component->GetOwner())
		{
			if (Component->CreationMethod == EComponentCreationMethod::UserConstructionScript)
			{
				ComponentOwner->RemoveOwnedComponent(Component);
				Component->CreationMethod = EComponentCreationMethod::Instance;
				ComponentOwner->AddOwnedComponent(Component);
			}
		}
	}
}

void UPCGManagedComponent::SetGeneratedComponentFromBP(TSoftObjectPtr<UActorComponent> InGeneratedComponent)
{
	GeneratedComponent = InGeneratedComponent;
	SetupGeneratedComponentFromBP(InGeneratedComponent);
}

void UPCGManagedComponentList::SetGeneratedComponentsFromBP(const TArray<TSoftObjectPtr<UActorComponent>>& InGeneratedComponents)
{
	GeneratedComponents = InGeneratedComponents;

	for (TSoftObjectPtr<UActorComponent> GeneratedComponent : GeneratedComponents)
	{
		SetupGeneratedComponentFromBP(GeneratedComponent);
	}
}

void UPCGManagedComponentDefaultList::AddGeneratedComponentsFromBP(const TArray<TSoftObjectPtr<UActorComponent>>& InGeneratedComponents)
{
	GeneratedComponents.Append(InGeneratedComponents);

	for (TSoftObjectPtr<UActorComponent> GeneratedComponent : GeneratedComponents)
	{
		SetupGeneratedComponentFromBP(GeneratedComponent);
	}
}

#if WITH_EDITOR
void UPCGManagedComponentBase::ChangeTransientState(EPCGEditorDirtyMode NewEditingMode)
{
	const bool bNowTransient = (NewEditingMode == EPCGEditorDirtyMode::Preview);

	TArrayView<TSoftObjectPtr<UActorComponent>> GeneratedComponents = GetComponentsArray();
	for (TSoftObjectPtr<UActorComponent> GeneratedComponent : GeneratedComponents)
	{
		if (GeneratedComponent.Get())
		{
			const bool bWasTransient = GeneratedComponent->HasAnyFlags(RF_Transient);

			if (bWasTransient != bNowTransient)
			{
				if (bNowTransient)
				{
					GeneratedComponent->SetFlags(RF_Transient);
				}
				else
				{
					GeneratedComponent->ClearFlags(RF_Transient);
				}

				ForEachObjectWithOuter(GeneratedComponent.Get(), [bNowTransient](UObject* Object)
				{
					if (bNowTransient)
					{
						Object->SetFlags(RF_Transient);
					}
					else
					{
						Object->ClearFlags(RF_Transient);
					}
				});

				GeneratedComponent->MarkPackageDirty(); // should dirty actor this component is attached to
			}
		}
	}

	Super::ChangeTransientState(NewEditingMode);
}
#endif // WITH_EDITOR

void UPCGManagedISMComponent::PostLoad()
{
	Super::PostLoad();

	// Cache raw ptr
	GetComponent();
}

void UPCGManagedISMComponent::SetDescriptor(const FISMComponentDescriptor& InDescriptor)
{
	bHasDescriptor = true;
	Descriptor = InDescriptor;
}

bool UPCGManagedISMComponent::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	if (Super::ReleaseIfUnused(OutActorsToDelete) || !GetComponent())
	{
		return true;
	}
	else if (GetComponent()->GetInstanceCount() == 0)
	{
		GeneratedComponent->DestroyComponent();
		ForgetComponent();
		return true;
	}
	else
	{
		return false;
	}
}

void UPCGManagedISMComponent::ResetComponent()
{
	DataCrc = FPCGCrc();

	if (UInstancedStaticMeshComponent* ISMC = GetComponent())
	{
		ISMC->ClearInstances();
		ISMC->UpdateBounds();
	}
}

void UPCGManagedISMComponent::MarkAsUsed()
{
	const bool bWasMarkedUnused = bIsMarkedUnused;
	Super::MarkAsUsed();

	if (!bWasMarkedUnused)
	{
		return;
	}

	DataCrc = FPCGCrc();
	DataCrcReuseCount = INDEX_NONE;
	CrcReuseCount = INDEX_NONE;

	if (UInstancedStaticMeshComponent* ISMC = GetComponent())
	{
		const bool bHasPreviousRootLocation = bHasRootLocation;

		// Keep track of the current root location so if we reuse this later we are able to update this appropriately
		if (USceneComponent* RootComponent = ISMC->GetAttachmentRoot())
		{
			bHasRootLocation = true;
			RootLocation = RootComponent->GetComponentLocation();
		}
		else
		{
			bHasRootLocation = false;
			RootLocation = FVector::ZeroVector;
		}

		// Reset the rotation/scale to be identity otherwise if the root component transform has changed, the final transform will be wrong
		const FTransform NewComponentWorldTransform(FQuat::Identity, RootLocation, FVector::OneVector);

		if (bHasPreviousRootLocation != bHasRootLocation || !ISMC->GetComponentTransform().Equals(NewComponentWorldTransform, UE_DOUBLE_SMALL_NUMBER))
		{
			// Since this is technically 'moving' the ISM, we need to unregister it before moving otherwise we could get a warning that we're moving a component with static mobility
			ISMC->UnregisterComponent();
			ISMC->SetWorldTransform(NewComponentWorldTransform);
			ISMC->RegisterComponent();
		}
	}
}

void UPCGManagedISMComponent::MarkAsReused()
{
	Super::MarkAsReused();

	CrcReuseCount = INDEX_NONE;
	DataCrcReuseCount = INDEX_NONE;

	if (UInstancedStaticMeshComponent* ISMC = GetComponent())
	{
		// Reset the rotation/scale to be identity otherwise if the root component transform has changed, the final transform will be wrong
		FVector TentativeRootLocation = RootLocation;

		if (!bHasRootLocation)
		{
			if (USceneComponent* RootComponent = ISMC->GetAttachmentRoot())
			{
				TentativeRootLocation = RootComponent->GetComponentLocation();
			}
		}

		// Reset the rotation/scale to be identity otherwise if the root component transform has changed, the final transform will be wrong
		const FTransform NewComponentWorldTransform(FQuat::Identity, TentativeRootLocation, FVector::OneVector);

		if (!ISMC->GetComponentTransform().Equals(NewComponentWorldTransform, UE_DOUBLE_SMALL_NUMBER))
		{
			// Since this is technically 'moving' the ISM, we need to unregister it before moving otherwise we could get a warning that we're moving a component with static mobility
			ISMC->UnregisterComponent();
			ISMC->SetWorldTransform(NewComponentWorldTransform);
			ISMC->RegisterComponent();
		}
	}
}

void UPCGManagedISMComponent::SetRootLocation(const FVector& InRootLocation)
{
	bHasRootLocation = true;
	RootLocation = InRootLocation;
}

UInstancedStaticMeshComponent* UPCGManagedISMComponent::GetComponent() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGManagedISMComponent::GetComponent);
	return Cast<UInstancedStaticMeshComponent>(GeneratedComponent.Get());
}

void UPCGManagedISMComponent::SetComponent(UInstancedStaticMeshComponent* InComponent)
{
	GeneratedComponent = InComponent;
}

void UPCGManagedISKMComponent::PostLoad()
{
	Super::PostLoad();

	if (!bHasDescriptor)
	{
		if (UInstancedSkinnedMeshComponent* ISKMC = GetComponent())
		{
			FSkinnedMeshComponentDescriptor NewDescriptor;
			NewDescriptor.InitFrom(ISKMC);

			SetDescriptor(NewDescriptor);
		}
	}

	// Cache raw ptr
	GetComponent();
}

void UPCGManagedISKMComponent::SetDescriptor(const FSkinnedMeshComponentDescriptor& InDescriptor)
{
	bHasDescriptor = true;
	Descriptor = InDescriptor;
}

bool UPCGManagedISKMComponent::ReleaseIfUnused(TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	if (Super::ReleaseIfUnused(OutActorsToDelete) || !GetComponent())
	{
		return true;
	}
	else if (GetComponent()->GetInstanceCount() == 0)
	{
		GeneratedComponent->DestroyComponent();
		ForgetComponent();
		return true;
	}
	else
	{
		return false;
	}
}

void UPCGManagedISKMComponent::ResetComponent()
{
	if (UInstancedSkinnedMeshComponent* ISKMC = GetComponent())
	{
		ISKMC->ClearInstances();
		ISKMC->UpdateBounds();
	}
}

void UPCGManagedISKMComponent::MarkAsUsed()
{
	const bool bWasMarkedUnused = bIsMarkedUnused;
	Super::MarkAsUsed();

	if (!bWasMarkedUnused)
	{
		return;
	}

	if (UInstancedSkinnedMeshComponent* ISKMC = GetComponent())
	{
		const bool bHasPreviousRootLocation = bHasRootLocation;

		// Keep track of the current root location so if we reuse this later we are able to update this appropriately
		if (USceneComponent* RootComponent = ISKMC->GetAttachmentRoot())
		{
			bHasRootLocation = true;
			RootLocation = RootComponent->GetComponentLocation();
		}
		else
		{
			bHasRootLocation = false;
			RootLocation = FVector::ZeroVector;
		}

		// Reset the rotation/scale to be identity otherwise if the root component transform has changed, the final transform will be wrong
		const FTransform NewComponentWorldTransform(FQuat::Identity, RootLocation, FVector::OneVector);

		if (bHasPreviousRootLocation != bHasRootLocation || !ISKMC->GetComponentTransform().Equals(NewComponentWorldTransform, UE_DOUBLE_SMALL_NUMBER))
		{
			// Since this is technically 'moving' the ABM, we need to unregister it before moving otherwise we could get a warning that we're moving a component with static mobility
			ISKMC->UnregisterComponent();
			ISKMC->SetWorldTransform(NewComponentWorldTransform);
			ISKMC->RegisterComponent();
		}
	}
}

void UPCGManagedISKMComponent::MarkAsReused()
{
	Super::MarkAsReused();

	if (UInstancedSkinnedMeshComponent* ISKMC = GetComponent())
	{
		// Reset the rotation/scale to be identity otherwise if the root component transform has changed, the final transform will be wrong
		FVector TentativeRootLocation = RootLocation;

		if (!bHasRootLocation)
		{
			if (USceneComponent* RootComponent = ISKMC->GetAttachmentRoot())
			{
				TentativeRootLocation = RootComponent->GetComponentLocation();
			}
		}

		// Reset the rotation/scale to be identity otherwise if the root component transform has changed, the final transform will be wrong
		const FTransform NewComponentWorldTransform(FQuat::Identity, TentativeRootLocation, FVector::OneVector);

		if (!ISKMC->GetComponentTransform().Equals(NewComponentWorldTransform, UE_DOUBLE_SMALL_NUMBER))
		{
			// Since this is technically 'moving' the ABM, we need to unregister it before moving otherwise we could get a warning that we're moving a component with static mobility
			ISKMC->UnregisterComponent();
			ISKMC->SetWorldTransform(NewComponentWorldTransform);
			ISKMC->RegisterComponent();
		}
	}
}

void UPCGManagedISKMComponent::SetRootLocation(const FVector& InRootLocation)
{
	bHasRootLocation = true;
	RootLocation = InRootLocation;
}

UInstancedSkinnedMeshComponent* UPCGManagedISKMComponent::GetComponent() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGManagedISKMComponent::GetComponent);
	return Cast<UInstancedSkinnedMeshComponent>(GeneratedComponent.Get());
}

void UPCGManagedISKMComponent::SetComponent(UInstancedSkinnedMeshComponent* InComponent)
{
	GeneratedComponent = InComponent;
}

void UPCGManagedSplineMeshComponent::PostLoad()
{
	Super::PostLoad();

	// Cache raw ptr
	CacheAndGetComponent();
}

void UPCGManagedSplineMeshComponent::ForgetComponent()
{
	Super::ForgetComponent();
	CachedGeneratedComponent.Reset();
}

USplineMeshComponent* UPCGManagedSplineMeshComponent::GetComponent() const
{
	// Read first, if it is cached, return it.
	{
		PCG::TSharedScopeLock ReadLock(CachedLock);
		if (USplineMeshComponent* GeneratedComponentPtr = CachedGeneratedComponent.Get())
		{
			return GeneratedComponentPtr;
		}
	}
	
	// Otherwise, lock on write, check again and cache it if needed.
	return CacheAndGetComponent();
}

void UPCGManagedSplineMeshComponent::SetComponent(USplineMeshComponent* InComponent)
{
	GeneratedComponent = InComponent;
	CachedGeneratedComponent = InComponent;
}

USplineMeshComponent* UPCGManagedSplineMeshComponent::CacheAndGetComponent() const
{
	PCG::TUniqueScopeLock WriteLock(CachedLock);
	USplineMeshComponent* GeneratedComponentPtr = CachedGeneratedComponent.Get();
	if (!GeneratedComponentPtr)
	{
		GeneratedComponentPtr = Cast<USplineMeshComponent>(GeneratedComponent.Get());
		CachedGeneratedComponent = GeneratedComponentPtr;
	}
		
	return GeneratedComponentPtr;
}
