// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGManagedResourceContainer.h"

#include "PCGGraphExecutionStateInterface.h"
#include "PCGManagedResource.h"
#include "PCGModule.h"
#include "Helpers/PCGActorHelpers.h"
#include "Utils/PCGGeneratedResourcesLogging.h"

#include "UObject/Package.h"

#if WITH_EDITOR
#include "Editor.h"
#include "EditorActorFolders.h"
#endif

void FPCGManagedResourceContainerHelper::ForgetAll()
{
	check(IsValid());
	PCG::TScopeLock ResourcesLock(Owner->GetExecutionState().GetManagedResourceContainerLock());

	for (TObjectPtr<UPCGManagedResource>& ManagedResource : Container->GeneratedResources)
	{
		if (UPCGManagedResource* ValidManagedResource = GetValid(ManagedResource))
		{
			ValidManagedResource->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		}
	}
	
	Container->GeneratedResources.Empty();
}

void FPCGManagedResourceContainerHelper::AddManagedResource(UPCGManagedResource* InManagedResource)
{
	check(IsValid());

	AddManagedResourceInternal(InManagedResource, Owner->GetExecutionState().GetManagedResourceContainerLock());
}

void FPCGManagedResourceContainerHelper::AddManagedResourceNoLock(UPCGManagedResource* InManagedResource)
{
	AddManagedResourceInternal(InManagedResource, nullptr);
}

void FPCGManagedResourceContainerHelper::AddManagedResourceInternal(UPCGManagedResource* InManagedResource, FTransactionallySafeCriticalSection* InLock)
{
	check(IsValid());

	PCGGeneratedResourcesLogging::LogAddToManagedResources(Owner, InManagedResource);
	if (InManagedResource)
	{
		PCG::TScopeLock ResourcesLock(InLock);
		ensure(!Container->bAreResourcesInaccessible);
		Container->GeneratedResources.Add(InManagedResource);
		Owner->GetExecutionState().OnManagedResourceAdded(InManagedResource);
	}
}

void FPCGManagedResourceContainerHelper::RemoveManagedResourceAtNoLock(int32 ResourceIndex)
{
	check(IsValid());
	Container->GeneratedResources.RemoveAtSwap(ResourceIndex);
}
void FPCGManagedResourceContainerHelper::RemoveManagedResourceNoLock(UPCGManagedResource* InManagedResource)
{
	check(IsValid());
	Container->GeneratedResources.Remove(InManagedResource);
}

void FPCGManagedResourceContainerHelper::SetManagedResources(TArray<TObjectPtr<UPCGManagedResource>> InManagedResources)
{
	check(IsValid());
	PCG::TScopeLock ResourcesLock(Owner->GetExecutionState().GetManagedResourceContainerLock());

	// We expect the GeneratedResources to be empty here, as otherwise they might not be taken care of properly - they
	// will be lost down below, but this should not happen. However, if the GeneratedResources are marked as Visible,
	// then they will be copied over during BP duplication, hence why this will happen, hence the ensure here.
	ensure(Container->GeneratedResources.IsEmpty());

	Container->GeneratedResources = MoveTemp(InManagedResources);

	// Remove any null entries
	for (int32 ResourceIndex = Container->GeneratedResources.Num() - 1; ResourceIndex >= 0; --ResourceIndex)
	{
		if (!Container->GeneratedResources[ResourceIndex])
		{
			Container->GeneratedResources.RemoveAtSwap(ResourceIndex);
		}
	}
}

TArray<TObjectPtr<UPCGManagedResource>> FPCGManagedResourceConstContainerHelper::GetManagedResourcesCopy() const
{
	check(IsValid());
	TArray<TObjectPtr<UPCGManagedResource>> ManagedResourcesCopy;
	
	{
		PCG::TScopeLock ResourcesLock(Owner->GetExecutionState().GetManagedResourceContainerLock());
		ManagedResourcesCopy = Container->GeneratedResources;
	}

	return ManagedResourcesCopy;
}

void FPCGManagedResourceContainerHelper::CleanupUnusedManagedResources()
{
	check(IsValid());
	PCGGeneratedResourcesLogging::LogCleanupUnusedManagedResources(Owner, Container->GeneratedResources);

	TSet<TSoftObjectPtr<AActor>> ActorsToDelete;

	{
		PCG::TScopeLock ResourcesLock(Owner->GetExecutionState().GetManagedResourceContainerLock());
		ensure(!Container->bAreResourcesInaccessible);
		for (int32 ResourceIndex = Container->GeneratedResources.Num() - 1; ResourceIndex >= 0; --ResourceIndex)
		{
			UPCGManagedResource* Resource = GetValid(Container->GeneratedResources[ResourceIndex]);

			PCGGeneratedResourcesLogging::LogCleanupUnusedManagedResourcesResource(Owner, Resource);

			if (!Resource)
			{
				UE_LOGF(LogPCG, Error, "[UPCGComponent::CleanupUnusedManagedResources] Null generated resource encountered on execution source \"%ls\".", *Owner->GetExecutionState().GetDebugName());
			}

			if (!Resource || Resource->ReleaseIfUnused(ActorsToDelete))
			{
#if WITH_EDITOR
				if (Resource && !Resource->IsMarkedTransientOnLoad())
				{
					Resource->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
				}
#endif
				Container->GeneratedResources.RemoveAtSwap(ResourceIndex);
			}
		}
	}

	if (ActorsToDelete.Num() > 0)
	{
		UPCGActorHelpers::DeleteActors(Owner->GetExecutionState().GetWorld(), ActorsToDelete.Array());
	}

	PCGGeneratedResourcesLogging::LogCleanupUnusedManagedResourcesFinished(Owner, Container->GeneratedResources);
}

void FPCGManagedResourceContainerHelper::Cleanup(bool bReleaseManagedResources)
{
	check(IsValid());

	TSet<TSoftObjectPtr<AActor>> ActorsToDelete;
	FPCGManagedActorLoadingScope Scope;

	if (!bReleaseManagedResources && UPCGManagedResource::DebugForcePurgeAllResourcesOnGenerate())
	{
		bReleaseManagedResources = true;
	}

	PCG::TScopeLock ResourcesLock(Owner->GetExecutionState().GetManagedResourceContainerLock());
	ensure(!Container->bAreResourcesInaccessible);
	Scope.AddResources(Owner, Container->GeneratedResources);
	for (int32 ResourceIndex = Container->GeneratedResources.Num() - 1; ResourceIndex >= 0; --ResourceIndex)
	{
		// Note: resources can be null here in some loading + bp object cases
		UPCGManagedResource* Resource = Container->GeneratedResources[ResourceIndex];

		PCGGeneratedResourcesLogging::LogCleanupLocalImmediateResource(Owner, Resource);

		if (!Resource || Resource->Release(bReleaseManagedResources, ActorsToDelete))
		{
#if WITH_EDITOR
			if (Resource && !Resource->IsMarkedTransientOnLoad())
			{
				Resource->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			}
#endif

			Container->GeneratedResources.RemoveAt(ResourceIndex);
		}
	}

	UPCGActorHelpers::DeleteActors(Owner->GetExecutionState().GetWorld(), ActorsToDelete.Array());
}

void FPCGManagedResourceContainerHelper::ForEachManagedResource(TFunctionRef<void(UPCGManagedResource*)> InFunction)
{
	check(IsValid());

	PCG::TScopeLock ResourcesLock(Owner->GetExecutionState().GetManagedResourceContainerLock());
	ensure(!Container->bAreResourcesInaccessible);
	for (TObjectPtr<UPCGManagedResource>& ManagedResource : Container->GeneratedResources)
	{
		if (ManagedResource)
		{
			InFunction(ManagedResource);
		}
	}
}

void FPCGManagedResourceConstContainerHelper::ForEachConstManagedResource(TFunctionRef<void(const UPCGManagedResource*)> InFunction) const
{
	check(IsValid());

	PCG::TScopeLock ResourcesLock(Owner->GetExecutionState().GetManagedResourceContainerLock());
	ensure(!Container->bAreResourcesInaccessible);
	for (const TObjectPtr<UPCGManagedResource>& ManagedResource : Container->GeneratedResources)
	{
		if (ManagedResource)
		{
			InFunction(ManagedResource);
		}
	}
}

bool FPCGManagedResourceConstContainerHelper::IsAnyObjectManagedByResource(const TArrayView<const UObject*> InObjects) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGManagedResourceContainerHelper::IsAnyObjectManagedByResource);

	check(IsValid());

	if (!ensure(!Container->bAreResourcesInaccessible))
	{
		return false;
	}

	bool bIsManagedByResource = false;
	ForEachConstManagedResource([&bIsManagedByResource, &InObjects](const UPCGManagedResource* ManagedResource)
	{
		if (bIsManagedByResource || !ManagedResource)
		{
			return;
		}

		for (const UObject* Object : InObjects)
		{
			if (ManagedResource->IsManaging(Object))
			{
				bIsManagedByResource = true;
				break;
			}
		}
	});

	return bIsManagedByResource;
}

#if WITH_EDITOR

void FPCGManagedResourceContainerHelper::MarkResourcesAsTransientOnLoad()
{
	check(IsValid());

	for (TObjectPtr<UPCGManagedResource>& GeneratedResource : Container->GeneratedResources)
	{
		if (GeneratedResource)
		{
			GeneratedResource->MarkTransientOnLoad();
		}
	}

	Container->LoadedPreviewResources = Container->GeneratedResources;
}

bool FPCGManagedResourceContainerHelper::ChangeTransientState(EPCGEditorDirtyMode NewEditingMode)
{
	check(IsValid());

	bool bShouldMarkDirty = false;

	PCG::TScopeLock ResourcesLock(Owner->GetExecutionState().GetManagedResourceContainerLock());
	ensure(!Container->bAreResourcesInaccessible);

	for (TObjectPtr<UPCGManagedResource>& GeneratedResource : Container->GeneratedResources)
	{
		if (GeneratedResource)
		{
			// Avoid changing transient state when switching to preview mode on marked as transient resources.
			// When serializing a component we need its marked as transient resources to remain unchanged so they get serialized properly.
			if (NewEditingMode == EPCGEditorDirtyMode::Preview && GeneratedResource->IsMarkedTransientOnLoad())
			{
				continue;
			}

			GeneratedResource->ChangeTransientState(NewEditingMode);
			bShouldMarkDirty = true;
		}
	}

	// If switching from preview mode to normal or preview-on-load,
	// we must materialize any kind of change we've done on the packages that had a different behavior on load (e.g. actor packages)
	if (NewEditingMode != EPCGEditorDirtyMode::Preview)
	{
		bShouldMarkDirty |= DeletePreviewResources();
	}

	return bShouldMarkDirty;
}

bool FPCGManagedResourceContainerHelper::DeletePreviewResources()
{
	check(IsValid());

	bool bResourceWasReleased = false;

	TSet<TSoftObjectPtr<AActor>> ActorsToDelete;
	// Make sure to release fully the resources that were loaded
	for (TObjectPtr<UPCGManagedResource> ResourceToRelease : Container->LoadedPreviewResources)
	{
		if (!Container->GeneratedResources.Contains(ResourceToRelease))
		{
			// Changing the transient state will clear the "marked transient on load" flag
			ResourceToRelease->ChangeTransientState(EPCGEditorDirtyMode::Normal);
			ResourceToRelease->Release(/*bHardRelease=*/true, ActorsToDelete);
			bResourceWasReleased = true;
		}

		// Either this resource was released in the previous code block or prior to calling DeletePreviewResources its transient state was changed back to normal
		ensure(!ResourceToRelease->IsMarkedTransientOnLoad());
	}

	Container->LoadedPreviewResources.Empty();

	if (!ActorsToDelete.IsEmpty())
	{
		UPCGActorHelpers::DeleteActors(Owner->GetExecutionState().GetWorld(), ActorsToDelete.Array());
	}

	return bResourceWasReleased;
}

void FPCGManagedResourceContainerHelper::BeginPreviewSave(TArray<TObjectPtr<UPCGManagedResource>>& OutGeneratedCopy)
{
	check(IsValid());

	OutGeneratedCopy = Container->GeneratedResources;
	Container->GeneratedResources = Container->LoadedPreviewResources;
}

void FPCGManagedResourceContainerHelper::EndPreviewSave(const TArray<TObjectPtr<UPCGManagedResource>>& InGeneratedCopy)
{
	check(IsValid());

	Container->GeneratedResources = InGeneratedCopy;
}

#endif // WITH_EDITOR

void FPCGManagedResourceContainerHelper::EndPlay()
{
	check(IsValid());

	if (ensure(!Container->bAreResourcesInaccessible))
	{
		PCG::TScopeLock ResourcesLock(Owner->GetExecutionState().GetManagedResourceContainerLock());

		for (int32 ResourceIndex = Container->GeneratedResources.Num() - 1; ResourceIndex >= 0; --ResourceIndex)
		{
			if (!Container->GeneratedResources[ResourceIndex])
			{
				// Remove null entries.
				Container->GeneratedResources.RemoveAtSwap(ResourceIndex);
			}
			else if (Container->GeneratedResources[ResourceIndex]->ReleaseOnTeardown())
			{
				if (Container->GeneratedResources[ResourceIndex])
				{
					TSet<TSoftObjectPtr<AActor>> ActorsToDelete;
					Container->GeneratedResources[ResourceIndex]->Release(/*bHardRelease=*/true, ActorsToDelete);

					// Don't support deleting actors during teardown.
					ensure(ActorsToDelete.IsEmpty());
				}

				Container->GeneratedResources.RemoveAtSwap(ResourceIndex);
			}
		}
	}
}

TSharedPtr<FPCGManagedResourceContainerHelper::FCleanupTask> FPCGManagedResourceContainerHelper::CreateCleanupTask()
{
	check(IsValid());

	TSharedPtr<FCleanupTask> CleanupTask = MakeShared<FCleanupTask>();
	CleanupTask->WeakExecutionSource = Owner;

	return CleanupTask;
}

void FPCGManagedResourceContainerHelper::FCleanupTask::Abort()
{
	// If the component is not valid anymore, just early out
	if (!WeakExecutionSource.IsValid())
	{
		return;
	}

	FPCGManagedResourceContainer* ManagedResourceContainer = WeakExecutionSource->GetExecutionState().GetManagedResourceContainer();
	if (ManagedResourceContainer)
	{
		PCG::TScopeLock ContainerLock(WeakExecutionSource->GetExecutionState().GetManagedResourceContainerLock());
		ManagedResourceContainer->bAreResourcesInaccessible = false;
	}
}

bool FPCGManagedResourceContainerHelper::FCleanupTask::Execute(bool bReleaseManagedResources)
{
	IPCGGraphExecutionSource* ExecutionSource = WeakExecutionSource.Get();
	if (!ExecutionSource)
	{
		return true;
	}

	FPCGManagedResourceContainer* ManagedResourceContainer = ExecutionSource->GetExecutionState().GetManagedResourceContainer();
	if (!ManagedResourceContainer)
	{
		return true;
	}

	PCG::TScopeLock ContainerLock(ExecutionSource->GetExecutionState().GetManagedResourceContainerLock());

	// Safeguard to track illegal modifications of the generated resources array while doing cleanup
	if (bIsFirstIteration)
	{
		ensure(!ManagedResourceContainer->bAreResourcesInaccessible);
		ManagedResourceContainer->bAreResourcesInaccessible = true;
		ResourceIndex = ManagedResourceContainer->GeneratedResources.Num() - 1;
		bIsFirstIteration = false;
		AddResources(WeakExecutionSource.Get(), ManagedResourceContainer->GeneratedResources);
	}

	// Going backward
	if (ResourceIndex >= 0)
	{
		UPCGManagedResource* Resource = ManagedResourceContainer->GeneratedResources[ResourceIndex];

		if (!Resource)
		{
			UE_LOGF(LogPCG, Error, "[FPCGManagedResourceContainerHelper::FCleanupTask::Execute] Null generated resource encountered on execution source \"%ls\".", *ExecutionSource->GetExecutionState().GetDebugName());
		}

		PCGGeneratedResourcesLogging::LogCreateCleanupTaskResource(WeakExecutionSource.Get(), Resource);

		if (!Resource || Resource->Release(bReleaseManagedResources, ActorsToDelete))
		{
#if WITH_EDITOR
			if (Resource && !Resource->IsMarkedTransientOnLoad())
			{
				Resource->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			}
#endif

			ManagedResourceContainer->GeneratedResources.RemoveAtSwap(ResourceIndex);
		}

		ResourceIndex--;

		// Returning false means the task is not over
		return false;
	}
	else
	{
		ManagedResourceContainer->bAreResourcesInaccessible = false;
	}

	TSet<FName> DeletedActorFolders;

	if (UWorld* World = ExecutionSource->GetExecutionState().GetWorld())
	{
		const TArray<TSoftObjectPtr<AActor>> ActorsToDeleteSoft = ActorsToDelete.Array();

#if WITH_EDITOR
		for (const TSoftObjectPtr<AActor>& Actor : ActorsToDelete)
		{
			if (Actor.IsValid())
			{
				FName ActorFolderPath = Actor->GetFolderPath();
				if (ActorFolderPath != NAME_None)
				{
					DeletedActorFolders.Add(ActorFolderPath);
				}
			}
		}
#endif

		UPCGActorHelpers::DeleteActors(World, ActorsToDeleteSoft);
	}

#if WITH_EDITOR
	if (UWorld* ThisWorld = ExecutionSource->GetExecutionState().GetWorld(); ThisWorld && GEditor) // FActorFolders require the editor
	{
		for (FName FolderPath : DeletedActorFolders)
		{
			FFolder GeneratedFolder(FFolder::GetWorldRootFolder(ThisWorld).GetRootObject(), FolderPath);
			const bool bFolderExists = GeneratedFolder.IsValid() && FActorFolders::Get().ContainsFolder(*ThisWorld, GeneratedFolder);
			bool bFoundActors = false;

			if (bFolderExists)
			{
				TSet<FName> Folders{ GeneratedFolder.GetPath() };
				FActorFolders::ForEachActorInFolders(*ThisWorld, Folders, [&bFoundActors](AActor* InActor)
				{
					if (InActor)
					{
						bFoundActors = true;
						return false;
					}
					else
					{
						return true;
					}
				});
			}

			if (bFolderExists && !bFoundActors)
			{
				// Delete all subfolders
				TArray<FFolder> SubfoldersToDelete;
				FActorFolders::Get().ForEachFolder(*ThisWorld, [&GeneratedFolder, &SubfoldersToDelete](const FFolder& InFolder)
				{
					if (InFolder.IsChildOf(GeneratedFolder))
					{
						SubfoldersToDelete.Add(InFolder);
					}
					
					return true;
				});

				for (const FFolder& FolderToDelete : SubfoldersToDelete)
				{
					FActorFolders::Get().DeleteFolder(*ThisWorld, FolderToDelete);
				}

				// Finally, delete folder
				FActorFolders::Get().DeleteFolder(*ThisWorld, GeneratedFolder);
			}
		}
	}
#endif

	PCGGeneratedResourcesLogging::LogCreateCleanupTaskFinished(ExecutionSource, &ManagedResourceContainer->GeneratedResources);

	return true;
}