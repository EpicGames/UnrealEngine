// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGModule.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGDefaultExecutionSource.h"
#include "PCGElement.h"
#include "PCGGraph.h"
#include "PCGParamData.h"
#include "Compute/PCGComputeKernel.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPolyLineData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGPolygon2DData.h"
#include "Data/DataView/PCGDataViewCSVConverter.h"
#include "Data/DataView/PCGDataViewInterface.h"
#include "Data/DataView/PCGDataViewNativeJsonConverters.h"
#include "Data/DataView/PCGDataViewNativePropertySelectors.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGWorldQueryHelpers.h"
#include "RuntimeGen/PCGRuntimeGenExecutionSource.h"

#include "Engine/Engine.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "Elements/PCGDifferenceElement.h"
#include "Hash/PCGGraphHashContext.h"
#include "Hash/PCGSettingsHashContext.h"
#include "Tests/Determinism/PCGDeterminismNativeTests.h"
#include "Tests/Determinism/PCGDifferenceDeterminismTest.h"

#include "ISettingsModule.h"
#include "ShaderCore.h"
#include "ShowFlags.h"
#include "Misc/Paths.h"
#endif

LLM_DEFINE_TAG(PCG);

#define LOCTEXT_NAMESPACE "FPCGModule"

#if WITH_EDITOR
FPCGGraphChangedDelegate FPCGModule::OnGraphChangedDelegate;
#endif

namespace PCGModule
{
	FPCGModule* PCGModulePtr = nullptr;

	static const TCHAR* DataTypeRegistryError = TEXT("Data type operations like compatibility or intersection can't run before the module is loaded."
		"The logic will not function in a CDO and must be deferred until after the load.");

	class FPCGGraphExecutionSourceProvider : public IPCGGraphExecutionSourceProvider
	{
	public:
		FPCGGraphExecutionSourceProvider()
		{
#if WITH_EDITOR
			if (GEngine)
			{
				GEngine->OnLevelActorDeleted().AddRaw(this, &FPCGGraphExecutionSourceProvider::OnLevelActorDeleted);
			}
#endif
		}

		~FPCGGraphExecutionSourceProvider()
		{
#if WITH_EDITOR
			if (GEngine)
			{
				GEngine->OnLevelActorDeleted().RemoveAll(this);
			}
#endif
		}

		virtual TArray<FPCGGraphExecutionSourceDescriptor> GatherExecutionSources() const override
		{
			const TStaticArray<const UClass*, 3> ClassToGather =
			{
				UPCGComponent::StaticClass(),
				UPCGDefaultExecutionSource::StaticClass(),
				UPCGRuntimeGenExecutionSource::StaticClass()
			};

			TArray<UObject*> PCGSources;

			for (const UClass* Class : ClassToGather)
			{
				GetObjectsOfClass(Class, PCGSources, /*bIncludeDerivedClasses=*/true);
			}

			TArray<FPCGGraphExecutionSourceDescriptor> ExecutionSources;

			for (UObject* PCGSource : PCGSources)
			{
				IPCGGraphExecutionSource* ExecutionSource = CastChecked<IPCGGraphExecutionSource>(PCGSource, ECastCheckedType::NullAllowed);

				bool bIsValid = IsValid(PCGSource) && ExecutionSource->GetExecutionState().GetGraph();

				if (UPCGComponent* Component = Cast<UPCGComponent>(ExecutionSource))
				{
					bIsValid &= Component->IsRegistered();
				}

				if (bIsValid)
				{
					FPCGGraphExecutionSourceDescriptor Descriptor = { /*ExecutionSource=*/ ExecutionSource};
#if WITH_EDITOR
					Descriptor.bShowInDebugger = true;
#endif
					ExecutionSources.Add(MoveTemp(Descriptor));
				}
			}

			return ExecutionSources;
		}

	private:
#if WITH_EDITOR
		void OnLevelActorDeleted(AActor* InActor)
		{
			BroadcastExecutionSourcesChanged();
		}
#endif
	};

	constexpr FGuid PCGHitResultFilter_PrimitiveComponent(0xF6ABE3BA, 0x41C34DA0, 0x86D96E82, 0x313893BC);
	constexpr FGuid PCGOverlapResultFilter_PrimitiveComponent(0x719F00B8, 0x729E4EF0, 0xA45FF00F, 0x3C8B75FF);
	constexpr FGuid PCGApplyHitResultAttributes_PrimitiveComponent(0xA1F51F3D, 0x0B78435C, 0x8AAC7FF6, 0xF1CF6582);
}

FPCGModule& FPCGModule::GetPCGModuleChecked()
{
	check(PCGModule::PCGModulePtr);
	return *PCGModule::PCGModulePtr;
}

FPCGModule* FPCGModule::GetPCGModule()
{
	return PCGModule::PCGModulePtr;
}

const FPCGDataTypeRegistry& FPCGModule::GetConstDataTypeRegistry()
{
	checkf(IsPCGModuleLoaded(), TEXT("%s"), PCGModule::DataTypeRegistryError);
	return GetPCGModuleChecked().DataTypeRegistry;
}

FPCGDataTypeRegistry& FPCGModule::GetMutableDataTypeRegistry()
{
	checkf(IsPCGModuleLoaded(), TEXT("%s"), PCGModule::DataTypeRegistryError);
	return GetPCGModuleChecked().DataTypeRegistry;
}

bool FPCGModule::IsPCGModuleLoaded()
{
	return PCGModule::PCGModulePtr != nullptr;
}

void FPCGModule::StartupModule()
{
	LLM_SCOPE_BYTAG(PCG);

	// Cache for fast access
	check(!PCGModule::PCGModulePtr);
	PCGModule::PCGModulePtr = this;
	
#if WITH_EDITOR
	PCGDeterminismTests::FNativeTestRegistry::Create();

	RegisterNativeElementDeterminismTests();

	FEngineShowFlags::RegisterCustomShowFlag(PCGEngineShowFlags::Debug, /*DefaultEnabled=*/true, EShowFlagGroup::SFG_Developer, LOCTEXT("ShowFlagDisplayName", "PCG Debug"));

	const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("PCG"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/PCG"), PluginShaderDir);

	LoadAdditionalKernelSources();
#endif // WITH_EDITOR

	// Registering accessor methods
	AttributeAccessorFactory.RegisterDefaultMethods();
	AttributeAccessorFactory.RegisterMethods<UPCGBasePointData>(UPCGBasePointData::GetPointAccessorMethods());
	// @todo_pcg: Eventually remove the UPCGPointData method registration because the UPCGBasePointData accessors are compatible
	AttributeAccessorFactory.RegisterMethods<UPCGPointData>(UPCGPointData::GetPointAccessorMethods());
	AttributeAccessorFactory.RegisterMethods<UPCGPolygon2DData>(UPCGPolygon2DData::GetPolygon2DAccessorMethods());
	AttributeAccessorFactory.RegisterMethods<UPCGSplineData>(UPCGSplineData::GetSplineAccessorMethods());

	DataViewRegistry.RegisterPropertySelector(UPCGData::StaticClass(), MakeUnique<FPCGDataViewPropertySelector>());
	DataViewRegistry.RegisterPropertySelector(UPCGBasePointData::StaticClass(), MakeUnique<FPCGDataViewBasePointDataPropertySelector>());
	DataViewRegistry.RegisterPropertySelector(UPCGSplineData::StaticClass(), MakeUnique<FPCGDataViewSplinePropertySelector>());

	DataViewRegistry.RegisterConverter(UPCGParamData::StaticClass(), UPCGDataViewJsonConverter::StaticClass());
	DataViewRegistry.RegisterConverter(UPCGSpatialData::StaticClass(), UPCGDataViewJsonConverter::StaticClass());
	DataViewRegistry.RegisterConverter(UPCGData::StaticClass(), UPCGDataViewCSVConverter::StaticClass());

	ChangeTrackingRegistry.RegisterGetExecutionSourcesFromSelectionKey(AActor::StaticClass(), FPCGGetExecutionSourcesFromSelectionKey::CreateStatic(&UPCGActorHelpers::GetExecutionSourcesFromSelectionKey));
	
	PhysicsRegistry.RegisterHitResultFilter(PCGModule::PCGHitResultFilter_PrimitiveComponent, FPCGFilterHitResult::CreateStatic(&PCGWorldQueryHelpers::FilterRayHitResult));
	PhysicsRegistry.RegisterOverlapResultFilter(PCGModule::PCGOverlapResultFilter_PrimitiveComponent, FPCGFilterOverlapResult::CreateStatic(&PCGWorldQueryHelpers::FilterOverlapResult));
	PhysicsRegistry.RegisterApplyHitResultAttributes(PCGModule::PCGApplyHitResultAttributes_PrimitiveComponent, FPCGApplyHitResultAttributes::CreateStatic(&PCGWorldQueryHelpers::ApplyHitResultAttributes));

#if WITH_EDITOR
	ObjectHashFactory.RegisterObjectHashContextFactory(UPCGGraphInterface::StaticClass(), FPCGOnCreateObjectHashContext::CreateStatic(&FPCGGraphHashContext::MakeInstance));
	ObjectHashFactory.RegisterObjectHashContextFactory(UPCGSettingsInterface::StaticClass(), FPCGOnCreateObjectHashContext::CreateStatic(&FPCGSettingsHashContext::MakeInstance));
#endif

	// Register onto the PreExit, because we need the class to be still valid to remove them from the mapping
	FCoreDelegates::OnPreExit.AddRaw(this, &FPCGModule::PreExit);

	// Also register on post init to gather all PCG types to register them. Call it immediately if we already passed the PostInit phase (like in the UnitTests for example)
	if (IPluginManager::Get().GetLastCompletedLoadingPhase() >= ELoadingPhase::PostEngineInit)
	{
		OnPostInitEngine();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FPCGModule::OnPostInitEngine);
	}

	TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FPCGModule::Tick));
}

void FPCGModule::ShutdownModule()
{
	// Make sure to do the cleanup in PreExit if the module is shutdown while the engine still runs (like in the UnitTests for example)
	if (!IsEngineExitRequested())
	{
		PreExit();
	}

	FTSTicker::RemoveTicker(TickDelegateHandle);

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreDelegates::OnPreExit.RemoveAll(this);

	PCGModule::PCGModulePtr = nullptr;
}

void FPCGModule::PreExit()
{
	DataViewRegistry.UnregisterConverter(UPCGData::StaticClass(), UPCGDataViewCSVConverter::StaticClass());
	DataViewRegistry.UnregisterConverter(UPCGSpatialData::StaticClass(), UPCGDataViewJsonConverter::StaticClass());
	DataViewRegistry.UnregisterConverter(UPCGParamData::StaticClass(), UPCGDataViewJsonConverter::StaticClass());

	DataViewRegistry.UnregisterPropertySelector(UPCGSplineData::StaticClass());
	DataViewRegistry.UnregisterPropertySelector(UPCGBasePointData::StaticClass());
	DataViewRegistry.UnregisterPropertySelector(UPCGData::StaticClass());

	// Unregistering accessor methods
	AttributeAccessorFactory.UnregisterMethods<UPCGSplineData>();
	AttributeAccessorFactory.UnregisterMethods<UPCGPolygon2DData>();
	AttributeAccessorFactory.UnregisterMethods<UPCGPointData>();
	AttributeAccessorFactory.UnregisterMethods<UPCGBasePointData>();
	AttributeAccessorFactory.UnregisterDefaultMethods();

	DataTypeRegistry.Shutdown();

	GraphExecutionRegistry.Shutdown();

	ChangeTrackingRegistry.UnregisterGetExecutionSourcesFromSelectionKey(AActor::StaticClass());

	PhysicsRegistry.UnregisterHitResultFilter(PCGModule::PCGHitResultFilter_PrimitiveComponent);
	PhysicsRegistry.UnregisterOverlapResultFilter(PCGModule::PCGOverlapResultFilter_PrimitiveComponent);
	PhysicsRegistry.UnregisterApplyHitResultAttributes(PCGModule::PCGApplyHitResultAttributes_PrimitiveComponent);

#if WITH_EDITOR
	DeregisterNativeElementDeterminismTests();

	PCGDeterminismTests::FNativeTestRegistry::Destroy();
#endif // WITH_EDITOR
}

void FPCGModule::OnPostInitEngine()
{
	LLM_SCOPE_BYTAG(PCG);
	DataTypeRegistry.RegisterKnownTypes();

	GraphExecutionRegistry.RegisterExecutionSourceProvider(MakeShared<PCGModule::FPCGGraphExecutionSourceProvider>());
}

void FPCGModule::ExecuteNextTick(TFunction<void()> TickFunction)
{
	PCG::TScopeLock Lock(ExecuteNextTickLock);
	ExecuteNextTicks.Add(TickFunction);
}

bool FPCGModule::Tick(float DeltaTime)
{
	LLM_SCOPE_BYTAG(PCG);

	TArray<TFunction<void()>> LocalExecuteNextTicks;

	{
		PCG::TScopeLock Lock(ExecuteNextTickLock);
		LocalExecuteNextTicks = MoveTemp(ExecuteNextTicks);
	}

	for (TFunction<void()>& LocalExecuteNextTick : LocalExecuteNextTicks)
	{
		LocalExecuteNextTick();
	}

	return true;
}

#if WITH_EDITOR
// todo_pcg: Could move to editor module once PCGComputeKernel.h is publicly exposed.
void FPCGModule::LoadAdditionalKernelSources()
{
	auto LoadSources = [](TConstArrayView<FSoftObjectPath> SourcePaths)
	{
		for (const FSoftObjectPath& SourcePath : SourcePaths)
		{
			if (!ensure(SourcePath.TryLoad()))
			{
				UE_LOGF(LogPCG, Error, "Failed to load compute source asset '%ls'.", *SourcePath.ToString());
			}
		}
	};

	// Load all additional sources needed by all kernels in PCG plugin here.
	LoadSources(PCGComputeKernel::GetDefaultAdditionalSourcePaths());
}

void FPCGModule::RegisterNativeElementDeterminismTests()
{
	PCGDeterminismTests::FNativeTestRegistry::RegisterTestFunction(UPCGDifferenceSettings::StaticClass(), PCGDeterminismTests::DifferenceElement::RunTestSuite);
	// TODO: Add other native test functions
}

void FPCGModule::DeregisterNativeElementDeterminismTests()
{
	PCGDeterminismTests::FNativeTestRegistry::DeregisterTestFunction(UPCGDifferenceSettings::StaticClass());
}

void FPCGModule::SetCPUOnlyExecution(bool bEnabled)
{
	FPCGModule::GetPCGModuleChecked().bIsCPUOnlyExecution = bEnabled;
}

bool FPCGModule::IsCPUOnlyExecution()
{
	return FPCGModule::GetPCGModuleChecked().bIsCPUOnlyExecution;
}

void FPCGModule::SetEditorOnlyGeneration(bool bEnabled)
{
	FPCGModule::GetPCGModuleChecked().bIsEditorOnlyGeneration = bEnabled;
}

bool FPCGModule::IsEditorOnlyGeneration()
{
	return FPCGModule::GetPCGModuleChecked().bIsEditorOnlyGeneration;
}
#endif // WITH_EDITOR

IMPLEMENT_MODULE(FPCGModule, PCG);

PCG_API DEFINE_LOG_CATEGORY(LogPCG);

#undef LOCTEXT_NAMESPACE
