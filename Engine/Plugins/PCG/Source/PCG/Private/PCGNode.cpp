// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGNode.h"

#include "PCGCustomVersion.h"
#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGPin.h"
#include "PCGSubsystem.h"

#include "Algo/Find.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include "Logging/StructuredLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGNode)

namespace PCGNodeHelpers
{
#if WITH_EDITOR
	// Info to aid element cache analysis / debugging
	void GetGraphCacheDebugInfo(const UPCGNode* InNode, bool& bOutDebuggingEnabled, uint32& OutNumCacheEntries)
	{
		UWorld* World = GEditor ? (GEditor->PlayWorld ? GEditor->PlayWorld.Get() : GEditor->GetEditorWorldContext().World()) : nullptr;
		UPCGSubsystem* Subsystem = UPCGSubsystem::GetInstance(World);
		bOutDebuggingEnabled = Subsystem && Subsystem->IsGraphCacheDebuggingEnabled();

		if (bOutDebuggingEnabled)
		{
			IPCGElement* Element = (InNode && InNode->GetSettings()) ? InNode->GetSettings()->GetElement().Get() : nullptr;
			OutNumCacheEntries = Element ? Subsystem->GetGraphCacheEntryCount(Element) : 0;
		}
	}
#endif
}

UPCGNode::UPCGNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SettingsInterface = ObjectInitializer.CreateOptionalDefaultSubobject<UPCGTrivialSettings>(this, TEXT("DefaultNodeSettings"));
}

void UPCGNode::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (DefaultSettings_DEPRECATED)
	{
		SettingsInterface = DefaultSettings_DEPRECATED;
		DefaultSettings_DEPRECATED = nullptr;
	}

	if (SettingsInterface)
	{
		SettingsInterface->OnSettingsChangedDelegate.AddUObject(this, &UPCGNode::OnSettingsChanged);
		SettingsInterface->ConditionalPostLoad();
	}

	// Make sure legacy nodes support transactions.
	if (HasAllFlags(RF_Transactional) == false)
	{
		SetFlags(RF_Transactional);
	}

	for (UPCGPin* InputPin : InputPins)
	{
		check(InputPin);
		InputPin->ConditionalPostLoad();
	}

	for (UPCGPin* OutputPin : OutputPins)
	{
		check(OutputPin);
		OutputPin->ConditionalPostLoad();
	}
#endif
}

#if WITH_EDITOR
void UPCGNode::ApplyDeprecationBeforeUpdatePins()
{
	if (UPCGSettings* Settings = GetSettings())
	{
		const int VersionBefore = Settings->DataVersion;

		// Give the chance to the settings to rename the pins
		Settings->ApplyDeprecationBeforeUpdatePins(this, InputPins, OutputPins);

		// Then make sure the pin properties are up to date on their types. We won't break edges here, it will be done in the update pins.
		// But all the nodes need to have their pins types up-to-date to know if an edge should be broken or not. Otherwise, the first node gets
		// its pin updated, check the other side of the edge, finds that the (old) type is not matching and break the edge, even if the new type would have been
		// compatible.
		auto UpdatePinTypes = [](const TArray<FPCGPinProperties>& PinProperties, UPCGPin* Pin)
		{
			if (const FPCGPinProperties* MatchingProperties = PinProperties.FindByPredicate([Pin](const FPCGPinProperties& Prop) { return Prop.Label == Pin->Properties.Label; }))
			{
				if (!(Pin->Properties == *MatchingProperties))
				{
					Pin->Modify();
					Pin->Properties = *MatchingProperties;
					Pin->bPropertiesHaveChangedOnDeprecation = true;
				}
			}
		};

		for (UPCGPin* InputPin : InputPins)
		{
			UpdatePinTypes(Settings->AllInputPinProperties(), InputPin);
		}

		for (UPCGPin* OutputPin : OutputPins)
		{
			UpdatePinTypes(Settings->AllOutputPinProperties(), OutputPin);
		}

		// Version number should not be bumped in this before-update-pins migration, if it does
		// we risk deprecation code not running in post-update-pins because version number is latest.
		ensure(VersionBefore == Settings->DataVersion);
	}
}

void UPCGNode::ApplyDeprecation()
{
	UPCGPin* DefaultOutputPin = OutputPins.IsEmpty() ? nullptr : OutputPins[0];
	for (const TObjectPtr<UPCGNode>& OutboundNode : OutboundNodes_DEPRECATED)
	{
		UPCGPin* OtherNodeInputPin = OutboundNode->InputPins.IsEmpty() ? nullptr : OutboundNode->InputPins[0];

		if (DefaultOutputPin && OtherNodeInputPin)
		{
			DefaultOutputPin->AddEdgeTo(OtherNodeInputPin);
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Unable to apply deprecation on outbound nodes");
		}
	}
	OutboundNodes_DEPRECATED.Reset();

	// Deprecated edges -> pins & edges
	// Inbound edges will be taken care of by other nodes outbounds
	InboundEdges_DEPRECATED.Reset();

	for (UPCGEdge* OutboundEdge : OutboundEdges_DEPRECATED)
	{
		check(OutboundEdge->InboundNode_DEPRECATED == this);
		check(OutboundEdge->OutboundNode_DEPRECATED);

		UPCGPin* OutputPin = nullptr;
		if (OutboundEdge->InboundLabel_DEPRECATED == NAME_None)
		{
			OutputPin = OutputPins.IsEmpty() ? nullptr : OutputPins[0];
		}
		else
		{
			OutputPin = GetOutputPin(OutboundEdge->InboundLabel_DEPRECATED);
		}

		if (!OutputPin)
		{
			UE_LOGF(LogPCG, Error, "Unable to apply deprecation on outbound edge on node '%ls' - can't find output pin '%ls'", *GetFName().ToString(), *OutboundEdge->InboundLabel_DEPRECATED.ToString());
			continue;
		}

		UPCGNode* OtherNode = OutboundEdge->OutboundNode_DEPRECATED;
		if (!OtherNode)
		{
			UE_LOGF(LogPCG, Error, "Unable to apply deprecation on outbound edge on node '%ls' - can't find other node", *GetFName().ToString());
			continue;
		}

		UPCGPin* OtherNodeInputPin = nullptr;
		if (OutboundEdge->OutboundLabel_DEPRECATED == NAME_None)
		{
			OtherNodeInputPin = OtherNode->InputPins.IsEmpty() ? nullptr : OtherNode->InputPins[0];
		}
		else
		{
			OtherNodeInputPin = OtherNode->GetInputPin(OutboundEdge->OutboundLabel_DEPRECATED);
		}

		if (OtherNodeInputPin)
		{
			OutputPin->AddEdgeTo(OtherNodeInputPin);
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Unable to apply deprecation on outbound edge on node %ls output pin %ls - can't find node %ls input pin %ls", *GetFName().ToString(), *OutboundEdge->InboundLabel_DEPRECATED.ToString(), *OtherNode->GetFName().ToString(), *OutboundEdge->OutboundLabel_DEPRECATED.ToString());
		}
	}
	OutboundEdges_DEPRECATED.Reset();

	if (UPCGSettings* Settings = GetSettings())
	{
		Settings->ApplyDeprecation(this);

		// Once deprecation has run, version should be up to date.
		ensure(Settings->DataVersion == FPCGCustomVersion::LatestVersion);
	}
}

void UPCGNode::ApplyStructuralDeprecation()
{
	if (UPCGSettings* Settings = GetSettings())
	{
		Settings->ApplyStructuralDeprecation(this);
	}
}

void UPCGNode::RebuildAfterPaste()
{
	// When Pasting a node it will get created through NewObject which will create a DefaultNodeSettings object but if
	// the Source of the Copy has its own non-DefaultNodeSettings this is what is actually going to be pasted on this node through ImportObjectProperties
	// leaving this DefaultNodeSettings outered to the new node but unused. Here we make sure to find the orphaned UPCGSettingsInterface objects under the node and
	// if they aren't the one that we are actually using (SettingsInterface member) then we just discard them
	TArray<UObject*> Objects;
	GetObjectsWithOuter(this, Objects, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);

	for (UObject* Object : Objects)
	{
		if (UPCGSettingsInterface* OuteredSettingsInterface = Cast<UPCGSettingsInterface>(Object); OuteredSettingsInterface && SettingsInterface != OuteredSettingsInterface)
		{
			OuteredSettingsInterface->OnSettingsChangedDelegate.RemoveAll(this);
			OuteredSettingsInterface->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			OuteredSettingsInterface->MarkAsGarbage();
		}
	}

	if (SettingsInterface)
	{
		SettingsInterface->OnSettingsChangedDelegate.AddUObject(this, &UPCGNode::OnSettingsChanged);
	}
}

#endif

#if WITH_EDITOR
void UPCGNode::PreEditUndo()
{
	if (SettingsInterface)
	{
		SettingsInterface->OnSettingsChangedDelegate.RemoveAll(this);
	}

	UObject* Outer = GetOuter();
	if (UPCGGraph* PCGGraph = Cast<UPCGGraph>(Outer))
	{
		PCGGraph->PreNodeUndo(this);
	}

	Super::PreEditUndo();
}

void UPCGNode::PostEditUndo()
{
	Super::PostEditUndo();

	if (SettingsInterface)
	{
		SettingsInterface->OnSettingsChangedDelegate.AddUObject(this, &UPCGNode::OnSettingsChanged);
	}

 	UObject* Outer = GetOuter();
	if (UPCGGraph* PCGGraph = Cast<UPCGGraph>(Outer))
	{
		PCGGraph->PostNodeUndo(this);
	}
}
#endif

void UPCGNode::BeginDestroy()
{
#if WITH_EDITOR
	if (SettingsInterface)
	{
		SettingsInterface->OnSettingsChangedDelegate.RemoveAll(this);
	}
#endif

	Super::BeginDestroy();
}

UPCGGraph* UPCGNode::GetGraph() const
{
	return Cast<UPCGGraph>(GetOuter());
}

UPCGNode* UPCGNode::AddEdgeTo(FName FromPinLabel, UPCGNode* To, FName ToPinLabel)
{
	if (UPCGGraph* Graph = GetGraph())
	{
		return Graph->AddEdge(this, FromPinLabel, To, ToPinLabel);
	}
	else
	{
		return nullptr;
	}
}

bool UPCGNode::RemoveEdgeTo(FName FromPinLabel, UPCGNode* To, FName ToPinLabel)
{
	if (UPCGGraph* Graph = GetGraph())
	{
		return Graph->RemoveEdge(this, FromPinLabel, To, ToPinLabel);
	}
	else
	{
		return false;
	}
}

FText UPCGNode::GetNodeTitle(EPCGNodeTitleType TitleType) const
{
	// Title length that looks reasonable.
	constexpr int NodeTitleMaxLen = 45;

	// Clip string at right hand side (standard overflow).
	auto ClipRightSide = [NodeTitleMaxLen](FString& InOutTitle, int MaxLen)
	{
		if (InOutTitle.Len() > MaxLen)
		{
			InOutTitle = InOutTitle.Left(MaxLen - 3) + TEXT("...");
		}
	};

	// Clip left hand side to maximize displayed information at end of string.
	auto ClipLeftSide = [NodeTitleMaxLen](FString& InOutTitle, int MaxLen)
	{
		if (InOutTitle.Len() > MaxLen)
		{
			InOutTitle = TEXT("...") + InOutTitle.Right(MaxLen - 3);
		}
	};

	const bool bFlipTitleLines = HasFlippedTitleLines();
	const bool bIsTitleAuthored = HasAuthoredTitle();

	FString GeneratedTitle = GetGeneratedTitleLine().ToString();
	const bool bHasMultipleTitleLines = !GeneratedTitle.IsEmpty();

	FString PrimaryTitleLine;

	// The normal title line is used if there is no generated title, or if the node has not requested flipped
	// title lines. But if the user has edited the title, always use the users title as primary. 
	if (!bHasMultipleTitleLines || !bFlipTitleLines || bIsTitleAuthored)
	{
		PrimaryTitleLine = GetAuthoredTitleLine().ToString();
		ClipRightSide(PrimaryTitleLine, NodeTitleMaxLen);
	}
	else
	{
		PrimaryTitleLine = GeneratedTitle;

		// Generated title clipped on left side because the end of the string often has the interesting part.
		ClipLeftSide(PrimaryTitleLine, NodeTitleMaxLen);
	}

	// Only add a second title line if the full title is being requested (and if the has multiple title lines).
	if (TitleType == EPCGNodeTitleType::FullTitle)
	{
		FString SecondaryTitleLine;

		if (bHasMultipleTitleLines)
		{
			// Secondary title is the node-generated title line if titles aren't flipped.
			if (!bFlipTitleLines)
			{
				SecondaryTitleLine = GeneratedTitle;

				// Generated title clipped on left side because the end of the string often has the interesting part.
				ClipLeftSide(SecondaryTitleLine, NodeTitleMaxLen);
			}
			// If titles are flipped and user has not authored something, then just show the standard node name.
			else if (!bIsTitleAuthored)
			{
				SecondaryTitleLine = GetDefaultTitle().ToString();
				ClipRightSide(SecondaryTitleLine, NodeTitleMaxLen);
			}
			// If title has been authored and title lines are flipped, display "<DefaultTitle> - <GeneratedTitle>"
			else
			{
				const FString DefaultTitle = GetDefaultTitle().ToString();

				// Generated title clipped on left side because the end of the string often has the interesting part.
				ClipLeftSide(GeneratedTitle, NodeTitleMaxLen - (DefaultTitle.Len() + 3));
				SecondaryTitleLine = DefaultTitle + TEXT(" - ") + GeneratedTitle;
			}
		}

		// Debug info - append how many copies of this element are currently in the cache to the node title.
#if WITH_EDITOR
		bool bDebuggingEnabled = false;
		uint32 NumCacheEntries = 0;
		PCGNodeHelpers::GetGraphCacheDebugInfo(this, bDebuggingEnabled, NumCacheEntries);

		if (bDebuggingEnabled)
		{
			SecondaryTitleLine += FString::Format(TEXT(" [{0}]"), { NumCacheEntries });
		}
#endif

		if (!SecondaryTitleLine.IsEmpty())
		{
			return FText::Format(FText::FromString("{0}\r\n{1}"), FText::FromString(PrimaryTitleLine), FText::FromString(SecondaryTitleLine));
		}
	}

	return FText::FromString(PrimaryTitleLine);
}

FText UPCGNode::GetDefaultTitle() const
{
#if WITH_EDITOR
	if (UPCGSettings* Settings = GetSettings())
	{
		return Settings->GetDefaultNodeTitle();
	}
#endif

	return NSLOCTEXT("PCGNode", "NodeTitle", "Unnamed Node");
}

FText UPCGNode::GetAuthoredTitleLine() const
{
	if (NodeTitle != NAME_None)
	{
		return FText::FromString(FName::NameToDisplayString(NodeTitle.ToString(), false));
	}
	else
	{
		return GetDefaultTitle();
	}
}

bool UPCGNode::HasFlippedTitleLines() const
{
	const UPCGSettings* Settings = GetSettings();
	return Settings && Settings->HasFlippedTitleLines();
}

FText UPCGNode::GetGeneratedTitleLine() const
{
	if (UPCGSettings* Settings = GetSettings())
	{
		const FString AdditionalInformation = Settings->GetAdditionalTitleInformation();
		if (!AdditionalInformation.IsEmpty())
		{
			return FText::FromString(AdditionalInformation);
		}
	}

	return FText::GetEmpty();
}

#if WITH_EDITOR
FText UPCGNode::GetNodeTooltipText() const
{
	if (UPCGSettings* Settings = GetSettings())
	{
		return Settings->GetNodeTooltipText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

bool UPCGNode::SetNodeTitle(FName NewTitle, bool bApplySanitization)
{	
	if (NewTitle == NodeTitle)
	{
		return false;
	}

	if (bApplySanitization)
	{
		// Ask the graph and the settings to do sanitization if needed.
		if (const UPCGGraph* Graph = GetGraph())
		{
			Graph->SanitizeNodeTitle(NewTitle, this);
		}

		if (const UPCGSettings* Settings = GetSettings())
		{
			Settings->SanitizeNodeTitle(NewTitle);
		}
	}

	// Re-check if after the sanitization the name is the same or not.
	if (NewTitle == NodeTitle)
	{
		return false;
	}

	Modify();
	NodeTitle = NewTitle;

	return true;
}
#endif

bool UPCGNode::IsInstance() const
{
	return SettingsInterface && SettingsInterface->IsInstance();
}

TArray<FPCGPinProperties> UPCGNode::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	Algo::Transform(InputPins, PinProperties, [](const UPCGPin* InputPin) { return InputPin->Properties; });
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGNode::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	Algo::Transform(OutputPins, PinProperties, [](const UPCGPin* OutputPin) { return OutputPin->Properties; });
	return PinProperties;
}

UPCGPin* UPCGNode::GetInputPin(const FName& Label)
{
	for (UPCGPin* InputPin : InputPins)
	{
		if (InputPin->Properties.Label == Label)
		{
			return InputPin;
		}
	}

	return nullptr;
}

const UPCGPin* UPCGNode::GetInputPin(const FName& Label) const
{
	for (const UPCGPin* InputPin : InputPins)
	{
		if (InputPin->Properties.Label == Label)
		{
			return InputPin;
		}
	}

	return nullptr;
}

UPCGPin* UPCGNode::GetOutputPin(const FName& Label)
{
	for (UPCGPin* OutputPin : OutputPins)
	{
		if (OutputPin->Properties.Label == Label)
		{
			return OutputPin;
		}
	}

	return nullptr;
}

const UPCGPin* UPCGNode::GetOutputPin(const FName& Label) const
{
	for (const UPCGPin* OutputPin : OutputPins)
	{
		if (OutputPin->Properties.Label == Label)
		{
			return OutputPin;
		}
	}

	return nullptr;
}

bool UPCGNode::IsInputPinConnected(const FName& Label) const
{
	if (const UPCGPin* InputPin = GetInputPin(Label))
	{
		return InputPin->IsConnected();
	}
	else
	{
		return false;
	}
}

bool UPCGNode::IsOutputPinConnected(const FName& Label) const
{
	if (const UPCGPin* OutputPin = GetOutputPin(Label))
	{
		return OutputPin->IsConnected();
	}
	else
	{
		return false;
	}
}

void UPCGNode::RenameInputPin(const FName& InOldLabel, const FName& InNewLabel, bool bInBroadcastUpdate)
{
	if (UPCGPin* Pin = GetInputPin(InOldLabel))
	{
		Pin->Modify();
		Pin->Properties.Label = InNewLabel;

#if WITH_EDITOR
		if (bInBroadcastUpdate)
		{
			OnNodeChangedDelegate.Broadcast(this, EPCGChangeType::Node);
		}
#endif // WITH_EDITOR
	}
}

void UPCGNode::RenameOutputPin(const FName& InOldLabel, const FName& InNewLabel, bool bInBroadcastUpdate)
{
	if (UPCGPin* Pin = GetOutputPin(InOldLabel))
	{
		Pin->Modify();
		Pin->Properties.Label = InNewLabel;

#if WITH_EDITOR
		if (bInBroadcastUpdate)
		{
			OnNodeChangedDelegate.Broadcast(this, EPCGChangeType::Node);
		}
#endif // WITH_EDITOR
	}
}

bool UPCGNode::HasInboundEdges() const
{
	for (const UPCGPin* InputPin : InputPins)
	{
		for (const UPCGEdge* InboundEdge : InputPin->Edges)
		{
			if (InboundEdge->IsValid())
			{
				return true;
			}
		}
	}

	return false;
}

int32 UPCGNode::GetInboundEdgesNum() const
{
	int32 NumInboundEdges = 0;

	for (const UPCGPin* InputPin : InputPins)
	{
		check(InputPin);
		NumInboundEdges += InputPin->EdgeCount();
	}
	
	return NumInboundEdges;
}

const UPCGPin* UPCGNode::GetPassThroughInputPin() const
{
	const UPCGPin* PassThroughOutput = GetPassThroughOutputPin();
	if (!PassThroughOutput)
	{
		return nullptr;
	}

	const TArray<TObjectPtr<UPCGPin>>& AllInputPins = GetInputPins();
	if (AllInputPins.IsEmpty())
	{
		return nullptr;
	}

	const UPCGSettings* Settings = GetSettings();
	if (!Settings)
	{
		return nullptr;
	}

	// Always take the first valid input pin as the pass-through pin
	auto FindFirstPassThrough = [Settings](const TObjectPtr<UPCGPin>& InPin) { return Settings->DoesPinSupportPassThrough(InPin); };
	if (const TObjectPtr<UPCGPin>* FirstPassThroughPin = Algo::FindByPredicate(AllInputPins, FindFirstPassThrough))
	{
		// Finally in order to be a candidate for passing through, data must be compatible.
		const FPCGDataTypeIdentifier InputType = FirstPassThroughPin->Get()->GetCurrentTypesID();
		const FPCGDataTypeIdentifier OutputType = PassThroughOutput->GetCurrentTypesID();

		// Misc note - it would be nice if we could be stricter, like line below this comment. However it would mean it will stop an Any
		// input being passed through to a Point output, even if the incoming edge will be receiving points dynamically/during execution. If a
		// user creates a BP node with an Any input, which they may do lazily or unknowingly, this blocks passthrough. So instead we'll indicate
		// that this pin *may* be used as a passthrough, and during execution in DisabledPassThroughData() we check dynamic types.
		//const bool bInputTypeNotWiderThanOutputType = !(InputType & ~OutputType);
		if (!!(InputType & OutputType))
		{
			return FirstPassThroughPin->Get();
		}
	}

	return nullptr;
}

const UPCGPin* UPCGNode::GetPassThroughOutputPin() const
{
	const int32 PrimaryOutputPinIndex = 0;
	return GetOutputPins().Num() > PrimaryOutputPinIndex ? GetOutputPins()[PrimaryOutputPinIndex] : nullptr;
}

bool UPCGNode::IsInputPinRequiredByExecution(const UPCGPin* InPin) const
{
	const UPCGSettings* Settings = GetSettings();
	return !Settings || Settings->IsInputPinRequiredByExecution(InPin);
}

bool UPCGNode::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
	check(InPin);

	UPCGSettings* Settings = GetSettings();
	if (!Settings)
	{
		// Safe default - assume used
		return true;
	}

	// Disabled nodes only use the 'pass through pin'
	if (!Settings->bEnabled)
	{
		return InPin == GetPassThroughInputPin();
	}

	// Let Settings signal whether it uses a pin or not
	return Settings->IsPinUsedByNodeExecution(InPin);
}

bool UPCGNode::IsEdgeUsedByNodeExecution(const UPCGEdge* InEdge) const
{
	check(InEdge);

	// Locate the pin on this node that the edge is connected to
	const UPCGPin* Pin = nullptr;
	if (InEdge->InputPin->Node == this)
	{
		Pin = InEdge->InputPin;
	}
	else if (InEdge->OutputPin->Node == this)
	{
		Pin = InEdge->OutputPin;
	}

	if (!ensure(Pin))
	{
		// Safe default - assume used
		return true;
	}

	UPCGSettings* Settings = GetSettings();
	if (!Settings)
	{
		// Safe default - assume used
		return true;
	}

	// Disabled nodes only use the 'pass through pin'
	if (!Settings->bEnabled)
	{
		// Only accept first edge if node is disabled
		const bool bConnectedToPassThrough = (Pin == GetPassThroughInputPin());
		const bool bMustBeFirstEdge = Settings->OnlyPassThroughOneEdgeWhenDisabled();
		const bool bIsFirstEdge = Pin->Edges.Num() > 0 && Pin->Edges[0] == InEdge;
		if (!bConnectedToPassThrough || (bMustBeFirstEdge && !bIsFirstEdge))
		{
			return false;
		}
	}

	// Ask Settings if the pin is in use, good opportunity to gray out pins based on parameters
	return Settings->IsPinUsedByNodeExecution(Pin);
}

const UPCGPin* UPCGNode::GetFirstConnectedInputPin() const
{
	for (const UPCGPin* InputPin : InputPins)
	{
		if (InputPin && InputPin->EdgeCount() > 0)
		{
			return InputPin;
		}
	}

	return nullptr;
}

const UPCGPin* UPCGNode::GetFirstConnectedOutputPin() const
{
	for (const UPCGPin* OutputPin : OutputPins)
	{
		if (OutputPin && OutputPin->EdgeCount() > 0)
		{
			return OutputPin;
		}
	}

	return nullptr;
}

void UPCGNode::SetSettingsInterface(UPCGSettingsInterface* InSettingsInterface, bool bUpdatePins)
{
	const bool bDifferentInterface = (SettingsInterface.Get() != InSettingsInterface);
	if (bDifferentInterface && SettingsInterface)
	{
#if WITH_EDITOR
		SettingsInterface->OnSettingsChangedDelegate.RemoveAll(this);
#endif

		// Un-outer the current settings to disassociate old settings from node. Without this one can copy paste
		// a node and get both settings objects in the clipboard text, and the wrong settings can be used upon paste.
		if (ensure(SettingsInterface->GetOuter() == this))
		{
#if WITH_EDITOR
			SettingsInterface->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
#endif
			SettingsInterface->MarkAsGarbage();
		}
	}

	SettingsInterface = InSettingsInterface;

#if WITH_EDITOR
	if (bDifferentInterface && SettingsInterface)
	{
		check(SettingsInterface->GetSettings());
		SettingsInterface->OnSettingsChangedDelegate.AddUObject(this, &UPCGNode::OnSettingsChanged);
	}
#endif

	if (bUpdatePins)
	{
		UpdatePins();
	}
}

UPCGSettings* UPCGNode::GetSettings() const
{
	if (SettingsInterface)
	{
		return SettingsInterface->GetSettings();
	}
	else
	{
		return nullptr;
	}
}

#if WITH_EDITOR
void UPCGNode::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGNode, NodeTitle))
	{
		OnNodeChangedDelegate.Broadcast(this, EPCGChangeType::Cosmetic);
	}
}

void UPCGNode::OnSettingsChanged(UPCGSettings* InSettings, EPCGChangeType ChangeType)
{
	if (InSettings == GetSettings())
	{
		EPCGChangeType PinChangeType = ChangeType | UpdatePins();
		if (InSettings->HasDynamicPins())
		{
			// Add in case dynamic pin types changed when settings changed.
			PinChangeType |= EPCGChangeType::Node;
		}

		OnNodeChangedDelegate.Broadcast(this, PinChangeType);
	}
}

void UPCGNode::TransferEditorProperties(UPCGNode* OtherNode) const
{
	OtherNode->PositionX = PositionX;
	OtherNode->PositionY = PositionY;
	OtherNode->bCommentBubblePinned = bCommentBubblePinned;
	OtherNode->bCommentBubbleVisible = bCommentBubbleVisible;
	OtherNode->NodeComment = NodeComment;
}

#endif // WITH_EDITOR

void UPCGNode::UpdateAfterSettingsChangeDuringCreation()
{
	UpdatePins();
}

EPCGChangeType UPCGNode::UpdatePins(bool bQuiet)
{
	return UpdatePins([](UPCGNode* Node){ return NewObject<UPCGPin>(Node); }, bQuiet);
}

void UPCGNode::CreateDefaultPins(TFunctionRef<UPCGPin* (UPCGNode*)> PinAllocator)
{
	auto CreatePins = [this, &PinAllocator](TArray<UPCGPin*>& Pins, const TArray<FPCGPinProperties>& PinProperties)
	{
		for (const FPCGPinProperties& Properties : PinProperties)
		{
			UPCGPin* NewPin = PinAllocator(this);
			NewPin->Node = this;
			NewPin->Properties = Properties;
			Pins.Add(NewPin);
		};
	};

	UPCGSettings* Settings = GetSettings();
	check(Settings);
	CreatePins(MutableView(InputPins), Settings->DefaultInputPinProperties());
	CreatePins(MutableView(OutputPins), Settings->DefaultOutputPinProperties());
}

EPCGChangeType UPCGNode::UpdatePins(TFunctionRef<UPCGPin*(UPCGNode*)> PinAllocator, bool bQuiet)
{
	TSet<UPCGNode*> TouchedNodes;
	const UPCGSettings* Settings = GetSettings();
	
	if (!Settings)
	{
		bool bChanged = !InputPins.IsEmpty() || !OutputPins.IsEmpty();

		if (bChanged)
		{
			Modify();
		}

		// Clean up edges
		for (UPCGPin* Pin : InputPins)
		{
			if (Pin)
			{
				Pin->BreakAllEdges(&TouchedNodes);
			}
		}

		for (UPCGPin* Pin : OutputPins)
		{
			if (Pin)
			{
				Pin->BreakAllEdges(&TouchedNodes);
			}
		}

		InputPins.Reset();
		OutputPins.Reset();
		return EPCGChangeType::Edge | EPCGChangeType::Node;
	}

	UE_LOG_CONTEXT("PCGNode", FSoftObjectPath(Settings));

	const FName SettingsName = Settings->GetFName();
	const FString GraphPath = GetGraph() ?  FSoftObjectPath(GetGraph()).ToString() : TEXT("Unknown");
	
	TArray<FPCGPinProperties> InboundPinProperties = Settings->AllInputPinProperties();
	TArray<FPCGPinProperties> OutboundPinProperties = Settings->AllOutputPinProperties();

	auto RemoveDuplicates = [this, &GraphPath](TArray<FPCGPinProperties>& Properties)
	{
		for (int32 i = Properties.Num() - 2; i >= 0; --i)
		{
			for (int32 j = i + 1; j < Properties.Num(); ++j)
			{
				if (Properties[i].Label == Properties[j].Label)
				{
					UE_LOGF(LogPCG, Warning, "UpdatePins: Pin properties from the settings on node '%ls' in graph '%ls' contained a duplicate pin '%ls', removing this pin properties.",
						*GetName(), *GraphPath, *Properties[i].Label.ToString());

					// Remove but preserve order
					Properties.RemoveAt(j);
					break;
				}
			}
		}
	};
	RemoveDuplicates(InboundPinProperties);
	RemoveDuplicates(OutboundPinProperties);

	auto UpdatePins = [this, &PinAllocator, &TouchedNodes, &SettingsName, bQuiet, &GraphPath](TArray<UPCGPin*>& Pins, const TArray<FPCGPinProperties>& PinProperties)
	{
		bool bAppliedEdgeChanges = false;
		bool bChangedPins = false;
		bool bChangedTooltips = false;

		// Find unmatched pins vs. properties on a name basis
		TArray<UPCGPin*> UnmatchedPins;
		for (UPCGPin* Pin : Pins)
		{
			if (const FPCGPinProperties* MatchingProperties = PinProperties.FindByPredicate([Pin](const FPCGPinProperties& Prop) { return Prop.Label == Pin->Properties.Label; }))
			{
				if (!(Pin->Properties == *MatchingProperties))
				{
					Pin->Modify();
					Pin->Properties = *MatchingProperties;

					FString Reason{};

					if (Pin->BreakAllIncompatibleEdges(&TouchedNodes, !bQuiet ? &Reason : nullptr))
					{
						bAppliedEdgeChanges = true;
						if (!bQuiet)
						{
							UE_LOGF(LogPCG, Warning, "[%ls - %ls] UpdatePins: %ls", *GraphPath, *SettingsName.ToString(), *Reason);	
						}
					}

					bChangedPins = true;
				}
#if WITH_EDITOR
				else if (Pin->Properties.Tooltip.CompareTo(MatchingProperties->Tooltip))
				{
					Pin->Properties.Tooltip = MatchingProperties->Tooltip;
					bChangedTooltips = true;
				}
#endif // WITH_EDITOR
				
				// Make sure that if the pin type changed in the ApplyDeprecationBeforeUpdatePins to break incompatible edges.
				if (Pin->bPropertiesHaveChangedOnDeprecation)
				{
					FString Reason{};
					
					if (Pin->BreakAllIncompatibleEdges(&TouchedNodes, !bQuiet ? &Reason : nullptr))
					{
						bAppliedEdgeChanges = true;
						if (!bQuiet)
						{
							UE_LOGF(LogPCG, Warning, "[%ls-%ls] UpdatePins: %ls", *GraphPath, *SettingsName.ToString(), *Reason);	
						}
					}
					
					Pin->bPropertiesHaveChangedOnDeprecation = false;
				}
			}
			else
			{
				UnmatchedPins.Add(Pin);
			}
		}

		// Find unmatched properties vs pins on a name basis
		TArray<FPCGPinProperties> UnmatchedProperties;
		for (const FPCGPinProperties& Properties : PinProperties)
		{
			if (!Pins.FindByPredicate([&Properties](const UPCGPin* Pin) { return Pin->Properties.Label == Properties.Label; }))
			{
				UnmatchedProperties.Add(Properties);
			}
		}

		bool bWasModified = false;
		const bool bUpdateUnmatchedPin = UnmatchedPins.Num() == 1 && UnmatchedProperties.Num() == 1;
		if (bUpdateUnmatchedPin)
		{
			const FName PreviousName = UnmatchedPins[0]->Properties.Label;
			const FName NewName = UnmatchedProperties[0].Label;

			UnmatchedPins[0]->Modify();
			UnmatchedPins[0]->Properties = UnmatchedProperties[0];

			FString Reason{};

			if (UnmatchedPins[0]->BreakAllIncompatibleEdges(&TouchedNodes, !bQuiet ? &Reason : nullptr))
			{
				bAppliedEdgeChanges = true;
				if (!bQuiet)
				{
					UE_LOGF(LogPCG, Warning, "[%ls-%ls] UpdatePins: Pin %ls was matched and renamed to %ls\n%ls", *GraphPath, *SettingsName.ToString(), *PreviousName.ToString(), *NewName.ToString(), *Reason);	
				}
			}

			bChangedPins = true;
		}
		else
		{
			// Verification that we don't have 2 pins with the same name
			// If so, mark them to be removed.
			TSet<FName> AllPinNames;
			TArray<UPCGPin*> DuplicatedNamePins;

			for (UPCGPin* Pin : Pins)
			{
				if (AllPinNames.Contains(Pin->Properties.Label))
				{
					DuplicatedNamePins.Add(Pin);
				}

				AllPinNames.Add(Pin->Properties.Label);
			}

			if(!UnmatchedPins.IsEmpty() || !UnmatchedProperties.IsEmpty() || !DuplicatedNamePins.IsEmpty())
			{
				bWasModified = true;
				Modify();
				bChangedPins = true;
			}

			auto RemovePins = [&Pins, &AllPinNames, &bAppliedEdgeChanges, &TouchedNodes, bQuiet, &SettingsName, &GraphPath](TArray<UPCGPin*>& PinsToRemove, bool bRemoveFromAllNames, bool bWarn)
			{
				for (int32 RemovedPinIndex = PinsToRemove.Num() - 1; RemovedPinIndex >= 0; --RemovedPinIndex)
				{
					const int32 PinIndex = Pins.IndexOfByKey(PinsToRemove[RemovedPinIndex]);
					if (PinIndex >= 0)
					{
						if (bRemoveFromAllNames)
						{
							AllPinNames.Remove(Pins[PinIndex]->Properties.Label);
						}

						TSet<UPCGPin*> OtherPinsTouched{};

						if (Pins[PinIndex]->BreakAllEdges(&TouchedNodes, !bQuiet ? &OtherPinsTouched : nullptr))
						{
							bAppliedEdgeChanges = true;

							if (bWarn && !bQuiet)
							{
								TStringBuilder<512> WarnMessage;
								WarnMessage += FString::Printf(TEXT("[%s-%s] UpdatePins: Pin %s does not exist anymore.\n"), *GraphPath, *SettingsName.ToString(), *Pins[PinIndex]->Properties.Label.ToString());
								WarnMessage += TEXT("List of broken edges:\n");
								for (const UPCGPin* OtherPin : OtherPinsTouched)
								{
									if (!OtherPin)
									{
										continue;
									}

									const FName OtherPinLabel = OtherPin->Properties.Label;
									const UPCGSettings* OtherSettings = OtherPin->Node ? OtherPin->Node->GetSettings() : nullptr;
									const FName OtherSettingsName = OtherSettings ? OtherSettings->GetFName() : "Unknown";

									WarnMessage += FString::Printf(TEXT(" - %s:%s"), *OtherSettingsName.ToString(), *OtherPinLabel.ToString());
								}

								UE_LOGF(LogPCG, Warning, "%ls", WarnMessage.ToString());
							}
						}

						Pins.RemoveAt(PinIndex);
					}
				}
			};

			RemovePins(UnmatchedPins, /*bRemoveFromAllNames=*/true, /*bWarn=*/true);
			RemovePins(DuplicatedNamePins, /*bRemoveFromAllNames=*/false, /*bWarn=*/false);

			// Add new pins
			for (const FPCGPinProperties& UnmatchedProperty : UnmatchedProperties)
			{
				if (ensure(!AllPinNames.Contains(UnmatchedProperty.Label)))
				{
					AllPinNames.Add(UnmatchedProperty.Label);

					const int32 InsertIndex = FMath::Min(PinProperties.IndexOfByKey(UnmatchedProperty), Pins.Num());
					UPCGPin* NewPin = PinAllocator(this);
					NewPin->Modify();
					NewPin->Node = this;
					NewPin->Properties = UnmatchedProperty;
					Pins.Insert(NewPin, InsertIndex);
				}
			}
		}

		// Final pass, to check the order. We re-order if the order is not the same in PinProperties and Pins, without breaking the edges.
		// Also, at this point, we should have the same number of items in the pins and in the pin properties
		check(Pins.Num() == PinProperties.Num());
		for (int32 i = 0; i < PinProperties.Num(); ++i)
		{
			const FPCGPinProperties& CurrentPinProperties = PinProperties[i];
			int32 AssociatedPinPindex = Pins.IndexOfByPredicate([&CurrentPinProperties](const UPCGPin* Pin) -> bool { return Pin->Properties.Label == CurrentPinProperties.Label; });
			if (i != AssociatedPinPindex)
			{
				if (!bWasModified)
				{
					bWasModified = true;
					Modify();
				}

				bChangedPins = true;
				Pins.Swap(i, AssociatedPinPindex);
			}
		}

		return (bAppliedEdgeChanges ? EPCGChangeType::Edge : EPCGChangeType::None)
			| (bChangedPins ? EPCGChangeType::Node : EPCGChangeType::None)
			| (bChangedTooltips ? EPCGChangeType::Cosmetic : EPCGChangeType::None);
	};

	EPCGChangeType ChangeType = EPCGChangeType::None;
	ChangeType |= UpdatePins(MutableView(InputPins), InboundPinProperties);
	ChangeType |= UpdatePins(MutableView(OutputPins), OutboundPinProperties);

#if WITH_EDITOR
	for (UPCGNode* Node : TouchedNodes)
	{
		if (Node)
		{
			// Only this node gets full change type
			Node->OnNodeChangedDelegate.Broadcast(Node, (Node == this) ? ChangeType : EPCGChangeType::Node);
		}
	}
#endif // WITH_EDITOR

	return ChangeType;
}

EPCGChangeType UPCGNode::PropagateDynamicPinTypes(TSet<UPCGNode*>& TouchedNodes, const UPCGNode* FromNode /*= nullptr*/)
{
	// TODO - we have a performance issue - this function can recurse very deeply and take a long time. Perf can be tested by wiring
	// nodes from reroutes in large graph. Functionality can be tested by chaining reroute nodes and checking type propagation.
	// We could probably have a visited set - if we ensure that we update the most-upstream nodes first,
	// so that each node receives the final type? 

	EPCGChangeType ChangeType = EPCGChangeType::None;
	const UPCGSettings* Settings = GetSettings();
	if (!Settings || !Settings->HasDynamicPins())
	{
		return ChangeType;
	}

	ChangeType |= UpdatePins();

	// ChangeType intentionally ignored here - using it breaks propagation of types across chained reroutes.
	TouchedNodes.Add(this);

	TArray<UPCGNode*, TInlineAllocator<8>> OtherNodesPropagatedTo;

	for (UPCGPin* OutputPin : OutputPins)
	{
		for (UPCGEdge* Edge : OutputPin->Edges)
		{
			const UPCGPin* OtherPin = Edge->GetOtherPin(OutputPin);
			if (!OtherPin)
			{
				continue;
			}
			
			UPCGNode* OtherNode = OtherPin->Node;
			if (!OtherNode || OtherNode == FromNode || OtherNodesPropagatedTo.Contains(OtherNode))
			{
				continue;
			}
			
			const UPCGSettings* OtherSettings = OtherNode->GetSettings();
			if (!OtherSettings || !OtherSettings->HasDynamicPins())
			{
				continue;
			}

			OtherNodesPropagatedTo.Add(OtherNode);
			ChangeType |= OtherNode->PropagateDynamicPinTypes(TouchedNodes, this);
		}
	}
	
	return ChangeType;
}

#if WITH_EDITOR

void UPCGNode::GetNodePosition(int32& OutPositionX, int32& OutPositionY) const
{
	OutPositionX = PositionX;
	OutPositionY = PositionY;
}

void UPCGNode::SetNodePosition(int32 InPositionX, int32 InPositionY)
{
	PositionX = InPositionX;
	PositionY = InPositionY;
}

#endif
