// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGParamData.h"
#include "PCGContext.h"
#include "Elements/PCGCollapseElement.h"
#include "Elements/PCGExecuteBlueprint.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGParamData)

#define LOCTEXT_NAMESPACE "PCGParamData"

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoParam, UPCGParamData)

EPCGDataTypeCompatibilityResult FPCGDataTypeInfoParam::IsCompatibleForSubtype(const FPCGDataTypeIdentifier& InType, const FPCGDataTypeIdentifier& OutType, FText* OptionalOutCompatibilityMessage) const
{	
	// We can only compare subtypes if they are all params with valid custom subtype
	if (HasValidSubtype(InType) && HasValidSubtype(OutType))
	{
		if (!PCG::Private::IsBroadcastableOrConstructible(static_cast<uint16>(InType.CustomSubtype), static_cast<uint16>(OutType.CustomSubtype)))
		{
			if (OptionalOutCompatibilityMessage)
			{
				*OptionalOutCompatibilityMessage = FText::Format(LOCTEXT("IncompatibleSubtypes", "Input pin type '{0}' is not a compatible with type '{1}'"), PCG::Private::GetTypeNameText(static_cast<uint16>(InType.CustomSubtype)), PCG::Private::GetTypeNameText(static_cast<uint16>(OutType.CustomSubtype)));
			}

			return EPCGDataTypeCompatibilityResult::TypeCompatibleSubtypeNotCompatible;
		}
		else
		{
			return EPCGDataTypeCompatibilityResult::Compatible;
		}
	}
	else
	{
		return EPCGDataTypeCompatibilityResult::Compatible;
	}
}

bool FPCGDataTypeInfoParam::SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const
{
	// Param can convert to points
	if (OutputType.IsSameType(EPCGDataType::Point))
	{
		if (OptionalOutConversionSettings)
		{
			*OptionalOutConversionSettings = UPCGConvertToPointDataSettings::StaticClass();
		}

		return true;
	}
	else
	{
		return FPCGDataTypeInfo::SupportsConversionTo(ThisType, OutputType, OptionalOutConversionSettings, OptionalOutCompatibilityMessage);
	}
}

#if WITH_EDITOR
TOptional<FText> FPCGDataTypeInfoParam::GetSubtypeTooltip(const FPCGDataTypeIdentifier& ThisType) const
{
	if (HasValidSubtype(ThisType))
	{
		return PCG::Private::GetTypeNameText(static_cast<uint16>(ThisType.CustomSubtype));
	}
	else
	{
		return LOCTEXT("NoTypeParam", "Any");
	}
}
#endif // WITH_EDITOR

bool FPCGDataTypeInfoParam::HasValidSubtype(const FPCGDataTypeIdentifier& Id)
{
	return !Id.IsComposition() && Id.GetId() == AsId() && Id.CustomSubtype >= 0 && Id.CustomSubtype < static_cast<int32>(EPCGMetadataTypes::Count);
}

UPCGParamData::UPCGParamData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Metadata = ObjectInitializer.CreateDefaultSubobject<UPCGMetadata>(this, TEXT("Metadata"));
	for (const FPCGMetadataDomainID& MetadataLevel : UPCGParamData::GetAllSupportedMetadataDomainIDs())
	{
		Metadata->SetupDomain(MetadataLevel, MetadataLevel == UPCGParamData::GetDefaultMetadataDomainID());
	}
}

void UPCGParamData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	FString ClassName = StaticClass()->GetPathName();
	Ar << ClassName;

	if (Metadata)
	{
		Metadata->AddToCrc(Ar, bFullDataCrc);
	}
}

int64 UPCGParamData::FindMetadataKey(const FName& InName) const
{
	if (const PCGMetadataEntryKey* FoundKey = NameMap.Find(InName))
	{
		return *FoundKey;
	}
	else
	{
		return PCGInvalidEntryKey;
	}
}

int64 UPCGParamData::FindOrAddMetadataKey(const FName& InName)
{
	if (const PCGMetadataEntryKey* FoundKey = NameMap.Find(InName))
	{
		return *FoundKey;
	}
	else
	{
		check(Metadata);
		PCGMetadataEntryKey NewKey = Metadata->AddEntry();
		NameMap.Add(InName, NewKey);
		return NewKey;
	}
}

UPCGParamData* UPCGParamData::K2_FilterParamsByName(const FName& InName) const
{
	return FilterParamsByName(UPCGBlueprintBaseElement::ResolveContext(), InName);
}

UPCGParamData* UPCGParamData::FilterParamsByName(FPCGContext* Context, const FName& InName) const
{
	PCGMetadataEntryKey EntryKey = FindMetadataKey(InName);
	UPCGParamData* NewParams = FilterParamsByKey(Context, EntryKey);

	if (EntryKey != PCGInvalidEntryKey)
	{
		// NOTE: this relies on the fact that there will be only one entry
		NewParams->NameMap.Add(InName, 0);
	}

	return NewParams;
}

UPCGParamData* UPCGParamData::K2_FilterParamsByKey(int64 InKey) const
{
	return FilterParamsByKey(UPCGBlueprintBaseElement::ResolveContext(), InKey);
}

UPCGParamData* UPCGParamData::FilterParamsByKey(FPCGContext* Context, int64 InKey) const
{
	UPCGParamData* NewParams = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);

	// Here instead of parenting the metadata, we will create a copy
	// so that the only entry in the metadata (if any) will have the 0 key.
	check(NewParams && NewParams->Metadata);

	NewParams->Metadata->AddAttributes(Metadata);

	if (InKey != PCGInvalidEntryKey)
	{
		PCGMetadataEntryKey OutKey = PCGInvalidEntryKey;
		NewParams->Metadata->SetAttributes(InKey, Metadata, OutKey);
	}

	return NewParams;
}

bool UPCGParamData::HasCachedLastSelector() const
{
	return bHasCachedLastSelector || (Metadata && Metadata->GetAttributeCount() > 0);
}

FPCGAttributePropertyInputSelector UPCGParamData::GetCachedLastSelector() const
{
	if (bHasCachedLastSelector)
	{
		return CachedLastSelector;
	}

	FPCGAttributePropertyInputSelector TempSelector{};

	// If we have attribute and no last selector, create a cached last selector on the latest attribute, to catch "CreateAttribute" calls that didn't use accessors.
	if (Metadata && Metadata->GetAttributeCount() > 0)
	{
		TempSelector.SetAttributeName(Metadata->GetLatestAttributeNameOrNone());
	}

	return TempSelector;
}

void UPCGParamData::SetLastSelector(const FPCGAttributePropertySelector& InSelector)
{
	// Check that it is not a not Attribute selector or Last/Source selector
	if (InSelector.GetSelection() != EPCGAttributePropertySelection::Attribute 
		|| InSelector.GetAttributeName() == PCGMetadataAttributeConstants::LastAttributeName
		|| InSelector.GetAttributeName() == PCGMetadataAttributeConstants::LastCreatedAttributeName
		|| InSelector.GetAttributeName() == PCGMetadataAttributeConstants::SourceAttributeName
		|| InSelector.GetAttributeName() == PCGMetadataAttributeConstants::SourceNameAttributeName)
	{
		return;
	}

	bHasCachedLastSelector = true;
	CachedLastSelector.ImportFromOtherSelector(InSelector);
}

UPCGParamData* UPCGParamData::DuplicateData(FPCGContext* Context, bool bInitializeMetadata) const
{
	UPCGParamData* NewParamData = FPCGContext::NewObject_AnyThread<UPCGParamData>(Context);
	if (bInitializeMetadata)
	{
		check(NewParamData && NewParamData->Metadata);
		NewParamData->Metadata->InitializeAsCopy(FPCGMetadataInitializeParams(Metadata));
	}

	return NewParamData;
}

FPCGMetadataDomainID UPCGParamData::GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const
{
	const FName DomainName = InSelector.GetDomainName();
	if (DomainName == PCGParamDataConstants::ElementsDomainName)
	{
		return PCGMetadataDomainID::Elements;
	}
	else
	{
		return Super::GetMetadataDomainIDFromSelector(InSelector);
	}
}

bool UPCGParamData::MetadataDomainSupportsParenting(const FPCGMetadataDomainID& InDomainID) const
{
	// Param data doesn't support parenting.
	return false;
}

bool UPCGParamData::SetDomainFromDomainID(const FPCGMetadataDomainID& InDomainID, FPCGAttributePropertySelector& InOutSelector) const
{
	if (InDomainID == PCGMetadataDomainID::Elements)
	{
		InOutSelector.SetDomainName(PCGParamDataConstants::ElementsDomainName, /*bResetExtraNames=*/false);
		return true;
	}
	else
	{
		return Super::SetDomainFromDomainID(InDomainID, InOutSelector);
	}
}

#undef LOCTEXT_NAMESPACE