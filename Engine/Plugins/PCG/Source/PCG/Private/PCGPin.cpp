// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGPin.h"

#include "PCGCommon.h"
#include "PCGEdge.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "PCGSettings.h"

#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPin)

#if WITH_EDITOR
namespace PCGPinPropertiesHelpers
{
	bool GetDefaultPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip)
	{
		return InPin && PCGPinPropertiesHelpers::GetDefaultPinExtraIcon(InPin->Properties, OutExtraIcon, OutTooltip);
	}

	bool GetDefaultPinExtraIcon(const FPCGPinProperties& InPinProperties, FName& OutExtraIcon, FText& OutTooltip)
	{
		if (InPinProperties.Usage == EPCGPinUsage::Loop)
		{
			OutTooltip = NSLOCTEXT("PCGPins", "LoopTooltip", "Loop pin, data collection will be split to one data per execution.");
			OutExtraIcon = PCGPinConstants::Icons::LoopPinIcon;
			return true;
		}
		else if (InPinProperties.Usage == EPCGPinUsage::Feedback)
		{
			OutTooltip = NSLOCTEXT("PCGPins", "FeedbackTooltip", "Feedback pin, will daisy-chain results from graph entry or previous loop iteration.");
			OutExtraIcon = PCGPinConstants::Icons::FeedbackPinIcon;
			return true;
		}

		return false;
	}
}
#endif // WITH_EDITOR

FPCGPinProperties::FPCGPinProperties()
{
	AllowedTypes = FPCGDataTypeIdentifier{EPCGDataType::Any};
}

FPCGPinProperties::FPCGPinProperties(const FName& InLabel, FPCGDataTypeIdentifier InAllowedTypes, bool bInAllowMultipleConnections, bool bInAllowMultipleData, const FText& InTooltip)
	: Label(InLabel), AllowedTypes(MoveTemp(InAllowedTypes)), bAllowMultipleData(bInAllowMultipleData)
#if WITH_EDITORONLY_DATA
	, Tooltip(InTooltip)
#endif
{
	SetAllowMultipleConnections(bInAllowMultipleConnections);
}

void FPCGPinProperties::SetAllowMultipleConnections(bool bInAllowMultipleConnectons)
{
	if (bInAllowMultipleConnectons)
	{
		bAllowMultipleConnections = true;
		bAllowMultipleData = true;
	}
	else
	{
		bAllowMultipleConnections = false;
	}
}

bool FPCGPinProperties::operator==(const FPCGPinProperties& Other) const
{
	return Label == Other.Label &&
		AllowedTypes == Other.AllowedTypes &&
		bAllowMultipleConnections == Other.bAllowMultipleConnections &&
		bAllowMultipleData == Other.bAllowMultipleData &&
		Usage == Other.Usage &&
		PinStatus == Other.PinStatus &&
		bInvisiblePin == Other.bInvisiblePin;
}

uint32 GetTypeHash(const FPCGPinProperties& Value)
{
	uint32 Hash = GetTypeHash(Value.Label);
	Hash = HashCombine(Hash, GetTypeHash(Value.AllowedTypes));
	Hash = HashCombine(Hash, Value.bAllowMultipleConnections);
	Hash = HashCombine(Hash, Value.bAllowMultipleData);
	Hash = HashCombine(Hash, static_cast<uint32>(Value.Usage));
	Hash = HashCombine(Hash, static_cast<uint32>(Value.PinStatus));
	Hash = HashCombine(Hash, Value.bInvisiblePin);

	return Hash;
}

void FPCGPinProperties::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.IsLoading() && Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
	{
		if (bAdvancedPin_DEPRECATED)
		{
			PinStatus = EPCGPinStatus::Advanced;
		}

		bAdvancedPin_DEPRECATED = false;
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
bool FPCGPinProperties::CanEditChange(const FEditPropertyChain& PropertyChain) const
{
	if (FProperty* Property = PropertyChain.GetActiveNode()->GetValue())
	{
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, bAllowMultipleData))
		{
			return bAllowEditMultipleData;
		}
		else if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(FPCGPinProperties, bAllowMultipleConnections))
		{
			return bAllowMultipleData && bAllowEditMultipleConnections;
		}
	}

	return true;
}
#endif // WITH_EDITOR

UPCGPin::UPCGPin(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetFlags(RF_Transactional);
}

void UPCGPin::PostLoad()
{
	Super::PostLoad();

	if (Label_DEPRECATED != NAME_None)
	{
		Properties = FPCGPinProperties(Label_DEPRECATED);
		Label_DEPRECATED = NAME_None;
	}
}

FText UPCGPin::GetTooltip() const
{
#if WITH_EDITOR
	return Properties.Tooltip;
#else
	return FText::GetEmpty();
#endif
}

void UPCGPin::SetTooltip(const FText& InTooltip)
{
#if WITH_EDITOR
	Properties.Tooltip = InTooltip;
#endif
}

bool UPCGPin::AddEdgeTo(UPCGPin* OtherPin, TSet<UPCGNode*>* InTouchedNodes/*= nullptr*/)
{
	if (!OtherPin)
	{
		return false;
	}

	for (UPCGEdge* Edge : Edges)
	{
		if (Edge->GetOtherPin(this) == OtherPin)
		{
			return false;
		}
	}

	// This pin is upstream if the pin is an output pin
	const bool bThisPinIsUpstream = IsOutputPin();
	const bool bOtherPinIsUpstream = OtherPin->IsOutputPin();

	// Pins should not both be upstream or both be downstream..
	if (!ensure(bThisPinIsUpstream != bOtherPinIsUpstream))
	{
		return false;
	}

	Modify();
	OtherPin->Modify();

	UPCGEdge* NewEdge = Edges.Add_GetRef(NewObject<UPCGEdge>(this));
	OtherPin->Edges.Add(NewEdge);
	
	NewEdge->Modify();
	NewEdge->InputPin = bThisPinIsUpstream ? this : OtherPin;
	NewEdge->OutputPin = bThisPinIsUpstream ? OtherPin : this;

	if (InTouchedNodes)
	{
		InTouchedNodes->Add(Node);
		InTouchedNodes->Add(OtherPin->Node);
	}

	return true;
}

bool UPCGPin::BreakEdgeTo(UPCGPin* OtherPin, TSet<UPCGNode*>* InTouchedNodes/*= nullptr*/)
{
	if (!OtherPin)
	{
		return false;
	}

	for (UPCGEdge* Edge : Edges)
	{
		if (Edge->GetOtherPin(this) == OtherPin)
		{
			Modify();
			OtherPin->Modify();

			ensure(OtherPin->Edges.Remove(Edge));
			Edges.Remove(Edge);

			// In case this happens before Edge is fully loaded we need to invalidate its loading before renaming it out to the transient package.
			FLinkerLoad::InvalidateExport(Edge);
			Edge->Rename(/*NewName=*/nullptr, GetTransientPackage(), REN_DontCreateRedirectors);

			if (InTouchedNodes)
			{
				InTouchedNodes->Add(Node);
				InTouchedNodes->Add(OtherPin->Node);
			}

			return true;
		}
	}

	return false;
}

bool UPCGPin::BreakAllEdges(TSet<UPCGNode*>* InTouchedNodes, TSet<UPCGPin*>* OptionalOtherPinsTouched)
{
	bool bChanged = false;

	if (!Edges.IsEmpty())
	{
		if (InTouchedNodes)
		{
			InTouchedNodes->Add(Node);
		}

		Modify();
	}

	for (UPCGEdge* Edge : Edges)
	{
		if (UPCGPin* OtherPin = Edge->GetOtherPin(this))
		{
			OtherPin->Modify();
			ensure(OtherPin->Edges.Remove(Edge));
			
			// In case this happens before Edge is fully loaded we need to invalidate its loading before renaming it out to the transient package.
			FLinkerLoad::InvalidateExport(Edge);
			Edge->Rename(/*NewName=*/nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
			bChanged = true;

			if (InTouchedNodes)
			{
				InTouchedNodes->Add(OtherPin->Node);
			}

			if (OptionalOtherPinsTouched)
			{
				OptionalOtherPinsTouched->Add(OtherPin);
			}
		}
	}

	Edges.Reset();

	return bChanged;
}

bool UPCGPin::BreakAllIncompatibleEdges(TSet<UPCGNode*>* InTouchedNodes/*= nullptr*/, FString* OptionalReason)
{
	bool bChanged = false;
	bool bHasAValidEdge = false;

	auto AppendReason = [OptionalReason](FString InReason)
	{
		if (!OptionalReason)
		{
			return;
		}

		if (OptionalReason->IsEmpty())
		{
			*OptionalReason = MoveTemp(InReason);
		}
		else
		{
			*OptionalReason = FString::Printf(TEXT("%s\n%s"), **OptionalReason, *InReason);
		}
	};

	const FName ThisPinName = Properties.Label;

	for (int32 EdgeIndex = Edges.Num() - 1; EdgeIndex >= 0; --EdgeIndex)
	{
		UPCGEdge* Edge = Edges[EdgeIndex];
		UPCGPin* OtherPin = Edge->GetOtherPin(this);

		const UPCGNode* OtherNode = OtherPin ? Cast<const UPCGNode>(OtherPin->GetOuter()) : nullptr;
		const FName OtherNodeName = OtherNode ? OtherNode->GetFName() : FName("Unknown");
		const FName OtherPinName = OtherPin ? OtherPin->Properties.Label : FName("Unknown");

		bool bRemoveEdge = (!AllowsMultipleConnections() && bHasAValidEdge);
		if (bRemoveEdge && OptionalReason)
		{
			AppendReason(FString::Printf(TEXT("Breaking edge between %s on node %s and %s because the pin does not support multi connection."),
				*OtherPinName.ToString(),
				*OtherNodeName.ToString(),
				*ThisPinName.ToString()));
		}

		// If the edge is not compatible because of a subtype mismatch, we do not want to break it for deprecation purposes.
		const EPCGDataTypeCompatibilityResult Compatibility = GetCompatibilityWithOtherPin(OtherPin);
		if (!PCGDataTypeCompatibilityResult::IsValid(Compatibility) && Compatibility != EPCGDataTypeCompatibilityResult::TypeCompatibleSubtypeNotCompatible)
		{
			if (OptionalReason)
			{
				AppendReason(FString::Printf(TEXT("Breaking edge between %s on node %s and %s because the pin types are not compatible."),
					*OtherPinName.ToString(),
					*OtherNodeName.ToString(),
					*ThisPinName.ToString()));
			}
			
			bRemoveEdge = true;
		}

		if (bRemoveEdge)
		{
			Modify();
			Edges.RemoveAtSwap(EdgeIndex);

			if (InTouchedNodes)
			{
				InTouchedNodes->Add(Node);
			}

			if (OtherPin)
			{
				OtherPin->Modify();
				ensure(OtherPin->Edges.Remove(Edge));
				bChanged = true;

				if (InTouchedNodes)
				{
					InTouchedNodes->Add(OtherPin->Node);
				}
			}
		}
		else
		{
			bHasAValidEdge = true;
		}
	}

	return bChanged;
}

bool UPCGPin::IsConnected() const
{
	for (const UPCGEdge* Edge : Edges)
	{
		if (Edge->IsValid())
		{
			return true;
		}
	}

	return false;
}

bool UPCGPin::IsOutputPin() const
{
	check(Node);
	return Node->GetOutputPin(Properties.Label) == this;
}

int32 UPCGPin::EdgeCount() const
{
	int32 EdgeNum = 0;
	for (const UPCGEdge* Edge : Edges)
	{
		if (Edge->IsValid())
		{
			++EdgeNum;
		}
	}

	return EdgeNum;
}

EPCGDataType UPCGPin::GetCurrentTypes() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetCurrentTypesID();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FPCGDataTypeIdentifier UPCGPin::GetCurrentTypesID() const
{
	check(Node);
	const UPCGSettings* Settings = Node->GetSettings();
	return Settings ? Settings->GetCurrentPinTypesID(this) : Properties.AllowedTypes;
}

bool UPCGPin::IsCompatible(const UPCGPin* OtherPin) const
{
	const EPCGDataTypeCompatibilityResult Compatibility = GetCompatibilityWithOtherPin(OtherPin);
	return PCGDataTypeCompatibilityResult::IsValid(Compatibility);
}

EPCGDataTypeCompatibilityResult UPCGPin::GetCompatibilityWithOtherPin(const UPCGPin* OtherPin, FText* OptionalOutCompatibilityMessage) const
{
	const bool bThisIsOutputPin = IsOutputPin();
	if (!OtherPin || bThisIsOutputPin == OtherPin->IsOutputPin())
	{
		return EPCGDataTypeCompatibilityResult::NotCompatible;
	}

	const UPCGPin* UpstreamPin = bThisIsOutputPin ? this : OtherPin;
	const UPCGPin* DownstreamPin = bThisIsOutputPin ? OtherPin : this;

	return FPCGModule::GetConstDataTypeRegistry().IsCompatible(UpstreamPin->GetCurrentTypesID(), DownstreamPin->Properties.AllowedTypes, OptionalOutCompatibilityMessage);
}

bool UPCGPin::IsDownstreamPinTypeCompatible(const EPCGDataType UpstreamTypes) const
{
	const EPCGDataTypeCompatibilityResult Compatibility = FPCGModule::GetConstDataTypeRegistry().IsCompatible(FPCGDataTypeIdentifier{UpstreamTypes}, Properties.AllowedTypes);
	return PCGDataTypeCompatibilityResult::IsValid(Compatibility);
}

bool UPCGPin::AllowsMultipleConnections() const
{
	// Always allow multiple connection on output pin
	return IsOutputPin() || Properties.AllowsMultipleConnections();
}

bool UPCGPin::AllowsMultipleData() const
{
	return Properties.bAllowMultipleData;
}

bool UPCGPin::CanConnect(const UPCGPin* OtherPin) const
{
	return OtherPin && (Edges.IsEmpty() || AllowsMultipleConnections());
}
