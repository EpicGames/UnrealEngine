// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGPoint.h"

#include "PCGCommon.h"

#include "Metadata/Accessors/PCGCustomAccessor.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"

#include "Math/BoxSphereBounds.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPoint)

/** Serialized fields of a FPCGPoint, the values here can't change as they are being used to mask out serialization */
enum class EPCGPointSerializeFields : uint8
{
	None = 0 ,
	Density = 1 << 0,
	BoundsMin = 1 << 1,
	BoundsMax = 1 << 2,
	Color = 1 << 3,
	Steepness = 1 << 4,
	Seed = 1 << 5,
	MetadataEntry = 1 << 6
};

ENUM_CLASS_FLAGS(EPCGPointSerializeFields);

FPCGPoint::FPCGPoint(const FTransform& InTransform, float InDensity, int32 InSeed)
	: Transform(InTransform)
	, Density(InDensity)
	, Seed(InSeed)
{
}

bool FPCGPoint::Serialize(FStructuredArchive::FSlot Slot)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	
	// Usage of a branch version instead of the PCG version ensures that we can't end up in a situation where two developers modify the PCG version in
	// two different branches causing issues with saved assets in those branches when integrating.
	UnderlyingArchive.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

	// Previous versions was using default serialization, returning false here ensures older data gets loaded through the default serialization
	if (UnderlyingArchive.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::PCGPointStructuredSerializer)
	{
		return false;
	}
	
	const FPCGPoint Default;
	EPCGPointSerializeFields SerializeMask = EPCGPointSerializeFields::None;
	if (!UnderlyingArchive.IsLoading())
	{
		if (Density != Default.Density)
		{
			EnumAddFlags(SerializeMask, EPCGPointSerializeFields::Density);
		}

		if (BoundsMin != Default.BoundsMin)
		{
			EnumAddFlags(SerializeMask, EPCGPointSerializeFields::BoundsMin);
		}

		if (BoundsMax != Default.BoundsMax)
		{
			EnumAddFlags(SerializeMask, EPCGPointSerializeFields::BoundsMax);
		}

		if (Color != Default.Color)
		{
			EnumAddFlags(SerializeMask, EPCGPointSerializeFields::Color);
		}

		if (Steepness != Default.Steepness)
		{
			EnumAddFlags(SerializeMask, EPCGPointSerializeFields::Steepness);
		}

		if (Seed != Default.Seed)
		{
			EnumAddFlags(SerializeMask, EPCGPointSerializeFields::Seed);
		}

		if (MetadataEntry != Default.MetadataEntry)
		{
			EnumAddFlags(SerializeMask, EPCGPointSerializeFields::MetadataEntry);
		}
	}
		
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	Record << SA_VALUE(TEXT("SerializeMask"), SerializeMask);
	Record << SA_VALUE(TEXT("Transform"), Transform);

	if (EnumHasAnyFlags(SerializeMask, EPCGPointSerializeFields::Density))
	{
		Record << SA_VALUE(TEXT("Density"), Density);
	}
		
	if (EnumHasAnyFlags(SerializeMask, EPCGPointSerializeFields::BoundsMin))
	{
		Record << SA_VALUE(TEXT("BoundsMin"), BoundsMin);
	}
	
	if (EnumHasAnyFlags(SerializeMask, EPCGPointSerializeFields::BoundsMax))
	{
		Record << SA_VALUE(TEXT("BoundsMax"), BoundsMax);
	}
	
	if (EnumHasAnyFlags(SerializeMask, EPCGPointSerializeFields::Color))
	{
		Record << SA_VALUE(TEXT("Color"), Color);
	}
	
	if (EnumHasAnyFlags(SerializeMask, EPCGPointSerializeFields::Steepness))
	{
		Record << SA_VALUE(TEXT("Steepness"), Steepness);
	}
	
	if (EnumHasAnyFlags(SerializeMask, EPCGPointSerializeFields::Seed))
	{
		Record << SA_VALUE(TEXT("Seed"), Seed);
	}
	
	if (EnumHasAnyFlags(SerializeMask, EPCGPointSerializeFields::MetadataEntry))
	{
		Record << SA_VALUE(TEXT("MetadataEntry"), MetadataEntry);
	}
	
	return true;
}

bool FPCGPoint::HasCustomPropertyGetterSetter(FName Name)
{
	return PCGPointCustomPropertyNames::IsCustomPropertyName(Name);
}

TUniquePtr<IPCGAttributeAccessor> FPCGPoint::CreateCustomPropertyAccessor(FName Name)
{
	if (Name == PCGPointCustomPropertyNames::ExtentsName)
	{
		return MakeUnique<FPCGCustomPointAccessor<FVector>>(
			[](const FPCGPoint& Point, void* OutValue) { *reinterpret_cast<FVector*>(OutValue) = Point.GetExtents(); return true; },
			[](FPCGPoint& Point, const void* InValue) { Point.SetExtents(*reinterpret_cast<const FVector*>(InValue)); return true; }
		);
	}
	else if (Name == PCGPointCustomPropertyNames::LocalCenterName)
	{
		return MakeUnique<FPCGCustomPointAccessor<FVector>>(
			[](const FPCGPoint& Point, void* OutValue) { *reinterpret_cast<FVector*>(OutValue) = Point.GetLocalCenter(); return true; },
			[](FPCGPoint& Point, const void* InValue) { Point.SetLocalCenter(*reinterpret_cast<const FVector*>(InValue)); return true; }
		);
	}
	else if (Name == PCGPointCustomPropertyNames::PositionName)
	{
		return MakeUnique<FPCGCustomPointAccessor<FVector>>(
			[](const FPCGPoint& Point, void* OutValue) { *reinterpret_cast<FVector*>(OutValue) = Point.Transform.GetLocation(); return true; },
			[](FPCGPoint& Point, const void* InValue) { Point.Transform.SetLocation(*reinterpret_cast<const FVector*>(InValue)); return true; }
		);
	}
	else if (Name == PCGPointCustomPropertyNames::RotationName)
	{
		return MakeUnique<FPCGCustomPointAccessor<FQuat>>(
			[](const FPCGPoint& Point, void* OutValue) { *reinterpret_cast<FQuat*>(OutValue) = Point.Transform.GetRotation(); return true; },
			[](FPCGPoint& Point, const void* InValue) { Point.Transform.SetRotation(*reinterpret_cast<const FQuat*>(InValue)); return true; }
		);
	}
	else if (Name == PCGPointCustomPropertyNames::ScaleName)
	{
		return MakeUnique<FPCGCustomPointAccessor<FVector>>(
			[](const FPCGPoint& Point, void* OutValue) { *reinterpret_cast<FVector*>(OutValue) = Point.Transform.GetScale3D(); return true; },
			[](FPCGPoint& Point, const void* InValue) { Point.Transform.SetScale3D(*reinterpret_cast<const FVector*>(InValue)); return true; }
		);
	}
	else if (Name == PCGPointCustomPropertyNames::LocalSizeName)
	{
		return MakeUnique<FPCGCustomPointAccessor<FVector>>(
			[](const FPCGPoint& Point, void* OutValue) { *reinterpret_cast<FVector*>(OutValue) = Point.GetLocalSize(); return true; }
		);
	}
	else if (Name == PCGPointCustomPropertyNames::ScaledLocalSizeName)
	{
		return MakeUnique<FPCGCustomPointAccessor<FVector>>(
			[](const FPCGPoint& Point, void* OutValue) { *reinterpret_cast<FVector*>(OutValue) = Point.GetScaledLocalSize(); return true; }
		);
	}

	return TUniquePtr<IPCGAttributeAccessor>();
}
