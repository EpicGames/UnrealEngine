// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGPointArray.h"

#include "PCGModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPointArray)

FPCGPointArray::FPCGPointArray()
{
	FPCGPoint DefaultPoint;
	Transform.SetValue(DefaultPoint.Transform);
	Density.SetValue(DefaultPoint.Density);
	BoundsMin.SetValue(DefaultPoint.BoundsMin);
	BoundsMax.SetValue(DefaultPoint.BoundsMax);
	Color.SetValue(DefaultPoint.Color);
	Steepness.SetValue(DefaultPoint.Steepness);
	Seed.SetValue(DefaultPoint.Seed);
	MetadataEntry.SetValue(DefaultPoint.MetadataEntry);
}

void FPCGPointArray::SetNumPoints(int32 InNumPoints, bool bInitializeValues)
{
	NumPoints = InNumPoints;
	bInitializedValues = bInitializeValues;

	// Reallocate arrays that are currently allocated
	Transform.SetNum(NumPoints, /*bAllocate=*/false, bInitializedValues);
	Density.SetNum(NumPoints, /*bAllocate=*/false, bInitializedValues);
	BoundsMin.SetNum(NumPoints, /*bAllocate=*/false, bInitializedValues);
	BoundsMax.SetNum(NumPoints, /*bAllocate=*/false, bInitializedValues);
	Color.SetNum(NumPoints, /*bAllocate=*/false, bInitializedValues);
	Steepness.SetNum(NumPoints, /*bAllocate=*/false, bInitializedValues);
	Seed.SetNum(NumPoints, /*bAllocate=*/false, bInitializedValues);
	MetadataEntry.SetNum(NumPoints, /*bAllocate=*/false, bInitializedValues);
}

void FPCGPointArray::RemoveAt(int32 Index)
{
	if(Index >= 0 && Index < NumPoints)
	{
		NumPoints -= 1;

		Transform.RemoveAt(Index);
		Density.RemoveAt(Index);
		BoundsMin.RemoveAt(Index);
		BoundsMax.RemoveAt(Index);
		Color.RemoveAt(Index);
		Steepness.RemoveAt(Index);
		Seed.RemoveAt(Index);
		MetadataEntry.RemoveAt(Index);

		if(NumPoints == 1)
		{
			bInitializedValues = false;
		}
	}
}

void FPCGPointArray::Allocate(EPCGPointNativeProperties InProperties)
{
	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::Transform))
	{
		Transform.Allocate(bInitializedValues);
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::Density))
	{
		Density.Allocate(bInitializedValues);
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::BoundsMin))
	{
		BoundsMin.Allocate(bInitializedValues);
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::BoundsMax))
	{
		BoundsMax.Allocate(bInitializedValues);
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::Color))
	{
		Color.Allocate(bInitializedValues);
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::Steepness))
	{
		Steepness.Allocate(bInitializedValues);
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::Seed))
	{
		Seed.Allocate(bInitializedValues);
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::MetadataEntry))
	{
		MetadataEntry.Allocate(bInitializedValues);
	}
}

void FPCGPointArray::Free(EPCGPointNativeProperties InProperties)
{
	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::Transform))
	{
		Transform.Free();
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::Density))
	{
		Density.Free();
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::BoundsMin))
	{
		BoundsMin.Free();
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::BoundsMax))
	{
		BoundsMax.Free();
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::Color))
	{
		Color.Free();
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::Steepness))
	{
		Steepness.Free();
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::Seed))
	{
		Seed.Free();
	}

	if (EnumHasAnyFlags(InProperties, EPCGPointNativeProperties::MetadataEntry))
	{
		MetadataEntry.Free();
	}
}

void FPCGPointArray::MoveRange(int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements)
{
	if (RangeStartIndex != MoveToIndex && NumElements > 0)
	{
		Transform.MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		Density.MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		BoundsMin.MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		BoundsMax.MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		Color.MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		Steepness.MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		Seed.MoveRange(RangeStartIndex, MoveToIndex, NumElements);
		MetadataEntry.MoveRange(RangeStartIndex, MoveToIndex, NumElements);
	}
}

TArray<FTransform> FPCGPointArray::GetTransformCopy() const
{
	return Transform.GetCopy();
}

bool FPCGPointArray::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYTAG(PCG);

	Ar << NumPoints;
	Transform.Serialize(Ar);
	Density.Serialize(Ar);
	BoundsMin.Serialize(Ar);
	BoundsMax.Serialize(Ar);
	Color.Serialize(Ar);
	Steepness.Serialize(Ar);
	Seed.Serialize(Ar);
	MetadataEntry.Serialize(Ar);

	return true;
}

void FPCGPointArray::CopyPropertiesTo(FPCGPointArray& OutPointArray, int32 StartReadIndex, int32 StartWriteIndex, int32 Count, EPCGPointNativeProperties Properties) const
{
	if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::Transform))
	{
		Transform.CopyTo(OutPointArray.Transform, StartReadIndex, StartWriteIndex, Count);
	}

	if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::Density))
	{
		Density.CopyTo(OutPointArray.Density, StartReadIndex, StartWriteIndex, Count);
	}

	if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::BoundsMin))
	{
		BoundsMin.CopyTo(OutPointArray.BoundsMin, StartReadIndex, StartWriteIndex, Count);
	}

	if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::BoundsMax))
	{
		BoundsMax.CopyTo(OutPointArray.BoundsMax, StartReadIndex, StartWriteIndex, Count);
	}

	if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::Color))
	{
		Color.CopyTo(OutPointArray.Color, StartReadIndex, StartWriteIndex, Count);
	}

	if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::Steepness))
	{
		Steepness.CopyTo(OutPointArray.Steepness, StartReadIndex, StartWriteIndex, Count);
	}

	if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::Seed))
	{
		Seed.CopyTo(OutPointArray.Seed, StartReadIndex, StartWriteIndex, Count);
	}

	if (EnumHasAllFlags(Properties, EPCGPointNativeProperties::MetadataEntry))
	{
		MetadataEntry.CopyTo(OutPointArray.MetadataEntry, StartReadIndex, StartWriteIndex, Count);
	}
}

EPCGPointNativeProperties FPCGPointArray::GetAllocatedProperties() const
{
	EPCGPointNativeProperties AllocatedProperties = EPCGPointNativeProperties::None;

	if (Transform.IsAllocated())
	{
		AllocatedProperties |= EPCGPointNativeProperties::Transform;
	}

	if (Density.IsAllocated())
	{
		AllocatedProperties |= EPCGPointNativeProperties::Density;
	}

	if (BoundsMin.IsAllocated())
	{
		AllocatedProperties |= EPCGPointNativeProperties::BoundsMin;
	}

	if (BoundsMax.IsAllocated())
	{
		AllocatedProperties |= EPCGPointNativeProperties::BoundsMax;
	}

	if (Color.IsAllocated())
	{
		AllocatedProperties |= EPCGPointNativeProperties::Color;
	}

	if (Steepness.IsAllocated())
	{
		AllocatedProperties |= EPCGPointNativeProperties::Steepness;
	}

	if (Seed.IsAllocated())
	{
		AllocatedProperties |= EPCGPointNativeProperties::Seed;
	}

	if (MetadataEntry.IsAllocated())
	{
		AllocatedProperties |= EPCGPointNativeProperties::MetadataEntry;
	}

	return AllocatedProperties;
}
