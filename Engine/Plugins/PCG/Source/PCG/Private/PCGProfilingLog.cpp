// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGProfilingLog.h"

#include "CoreMinimal.h"

#if PCG_PROFILING_ENABLED

#include "PCGGraph.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "PCGSettings.h"
#include "PCGSubgraph.h"
#include "Subsystems/IPCGBaseSubsystem.h"

#include "HAL/IConsoleManager.h"

namespace PCGProfilingLog
{
    static bool bLogProfilingData = false;
    static FAutoConsoleVariableRef CVarPCGLogProfilingData(
        TEXT("pcg.LogProfilingData"),
        bLogProfilingData,
        TEXT("If true, logs PCG node profiling data (CPU/GPU times and memory) to the output log after each graph execution. Editor and non-shipping builds only."));

    static FString GetNodeName(const UPCGNode* InNode)
    {
        if (!InNode)
        {
            return TEXT("<unknown>");
        }

        // User-authored rename (set in graph editor) takes priority.
        if (InNode->HasAuthoredTitle())
        {
            return InNode->GetAuthoredTitleLine().ToString();
        }

        // Generated title contains useful runtime info (e.g. function/asset name for CustomHLSL).
        FText GeneratedTitle = InNode->GetGeneratedTitleLine();
        if (!GeneratedTitle.IsEmpty())
        {
            return GeneratedTitle.ToString();
        }

        // Fall back to stripping "PCG" prefix and "Settings" suffix from the class name.
        const UPCGSettings* Settings = InNode->GetSettings();
        if (!Settings)
        {
            return TEXT("<unknown>");
        }

        FString Name = Settings->GetClass()->GetName();
        if (Name.StartsWith(TEXT("PCG")))
        {
            Name = Name.Mid(3);
        }
        if (Name.EndsWith(TEXT("Settings")))
        {
            Name = Name.LeftChop(8);
        }
        return Name;
    }

    static void LogRow(const FString& IndentedName, const PCGUtils::FCallTime& Timer, int32 CallCount)
    {
        const double PrepMs = Timer.PrepareDataTime * 1000.0;
        const double ExecMs = Timer.ExecutionTime * 1000.0;
        const double PostMs = Timer.PostExecuteTime * 1000.0;
        const double GpuMs  = Timer.GPUTime.Get(0.0) * 1000.0;
        const double CpuMB  = Timer.OutputCPUMemorySize.Get(0) / (1024.0 * 1024.0);
        const double GpuMB  = Timer.OutputGPUMemorySize.Get(0) / (1024.0 * 1024.0);

        const FString DisplayName = IndentedName.Len() > 40 ? IndentedName.Left(37) + TEXT("...") : IndentedName;
        UE_LOGF(LogPCG, Log, "  %-40ls  %6d  %13.3f  %13.3f  %13.3f  %9.3f  %13.3f  %13.3f", *DisplayName, CallCount, PrepMs, ExecMs, PostMs, GpuMs, CpuMB, GpuMB);
    }
}

bool PCGProfilingLog::IsEnabled()
{
    return bLogProfilingData;
}

void PCGProfilingLog::LogProfilingData(IPCGGraphExecutionSource* InExecutionSource, EPCGGenerationStatus InStatus)
{
    check(IsInGameThread());

    if (InStatus == EPCGGenerationStatus::Aborted || !InExecutionSource)
    {
        return;
    }

    IPCGGraphExecutionState& State = InExecutionSource->GetExecutionState();
    FPCGGraphExecutionInspection& Inspection = State.GetInspection();
    TMap<TObjectKey<const UPCGNode>, TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData>> ExecutedStacks = Inspection.GetExecutedNodeStacks();

    if (ExecutedStacks.IsEmpty())
    {
        return;
    }

    // Aggregate per-node totals across all loop iterations and stacks.
    struct FNodeStats
    {
        PCGUtils::FCallTime Timer;
        int32 CallCount = 0;
    };

    TMap<TObjectKey<const UPCGNode>, FNodeStats> Timers;

    for (const auto& [NodeKey, Executions] : ExecutedStacks)
    {
        FNodeStats& Stats = Timers.FindOrAdd(NodeKey);
        Stats.CallCount += Executions.Num();

        for (const FPCGGraphExecutionInspection::FNodeExecutedNotificationData& Exec : Executions)
        {
            Stats.Timer.PrepareDataTime += Exec.Timer.PrepareDataTime;
            Stats.Timer.ExecutionTime += Exec.Timer.ExecutionTime;
            Stats.Timer.PostExecuteTime += Exec.Timer.PostExecuteTime;
            if (Exec.Timer.GPUTime.IsSet())
            {
                Stats.Timer.GPUTime = Stats.Timer.GPUTime.Get(0.0) + Exec.Timer.GPUTime.GetValue();
            }
            if (Exec.Timer.OutputCPUMemorySize.IsSet())
            {
                Stats.Timer.OutputCPUMemorySize = Stats.Timer.OutputCPUMemorySize.Get(0) + Exec.Timer.OutputCPUMemorySize.GetValue();
            }
            if (Exec.Timer.OutputGPUMemorySize.IsSet())
            {
                Stats.Timer.OutputGPUMemorySize = Stats.Timer.OutputGPUMemorySize.Get(0) + Exec.Timer.OutputGPUMemorySize.GetValue();
            }
        }
    }

#if !WITH_EDITOR
    // In editor the inspection system manages its own lifecycle; in non-editor we clear after reading.
    Inspection.ClearExecutedNodeData();
#endif

    struct FRow
    {
        FString Name;
        const UPCGNode* Node = nullptr;
        PCGUtils::FCallTime Timer;
        int32 CallCount = 0;

        double TotalMs() const
        {
            return (Timer.PrepareDataTime + Timer.ExecutionTime + Timer.PostExecuteTime + Timer.GPUTime.Get(0.0)) * 1000.0;
        }
    };

    TArray<FRow> Rows;
    Rows.Reserve(Timers.Num());
    for (const auto& Pair : Timers)
    {
        const UPCGNode* Node = Pair.Key.ResolveObjectPtr();
        Rows.Add({ .Name = PCGProfilingLog::GetNodeName(Node), .Node = Node, .Timer = Pair.Value.Timer, .CallCount = Pair.Value.CallCount });
    }

    // Group rows by their owning graph; sort each group descending by total cost.
    TMap<UPCGGraph*, TArray<int32>> RowsByGraph;
    for (int32 i = 0; i < Rows.Num(); ++i)
    {
        UPCGGraph* Graph = Rows[i].Node ? Rows[i].Node->GetGraph() : nullptr;
        RowsByGraph.FindOrAdd(Graph).Add(i);
    }
    for (auto& Pair : RowsByGraph)
    {
        Pair.Value.Sort([&Rows](int32 A, int32 B) { return Rows[A].TotalMs() > Rows[B].TotalMs(); });
    }

    UPCGGraph* RootGraph = State.GetGraph();
    const FString SourceName = State.GetDebugName();
    const FString GraphName = RootGraph ? RootGraph->GetName() : FString(TEXT("<unknown>"));
    const uint32 GridSize = State.GetGenerationGridSize();
    const FString GridStr = (GridSize != 0) ? FString::Printf(TEXT(" Grid: %u"), GridSize) : FString();

    // Table: 2 indent + 40 node + (2 sep + 6 calls) + 3x(2 sep + 13 CPU time) + (2 sep + 9 GPU ms) + 2x(2 sep + 13 memory) = 2+40+8+45+11+30 = 136 chars wide.
    const int32 TableWidth = 136;
    const FString Sep = FString(TEXT("  ")) + FString::ChrN(TableWidth - 2, '-');
    const int32 TitleStart = TableWidth / 5;
    const FString TitleContent = FString::Printf(TEXT(" %ls [%ls]%ls "), *SourceName, *GraphName, *GridStr);
    const FString TitleLine = FString(TEXT("  ")) + FString::ChrN(TitleStart - 2, '-') + TitleContent + FString::ChrN(FMath::Max(0, TableWidth - TitleStart - TitleContent.Len()), '-');

    UE_LOG(LogPCG, Log, TEXT(" "));
    UE_LOGF(LogPCG, Log, "%ls", *TitleLine);
    UE_LOGF(LogPCG, Log, "  %-40s  %6s  %13s  %13s  %13s  %9s  %13s  %13s", "Node", "Calls", "CPU Prep (ms)", "CPU Exec (ms)", "CPU Post (ms)", "GPU (ms)", "CPU data (MB)", "GPU data (MB)");
    UE_LOGF(LogPCG, Log, "%ls", *Sep);

    // Recursive: print nodes for Graph at the given indent depth, then recurse into any subgraphs.
    TSet<UPCGGraph*> PrintedGraphs;
    TFunction<void(UPCGGraph*, int32)> LogGraph;
    LogGraph = [&](UPCGGraph* Graph, int32 Depth)
    {
        if (!Graph || PrintedGraphs.Contains(Graph))
        {
            return;
        }

        PrintedGraphs.Add(Graph);

        const FString Indent = FString::ChrN(Depth * 2, ' ');

        if (const TArray<int32>* Indices = RowsByGraph.Find(Graph))
        {
            for (int32 Idx : *Indices)
            {
                const FRow& Row = Rows[Idx];
                PCGProfilingLog::LogRow(Indent + Row.Name, Row.Timer, Row.CallCount);
            }
        }

        // Walk the graph's node list to find subgraph calls and recurse in graph-order.
        Graph->ForEachNode([&](UPCGNode* Node) -> bool
        {
            if (const UPCGBaseSubgraphNode* SubgraphNode = Cast<UPCGBaseSubgraphNode>(Node))
            {
                UPCGGraph* ChildGraph = SubgraphNode->GetSubgraph().Get();
                if (ChildGraph && RowsByGraph.Contains(ChildGraph) && !PrintedGraphs.Contains(ChildGraph))
                {
                    const FString NodeLabel = PCGProfilingLog::GetNodeName(Node);
                    UE_LOGF(LogPCG, Log, "  %ls[%ls (Graph: %ls)]", *Indent, *NodeLabel, *ChildGraph->GetName());
                    LogGraph(ChildGraph, Depth + 1);
                }
            }
            return true;
        });
    };

    LogGraph(RootGraph, 0);

    // Fallback: any graphs not reachable by walking from root (e.g. dynamic subgraph invocations).
    for (auto& Pair : RowsByGraph)
    {
        if (Pair.Key && !PrintedGraphs.Contains(Pair.Key))
        {
            UE_LOGF(LogPCG, Log, "  [Graph: %ls]", *Pair.Key->GetName());
            LogGraph(Pair.Key, 1);
        }
    }

    UE_LOGF(LogPCG, Log, "%ls", *Sep);
}

#endif // PCG_PROFILING_ENABLED
