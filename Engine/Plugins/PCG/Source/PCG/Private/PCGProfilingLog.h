// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#if PCG_PROFILING_ENABLED

class IPCGGraphExecutionSource;
enum class EPCGGenerationStatus : uint8;

namespace PCGProfilingLog
{
    bool IsEnabled();

    /** Emits a per-node timing table to the output log when logging is enabled. */
    void LogProfilingData(IPCGGraphExecutionSource* InExecutionSource, EPCGGenerationStatus InStatus);
}

#endif // PCG_PROFILING_ENABLED
