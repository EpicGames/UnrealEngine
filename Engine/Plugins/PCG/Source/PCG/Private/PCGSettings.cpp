// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGSettings.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGDynamicPins.h"
#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSubgraph.h"
#include "PCGSubsystem.h"
#include "Elements/PCGAddTag.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Misc/Crc.h"
#include "Serialization/ArchiveObjectCrc32.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSettings)

#if WITH_EDITOR
#include "Editor.h"
#include "Engine/Texture.h"
#include "Hash/PCGSettingsHashContext.h"
#endif

#define LOCTEXT_NAMESPACE "PCGSettings"

PCG_DEFINE_TYPE_INFO(FPCGDataTypeInfoSettings, UPCGSettings)

namespace PCGSettings
{
	// For the deprecation of overridable params as display name to authored names
	
	// Arbitrary invalid index
	static constexpr int DeprecationAliasIndex = -2;
	// For property path concatenation
	static constexpr const TCHAR* PropertyPathSeparator = TEXT("/");

	static TAutoConsoleVariable<bool> CVarWarnOverridePinsCollideWithInputPins(
		TEXT("pcg.Graph.WarnPinNameCollisions"),
		false,
		TEXT("Enables warnings when there are name collision between pin names and overrides/user parameters."));
}

/** Custom Crc computation that ignores properties that will not affect the computed result of a node. */
class FPCGSettingsObjectCrc : public FArchiveObjectCrc32
{
public:
	static FPCGCrc PCGCrc(const UPCGSettings* PCGSettings)
	{
		// If Settings has an original, use it to get a unique Id
		const UPCGSettings* OriginalSettings = PCGSettings->OriginalSettings ? PCGSettings->OriginalSettings : PCGSettings;
				
		// If Settings is not transient, use its Object Path as a unique Id (this allows persistent Crcs)
		const bool bSkipUID = !OriginalSettings->GetPackage()->HasAnyPackageFlags(PKG_TransientFlags) && OriginalSettings->GetPackage() != GetTransientPackage();
		const uint32 SettingsPathOrUIDCrc = bSkipUID ? FCrc::StrCrc32(*OriginalSettings->GetPathName()) : FCrc::TypeCrc32(OriginalSettings->GetStableUID());
				
		FPCGSettingsObjectCrc Ar(PCGSettings);
		uint32 ArCrc = Ar.Crc32(const_cast<UPCGSettings*>(PCGSettings), SettingsPathOrUIDCrc);
		return FPCGCrc(ArCrc);
	}

#if WITH_EDITOR
	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
	{
		// @todo_pcg: convert FPCGSettingsObjectCrc to use the new hashing framework (FPCGSettingsHashContext) which skips transient properties by default.
		if (InProperty && InProperty->HasAnyPropertyFlags(CPF_Transient))
		{
			return true;
		}

		static FPCGSettingsHashPolicy Policy;
		return !Policy.ShouldHashProperty(PCGSettings, InProperty);
	}
#endif // WITH_EDITOR

private: 
	FPCGSettingsObjectCrc(const UPCGSettings* InPCGSettings)
		: PCGSettings(InPCGSettings)
	{
	}

	const UPCGSettings* PCGSettings = nullptr;
};

#if WITH_EDITOR
FPCGNodeThumbnailProxy FPCGNodeThumbnailProxy::FromAssetPath(const FSoftObjectPath& InAssetPath)
{
	return FPCGNodeThumbnailProxy{FTextureType{TInPlaceType<FSoftObjectPath>(), InAssetPath}};
}

FPCGNodeThumbnailProxy FPCGNodeThumbnailProxy::FromTexture(const UTexture* InTexture)
{
	return FPCGNodeThumbnailProxy{FTextureType{TInPlaceType<const UTexture*>(), InTexture}};
}
#endif // WITH_EDITOR

FString FPCGSettingsOverridableParam::GetPropertyPath() const
{
	return FString::JoinBy(PropertiesNames, PCGSettings::PropertyPathSeparator, [](const FName InName) { return InName.ToString(); });
}

bool FPCGSettingsOverridableParam::IsHardReferenceOverride() const
{
	return !Properties.IsEmpty() && CastField<const FObjectProperty>(Properties.Last());
}

TArray<FName> FPCGSettingsOverridableParam::GenerateAllPossibleAliases() const
{
	if (Properties.IsEmpty() || !HasAliases())
	{
		return TArray<FName>{};
	}

	auto ConcatenateNames = [](const FName& Name1, const FName& Name2) -> FName
	{
		return FName(FString::Printf(TEXT("%s/%s"), *Name1.ToString(), *Name2.ToString()));
	};

	// If we have multiple properties, we'll generate all possible aliases, which is a combination of each aliases for each property in the path.
	TArray<FName> Result;
	for (int32 i = 0; i < Properties.Num(); ++i)
	{
		const FPCGPropertyAliases* It = MapOfAliases.Find(i);
		const TArray<FName>* CurrentAliases = It ? &It->Aliases : nullptr;
		const FProperty* CurrentProperty = Properties[i];
		const FName PropertyName = CurrentProperty ? CurrentProperty->GetFName() : NAME_None;

		// If no alias at this level, just concatenate the property name to all existing aliases
		if (!CurrentAliases || CurrentAliases->IsEmpty())
		{
			if (Result.IsEmpty())
			{
				Result.Add(PropertyName);
			}
			else
			{
				for (FName& Alias : Result)
				{
					Alias = ConcatenateNames(Alias, PropertyName);
				}
			}
		}
		else if (Result.IsEmpty())
		{
			if (CurrentAliases)
			{
				Result = *CurrentAliases;
			}

			// Also need to add the property name
			Result.Add(PropertyName);
		}
		else
		{
			TArray<FName> Temp;
			Temp.Reserve(Result.Num() * CurrentAliases->Num());
			for (const FName& Alias : Result)
			{
				for (const FName& OtherAlias : *CurrentAliases)
				{
					Temp.Add(ConcatenateNames(Alias, OtherAlias));
				}

				// Also need to add the property name
				Temp.Add(ConcatenateNames(Alias, PropertyName));
			}

			Result = std::move(Temp);
		}
	}

	return Result;
}

#if WITH_EDITOR
FString FPCGSettingsOverridableParam::GetDisplayPropertyPath() const
{
	return GetDisplayPropertyPathText().ToString();
}

FText FPCGSettingsOverridableParam::GetDisplayPropertyPathText() const
{
	if (bHasNameClash)
	{
		TArray<FText> PropertyNames;
		Algo::Transform(Properties, PropertyNames, [](const FProperty* InProperty) { return InProperty ? InProperty->GetDisplayNameText() : FText(); });
		return FText::Join(FText::FromString(PCGSettings::PropertyPathSeparator), PropertyNames);
	}
	else
	{
		return !Properties.IsEmpty() && Properties.Last() ? Properties.Last()->GetDisplayNameText() : FText();
	}
}
#endif // WITH_EDITOR

bool UPCGSettingsInterface::IsInstance() const
{
	return this != GetSettings();
}

void UPCGSettingsInterface::SetEnabled(bool bInEnabled)
{
	if (bEnabled != bInEnabled)
	{
		bEnabled = bInEnabled;

		if (UPCGSettings* Settings = GetSettings())
		{
			Settings->CacheCrc();

#if WITH_EDITOR
			const EPCGChangeType ChangeType = Settings->GetChangeTypeForProperty(GET_MEMBER_NAME_CHECKED(UPCGSettingsInterface, bEnabled));
			OnSettingsChangedDelegate.Broadcast(Settings, ChangeType);
#endif
		}
	}
}

bool UPCGSettingsInterface::SetDebugged(bool bInDebugged)
{
	if (bDebug == bInDebugged)
	{
		return false;
	}

	Modify(/*bAlwaysMarkDirty=*/false);
	bDebug = bInDebugged;

#if WITH_EDITOR
	if (UPCGSettings* Settings = GetSettings())
	{
		const EPCGChangeType ChangeType = Settings->GetChangeTypeForProperty(GET_MEMBER_NAME_CHECKED(UPCGSettingsInterface, bDebug));
		OnSettingsChangedDelegate.Broadcast(Settings, ChangeType);
	}
#endif

	return true;
}

#if WITH_EDITOR
void UPCGSettingsInterface::SetTemporaryManualEditingEnabled(const bool bInEnabled)
{
	bTemporaryManualEditingEnabled = bInEnabled;
}

void UPCGSettingsInterface::SetMarkedForManualEditing(const bool bInMarked)
{
	bMarkedForManualEditing = bInMarked;
}
#endif // WITH_EDITOR

uint32 UPCGSettings::GetTypeNameHash() const
{
	return GetTypeHash(GetClass()->GetName());
}

bool UPCGSettings::operator==(const UPCGSettings& Other) const
{
	if (this == &Other)
	{
		return true;
	}
	else
	{
		return FPCGSettingsObjectCrc::PCGCrc(this) == FPCGSettingsObjectCrc::PCGCrc(&Other);
	}
}

#if WITH_EDITOR
void UPCGSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	// For versions older than the update of input selector, look for any Input Selector in the properties, and if we find one that is default,
	// we force to @LastCreated to keep the old behavior.
	if (DataVersion < FPCGCustomVersion::UpdateAttributePropertyInputSelector)
	{
		auto Recurse = [this](UStruct* InStructClass, void* InContainer)
		{
			auto RecurseImpl = [this](UStruct* InStructClass, void* InContainer, auto Callback) -> void
			{
				for (TFieldIterator<FStructProperty> It(InStructClass, EFieldIterationFlags::IncludeSuper); It; ++It)
				{
					const FStructProperty* Property = CastField<FStructProperty>(*It);
					if (!Property)
					{
						continue;
					}

					// If it is an Input Selector, apply deprecation
					if (Property->Struct == FPCGAttributePropertyInputSelector::StaticStruct())
					{
						Property->ContainerPtrToValuePtr<FPCGAttributePropertyInputSelector>(InContainer)->ApplyDeprecation(DataVersion);
					}
					else
					{
						// Otherwise, go deeper.
						Callback(Property->Struct.Get(), Property->ContainerPtrToValuePtr<void>(InContainer), Callback);
					}
				}
			};

			RecurseImpl(InStructClass, InContainer, RecurseImpl);
		};

		Recurse(GetClass(), this);
	}

	DataVersion = FPCGCustomVersion::LatestVersion;
}

bool UPCGSettings::GetPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip) const
{
	return PCGPinPropertiesHelpers::GetDefaultPinExtraIcon(InPin, OutExtraIcon, OutTooltip);
}

void UPCGSettings::PostEditUndo()
{
	// CachedOverridableParams was reset to previous value
	// Therefore we need to rebuild the properties array since it is transient.
	InitializeCachedOverridableParams();
	InitializeOverriddenSettingsPropertyBag();

	Super::PostEditUndo();
}

FPCGAssetCachedPins UPCGSettings::BuildCachedPins() const
{
	FPCGAssetCachedPins Pins;
	Pins.bHasCachedPins = true;
	Pins.InputPins = InputPinProperties();
	Pins.OutputPins = OutputPinProperties();

	return Pins;
}

void UPCGSettings::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Update cached pins - note that we need to cache pins only for asset-level settings. Otherwise, make them always empty.
	CachedPins = IsAsset() ? BuildCachedPins() : FPCGAssetCachedPins();
}
#endif // WITH_EDITOR

void UPCGSettings::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	// This data does not have a bespoke CRC implementation so just use a global unique data CRC.
	AddUIDToCrc(Ar);
}

void UPCGSettings::PostLoad()
{
	Super::PostLoad();

	// In editor, we always want to rebuild our cache, to make sure we are up to date.
	// At runtime, we can't rebuild our cache (that is why we cache it in the first place).
#if WITH_EDITOR
	InitializeCachedOverridableParams(/*bReset=*/true);
#else
	InitializeCachedOverridableParams(/*bReset=*/false);
#endif // WITH_EDITOR

	InitializeOverriddenSettingsPropertyBag();

#if WITH_EDITOR
	if (ExecutionMode_DEPRECATED != EPCGSettingsExecutionMode::Enabled)
	{
		bEnabled = ExecutionMode_DEPRECATED != EPCGSettingsExecutionMode::Disabled;
		bDebug = ExecutionMode_DEPRECATED == EPCGSettingsExecutionMode::Debug;
		ExecutionMode_DEPRECATED = EPCGSettingsExecutionMode::Enabled;
	}

	if (FPCGModule::IsCPUOnlyExecution())
	{
		bExecuteOnGPU = false;
	}
#endif

	CacheCrc();
}

void UPCGSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	InitializeCachedOverridableParams();
	OnSettingsChangedDelegate.AddUObject(this, &UPCGSettings::OnSettingsChanged);
#endif //WITH_EDITOR

	InitializeOverriddenSettingsPropertyBag();

	// For new objects, use type hash as a seed so we don't get multiple nodes doing randomness in an identical way (we had a case where
	// a Point Subset followed by a Density Noise produced uniform densities because the random seeds/streams were correllated).
	// It is still possible to chain nodes of the same type (Point Subset -> Point Subset) and see correllated behaviour, in which
	// case the random seed on one of them should be changed in the node settings.
	if (PCGHelpers::IsNewObjectAndNotDefault(this, /*bCheckHierarchy=*/true))
	{
		Seed = GetTypeNameHash();
	}

	CacheCrc();
}

void UPCGSettings::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYTAG(PCG);
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FPCGCustomVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		// Some data migration must happen after the graph is fully initialized, such as manipulating node connections, so we
		// store off the loaded version number to be used later.
		DataVersion = Ar.CustomVer(FPCGCustomVersion::GUID);
	}
#endif // WITH_EDITOR

	// An additional custom version number that can be driven by external system users to track system modifications. To use a custom
	// version in user settings objects, override the GetUserCustomVersionGuid() method.
	const FGuid UserDataGuid = GetUserCustomVersionGuid();
	const bool bUsingCustomUserVersion = UserDataGuid != FGuid();

	if (bUsingCustomUserVersion)
	{
		Ar.UsingCustomVersion(UserDataGuid);

#if WITH_EDITOR
		if (Ar.IsLoading())
		{
			// Some data migration must happen after the graph is fully initialized, such as manipulating node connections, so we
			// store off the loaded version number to be used later.
			UserDataVersion = Ar.CustomVer(UserDataGuid);
		}
#endif // WITH_EDITOR
	}
}

void UPCGSettings::PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext)
{
	Super::PostSaveRoot(ObjectSaveContext);

#if WITH_EDITOR
	// This will get called when an external settings gets saved;
	// This is to trigger generation on save, if we've called changed properties from a blueprint
	OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Structural);
#endif
}

#if WITH_EDITOR
UObject* UPCGSettings::GetJumpTargetForDoubleClick() const
{
	return const_cast<UObject*>(Cast<UObject>(this));
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& InputPinProperty = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any);
	InputPinProperty.SetRequiredPin();

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	// This is not true for everything, use a virtual call?
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spatial);

	return PinProperties;
}

bool UPCGSettings::HasOverridableParam(FName InParamName) const
{
	return CachedOverridableParams.FindByPredicate([InParamName](const FPCGSettingsOverridableParam& ParamToCheck)
	{
		return !ParamToCheck.PropertiesNames.IsEmpty() && ParamToCheck.PropertiesNames.Last() == InParamName;
	}) != nullptr;
}

FInstancedPropertyBag UPCGSettings::CreateEmptyPropertyBagInstance() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSettings::CreateEmptyPropertyBagInstance);

	FInstancedPropertyBag Instance;
	Instance.InitializeFromBagStruct(OverriddenSettingsPropertyBag);

	return Instance;
}

void UPCGSettings::AddExecutionDependencyPin(TArray<FPCGPinProperties>& OutPins) const
{
	if (HasExecutionDependencyPin())
	{
		FPCGPinProperties& DepPin = OutPins.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any, /*bInAllowMultipleConnections=*/true, /*bAllowMultipleData=*/true);
#if WITH_EDITOR
		DepPin.Tooltip = PCGPinConstants::Tooltips::ExecutionDependencyTooltip;
#endif // WITH_EDITOR
		DepPin.Usage = EPCGPinUsage::DependencyOnly;
		DepPin.PinStatus = bExecutionDependencyRequired ? EPCGPinStatus::Required : EPCGPinStatus::Advanced;
	}
}

void UPCGSettings::FillOverridableParamsPins(TArray<FPCGPinProperties>& OutPins) const
{
	if (!HasOverridableParams())
	{
		return;
	}

	// Validating that we are not clashing with existing pins
	TMap<FName, FPCGDataTypeIdentifier> InputPinsLabelsAndTypes;

	for (const FPCGPinProperties& InputPinProperties : OutPins)
	{
		InputPinsLabelsAndTypes.Emplace(InputPinProperties.Label, InputPinProperties.AllowedTypes);
	}

	// For debugging
	FString GraphName;
	FString NodeName;

	if (const UPCGNode* Node = Cast<UPCGNode>(GetOuter()))
	{
		NodeName = ((Node->NodeTitle != NAME_None) ? Node->NodeTitle : Node->GetFName()).ToString();

		if (Node->GetGraph())
		{
			GraphName = Node->GetGraph()->GetName();
		}
	}
	else
	{
		NodeName = GetName();
	}

	// Adding the multi-pin connection for params.
	// If it already exists (and is the correct type), we can keep it.
	const FPCGDataTypeIdentifier* PinType = InputPinsLabelsAndTypes.Find(PCGPinConstants::DefaultParamsLabel);
	if (PinType)
	{
		if (*PinType != EPCGDataType::Param)
		{
			const FString ParamsName = PCGPinConstants::DefaultParamsLabel.ToString();
			UE_LOGF(LogPCG, Error, "[%ls-%ls] While adding pin '%ls', another pin '%ls' was found with invalid type (type must be Attribute Set). "
				"Rename or remove this pin to allow an override pin to be added automatically.",
				*GraphName, *NodeName, *ParamsName, *ParamsName);
		}
	}
	else
	{
		FPCGPinProperties& ParamPin = OutPins.Emplace_GetRef(PCGPinConstants::DefaultParamsLabel, EPCGDataType::Param, /*bInAllowMultipleConnections=*/ true, /*bAllowMultipleData=*/ true);
		ParamPin.AllowedTypes.CustomSubtype = static_cast<int32>(EPCGMetadataTypes::Unknown);
		ParamPin.SetOverrideOrUserParamPin();

#if WITH_EDITOR
		ParamPin.Tooltip = LOCTEXT("GlobalParamPinTooltipV2", "Attribute Set containing multiple parameters to override. Names must match perfectly.");
#endif // WITH_EDITOR
	}

	InputPinsLabelsAndTypes.Emplace(PCGPinConstants::DefaultParamsLabel, EPCGDataType::Param);

	for (const FPCGSettingsOverridableParam& OverridableParam : OverridableParams())
	{
		if (InputPinsLabelsAndTypes.Contains(OverridableParam.Label))
		{
			if (PCGSettings::CVarWarnOverridePinsCollideWithInputPins.GetValueOnAnyThread())
			{
				const FString ParamsName = OverridableParam.Label.ToString();
				UE_LOGF(LogPCG, Warning, "[%ls-%ls] While automatically adding override pins, an existing pin was found with conflicting name '%ls'. "
					"Rename or remove this pin to allow the automatic override pin to be added. Automatic override pin '%ls' skipped.",
					*GraphName, *NodeName, *ParamsName, *ParamsName);
			}
			continue;
		}

		InputPinsLabelsAndTypes.Emplace(OverridableParam.Label, EPCGDataType::Param);

		FPCGPinProperties& ParamPin = OutPins.Emplace_GetRef(OverridableParam.Label, EPCGDataType::Param, /*bInAllowMultipleConnections=*/ false, /*bAllowMultipleData=*/ false);
		ParamPin.SetOverrideOrUserParamPin();
#if WITH_EDITOR
		ParamPin.AllowedTypes.CustomSubtype = static_cast<int32>(OverridableParam.UnderlyingType);

		if (!OverridableParam.Properties.IsEmpty())
		{
			const FProperty* Property = OverridableParam.Properties.Last();
			check(Property);
			static FName TooltipMetadata("Tooltip");
			FString Tooltip;
			if (const FString* TooltipPtr = Property->FindMetaData(TooltipMetadata))
			{
				Tooltip = *TooltipPtr + TEXT("\n");
			}

			const FText ContainerInfo = OverridableParam.NumContainers > 0 ? LOCTEXT("ContainerTooltip", " This is an array.") : FText{};

			ParamPin.Tooltip = FText::Format(LOCTEXT("OverridableParamPinTooltip", "{0}Attribute type is \"{1}\" and its exact name is \"{2}\".{3}"),
				FText::FromString(Tooltip),
				FText::FromString(Property->GetCPPType()),
				FText::FromString(OverridableParam.GetPropertyPath()),
				ContainerInfo);
		}
#endif // WITH_EDITOR
	}
}

TArray<FPCGPinProperties> UPCGSettings::AllInputPinProperties() const
{
	TArray<FPCGPinProperties> InputPins = InputPinProperties();
	AddExecutionDependencyPin(InputPins);
	FillOverridableParamsPins(InputPins);
	return InputPins;
}

EPCGDataType UPCGSettings::GetTypeUnionOfIncidentEdges(const FName& PinLabel) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetTypeUnionIDOfIncidentEdges(PinLabel);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FPCGDataTypeIdentifier UPCGSettings::GetTypeUnionIDOfIncidentEdges(const FName& PinLabel) const
{
	return GetTypeUnionIDOfAllIncidentEdges({PinLabel});
}

FPCGDataTypeIdentifier UPCGSettings::GetTypeUnionIDOfAllIncidentEdges(TConstArrayView<FName> PinLabels) const
{
	TArray<FPCGDataTypeIdentifier, TInlineAllocator<32>> AllTypes;
	PCGSettingsHelpers::FPinTypeScopeHelper PinTypeHelper;
	
	for (const FName PinLabel : PinLabels)
	{
		const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
		const UPCGPin* Pin = Node ? Node->GetInputPin(PinLabel) : nullptr;
		if (Pin && Pin->EdgeCount() > 0)
		{
			for (const UPCGEdge* Edge : Pin->Edges)
			{
				const UPCGPin* OtherOutputPin = Edge ? Edge->GetOtherPin(Pin) : nullptr;
				if (OtherOutputPin)
				{
					AllTypes.AddUnique(PinTypeHelper.GetCurrentPinTypeID(OtherOutputPin));
				}
			}
		}
	}

	return FPCGModule::GetConstDataTypeRegistry().GetIdentifiersUnion(AllTypes);
}

TArray<FPCGPinProperties> UPCGSettings::AllOutputPinProperties() const
{
	return OutputPinProperties();
}

TArray<FPCGPinProperties> UPCGSettings::DefaultInputPinProperties() const
{
	return InputPinProperties();
}

TArray<FPCGPinProperties> UPCGSettings::DefaultOutputPinProperties() const
{
	return OutputPinProperties();
}

void UPCGSettings::OnOverrideSettingsDuplicated(bool bSkippedPostLoad)
{
	OnOverrideSettingsDuplicatedInternal(bSkippedPostLoad);
}

FPCGElementPtr UPCGSettings::GetElement() const
{
	if (!CachedElement)
	{
		PCG::TScopeLock ScopeLock(CacheLock);

		if (!CachedElement)
		{
			CachedElement = CreateElement();
		}
	}

	return CachedElement;
}

UPCGNode* UPCGSettings::CreateNode() const
{
	return NewObject<UPCGNode>();
}

int UPCGSettings::GetSeed(const IPCGGraphExecutionSource* InExecutionSource) const
{
	return !UseSeed() ? PCGValueConstants::DefaultSeed : (InExecutionSource ? PCGHelpers::ComputeSeed(Seed, InExecutionSource->GetExecutionState().GetSeed()) : Seed);
}

#if WITH_EDITOR
void UPCGSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		return;
	}

	if (PropertyChangedEvent.Property && (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGSettings, CachedOverridableParams)))
	{
		// Need to rebuild properties, if it ever changes.
		InitializeCachedOverridableParams(/*bReset=*/true);
	}

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName != GET_MEMBER_NAME_CHECKED(UPCGSettings, DeterminismSettings))
	{
		// If we have a property name then get the change type for that property, otherwise assume deepest change type.
		EPCGChangeType ChangeType = EPCGChangeType::Cosmetic | EPCGChangeType::Settings | EPCGChangeType::Structural | EPCGChangeType::GenerationGrid;
		if (PropertyName != NAME_None)
		{
			ChangeType = GetChangeTypeForProperty(PropertyChangedEvent);
		}

		OnSettingsChangedDelegate.Broadcast(this, ChangeType);
	}

	CacheCrc();
}

EPCGChangeType UPCGSettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	return GetChangeTypeForProperty(PropertyChangedEvent.GetPropertyName());
}

EPCGChangeType UPCGSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = EPCGChangeType::Settings;

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSettings, bExecuteOnGPU) ||
		InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSettings, bExecutionDependencyRequired))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ChangeType |= IsStructuralProperty(InPropertyName) ? EPCGChangeType::Structural : EPCGChangeType::None;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Changes to GPU nodes can deeply affect the compiled compute graph so trigger recompiles on any non-trivial changes.
	if (ChangeType != EPCGChangeType::Cosmetic && ShouldExecuteOnGPU())
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}

void UPCGSettings::DirtyCache()
{
	if (GEditor)
	{
		if (GEditor->PlayWorld)
		{
			if (UPCGSubsystem* PCGPIESubsystem = UPCGSubsystem::GetInstance(GEditor->PlayWorld.Get()))
			{
				PCGPIESubsystem->CleanFromCache(GetElement().Get(), this);
			}
		}

		if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World()))
		{
			PCGSubsystem->CleanFromCache(GetElement().Get(), this);
		}
	}
}

bool UPCGSettings::CanEditChange(const FEditPropertyChain& InPropertyChain) const
{
	// Property is the actual property currently checked
	FProperty* Property = InPropertyChain.GetActiveNode() ? InPropertyChain.GetActiveNode()->GetValue() : nullptr;
	// Member property is the same as Property if it is not in a struct/array, otherwise it is the struct/array.
	FProperty* MemberProperty = InPropertyChain.GetActiveMemberNode() ? InPropertyChain.GetActiveMemberNode()->GetValue() : nullptr;

	// No property/member property, or if the property is marked edit const (in metadata), it is not editable.
	if (!Property || !MemberProperty || !Super::CanEditChange(InPropertyChain))
	{
		return false;
	}

	// If it is not marked edit const, properties that are not marked overridable or child properties explicitly marked non overridable are always editable.
	const bool bIsOverridable = MemberProperty->HasMetaData(PCGObjectMetadata::Overridable)
		|| MemberProperty->HasMetaData(PCGObjectMetadata::OverridableCPUAndGPU)
		|| MemberProperty->HasMetaData(PCGObjectMetadata::OverridableCPUAndGPUWithReadback);

	if (!bIsOverridable || Property->HasMetaData(PCGObjectMetadata::NotOverridable))
	{
		return true;
	}

	// If the property can be overridden, mark it as EditConst if it is actually overridden 
	// (ie. the override pin associated with this property is connected)
	return !IsPropertyOverriddenByPin(Property);
}

bool UPCGSettings::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
		return false;

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGSettings, Seed))
	{
		return UseSeed();
	}

	return true;
}

void UPCGSettings::PostPaste()
{
	// Params are not properly built when importing from pasted text (because param
	// Properties are transient/unserialized), so we should reinitialize in post
	InitializeCachedOverridableParams(/*bReset=*/true);
	InitializeOverriddenSettingsPropertyBag();
}

bool UPCGSettings::CanUseExecutionDependencyPinForCulling() const
{
	return HasExecutionDependencyPin();
}

bool UPCGSettings::CanUserEditTitle() const
{
	return !ShouldDrawNodeCompact();
}

void UPCGSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	check(InOutNode);

	if (DataVersion < FPCGCustomVersion::AddParamPinToOverridableNodes)
	{
		PCGSettingsHelpers::DeprecationBreakOutParamsToNewPin(InOutNode, InputPins, OutputPins);
	}

	if (DataVersion < FPCGCustomVersion::RenameDefaultParamsToOverride)
	{
		InOutNode->RenameInputPin(PCGPinConstants::Private::OldDefaultParamsLabel, PCGPinConstants::DefaultParamsLabel);
	}

	// FIXME: Should be behind a version number
	// Deprecation, renaming all the pins generated with the old labels to the new labels
	for (const FPCGSettingsOverridableParam& Param : CachedOverridableParams)
	{
		if (const FPCGPropertyAliases* OldLabels = Param.MapOfAliases.Find(PCGSettings::DeprecationAliasIndex))
		{
			for (const FName& OldLabel : OldLabels->Aliases)
			{
				InOutNode->RenameInputPin(OldLabel, Param.Label);
			}
		}
	}
}

void UPCGSettings::ApplyStructuralDeprecation(UPCGNode* InOutNode)
{
	if (!InOutNode)
	{
		return;
	}

	// Deprecate the "TagsAppliedOnOutput" feature, replaced by usage of the AddTag node.
	// Note that the additional nodes on a per-pin basis (we don't have a AddTag passthrough node, and would be hard to understand for users)
	// and will not create the additional nodes if the pins aren't connected, meaning that in this specific case the tags would be lost.
	// Implementation note: the version here doesn't match the feature change, but has been bumped just after this was deprecated.
	if (DataVersion < FPCGCustomVersion::GetPCGComponentDataMustOverlapSourceComponentByDefault && !TagsAppliedOnOutput_DEPRECATED.IsEmpty())
	{
		UPCGGraph* PCGGraph = InOutNode->GetGraph();
		check(PCGGraph);

		const FString TagsToAdd = FString::Join(TagsAppliedOnOutput_DEPRECATED, TEXT(","));
		bool bAddedAddTagNodes = false;

		const TArray<TObjectPtr<UPCGPin>> OutputPins = InOutNode->GetOutputPins();

		for (TObjectPtr<UPCGPin> OutputPin : OutputPins)
		{
			check(OutputPin);
			const int32 OutputEdgeCount = OutputPin->EdgeCount();

			if (OutputEdgeCount == 0)
			{
				continue;
			}

			// Insert an "Add Tag" node from that pin and move the edges accordingly
			UPCGAddTagSettings* AddTagSettings = nullptr;
			UPCGNode* AddTagNode = PCGGraph->AddNodeOfType(AddTagSettings);
			check(AddTagNode && AddTagSettings);

			AddTagSettings->bEnabled = InOutNode->GetSettings()->bEnabled;
			AddTagSettings->TagsToAdd = TagsToAdd;

			UPCGPin* AddTagInputPin = AddTagNode->GetInputPins()[0];
			UPCGPin* AddTagOutputPin = AddTagNode->GetOutputPins()[0];

			int32 SumSourceNodesPositionX = 0;
			int32 SumSourceNodesPositionY = 0;
			while (!OutputPin->Edges.IsEmpty())
			{
				UPCGPin* DownstreamPin = OutputPin->Edges[0]->OutputPin;
				check(DownstreamPin);

				int32 NodePositionX, NodePositionY;
				DownstreamPin->Node->GetNodePosition(NodePositionX, NodePositionY);
				SumSourceNodesPositionX += NodePositionX;
				SumSourceNodesPositionY += NodePositionY;

				check(DownstreamPin->Node);
				OutputPin->BreakEdgeTo(DownstreamPin);
				AddTagOutputPin->AddEdgeTo(DownstreamPin);
			}

			// Place the injected node halfway between the original node and the downstream nodes
			int32 NodePositionX, NodePositionY;
			InOutNode->GetNodePosition(NodePositionX, NodePositionY);
			const int32 SourceNodeLocalPositionMeanX = SumSourceNodesPositionX / OutputEdgeCount;
			const int32 SourceNodeLocalPositionMeanY = SumSourceNodesPositionY / OutputEdgeCount;

			AddTagNode->SetNodePosition((NodePositionX + SourceNodeLocalPositionMeanX) / 2, (NodePositionY + SourceNodeLocalPositionMeanY) / 2);
			AddTagNode->NodeComment = LOCTEXT("DeprecationAddTagNodeCreated", "Node added to replicate deprecated behavior of the Add Tags To Output functionality.").ToString();
			AddTagNode->bCommentBubbleVisible = 1;

			OutputPin->AddEdgeTo(AddTagInputPin);
			bAddedAddTagNodes = true;
		}

		if (bAddedAddTagNodes)
		{
			UE_LOGF(LogPCG, Log, "Tags applied to output detected. One or more 'Add Tags' node was created automatically to repliecate the previous behavior.");
		}
	}
}
#endif // WITH_EDITOR

bool UPCGSettings::IsPropertyOverriddenByPin(const FProperty* InProperty) const
{
	if (!InProperty)
	{
		return false;
	}

	if (const UPCGNode* Node = Cast<UPCGNode>(GetOuter()))
	{
		const FPCGSettingsOverridableParam* Param = OverridableParams().FindByPredicate([InProperty](const FPCGSettingsOverridableParam& ParamToCheck)
		{
			// In OverridableParam, the array of properties is the chain of properties from the Settings class to the wanted param.
			// Therefore the property we are editing would match the latest property of the array.
			return !ParamToCheck.Properties.IsEmpty() && ParamToCheck.Properties.Last() == InProperty;
		});

		if (Param)
		{
			if (const UPCGPin* Pin = Node->GetInputPin(Param->Label))
			{
				return Pin->IsConnected();
			}
		}
	}

	return false;
}

bool UPCGSettings::IsPropertyOverriddenByPin(const FName PropertyName) const
{
	return IsPropertyOverriddenByPin(TArrayView<const FName>(&PropertyName, 1));
}

bool UPCGSettings::IsPropertyOverriddenByPin(const TArrayView<const FName>& PropertyNameChain) const
{
	if (PropertyNameChain.IsEmpty())
	{
		return false;
	}

	if (const UPCGNode* Node = Cast<UPCGNode>(GetOuter()))
	{
		const FPCGSettingsOverridableParam* Param = OverridableParams().FindByPredicate([&PropertyNameChain](const FPCGSettingsOverridableParam& ParamToCheck)
		{
			return ParamToCheck.PropertiesNames == PropertyNameChain;
		});

		if (Param)
		{
			if (const UPCGPin* Pin = Node->GetInputPin(Param->Label))
			{
				return Pin->IsConnected();
			}
		}
	}
	return false;
}

bool UPCGSettings::IsPinUsedByNodeExecution(const UPCGPin* InPin) const
{
#if WITH_EDITOR
	if (InPin && ShouldExecuteOnGPU())
	{
		auto MatchesPinLabel = [PinLabel = InPin->Properties.Label](const FPCGSettingsOverridableParam& OverridableParam) { return OverridableParam.Label == PinLabel; };

		const FPCGSettingsOverridableParam* Param = OverridableParams().FindByPredicate(MatchesPinLabel);

		// Pin is not in use if we are in GPU execution mode and the pin is an OverridableParam that doesn't support GPU overrides.
		// Generic overrides pin is also not supported.
		if ((Param && !Param->bSupportsGPU) || InPin->Properties.Label == PCGPinConstants::DefaultParamsLabel)
		{
			return false;
		}
	}
#endif

	return true;
}

bool UPCGSettings::DoesPinSupportPassThrough(UPCGPin* InPin) const
{
	return InPin && !InPin->Properties.IsAdvancedPin();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EPCGDataType UPCGSettings::GetCurrentPinTypes(const UPCGPin* InPin) const
{
	return EPCGDataType::DeprecationSentinel;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FPCGDataTypeIdentifier UPCGSettings::GetCurrentPinTypesID(const UPCGPin* InPin) const
{
	// To support old code, we call the legacy version. If we have anything different from the Sentinel, it means the function is overridden
	// in a child class and we want to use that result to avoid any issue, even if we have a loss of info. We'll show a message to indicate
	// the child class to have its code updated
	// All PCG vanilla nodes are updated, and none is implementing `GetCurrentPinTypes` anymore.
	// @todo_pcg: To be removed when GetCurrentPinTypes is removed, in 2 versions
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const EPCGDataType LegacyResult = GetCurrentPinTypes(InPin);
	if (LegacyResult != EPCGDataType::DeprecationSentinel)
	{
		UE_LOGF(LogPCG, Warning, "Class '%ls' implements `GetCurrentPinTypes` and not `GetCurrentPinTypesID`. Needs to be updated.", *GetClass()->GetName());
		return LegacyResult;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// @todo_pcg: investigate the nodes that still rely on this fallback (i.e. CustomHLSL kernels) and migrate them to IPCGDynamicPinsProvider.
	// IPCGDynamicPinsProvider implementers store explicit pin types in their containers, so use the declared type directly.
	// Other HasDynamicPins() nodes need their output type inferred from the first input pin's incident edges.
	if (HasDynamicPins() && InPin->IsOutputPin() && !GetClass()->ImplementsInterface(UPCGDynamicPinsProvider::StaticClass()))
	{
		const UPCGNode* Node = Cast<const UPCGNode>(GetOuter());
		const UPCGPin* FirstPin = Node && !Node->GetInputPins().IsEmpty() ? Node->GetInputPins()[0] : nullptr;

		if (FirstPin != nullptr && !FirstPin->Properties.IsAdvancedPin() && !FirstPin->Properties.IsDatalessPin())
		{
			const FPCGDataTypeIdentifier InputTypeUnion = GetTypeUnionIDOfIncidentEdges(FirstPin->Properties.Label);
			return InputTypeUnion.IsValid() ? InputTypeUnion : InPin->Properties.AllowedTypes;
		}
	}

	return InPin->Properties.AllowedTypes;
}

#if WITH_EDITOR
TArray<FPCGSettingsOverridableParam> UPCGSettings::GatherOverridableParams() const
{
	PCGSettingsHelpers::FPCGGetAllOverridableParamsConfig Config;
	Config.bExtractArrays = true;
	Config.MaxContainersNum = 1; // Do not support more than 1 container deep.
	Config.bUseSeed = UseSeed();

	Config.ShouldKeepPropertyFunc = [](const FProperty* InProperty, int32 Depth) -> bool
	{
		check(InProperty);
		if (InProperty->HasMetaData(PCGObjectMetadata::Overridable)
			|| InProperty->HasMetaData(PCGObjectMetadata::OverridableCPUAndGPU)
			|| InProperty->HasMetaData(PCGObjectMetadata::OverridableCPUAndGPUWithReadback))
		{
			// All properties explicitly marked "Overridable" are overridable
			return true;
		}
		else if (InProperty->HasMetaData(PCGObjectMetadata::NotOverridable))
		{
			// All properties explicitly marked "Not Overridable" are not overridable
			return false;
		}
		else if (!InProperty->HasAnyPropertyFlags(CPF_Edit))
		{
			// Then for recursive structs, make sure to not take properties that are not editable.
			return false;
		}
		else
		{
			// If we have none of that, we only allow overrides for nested properties.
			return Depth > 0;
		}
	};

	return PCGSettingsHelpers::GetAllOverridableParams(GetClass(), Config);
}

void UPCGSettings::CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const
{
	CreateKernels(InOutContext, InObjectOuter, GetTypedOuter<UPCGNode>(), OutKernels, OutEdges);
}

bool UPCGSettings::DisplayExecuteOnGPUSettingCondition() const
{
	if (FPCGModule::IsCPUOnlyExecution())
	{
		return false;
	}

	return DisplayExecuteOnGPUSetting();
}

bool UPCGSettings::IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo) const
{
	return DisplayExecuteOnGPUSetting() || GetType() == EPCGSettingsType::GPU;
}
#endif // WITH_EDITOR

bool UPCGSettings::ShouldExecuteOnGPU() const
{
#if WITH_EDITOR
	if (FPCGModule::IsCPUOnlyExecution())
	{
		return false;
	}
#endif

	return bExecuteOnGPU;
}

void UPCGSettings::InitializeCachedOverridableParams(bool bReset)
{
	// Don't do it for default object
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSettings::InitializeCachedOverridableParams);

#if WITH_EDITOR
	if (CachedOverridableParams.IsEmpty() || bReset)
	{
		// Deprecation: Previously params labels were using the GetDisplayTextName, which is localized and will break existing pins.
		// Now it will be the authored name, but we still need to make sure the pins are not broken. 
		// For that matter, we will match names for properties between serialized and newly found ones. If their label don't match, we add the previous label as alias.
		// Also we keep a list of previously serialized labels. If we have an alias that is the same as one of those labels, it will also mark this label up for deprecation to rename and not breaking pins.
		TMap<FString, FName> PropertyPathToLabel;
		TSet<FName> PreUpdatedLabels;
		if (!CachedOverridableParams.IsEmpty())
		{
			for (const FPCGSettingsOverridableParam& Param : CachedOverridableParams)
			{
				if (Param.PropertiesNames.IsEmpty())
				{
					continue;
				}

				PropertyPathToLabel.Emplace(Param.GetPropertyPath(), Param.Label);
				PreUpdatedLabels.Add(Param.Label);
			}
		}

		CachedOverridableParams = GatherOverridableParams();

		if (!PropertyPathToLabel.IsEmpty())
		{
			for (FPCGSettingsOverridableParam& Param : CachedOverridableParams)
			{
				if (Param.PropertiesNames.IsEmpty())
				{
					continue;
				}

				const FString Path = Param.GetPropertyPath();
				if (FName* OldLabel = PropertyPathToLabel.Find(Path))
				{
					if (*OldLabel != Param.Label)
					{
						Param.MapOfAliases.FindOrAdd(PCGSettings::DeprecationAliasIndex).Aliases.Emplace(*OldLabel);
						continue;
					}
				}

				for (const FName Alias : Param.GenerateAllPossibleAliases())
				{
					if (PreUpdatedLabels.Contains(Alias))
					{
						Param.MapOfAliases.FindOrAdd(PCGSettings::DeprecationAliasIndex).Aliases.Emplace(Alias);
						break;
					}
				}
			}
		}
	}
#endif // WITH_EDITOR

	// Reset the value
	bHasAnyOverridableHardReferences = false;
	
	for (int32 i = 0; i < CachedOverridableParams.Num(); ++i)
	{
		FPCGSettingsOverridableParam& Param = CachedOverridableParams[i];

		if (Param.PropertiesNames.IsEmpty())
		{
			UE_LOGF(LogPCG, Error, "[InitializeCachedOverridableParams] %ls one param is missing its property names, we cannot use this one as overridable param", *GetName());
			CachedOverridableParams.RemoveAt(i--);
			continue;
		}

		// In some cases, the property class is not serialized correctly. We can still try to recover from this.
		if (!Param.PropertyClass)
		{
			FixingOverridableParamPropertyClass(Param);
			
			if (!Param.PropertyClass)
			{
				UE_LOGF(LogPCG, Error, "[InitializeCachedOverridableParams] %ls one param is missing its property class, we cannot use this one as overridable param", *GetName());
				CachedOverridableParams.RemoveAt(i--);
				continue;
			}
		}

		Param.Properties.Reset(Param.PropertiesNames.Num());

		// Do not support multi containers
		if (Param.NumContainers > 1)
		{
			CachedOverridableParams.RemoveAt(i--);
			continue;
		}

		// Some properties might not be available at runtime. Remove them from the list, since they won't be overridable anymore.
		const FProperty* CurrentProperty = PCGPropertyHelpers::FindPropertyByName(Param.PropertyClass, Param.PropertiesNames[0]);
		if (!CurrentProperty)
		{
			CachedOverridableParams.RemoveAt(i--);
			continue;
		}

		for (int32 j = 0; j < Param.PropertiesNames.Num(); ++j)
		{
			// If we have multiple depth properties, it can be containers. In that case, we have 2 properties for a single name (the container property and inner property)
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(CurrentProperty))
			{
				Param.Properties.Add(CurrentProperty);
				CurrentProperty = ArrayProperty->Inner;
			}
			else if (const FSetProperty* SetProperty = CastField<FSetProperty>(CurrentProperty))
			{
				Param.Properties.Add(CurrentProperty);
				CurrentProperty = SetProperty->ElementProp;
			}

			// Otherwise it can be objects/structs.
			const UStruct* UnderlyingStruct = nullptr;
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(CurrentProperty))
			{
				UnderlyingStruct = StructProperty->Struct;
			}
			else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(CurrentProperty))
			{
				UnderlyingStruct = ObjectProperty->PropertyClass;
			}

			if (UnderlyingStruct && j < Param.PropertiesNames.Num() - 1)
			{
				Param.Properties.Add(CurrentProperty);
				CurrentProperty = PCGPropertyHelpers::FindPropertyByName(UnderlyingStruct, Param.PropertiesNames[j + 1]);
				check(CurrentProperty);
			}
			else if (j == Param.PropertiesNames.Num() - 1)
			{
				// If we are at the last property we're fine.
				Param.Properties.Add(CurrentProperty);
			}
			else
			{
				// Otherwise we're ill-formed, reset the properties.
				Param.Properties.Empty();
				break;
			}
		}

		// Keep the information if the last property is a hard ref
		if (Param.IsHardReferenceOverride())
		{
			bHasAnyOverridableHardReferences = true;
		}
	}
}

void UPCGSettings::InitializeOverriddenSettingsPropertyBag()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSettings::InitializeOverriddenSettingsPropertyBag);

	const UClass* BaseClass = UPCGSettings::StaticClass();
	const UClass* SettingsClass = GetClass();

	TArray<FPropertyBagPropertyDesc> Descriptors;

	for (TFieldIterator<FProperty> It(SettingsClass); It; ++It)
	{
		// Only include properties declared in classes *after* the abstract base class UPCGSettings (almost all properties in UPCGSettings are not interesting for override).
		// @todo_pcg: We should probably allow some UPCGSettings properties (e.g. Seed).
		if (It->GetOwnerClass()->IsChildOf(BaseClass) && It->GetOwnerClass() != BaseClass)
		{
			Descriptors.Emplace(It->GetFName(), *It);
		}
	}

	OverriddenSettingsPropertyBag = UPropertyBag::GetOrCreateFromDescs(Descriptors);
}

void UPCGSettings::FixingOverridableParamPropertyClass(FPCGSettingsOverridableParam& Param) const
{
	if (!GetClass() || Param.PropertiesNames.IsEmpty())
	{
		return;
	}

	if (GetClass()->FindPropertyByName(Param.PropertiesNames[0]) != nullptr)
	{
		Param.PropertyClass = GetClass();
	}
}

// deprecated
const FPCGCrc& UPCGSettings::GetCachedCrc() const
{
	static FPCGCrc InvalidCrc;
	return InvalidCrc;
}

FPCGCrc UPCGSettings::GetSettingsCrc() const
{
#if WITH_EDITOR
	if (!CachedCrc.IsValid())
	{
		const_cast<UPCGSettings*>(this)->CacheCrc();
	}
	return CachedCrc;
#else
	// In non-editor builds we don't track changes to the UPCGSettings object so we always compute it
	return FPCGSettingsObjectCrc::PCGCrc(this);
#endif
}

void UPCGSettings::CacheCrc()
{
#if WITH_EDITOR
	CachedCrc = FPCGSettingsObjectCrc::PCGCrc(this);
#endif
}

#if WITH_EDITOR
void UPCGSettings::OnSettingsChanged(UPCGSettings* InSettings, EPCGChangeType ChangeType)
{
	if (ensure(InSettings == this) && EnumHasAnyFlags(ChangeType, EPCGChangeType::Settings))
	{
		// Clear the cached crc if an external call has settings changed externally.
		CachedCrc = FPCGCrc();
	}
}
#endif

TArray<FPCGPinProperties> UPCGSettings::DefaultPointInputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	FPCGPinProperties& InputPinProperty = Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Point);
	InputPinProperty.SetRequiredPin();
	return Properties;
}

TArray<FPCGPinProperties> UPCGSettings::DefaultPointOutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point);
	return Properties;
}

void UPCGSettingsInstance::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PostSettingsChanged();
#endif
}

void UPCGSettingsInstance::BeginDestroy()
{
#if WITH_EDITOR
	if (Settings)
	{
		Settings->OnSettingsChangedDelegate.RemoveAll(this);
	}
#endif

	Super::BeginDestroy();
}

void UPCGSettingsInstance::PostEditImport()
{
	Super::PostEditImport();

#if WITH_EDITOR
	PostSettingsChanged();
#endif
}

void UPCGSettingsInstance::SetSettings(UPCGSettings* InSettings)
{
#if WITH_EDITOR
	if (Settings)
	{
		Settings->OnSettingsChangedDelegate.RemoveAll(this);
	}
#endif

	Settings = InSettings;

#if WITH_EDITOR
	PostSettingsChanged();
#endif
}

#if WITH_EDITOR
void UPCGSettingsInstance::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	// Some setting in the instance has changed. We don't have a flag for that yet (to add if needed)
	// However, we can make it behave like a standard change
	OnSettingsChangedDelegate.Broadcast(Settings, EPCGChangeType::Settings);
}

void UPCGSettingsInstance::OnSettingsChanged(UPCGSettings* InSettings, EPCGChangeType ChangeType)
{
	if (InSettings == Settings)
	{
		OnSettingsChangedDelegate.Broadcast(InSettings, ChangeType);
	}
}

void UPCGSettingsInstance::PostSettingsChanged()
{
	OriginalSettings = Settings;

	if (Settings)
	{
		Settings->OnSettingsChangedDelegate.AddUObject(this, &UPCGSettingsInstance::OnSettingsChanged);
		Settings->ConditionalPostLoad();
	}
}
#endif

UPCGTrivialSettings::UPCGTrivialSettings()
{
#if WITH_EDITORONLY_DATA
	bExposeToLibrary = false;
#endif
}

FPCGElementPtr UPCGTrivialSettings::CreateElement() const
{
	return MakeShared<FPCGTrivialElement>();
}

bool FPCGTrivialElement::ExecuteInternal(FPCGContext* Context) const
{
	// Pass-through
	Context->OutputData = Context->InputData;
	return true;
}

#undef LOCTEXT_NAMESPACE
