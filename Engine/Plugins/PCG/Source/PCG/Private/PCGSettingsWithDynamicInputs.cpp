// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGSettingsWithDynamicInputs.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSettingsWithDynamicInputs)

TArray<FPCGPinProperties> UPCGSettingsWithDynamicInputs::InputPinProperties() const
{
	TArray<FPCGPinProperties> ConcatenatedProperties(StaticInputPinProperties());
	ConcatenatedProperties.Append(DynamicInputPinProperties);
	return ConcatenatedProperties;
}

#if WITH_EDITOR
  void UPCGSettingsWithDynamicInputs::OnUserAddDynamicInputPin()
{
	AddDefaultDynamicInputPin();
}

bool UPCGSettingsWithDynamicInputs::CanUserRemoveDynamicInputPin(int32 PinIndex)
{
	const int32 Index = PinIndex - GetStaticInputPinNum(); // Convert to the index of the dynamic pins
	return (Index >= 0 && Index < DynamicInputPinProperties.Num());
}

void UPCGSettingsWithDynamicInputs::OnUserRemoveDynamicInputPin(UPCGNode* InOutNode, int32 PinIndex)
{
	const int32 Index = PinIndex - GetStaticInputPinNum(); // Convert to the index of the dynamic pins
	check(InOutNode && Index >= 0 && Index < DynamicInputPinProperties.Num());

	// Prepare the pin for removal by breaking edges and giving it a dummy placeholder label
	UPCGPin* RemovedPin = InOutNode->GetInputPin(DynamicInputPinProperties[Index].Label);
	check(RemovedPin);
	const bool bEdgeRemoved = RemovedPin->BreakAllEdges();
	// The dummy placeholder label prevents the pin label from conflicting with the subsequent pin that will replace it
	// during the node's reconstruction within UpdatePins
	InOutNode->RenameInputPin(RemovedPin->Properties.Label, NAME_Error, /*bBroadcastUpdate=*/false);
	
	// Update the pin labels on every input pin after the one being removed
	for (int I = Index + 1; I < DynamicInputPinProperties.Num(); ++I)
	{
		const FName NewLabel = FName(GetDynamicInputPinsBaseLabel().ToString() + FString::FromInt(I + 1));
		InOutNode->RenameInputPin(DynamicInputPinProperties[I].Label, NewLabel, /*bBroadcastUpdate=*/false);
		DynamicInputPinProperties[I].Label = NewLabel;
	}

	DynamicInputPinProperties.RemoveAt(Index);
	// This broadcast will force reconstruction of the node using the now updated settings
	const EPCGChangeType ChangeType = EPCGChangeType::Node | EPCGChangeType::Settings | (bEdgeRemoved ? EPCGChangeType::Edge : EPCGChangeType::None);
	OnSettingsChangedDelegate.Broadcast(this, ChangeType);
}

int32 UPCGSettingsWithDynamicInputs::GetStaticInputPinNum() const
{
	return StaticInputPinProperties().Num();
}

void UPCGSettingsWithDynamicInputs::AddDynamicInputPin(FPCGPinProperties&& CustomProperties)
{
	if (CustomPropertiesAreValid(CustomProperties))
	{
		DynamicInputPinProperties.Emplace(std::forward<FPCGPinProperties>(CustomProperties));
		OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Node | EPCGChangeType::Settings);
	}
}
#endif // WITH_EDITOR

TArray<FName> UPCGSettingsWithDynamicInputs::GetNodeDefinedPinLabels() const
{
	const TArray<FPCGPinProperties> PinProperties = InputPinProperties();

	TArray<FName> Labels;
	Labels.Reserve(PinProperties.Num());
	Algo::Transform(PinProperties, Labels, [](const FPCGPinProperties& Properties) { return Properties.Label; });

	return Labels;
}

TArray<FPCGPinProperties> UPCGSettingsWithDynamicInputs::StaticInputPinProperties() const
{
	return Super::InputPinProperties();
}
