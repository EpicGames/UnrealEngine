// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGSubgraph.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGPin.h"
#include "Data/PCGUserParametersData.h"
#include "Graph/PCGStackContext.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"

#include "Algo/Find.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"

#if WITH_EDITOR
#include "DataHierarchyViewModelBase.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSubgraph)

#define LOCTEXT_NAMESPACE "PCGSubgraphElement"

namespace PCGSubgraphSettings
{
	void RemoveAdvancedAndInvisibleOnConnectedPins(const UPCGGraph* Subgraph, TArray<FPCGPinProperties>& InOutPinProperties, const bool bIsInput)
	{
		const UPCGNode* SubgraphNode = bIsInput ? Subgraph->GetInputNode() : Subgraph->GetOutputNode();
		check(SubgraphNode);

		for (FPCGPinProperties& PinProperties : InOutPinProperties)
		{
			const UPCGPin* Pin = bIsInput ? SubgraphNode->GetOutputPin(PinProperties.Label) : SubgraphNode->GetInputPin(PinProperties.Label);
			if (ensure(Pin))
			{
				PinProperties.bInvisiblePin = false;

				if (Pin->IsConnected() && PinProperties.IsAdvancedPin())
				{
					PinProperties.SetNormalPin();
				}
			}
		}
	}
}

#if WITH_EDITOR
void UPCGBaseSubgraphSettings::SetupCallbacks()
{
	if (FPCGContext::IsInitializingSettings())
	{
		return;
	}

	UPCGGraphInterface* Subgraph = GetSubgraphInterface();

	if (Subgraph && !Subgraph->OnGraphChangedDelegate.IsBoundToObject(this))
	{
		Subgraph->OnGraphChangedDelegate.AddUObject(this, &UPCGBaseSubgraphSettings::OnSubgraphChanged);
	}
}

void UPCGBaseSubgraphSettings::TeardownCallbacks()
{
	if (FPCGContext::IsInitializingSettings())
	{
		return;
	}

	if (UPCGGraphInterface* Subgraph = GetSubgraphInterface())
	{
		Subgraph->OnGraphChangedDelegate.RemoveAll(this);
	}
}
#endif // WITH_EDITOR

UPCGGraph* UPCGBaseSubgraphSettings::GetSubgraph() const
{
	UPCGGraphInterface* SubgraphInterface = GetSubgraphInterface();
	return SubgraphInterface ? SubgraphInterface->GetGraph() : nullptr;
}

void UPCGBaseSubgraphSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	SetupCallbacks();
#endif
}

void UPCGBaseSubgraphSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	SetupCallbacks();
#endif
}

void UPCGBaseSubgraphSettings::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

#if WITH_EDITOR
	SetupCallbacks();
#endif
}

void UPCGBaseSubgraphSettings::PostEditImport()
{
	Super::PostEditImport();

#if WITH_EDITOR
	SetupCallbacks();
#endif
}

void UPCGBaseSubgraphSettings::SetSubgraph(UPCGGraphInterface* InGraph)
{
	SetSubgraphInternal(InGraph);
}

void UPCGBaseSubgraphSettings::BeginDestroy()
{
#if WITH_EDITOR
	TeardownCallbacks();
#endif

	Super::BeginDestroy();
}

#if WITH_EDITOR
void UPCGBaseSubgraphSettings::PreEditUndo()
{
	Super::PreEditUndo();

	TeardownCallbacks();
}

void UPCGBaseSubgraphSettings::PostEditUndo()
{
	Super::PostEditUndo();

	SetupCallbacks();
}

void UPCGBaseSubgraphSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange && !!(GetChangeTypeForProperty(PropertyAboutToChange->GetFName()) & EPCGChangeType::Structural))
	{
		TeardownCallbacks();
	}

	Super::PreEditChange(PropertyAboutToChange);
}

void UPCGBaseSubgraphSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && !!(GetChangeTypeForProperty(PropertyChangedEvent.Property->GetFName()) & EPCGChangeType::Structural))
	{
		SetupCallbacks();
	}
}

void UPCGBaseSubgraphSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& VisitedGraphs) const
{
	if (UPCGGraph* Subgraph = GetSubgraph())
	{
		Subgraph->GetTrackedSelectionKeysToSettings(OutKeysToSettings, VisitedGraphs);
	}
}

EPCGChangeType UPCGBaseSubgraphSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSettingsInterface, bEnabled))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}

void UPCGBaseSubgraphSettings::OnSubgraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType)
{
	if (InGraph == GetSubgraphInterface())
	{
		// Only add settings if not cosmetic - we don't want to promote a cosmetic change to something deeper.
		if (ChangeType != EPCGChangeType::Cosmetic)
		{
			ChangeType |= EPCGChangeType::Settings;
		}

		// Also rebuild the overrides
		InitializeCachedOverridableParams(/*bReset=*/true);

		OnSettingsChangedDelegate.Broadcast(this, ChangeType);
	}
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGBaseSubgraphSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> InputPins;
	if (UPCGGraph* Subgraph = GetSubgraph())
	{
		InputPins = Subgraph->GetInputNode()->InputPinProperties();
		PCGSubgraphSettings::RemoveAdvancedAndInvisibleOnConnectedPins(Subgraph, InputPins, /*bIsInput=*/true);
	}
	else
	{
		InputPins = Super::InputPinProperties();
		// Considering this is likely a case where we'll have dynamic graph dispatch, don't make the default input pins required
		for (FPCGPinProperties& PinProperties : InputPins)
		{
			if (PinProperties.IsRequiredPin())
			{
				PinProperties.SetNormalPin();
			}
		}
	}

	return InputPins;
}

TArray<FPCGPinProperties> UPCGBaseSubgraphSettings::OutputPinProperties() const
{
	if (UPCGGraph* Subgraph = GetSubgraph())
	{
		TArray<FPCGPinProperties> OutputPins = Subgraph->GetOutputNode()->OutputPinProperties();
		PCGSubgraphSettings::RemoveAdvancedAndInvisibleOnConnectedPins(Subgraph, OutputPins, /*bIsInput=*/false);
		return OutputPins;
	}
	else
	{
		// Here we do not want the base class implementation as it forces Spatial but that might not be the case here
		TArray<FPCGPinProperties> OutputPins;
		OutputPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
		return OutputPins;
	}
}

void UPCGBaseSubgraphSettings::FixingOverridableParamPropertyClass(FPCGSettingsOverridableParam& Param) const
{
	bool bFound = false;
	const UPCGGraph* PCGGraph = GetSubgraph();

	if (PCGGraph && !Param.PropertiesNames.IsEmpty())
	{
		if (const FInstancedPropertyBag* UserParameterStruct = PCGGraph->GetUserParametersStruct())
		{
			const UScriptStruct* ScriptStruct = UserParameterStruct->GetPropertyBagStruct();
			if (ScriptStruct && ScriptStruct->FindPropertyByName(Param.PropertiesNames[0]))
			{
				Param.PropertyClass = ScriptStruct;
				bFound = true;
			}
		}
	}

	if (!bFound)
	{
		Super::FixingOverridableParamPropertyClass(Param);
	}
}

#if WITH_EDITOR
TArray<FPCGSettingsOverridableParam> UPCGBaseSubgraphSettings::GatherOverridableParams() const
{
	TArray<FPCGSettingsOverridableParam> OverridableParams = Super::GatherOverridableParams();

	const UPCGGraph* PCGGraph = GetSubgraph();
	if (PCGGraph)
	{
		if (const FInstancedPropertyBag* UserParameterStruct = PCGGraph->GetUserParametersStruct())
		{
			if (const UPropertyBag* ScriptStruct = UserParameterStruct->GetPropertyBagStruct())
			{
				PCGSettingsHelpers::FPCGGetAllOverridableParamsConfig Config;
				Config.bExcludeSuperProperties = true;
				Config.ShouldKeepPropertyFunc = [](const FProperty* InProperty, int32) { return InProperty && !InProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance); };
				Config.bExtractArrays = true;
				Config.MaxContainersNum = 1; // Do not support more than 1 container deep.

				TArray<FPCGSettingsOverridableParam> ParamsFromGraph = PCGSettingsHelpers::GetAllOverridableParams(ScriptStruct, Config);

				// Gather order of properties from the hierarchy root, if any.
				if (UHierarchyRoot* HierarchyRoot = Cast<UHierarchyRoot>(PCGGraph->GetUserParameterHierarchyRoot()); HierarchyRoot && !ParamsFromGraph.IsEmpty())
				{
					TArray<FHierarchyElementIdentity> PropsIds;
					PropsIds.Reserve(ParamsFromGraph.Num());

					for(const FPCGSettingsOverridableParam& Param : ParamsFromGraph)
					{
						const FPropertyBagPropertyDesc* PropDesc = ScriptStruct->FindPropertyDescByProperty(Param.Properties.Last());
						PropsIds.Add(PropDesc ? FHierarchyElementIdentity({ PropDesc->ID }, {}) : FHierarchyElementIdentity());
					}

					// Push overridable params in order of the hierarchy
					TArray<UHierarchyItem*> AllItems;
					HierarchyRoot->GetChildrenOfType<UHierarchyItem>(AllItems, /*bRecursive=*/true);

					for (UHierarchyItem* Item : AllItems)
					{
						check(Item);
						FHierarchyElementIdentity ItemIdentity = Item->GetPersistentIdentity();
						const int32 PropIndex = PropsIds.IndexOfByKey(ItemIdentity);
						if (PropIndex != INDEX_NONE)
						{
							OverridableParams.Add(MoveTemp(ParamsFromGraph[PropIndex]));
							ParamsFromGraph.RemoveAtSwap(PropIndex);
							PropsIds.RemoveAtSwap(PropIndex);
						}
					}
				}

				// Any remaining properties that weren't found from the hierarchy, push them in the current order
				OverridableParams.Append(ParamsFromGraph);
			}
		}
	}

	return OverridableParams;
}
#endif // WITH_EDITOR

UPCGSubgraphSettings::UPCGSubgraphSettings(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	SubgraphInstance = InObjectInitializer.CreateDefaultSubobject<UPCGGraphInstance>(this, TEXT("PCGSubgraphInstance"));
}

UPCGNode* UPCGSubgraphSettings::CreateNode() const
{
	return NewObject<UPCGSubgraphNode>();
}

FString UPCGSubgraphSettings::GetAdditionalTitleInformation() const
{
#if WITH_EDITOR
	if (IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGSubgraphSettings, SubgraphOverride)))
	{
		// Subgraphs with the subgraph override pin connected should not display any asset path.
		return FString();
	}

	TOptional<FText> OverrideTitle = SubgraphInstance->GetTitleOverride();
	if (OverrideTitle.IsSet())
	{
		return OverrideTitle.GetValue().ToString();
	}
#endif

	if (UPCGGraph* TargetSubgraph = GetSubgraph())
	{
		return TargetSubgraph->GetDisplayName().ToString();
	}
	else
	{
		return LOCTEXT("NodeTitleExtendedInvalidSubgraph", "Missing Subgraph").ToString();
	}
}

void UPCGSubgraphSettings::SetSubgraphInternal(UPCGGraphInterface* InGraph)
{
	SubgraphInstance->SetGraph(InGraph);
}

bool UPCGSubgraphSettings::SubgraphAssetFilter(const FAssetData& AssetData) const
{
#if WITH_EDITOR
	// Need access to this graph to get the customization
	UPCGGraph* Graph = GetOuter() ? Cast<UPCGGraph>(GetOuter()->GetOuter()) : nullptr;

	if (!Graph || !Graph->GraphCustomization.FiltersSubgraphs())
	{
		return false;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	auto AssetTest = [Graph, &AssetRegistryModule](const FAssetData& AssetData) -> bool
	{
		FSoftObjectPath PathToTest;
		if (AssetData.IsInstanceOf<UPCGGraphInstance>())
		{
			PathToTest = FSoftObjectPath(AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPCGGraphInstance, Graph)));
		}
		else
		{
			PathToTest = AssetData.GetSoftObjectPath();
		}

		// Check if this is accepted by the filter, or recurse a level otherwise
		if (Graph->GraphCustomization.FilterSubgraph(PathToTest))
		{
			// Get parent path and recurse
			FAssetData CurrentAsset = AssetData;
			if (AssetData.IsInstanceOf<UPCGGraphInstance>())
			{
				CurrentAsset = AssetRegistryModule.Get().GetAssetByObjectPath(PathToTest);
			}

			// Refused by the filter
			return true;
		}
		else
		{
			// Accepted by the filter
			return false;
		}
	};

	return AssetTest(AssetData);
#else
	return false; // always accept.
#endif
}

void UPCGSubgraphSettings::PostLoad()
{
#if WITH_EDITOR
	if (Subgraph_DEPRECATED)
	{
		SubgraphInstance->SetGraph(Subgraph_DEPRECATED);
		Subgraph_DEPRECATED = nullptr;
	}
#endif // WITH_EDITOR

	Super::PostLoad();
}

#if WITH_EDITOR
void UPCGSubgraphSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGSubgraphSettings, SubgraphInstance))
	{
		// Also rebuild the overrides
		InitializeCachedOverridableParams(/*bReset=*/true);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FLinearColor UPCGSubgraphSettings::GetNodeTitleColor() const
{
	if (!IsSubgraphOverridden())
	{
		TOptional<FLinearColor> OverrideColor = SubgraphInstance->GetColorOverride();
		if (OverrideColor.IsSet())
		{
			return OverrideColor.GetValue();
		}
	}

	return Super::GetNodeTitleColor();
}
#endif

#if WITH_EDITOR
UObject* UPCGSubgraphSettings::GetJumpTargetForDoubleClick() const
{
	// Note that there is a const_cast done behind the scenes in Cast.
	// And this behavior is already used in similar part of UE.
	return Cast<UObject>(GetSubgraph());
}

EPCGChangeType UPCGSubgraphSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	// Force structural if name is none. We are probably in a undo/redo situation
	if ((InPropertyName == NAME_None) || (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGSubgraphSettings, SubgraphInstance)))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif

FPCGElementPtr UPCGSubgraphSettings::CreateElement() const
{
	return MakeShared<FPCGSubgraphElement>();
}

UPCGGraphInterface* UPCGSubgraphSettings::GetSubgraphInterface() const
{
	// The only place when SubgraphOverride is not null, is when we execute a dynamic subgraph.
	// Everywhere else (for UI, parameter overrides, normal subgraph flow, etc...) we will use the SubgraphInstance.
	return SubgraphOverride ? SubgraphOverride.Get() : SubgraphInstance.Get(); 
}

bool UPCGSubgraphSettings::IsDynamicGraph() const
{
	return IsSubgraphOverridden();
}

bool UPCGSubgraphSettings::IsSubgraphOverridden() const
{
	UPCGNode* Node = Cast<UPCGNode>(GetOuter());

	if (!Node && OriginalSettings)
	{
		Node = Cast<UPCGNode>(OriginalSettings->GetOuter());

	}

	if (!Node)
	{
		return false;
	}

	const FName PropertyName = GET_MEMBER_NAME_CHECKED(UPCGSubgraphSettings, SubgraphOverride);

	const FPCGSettingsOverridableParam* Param = CachedOverridableParams.FindByPredicate([PropertyName](const FPCGSettingsOverridableParam& ParamToCheck)
	{
		return !ParamToCheck.PropertiesNames.IsEmpty() && ParamToCheck.PropertiesNames.Last() == PropertyName;
	});

	if (Param)
	{
		if (const UPCGPin* Pin = Node->GetInputPin(Param->Label))
		{
			return Pin->IsConnected();
		}
	}

	return false;
}

TObjectPtr<UPCGGraph> UPCGBaseSubgraphNode::GetSubgraph() const
{
	UPCGGraphInterface* SubgraphInterface = GetSubgraphInterface();
	return SubgraphInterface ? SubgraphInterface->GetGraph() : nullptr;
}

TObjectPtr<UPCGGraphInterface> UPCGSubgraphNode::GetSubgraphInterface() const
{
	TObjectPtr<UPCGBaseSubgraphSettings> Settings = Cast<UPCGBaseSubgraphSettings>(GetSettings());
	return Settings ? Settings->GetSubgraphInterface() : nullptr;
}

FStructView FPCGSubgraphContext::GetExternalStructContainerForOverridableParam(const FPCGSettingsOverridableParam& InParam)
{
	const UPCGBaseSubgraphSettings* Settings = GetInputSettings<UPCGBaseSubgraphSettings>();
	check(Settings);
	const UPCGGraphInterface* Graph = Settings->GetSubgraphInterface();
	const FInstancedPropertyBag* UserParameters = Graph ? Graph->GetUserParametersStruct() : nullptr;

	if (UserParameters && !InParam.PropertiesNames.IsEmpty() && UserParameters->FindPropertyDescByName(InParam.PropertiesNames[0]) && GraphInstanceParametersOverride.IsValid())
	{
		return GraphInstanceParametersOverride;
	}
	else
	{
		return nullptr;
	}
}

void FPCGSubgraphContext::InitializeUserParametersStruct()
{
	// Only duplicate the UserParameters if we have overridable params and we have at least one param pin connected.
	GraphInstanceParametersOverride.Reset();

	// Will return the OG settings the first time this is called (when initializing the element/context)
	// and will contain the "hardcoded" graph in the subgraph node.
	// If subgraph is overridden, this be called a second time with the SubgraphInstance containing the updated overridden subgraph.
	const UPCGBaseSubgraphSettings* Settings = GetInputSettings<UPCGBaseSubgraphSettings>();
	check(Settings);

	const UClass* SettingsClass = Settings->GetClass();
	check(SettingsClass);

	const TArray<FPCGSettingsOverridableParam>& OverridableParams = Settings->OverridableParams();

	const UPCGGraphInterface* Graph = Settings->GetSubgraphInterface();
	const FInstancedPropertyBag* UserParameters = Graph ? Graph->GetUserParametersStruct() : nullptr;
	FConstStructView UserParametersView = UserParameters ? UserParameters->GetValue() : FConstStructView{};

	if (!OverridableParams.IsEmpty() && UserParameters)
	{
		// In the case where we have a subgraph override, we'll need to make sure the override properties are also taken into account when we apply the overrides
		// and copy the overriden values.
		const UPCGGraphInstance* OriginalGraphInstance = Cast<UPCGGraphInstance>(Settings->GetOriginalSubgraphInterface());
		if (OriginalGraphInstance && Graph != OriginalGraphInstance && !OriginalGraphInstance->ParametersOverrides.PropertiesIDsOverridden.IsEmpty())
		{
			// Copy the subgraph override parameters first and copy to the subgraph override parameters, copying over compatible parameters
			FInstancedPropertyBag TempGraphInstanceParametersOverride = *UserParameters; // Full copy - Also set the layout of the property bag
			TArray<FName> PropertyNames;
			PropertyNames.Reserve(OriginalGraphInstance->ParametersOverrides.PropertiesIDsOverridden.Num());
			const UPropertyBag* OriginalBag = OriginalGraphInstance->ParametersOverrides.Parameters.GetPropertyBagStruct();
			if (ensure(OriginalBag))
			{
				Algo::Transform(OriginalGraphInstance->ParametersOverrides.PropertiesIDsOverridden, PropertyNames,
				[OriginalBag](const FGuid& InId) -> FName
					{
						// If the property is overridden the desc must exist.
						const FPropertyBagPropertyDesc* Desc = OriginalBag->FindPropertyDescByID(InId);
						return ensure(Desc) ? Desc->Name : NAME_None;
					});

				TempGraphInstanceParametersOverride.CopyMatchingValuesByName(OriginalGraphInstance->ParametersOverrides.Parameters, MakeConstArrayView(PropertyNames)); // Subset copy
				GraphInstanceParametersOverride = TempGraphInstanceParametersOverride.Detach(); // Move
			}

			return;
		}

		// Otherwise look for any overridden param
		bool bHasParamConnected = !InputData.GetParamsByPin(PCGPinConstants::DefaultParamsLabel).IsEmpty();

		int32 Index = 0;
		while (!bHasParamConnected && Index < OverridableParams.Num())
		{
			// Discard any override that is a property of the settings (we are looking for overrides for the graph instance)
			// We use the first property name, since it will be the one related to this settings properties.
			const FName PropertyName = OverridableParams[Index].PropertiesNames.IsEmpty() ? NAME_None : OverridableParams[Index].PropertiesNames[0];
			const FName Label = OverridableParams[Index].Label;
			++Index;

			if (PropertyName != NAME_None && SettingsClass->FindPropertyByName(PropertyName))
			{
				continue;
			}

			bHasParamConnected |= !InputData.GetParamsByPin(Label).IsEmpty();
		}

		if (bHasParamConnected)
		{
			GraphInstanceParametersOverride = FInstancedStruct(UserParametersView);
		}
	}
}

void FPCGSubgraphContext::UpdateOverridesWithOverriddenGraph()
{
	// We have a "catch-22" kind of problem here. When we initialize the subgraph element, we look for the graph "hardcoded"
	// in the settings to duplicate its user parameters and override it with the override pins. But Subgraph override is also coming from
	// the override pins, meaning that without adding some kind of "read order" on the override, we have to read them twice. It's less efficient
	// but makes things simpler to understand. We have a first override read to get the Subgraph Override, then if it is set, we have a second
	// read with the user parameters of the subgraph override.
	InitializeUserParametersStruct();
	if (GraphInstanceParametersOverride.IsValid())
	{
		// In the case of subgraph override, and the subgraph override is different from the one setup on the node, the settings might not have 
		// all the overrides. If the "Overrides" pin is also connected, first make a pass on it to setup the graph overrides.
		// This will be super-seeded by pin overrides connected. 
		const UPCGSubgraphSettings* SubgraphSettings = GetInputSettings<UPCGSubgraphSettings>();
		const bool bHasSubgraphOverride = SubgraphSettings && SubgraphSettings->SubgraphOverride;
		const UPCGGraph* RootSubgraph = SubgraphSettings && SubgraphSettings->SubgraphInstance ? SubgraphSettings->SubgraphInstance->GetGraph() : nullptr;
		const bool bHasDifferentSubgraphOverride = bHasSubgraphOverride && SubgraphSettings->SubgraphOverride->GetGraph() != RootSubgraph;
		
		if (bHasDifferentSubgraphOverride && Node && Node->IsInputPinConnected(PCGPinConstants::DefaultParamsLabel))
		{
			const TArray<FPCGTaggedData> OverrideData = InputData.GetParamsByPin(PCGPinConstants::DefaultParamsLabel);
			if (OverrideData.Num() > 1)
			{
				PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGPinConstants::DefaultParamsLabel, this);
			}
			
			if (!OverrideData.IsEmpty())
			{
				void* ParametersPtr = GraphInstanceParametersOverride.GetMutableMemory();
				PCGPropertyHelpers::ExtractAttributeSetToContainers(
					CastChecked<UPCGParamData>(OverrideData[0].Data),
					GraphInstanceParametersOverride.GetScriptStruct(),
					MakeArrayView<void*>(&ParametersPtr, 1),
					/*OptionalNameMapping=*/nullptr,
					this);
			}
		}

		OverrideSettings();
	}
}

void FPCGSubgraphContext::AddToReferencedObjects(const FPCGDataCollection& InDataCollection)
{
	for (const FPCGTaggedData& TaggedData : InDataCollection.TaggedData)
	{
		if (TaggedData.Data)
		{
			ReferencedObjects.Add(TaggedData.Data);
		}
	}
}

void FPCGSubgraphContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ReferencedObjects);
}

FPCGContext* FPCGSubgraphElement::Initialize(const FPCGInitializeElementParams& InParams)
{
	FPCGSubgraphContext* Context = new FPCGSubgraphContext();
	Context->InitFromParams(InParams);
	Context->InitializeUserParametersStruct();

	return Context;
}

void FPCGSubgraphElement::PrepareSubgraphData(const UPCGBaseSubgraphSettings* Settings, FPCGSubgraphContext* Context, const FPCGDataCollection& InputData, FPCGDataCollection& OutputData) const
{
	// Don't forward overrides
	if (Settings->HasOverridableParams())
	{
		OutputData.TaggedData.Reserve(InputData.TaggedData.Num());
		const TArray<FPCGPinProperties> InputPins = Settings->DefaultInputPinProperties();

		for (const FPCGTaggedData& Input : InputData.TaggedData)
		{
			if (!Input.Data)
			{
				continue;
			}

			// Discard params that don't have a pin on the subgraph input node
			if (!Input.Data->IsA<UPCGParamData>() || Algo::FindByPredicate(InputPins, [Pin = Input.Pin](const FPCGPinProperties& PinProperty) { return PinProperty.Label == Pin; }))
			{
				OutputData.TaggedData.Add(Input);
			}
		}
	}
	else
	{
		OutputData = InputData;
	}

	// We also need to make sure we are not forwarding any UserParameterData from previous parents. So we remove all of them.
	OutputData.TaggedData.RemoveAllSwap([](const FPCGTaggedData& OutData) { return Cast<UPCGUserParametersData>(OutData.Data); });

	// Note for the future: Dynamic subgraphs are forwarding any data coming from the PreTask of their parent, so if you ever get data that should not be there,
	// you should probably do some filtering, like the UPCGUserParametersData.
}

void FPCGSubgraphElement::PrepareSubgraphUserParameters(const UPCGBaseSubgraphSettings* Settings, FPCGSubgraphContext* Context, FPCGDataCollection& OutputData) const
{
	// Also create a new data containing information about the original subgraph and the parameters override
	// It is used mainly by the UserParameterGetElement to access the correct value.
	// By construction, there should be one and only one of this data. (Filtering of previous data is done in PrepareSubgraphData)
	if (const UPCGGraphInterface* SubgraphInterface = Settings->GetSubgraphInterface())
	{
		UPCGUserParametersData* UserParamData = FPCGContext::NewObject_AnyThread<UPCGUserParametersData>(Context);

		if (Context->GraphInstanceParametersOverride.IsValid())
		{
			UserParamData->UserParameters = std::move(Context->GraphInstanceParametersOverride);
		}
		else if (const FInstancedPropertyBag* InstancedPropertyBag = SubgraphInterface->GetUserParametersStruct())
		{
			UserParamData->UserParameters = InstancedPropertyBag->GetValue();
		}

		// Hook up user parameter data from upstream
		TArray<FPCGTaggedData> UpstreamUserParameterData = Context->InputData.GetTaggedTypedInputs<UPCGUserParametersData>(PCG::Private::UserParameterTagData);
		if (!UpstreamUserParameterData.IsEmpty())
		{
#if WITH_EDITOR
			// Safeguard to make sure we always have one and only one data of this type
			ensure(UpstreamUserParameterData.Num() == 1);
#endif

			if (UPCGUserParametersData* UpstreamData = Cast<UPCGUserParametersData>(const_cast<UPCGData*>(UpstreamUserParameterData[0].Data.Get())))
			{
				UserParamData->UpstreamData = UpstreamData;
			}
		}

		FPCGTaggedData& TaggedData = OutputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = UserParamData;
		TaggedData.Tags.Add(PCG::Private::UserParameterTagData);
		// Mark this data pinless, since it is internal data, not meant to be shown in the graph editor.
		TaggedData.bPinlessData = true;
	}
}

bool FPCGSubgraphElement::IsPassthrough(const UPCGSettings* InSettings) const
{
	const UPCGBaseSubgraphSettings* Settings = Cast<UPCGBaseSubgraphSettings>(InSettings);
	return (!Settings || (Settings->bEnabled && Settings->GetSubgraph()));
}

bool FPCGSubgraphElement::ExecuteInternal(FPCGContext* InContext) const
{
	FPCGSubgraphContext* Context = static_cast<FPCGSubgraphContext*>(InContext);

	const UPCGBaseSubgraphSettings* Settings = Context->GetInputSettings<UPCGBaseSubgraphSettings>();
	check(Settings);

	bool bIsDynamic = Settings->IsDynamicGraph();

#if WITH_EDITOR
	bIsDynamic |= PCGSystemSwitches::CVarForceDynamicGraphDispatch.GetValueOnAnyThread();
#endif

	if (bIsDynamic && !Context->bScheduledSubgraph && Settings->GetSubgraphOverride())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSubgraphElement::ExecuteInternal::DynamicTrackingAndOverrides);

#if WITH_EDITOR
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, FPCGSelectionKey::CreateFromPath(Settings->GetSubgraphOverride()), /*bIsCulled=*/false);
		// Also need to add statically tracked keys
		if (InContext && InContext->ExecutionSource.IsValid())
		{
			if (const UPCGGraph* Graph = Settings->GetSubgraphOverride()->GetGraph())
			{
				InContext->ExecutionSource->GetExecutionState().RegisterDynamicTracking(Graph->GetTrackedSelectionKeysToSettings());
			}
		}
#endif // WITH_EDITOR

		Context->UpdateOverridesWithOverriddenGraph();
	}

	UPCGGraph* Subgraph = Settings->GetSubgraph();

	// Implementation note: recursivity test here must be consequential with the way the compilation has been done,
	// otherwise the other tasks will not behave as expected.
	// If the current graph is present in the subgraph downstream, then this must be a dynamic graph execution
	bool bIsRecursive = false;
	if (Subgraph)
	{
		if (const FPCGStack* Stack = Context->GetStack())
		{
			bIsRecursive = (bIsDynamic && Stack->HasObject(Subgraph)) || Subgraph->Contains(Stack->GetGraphForCurrentFrame());
		}
		else if(Context->ExecutionSource.Get())
		{
			const UPCGGraph* CurrentSourceGraph = Context->ExecutionSource->GetExecutionState().GetGraph();
			bIsRecursive = (Subgraph == CurrentSourceGraph || Subgraph->Contains(CurrentSourceGraph));
		}
	}

	if (bIsDynamic || bIsRecursive)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSubgraphElement::Execute);

		if (!Context->bScheduledSubgraph)
		{
			if (Subgraph)
			{
				// Dispatch graph to execute with the given information we have
				// using this node's task id as additional inputs
				FPCGDataCollection PreSubgraphInputData;
				PrepareSubgraphUserParameters(Settings, Context, PreSubgraphInputData);
				Context->AddToReferencedObjects(PreSubgraphInputData);

				FPCGDataCollection SubgraphInputData;
				PrepareSubgraphData(Settings, Context, Context->InputData, SubgraphInputData);
				Context->AddToReferencedObjects(SubgraphInputData);

				// At this point, if we're in a recursive context and we have no input, we must terminate execution
				if (bIsRecursive && SubgraphInputData.TaggedData.IsEmpty())
				{
					return true;
				}

				// Prepare the invocation stack - which is the stack up to this node, and then this node and the 'not-a-loop index' which we use to differentiate dynamic vs static subgraphs
				const FPCGStack* Stack = Context->GetStack();
				FPCGStack InvocationStack = ensure(Stack) ? *Stack : FPCGStack();
				InvocationStack.GetStackFramesMutable().Emplace(Context->Node);
				InvocationStack.GetStackFramesMutable().Emplace(INDEX_NONE); // not a loop index

				// Higen is not allowed in dynamic subgraphs, entire subgraph is executed on the same grid as this subgraph node.
				FPCGTaskId SubgraphTaskId = Context->ScheduleGraph(FPCGScheduleGraphParams(
					Subgraph,
					Context->ExecutionSource.Get(),
					MakeShared<FPCGInputForwardingElement>(PreSubgraphInputData),
					MakeShared<FPCGInputForwardingElement>(SubgraphInputData),
					/*Dependencies=*/{},
					&InvocationStack,
					/*bAllowHierarchicalGeneration=*/false));

				if (SubgraphTaskId != InvalidPCGTaskId)
				{
					Context->SubgraphTaskIds.Add(SubgraphTaskId);
					Context->bScheduledSubgraph = true;
					Context->bIsPaused = true;
					Context->DynamicDependencies.Add(SubgraphTaskId);
					return false;
				}
				else
				{
					// Scheduling failed - early out
					Context->OutputData.bCancelExecution = true;
					return true;
				}
			}
			else if (!Subgraph)
			{
				// Simple pass-through
				Context->OutputData.TaggedData = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
				return true;
			}
			else
			{
				// Job cannot run; cancel
				Context->OutputData.bCancelExecution = true;
				return true;
			}
		}
		else if (Context->bIsPaused)
		{
			// Should not happen once we skip it in the graph executor
			return false;
		}
		else
		{
			// when woken up, get the output data from the subgraph
			// and copy it to the current context output data, and finally return true
			if (Context->SubgraphTaskIds.Num() > 0)
			{
				// This element does not support multiple results/dispatches. Note that this would be easy to fix;
				// as we just can call the GetOutputData on a fresh data collection and merge it afterwards in the output data,
				// but this is not needed here so we will keep the full assignment & benefit from the crc as well.
				ensure(Context->SubgraphTaskIds.Num() == 1);
				if (Context->GetOutputData(Context->SubgraphTaskIds[0], Context->OutputData))
				{
					Context->ClearOutputData(Context->SubgraphTaskIds[0]);
				}
			}

			return true;
		}
	}
	else
	{
		// This node acts as the pre-graph node only.
		PrepareSubgraphUserParameters(Settings, Context, Context->OutputData);
		return true;
	}
}

FPCGInputForwardingElement::FPCGInputForwardingElement(const FPCGDataCollection& InputToForward)
	: Input(InputToForward)
{
}

bool FPCGInputForwardingElement::ExecuteInternal(FPCGContext* Context) const
{
	Context->OutputData = Input;
	return true;
}

#undef LOCTEXT_NAMESPACE
