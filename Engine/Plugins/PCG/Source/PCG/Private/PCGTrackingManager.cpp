// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGTrackingManager.h"

#include "PCGComponent.h"
#include "PCGDataAsset.h"
#include "PCGGraph.h"
#include "PCGManagedResource.h"
#include "PCGModule.h"
#include "PCGSubsystem.h"
#include "PCGWorldActor.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Utils/PCGGraphExecutionLogging.h"

#include "Landscape.h"
#include "StaticMeshCompiler.h"
#include "TextureCompiler.h"
#include "Algo/AnyOf.h"
#include "Algo/ForEach.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/StaticMesh.h"
#include "Kismet/GameplayStatics.h"
#include "Materials/MaterialInterface.h"

#if WITH_EDITOR
#include "ChangeTracking/PCGChangeTrackingRegistry.h"
#include "Grid/PCGPartitionActorDesc.h"
#include "UObject/PackageReload.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#endif

namespace PCGTrackingManager
{
#if WITH_EDITOR
	static TAutoConsoleVariable<bool> CVarDisableObjectDependenciesTracking(
		TEXT("pcg.DisableObjectDependenciesTracking"),
		false,
		TEXT("If depencencies are being unstable, disable the tracking, allowing people to continue working while we investigate."));

	static TAutoConsoleVariable<bool> CVarDisablePCGDataInterdependencyOptimization(
		TEXT("pcg.DisablePCGDataInterdependencyOptimization"),
		false,
		TEXT("Disable the optimization that keep track of components depending on others, as a safety measure."));
#endif // WITH_EDITOR

	bool IsValidOriginalExecutionSource(const IPCGGraphExecutionSource* InExecutionSource)
	{
		const UObject* ExecutionSourceObject = Cast<UObject>(InExecutionSource);

		// Ignore invalid, templates and local sources
		if (!IsValid(ExecutionSourceObject) || ExecutionSourceObject->IsTemplate() || InExecutionSource->GetExecutionState().IsLocalSource())
		{
			return false;
		}
		// Keep existing PCG Component conditions for now (should probably be moved inside execution source api)
		else if (const UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource))
		{
			return PCGComponent->GetOwner() && !PCGComponent->GetOwner()->IsA<APCGPartitionActor>();
		}
		else
		{
			return true;
		}
	}

	bool IsValidExecutionSource(const IPCGGraphExecutionSource* InExecutionSource)
	{
		const UObject* ExecutionSourceObject = Cast<UObject>(InExecutionSource);
	
		// Ignore invalid and templates
		if (!IsValid(ExecutionSourceObject) || ExecutionSourceObject->IsTemplate())
		{
			return false;
		}
		// Keep existing PCG Component conditions for now (should probably be moved inside execution source api)
		else if (const UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource))
		{
			return PCGComponent->GetOwner() != nullptr;
		}
		else
		{
			return true;
		}
	}
}

FPCGTrackingManager::FPCGTrackingManager(UPCGSubsystem* InPCGSubsystem)
	: PCGSubsystem(InPCGSubsystem)
{
	check(PCGSubsystem);

	// TODO: For now we set our octree to be 2km wide, but it would be perhaps better to
	// scale it to the size of our world.
	constexpr FVector::FReal OctreeExtent = 200000; // 2km
	PartitionedOctree.Reset(FVector::ZeroVector, OctreeExtent);
	NonPartitionedOctree.Reset(FVector::ZeroVector, OctreeExtent);
}

void FPCGTrackingManager::Initialize()
{
#if WITH_EDITOR
	BuildPartitionActorRecords();
	RegisterDelegates();

	ChangeTrackers = FPCGModule::GetPCGModuleChecked().GetConstChangeTrackingRegistry().CreateChangeTrackers(this);
	ChangeHandlers = FPCGModule::GetPCGModuleChecked().GetConstChangeTrackingRegistry().CreateChangeHandlers(this);
#endif // WITH_EDITOR
}

void FPCGTrackingManager::Deinitialize()
{
#if WITH_EDITOR
	UnregisterDelegates();

	ChangeTrackers.Empty();
	ChangeHandlers.Empty();
#endif
}

void FPCGTrackingManager::Tick()
{
#if WITH_EDITOR

	// Process delayed Registrations
	TSet<TWeakObjectPtr<UPCGComponent>> ComponentsToUnregister;
	TSet<TWeakObjectPtr<UPCGComponent>> ComponentsToRegister;
	{
		PCG::TScopeLock Lock(DelayedComponentsRegistrationLock);
		ComponentsToUnregister = MoveTemp(DelayedComponentsToUnregister);
		ComponentsToRegister = MoveTemp(DelayedComponentsToRegister);
	}

	Algo::ForEach(ComponentsToRegister, [this](TWeakObjectPtr<UPCGComponent> WeakComponent) 
	{
		if(UPCGComponent* Component = WeakComponent.Get())
		{
			RegisterOrUpdateExecutionSource(Component, Component->bGenerated, /*bForce=*/true); 
		}
	});
	
	// Implementation note: When a PCG Component is in delayed, it means it is part of a BP actor that was created through a construction script.
	// If it gets reconstructed, it will be picked up by `RemapPCGComponent` and removed from delayed.
	// If it is still in delayed on the next tick, it means it is ready to be unregister.
	// If the actor gets destroyed, the component is now unreachable and will be unregister at the next GC (when we run BeginDestroy)
	// To force the unregistration, we allow to resolve the component even if pending kill.
	Algo::ForEach(ComponentsToUnregister, [this](TWeakObjectPtr<UPCGComponent> WeakComponent)
	{ 
		if (UPCGComponent* Component = WeakComponent.GetEvenIfUnreachable(); WeakComponent.IsValid(/*bEvenIfPendingKill=*/true) && Component)
		{
			UnregisterExecutionSource(Component, /*bForce=*/true); 
		}
	});

	for (TUniquePtr<IPCGChangeTracker>& ChangeTracker : ChangeTrackers)
	{
		ChangeTracker->Tick();
	}

	if (PCGTrackingManager::CVarDisablePCGDataInterdependencyOptimization.GetValueOnAnyThread() && !ExecutionSourcesToDependencyMap.IsEmpty())
	{
		ExecutionSourcesToDependencyMap.Empty();
	}
#endif // WITH_EDITOR
}

UWorld* FPCGTrackingManager::GetWorld() const
{
	return PCGSubsystem->GetWorld();
}

TArray<FPCGTaskId> FPCGTrackingManager::DispatchToRegisteredLocalComponents(UPCGComponent* OriginalComponent, const TFunctionRef<FPCGTaskId(UPCGComponent*, const TArray<FPCGTaskId>&)>& InFunc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTrackingManager::DispatchToRegisteredLocalComponents);
	if (!ensure(OriginalComponent))
	{
		return {};
	}

	// TODO: Might be more interesting to copy the set and release the lock.
	PCG::TSharedScopeLock ReadLock(ComponentToPartitionActorsMapLock);
	const TSet<TObjectPtr<APCGPartitionActor>>* PartitionActorsPtr = ComponentToPartitionActorsMap.Find(OriginalComponent);

	if (!PartitionActorsPtr)
	{
		return TArray<FPCGTaskId>();
	}

	return DispatchToLocalComponents(OriginalComponent, *PartitionActorsPtr, InFunc);
}

TArray<FPCGTaskId> FPCGTrackingManager::DispatchToLocalComponents(UPCGComponent* OriginalComponent, const TSet<TObjectPtr<APCGPartitionActor>>& PartitionActors, const TFunctionRef<FPCGTaskId(UPCGComponent*, const TArray<FPCGTaskId>&)>& InFunc) const
{
	TArray<FPCGTaskId> TaskIds;

	auto DispatchLambda = [this, OriginalComponent, &InFunc, &TaskIds](const TObjectPtr<APCGPartitionActor>& PartitionActor)
	{
		if (PartitionActor)
		{
			if (UPCGComponent* LocalComponent = PartitionActor->GetLocalComponent(OriginalComponent))
			{
				// Add check to avoid infinite loop
				if (ensure(!LocalComponent->IsPartitioned()))
				{
					FPCGTaskId LocalTask = InvalidPCGTaskId;
#if WITH_EDITOR
					if (bChainedDispatchToLocalComponents && TaskIds.Num() > 0)
					{
						LocalTask = InFunc(LocalComponent, { TaskIds.Last() });
					}
					else
#endif // WITH_EDITOR
					{
						LocalTask = InFunc(LocalComponent, { });
					}

					if (LocalTask != InvalidPCGTaskId)
					{
						TaskIds.Add(LocalTask);
					}
				}
			}
		}
	};

#if WITH_EDITOR
	if (bChainedDispatchToLocalComponents)
	{
		TArray<TObjectPtr<APCGPartitionActor>> SortedPartitionActors = PartitionActors.Array();
		SortedPartitionActors.Remove(nullptr);
		SortedPartitionActors.Sort([](const APCGPartitionActor& ActorA, const APCGPartitionActor& ActorB)
		{
			return ActorA.GetPathName() < ActorB.GetPathName();
		});
		Algo::ForEach(SortedPartitionActors, DispatchLambda);
	}
	else
#endif // WITH_EDITOR
	{
		Algo::ForEach(PartitionActors, DispatchLambda);
	}

	return TaskIds;
}

bool FPCGTrackingManager::RegisterOrUpdateExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bDoPartitionMapping)
{
	return RegisterOrUpdateExecutionSource(InExecutionSource, bDoPartitionMapping, /*bForce=*/false);
}

bool FPCGTrackingManager::RegisterOrUpdateExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bDoPartitionMapping, bool bForce)
{
	check(InExecutionSource);

	if (!PCGTrackingManager::IsValidOriginalExecutionSource(InExecutionSource))
	{
		return false;
	}

	// Check also that the bounds are valid. If not early out.
	if (!InExecutionSource->GetExecutionState().GetBounds().IsValid)
	{
		UE_LOGF(LogPCG, Error, "[RegisterOrUpdateExecutionSource] Component has invalid bounds, not registered nor updated.");
		return false;
	}

	// @todo_pcg: support execution source
	UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource);
#if WITH_EDITOR
	if(PCGComponent)
	{
		PCG::TScopeLock Lock(DelayedComponentsRegistrationLock);

		DelayedComponentsToUnregister.Remove(PCGComponent);
		DelayedComponentsToRegister.Remove(PCGComponent);

		// When we are running Construction Script there are 2 possibilities
		// 1. This component is replacing an existing PCG Component and we don't want to register, we want to replace in the RemapPCGComponent call (remap will remove this component from the Delayed list)
		// 2. This component is new and so it will be registered after the potential replacement call on the next frame in FPCGTrackingManager::Tick
		if (!bForce && GetWorld()->bIsRunningConstructionScript && PCGComponent->IsCreatedByConstructionScript())
		{
			DelayedComponentsToRegister.Add(PCGComponent);
			return true;
		}
	}
#endif // WITH_EDITOR

	const bool bWasAlreadyRegistered = IsExecutionSourceRegistered(InExecutionSource);

	// First check if the execution source has changed its partitioned flag.
	const bool bIsPartitioned = InExecutionSource->GetExecutionState().IsPartitioned();
	if (bIsPartitioned && NonPartitionedOctree.Contains(InExecutionSource))
	{
		UnregisterNonPartitionedExecutionSource(InExecutionSource);
	}
	else if (!bIsPartitioned && PartitionedOctree.Contains(InExecutionSource))
	{
		UnregisterPartitionedExecutionSource(InExecutionSource);
	}

	PCGSubsystem->OnOriginalExecutionSourceRegistered(InExecutionSource);

	// Then register/update accordingly
	bool bHasChanged = false;
	if (bIsPartitioned)
	{
		bHasChanged = RegisterOrUpdatePartitionedExecutionSource(InExecutionSource, bDoPartitionMapping);
	}
	else
	{
		bHasChanged = RegisterOrUpdateNonPartitionedExecutionSource(InExecutionSource);
	}

#if WITH_EDITOR
	// And finally handle the tracking. Only do it when the execution source is registered for the first time.
	if (!bWasAlreadyRegistered && bHasChanged)
	{
		RegisterTracking(InExecutionSource);
	}
#endif // WITH_EDITOR

	return bHasChanged;
}

bool FPCGTrackingManager::RegisterOrUpdatePartitionedExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bDoActorMapping)
{
	FBox Bounds(EForceInit::ForceInit);
	bool bExecutionSourceHasChanged = false;
	bool bExecutionSourceWasAdded = false;

	PartitionedOctree.AddOrUpdateExecutionSource(InExecutionSource, Bounds, bExecutionSourceHasChanged, bExecutionSourceWasAdded);

	// @todo_pcg: support execution source
	UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource);
	if (!PCGComponent)
	{
		return bExecutionSourceHasChanged;
	}

#if WITH_EDITOR
	// In Editor only, we will create new partition actors depending on the new bounds and generation trigger. Runtime managed execution sources should not create PAs here
	if ((bExecutionSourceHasChanged || bExecutionSourceWasAdded) && !InExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
	{
		bool bHasUnbounded = false;
		PCGHiGenGrid::FSizeArray GridSizes;
		ensure(PCGHelpers::GetGenerationGridSizes(InExecutionSource->GetExecutionState().GetGraph(), PCGSubsystem->GetPCGWorldActor(), GridSizes, bHasUnbounded));
		PCGSubsystem->CreatePartitionActorsWithinBounds(PCGComponent, Bounds, GridSizes);
	}
#endif // WITH_EDITOR

	// After adding/updating, try to do the mapping (if we asked for it and the component changed)
	if (bDoActorMapping)
	{
		if (bExecutionSourceHasChanged)
		{
			UpdateMappingPCGComponentPartitionActor(PCGComponent);
		}
	}
	else
	{
		if (!bExecutionSourceWasAdded)
		{
			// If we do not want a mapping, delete the existing one
			DeleteMappingPCGComponentPartitionActor(PCGComponent);
		}
		else
		{
			// Here bDoActorMapping is false so we just want to update the ComponentToPartitionActorsMap without adding/removing graph instances
			UpdateMappingPCGComponentPartitionActor(PCGComponent, /*bChangeGraphInstances=*/false);
		}
	}

	return bExecutionSourceHasChanged;
}

bool FPCGTrackingManager::RegisterOrUpdateNonPartitionedExecutionSource(IPCGGraphExecutionSource* InExecutionSource)
{
	FBox Bounds(EForceInit::ForceInit);
	bool bExecutionSourceHasChanged = false;
	bool bExecutionSourceWasAdded = false;

	NonPartitionedOctree.AddOrUpdateExecutionSource(InExecutionSource, Bounds, bExecutionSourceHasChanged, bExecutionSourceWasAdded);

	return bExecutionSourceHasChanged;
}

bool FPCGTrackingManager::RemapExecutionSource(const IPCGGraphExecutionSource* InOldExecutionSource, IPCGGraphExecutionSource* InNewExecutionSource, bool bDoPartitionMapping)
{
	check(InOldExecutionSource && InNewExecutionSource);

	bool bBoundsChanged = false;

	if (InOldExecutionSource->GetExecutionState().IsPartitioned())
	{
		if (!PartitionedOctree.RemapExecutionSource(InOldExecutionSource, InNewExecutionSource, bBoundsChanged))
		{
			return false;
		}
	}
	else
	{
		if (!NonPartitionedOctree.RemapExecutionSource(InOldExecutionSource, InNewExecutionSource, bBoundsChanged))
		{
			return false;
		}
	}

	// @todo_pcg: support execution source
	const UPCGComponent* OldComponent = Cast<UPCGComponent>(InOldExecutionSource);
	UPCGComponent* NewComponent = Cast<UPCGComponent>(InNewExecutionSource);
	
	if (OldComponent && NewComponent)
	{
#if WITH_EDITOR
		// This is called from within construction script to notify us that an execution source has been replaced with a new version.
		// - We expect the old execution source might be in the delayed unregisters and we remove it
		// - No need to touch delayed registers
		{
			PCG::TScopeLock Lock(DelayedComponentsRegistrationLock);
			DelayedComponentsToUnregister.Remove(OldComponent);
		}
#endif

		// Remap all previous instances
		auto RemapPreviousInstances = [OldComponent, NewComponent](TMap<const UPCGComponent*, TSet<TObjectPtr<APCGPartitionActor>>>& Map, PCG::FSharedLock& Lock)
		{
			PCG::TUniqueScopeLock WriteLock(Lock);

			if (TSet<TObjectPtr<APCGPartitionActor>>* PartitionActorsPtr = Map.Find(OldComponent))
			{
				TSet<TObjectPtr<APCGPartitionActor>> PartitionActorsToRemap = MoveTemp(*PartitionActorsPtr);
				Map.Remove(OldComponent);

				for (APCGPartitionActor* Actor : PartitionActorsToRemap)
				{
					Actor->RemapGraphInstance(OldComponent, NewComponent);
				}

				Map.Add(NewComponent, MoveTemp(PartitionActorsToRemap));
			}
		};

		RemapPreviousInstances(ComponentToPartitionActorsMap, ComponentToPartitionActorsMapLock);

		// And update the mapping if bounds changed and we want to do actor mapping
		if (bBoundsChanged && NewComponent->IsPartitioned() && bDoPartitionMapping)
		{
			UpdateMappingPCGComponentPartitionActor(NewComponent);
		}
	}

#if WITH_EDITOR
	RemapTracking(InOldExecutionSource, InNewExecutionSource);
#endif // WITH_EDITOR

	PCGSubsystem->OnOriginalExecutionSourceReplaced(InOldExecutionSource, InNewExecutionSource);

	return true;
}

void FPCGTrackingManager::UnregisterExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bForce)
{
	if (!InExecutionSource)
	{
		return;
	}

#if WITH_EDITOR
	UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource);

	if(PCGComponent)
	{
		PCG::TScopeLock Lock(DelayedComponentsRegistrationLock);
		DelayedComponentsToRegister.Remove(PCGComponent);
		DelayedComponentsToUnregister.Remove(PCGComponent);
	}

	if ((PartitionedOctree.Contains(InExecutionSource) || NonPartitionedOctree.Contains(InExecutionSource)))
	{
		// We also need to check that our current PCG Component is not deleted while being reconstructed by a construction script.
		// If so, it will be "re-created" at some point with the same properties.
		// In this particular case, we don't remove the PCG component from the octree and we won't delete the mapping, but mark it to be removed
		// at next Subsystem tick. If we call "RemapPCGComponent" before, we will re-connect everything correctly.
		// Ignore this if we force (aka when we actually unregister the delayed one)		
		if (!bForce && PCGComponent && PCGComponent->IsCreatedByConstructionScript())
		{
			PCG::TScopeLock Lock(DelayedComponentsRegistrationLock);
			DelayedComponentsToUnregister.Add(PCGComponent);
			return;
		}

		UnregisterTracking(InExecutionSource);
	}
#endif // WITH_EDITOR

	UnregisterPartitionedExecutionSource(InExecutionSource);
	UnregisterNonPartitionedExecutionSource(InExecutionSource);
}

void FPCGTrackingManager::UnregisterPartitionedExecutionSource(IPCGGraphExecutionSource* InExecutionSource)
{
	PCGSubsystem->OnOriginalExecutionSourceUnregistered(InExecutionSource);

	if (!PartitionedOctree.RemoveExecutionSource(InExecutionSource) || InExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
	{
		return;
	}

	// @todo_pcg: support execution source
	if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource))
	{
		// Because of recursive component deletes actors that has components, we cannot do RemoveGraphInstance
		// inside a lock. So copy the actors to clean up and release the lock before doing RemoveGraphInstance.
		TSet<TObjectPtr<APCGPartitionActor>> PartitionActorsToCleanUp;
		{
			PCG::TUniqueScopeLock WriteLock(ComponentToPartitionActorsMapLock);
			TSet<TObjectPtr<APCGPartitionActor>>* PartitionActorsPtr = ComponentToPartitionActorsMap.Find(PCGComponent);

			if (PartitionActorsPtr)
			{
				PartitionActorsToCleanUp = MoveTemp(*PartitionActorsPtr);
				ComponentToPartitionActorsMap.Remove(PCGComponent);
			}
		}

#if WITH_EDITOR
		// Avoid removing graph instances when we are being unloaded (non-destructive)
		if (!PCGComponent->bUnregisteredThroughLoading)
#endif
		{
			for (APCGPartitionActor* Actor : PartitionActorsToCleanUp)
			{
				Actor->RemoveGraphInstance(PCGComponent);
			}
		}
	}
}

void FPCGTrackingManager::UnregisterNonPartitionedExecutionSource(IPCGGraphExecutionSource* InExecutionSource)
{
	PCGSubsystem->OnOriginalExecutionSourceUnregistered(InExecutionSource);

	NonPartitionedOctree.RemoveExecutionSource(InExecutionSource);
}

void FPCGTrackingManager::ForAllIntersectingPartitionedExecutionSources(const FBoxCenterAndExtent& InBounds, TFunctionRef<void(IPCGGraphExecutionSource*)> InFunc) const
{
	PartitionedOctree.FindElementsWithBoundsTest(InBounds, [&InFunc](const FPCGExecutionSourceRef& ExecutionSourceRef)
	{
		InFunc(ExecutionSourceRef.ExecutionSource);
	});
}

void FPCGTrackingManager::ForAllOriginalExecutionSources(TFunctionRef<void(IPCGGraphExecutionSource*)> InFunc)
{
	for (IPCGGraphExecutionSource* ExecutionSource : PartitionedOctree.GetAllExecutionSources())
	{
		InFunc(ExecutionSource);
	}

	for (IPCGGraphExecutionSource* ExecutionSource : NonPartitionedOctree.GetAllExecutionSources())
	{
		InFunc(ExecutionSource);
	}
}

TArray<IPCGGraphExecutionSource*> FPCGTrackingManager::GetAllIntersectingExecutionSources(const FBoxCenterAndExtent& InBounds) const
{
	TArray<IPCGGraphExecutionSource*> Result;
	auto AddToResult = [&Result](const FPCGExecutionSourceRef& ExecutionSourceRef)
	{
		Result.Add(ExecutionSourceRef.ExecutionSource);
	};

	PartitionedOctree.FindElementsWithBoundsTest(InBounds, AddToResult);
	NonPartitionedOctree.FindElementsWithBoundsTest(InBounds, AddToResult);

	return Result;
}

void FPCGTrackingManager::RegisterPartitionActor(APCGPartitionActor* InActor)
{
	check(InActor);
		
#if WITH_EDITOR
	if (InActor->IsInvalidForPCG())
	{
		// No need to log actor was already flagged to be invalid (and logged a message for it), we can ignore it.
		return;
	}
#endif

	const FPCGGridDescriptor PartitionActorDescriptor = InActor->GetGridDescriptor();

#if WITH_EDITOR
	// Ignore Invalid Actors 
	if (InvalidPartitionActors.Contains(InActor->GetActorGuid()))
	{
		InActor->SetInvalidForPCG();
		UE_LOGF(LogPCG, Warning, "[RegisterPartitionActor] Invalid PCG Partiton Actor '%ls' (%ls). Please delete actor to remove warning.", *InActor->GetName(), *InActor->GetPackage()->GetName());
		return;
	} // Invalidate actor if there is a duplicate for the same grid information
	else if (FGuid* FoundGuid = PartitionActorRecords.Find({ PartitionActorDescriptor, InActor->GetGridCoord()}); FoundGuid && *FoundGuid != InActor->GetActorGuid())
	{
		InActor->SetInvalidForPCG();
		UE_LOGF(LogPCG, Warning, "[RegisterPartitionActor] Duplicate PCG Partition Actor '%ls' (%ls). Please delete actor to remove warning.", *InActor->GetName(), *InActor->GetPackage()->GetName());
		return;
	}
#endif
	const FIntVector GridCoord = InActor->GetGridCoord();

	check(PartitionActorDescriptor.GetGridSize() > 0);

	{
		PCG::TUniqueScopeLock WriteLock(PartitionActorsMapLock);

		TMap<FIntVector, TObjectPtr<APCGPartitionActor>>& PartitionActorsMapGrid = PartitionActorsMap.FindOrAdd(PartitionActorDescriptor);
		if (PartitionActorsMapGrid.Contains(GridCoord))
		{
			return;
		}

		PartitionActorsMapGrid.Add(GridCoord, InActor);
	}

	APCGWorldActor* WorldActor = PCGSubsystem->GetPCGWorldActorForPartitionActor(InActor);
	if (!ensureMsgf(WorldActor, TEXT("When registering %s (Level: %s), the PCG World Actor does not exist in the world of the subsystem nor the partition actor level.\n"
		  "This is ill-formed, make sure the PCG World Actor exists if you use PCG Partition actors."),
		  *InActor->GetName(), InActor->GetLevel() ? *InActor->GetLevel()->GetName() : TEXT("Unknown")))
	{
		return;
	}

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Handle move of the Use2DGrid flag for old partition actors that haven't been resaved
	InActor->UpdateUse2DGridIfNeeded(WorldActor->bUse2DGrid);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

	// Register to all the components that intersect with the PA. Ignore for runtime generated, it is handled manually
	if (!PartitionActorDescriptor.IsRuntime())
	{
		PCG::TUniqueScopeLock WriteLock(ComponentToPartitionActorsMapLock);
		ForAllIntersectingPartitionedExecutionSources(FBoxCenterAndExtent(InActor->GetFixedBounds()), [this, InActor, WorldActor, PartitionActorDescriptor](IPCGGraphExecutionSource* ExecutionSource)
		{
			// For each component, do the mapping if the component is generated or if the Partition Actor already has a local component
			// Partition Actors could have a local component while the original doesn't have a mapping if for instance a cleanup/save on the original component happened
			// while not saving the Partition Actors
			if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(ExecutionSource))
			{
				FPCGGridDescriptor ComponentGridDescriptor = PCGComponent->GetGridDescriptor(InActor->GetPCGGridSize());
				if (ComponentGridDescriptor == PartitionActorDescriptor)
				{
					if (PCGComponent->bGenerated || (InActor->GetLocalComponent(PCGComponent) != nullptr))
					{
						bool bHasUnbounded = false;
						PCGHiGenGrid::FSizeArray GridSizes;
						ensure(PCGHelpers::GetGenerationGridSizes(PCGComponent->GetGraph(), WorldActor, GridSizes, bHasUnbounded));

						if (GridSizes.Contains(PartitionActorDescriptor.GetGridSize()))
						{
							// In editor we might load/create partition actors while the component is registering. Because of that,
							// the mapping might not already exists, even if the component is marked generated.
							// Make sure that this condition is only checked when we are in an editor world.
							const UWorld* World = GetWorld();
							if ((World && World->IsGameWorld()) || !UE::GetIsEditorLoadingPackage())
							{
								InActor->AddGraphInstance(PCGComponent);
							}
							
							// Either we already had a local component or one was just added
							if (InActor->GetLocalComponent(PCGComponent))
							{
								// Add mapping
								ComponentToPartitionActorsMap.FindOrAdd(PCGComponent).Add(InActor);
							}
						}
					}
				}
			}
		});
	}
}

void FPCGTrackingManager::UnregisterPartitionActor(APCGPartitionActor* Actor)
{
	check(Actor);

#if WITH_EDITOR
	// Ignore Invalid Actors 
	if (Actor->IsInvalidForPCG())
	{
		return;
	}
#endif

	const FIntVector GridCoord = Actor->GetGridCoord();
	const FPCGGridDescriptor GridDescriptor = Actor->GetGridDescriptor();
	if (!ensure(GridDescriptor.GetGridSize() > 0))
	{
		return;
	}

	if (TMap<FIntVector, TObjectPtr<APCGPartitionActor>>* PartitionActorsMapGrid = PartitionActorsMap.Find(GridDescriptor))
	{
		PCG::TUniqueScopeLock WriteLock(PartitionActorsMapLock);
		PartitionActorsMapGrid->Remove(GridCoord);
	}

	// Unregister from all intersecting components. Ignore for runtime generated, it is handled manually
	if (!GridDescriptor.IsRuntime())
	{
		PCG::TUniqueScopeLock WriteLock(ComponentToPartitionActorsMapLock);
		ForAllIntersectingPartitionedExecutionSources(FBoxCenterAndExtent(Actor->GetFixedBounds()), [this, Actor](IPCGGraphExecutionSource* ExecutionSource)
		{
			if(UPCGComponent* PCGComponent = Cast<UPCGComponent>(ExecutionSource))
			{
				TSet<TObjectPtr<APCGPartitionActor>>* PartitionActorsPtr = ComponentToPartitionActorsMap.Find(PCGComponent);
				if (PartitionActorsPtr)
				{
					PartitionActorsPtr->Remove(Actor);
				}
			}
		});
	}
}

void FPCGTrackingManager::ForAllIntersectingPartitionActors(const FBox& InBounds, TFunctionRef<void(APCGPartitionActor*)> InFunc) const
{
	if (!InBounds.IsValid)
	{
		return;
	}

	const bool bInBoundsAreEmpty = FMath::IsNearlyZero(InBounds.GetVolume());

	auto ForAllIntersectingPartitionActorsOfGridSize = [InFunc, &InBounds, this, bInBoundsAreEmpty](const FPCGGridDescriptor& GridDescriptor)
	{
		PCG::TSharedScopeLock ReadLock(PartitionActorsMapLock);

		const TMap<FIntVector, TObjectPtr<APCGPartitionActor>>* PartitionActorsMapGrid = PartitionActorsMap.Find(GridDescriptor);
		if (!PartitionActorsMapGrid || PartitionActorsMapGrid->IsEmpty())
		{
			return;
		}

		const FIntVector MinCellCoords = UPCGActorHelpers::GetCellCoord(InBounds.Min, GridDescriptor.GetGridSize(), GridDescriptor.Is2DGrid());
		const FIntVector MaxCellCoords = UPCGActorHelpers::GetCellCoord(InBounds.Max, GridDescriptor.GetGridSize(), GridDescriptor.Is2DGrid());

		for (int32 z = MinCellCoords.Z; z <= MaxCellCoords.Z; z++)
		{
			for (int32 y = MinCellCoords.Y; y <= MaxCellCoords.Y; y++)
			{
				for (int32 x = MinCellCoords.X; x <= MaxCellCoords.X; x++)
				{
					FIntVector CellCoords(x, y, z);
					if (const TObjectPtr<APCGPartitionActor>* ActorPtr = PartitionActorsMapGrid->Find(CellCoords))
					{
						if (APCGPartitionActor* Actor = ActorPtr->Get())
						{
							const FBox ActorBounds = Actor->GetFixedBounds();
							// Exclude any surrounding cells which are touching this cell but not meaningfully overlapping, only if the InBounds are
							// not "empty" (volume > 0). If the InBounds are "empty" (just a single point), the overlap will always be zero,
							// so just do an intersection check.
							const bool bShouldProcess = bInBoundsAreEmpty
								? ActorBounds.Intersect(InBounds)
								: !FMath::IsNearlyZero(Actor->GetFixedBounds().Overlap(InBounds).GetVolume());
							
							if (bShouldProcess)
							{
								InFunc(Actor);
							}
						}
					}
				}
			}
		}
	};

	TArray<FPCGGridDescriptor> GridDescriptors;
	{
		PCG::TSharedScopeLock ReadLock(PartitionActorsMapLock);

		PartitionActorsMap.GenerateKeyArray(GridDescriptors);
	}

	for (const FPCGGridDescriptor& GridDescriptor : GridDescriptors)
	{
		ForAllIntersectingPartitionActorsOfGridSize(GridDescriptor);
	}
}

void FPCGTrackingManager::UpdateMappingPCGComponentPartitionActor(UPCGComponent* InComponent, bool bInChangeGraphInstances)
{
	if (!PCGSubsystem->IsInitialized())
	{
		return;
	}

	check(InComponent);

	// Do not modify Partition Actors when loading
	const bool bChangeGraphInstances = bInChangeGraphInstances && !UE::GetIsEditorLoadingPackage();

	if (!InComponent->GetGraph())
	{
		return;
	}

	// Get the bounds
	FBox Bounds = PartitionedOctree.GetBounds(InComponent);

	if (!Bounds.IsValid)
	{
		return;
	}

	if (const APCGWorldActor* WorldActor = PCGSubsystem->GetPCGWorldActor())
	{
		bool bHasUnbounded = false;
		PCGHiGenGrid::FSizeArray GridSizes;
		ensure(PCGHelpers::GetGenerationGridSizes(InComponent->GetGraph(), WorldActor, GridSizes, bHasUnbounded));
				
		TSet<FPCGGridDescriptor> GridDescriptors;
		Algo::Transform(GridSizes, GridDescriptors, [InComponent](uint32 GridSize)
		{ 
			return InComponent->GetGridDescriptor(GridSize);
		});

		auto UpdateMapping = [this, InComponent, &Bounds, WorldActor, &GridDescriptors, bChangeGraphInstances]()
		{
			TSet<TObjectPtr<APCGPartitionActor>> RemovedActors;

			{
				PCG::TUniqueScopeLock WriteLock(ComponentToPartitionActorsMapLock);
				TSet<TObjectPtr<APCGPartitionActor>>& PartitionActors = ComponentToPartitionActorsMap.FindOrAdd(InComponent);

				TSet<TObjectPtr<APCGPartitionActor>> NewMapping;
				ForAllIntersectingPartitionActors(Bounds, [&NewMapping, InComponent, WorldActor, &GridDescriptors, bChangeGraphInstances](APCGPartitionActor* Actor)
				{
					if (!Actor)
					{
						return;
					}

					// Only add a graph instance to partition actors with compatible Grid Descriptors
					if (GridDescriptors.Contains(Actor->GetGridDescriptor()))
					{
						if (bChangeGraphInstances)
						{
							Actor->AddGraphInstance(InComponent);
						}

						// Either we already had a local component or one was just added
						if (Actor->GetLocalComponent(InComponent))
						{
							NewMapping.Add(Actor);
						}
					}
				});

				// Find the ones that were removed
				RemovedActors = PartitionActors.Difference(NewMapping);

				PartitionActors = MoveTemp(NewMapping);
			}

			// Here we can only remove graph instances if bChangeGraphInstances, which means in some edge cases
			// we might have Partition Actors that have a PCG Component which is no longer linked to the original component.
			// Those need to be manually cleaned up
			if (bChangeGraphInstances)
			{
				// No need to be locked to do this.
				for (APCGPartitionActor* RemovedActor : RemovedActors)
				{
					if (RemovedActor)
					{
						RemovedActor->RemoveGraphInstance(InComponent);
					}
				}
			}
		};

		UpdateMapping();
	}
}

TSet<TObjectPtr<APCGPartitionActor>> FPCGTrackingManager::GetPCGComponentPartitionActorMappings(UPCGComponent* InComponent) const
{
	if (const TSet<TObjectPtr<APCGPartitionActor>>* PartitionActorsPtr = ComponentToPartitionActorsMap.Find(InComponent))
	{
		return *PartitionActorsPtr;
	}

	return TSet<TObjectPtr<APCGPartitionActor>>();
}

void FPCGTrackingManager::DeleteMappingPCGComponentPartitionActor(UPCGComponent* InComponent)
{
	check(InComponent);

	if (!InComponent->IsPartitioned())
	{
		return;
	}

	auto DeleteMapping = [this, InComponent](TMap<const UPCGComponent*, TSet<TObjectPtr<APCGPartitionActor>>>& Map, PCG::FSharedLock& Lock)
	{
		PCG::TUniqueScopeLock WriteLock(Lock);

		if (TSet<TObjectPtr<APCGPartitionActor>>* PartitionActorsPtr = Map.Find(InComponent))
		{
			for (APCGPartitionActor* Actor : *PartitionActorsPtr)
			{
				Actor->RemoveGraphInstance(InComponent);
			}

			PartitionActorsPtr->Empty();
		}
	};

	DeleteMapping(ComponentToPartitionActorsMap, ComponentToPartitionActorsMapLock);
}

bool FPCGTrackingManager::IsExecutionSourceRegistered(const IPCGGraphExecutionSource* InExecutionSource) const
{
	return PartitionedOctree.Contains(InExecutionSource) || NonPartitionedOctree.Contains(InExecutionSource);
}

bool FPCGTrackingManager::AnyRuntimeGenExecutionSourcesExist() const
{
	for (IPCGGraphExecutionSource* ExecutionSource : PartitionedOctree.GetAllExecutionSources())
	{
		if (ExecutionSource && ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			return true;
		}
	}

	for (IPCGGraphExecutionSource* ExecutionSource : NonPartitionedOctree.GetAllExecutionSources())
	{
		if (ExecutionSource && ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			return true;
		}
	}

	return false;
}

TSet<IPCGGraphExecutionSource*> FPCGTrackingManager::GetAllRegisteredPartitionedExecutionSources() const
{
	return PartitionedOctree.GetAllExecutionSources();
}

TSet<IPCGGraphExecutionSource*> FPCGTrackingManager::GetAllRegisteredNonPartitionedExecutionSources() const
{
	return NonPartitionedOctree.GetAllExecutionSources();
}

TSet<IPCGGraphExecutionSource*> FPCGTrackingManager::GetAllRegisteredExecutionSources() const
{
	TSet<IPCGGraphExecutionSource*> Res = GetAllRegisteredPartitionedExecutionSources();
	Res.Append(GetAllRegisteredNonPartitionedExecutionSources());
	return Res;
}

UPCGComponent* FPCGTrackingManager::GetLocalComponent(const FPCGGridDescriptor& GridDescriptor, const FIntVector& CellCoords, const UPCGComponent* InOriginalComponent) const
{	
	PCG::TSharedScopeLock ReadLock(PartitionActorsMapLock);

	if (const TMap<FIntVector, TObjectPtr<APCGPartitionActor>>* PartitionActorsOnGrid = PartitionActorsMap.Find(GridDescriptor))
	{
		const TObjectPtr<APCGPartitionActor>* PartitionActor = PartitionActorsOnGrid->Find(CellCoords);
		if (PartitionActor && *PartitionActor)
		{
			return (*PartitionActor)->GetLocalComponent(InOriginalComponent);
		}
	}

	return nullptr;
}

APCGPartitionActor* FPCGTrackingManager::GetPartitionActor(const FPCGGridDescriptor& GridDescriptor, const FIntVector& CellCoords) const
{
	PCG::TSharedScopeLock ReadLock(PartitionActorsMapLock);

	if (const TMap<FIntVector, TObjectPtr<APCGPartitionActor>>* PartitionActorsOnGrid = PartitionActorsMap.Find(GridDescriptor))
	{
		const TObjectPtr<APCGPartitionActor>* PartitionActor = PartitionActorsOnGrid->Find(CellCoords);
		return PartitionActor ? *PartitionActor : nullptr;
	}

	return nullptr;
}

#if WITH_EDITOR
bool FPCGTrackingManager::DoesPartitionActorRecordExist(const FPCGGridDescriptor& GridDescriptor, const FIntVector& GridCoords) const
{
	// In Editor this will not be empty (including -game)
	if (PartitionActorRecords.Contains({ GridDescriptor, GridCoords }))
	{
		return true;
	}
	
	return false;
}

void FPCGTrackingManager::RegisterTracking(IPCGGraphExecutionSource* InExecutionSource)
{
	if (!PCGTrackingManager::IsValidOriginalExecutionSource(InExecutionSource))
	{
		return;
	}

	UWorld* World = GetWorld();
	check(World);

	// Components owner needs to be always tracked
	AlwaysTrackedKeysToExecutionSourcesMap.FindOrAdd(FPCGSelectionKey(EPCGActorFilter::Self)).Add(InExecutionSource);

	UpdateTracking(InExecutionSource);
}

void FPCGTrackingManager::UpdateTracking(IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGSelectionKey>* ChangedKeys)
{
	if (!PCGTrackingManager::IsValidOriginalExecutionSource(InExecutionSource))
	{
		return;
	}

	UWorld* World = GetWorld();
	check(World);

	// If no keys are provided, update all tracking keys.
	TArray<FPCGSelectionKey> AllKeys;
	if (ChangedKeys == nullptr)
	{
		AllKeys = InExecutionSource->GetExecutionState().GatherTrackingKeys();
		ChangedKeys = &AllKeys;
	}

	check(ChangedKeys);

	auto RemoveFromMap = [InExecutionSource](TMap<FPCGSelectionKey, TSet<IPCGGraphExecutionSource*>>& InMap, const FPCGSelectionKey& InKey)
	{
		if (TSet<IPCGGraphExecutionSource*>* ExecutionSources = InMap.Find(InKey))
		{
			ExecutionSources->Remove(InExecutionSource);
			if (ExecutionSources->IsEmpty())
			{
				InMap.Remove(InKey);
			}
		}
	};

	for (const FPCGSelectionKey& Key : *ChangedKeys)
	{
		// bShouldBeCulled is modified in IsKeyTrackedAndCulled
		bool bShouldBeCulled = false;
		if (!InExecutionSource->GetExecutionState().IsKeyTrackedAndCulled(Key, bShouldBeCulled))
		{
			// Untrack
			RemoveFromMap(CulledTrackedKeysToExecutionSourcesMap, Key);
			RemoveFromMap(AlwaysTrackedKeysToExecutionSourcesMap, Key);
		}
		else if (bShouldBeCulled)
		{
			RemoveFromMap(AlwaysTrackedKeysToExecutionSourcesMap, Key);
			CulledTrackedKeysToExecutionSourcesMap.FindOrAdd(Key).Add(InExecutionSource);
		}
		else
		{
			RemoveFromMap(CulledTrackedKeysToExecutionSourcesMap, Key);
			AlwaysTrackedKeysToExecutionSourcesMap.FindOrAdd(Key).Add(InExecutionSource);
		}
	}
}

void FPCGTrackingManager::RemapTracking(const IPCGGraphExecutionSource* InOldExecutionSource, IPCGGraphExecutionSource* InNewExecutionSource)
{
	const FObjectKey OldKey = Cast<UObject>(InOldExecutionSource);
	const FObjectKey NewKey = Cast<UObject>(InNewExecutionSource);

	// If this execution source has dependencies, we transfer them.
	if (ExecutionSourcesToDependencyMap.Contains(OldKey))
	{
		TArray<FObjectKey> Temp;
		ExecutionSourcesToDependencyMap.RemoveAndCopyValue(OldKey, Temp);
		ExecutionSourcesToDependencyMap.Emplace(NewKey, std::move(Temp));
	}

	// If this execution source was a dependency to any other execution source, just remove it. New one will register itself when generating if needed.
	for (TPair<FObjectKey, TArray<FObjectKey>>& It : ExecutionSourcesToDependencyMap)
	{
		It.Value.RemoveSwap(OldKey);
	}

	// Unregister tracking and register it so that we don't leak keys that might have changed
	const bool bFoundExecutionSource = UnregisterTracking(InOldExecutionSource);
	if (bFoundExecutionSource)
	{
		RegisterTracking(InNewExecutionSource);
	}
}

bool FPCGTrackingManager::UnregisterTracking(const IPCGGraphExecutionSource* InExecutionSource, const TSet<FPCGSelectionKey>* OptionalKeysToUntrack)
{
	if (!InExecutionSource)
	{
		return false;
	}

	bool bExecutionSourceFound = false;

	TSet<FPCGSelectionKey> KeysToRemove;
	auto RemoveAllFromMap = [InExecutionSource, &KeysToRemove, &bExecutionSourceFound](TMap<FPCGSelectionKey, TSet<IPCGGraphExecutionSource*>>& InMap)
	{
		for (auto& It : InMap)
		{
			if (It.Value.Remove(InExecutionSource) > 0)
			{
				bExecutionSourceFound = true;
				if (It.Value.IsEmpty())
				{
					KeysToRemove.Add(It.Key);
				}
			}
		}
	};

	auto RemoveKeysFromMap = [InExecutionSource, &KeysToRemove, &bExecutionSourceFound](TMap<FPCGSelectionKey, TSet<IPCGGraphExecutionSource*>>& InMap, const TSet<FPCGSelectionKey>& SetToIterateOn)
	{
		for (const FPCGSelectionKey& KeyIt : SetToIterateOn)
		{
			if (TSet<IPCGGraphExecutionSource*>* ExecutionSourceSetPtr = InMap.Find(KeyIt))
			{
				if (ExecutionSourceSetPtr->Remove(InExecutionSource) > 0)
				{
					bExecutionSourceFound = true;
					if (ExecutionSourceSetPtr->IsEmpty())
					{
						KeysToRemove.Add(KeyIt);
					}
				}
			}
		}
	};

	if (OptionalKeysToUntrack)
	{
		RemoveKeysFromMap(CulledTrackedKeysToExecutionSourcesMap, *OptionalKeysToUntrack);
		RemoveKeysFromMap(AlwaysTrackedKeysToExecutionSourcesMap, *OptionalKeysToUntrack);
	}
	else
	{
		RemoveAllFromMap(CulledTrackedKeysToExecutionSourcesMap);
		RemoveAllFromMap(AlwaysTrackedKeysToExecutionSourcesMap);
	}

	for (const FPCGSelectionKey& Key : KeysToRemove)
	{
		CulledTrackedKeysToExecutionSourcesMap.Remove(Key);
		AlwaysTrackedKeysToExecutionSourcesMap.Remove(Key);
	}

	return bExecutionSourceFound;
}

bool FPCGTrackingManager::UnregisterTracking(const IPCGGraphExecutionSource* InExecutionSource)
{
	if (!InExecutionSource)
	{
		return false;
	}

	return UnregisterTracking(InExecutionSource, nullptr);
}

bool FPCGTrackingManager::IsKeyTracked(const FPCGSelectionKey& InKey) const
{
	return CulledTrackedKeysToExecutionSourcesMap.Contains(InKey) || AlwaysTrackedKeysToExecutionSourcesMap.Contains(InKey);
}

void FPCGTrackingManager::ForEachTrackedKey(TFunctionRef<bool(const FPCGSelectionKey&, const TSet<IPCGGraphExecutionSource*>&)> InCallback) const
{
	for (const auto& KeyValuePair : CulledTrackedKeysToExecutionSourcesMap)
	{
		if (!InCallback(KeyValuePair.Key, KeyValuePair.Value))
		{
			return;
		}
	}

	for (const auto& KeyValuePair : AlwaysTrackedKeysToExecutionSourcesMap)
	{
		if (!InCallback(KeyValuePair.Key, KeyValuePair.Value))
		{
			return;
		}
	}
}

void FPCGTrackingManager::ResetPartitionActorsMap()
{
	PCG::TUniqueScopeLock WriteLock(PartitionActorsMapLock);
	PartitionActorsMap.Empty();
}

bool FPCGTrackingManager::RemoveExecutionSourceDependency(const IPCGGraphExecutionSource* InOriginatingExecutionSource, const IPCGGraphExecutionSource* InDependentExecutionSource)
{
	const UObject* OriginatingObject = Cast<UObject>(InOriginatingExecutionSource);
	const UObject* DependentObject = Cast<UObject>(InDependentExecutionSource);
	if (OriginatingObject && DependentObject)
	{
		if (TArray<FObjectKey>* Dependencies = ExecutionSourcesToDependencyMap.Find(DependentObject))
		{
			Dependencies->RemoveSwap(OriginatingObject);
			if (Dependencies->IsEmpty())
			{
				ExecutionSourcesToDependencyMap.Remove(DependentObject);
				return true;
			}
			else
			{
				return false;
			}
		}
	}

	return true;
}

bool FPCGTrackingManager::IsTracking() const
{
	if (IsRunningCommandlet())
	{
		return false;
	}

	return !CulledTrackedKeysToExecutionSourcesMap.IsEmpty() || !AlwaysTrackedKeysToExecutionSourcesMap.IsEmpty();
}

void FPCGTrackingManager::RegisterDelegates()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FPCGTrackingManager::OnObjectPropertyChanged);
	FCoreUObjectDelegates::OnPackageReloaded.AddRaw(this, &FPCGTrackingManager::OnPackageReloaded);

	UWorld* World = GetWorld();
	check(World);

	// @todo_pcg: move this out when partitioning gets abstracted
	if (UWorldPartition* WorldPartition = World->GetWorldPartition())
	{
		WorldPartition->OnActorDescInstanceAddedEvent.AddRaw(this, &FPCGTrackingManager::OnActorDescInstanceAdded);
		WorldPartition->OnActorDescInstanceRemovedEvent.AddRaw(this, &FPCGTrackingManager::OnActorDescInstanceRemoved);
	}
}

void FPCGTrackingManager::UnregisterDelegates()
{
	GEngine->OnLevelActorAdded().RemoveAll(this);
	GEngine->OnLevelActorDeleted().RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectModified.RemoveAll(this);
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);

	UWorld* World = GetWorld();
	check(World);

	if (UWorldPartition* WorldPartition = World->GetWorldPartition())
	{
		WorldPartition->OnActorDescInstanceAddedEvent.RemoveAll(this);
		WorldPartition->OnActorDescInstanceRemovedEvent.RemoveAll(this);
	}
}

void FPCGTrackingManager::OnPackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	if (InPackageReloadPhase == EPackageReloadPhase::PrePackageFixup)
	{
		for (const auto& RepointedObjectPair : InPackageReloadedEvent->GetRepointedObjects())
		{
			if (RepointedObjectPair.Key && RepointedObjectPair.Key->IsAsset() && RepointedObjectPair.Key->IsA<UPCGGraphInterface>())
			{
				ForEachObjectOfClass(UPCGGraphInstance::StaticClass(), [OldGraphInterface = RepointedObjectPair.Key](UObject* InObj)
				{
					if (UPCGGraphInstance* GraphInstance = Cast<UPCGGraphInstance>(InObj); GraphInstance && GraphInstance->Graph == OldGraphInterface)
					{
						GraphInstance->TeardownCallbacks();
					}
				});
			}
		}
	}
	else if (InPackageReloadPhase == EPackageReloadPhase::PostPackageFixup)
	{
		TSet<UPCGGraphInstance*> GraphInstancesToRefresh;

		for (const auto& RepointedObjectPair : InPackageReloadedEvent->GetRepointedObjects())
		{
			if (RepointedObjectPair.Value && RepointedObjectPair.Value->IsAsset() && RepointedObjectPair.Value->IsA<UPCGGraphInterface>())
			{
				ForEachObjectOfClass(UPCGGraphInstance::StaticClass(), [&GraphInstancesToRefresh, NewGraphInterface = RepointedObjectPair.Value](UObject* InObj)
				{
					if (UPCGGraphInstance* GraphInstance = Cast<UPCGGraphInstance>(InObj); GraphInstance && GraphInstance->Graph == NewGraphInterface)
					{
						GraphInstancesToRefresh.Add(GraphInstance);
					}
				});
			}
		}

		for (UPCGGraphInstance* GraphInstance : GraphInstancesToRefresh)
		{
			if (GraphInstance)
			{
				GraphInstance->SetupCallbacks();
				GraphInstance->OnGraphParametersChanged(EPCGGraphParameterEvent::GraphChanged, NAME_None);
			}
		}
	}
}

void FPCGTrackingManager::OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (InActorDescInstance->GetActorNativeClass()->IsChildOf(APCGPartitionActor::StaticClass()))
	{
		const FPCGPartitionActorDesc* ActorDesc = static_cast<const FPCGPartitionActorDesc*>(InActorDescInstance->GetActorDesc());
		FPCGGridCellDescriptor PartitionActorRecord = { ActorDesc->GetGridDescriptor(InActorDescInstance), FIntVector3(ActorDesc->GridIndexX, ActorDesc->GridIndexY, ActorDesc->GridIndexZ)};
		if (!PartitionActorRecords.Contains(PartitionActorRecord))
		{
			PartitionActorRecords.Add(MoveTemp(PartitionActorRecord), ActorDesc->GetGuid());
		}
	}
}

void FPCGTrackingManager::OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (InActorDescInstance->GetActorNativeClass()->IsChildOf(APCGPartitionActor::StaticClass()))
	{
		const FPCGPartitionActorDesc* ActorDesc = static_cast<const FPCGPartitionActorDesc*>(InActorDescInstance->GetActorDesc());
		FPCGGridCellDescriptor PartitionActorRecord = { ActorDesc->GetGridDescriptor(InActorDescInstance), FIntVector3(ActorDesc->GridIndexX, ActorDesc->GridIndexY, ActorDesc->GridIndexZ)};
		if (FGuid* FoundGuid = PartitionActorRecords.Find(PartitionActorRecord); FoundGuid && *FoundGuid == InActorDescInstance->GetGuid())
		{
			PartitionActorRecords.Remove(PartitionActorRecord);
		}
	}
}

void FPCGTrackingManager::BuildPartitionActorRecords(APCGWorldActor* PCGWorldActor, UWorldPartition* WorldPartition, TMap<FPCGGridCellDescriptor, FGuid>& OutPartitionActorRecords, TSet<FGuid>& OutInvalidPartitionActors)
{
	check(WorldPartition);
	
	FWorldPartitionHelpers::ForEachActorDescInstance<APCGPartitionActor>(WorldPartition, [PCGWorldActor, &OutPartitionActorRecords, &OutInvalidPartitionActors](const FWorldPartitionActorDescInstance* InActorDescInstance)
	{
		const FPCGPartitionActorDesc* ActorDesc = static_cast<const FPCGPartitionActorDesc*>(InActorDescInstance->GetActorDesc());
		
		// Fixup data
		if (ActorDesc->bRequiresUse2DGridFixup && PCGWorldActor)
		{
			FPCGPartitionActorDesc* NonConstActorDesc = const_cast<FPCGPartitionActorDesc*>(ActorDesc);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			NonConstActorDesc->bUse2DGrid = PCGWorldActor->bUse2DGrid;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			NonConstActorDesc->bRequiresUse2DGridFixup = false;
		}

		// Add to invalid list if we couldn't properly fixup or if it was already marked invalid
		if (ActorDesc->bInvalid)
		{
			OutInvalidPartitionActors.Add(ActorDesc->GetGuid());
		}
		else // Add to existing valid PCG Partition actors
		{
			FPCGGridCellDescriptor PartitionActorRecord = { ActorDesc->GetGridDescriptor(InActorDescInstance), FIntVector3(ActorDesc->GridIndexX, ActorDesc->GridIndexY, ActorDesc->GridIndexZ)};
			if (!OutPartitionActorRecords.Contains(PartitionActorRecord))
			{
				OutPartitionActorRecords.Add(MoveTemp(PartitionActorRecord), ActorDesc->GetGuid());
			}
		}
		return true;
	});
}

void FPCGTrackingManager::BuildPartitionActorRecords()
{
	UWorld* World = GetWorld();
	if (UWorldPartition* WorldPartition = World->GetWorldPartition())
	{
		APCGWorldActor* PCGWorldActor = PCGSubsystem->FindPCGWorldActor();

		InvalidPartitionActors.Empty();
		PartitionActorRecords.Empty();
		BuildPartitionActorRecords(PCGWorldActor, WorldPartition, PartitionActorRecords, InvalidPartitionActors);
	}
}

void FPCGTrackingManager::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InEvent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTrackingManager::OnObjectPropertyChanged);

	// Nothing to do if we track nothing
	if (!IsTracking())
	{
		return;
	}

	// Check if the event is handled by one of our trackers
	for (TUniquePtr<IPCGChangeTracker>& ChangeTracker : ChangeTrackers)
	{
		if (ChangeTracker->OnObjectPropertyChanged(InObject, InEvent))
		{
			return;
		}
	}

	// Another special exception for texture/mesh compilation. If the InEvent is empty and the object is a texture/mesh, we ignore it.
	const bool bEventIsEmpty = (InEvent.Property == nullptr) && (InEvent.ChangeType == EPropertyChangeType::Unspecified);
	const bool bIsTextureCompilationResult = bEventIsEmpty && InObject && InObject->IsA<UTexture>()
		&& FTextureCompilingManager::Get().IsCompilingTexture(Cast<UTexture>(InObject));
	// There is no equivalent for StaticMesh to know if we are in PostCompilation, so we assume there are still some meshes to compile (including this one).
	// Might be an over-optimistic approach, might need a revisit.
	const bool bIsStaticMeshCompilationResult = bEventIsEmpty && InObject && InObject->IsA<UStaticMesh>()
		&& FStaticMeshCompilingManager::Get().GetNumRemainingMeshes() > 0;

	if (bIsTextureCompilationResult || bIsStaticMeshCompilationResult)
	{
		return;
	}

	OnObjectChanged(InObject, PCGTrackingManager::ObjectChanged, /*InOriginatingChangeObject=*/ InObject);
}

void FPCGTrackingManager::OnObjectChanged(const UObject* InObject, const FPCGTrackingChangeID& InChangeID, const UObject* InOriginatingChangeObject)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTrackingManager::OnObjectChanged);

	// Nothing to do if we track nothing or there is no object.
	if (!InObject || !IsTracking())
	{
		return;
	}

	// @todo_pcg: excluded classes should be done through handler/tracker api
	// Don't react to what the PCG Component is already reacting to.
	static const TArray<const UClass*> ExcludedClasses =
	{
		UPCGComponent::StaticClass()
	};

	if (Algo::AnyOf(ExcludedClasses, [InObject](const UClass* Class) -> bool { return InObject->IsA(Class); }))
	{
		return;
	}

	const bool bNoRefreshOwner = InChangeID == PCGTrackingManager::ExecutionSourceGenerated;

	FPCGSelectionKey SelectionKey = FPCGSelectionKey::CreateFromObjectPtr(InObject);
	OnSelectionKeyChanged(SelectionKey, InOriginatingChangeObject, {}, bNoRefreshOwner, {FBox(EForceInit::ForceInit)});
}

void FPCGTrackingManager::NotifyObjectChanged(const UObject* InObject, const FPCGTrackingChangeID& InChangeID, const UObject* InOriginatingChangeObject)
{
	for (TUniquePtr<IPCGChangeTracker>& ChangeTracker : ChangeTrackers)
	{
		if (ChangeTracker->OnNotifyObjectChanged(InObject, InChangeID, InOriginatingChangeObject))
		{
			return;
		}
	}

	OnObjectChanged(InObject, InChangeID, InOriginatingChangeObject);
}

void FPCGTrackingManager::OnSelectionKeyChanged(const FPCGSelectionKey& InSelectionKey)
{
	const UObject* OriginatingObject = InSelectionKey.GetObjectFromPath();
	check(OriginatingObject);

	OnSelectionKeyChanged(InSelectionKey, OriginatingObject, {}, /*bInNoRefreshOwner=*/false, { FBox(EForceInit::ForceInit) });
}

void FPCGTrackingManager::OnSelectionKeyChanged(const FPCGSelectionKey& InSelectionKey, const UObject* InOriginatingChangeObject, TArrayView<const FBox> InBounds)
{
	const UObject* ChangedObject = InSelectionKey.GetObjectFromPath();
	ensure(ChangedObject);

	OnSelectionKeyChanged(InSelectionKey, InOriginatingChangeObject, {}, /*bInNoRefreshOwner=*/false, InBounds);
}

void FPCGTrackingManager::OnSelectionKeyChanged(const FPCGSelectionKey& InSelectionKey, const UObject* InOriginatingChangeObject, const TSet<FName>& InRemovedTags, bool bInNoRefreshOwner, TArrayView<const FBox> InBounds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTrackingManager::OnSelectionKeyChanged);

	/** First gather all the execution sources that are tracking this key */
	TSet<IPCGGraphExecutionSource*> CulledTrackedExecutionSources;
	TSet<IPCGGraphExecutionSource*> AlwaysTrackedExecutionSources;

	FPCGSelectionKey InOriginatingSelectionKey = FPCGSelectionKey::CreateFromObjectPtr(InOriginatingChangeObject);
	InOriginatingSelectionKey.OptionalBounds = InBounds;
		
	TSharedRef<IPCGChangeHandler::FPCGChangeHandlerChange> Change = MakeShared<IPCGChangeHandler::FPCGChangeHandlerChange>();
	Change->ChangedObject = InSelectionKey.GetObjectFromPath();
	Change->OriginatingChangeObject = InOriginatingChangeObject;
	Change->bNoRefreshOwner = bInNoRefreshOwner;
	
	auto Gather = [InSelectionKey, InOriginatingSelectionKey, &InRemovedTags, &Change](const TMap<FPCGSelectionKey, TSet<IPCGGraphExecutionSource*>>& InMap, TSet<IPCGGraphExecutionSource*>& OutSet)
	{
		for (const auto& [SelectionKey, ExecutionSource] : InMap)
		{
			if (SelectionKey.IsMatching(InSelectionKey, InRemovedTags, ExecutionSource, &OutSet) || SelectionKey.IsMatching(InOriginatingSelectionKey, InRemovedTags, ExecutionSource, &OutSet))
			{
				Change->MatchedKeys.Add(SelectionKey);
			}
		}
	};

	Gather(CulledTrackedKeysToExecutionSourcesMap, CulledTrackedExecutionSources);
	Gather(AlwaysTrackedKeysToExecutionSourcesMap, AlwaysTrackedExecutionSources);

	// If this actor is not tracked, just early out
	if (CulledTrackedExecutionSources.IsEmpty() && AlwaysTrackedExecutionSources.IsEmpty())
	{
		return;
	}

	// Prepare Change handlers
	for (TUniquePtr<IPCGChangeHandler>& Handler : ChangeHandlers)
	{
		// Give handler chance to store some information to be used to handle the current change
		Handler->BeginChangeHandling(Change);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTrackingManager::OnObjectChanged::AlwaysTrackedUpdate);

		for (IPCGGraphExecutionSource* ExecutionSource : AlwaysTrackedExecutionSources)
		{
			for (TUniquePtr<IPCGChangeHandler>& Handler : ChangeHandlers)
			{
				Handler->HandleChange(ExecutionSource);
			}
		}
	}

	if (!CulledTrackedExecutionSources.IsEmpty())
	{
		// Then do an octree find to get all components that intersect with this actor.
		// If the actor has moved, we also need to find components that intersected with it before
		// We first do it for non-partitioned, then we do it for partitioned
		auto UpdateNonPartitioned = [this, CulledTrackedExecutionSources, InOriginatingChangeObject](const FPCGExecutionSourceRef& ExecutionSourceRef, const FBox& InBounds) -> void
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTrackingManager::OnObjectChanged::UpdateNonPartitioned);

			if (!CulledTrackedExecutionSources.Contains(ExecutionSourceRef.ExecutionSource))
			{
				return;
			}

			for (TUniquePtr<IPCGChangeHandler>& Handler : ChangeHandlers)
			{
				Handler->HandleBoundedChange(ExecutionSourceRef.ExecutionSource, ExecutionSourceRef.Bounds.GetBox(), InBounds);
			}
		};

		// For partitioned, we first need check if the original component intersect with the bounds, then forward the dirty call only to locals that intersect with the bounds.
		// Note: CurrentActorBoundsPtr is passed by reference because it will be modified between lambda calls (cf comment above).
		auto UpdatePartitioned = [this, CulledTrackedExecutionSources, InOriginatingChangeObject](const FPCGExecutionSourceRef& ExecutionSourceRef, const FBox& InBounds)  -> void
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTrackingManager::OnObjectChanged::UpdatePartitioned);

			if (!CulledTrackedExecutionSources.Contains(ExecutionSourceRef.ExecutionSource))
			{
				return;
			}

			for (TUniquePtr<IPCGChangeHandler>& Handler : ChangeHandlers)
			{
				Handler->HandleBoundedChange(ExecutionSourceRef.ExecutionSource, ExecutionSourceRef.Bounds.GetBox(), InBounds);
			}
		};

		// Gather for all bounds
		for (const FBox& Bounds : InBounds)
		{
			NonPartitionedOctree.FindElementsWithBoundsTest(Bounds, [&Bounds, &UpdateNonPartitioned](const FPCGExecutionSourceRef& ExecutionSourceRef) { UpdateNonPartitioned(ExecutionSourceRef, Bounds); });
			PartitionedOctree.FindElementsWithBoundsTest(Bounds, [&Bounds, &UpdatePartitioned](const FPCGExecutionSourceRef& ExecutionSourceRef) { UpdatePartitioned(ExecutionSourceRef, Bounds); });
		}
	}

	// Allow trackers to skip refresh
	bool bSkipRefresh = false;
	for (TUniquePtr<IPCGChangeTracker>& ChangeTracker : ChangeTrackers)
	{
		if (ChangeTracker->ShouldSkipRefresh(Change->ChangedObject))
		{
			bSkipRefresh = true;
			break;
		}
	}

	for (TUniquePtr<IPCGChangeHandler>& Handler : ChangeHandlers)
	{
		Handler->EndChangeHandling(bSkipRefresh);
	}
}

void FPCGTrackingManager::OnPCGGraphStartsGenerating(IPCGGraphExecutionSource* InExecutionSource)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGTrackingManager::OnPCGGraphStartsGenerating);

	if (!PCGTrackingManager::IsValidExecutionSource(InExecutionSource) || PCGTrackingManager::CVarDisablePCGDataInterdependencyOptimization.GetValueOnAnyThread())
	{
		return;
	}

	// @todo_pcg: support execution source
	UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource);
	if (!PCGComponent)
	{
		return;
	}

	// When a graph starts generating, look for component that depends on it and keep a count.
	// When this graph will be done generating, we'll trigger a dependency generation. But if multiple graphs are generated at the same time and 
	// they all contribute to the same dependency, we'll trigger the dependency only when all the graphs are done generating
	// We only need to gather components that depends on PCG Data, because other dependency changes will be caught by other engine callbacks (such as OnObjectPropertyChanged)
	TSet<IPCGGraphExecutionSource*> TrackedExecutionSources;
	TSet<FName> RemovedTags;

	const FBox ExecutionSourceBounds = InExecutionSource->GetExecutionState().GetBounds();

	auto IsKeyTrackingPCGData = [](const FPCGSelectionKey& InKey) { return InKey.OptionalExtraDependency && InKey.OptionalExtraDependency->IsChildOf(UPCGComponent::StaticClass()); };

	AActor* Actor = PCGComponent->GetOwner();
	const FPCGSelectionKey ActorKey = FPCGSelectionKey::CreateFromObjectPtr(Actor);

	for (TPair<FPCGSelectionKey, TSet<IPCGGraphExecutionSource*>>& It : CulledTrackedKeysToExecutionSourcesMap)
	{
		if (!IsKeyTrackingPCGData(It.Key))
		{
			continue;
		}

		TSet<IPCGGraphExecutionSource*> TempTrackedExecutionSources;
		It.Key.IsMatching(ActorKey, RemovedTags, It.Value, &TempTrackedExecutionSources);

		// Removing all components that aren't intersecting with it, since it won't contribute to refresh
		for (IPCGGraphExecutionSource* TrackedExecutionSource : TempTrackedExecutionSources)
		{
			if (UPCGComponent* TrackedComponent = Cast<UPCGComponent>(TrackedExecutionSource))
			{
				if (ensure(TrackedExecutionSource) && TrackedComponent->GetOwner() && PCGHelpers::GetActorBounds(TrackedComponent->GetOwner(), /*bIgnorePCGCreatedComponents=*/false).Intersect(ExecutionSourceBounds))
				{
					TrackedExecutionSources.Add(TrackedExecutionSource);
				}
			}
		}
	}

	for (TPair<FPCGSelectionKey, TSet<IPCGGraphExecutionSource*>>& It : AlwaysTrackedKeysToExecutionSourcesMap)
	{
		if (!IsKeyTrackingPCGData(It.Key))
		{
			continue;
		}

		It.Key.IsMatching(ActorKey, RemovedTags, It.Value, &TrackedExecutionSources);
	}

	if (UObject* InExecutionSourceObject = Cast<UObject>(InExecutionSource); ensure(InExecutionSource))
	{
		for (IPCGGraphExecutionSource* TrackedExecutionSource : TrackedExecutionSources)
		{
			// Don't have a dependency on itself.
			if (!TrackedExecutionSource || TrackedExecutionSource == InExecutionSource)
			{
				continue;
			}

			UPCGComponent* TrackedComponent = Cast<UPCGComponent>(TrackedExecutionSource);
			// If the tracked component is currently ignoring the refresh from InComponent, don't add to the dependencies
			if (TrackedComponent && TrackedComponent->IsIgnoringChangeOrigin(Actor))
			{
				continue;
			}

			if (UObject* ExecutionSourceObject = Cast<UObject>(TrackedExecutionSource); ensure(ExecutionSourceObject))
			{
				TArray<FObjectKey>& CurrentDependencies = ExecutionSourcesToDependencyMap.FindOrAdd(ExecutionSourceObject);
				CurrentDependencies.AddUnique(InExecutionSourceObject);
			}
		}
	}
}

void FPCGTrackingManager::OnPCGGraphCancelled(IPCGGraphExecutionSource* InExecutionSource)
{
	if (!InExecutionSource)
	{
		return;
	}

	if (UObject* InExecutionSourceObject = Cast<UObject>(InExecutionSource); ensure(InExecutionSourceObject))
	{
		TArray<FObjectKey> Keys;
		ExecutionSourcesToDependencyMap.GetKeys(Keys);
		for (const FObjectKey& Key : Keys)
		{
			TArray<FObjectKey>& Value = ExecutionSourcesToDependencyMap[Key];
			Value.Remove(InExecutionSourceObject);
			if (Value.IsEmpty())
			{
				ExecutionSourcesToDependencyMap.Remove(Key);
			}
		}
	}
}

void FPCGTrackingManager::OnPCGGraphGeneratedOrCleaned(IPCGGraphExecutionSource* InExecutionSource, TOptional<FBox> OptionalExecutionBounds)
{
	if (!PCGTrackingManager::IsValidExecutionSource(InExecutionSource))
	{
		return;
	}

	const UObject* ChangedObject = Cast<UObject>(InExecutionSource);
	
	// @todo_pcg: add concept of Owner to the execution source? to abstract out the actor code here.
	const UObject* ChangedRoot = ChangedObject;
	if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource))
	{
		ChangedRoot = PCGComponent->GetOwner();
	}

	FPCGSelectionKey SelectionKey = FPCGSelectionKey::CreateFromObjectPtr(ChangedRoot);

	const FBox Bounds = OptionalExecutionBounds ? *OptionalExecutionBounds : InExecutionSource->GetExecutionState().GetTotalBounds();
	OnSelectionKeyChanged(SelectionKey, ChangedObject, /*InRemoveTags=*/{}, /*bNoRefreshOwner=*/true, { Bounds });
}

bool FPCGTrackingManager::ClearCacheForKeys(TArrayView<FPCGSelectionKey> InKeys, const IPCGGraphExecutionSource* InExecutionSource, const bool bIntersect, const UObject* InOriginatingChange) const
{
	check(InExecutionSource);
	
	bool bShouldDirty = false;

	auto ClearCache = [this, InOriginatingChange, bIntersect, &bShouldDirty](const FPCGSelectionKey& Key, const FPCGSettingsAndCulling& SettingsAndCulling)
	{
		if (!SettingsAndCulling.Key.IsValid() || (SettingsAndCulling.Value && !bIntersect))
		{
			return;
		}

		// todo_pcg: validate if this is needed for all execution source types
		// Extra care if the change originates from a PCGComponent. Only dirty if we are tracking a PCG component.
		if (InOriginatingChange && InOriginatingChange->IsA<UPCGComponent>()
			&& (!Key.OptionalExtraDependency || !Key.OptionalExtraDependency->IsChildOf(UPCGComponent::StaticClass())))
		{
			return;
		}

		bShouldDirty = true;
		const UPCGSettings* Settings = SettingsAndCulling.Key.Get();
		PCGSubsystem->CleanFromCache(Settings->GetElement().Get(), Settings);
	};

	for (const FPCGSelectionKey& Key : InKeys)
	{
		InExecutionSource->GetExecutionState().ForEachSettingTrackingKey(Key, ClearCache);
	}

	return bShouldDirty;
}

#endif // WITH_EDITOR
