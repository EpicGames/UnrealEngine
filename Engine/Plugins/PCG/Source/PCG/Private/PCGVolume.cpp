// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGVolume.h"
#include "PCGComponent.h"
#include "PCGGraph.h"
#include "Helpers/PCGHelpers.h"

#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGVolume)

APCGVolume::APCGVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PCGComponent = ObjectInitializer.CreateDefaultSubobject<UPCGComponent>(this, TEXT("PCG Component"));

	if (PCGHelpers::IsNewObjectAndNotDefault(this, /*bCheckHierarchy=*/true))
	{
		if (UBrushComponent* MyBrushComponent = GetBrushComponent())
		{
			MyBrushComponent->SetCollisionObjectType(ECC_WorldStatic);
			MyBrushComponent->SetCollisionResponseToAllChannels(ECR_Ignore);
			MyBrushComponent->SetGenerateOverlapEvents(false);
		}
	}
}

#if WITH_EDITOR
bool APCGVolume::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (PCGComponent)
    {
    	if (UPCGGraph* PCGGraph = PCGComponent->GetGraph())
    	{
    		Objects.Add(PCGGraph);
    	}
    }
    return true;
}
#endif // WITH_EDITOR
