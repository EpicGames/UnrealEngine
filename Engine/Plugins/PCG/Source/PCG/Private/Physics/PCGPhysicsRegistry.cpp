// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/PCGPhysicsRegistry.h"

namespace PCGPhysicsRegistry
{
	template<typename DelegateType, typename ParamType>
	bool CallFunc(const TMap<FGuid, DelegateType>& InFuncs, const ParamType& InParams, bool bDefaultReturnValue)
	{
		for (const auto& [FuncID, Func] : InFuncs)
		{
			bool bReturnValue = bDefaultReturnValue;
			if (Func.IsBound() && Func.Execute(InParams, bReturnValue))
			{
				return bReturnValue;
			}
		}

		return bDefaultReturnValue;
	}
}

void FPCGPhysicsRegistry::RegisterHitResultFilter(const FGuid& InID, FPCGFilterHitResult InFilter)
{
	check(!HitResultFilters.Contains(InID));
	HitResultFilters.Add(InID, MoveTemp(InFilter));
}

void FPCGPhysicsRegistry::UnregisterHitResultFilter(const FGuid& InID)
{
	HitResultFilters.Remove(InID);
}

void FPCGPhysicsRegistry::RegisterOverlapResultFilter(const FGuid& InID, FPCGFilterOverlapResult InFilter)
{
	check(!OverlapResultFilters.Contains(InID));
	OverlapResultFilters.Add(InID, MoveTemp(InFilter));
}

void FPCGPhysicsRegistry::UnregisterOverlapResultFilter(const FGuid& InID)
{
	OverlapResultFilters.Remove(InID);
}

void FPCGPhysicsRegistry::RegisterApplyHitResultAttributes(const FGuid& InID, FPCGApplyHitResultAttributes InApplyHitResultAttributes)
{
	check(!ApplyHitResultAttributesFuncs.Contains(InID));
	ApplyHitResultAttributesFuncs.Add(InID, MoveTemp(InApplyHitResultAttributes));
}

void FPCGPhysicsRegistry::UnregisterApplyHitResultAttributes(const FGuid& InID)
{
	ApplyHitResultAttributesFuncs.Remove(InID);
}

bool FPCGPhysicsRegistry::FilterOverlapResult(const FPCGFilterOverlapResultParams& InParams) const
{
	return PCGPhysicsRegistry::CallFunc(OverlapResultFilters, InParams, /*bDefaultReturnValue=*/false);
}

bool FPCGPhysicsRegistry::FilterHitResult(const FPCGFilterHitResultParams& InParams) const
{
	return PCGPhysicsRegistry::CallFunc(HitResultFilters, InParams, /*bDefaultReturnValue=*/false);
}

bool FPCGPhysicsRegistry::ApplyHitResultAttributes(const FPCGApplyHitResultAttributesParams& InParams) const
{
	return PCGPhysicsRegistry::CallFunc(ApplyHitResultAttributesFuncs, InParams, /*bDefaultReturnValue*/true);
}
