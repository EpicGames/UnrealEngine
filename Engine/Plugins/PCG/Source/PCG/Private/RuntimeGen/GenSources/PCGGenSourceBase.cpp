// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/GenSources/PCGGenSourceBase.h"

#include "RuntimeGen/PCGRuntimeGenContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGenSourceBase)

static bool AreVectorsEquivalent(const TOptional<FVector>& InA, const TOptional<FVector>& InB)
{
	if (InA.IsSet() != InB.IsSet())
	{
		return false;
	}

	// Both unset, or both values match (approximately).
	return !InA.IsSet() || InA.GetValue().Equals(InB.GetValue());
}

static bool AreConvexVolumesEquivalent(const TOptional<FConvexVolume>& InA, const TOptional<FConvexVolume>& InB)
{
	if (InA.IsSet() != InB.IsSet())
	{
		return false;
	}

	if (!InA.IsSet())
	{
		// Both unset, so equal.
		return true;
	}
	
	if (InA->Planes.Num() != InB->Planes.Num())
	{
		return false;
	}

	// Verify all planes approximately equal.
	// Note: This is a weak comparison and will fail if order of planes do not match. Could be extended if we identify a beneficial case.
	for (int Index = 0; Index < InA->Planes.Num(); ++Index)
	{
		if (!InA->Planes[Index].Equals(InB->Planes[Index]))
		{
			return false;
		}
	}

	return true;
}

void IPCGGenSourceBase::Tick(const FPCGRuntimeGenContext& InContext)
{
	// Backwards compatibility.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Tick();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool IPCGGenSourceBase::IsEquivalent(IPCGGenSourceBase* InOther, const FPCGRuntimeGenContext& InContext) const
{
	if (this == InOther)
	{
		return true;
	}

	if (!AreVectorsEquivalent(GetPosition(), InOther->GetPosition())
		|| !AreVectorsEquivalent(GetDirection(), InOther->GetDirection()) // Always include for now, used for priority calculation
		|| IsLocal() != InOther->IsLocal())
	{
		return false;
	}

	if (InContext.bAnySourcesUseFrustumCulling)
	{
		if (InContext.bAnySourcesUse2DGrids)
		{
			if (!AreConvexVolumesEquivalent(GetViewFrustum(/*bIs2DGrid=*/true), InOther->GetViewFrustum(/*bIs2DGrid=*/true)))
			{
				return false;
			}
		}

		if (InContext.bAnySourcesUse3DGrids)
		{
			if (!AreConvexVolumesEquivalent(GetViewFrustum(/*bIs2DGrid=*/false), InOther->GetViewFrustum(/*bIs2DGrid=*/false)))
			{
				return false;
			}
		}
	}

	return true;
}
