// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/GenSources/PCGGenSourceComponent.h"

#include "PCGSubsystem.h"
#include "RuntimeGen/PCGGenSourceManager.h"

#include "PrimitiveDrawingUtils.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGenSourceComponent)

#define LOCTEXT_NAMESPACE "UPCGGenSourceComponent"

UPCGGenSourceComponent::~UPCGGenSourceComponent()
{
#if WITH_EDITOR
	// For the special case where a component is part of a reconstruction script (from a BP),
	// but gets destroyed immediately, we need to force the unregistering. 
	if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
	{
		if (FPCGGenSourceManager* GenSourceManager = PCGSubsystem->GetGenSourceManager())
		{
			GenSourceManager->UnregisterGenSource(this);
		}
	}
#endif // WITH_EDITOR
}

void UPCGGenSourceComponent::BeginPlay()
{
	Super::BeginPlay();

#if WITH_EDITOR
	// Disable registration on preview actors.
	if (GetOwner() && GetOwner()->bIsEditorPreviewActor)
	{
		return;
	}
#endif

	if (FPCGGenSourceManager* GenSourceManager = GetGenSourceManager())
	{
		GenSourceManager->RegisterGenSource(this);
	}
}

void UPCGGenSourceComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	// Always try to unregister itself, if it doesn't exist, it will early out. 
	// Just making sure that we don't left some resources registered while dead.
	if (FPCGGenSourceManager* GenSourceManager = GetGenSourceManager())
	{
		GenSourceManager->UnregisterGenSource(this);
	}

	Super::EndPlay(EndPlayReason);
}

void UPCGGenSourceComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITOR
	// Disable registration on preview actors.
	if (GetOwner() && GetOwner()->bIsEditorPreviewActor)
	{
		return;
	}

	if (UWorld* World = GetWorld())
	{
		// We won't be able to spawn any actors if we are currently running a construction script.
		if (!World->bIsRunningConstructionScript)
		{
			if (FPCGGenSourceManager* GenSourceManager = GetGenSourceManager())
			{
				GenSourceManager->RegisterGenSource(this);
			}
		}
	}
#endif //WITH_EDITOR
}

void UPCGGenSourceComponent::OnUnregister()
{
	Super::OnUnregister();

#if WITH_EDITOR
	// Disable registration on preview actors.
	if (GetOwner() && GetOwner()->bIsEditorPreviewActor)
	{
		return;
	}

	if (FPCGGenSourceManager* GenSourceManager = GetGenSourceManager())
	{
		GenSourceManager->UnregisterGenSource(this);
	}
#endif // WITH_EDITOR

}

void UPCGGenSourceComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	if (FPCGGenSourceManager* GenSourceManager = GetGenSourceManager())
	{
		GenSourceManager->RegisterGenSource(this);
	}
}

void UPCGGenSourceComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

	if (FPCGGenSourceManager* GenSourceManager = GetGenSourceManager())
	{
		GenSourceManager->UnregisterGenSource(this);
	}
}

TOptional<FVector> UPCGGenSourceComponent::GetPosition() const
{
	if (AActor* Actor = GetOwner())
	{
		return Actor->GetActorLocation();
	}

	return TOptional<FVector>();
}

TOptional<FVector> UPCGGenSourceComponent::GetDirection() const
{
	if (AActor* Actor = GetOwner())
	{
		return Actor->GetActorForwardVector();
	}

	return TOptional<FVector>();
}

FPCGGenSourceManager* UPCGGenSourceComponent::GetGenSourceManager() const
{
	if (AActor* Actor = GetOwner())
	{
		if (UPCGSubsystem* Subsystem = UPCGSubsystem::GetInstance(Actor->GetWorld()))
		{
			return Subsystem->GetGenSourceManager();
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
