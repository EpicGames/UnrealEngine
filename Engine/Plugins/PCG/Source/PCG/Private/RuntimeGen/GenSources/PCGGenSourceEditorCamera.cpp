// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/GenSources/PCGGenSourceEditorCamera.h"

#include "RuntimeGen/PCGRuntimeGenContext.h"

#include "SceneView.h"

#if WITH_EDITOR
#include "EditorViewportClient.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGenSourceEditorCamera)

void UPCGGenSourceEditorCamera::Tick(const FPCGRuntimeGenContext& InContext)
{
#if WITH_EDITOR
	ViewFrustum = TOptional<FConvexVolume>();

	if (InContext.bAnySourcesUseFrustumCulling && EditorViewportClient)
	{
		const UWorld* World = EditorViewportClient->GetWorld();
		const FViewport* Viewport = EditorViewportClient->Viewport;

		if (World && Viewport)
		{
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
				Viewport,
				World->Scene,
				EditorViewportClient->EngineShowFlags)
				.SetRealtimeUpdate(EditorViewportClient->IsRealtime()));

			if (FSceneView* SceneView = EditorViewportClient->CalcSceneView(&ViewFamily))
			{
				FConvexVolume ConvexVolume;
				GetViewFrustumBounds(ConvexVolume, SceneView->ViewMatrices.GetWorldToClip(), /*bUseNearPlane=*/true, /*bUseFarPlane=*/true);

				ViewFrustum = ConvexVolume;
			}
		}
	}
#endif
}

TOptional<FVector> UPCGGenSourceEditorCamera::GetPosition() const
{
#if WITH_EDITOR
	if (EditorViewportClient)
	{
		return EditorViewportClient->GetViewLocation();
	}
#endif

	return TOptional<FVector>();
}

TOptional<FVector> UPCGGenSourceEditorCamera::GetDirection() const
{
#if WITH_EDITOR
	if (EditorViewportClient)
	{
		return EditorViewportClient->GetViewRotation().Vector();
	}
#endif

	return TOptional<FVector>();
}

TOptional<FConvexVolume> UPCGGenSourceEditorCamera::GetViewFrustum(bool bIs2DGrid) const
{
#if WITH_EDITOR
	return ViewFrustum;
#else
	return TOptional<FConvexVolume>();
#endif	
}
