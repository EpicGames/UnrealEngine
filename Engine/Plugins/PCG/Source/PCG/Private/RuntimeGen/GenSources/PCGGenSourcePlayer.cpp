// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/GenSources/PCGGenSourcePlayer.h"

#include "RuntimeGen/PCGRuntimeGenContext.h"

#include "SceneView.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGenSourcePlayer)

void UPCGGenSourcePlayer::Tick(const FPCGRuntimeGenContext& InContext)
{
	ViewFrustum = TOptional<FConvexVolume>();

	const ULocalPlayer* LocalPlayer = PlayerController.IsValid() ? PlayerController->GetLocalPlayer() : nullptr;

	if (LocalPlayer)
	{
		if (InContext.bAnySourcesUseFrustumCulling)
		{
			if (LocalPlayer->ViewportClient)
			{
				FSceneViewProjectionData ProjectionData;

				if (LocalPlayer->GetProjectionData(LocalPlayer->ViewportClient->Viewport, ProjectionData))
				{
					FConvexVolume ConvexVolume;
					GetViewFrustumBounds(ConvexVolume, ProjectionData.ComputeViewProjectionMatrix(), /*bUseNearPlane=*/true, /*bUseFarPlane=*/true);

					ViewFrustum = ConvexVolume;
				}
			}
		}

		FVector CameraLocation;
		FRotator CameraRotation;
		PlayerController->GetPlayerViewPoint(CameraLocation, CameraRotation);
		CachedPosition = CameraLocation;
		CachedDirection = CameraRotation.Vector();
	}
	else
	{
		CachedPosition.Reset();
		CachedDirection.Reset();
	}
}

TOptional<FVector> UPCGGenSourcePlayer::GetPosition() const
{
	return CachedPosition;
}

TOptional<FVector> UPCGGenSourcePlayer::GetDirection() const
{
	return CachedDirection;
}

void UPCGGenSourcePlayer::SetPlayerController(const APlayerController* InPlayerController)
{
	PlayerController = InPlayerController;
}

bool UPCGGenSourcePlayer::IsLocal() const
{
	if (const APlayerController* PlayerControllerPtr = PlayerController.Get())
	{
		return PlayerControllerPtr->IsLocalController();
	}
	else
	{
		return false;
	}
}