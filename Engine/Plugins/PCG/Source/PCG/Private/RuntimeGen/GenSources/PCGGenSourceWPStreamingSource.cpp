// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/GenSources/PCGGenSourceWPStreamingSource.h"

#include "WorldPartition/WorldPartitionStreamingSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGenSourceWPStreamingSource)

TOptional<FVector> UPCGGenSourceWPStreamingSource::GetPosition() const
{
	return StreamingSource ? StreamingSource->Location : TOptional<FVector>();
}

TOptional<FVector> UPCGGenSourceWPStreamingSource::GetDirection() const
{
	return StreamingSource ? StreamingSource->Rotation.Vector() : TOptional<FVector>();
}
