// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/PCGGenSourceManager.h"

#include "PCGModule.h"
#include "PCGWorldActor.h"
#include "Helpers/PCGHelpers.h"
#include "RuntimeGen/PCGRuntimeGenContext.h"
#include "RuntimeGen/GenSources/PCGGenSourceBase.h"
#include "RuntimeGen/GenSources/PCGGenSourceComponent.h"
#include "RuntimeGen/GenSources/PCGGenSourceEditorCamera.h"
#include "RuntimeGen/GenSources/PCGGenSourcePlayer.h"
#include "RuntimeGen/GenSources/PCGGenSourceWPStreamingSource.h"

#include "Algo/NoneOf.h"
#include "Engine/GameViewportClient.h"
#include "Engine/World.h"
#include "GameFramework/GameModeBase.h"
#include "WorldPartition/WorldPartition.h"

#if WITH_EDITOR
#include "Editor/IPCGEditorModule.h"

#include "EditorViewportClient.h"
#endif

namespace PCGGenSourceManager
{
	enum class EPCGServerMode : int32
	{
		Disabled = 0,
		Local,
		All
	};

	static TAutoConsoleVariable<int32> CVarServerRuntimeGenerationMode(
		TEXT("pcg.RuntimeGeneration.Server.Mode"),
		2,
		TEXT("Enable the RuntimeGeneration system on servers. (0 disabled, 1 local generation sources only (listen server), 2 all generation sources"));
}

FPCGGenSourceManager::FPCGGenSourceManager(const FPCGRuntimeGenContext& InContext)
	: Context(InContext)
{
#if WITH_EDITOR
	EditorCameraGenSource = NewObject<UPCGGenSourceEditorCamera>();
#endif
}

FPCGGenSourceManager::~FPCGGenSourceManager()
{
	FGameModeEvents::GameModePostLoginEvent.RemoveAll(this);
	FGameModeEvents::GameModeLogoutEvent.RemoveAll(this);
}

TSet<IPCGGenSourceBase*> FPCGGenSourceManager::GetAllGenSources(const APCGWorldActor* InPCGWorldActor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGenSourceManager::GetAllGenSources);

	if (bDirty)
	{
		UpdatePerTickGenSources(InPCGWorldActor);
	}

	TSet<IPCGGenSourceBase*> AllGenSources;
	AllGenSources.Reserve(RegisteredGenSources.Num() + RegisteredNamedGenSources.Num());

	for (TScriptInterface<IPCGGenSourceBase> GenSource : RegisteredGenSources)
	{
		if (GenSource)
		{
			if (bDirty)
			{
				GenSource->Tick(Context);
			}

			AllGenSources.Add(GenSource.GetInterface());
		}
	}

	for (const TPair<FName, TScriptInterface<IPCGGenSourceBase>>& NamedGenSource : RegisteredNamedGenSources)
	{
		if (NamedGenSource.Value)
		{
			if (bDirty)
			{
				NamedGenSource.Value->Tick(Context);
			}

			AllGenSources.Add(NamedGenSource.Value.GetInterface());
		}
	}

	// Support deprecated behavior
	GatherWorldPartitionGenSources(AllGenSources);

#if WITH_EDITOR
	// Acquire a generation source for the active editor viewport if one exists.
	if (EditorCameraGenSource && EditorCameraGenSource->EditorViewportClient)
	{
		if (bDirty)
		{
			EditorCameraGenSource->Tick(Context);
		}

		AllGenSources.Add(EditorCameraGenSource);
	}
#endif

	// Flag that all gen sources have been ticked.
	bDirty = false;

	return AllGenSources;
}

void FPCGGenSourceManager::GetUniqueGenSources(const FPCGRuntimeGenContext& InContext, TSet<IPCGGenSourceBase*>& OutUniqueGenSources)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGenSourceManager::GetUniqueGenSources);

	TSet<IPCGGenSourceBase*> AllGenSources = GetAllGenSources(InContext.PCGWorldActor);

	for (IPCGGenSourceBase* GenSource : AllGenSources)
	{
		if (GenSource->GetPosition().IsSet() && Algo::NoneOf(OutUniqueGenSources, [&InContext, GenSource](const IPCGGenSourceBase* InAddedGenSource) { return InAddedGenSource && InAddedGenSource->IsEquivalent(GenSource, InContext); }))
		{
			OutUniqueGenSources.Add(GenSource);
		}
	}
}

bool FPCGGenSourceManager::RegisterGenSource(IPCGGenSourceBase* InGenSource, FName InGenSourceName)
{
	if (UPCGGenSourceComponent* GenSourceComponent = Cast<UPCGGenSourceComponent>(InGenSource))
	{
		if (GenSourceComponent->GetWorld() != Context.World)
		{
			return false;
		}
	}

	TScriptInterface<IPCGGenSourceBase> InterfacePtr(Cast<UObject>(InGenSource));
	if (InGenSourceName == NAME_None)
	{
		return RegisteredGenSources.Add(InterfacePtr).IsValidId();
	}
	else
	{
		if (RegisteredNamedGenSources.Find(InGenSourceName))
		{
			return false;
		}
		else
		{
			RegisteredNamedGenSources.Add(InGenSourceName, InterfacePtr);
			return true;
		}
	}
}

bool FPCGGenSourceManager::UnregisterGenSource(const IPCGGenSourceBase* InGenSource)
{
	if (const UPCGGenSourceComponent* GenSourceComponent = Cast<UPCGGenSourceComponent>(InGenSource))
	{
		if (GenSourceComponent->GetWorld() != Context.World)
		{
			return false;
		}
	}

	// Start by removing from the unnamed gen sources.
	// @todo_pcg We could remove from named sources here too, but that might be limiting for more complex cases, BUT it might be less error-prone too.
	TScriptInterface<IPCGGenSourceBase> InterfacePtr(Cast<UObject>(const_cast<IPCGGenSourceBase*>(InGenSource)));
	if (RegisteredGenSources.Remove(InterfacePtr) > 0)
	{
		return true;
	}
	else // Then try to remove from the named sources
	{
		if (const FName* GenSourceNamePtr = RegisteredNamedGenSources.FindKey(InterfacePtr))
		{
			FName GenSourceName = *GenSourceNamePtr;
			return UnregisterGenSource(GenSourceName);
		}
	}

	return false;
}

bool FPCGGenSourceManager::UnregisterGenSource(FName InGenSourceName)
{
	// Note: unnamed generation sources can't be unregistered by name, by definition.
	return RegisteredNamedGenSources.Remove(InGenSourceName) > 0;
}

void FPCGGenSourceManager::UpdatePerTickGenSources(const APCGWorldActor* InPCGWorldActor)
{
	// Gather Player controller sources
	TSet<APlayerController*> CurrentControllers;

	const bool bIsServer = PCGHelpers::IsListenServer(Context.World) || PCGHelpers::IsDedicatedServer(Context.World);
	const int32 ServerMode = PCGGenSourceManager::CVarServerRuntimeGenerationMode.GetValueOnAnyThread();

	if (InPCGWorldActor->bTreatPlayerControllersAsGenerationSources && (!bIsServer || ServerMode > 0))
	{
		// Update existing sources or add new ones
		for (auto It = Context.World->GetPlayerControllerIterator(); It; ++It)
		{
			APlayerController* PlayerController = It->Get();
			if (!PlayerController)
			{
				continue;
			}

			if (bIsServer && ServerMode == static_cast<std::underlying_type_t<PCGGenSourceManager::EPCGServerMode>>(PCGGenSourceManager::EPCGServerMode::Local) && !PlayerController->IsLocalController())
			{
				continue;
			}

			CurrentControllers.Add(PlayerController);

			if (TScriptInterface<IPCGGenSourceBase>* FoundSource = RegisteredNamedGenSources.Find(PlayerController->GetFName()))
			{
				// Only update if the registered source is a player controller
				if (UPCGGenSourcePlayer* SourcePlayer = Cast<UPCGGenSourcePlayer>(FoundSource->GetInterface()))
				{
					SourcePlayer->SetPlayerController(PlayerController);
				}
			}
			else
			{
				UPCGGenSourcePlayer* GenSource = NewObject<UPCGGenSourcePlayer>();
				GenSource->SetPlayerController(PlayerController);
				RegisterGenSource(GenSource, PlayerController->GetFName());
			}
		}
	}

	// Remove obsolete sources
	for (TMap<FName, TScriptInterface<IPCGGenSourceBase>>::TIterator It = RegisteredNamedGenSources.CreateIterator(); It; ++It)
	{
		const UPCGGenSourcePlayer* SourcePlayer = Cast<UPCGGenSourcePlayer>(It.Value().GetInterface());
		if (SourcePlayer && !CurrentControllers.Contains(SourcePlayer->GetPlayerController().Get()))
		{
			It.RemoveCurrent();
		}
	}

	// Support deprecated behavior
	UpdateWorldPartitionGenSources(InPCGWorldActor);

#if WITH_EDITOR
	EditorCameraGenSource->EditorViewportClient = nullptr;

	// Update the active editor viewport client for the EditorCameraGenSource, only if requested by the world actor, in-editor, and the viewport is visible.
	if (IPCGEditorModule::ShouldTreatViewportAsGenerationSource(InPCGWorldActor) && !Context.World->IsGameWorld() && GEditor)
	{
		if (FViewport* Viewport = GEditor->GetActiveViewport())
		{
			if (FEditorViewportClient* ViewportClient = static_cast<FEditorViewportClient*>(Viewport->GetClient()))
			{
				if (ViewportClient->IsVisible())
				{
					EditorCameraGenSource->EditorViewportClient = ViewportClient;
				}
			}
		}
	}
#endif
}

void FPCGGenSourceManager::GatherWorldPartitionGenSources(TSet<IPCGGenSourceBase*>& OutGenSources) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (TObjectPtr<UPCGGenSourceWPStreamingSource> WPGenSource : WorldPartitionGenSources)
	{
		if (WPGenSource)
		{
			if (bDirty)
			{
				WPGenSource->Tick(Context);
			}

			OutGenSources.Add(WPGenSource);
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FPCGGenSourceManager::UpdateWorldPartitionGenSources(const APCGWorldActor* InPCGWorldActor)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Deprecated : UPCGGenSourceComponent should be used instead
	// Start by updating, adding and removing WP generation sources if it's needed.
	// Note: GetStreamingSources only works in GameWorld, so StreamingSources do not act as generation sources in editor.
	const TArray<FWorldPartitionStreamingSource>* WPStreamingSources = nullptr;

	if (InPCGWorldActor->bEnableWorldPartitionGenerationSources &&
		InPCGWorldActor->GetWorld() &&
		InPCGWorldActor->GetWorld()->GetWorldPartition())
	{
		// Log deprecation warning once
		if (!bLoggedWorldPartitionGenSourceWarning && InPCGWorldActor->GetWorld()->IsGameWorld())
		{
			UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] APCGWorldActor::bEnableWorldPartitionGenerationSources is deprecated and will be removed. You can disable this property to remove the warning. By default, player controllers act as generation sources and you can use UPCGGenSourceComponent to provide custom generation sources.");
			bLoggedWorldPartitionGenSourceWarning = true;
		}

		WPStreamingSources = &InPCGWorldActor->GetWorld()->GetWorldPartition()->GetStreamingSources();
	}

	const int NumWPStreamingSources = WPStreamingSources ? WPStreamingSources->Num() : 0;
	const int NumWPGenerationSources = WorldPartitionGenSources.Num();
	const int NumGenerationSourcesToUpdate = FMath::Min(NumWPStreamingSources, NumWPGenerationSources);
	const int NumGenerationSourcesToRemove = FMath::Max(0, NumWPGenerationSources - NumWPStreamingSources);
	const int NumGenerationSourcesToAdd = FMath::Max(0, NumWPStreamingSources - NumWPGenerationSources);
	check((NumGenerationSourcesToRemove == 0 && NumGenerationSourcesToAdd == 0) || ((NumGenerationSourcesToRemove > 0) != (NumGenerationSourcesToAdd > 0)));

	for (int I = 0; I < NumGenerationSourcesToAdd; ++I)
	{
		UPCGGenSourceWPStreamingSource* GenSource = WorldPartitionGenSources.Add_GetRef(NewObject<UPCGGenSourceWPStreamingSource>());
	}

	check((NumGenerationSourcesToUpdate + NumGenerationSourcesToAdd == 0) || WPStreamingSources);
	// Implementation note: not really needed but we need to appease the static analysis gods
	if (WPStreamingSources)
	{
		for (int I = 0; I < NumGenerationSourcesToUpdate + NumGenerationSourcesToAdd; ++I)
		{
			WorldPartitionGenSources[I]->StreamingSource = &(*WPStreamingSources)[I];
		}
	}

	WorldPartitionGenSources.SetNum(NumGenerationSourcesToUpdate + NumGenerationSourcesToAdd);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FPCGGenSourceManager::AddReferencedObjects(FReferenceCollector& Collector)
{
#if WITH_EDITOR
	Collector.AddReferencedObject(EditorCameraGenSource);
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Collector.AddReferencedObjects(WorldPartitionGenSources);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (TScriptInterface<IPCGGenSourceBase>& GenSource : RegisteredGenSources)
	{
		GenSource.AddReferencedObjects(Collector);
	}

	for (auto& KeyValuePair : RegisteredNamedGenSources)
	{
		KeyValuePair.Value.AddReferencedObjects(Collector);
	}
}

// Deprecated (5.8)
FPCGGenSourceManager::FPCGGenSourceManager(const UWorld* InWorld)
	: Context(FallBackContext)
	, FallBackContext(FPCGRuntimeGenContext{ .World = const_cast<UWorld*>(InWorld), .bAnyRuntimeGenSourcesExist = true, .bAnySourcesUseFrustumCulling = true, .bAnySourcesUse2DGrids = true, .bAnySourcesUse3DGrids = true })
{
}
