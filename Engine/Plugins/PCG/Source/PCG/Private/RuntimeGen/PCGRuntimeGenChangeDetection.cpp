// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/PCGRuntimeGenChangeDetection.h"

#include "RuntimeGen/GenSources/PCGGenSourceBase.h"

namespace PCGRuntimeGenChangeDetection
{
	static TAutoConsoleVariable<float> CVarMaxPositionDeltaGridProportion(
		TEXT("pcg.RuntimeGeneration.ChangeDetection.MaxPositionDeltaGridProportion"),
		0.1f,
		TEXT("For partitioned components that have one or more grid, use this proportion of the smallest grid as the maximum generation source position delta before triggering a rescan of cells."));

	static TAutoConsoleVariable<float> CVarMaxPositionDeltaUnbounded(
		TEXT("pcg.RuntimeGeneration.ChangeDetection.MaxPositionDeltaUnbounded"),
		100.0f,
		TEXT("The maximum generation source position delta for unbounded generation (cm)."));

	void FDetector::FCachedState::UpdateFromInputs(const FDetectionInputs& InInputs)
	{
		GenSourcePosition = InInputs.GetGenSource()->GetPosition();

		if (InInputs.UseGenSourceDirection())
		{
			GenSourceDirection = InInputs.GetGenSource()->GetDirection();
		}
		else
		{
			GenSourceDirection.Reset();
		}

		GenerationRadiusMultiplier = InInputs.GetGenerationRadiusMultiplier();
	}

	bool FDetector::FCachedState::ShouldScanCells(const FDetectionInputs& InInputs) const
	{
		if (TOptional<FVector> NewPosition = InInputs.GetGenSource()->GetPosition())
		{
			if (!GenSourcePosition)
			{
				// Position has just become available, perform scan.
				return true;
			}

			// Rescan if position has changed significantly.
			const float MaxPositionDelta = (InInputs.GetMinGridSize() == PCGHiGenGrid::UnboundedGridSize())
				? CVarMaxPositionDeltaUnbounded.GetValueOnGameThread()
				: (CVarMaxPositionDeltaGridProportion.GetValueOnGameThread() * static_cast<float>(InInputs.GetMinGridSize()));

			if (FVector::DistSquared(*GenSourcePosition, *NewPosition) > MaxPositionDelta * MaxPositionDelta)
			{
				return true;
			}
		}

		if (InInputs.UseGenSourceDirection())
		{
			if (TOptional<FVector> NewDirection = InInputs.GetGenSource()->GetDirection())
			{
				if (!GenSourceDirection)
				{
					// Direction has just become available.
					return true;
				}

				// Rescan if direction has changed significantly.
				// todo_pcg: Possibly only take heading angle if 2D grid.
				const double MinDotProduct = 0.997; // Value tweaked by hand, could be driven by cvar if needed.
				if (FVector::DotProduct(*GenSourceDirection, *NewDirection) < MinDotProduct)
				{
					return true;
				}
			}
			else if (GenSourceDirection)
			{
				// Gen source direction has stopped coming through. Safest to trigger rescan - for e.g. this might disable frustum culling.
				return true;
			}
		}

		return false;
	}

	void FDetector::PreTick()
	{
		// Mark all inputs in the StateCache as pending deletion. They will be cleaned out unless directly queried by the RGS.
		StateCache.GetKeys(InputsPendingDeletion);
	}

	void FDetector::PostTick()
	{
		// Clear the cache state for any inputs that were not refreshed during this tick.
		for (const FDetectionInputs& DetectionInput : InputsPendingDeletion)
		{
			StateCache.Remove(DetectionInput);
		}

		InputsPendingDeletion.Reset();
	}

	bool FDetector::IsCellScanRequired(const FDetectionInputs& InInputs)
	{
		// These inputs are active, therefore they are no longer pending deletion at the end of this tick.
		InputsPendingDeletion.Remove(InInputs);
		
		// If no cached state was present it will be added here and ShouldScanCells will return true in this case.
		return StateCache.FindOrAdd(InInputs).ShouldScanCells(InInputs);
	}

	void FDetector::OnCellsScanned(const FDetectionInputs& InInputs)
	{
		// Add or update the cached state.
		StateCache.FindOrAdd(InInputs).UpdateFromInputs(InInputs);
	}

	void FDetector::FlushCachedState(const IPCGGraphExecutionSource* InOptionalOriginalExecutionSource)
	{
		if (InOptionalOriginalExecutionSource)
		{
			TArray<FDetectionInputs, TInlineAllocator<32>> KeysToRemove;

			for (const TPair<FDetectionInputs, FCachedState>& Entry : StateCache)
			{
				const IPCGGraphExecutionSource* ExecutionSource = Entry.Key.GetOriginalExecutionSource();
				if (!ExecutionSource || ExecutionSource == InOptionalOriginalExecutionSource)
				{
					KeysToRemove.Add(Entry.Key);
				}
			}

			for (const FDetectionInputs& Key : KeysToRemove)
			{
				StateCache.Remove(Key);
			}
		}
		else
		{
			StateCache.Empty();
		}
	}
}
