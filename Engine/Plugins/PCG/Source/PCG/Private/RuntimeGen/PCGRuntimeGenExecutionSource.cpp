// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/PCGRuntimeGenExecutionSource.h"

#include "PCGContext.h"
#include "PCGData.h"
#include "PCGGraph.h"
#include "PCGManagedResourceContainer.h"
#include "PCGSubsystem.h"
#include "Data/PCGVolumeData.h"
#include "Graph/PCGGraphPerExecutionCache.h"
#include "Utils/PCGGeneratedResourcesLogging.h"
#include "Utils/PCGGraphExecutionLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGRuntimeGenExecutionSource)

void UPCGRuntimeGenExecutionSource::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UPCGRuntimeGenExecutionSource* This = CastChecked<UPCGRuntimeGenExecutionSource>(InThis);

#if WITH_EDITOR
	This->ExecutionInspection.AddReferencedObjects(Collector);
#endif
}

UPCGRuntimeGenExecutionSource::UPCGRuntimeGenExecutionSource(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	ExecutionState.SetSource(this);
}

void UPCGRuntimeGenExecutionSource::Register(IPCGGraphExecutionSource* InOwningSource)
{
	OriginalSource = this;
	GridSize = PCGHiGenGrid::UnboundedGridSize();

	SetOwningSource(InOwningSource);

	if (UPCGSubsystem* Subsystem = Cast<UPCGSubsystem>(ExecutionState.GetSubsystem()))
	{
		Subsystem->RegisterOrUpdateExecutionSource(this, /*bDoPartitionMapping=*/false);
	}
}

void UPCGRuntimeGenExecutionSource::Unregister()
{
	if (UPCGSubsystem* Subsystem = Cast<UPCGSubsystem>(ExecutionState.GetSubsystem()))
	{
		Subsystem->UnregisterExecutionSource(this);
	}
}

void UPCGRuntimeGenExecutionSource::SetOwningSource(IPCGGraphExecutionSource* InOwningSource)
{
	check(InOwningSource);

	if (IsLocalSource())
	{
		return;
	}

	OwningSource = InOwningSource;

#if WITH_EDITOR
	UpdateTrackingCache();
#endif

	for (UPCGRuntimeGenExecutionSource* LocalSource : LocalSources)
	{
		if (!ensure(LocalSource))
		{
			continue;
		}

		LocalSource->OwningSource = OwningSource;
	}
}

UPCGData* UPCGRuntimeGenExecutionSource::GetSelfData() const
{
	return SelfData;
}

int32 UPCGRuntimeGenExecutionSource::GetSeed() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().GetSeed();
}

FString UPCGRuntimeGenExecutionSource::GetDebugName() const
{
	if (!IsLocalSource())
	{
		check(OwningSource);
		return OwningSource->GetExecutionState().GetDebugName();
	}

	TStringBuilderWithBuffer<TCHAR, 1024> StringBuilder;

	StringBuilder += FString::Printf(TEXT("PCGRuntimeGenExecutionSource_%d_"), GridSize);

	if (Use2DGrid())
	{
		StringBuilder += FString::Printf(TEXT("%d_%d"), GridCoords.X, GridCoords.Y);
	}
	else
	{
		StringBuilder += FString::Printf(TEXT("%d_%d_%d"), GridCoords.X, GridCoords.Y, GridCoords.Z);
	}

	return StringBuilder.ToString();
}

UWorld* UPCGRuntimeGenExecutionSource::GetWorld() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().GetWorld();
}

UObject* UPCGRuntimeGenExecutionSource::GetTarget() const
{
	check(OwningSource);

	// Local runtime gen execution sources don't have any target, but for original sources we can fallback to the original PCG component's actor.
	return IsLocalSource() ? nullptr : OwningSource->GetExecutionState().GetTarget();
}

bool UPCGRuntimeGenExecutionSource::IsActive() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().IsActive();
}

bool UPCGRuntimeGenExecutionSource::HasAuthority() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().HasAuthority();
}

FTransform UPCGRuntimeGenExecutionSource::GetTransform() const
{
	return IsLocalSource() ? FTransform(FQuat::Identity, (FVector(GridCoords) + FVector(0.5)) * GetGenerationGridSize(), FVector::OneVector) : GetOriginalTransform();
}

FTransform UPCGRuntimeGenExecutionSource::GetOriginalTransform() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().GetOriginalTransform();
}

FBox UPCGRuntimeGenExecutionSource::GetBoundsInternal() const
{
	if (IsLocalSource())
	{
		const FVector Center = GetTransform().GetLocation();
		const FVector::FReal HalfGridSize = GridSize / 2.0;

		FVector Extent(HalfGridSize, HalfGridSize, HalfGridSize);

		// In case of 2D grid, it's like the actor has infinite bounds on the Z axis
		if (Use2DGrid())
		{
			Extent.Z = HALF_WORLD_MAX1;
		}

		return FBox(Center - Extent, Center + Extent).Overlap(GetOriginalBoundsInternal());
	}
	else
	{
		return GetOriginalBoundsInternal();
	}
}

FBox UPCGRuntimeGenExecutionSource::GetBounds() const
{
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::Bounds, [this]() { return GetBoundsInternal(); });
}

FBox UPCGRuntimeGenExecutionSource::GetOriginalBoundsInternal() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().GetOriginalBounds();
}

FBox UPCGRuntimeGenExecutionSource::GetOriginalBounds() const
{
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::OriginalBounds, [this]() { return GetOriginalBoundsInternal(); });
}

FBox UPCGRuntimeGenExecutionSource::GetLocalSpaceBoundsInternal() const
{
	if (IsLocalSource())
	{
		const FBox Bounds = GetBounds();
		return Bounds.MoveTo(FVector::ZeroVector);
	}
	else
	{
		return GetOriginalLocalSpaceBoundsInternal();
	}
}

FBox UPCGRuntimeGenExecutionSource::GetLocalSpaceBounds() const
{
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::LocalSpaceBounds, [this]() { return GetLocalSpaceBoundsInternal(); });
}

FBox UPCGRuntimeGenExecutionSource::GetOriginalLocalSpaceBoundsInternal() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().GetOriginalLocalSpaceBounds();
}

FBox UPCGRuntimeGenExecutionSource::GetOriginalLocalSpaceBounds() const
{
	return UPCGSubsystem::GetOrCreateExecutionCacheValue(this, PCGPerExecutionCacheGuids::OriginalLocalSpaceBounds, [this]() { return GetOriginalLocalSpaceBoundsInternal(); });
}

UPCGGraph* UPCGRuntimeGenExecutionSource::GetGraph() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().GetGraph();
}

UPCGGraphInstance* UPCGRuntimeGenExecutionSource::GetGraphInstance() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().GetGraphInstance();
}

void UPCGRuntimeGenExecutionSource::Cancel()
{
	if (GenerationTaskId != InvalidPCGTaskId && ExecutionState.GetSubsystem())
	{
		ExecutionState.GetSubsystem()->CancelGeneration(this);
	}
}

void UPCGRuntimeGenExecutionSource::OnGraphExecutionAborted(bool bQuiet, bool bCleanupUnusedResources)
{
	GenerationTaskId = InvalidPCGTaskId;

	if (bCleanupUnusedResources)
	{
		FPCGManagedResourceContainerHelper ContainerHelper(this);
		ContainerHelper.CleanupUnusedManagedResources();
	}

#if PCG_PROFILING_ENABLED
	if (IPCGBaseSubsystem* Subsystem = GetExecutionState().GetSubsystem())
	{
		Subsystem->OnPCGSourceGenerationDone(this, EPCGGenerationStatus::Aborted);
	}
#endif // PCG_PROFILING_ENABLED
}

bool UPCGRuntimeGenExecutionSource::IsGenerating() const
{
	return GenerationTaskId != InvalidPCGTaskId;
}

bool UPCGRuntimeGenExecutionSource::IsGenerated() const
{
	return bGenerated;
}

FPCGTaskId UPCGRuntimeGenExecutionSource::GetGenerationTaskId() const
{
	return GenerationTaskId;
}

FPCGTaskId UPCGRuntimeGenExecutionSource::Generate(const IPCGGraphExecutionState::FGenerateParams& InGenerateParams)
{
	ClearPerPinGeneratedOutput();

	if (UPCGSubsystem* Subsystem = Cast<UPCGSubsystem>(ExecutionState.GetSubsystem()))
	{
		const FPCGTaskId NewGenerationTaskId = Subsystem->ScheduleGraph(this, InGenerateParams.Dependencies);

		if (NewGenerationTaskId != InvalidPCGTaskId)
		{
			auto PostProcessTask = [WeakThis = TWeakObjectPtr<UPCGRuntimeGenExecutionSource>(this)]()
			{
				UPCGRuntimeGenExecutionSource* This = WeakThis.Get();

				if (!IsValid(This))
				{
					return true;
				}

				PCGGraphExecutionLogging::LogPostProcessGraph(This);

				This->bGenerated = true;
				This->GenerationTaskId = InvalidPCGTaskId;

#if PCG_PROFILING_ENABLED
				if (IPCGBaseSubsystem* Subsystem = This->GetExecutionState().GetSubsystem())
				{
					Subsystem->OnPCGSourceGenerationDone(This, EPCGGenerationStatus::Completed);
				}
#endif // PCG_PROFILING_ENABLED

				return true;
			};

			// GenerationTaskId should actually be the post-process task, not the generation task itself.
			GenerationTaskId = Subsystem->ScheduleGeneric(PostProcessTask, this, /*Dependencies=*/{ NewGenerationTaskId });
			return GenerationTaskId;
		}
	}

	return InvalidPCGTaskId;
}

FPCGTaskId UPCGRuntimeGenExecutionSource::Cleanup(const IPCGGraphExecutionState::FCleanupParams& InCleanupParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGRuntimeGenExecutionSource::Cleanup);

	// Runtime gen cleanup is always immediate (for now), so we only implement the immediate path here.
	if (!InCleanupParams.bImmediate)
	{
		return InvalidPCGTaskId;
	}

	FPCGManagedResourceContainerHelper ContainerHelper(this);
	PCGGeneratedResourcesLogging::LogCleanupLocalImmediate(this, InCleanupParams.bReleaseManagedResources, ContainerHelper.GetManagedResourcesCopy());

	Cancel();

	ContainerHelper.Cleanup(InCleanupParams.bReleaseManagedResources);

	if (InCleanupParams.bReleaseManagedResources)
	{
		bGenerated = false;

		ClearPerPinGeneratedOutput();
	}

	if (!IsLocalSource() && InCleanupParams.bCleanupLocalSources)
	{
		for (UPCGRuntimeGenExecutionSource* LocalSource : LocalSources)
		{
			if (LocalSource)
			{
				LocalSource->GetExecutionState().Cleanup(InCleanupParams);
			}
		}
	}

	PCGGeneratedResourcesLogging::LogCleanupLocalImmediateFinished(this, ContainerHelper.GetManagedResourcesCopy());

	return InvalidPCGTaskId;
}

void UPCGRuntimeGenExecutionSource::StoreOutputDataForPin(const FString& InResourceKey, const FPCGDataCollection& InData)
{
	UE::TWriteScopeLock ScopedWriteLock(PerPinGeneratedOutputLock);

	InData.MarkUsage(EPCGDataUsage::ComponentPerPinOutputData);

	FPCGDataCollection* FoundExistingData = PerPinGeneratedOutput.Find(InResourceKey);

	// For all existing data items, clear their usage which may release transient resources if the data is not present in the new data collection.
	if (FoundExistingData)
	{
		for (const FPCGTaggedData& ExistingData : FoundExistingData->TaggedData)
		{
			if (ExistingData.Data && !InData.TaggedData.FindByPredicate([&ExistingData](const FPCGTaggedData& NewData) { return ExistingData.Data == NewData.Data; }))
			{
				ExistingData.Data->ClearUsage(EPCGDataUsage::ComponentPerPinOutputData);
			}
		}

		*FoundExistingData = InData;
	}
	else
	{
		PerPinGeneratedOutput.Add(InResourceKey, InData);
	}
}

const FPCGDataCollection* UPCGRuntimeGenExecutionSource::RetrieveOutputDataForPin(const FString& InResourceKey) const
{
	UE::TReadScopeLock ScopedReadLock(PerPinGeneratedOutputLock);
	return PerPinGeneratedOutput.Find(InResourceKey);
}

FPCGManagedResourceContainer* UPCGRuntimeGenExecutionSource::GetManagedResourceContainer()
{
	return &ManagedResourceContainer;
}

FTransactionallySafeCriticalSection* UPCGRuntimeGenExecutionSource::GetManagedResourceContainerLock()
{
	return &ManagedResourcesLock;
}

void UPCGRuntimeGenExecutionSource::ExecutePreGraph(FPCGContext* InContext)
{
	CreateSelfData();
	GetBounds();
	GetOriginalBounds();
	GetLocalSpaceBounds();
	GetOriginalLocalSpaceBounds();
}

#if PCG_PROFILING_ENABLED
const PCGUtils::FExtraCapture& UPCGRuntimeGenExecutionSource::GetExtraCapture() const
{
	return ExtraCapture;
}

PCGUtils::FExtraCapture& UPCGRuntimeGenExecutionSource::GetExtraCapture()
{
	return ExtraCapture;
}

const FPCGGraphExecutionInspection& UPCGRuntimeGenExecutionSource::GetInspection() const
{
	return ExecutionInspection;
}

FPCGGraphExecutionInspection& UPCGRuntimeGenExecutionSource::GetInspection()
{
	return ExecutionInspection;
}
#endif // PCG_PROFILING_ENABLED

#if WITH_EDITOR
FPCGDynamicTrackingPriority UPCGRuntimeGenExecutionSource::GetDynamicTrackingPriority() const
{
	return FPCGDynamicTrackingPriority(); 
}

void UPCGRuntimeGenExecutionSource::RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling)
{
	ChangeTracker.RegisterDynamicTracking(InSettings, InDynamicKeysAndCulling);
}

void UPCGRuntimeGenExecutionSource::RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings)
{
	ChangeTracker.RegisterDynamicTracking(InKeysToSettings);
}

TArray<FPCGSelectionKey> UPCGRuntimeGenExecutionSource::GatherTrackingKeys() const
{
	return ChangeTracker.GatherTrackingKeys();
}

bool UPCGRuntimeGenExecutionSource::IsKeyTrackedAndCulled(const FPCGSelectionKey& Key, bool& bOutIsCulled) const
{
	return ChangeTracker.IsKeyTrackedAndCulled(Key, bOutIsCulled);
}

void UPCGRuntimeGenExecutionSource::ForEachSettingTrackingKey(const FPCGSelectionKey& InKey, const TFunctionRef<void(const FPCGSelectionKey&, const FPCGSettingsAndCulling&)> InCallback) const
{
	ChangeTracker.ForEachSettingTrackingKey(InKey, InCallback);
}

void UPCGRuntimeGenExecutionSource::AddLocalSourceDynamicTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceDynamicTrackingKeys)
{
	ChangeTracker.AddLocalSourceDynamicTrackingKeys(LocalSourceDynamicTrackingKeys);
}

void UPCGRuntimeGenExecutionSource::AddLocalSourceCurrentTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceCurrentExecutionTrackingKeys, const TSet<const UPCGSettings*>& LocalSourceCurrentExecutionTrackingSettings)
{
	ChangeTracker.AddLocalSourceCurrentTrackingKeys(LocalSourceCurrentExecutionTrackingKeys, LocalSourceCurrentExecutionTrackingSettings);
}

bool UPCGRuntimeGenExecutionSource::IsRefreshInProgress() const
{
	return false;
}

bool UPCGRuntimeGenExecutionSource::WasGeneratedThisSession() const
{
	// Runtime gen execution sources are generated right away, so this is always true.
	return true;
}
#endif // WITH_EDITOR

IPCGGraphExecutionSource* UPCGRuntimeGenExecutionSource::GetOriginalSource() const
{
	return const_cast<UPCGRuntimeGenExecutionSource*>(OriginalSource);
}

IPCGGraphExecutionSource* UPCGRuntimeGenExecutionSource::GetLocalSource(const FPCGGridDescriptor& InGridDescriptor, const FIntVector& InGridCoords) const
{
	const uint32 LocalGridSize = InGridDescriptor.GetGridSize();

	// @todo_pcg: Could have a hashed lookup instead.
	for (UPCGRuntimeGenExecutionSource* LocalSource : OriginalSource->LocalSources)
	{
		if (LocalSource && LocalSource->GetGenerationGridSize() == LocalGridSize && LocalSource->GridCoords == InGridCoords)
		{
			return LocalSource;
		}
	}

	return nullptr;
}

bool UPCGRuntimeGenExecutionSource::IsPartitioned() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().IsPartitioned();
}

bool UPCGRuntimeGenExecutionSource::IsLocalSource() const
{
	return this != OriginalSource;
}

bool UPCGRuntimeGenExecutionSource::Use2DGrid() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().Use2DGrid();
}


uint32 UPCGRuntimeGenExecutionSource::GetGenerationGridSize() const
{
	return (IsPartitioned() || IsLocalSource()) ? GridSize : PCGHiGenGrid::UninitializedGridSize();
}

FIntVector UPCGRuntimeGenExecutionSource::GetGenerationGridCoords() const
{
	return GridCoords;
}

bool UPCGRuntimeGenExecutionSource::IsManagedByRuntimeGenSystem() const
{
	return true;
}

const UPCGSchedulingPolicyBase* UPCGRuntimeGenExecutionSource::GetRuntimeGenSchedulingPolicy() const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().GetRuntimeGenSchedulingPolicy();
}

const FPCGRuntimeGenerationRadii& UPCGRuntimeGenExecutionSource::GetGenerationRadii() const
{
	check(OwningSource);

	static const FPCGRuntimeGenerationRadii DefaultGenerationRadii;
	return OwningSource->GetExecutionState().GetGenerationRadii();
}

double UPCGRuntimeGenExecutionSource::GetGenerationRadiusFromGrid(uint32 InGrid) const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().GetGenerationRadiusFromGrid(InGrid);
}

double UPCGRuntimeGenExecutionSource::GetCleanupRadiusFromGrid(uint32 InGrid) const
{
	check(OwningSource);
	return OwningSource->GetExecutionState().GetCleanupRadiusFromGrid(InGrid);
}

UPCGRuntimeGenExecutionSource* UPCGRuntimeGenExecutionSource::CreateLocalSource(uint32 InGridSize, const FIntVector& InGridCoords)
{
	check(IsInGameThread());

	if (IsLocalSource() || !IsPartitioned())
	{
		return nullptr;
	}

	UPCGRuntimeGenExecutionSource* LocalSource = NewObject<UPCGRuntimeGenExecutionSource>();
	LocalSource->OwningSource = OwningSource;
	LocalSource->OriginalSource = this;
	LocalSource->GridSize = InGridSize;
	LocalSource->GridCoords = InGridCoords;

	LocalSources.Add(LocalSource);

	return LocalSource;
}

void UPCGRuntimeGenExecutionSource::DestroyLocalSource(UPCGRuntimeGenExecutionSource* InLocalSource)
{
	if (InLocalSource)
	{
		IPCGGraphExecutionState::FCleanupParams CleanupParams;
		CleanupParams.bImmediate = true;
		
		InLocalSource->GetExecutionState().Cleanup(CleanupParams);
		InLocalSource->Unregister();
		InLocalSource->MarkAsGarbage();
	}

	LocalSources.Remove(InLocalSource);
}

IPCGGraphExecutionSource* UPCGRuntimeGenExecutionSource::GetOwningSource()
{
	return OwningSource;
}

void UPCGRuntimeGenExecutionSource::ClearPerPinGeneratedOutput()
{
	UE::TWriteScopeLock ScopedWriteLock(PerPinGeneratedOutputLock);

	for (TPair<FString, FPCGDataCollection>& Entry : PerPinGeneratedOutput)
	{
		Entry.Value.ClearUsage(EPCGDataUsage::ComponentPerPinOutputData);
	}

	PerPinGeneratedOutput.Reset();
}

void UPCGRuntimeGenExecutionSource::CreateSelfData()
{
	check(IsInGameThread());

	UPCGVolumeData* VolumeData = NewObject<UPCGVolumeData>();
	VolumeData->Initialize(GetBounds());

	SelfData = VolumeData;
}

#if WITH_EDITOR
bool UPCGRuntimeGenExecutionSource::UpdateTrackingCache(TArray<FPCGSelectionKey>* InOptionalChangedKeys)
{
	return ChangeTracker.UpdateTrackingCache(GetGraph(), InOptionalChangedKeys);
}
#endif
