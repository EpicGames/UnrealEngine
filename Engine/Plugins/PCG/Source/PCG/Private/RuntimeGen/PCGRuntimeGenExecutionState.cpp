// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/PCGRuntimeGenExecutionState.h"

#include "RuntimeGen/PCGRuntimeGenExecutionSource.h"

UPCGData* FPCGRuntimeGenExecutionState::GetSelfData() const
{
	check(Source);
	return Source->GetSelfData();
}

int32 FPCGRuntimeGenExecutionState::GetSeed() const
{
	check(Source);
	return Source->GetSeed();
}

FString FPCGRuntimeGenExecutionState::GetDebugName() const
{
	check(Source);
	return Source->GetDebugName();
}

UWorld* FPCGRuntimeGenExecutionState::GetWorld() const
{
	check(Source);
	return Source->GetWorld();
}

UObject* FPCGRuntimeGenExecutionState::GetTarget() const
{
	check(Source);
	return Source->GetTarget();
}

bool FPCGRuntimeGenExecutionState::IsActive() const
{
	check(Source);
	return Source->IsActive();
}

bool FPCGRuntimeGenExecutionState::HasAuthority() const
{
	check(Source);
	return Source->HasAuthority();
}

FTransform FPCGRuntimeGenExecutionState::GetTransform() const
{
	check(Source);
	return Source->GetTransform();
}

FTransform FPCGRuntimeGenExecutionState::GetOriginalTransform() const
{
	check(Source);
	return Source->GetOriginalTransform();
}

FBox FPCGRuntimeGenExecutionState::GetBounds() const
{
	check(Source);
	return Source->GetBounds();
}

FBox FPCGRuntimeGenExecutionState::GetOriginalBounds() const
{
	check(Source);
	return Source->GetOriginalBounds();
}

FBox FPCGRuntimeGenExecutionState::GetLocalSpaceBounds() const
{
	check(Source);
	return Source->GetLocalSpaceBounds();
}

FBox FPCGRuntimeGenExecutionState::GetOriginalLocalSpaceBounds() const
{
	check(Source);
	return Source->GetOriginalLocalSpaceBounds();
}

UPCGGraph* FPCGRuntimeGenExecutionState::GetGraph() const
{
	check(Source);
	return Source->GetGraph();
}

UPCGGraphInstance* FPCGRuntimeGenExecutionState::GetGraphInstance() const
{
	check(Source);
	return Source->GetGraphInstance();
}

void FPCGRuntimeGenExecutionState::Cancel()
{
	check(Source);
	Source->Cancel();
}

void FPCGRuntimeGenExecutionState::OnGraphExecutionAborted(bool bQuiet, bool bCleanupUnusedResources)
{
	check(Source);
	Source->OnGraphExecutionAborted(bQuiet, bCleanupUnusedResources);
}

bool FPCGRuntimeGenExecutionState::IsGenerating() const
{
	check(Source);
	return Source->IsGenerating();
}

bool FPCGRuntimeGenExecutionState::IsGenerated() const
{
	check(Source);
	return Source->IsGenerated();
}

FPCGTaskId FPCGRuntimeGenExecutionState::GetGenerationTaskId() const
{
	check(Source);
	return Source->GetGenerationTaskId();
}

FPCGTaskId FPCGRuntimeGenExecutionState::Generate(const FGenerateParams& InGenerateParams)
{
	check(Source);
	return Source->Generate(InGenerateParams);
}

FPCGTaskId FPCGRuntimeGenExecutionState::Cleanup(const FCleanupParams& InCleanupParams)
{
	check(Source);
	return Source->Cleanup(InCleanupParams);
}

void FPCGRuntimeGenExecutionState::StoreOutputDataForPin(const FString& InResourceKey, const FPCGDataCollection& InData) const
{
	check(Source);
	Source->StoreOutputDataForPin(InResourceKey, InData);
}

const FPCGDataCollection* FPCGRuntimeGenExecutionState::RetrieveOutputDataForPin(const FString& InResourceKey) const
{
	check(Source);
	return Source->RetrieveOutputDataForPin(InResourceKey);
}

FPCGManagedResourceContainer* FPCGRuntimeGenExecutionState::GetManagedResourceContainer()
{
	check(Source);
	return Source->GetManagedResourceContainer();
}

FTransactionallySafeCriticalSection* FPCGRuntimeGenExecutionState::GetManagedResourceContainerLock()
{
	check(Source);
	return Source->GetManagedResourceContainerLock();
}

void FPCGRuntimeGenExecutionState::ExecutePreGraph(FPCGContext* InContext)
{
	check(Source);
	Source->ExecutePreGraph(InContext);
}

#if PCG_PROFILING_ENABLED
const PCGUtils::FExtraCapture& FPCGRuntimeGenExecutionState::GetExtraCapture() const
{
	check(Source);
	return Source->GetExtraCapture();
}

PCGUtils::FExtraCapture& FPCGRuntimeGenExecutionState::GetExtraCapture()
{
	check(Source);
	return Source->GetExtraCapture();
}

const FPCGGraphExecutionInspection& FPCGRuntimeGenExecutionState::GetInspection() const
{
	check(Source);
	return Source->GetInspection();
}

FPCGGraphExecutionInspection& FPCGRuntimeGenExecutionState::GetInspection()
{
	check(Source);
	return Source->GetInspection();
}
#endif // PCG_PROFILING_ENABLED

#if WITH_EDITOR
void FPCGRuntimeGenExecutionState::RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling)
{
	check(Source);
	Source->RegisterDynamicTracking(InSettings, InDynamicKeysAndCulling);
}

FPCGDynamicTrackingPriority FPCGRuntimeGenExecutionState::GetDynamicTrackingPriority() const
{
	check(Source);
	return Source->GetDynamicTrackingPriority();
}

void FPCGRuntimeGenExecutionState::RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings)
{
	check(Source);
	Source->RegisterDynamicTracking(InKeysToSettings);
}

TArray<FPCGSelectionKey> FPCGRuntimeGenExecutionState::GatherTrackingKeys() const
{
	check(Source);
	return Source->GatherTrackingKeys();
}

bool FPCGRuntimeGenExecutionState::IsKeyTrackedAndCulled(const FPCGSelectionKey& Key, bool& bOutIsCulled) const
{
	check(Source);
	return Source->IsKeyTrackedAndCulled(Key, bOutIsCulled);
}

void FPCGRuntimeGenExecutionState::ForEachSettingTrackingKey(const FPCGSelectionKey& InKey, const TFunctionRef<void(const FPCGSelectionKey&, const FPCGSettingsAndCulling&)> InCallback) const
{
	check(Source);
	Source->ForEachSettingTrackingKey(InKey, InCallback);
}

void FPCGRuntimeGenExecutionState::AddLocalSourceDynamicTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceDynamicTrackingKeys)
{
	check(Source);
	Source->AddLocalSourceDynamicTrackingKeys(LocalSourceDynamicTrackingKeys);
}

void FPCGRuntimeGenExecutionState::AddLocalSourceCurrentTrackingKeys(const FPCGSelectionKeyToSettingsMap& LocalSourceCurrentExecutionTrackingKeys, const TSet<const UPCGSettings*>& LocalSourceCurrentExecutionTrackingSettings)
{
	check(Source);
	Source->AddLocalSourceCurrentTrackingKeys(LocalSourceCurrentExecutionTrackingKeys, LocalSourceCurrentExecutionTrackingSettings);
}

bool FPCGRuntimeGenExecutionState::IsRefreshInProgress() const
{
	check(Source);
	return Source->IsRefreshInProgress();
}

bool FPCGRuntimeGenExecutionState::WasGeneratedThisSession() const
{
	check(Source);
	return Source->WasGeneratedThisSession();
}
#endif // WITH_EDITOR

IPCGGraphExecutionSource* FPCGRuntimeGenExecutionState::GetOriginalSource() const
{
	check(Source);
	return Source->GetOriginalSource();
}

IPCGGraphExecutionSource* FPCGRuntimeGenExecutionState::GetLocalSource(const FPCGGridDescriptor& GridDescriptor, const FIntVector& CellCoords) const
{
	check(Source);
	return Source->GetLocalSource(GridDescriptor, CellCoords);
}

bool FPCGRuntimeGenExecutionState::IsPartitioned() const
{
	check(Source);
	return Source->IsPartitioned();
}

bool FPCGRuntimeGenExecutionState::IsLocalSource() const
{
	check(Source);
	return Source->IsLocalSource();
}

bool FPCGRuntimeGenExecutionState::Use2DGrid() const
{
	check(Source);
	return Source->Use2DGrid();
}


uint32 FPCGRuntimeGenExecutionState::GetGenerationGridSize() const
{
	check(Source);
	return Source->GetGenerationGridSize();
}

FIntVector FPCGRuntimeGenExecutionState::GetGenerationGridCoords() const
{
	check(Source);
	return Source->GetGenerationGridCoords();
}

bool FPCGRuntimeGenExecutionState::IsManagedByRuntimeGenSystem() const
{
	check(Source);
	return Source->IsManagedByRuntimeGenSystem();
}

const UPCGSchedulingPolicyBase* FPCGRuntimeGenExecutionState::GetRuntimeGenSchedulingPolicy() const
{
	check(Source);
	return Source->GetRuntimeGenSchedulingPolicy();
}

const FPCGRuntimeGenerationRadii& FPCGRuntimeGenExecutionState::GetGenerationRadii() const
{
	check(Source);
	return Source->GetGenerationRadii();
}

double FPCGRuntimeGenExecutionState::GetGenerationRadiusFromGrid(uint32 InGrid) const
{
	check(Source);
	return Source->GetGenerationRadiusFromGrid(InGrid);
}

double FPCGRuntimeGenExecutionState::GetCleanupRadiusFromGrid(uint32 InGrid) const
{
	check(Source);
	return Source->GetCleanupRadiusFromGrid(InGrid);
}

void FPCGRuntimeGenExecutionState::SetSource(UPCGRuntimeGenExecutionSource* InSource)
{
	Source = InSource;
}
