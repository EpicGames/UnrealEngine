// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/PCGRuntimeGenScheduler.h"
#include "RuntimeGen/SchedulingPolicies/PCGSchedulingPolicyBase.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGRayTracingUVCacheUtils.h"
#include "PCGSubsystem.h"
#include "PCGTrackingManager.h"
#include "PCGWorldActor.h"
#include "Grid/PCGGridDescriptor.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "RuntimeGen/PCGGenSourceManager.h"
#include "RuntimeGen/PCGRuntimeGenExecutionSource.h"
#include "RuntimeGen/GenSources/PCGGenSourceBase.h"
#include "RuntimeGen/GenSources/PCGGenSourceComponent.h"

#include "DrawDebugHelpers.h"
#include "EngineDefines.h" // For UE_ENABLE_DEBUG_DRAWING
#include "Async/ParallelFor.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "HAL/IConsoleManager.h"
#include "Net/UnrealNetwork.h"
#include "Streaming/LevelStreamingDelegates.h"
#include "UObject/UObjectGlobals.h"
#include "VT/RuntimeVirtualTexture.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/WorldPartitionSubsystem.h"

#if WITH_EDITOR
#include "Editor/IPCGEditorModule.h"

#include "EditorViewportClient.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGRuntimeGenScheduler)

// Onscreen debug messages not supported in UE in shipping & test builds.
#define PCG_RGS_ONSCREENDEBUGMESSAGES (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

namespace PCGRuntimeGenSchedulerConstants
{
	const FString PooledPartitionActorName = TEXT("PCGRuntimePartitionGridActor_POOLED");
	static constexpr float MinWorldVirtualTextureTexelSize = 0.1f;
}

namespace PCGRuntimeGenSchedulerHelpers
{
	static TAutoConsoleVariable<bool> CVarRuntimeGenerationEnable(
		TEXT("pcg.RuntimeGeneration.Enable"),
		true,
		TEXT("Enable the RuntimeGeneration system."));

	static TAutoConsoleVariable<bool> CVarRuntimeGenerationEnableChangeDetection(
		TEXT("pcg.RuntimeGeneration.EnableChangeDetection"),
		true,
		TEXT("Skips execution of scheduling and cleanup loops if generation sources and other world state have not changed to reduce tick cost on game thread."));

	static TAutoConsoleVariable<int32> CVarNumGeneratingComponentsAtSameTime(
		TEXT("pcg.RuntimeGeneration.NumGeneratingComponents"),
		16,
		TEXT("Defines the maximum number of runtime components that can generate at the same time."));

	TAutoConsoleVariable<float> CVarRuntimeGenerationRadiusMultiplier(
		TEXT("pcg.RuntimeGeneration.GlobalRadiusMultiplier"),
		1.0f,
		TEXT("Global multiplier for generation radius of all runtime gen components."));

	static TAutoConsoleVariable<bool> CVarRuntimeGenerationEnableDebugging(
		TEXT("pcg.RuntimeGeneration.EnableDebugging"),
		false,
		TEXT("Enable verbose debug logging for the RuntimeGeneration system."));

	static TAutoConsoleVariable<bool> CVarRuntimeGenerationEnablePooling(
		TEXT("pcg.RuntimeGeneration.EnablePooling"),
		true,
		TEXT("Enable PartitionActor pooling for the RuntimeGeneration system."));

	static TAutoConsoleVariable<int32> CVarRuntimeGenerationBasePoolSize(
		TEXT("pcg.RuntimeGeneration.BasePoolSize"),
		100,
		TEXT("Defines the base PartitionActor pool size for the RuntimeGeneration system. Cannot be less than 1."));

	static FAutoConsoleCommand CommandFlushActorPool(
		TEXT("pcg.RuntimeGeneration.FlushActorPool"),
		TEXT("Flushes all pooled actors and regenerates all components."),
		FConsoleCommandDelegate::CreateLambda([]()
		{
			if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
			{
				PCGSubsystem->GetRuntimeGenScheduler()->FlushAllGeneratedActors();
			}
		}));

	static TAutoConsoleVariable<bool> CVarHideActorsFromOutliner(
		TEXT("pcg.RuntimeGeneration.HideActorsFromOutliner"),
		true,
		TEXT("Hides partition actors from Scene Outliner."),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
		{
			if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
			{
				PCGSubsystem->GetRuntimeGenScheduler()->FlushAllGeneratedActors();
			}
		}));

	static TAutoConsoleVariable<bool> CVarEnableWorldStreamingQueries(
		TEXT("pcg.RuntimeGeneration.EnableWorldStreamingQueries"),
		true,
		TEXT("Checks that the world is streamed in before triggering generation of local (partitioned) components."));

	static TAutoConsoleVariable<int32> CVarFramesBeforeFirstGenerate(
		TEXT("pcg.RuntimeGeneration.FramesBeforeFirstGenerate"),
		0,
		TEXT("Waits this many engine ticks before allowing runtime gen to schedule generation."));

	static TAutoConsoleVariable<bool> CVarEnableVirtualTexturePriming(
		TEXT("pcg.VirtualTexturePriming.Enable"),
		true,
		TEXT("Enable priming of virtual textures for PCG Components which request it."));

	static TAutoConsoleVariable<bool> CVarDebugDrawTexturePrimingBounds(
		TEXT("pcg.VirtualTexturePriming.DebugDrawTexturePrimingBounds"),
		false,
		TEXT("Draws debug boxes to indicate regions where PCG is requesting virtual texture priming."));

	static float GTimeBetweenTicks = 0.0f;
	static FAutoConsoleVariableRef CVarTimeBetweenTicks(
		TEXT("pcg.RuntimeGeneration.TimeBetweenRuntimeGenSchedulerTicks"),
		GTimeBetweenTicks,
		TEXT("Period between ticks of the runtime generation scheduler. Saves processing time. If scheduling work is in progress or execution sources updated or refreshed, tick will process every frame until scheduler is idle."));

#if PCG_RGS_ONSCREENDEBUGMESSAGES
	static TAutoConsoleVariable<bool> CVarEnableDebugOverlay(
		TEXT("pcg.RuntimeGeneration.EnableDebugOverlay"),
		false,
		TEXT("Display screen overlay with runtime generation statistics."));

	struct FStatsOverlay
	{
		void BeginTick()
		{
			*this = {};

			TickStartTime = FPlatformTime::Seconds();
		}

		void EndTick(const FPCGRuntimeGenScheduler& InRuntimeGenScheduler, const TSet<IPCGGenSourceBase*>& InGenSources)
		{
			if (CVarEnableDebugOverlay.GetValueOnGameThread())
			{
				const double ElapsedMs = (FPlatformTime::Seconds() - TickStartTime) * 1000.0;

				TStringBuilder<0> GenSourceStringBuilder;
				for (const IPCGGenSourceBase* GenSource : InGenSources)
				{
					if (GenSource)
					{
						const TOptional<FVector> Position = GenSource->GetPosition();
						const TOptional<FVector> Direction = GenSource->GetDirection();

						GenSourceStringBuilder.Appendf(TEXT("        [%s] Position: %s Direction: %s\n"),
							*GenSource->GetDebugName(),
							Position ? *Position->ToString() : TEXT("UNSET"),
							Direction ? *Direction->ToString() : TEXT("UNSET"));
					}
					else
					{
						GenSourceStringBuilder.Append(TEXT("        NULL SOURCE\n"));
					}
				}

				GEngine->AddOnScreenDebugMessage(
					/*Key=*/-1,
					/*TimeToDisplay=*/0.0f,
					FColor::Yellow,
					FString::Printf(
						TEXT("PCG Runtime Generation%s\n")
						TEXT("    Tick time: %.3fms\n")
						TEXT("        Generate Calls: %d\n")
						TEXT("        Cleanup Calls: %d\n")
						TEXT("        Grid Cell Scans: %d\n")
						TEXT("    Num Generating Components: %d / %d\n")
						TEXT("    PA Pool: %d / %d available\n")
						TEXT("    VT Preloads: %d\n")
						TEXT("    Unique generation sources:\n%s"),
						PCGHelpers::IsServer(InRuntimeGenScheduler.Context.World) ? TEXT(" (Server)") : TEXT(""),
						ElapsedMs,
						GenerateCallCounter,
						CleanupCallCounter,
						GridCellScanCounter,
						NumGeneratingComponents, FMath::Max(1, CVarNumGeneratingComponentsAtSameTime.GetValueOnGameThread()),
						InRuntimeGenScheduler.PartitionActorPool.Num(), InRuntimeGenScheduler.PartitionActorPoolSize,
						VTPreloadCounter,
						*GenSourceStringBuilder)
				);
			}
		}

		int GenerateCallCounter = 0;
		int CleanupCallCounter = 0;
		int GridCellScanCounter = 0;
		int NumGeneratingComponents = 0;
		int VTPreloadCounter = 0;

		double TickStartTime = 0.0;
	};

	static FStatsOverlay Stats;
#endif // PCG_RGS_ONSCREENDEBUGMESSAGES
}

FPCGGridDescriptor FPCGRuntimeGenScheduler::FGridGenerationKey::GetGridDescriptor() const
{
	return FPCGGridDescriptor().SetGridSize(GetGridSize()).SetIsRuntime(true).SetIs2DGrid(Use2DGrid());
}

// DEPRECATED 5.8
UPCGComponent* FPCGRuntimeGenScheduler::FGridGenerationKey::GetOriginalComponent() const
{
	return Cast<UPCGComponent>(GetOriginalSource());
}

// DEPRECATED 5.8
UPCGComponent* FPCGRuntimeGenScheduler::FGridGenerationKey::GetOriginalComponentEvenIfGarbage() const
{
	return Cast<UPCGComponent>(GetOriginalSourceEvenIfGarbage());
}

// DEPRECATED 5.8
void FPCGRuntimeGenScheduler::FGridGenerationKey::SetOriginalComponent(UPCGComponent* InComponent)
{
	SetOriginalSource(InComponent);
}

// DEPRECATED 5.8
IPCGGraphExecutionSource* FPCGRuntimeGenScheduler::FGridGenerationKey::GetCachedLocalComponent() const
{
	return GetCachedLocalSource();
}

// DEPRECATED 5.8
void FPCGRuntimeGenScheduler::FGridGenerationKey::SetCachedLocalComponent(IPCGGraphExecutionSource* InLocalComponent) const
{
	SetCachedLocalSource(InLocalComponent);
}

FPCGRuntimeGenScheduler::FPCGRuntimeGenScheduler(UWorld* InWorld, FPCGTrackingManager* InTrackingManager)
{
	check(InWorld && InTrackingManager);

	Context.World = InWorld;

	Subsystem = UPCGSubsystem::GetInstance(Context.World);
	WorldPartitionSubsystem = Context.World ? UWorld::GetSubsystem<UWorldPartitionSubsystem>(Context.World) : nullptr;

	TrackingManager = InTrackingManager;
	GenSourceManager = new FPCGGenSourceManager(Context);
	bPoolingWasEnabledLastFrame = PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnablePooling.GetValueOnAnyThread();
	BasePoolSizeLastFrame = PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationBasePoolSize.GetValueOnAnyThread();
	FramesUntilGeneration = PCGRuntimeGenSchedulerHelpers::CVarFramesBeforeFirstGenerate.GetValueOnGameThread();

	RefreshContext();

	FLevelStreamingDelegates::OnLevelStreamingStateChanged.AddRaw(this, &FPCGRuntimeGenScheduler::OnLevelStreamingStateChanged);
	FNetworkReplayDelegates::OnPreScrub.AddRaw(this, &FPCGRuntimeGenScheduler::OnNetworkReplayScrub);
	FNetworkReplayDelegates::OnReplayScrubComplete.AddRaw(this, &FPCGRuntimeGenScheduler::OnNetworkReplayScrubComplete);

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	CVarRefreshSink = MakeUnique<FAutoConsoleVariableSink>(FConsoleCommandDelegate::CreateRaw(this, &FPCGRuntimeGenScheduler::OnCVarSinkFired));
#endif // WITH_EDITOR || !UE_BUILD_SHIPPING

#if WITH_EDITOR
	// Handle UWorld::ReInitWorld
	// - Actors will get unregistered and re-registered in a new subsystem, thus creating a new RGS
	// - We need to find them and properly re-register them (Pooled and in-use runtime PAs)
	TArray<APCGPartitionActor*> ExistingPoolPartitionActors;
	int32 NumPartitionActors = 0;

	UPCGActorHelpers::ForEachActorInLevel<APCGPartitionActor>(Context.World->PersistentLevel, [&NumPartitionActors, &ExistingPoolPartitionActors](AActor* InActor)
	{
		if (APCGPartitionActor* InPartitionActor = Cast<APCGPartitionActor>(InActor); InPartitionActor && InPartitionActor->HasAnyFlags(RF_Transient) && InPartitionActor->IsRuntimeGenerated())
		{
			NumPartitionActors++;

			if (InPartitionActor->GetPCGGridSize() > 0)
			{
				InPartitionActor->RegisterPCG();
			}
			else
			{
				ExistingPoolPartitionActors.Add(InPartitionActor);
			}
		}
		return true;
	});

	if (NumPartitionActors > 0)
	{
		check(PartitionActorPool.IsEmpty());
		PartitionActorPool.Append(ExistingPoolPartitionActors);
		PartitionActorPoolSize = NumPartitionActors;
	}
#endif
}

FPCGRuntimeGenScheduler::~FPCGRuntimeGenScheduler()
{
#if WITH_EDITOR || !UE_BUILD_SHIPPING
	if (CVarRefreshSink)
	{
		IConsoleManager::Get().UnregisterConsoleVariableSink_Handle(CVarRefreshSink->Handle);
	}
#endif // WITH_EDITOR || !UE_BUILD_SHIPPING

	delete GenSourceManager;
	GenSourceManager = nullptr;

	FLevelStreamingDelegates::OnLevelStreamingStateChanged.RemoveAll(this);
	FNetworkReplayDelegates::OnPreScrub.RemoveAll(this);
	FNetworkReplayDelegates::OnReplayScrubComplete.RemoveAll(this);
}

void FPCGRuntimeGenScheduler::Tick(APCGWorldActor* InPCGWorldActor, double InDeltaTime, double InEndTime)
{
	check(InPCGWorldActor && GenSourceManager);

	// 0. Preamble - check if we should be active in this world and do lazy initialization.

	if (!ShouldTick(InDeltaTime))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::Tick);

#if PCG_RGS_ONSCREENDEBUGMESSAGES
	PCGRuntimeGenSchedulerHelpers::Stats.BeginTick();
#endif

	TickCVars(InPCGWorldActor);

	Context.PCGWorldActor = InPCGWorldActor;

	// Update and grab unique generation sources.
	GenSources.Empty(GenSources.Num());
	if (Context.bAnyRuntimeGenSourcesExist)
	{
		GenSourceManager->Tick();
		GenSourceManager->GetUniqueGenSources(Context, GenSources);
	}

	if (!CachedPrimingInfos.IsEmpty() && !GenSources.IsEmpty() && PCGRuntimeGenSchedulerHelpers::CVarEnableVirtualTexturePriming.GetValueOnGameThread())
	{
		// @todo_pcg: To support VT priming outside of RuntimeGen, this should probably move outside of the RGS tick, and be ticked directly by the PCGSubsystem.
		// However, that would require also moving the GenSourceManager out of the RGS, 
		TickRequestVirtualTexturePriming(GenSources);
	}

	// Initialize RuntimeGen PA pool if necessary. If PoolSize is 0, then we have not initialized the pool yet.
	if (!GenSources.IsEmpty() || !GeneratedSources.IsEmpty())
	{
		if (PartitionActorPoolSize == 0 && PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnablePooling.GetValueOnAnyThread())
		{
			AddPartitionActorPoolCount(PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationBasePoolSize.GetValueOnAnyThread());
		}
	}

	// Allow virtual texture priming to tick even when generation has not begun. This helps alleviate issues where we
	// generate before the virtual textures have finished streaming in, which is particularly problematic on load.
	if (FramesUntilGeneration > 0)
	{
		--FramesUntilGeneration;
		return;
	}

	CleanupDelayedRefreshSources();

	// 1. Queue nearby execution sources for generation.
	
	if (!GenSources.IsEmpty())
	{
		FTickQueueSourcesForGenerationInputs Inputs;
		Inputs.GenSources = &GenSources;
		Inputs.PCGWorldActor = InPCGWorldActor;
		Inputs.AllPartitionedExecutionSources = TrackingManager->GetAllRegisteredPartitionedExecutionSources();
		Inputs.AllNonPartitionedExecutionSources = TrackingManager->GetAllRegisteredNonPartitionedExecutionSources();
		Inputs.GeneratedSources = &GeneratedSources;

		ChangeDetector.PreTick();
		TickQueueSourcesForGeneration(Inputs, SourcesToGenerate);
		ChangeDetector.PostTick();
	}

	// 2. Schedule cleanup on execution sources that become out of range.

	if (!GeneratedSources.IsEmpty() && (GenSources.IsEmpty() || bCleanupScanRequired || !IsChangeDetectionEnabled()))
	{
		TickCleanup(GenSources, InPCGWorldActor, InEndTime);
	}

	// 3. Schedule generation on execution sources in priority order.
	if (!SourcesToGenerate.IsEmpty())
	{
		// Sort execution sources by priority (will be generated in descending order).
		SourcesToGenerate.ValueSort([](double PrioA, double PrioB)->bool { return PrioA > PrioB; });

		// Only apply time budget to cleanup currently. Currently too easy to introduce latency issues so don't hold back generation new execution sources
		// (and we already have CVarNumGeneratingComponentsAtSameTime to throttle new generations).
		TickScheduleGeneration(SourcesToGenerate);

		if (!SourcesToGenerate.IsEmpty())
		{
			// Continue next frame.
			TimeToNextTick = 0.0;
		}
	}

#if PCG_RGS_ONSCREENDEBUGMESSAGES
	PCGRuntimeGenSchedulerHelpers::Stats.EndTick(*this, GenSources);
#endif
}

bool FPCGRuntimeGenScheduler::ShouldTick(double InDeltaTime)
{
	check(Context.World);

	if (!PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnable.GetValueOnGameThread())
	{
		return false;
	}

	if (bSuspendedForReplayScrub)
	{
		return false;
	}

	TimeToNextTick -= InDeltaTime;
	if (TimeToNextTick > 0.0)
	{
		return false;
	}
	TimeToNextTick = PCGRuntimeGenSchedulerHelpers::GTimeBetweenTicks;

	// Disable tick of editor scheduling if in runtime or PIE.
	if (PCGHelpers::IsRuntimeOrPIE() && !Context.World->IsGameWorld())
	{
		return false;
	}

#if WITH_EDITOR
	// If we're in an editor world, stop updating preview if the editor window/viewport is not active (follows
	// same behaviour as other things).
	if (!Context.World->IsGameWorld())
	{
		FViewport* ActiveViewport = GEditor ? GEditor->GetActiveViewport() : nullptr;

		// The active viewport's client may not be a FEditorViewportClient, so look it up in the editor's registered client list to be safe.
		FEditorViewportClient* ViewportClient = nullptr;
		if (ActiveViewport && GEditor)
		{
			for (FEditorViewportClient* Candidate : GEditor->GetAllViewportClients())
			{
				if (Candidate && Candidate->Viewport == ActiveViewport)
				{
					ViewportClient = Candidate;
					break;
				}
			}
		}

		if (!ViewportClient || !ViewportClient->IsVisible())
		{
			return false;
		}
	}
#endif

	if (bContextDirty)
	{
		RefreshContext();
	}

	// We can stop ticking if there are no runtime gen execution sources alive and there are no generated execution sources that need cleaning up.
	if (!Context.bAnyRuntimeGenSourcesExist && GeneratedSources.IsEmpty() && GeneratedSourcesToRemove.IsEmpty())
	{
		return false;
	}

	// Check if the non-spatially streamed cell is ready (only if world streaming queries are enabled). We should not do any processing until the world is ready.
	if (!bNonSpatialCellFinishedStreaming)
	{
		if (WorldPartitionSubsystem && PCGRuntimeGenSchedulerHelpers::CVarEnableWorldStreamingQueries.GetValueOnGameThread())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IsNonSpatialWorldStreamingComplete);

			FWorldPartitionStreamingQuerySource NonSpatialStreamingQuerySource;
			NonSpatialStreamingQuerySource.bSpatialQuery = false;

			bNonSpatialCellFinishedStreaming = WorldPartitionSubsystem->IsStreamingCompleted(EWorldPartitionRuntimeCellState::Activated, { NonSpatialStreamingQuerySource }, /*bExactState*/false);

			if (!bNonSpatialCellFinishedStreaming)
			{
				UE_LOGF(LogPCG, Verbose, "Holding back tick of Runtime Gen Scheduler because the world is not yet loaded.");
				return false;
			}
		}
		else
		{
			// Do nothing if we are not in a WP level. The persistent level should already be loaded by now.
			bNonSpatialCellFinishedStreaming = true;
		}
	}

	return true;
}

void FPCGRuntimeGenScheduler::TickQueueSourcesForGeneration(
	const FTickQueueSourcesForGenerationInputs& Inputs,
	TMap<FGridGenerationKey, double>& OutSourcesToGenerate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::TickQueueSourcesForGeneration);

	check(Inputs.PCGWorldActor && Inputs.GenSources);

	// TODO: Thought - it would be possible to maintain a global maximum generation distance across all execution sources
	// perhaps in the actor&comp mapping system, and then do a spatial query to get the execution sources here.

	auto AddSourceToGenerate = [&OutSourcesToGenerate](FGridGenerationKey& InKey, const IPCGGenSourceBase* InGenSource, const UPCGSchedulingPolicyBase* InPolicy, const FBox& InSourceBounds, bool bInUse2DGrid)
	{
		const double PolicyPriority = InPolicy ? InPolicy->CalculatePriority(InGenSource, InSourceBounds, bInUse2DGrid) : 0.0;
		double Priority = FMath::Clamp(PolicyPriority, 0.0, 1.0);
		if (PolicyPriority != Priority)
		{
			UE_LOGF(LogPCG, Warning, "Priority from runtime generation policy (%lf) outside [0.0, 1.0] range, clamped.", PolicyPriority);
		}

		// Generate largest grid to smallest (and unbounded is larger than any grid).
		const uint32 GridSize = InKey.GetGridSize();
		Priority += GridSize;

		double* ExistingPriority = OutSourcesToGenerate.Find(InKey);
		if (!ExistingPriority)
		{
			OutSourcesToGenerate.Add(InKey, Priority);
		}
		else if (Priority > *ExistingPriority)
		{
			// If this generation source prioritizes this grid cell higher, then bump the priority.
			*ExistingPriority = Priority;
		}
	};

	// Prepare streaming queries up front.
	TArray<FWorldPartitionStreamingQuerySource> StreamingQuerySources;
	StreamingQuerySources.Reserve(1);
	
	bool bCheckStreaming = false;

	if (WorldPartitionSubsystem && PCGRuntimeGenSchedulerHelpers::CVarEnableWorldStreamingQueries.GetValueOnGameThread())
	{
		FWorldPartitionStreamingQuerySource& QuerySource = StreamingQuerySources.Emplace_GetRef();
		QuerySource.bSpatialQuery = true;
		QuerySource.bUseGridLoadingRange = false;
		QuerySource.bDataLayersOnly = false;
		bCheckStreaming = true;
	}

	auto IsWorldStreamingComplete = [WorldPartitionSubsystem=WorldPartitionSubsystem, &StreamingQuerySources, &CachedStreamingQueryResults=CachedStreamingQueryResults](const FVector& InLocation, float InGridSize, const TSet<FName>& InTargetGrids, uint64 InTargetGridsHash)
	{
		FStreamingCompleteQueryKey Key{ InLocation, InGridSize, InTargetGridsHash };

		if (bool* FoundResult = CachedStreamingQueryResults.Find(Key))
		{
			return *FoundResult;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(IsWorldStreamingComplete);

		// @todo_pcg: We should be querying a box instead of a radius. As is, this will miss corners, but maybe this is preferable to
		// dilating the streaming query radius to circumscribe the volume, which could induce significantly more generation latency.
		StreamingQuerySources[0].Radius = InGridSize / 2.0f;
		StreamingQuerySources[0].Location = InLocation;
		StreamingQuerySources[0].TargetGrids = InTargetGrids;

		const bool bIsLoaded = WorldPartitionSubsystem->IsStreamingCompleted(EWorldPartitionRuntimeCellState::Activated, StreamingQuerySources, /*bExactState*/ false);

		CachedStreamingQueryResults.Add(Key, bIsLoaded);

		if (!bIsLoaded)
		{
			UE_LOGF(LogPCG, Verbose, "Holding back generation of cell at (%.2f, %.2f, %.2f), grid size %f, due to world not loaded.", InLocation.X, InLocation.Y, InLocation.Z, InGridSize);
		}

		return bIsLoaded;
	};

	const bool bDoChangeDetection = IsChangeDetectionEnabled();
#if UE_ENABLE_DEBUG_DRAWING
	const bool bDebugDrawGenerationSources = PCGSystemSwitches::CVarPCGDebugDrawGeneratedCells.GetValueOnGameThread();
#endif

	// Collect local execution sources from all partitioned execution sources.
	for (IPCGGraphExecutionSource* OriginalExecutionSource : Inputs.AllPartitionedExecutionSources)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::CollectPartitionedSources);

		if (!OriginalExecutionSource)
		{
			continue;
		}

		// Skip PCG components in actor-componentless mode. They should have registered a UPCGRuntimeGenExecutionSource that will handle generation.
		if (UPCGComponent* OriginalComponent = Cast<UPCGComponent>(OriginalExecutionSource); OriginalComponent && OriginalComponent->UseActorComponentlessGeneration())
		{
			continue;
		}

		const IPCGGraphExecutionState& OriginalExecutionState = OriginalExecutionSource->GetExecutionState();

		if (!OriginalExecutionState.GetGraph() || !OriginalExecutionState.IsActive())
		{
			continue;
		}

		const UPCGSchedulingPolicyBase* Policy = OriginalExecutionState.GetRuntimeGenSchedulingPolicy();

		// todo_pcg: For each execution domain (for now only GenAtRuntime/dynamic), assuming we run Preview through this scheduler, which it seems like we will.
		// todo_pcg: All this stuff can be hoisted - whether valid, has graph, active, managed by runtime gen..
		if (OriginalExecutionState.IsManagedByRuntimeGenSystem() && ensure(Policy))
		{
			// Policy's target grid set is used by every streaming query below; its cached hash keys the cache.
			const TSet<FName>& PolicyTargetGrids = Policy->GetWorldPartitionTargetGrids();
			const uint64 PolicyTargetGridsHash = Policy->GetWorldPartitionTargetGridsHash();

			bool bHasUnbounded = false;
			PCGHiGenGrid::FSizeArray GridSizes;
			PCGHelpers::GetGenerationGridSizes(OriginalExecutionState.GetGraph(), Inputs.PCGWorldActor, GridSizes, bHasUnbounded);

			if (GridSizes.IsEmpty() && !bHasUnbounded)
			{
				continue;
			}

			// For each relevant grid index, the largest grid size that has been marked in the runtime gen policy as depending on world streaming.
			PCGHiGenGrid::FSizeArray WorldStreamingQueryGridSizes;

			if (bCheckStreaming)
			{
				WorldStreamingQueryGridSizes.SetNumUninitialized(GridSizes.Num());

				for (int GridIndex = 0; GridIndex < GridSizes.Num(); ++GridIndex)
				{
					WorldStreamingQueryGridSizes[GridIndex] = PCGHiGenGrid::UninitializedGridSize();

					PCGHiGenGrid::FSizeArray ParentGridsDescending;
					OriginalExecutionState.GetGraph()->GetParentGridSizes(GridSizes[GridIndex], ParentGridsDescending);
					for (uint32 ParentGridSize : ParentGridsDescending)
					{
						if (Policy->DoesGridDependOnWorldStreaming(ParentGridSize) && ensure(ParentGridSize > GridSizes[GridIndex]))
						{
							WorldStreamingQueryGridSizes[GridIndex] = ParentGridSize;
							break;
						}
					}

					if (WorldStreamingQueryGridSizes[GridIndex] == PCGHiGenGrid::UninitializedGridSize() && Policy->DoesGridDependOnWorldStreaming(GridSizes[GridIndex]))
					{
						WorldStreamingQueryGridSizes[GridIndex] = GridSizes[GridIndex];
					}
				}
			}
			else
			{
				WorldStreamingQueryGridSizes.SetNumZeroed(GridSizes.Num());
			}

			const uint32 MaxGrid = bHasUnbounded ? PCGHiGenGrid::UnboundedGridSize() : GridSizes[0];
			const double MaxGenerationRadius = OriginalExecutionState.GetGenerationRadiusFromGrid(MaxGrid);

			for (const IPCGGenSourceBase* GenSource : *Inputs.GenSources)
			{
				const PCGRuntimeGenChangeDetection::FDetectionInputs ChangeDetectionInputs =
				{
					OriginalExecutionSource,
					GenSource,
					OriginalExecutionState.GetGenerationRadii(),
					PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationRadiusMultiplier.GetValueOnGameThread(),
					Policy->CullsBasedOnDirection(),
					GridSizes.IsEmpty() ? PCGHiGenGrid::UnboundedGridSize() : GridSizes.Last(),
				};

				if (bDoChangeDetection && !ChangeDetector.IsCellScanRequired(ChangeDetectionInputs))
				{
					continue;
				}

				// If any runtime gen source changed, trigger a cleanup scan. Simple heuristic, could break this down per-component in the future.
				bCleanupScanRequired = true;

#if PCG_RGS_ONSCREENDEBUGMESSAGES
				++PCGRuntimeGenSchedulerHelpers::Stats.GridCellScanCounter;
#endif

				const TOptional<FVector> GenSourcePositionOptional = GenSource->GetPosition();
				if (!GenSourcePositionOptional.IsSet())
				{
					continue;
				}

				const FVector GenSourcePosition = GenSourcePositionOptional.GetValue();

#if UE_ENABLE_DEBUG_DRAWING
				if (bDebugDrawGenerationSources)
				{
					if (const UObject* GenSourceObject = Cast<UObject>(GenSource))
					{
						TOptional<FVector> Direction = GenSource->GetDirection();

						DrawDebugString(Context.World, Direction ? (GenSourcePosition + 100.0f * Direction.GetValue()) : GenSourcePosition, GenSourceObject->GetName(), /*TestBaseActor=*/nullptr, FColor::Red, /*Duration=*/0.0f);
					}

					for (uint32 GridSize : GridSizes)
					{
						ensure(PCGHiGenGrid::IsValidGridSize(GridSize));

						const double GenerationRadius = OriginalExecutionState.GetGenerationRadiusFromGrid(GridSize);

						DrawDebugSphere(Context.World, GenSourcePosition, GenerationRadius, 64, FColor::Red, /*bPersistentLines=*/false, /*LifeTime=*/0.0f);
					}
				}
#endif // UE_ENABLE_DEBUG_DRAWING

				const FBox OriginalSourceBounds = OriginalExecutionState.GetBounds();
				const bool bIs2DGrid = OriginalExecutionState.Use2DGrid();

				FVector ModifiedGenSourcePosition = GenSourcePosition;
				if (bIs2DGrid)
				{
					ModifiedGenSourcePosition.Z = OriginalSourceBounds.Min.Z;
				}

				const double DistanceSquared = OriginalSourceBounds.ComputeSquaredDistanceToPoint(ModifiedGenSourcePosition);

				// If GenSource is not within range of the execution source then skip it.
				if (DistanceSquared > MaxGenerationRadius * MaxGenerationRadius)
				{
					if (bDoChangeDetection)
					{
						ChangeDetector.OnCellsScanned(ChangeDetectionInputs);
					}

					continue;
				}

				if (bHasUnbounded)
				{
					// Poll for streaming to complete on the unbounded grid.
					if (bCheckStreaming && Policy->DoesGridDependOnWorldStreaming(PCGHiGenGrid::UnboundedGridSize()))
					{
						const FVector StreamingLocation = OriginalSourceBounds.GetCenter();
						const FVector StreamingSize = OriginalSourceBounds.GetSize();

						if (!IsWorldStreamingComplete(StreamingLocation, FMath::Max(StreamingSize.X, StreamingSize.Y), PolicyTargetGrids, PolicyTargetGridsHash))
						{
							UE_LOGF(LogPCG, VeryVerbose, "Partitioned original component '%ls' rejected as world is not fully loaded.", *OriginalExecutionState.GetDebugName());
							continue;
						}
					}
					
					// Ignore execution sources that have already been generated or marked for generation. Unbounded grid size means not-partitioned.
					FGridGenerationKey Key(PCGHiGenGrid::UnboundedGridSize(), FIntVector(0), OriginalExecutionSource);
					if (!Inputs.GeneratedSources->Contains(Key) &&
						(!Policy || Policy->ShouldGenerate(GenSource, OriginalSourceBounds, bIs2DGrid)))
					{
						check(Key.GetGridDescriptor().Is2DGrid() == bIs2DGrid);
						AddSourceToGenerate(Key, GenSource, Policy, OriginalSourceBounds, bIs2DGrid);
					}
				}

				bool bAnyCellGenerationBlockedByStreaming = false;

				// TODO: once one of the larger grid sizes is out of range, we can forego checking any smaller grid sizes. they can't possibly be closer!
				// This assumes we enforce generation radii to increase monotonically.
				for (int GridIndex = 0; GridIndex < GridSizes.Num(); ++GridIndex)
				{
					const uint32 GridSize = GridSizes[GridIndex];

					const FIntVector GenSourceGridPosition = UPCGActorHelpers::GetCellCoord(GenSourcePosition, GridSize, bIs2DGrid);
					const double GenerationRadius = OriginalExecutionState.GetGenerationRadiusFromGrid(GridSize);
					const int32 GridRadius = FMath::CeilToInt32(GenerationRadius / GridSize); // Radius discretized to # of grid cells.
					const int32 VerticalGridRadius = bIs2DGrid ? 0 : GridRadius; // Flatten the vertical grid radius in the 2D case.

					const double HalfGridSize = GridSize / 2.0f;
					FVector HalfExtent(HalfGridSize, HalfGridSize, HalfGridSize);

					if (bIs2DGrid)
					{
						// In case of 2D grid, it's like the actor has infinite bounds on the Z axis.
						HalfExtent.Z = HALF_WORLD_MAX1;
					}

					// TODO: Perhaps rasterize sphere instead of walking a naive cube. although maybe the perf on that isn't worthwhile.
					for (int32 Z = GenSourceGridPosition.Z - VerticalGridRadius; Z <= GenSourceGridPosition.Z + VerticalGridRadius; ++Z)
					{
						for (int32 Y = GenSourceGridPosition.Y - GridRadius; Y <= GenSourceGridPosition.Y + GridRadius; ++Y)
						{
							for (int32 X = GenSourceGridPosition.X - GridRadius; X <= GenSourceGridPosition.X + GridRadius; ++X)
							{
								FIntVector GridCoords(X, Y, Z);
								FGridGenerationKey Key(GridSize, GridCoords, OriginalExecutionSource);

								// Ignore execution sources that have already been generated or marked for generation.
								if (Inputs.GeneratedSources->Find({ GridSize, GridCoords, OriginalExecutionSource }))
								{
									continue;
								}

								const FVector Center = FVector(GridCoords.X + 0.5, GridCoords.Y + 0.5, GridCoords.Z + 0.5) * GridSize;
								const FBox CellBounds(Center - HalfExtent, Center + HalfExtent);

								// Overlap cell with the partitioned execution source.
								const FBox IntersectedBounds = OriginalSourceBounds.Overlap(CellBounds);
								if (!IntersectedBounds.IsValid || IntersectedBounds.GetVolume() <= UE_DOUBLE_SMALL_NUMBER)
								{
									continue;
								}

								if (Key.GetGridDescriptor().Is2DGrid())
								{
									ModifiedGenSourcePosition.Z = IntersectedBounds.Min.Z;
								}

								// Verify the grid cell actually lies within the generation radius.
								// TODO: this is no longer necessary if we rasterize the sphere instead.
								const double LocalDistanceSquared = IntersectedBounds.ComputeSquaredDistanceToPoint(ModifiedGenSourcePosition);
								if (LocalDistanceSquared <= GenerationRadius * GenerationRadius && 
									Policy->ShouldGenerate(GenSource, IntersectedBounds, Key.GetGridDescriptor().Is2DGrid()))
								{
									bool bStreamingComplete = true;

									if (WorldStreamingQueryGridSizes[GridIndex] == GridSize)
									{
										bStreamingComplete = IsWorldStreamingComplete(Center, GridSize, PolicyTargetGrids, PolicyTargetGridsHash);

										if (!bStreamingComplete)
										{
											UE_LOGF(LogPCG, VeryVerbose, "Cell %u (%d, %d, %d) rejected as world is not fully loaded.",
												GridSize, GridCoords.X, GridCoords.Y, GridCoords.Z);
										}
									}
									else if (WorldStreamingQueryGridSizes[GridIndex] != PCGHiGenGrid::UninitializedGridSize())
									{
										// Check world loaded status using the pre-calculated largest parent grid that depends on world streaming.
										const uint32 ParentGridSize = WorldStreamingQueryGridSizes[GridIndex];
										const FVector ParentCenter = UPCGActorHelpers::GetCellCenter(Center, ParentGridSize, Key.GetGridDescriptor().Is2DGrid());

										bStreamingComplete = IsWorldStreamingComplete(ParentCenter, ParentGridSize, PolicyTargetGrids, PolicyTargetGridsHash);

										if (!bStreamingComplete)
										{
											UE_LOGF(LogPCG, VeryVerbose, "Cell %u (%d, %d, %d) rejected as parent on grid size %u is not fully loaded.",
												GridSize, GridCoords.X, GridCoords.Y, GridCoords.Z, ParentGridSize);
										}
									}

									if (bStreamingComplete)
									{
										AddSourceToGenerate(Key, GenSource, Policy, IntersectedBounds, Key.GetGridDescriptor().Is2DGrid());
									}
									else
									{
										bAnyCellGenerationBlockedByStreaming = true;
									}
								}
							}
						}
					}
				}

				// Require repeated scanning if we are waiting for level to stream.
				// todo_pcg: We can definitely do better than this. We should probably have a separate list of keys for execution sources that are awaiting streaming so
				// that we can keep track of these specifically rather than requiring a full rescan.
				if (bDoChangeDetection && !bAnyCellGenerationBlockedByStreaming)
				{
					ChangeDetector.OnCellsScanned(ChangeDetectionInputs);
				}
			}
		}
	}

	// Collect all non-partitioned execution sources.
	for (IPCGGraphExecutionSource* OriginalExecutionSource : Inputs.AllNonPartitionedExecutionSources)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::CollectNonPartitionedSources);

		if (!OriginalExecutionSource)
		{
			continue;
		}

		// Skip PCG components in actor-componentless mode. They should have registered a UPCGRuntimeGenExecutionSource that will handle generation.
		if (UPCGComponent* OriginalComponent = Cast<UPCGComponent>(OriginalExecutionSource); OriginalComponent && OriginalComponent->UseActorComponentlessGeneration())
		{
			continue;
		}

		const IPCGGraphExecutionState& OriginalExecutionState = OriginalExecutionSource->GetExecutionState();

		if (!OriginalExecutionState.GetGraph() || !OriginalExecutionState.IsActive())
		{
			continue;
		}

		// todo_pcg: Worth adding change detection here? Decide on quantization - perhaps a proportion of minimum volume extent or such?
		
		// The generation key for a non-partitioned execution source should always have unbounded grid size and 0,0,0 cell coord.
		if (Inputs.GeneratedSources->Contains({ PCGHiGenGrid::UnboundedGridSize(), FIntVector(0), OriginalExecutionSource }))
		{
			// todo_pcg: Needs change detection to throttle clean up scans, for now just always cleanup if any runtime gen unbounded execution source is present.
			bCleanupScanRequired = true;

			continue;
		}

		const UPCGSchedulingPolicyBase* Policy = OriginalExecutionState.GetRuntimeGenSchedulingPolicy();

		// TODO: For each execution domain (for now only GenAtRuntime/dynamic), assuming we run Preview through this scheduler, which it seems like we will.
		if (OriginalExecutionState.IsManagedByRuntimeGenSystem() && ensure(Policy))
		{
			// todo_pcg: Needs change detection to throttle clean up scans, for now just always cleanup if any runtime gen unbounded execution source is present.
			bCleanupScanRequired = true;

			const FBox OriginalSourceBounds = OriginalExecutionState.GetBounds();

			// Poll for streaming to complete.
			if (bCheckStreaming && Policy->DoesGridDependOnWorldStreaming(PCGHiGenGrid::UnboundedGridSize()))
			{
				const TSet<FName>& PolicyTargetGrids = Policy->GetWorldPartitionTargetGrids();
				const uint64 PolicyTargetGridsHash = Policy->GetWorldPartitionTargetGridsHash();

				const FVector StreamingLocation = OriginalSourceBounds.GetCenter();
				const FVector StreamingSize = OriginalSourceBounds.GetSize();

				if (!IsWorldStreamingComplete(StreamingLocation, FMath::Max(StreamingSize.X, StreamingSize.Y), PolicyTargetGrids, PolicyTargetGridsHash))
				{
					UE_LOGF(LogPCG, VeryVerbose, "Non-partitioned original component '%ls' rejected as world is not fully loaded.", *OriginalExecutionState.GetDebugName());
					continue;
				}
			}

			// Unbounded will grab the base GenerationRadius used for non-partitioned and unbounded.
			const double MaxGenerationRadius = OriginalExecutionState.GetGenerationRadiusFromGrid(PCGHiGenGrid::UnboundedGridSize());

			for (const IPCGGenSourceBase* GenSource : *Inputs.GenSources)
			{
				const TOptional<FVector> GenSourcePositionOptional = GenSource->GetPosition();
				if (!GenSourcePositionOptional.IsSet())
				{
					continue;
				}

				const FVector GenSourcePosition = GenSourcePositionOptional.GetValue();

				FVector ModifiedGenSourcePosition = GenSourcePosition;
				if (OriginalExecutionState.Use2DGrid())
				{
					ModifiedGenSourcePosition.Z = OriginalSourceBounds.Min.Z;
				}

				const double DistanceSquared = OriginalSourceBounds.ComputeSquaredDistanceToPoint(ModifiedGenSourcePosition);

				// Max radius for a non-partitioned execution source is just the base GenerationRadius.
				if (DistanceSquared <= MaxGenerationRadius * MaxGenerationRadius && 
					(!Policy || Policy->ShouldGenerate(GenSource, OriginalSourceBounds, /*bUse2DGrid=*/false)))
				{
					// Unbounded grid size means not-partitioned.
					FGridGenerationKey Key(PCGHiGenGrid::UnboundedGridSize(), FIntVector(0), OriginalExecutionSource);
					AddSourceToGenerate(Key, GenSource, Policy, OriginalSourceBounds, /*bUse2DGrid=*/false);
				}
			}
		}
	}
}

void FPCGRuntimeGenScheduler::TickCleanup(const TSet<IPCGGenSourceBase*>& InGenSources, const APCGWorldActor* InPCGWorldActor, double InEndTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::TickCleanup);

	check(InPCGWorldActor);

	auto CheckIfAllGenSourcesWantToCleanup = [&InGenSources](const UPCGSchedulingPolicyBase* Policy, const FPCGGridDescriptor& GridDescriptor, const FBox& GridBounds, const double& CleanupRadiusSquared)
	{
		bool bAllGenSourcesWantToCleanup = true;

		for (const IPCGGenSourceBase* GenSource : InGenSources)
		{
			if (!ensure(GenSource))
			{
				continue;
			}

			const TOptional<FVector> GenSourcePositionOptional = GenSource->GetPosition();
			if (!GenSourcePositionOptional.IsSet())
			{
				continue;
			}

			FVector GenSourcePosition = GenSourcePositionOptional.GetValue();

			// Only consider 2D distance when using a 2D grid.
			if (GridDescriptor.Is2DGrid())
			{
				GenSourcePosition.Z = GridBounds.Min.Z;
			}

			const double SquaredDistToGenSource = GridBounds.ComputeSquaredDistanceToPoint(GenSourcePosition);

			// If the distance to the gen source is greater than the cleanup radius, it means this generation source votes for the execution source to be cleaned up.
			// Otherwise, the gen source might still vote for culling regardless.
			if (SquaredDistToGenSource <= CleanupRadiusSquared && 
				(!Policy || !Policy->ShouldCull(GenSource, GridBounds, GridDescriptor.Is2DGrid())))
			{
				bAllGenSourcesWantToCleanup = false;
				break;
			}
		}

		return bAllGenSourcesWantToCleanup;
	};

	TArray<FGridGenerationKey> GeneratedSourcesArray = GeneratedSources.Array();
	const int32 NumGeneratedSources = GeneratedSourcesArray.Num();

	// Generated Entry Key, Local/Generated Source
	using ExecutionSourceToClean = TTuple<FGridGenerationKey, IPCGGraphExecutionSource*>;
	TArray<ExecutionSourceToClean, TInlineAllocator<256>> SourcesToClean;
	TArray<FGridGenerationKey, TInlineAllocator<16>> InvalidKeys;

	SourcesToClean.SetNumUninitialized(NumGeneratedSources);
	InvalidKeys.SetNumUninitialized(NumGeneratedSources);

	std::atomic<int> SourcesToCleanCounter = 0;
	std::atomic<int> InvalidKeysCounter = 0;

	ParallelFor(
		TEXT("CleanupLoop"),
		NumGeneratedSources,
		/*MinBatchSize*/6, // Optimal value found in testing.
		[this, &GeneratedSourcesArray, &SourcesToClean, &InvalidKeys, &SourcesToCleanCounter, &InvalidKeysCounter, &CheckIfAllGenSourcesWantToCleanup](int32 Index)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SelectSourceForCleanup);
			LLM_SCOPE_BYTAG(PCG);

			const FGridGenerationKey& GenerationKey = GeneratedSourcesArray[Index];

			if (!GenerationKey.IsValid())
			{
				const int WriteIndex = InvalidKeysCounter.fetch_add(1);
				InvalidKeys[WriteIndex] = GenerationKey;
				return;
			}

			const FPCGGridDescriptor GridDescriptor = GenerationKey.GetGridDescriptor();
			const FIntVector& GridCoords = GenerationKey.GetGridCoords();
			IPCGGraphExecutionSource* OriginalSource = GenerationKey.GetOriginalSource();
			check(OriginalSource);

			const UPCGSchedulingPolicyBase* Policy = OriginalSource->GetExecutionState().GetRuntimeGenSchedulingPolicy();
			ensure(Policy);

			const double CleanupRadius = OriginalSource->GetExecutionState().GetCleanupRadiusFromGrid(GridDescriptor.GetGridSize());
			const double CleanupRadiusSquared = CleanupRadius * CleanupRadius;

			// If the Grid is unbounded, we have a non-partitioned or unbounded execution source.
			if (GridDescriptor.IsUnboundedGrid())
			{
				if (!OriginalSource->GetExecutionState().IsActive())
				{
					const int WriteIndex = SourcesToCleanCounter.fetch_add(1);
					SourcesToClean[WriteIndex] = { GenerationKey, OriginalSource };
					return;
				}

				// If the component is still generated while in actor-componentless mode, clean up!
				if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(OriginalSource); PCGComponent && PCGComponent->UseActorComponentlessGeneration())
				{
					const int WriteIndex = SourcesToCleanCounter.fetch_add(1);
					SourcesToClean[WriteIndex] = { GenerationKey, OriginalSource };
					return;
				}

				const FBox GridBounds = GenerationKey.GetCachedBounds().IsSet() ? GenerationKey.GetCachedBounds().GetValue() : OriginalSource->GetExecutionState().GetBounds();

				// Only clean up if all generation sources agreed to clean up.
				if (CheckIfAllGenSourcesWantToCleanup(Policy, GridDescriptor, GridBounds, CleanupRadiusSquared))
				{
					const int WriteIndex = SourcesToCleanCounter.fetch_add(1);
					SourcesToClean[WriteIndex] = { GenerationKey, OriginalSource };
					return;
				}
			}
			// Otherwise, we have a local execution source.
			else
			{
				IPCGGraphExecutionSource* LocalSource = GenerationKey.GetCachedLocalSource();

				if (!LocalSource)
				{
					LocalSource = OriginalSource->GetExecutionState().GetLocalSource(GridDescriptor, GridCoords);
				}

				if (!LocalSource || !OriginalSource->GetExecutionState().IsActive())
				{
					const int WriteIndex = SourcesToCleanCounter.fetch_add(1);
					SourcesToClean[WriteIndex] = { GenerationKey, LocalSource };
					return;
				}

				// If the component is still generated while in actor-componentless mode, clean up!
				if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(OriginalSource); PCGComponent && PCGComponent->UseActorComponentlessGeneration())
				{
					const int WriteIndex = SourcesToCleanCounter.fetch_add(1);
					SourcesToClean[WriteIndex] = { GenerationKey, LocalSource };
					return;
				}

				const FBox GridBounds = GenerationKey.GetCachedBounds().IsSet() ? GenerationKey.GetCachedBounds().GetValue() : LocalSource->GetExecutionState().GetBounds();

				// Only clean up if all generation sources agreed to clean up.
				if (CheckIfAllGenSourcesWantToCleanup(Policy, GridDescriptor, GridBounds, CleanupRadiusSquared))
				{
					const int WriteIndex = SourcesToCleanCounter.fetch_add(1);
					SourcesToClean[WriteIndex] = { GenerationKey, LocalSource };
					return;
				}
			}
		});

	const int NumInvalidKeys = InvalidKeysCounter.load();
	const int NumSourcesToClean = SourcesToCleanCounter.load();

	for (int I = 0; I < NumInvalidKeys; ++I)
	{
		GeneratedSources.Remove(InvalidKeys[I]);
	}

	int CleanupIndex = 0;
	for (; CleanupIndex < NumSourcesToClean; ++CleanupIndex)
	{
		const ExecutionSourceToClean& SourceToClean = SourcesToClean[CleanupIndex];
		CleanupSource(SourceToClean.Get<0>(), SourceToClean.Get<1>());

		if (FPlatformTime::Seconds() >= InEndTime)
		{
			UE_LOGF(LogPCG, Verbose, "FPCGRuntimeGenScheduler: Time budget exceeded, aborted after cleaning up %d / %d components", (CleanupIndex + 1), NumSourcesToClean);
			break;
		}
	}

	if (CleanupIndex == NumSourcesToClean)
	{
		// Record that we completed a cleanup scan.
		bCleanupScanRequired = false;
	}
	else
	{
		// Continue next frame.
		TimeToNextTick = 0.0;
	}
}

void FPCGRuntimeGenScheduler::TickScheduleGeneration(TMap<FGridGenerationKey, double>& InOutSourcesToGenerate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::TickScheduleGeneration);

	check(Subsystem && TrackingManager);

	// Count number of currently generating execution sources
	int NumGenerating = 0;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::TickScheduleGeneration::CountingCurrentlyGenerating);
		for (const FGridGenerationKey& Key : GeneratedSources)
		{
			if (!Key.IsValid())
			{
				continue;
			}

			const IPCGGraphExecutionSource* ExecutionSource = (Key.GetGridDescriptor().GetGridSize() == PCGHiGenGrid::UnboundedGridSize()) ? Key.GetOriginalSource() : Key.GetCachedLocalSource();

			if (ExecutionSource && ExecutionSource->GetExecutionState().IsGenerating())
			{
				++NumGenerating;
			}
		}
	}

	const int MaxNumGenerating = FMath::Max(1, PCGRuntimeGenSchedulerHelpers::CVarNumGeneratingComponentsAtSameTime.GetValueOnAnyThread());
	const int GeneratingCapacity = FMath::Max(0, MaxNumGenerating - NumGenerating);

	if (GeneratingCapacity == 0)
	{
		// No capacity to start generating new execution sources, try next frame.
		return;
	}

	GeneratedSources.Reserve(GeneratedSources.Num() + InOutSourcesToGenerate.Num());

	// Sources that we couldn't generate due to missed capacity. Generation will be attempted again next frame.
	TMap<FGridGenerationKey, double> MissedSources;
	MissedSources.Reserve(FMath::Max(0, InOutSourcesToGenerate.Num() - GeneratingCapacity));

	for (auto It = InOutSourcesToGenerate.CreateConstIterator(); It; ++It)
	{
		if (NumGenerating >= MaxNumGenerating)
		{
			// We've filled generation capacity, record the remaining execution sources to try again next frame.
			for (; It; ++It)
			{
				MissedSources.Add(*It);
			}

			break;
		}

		const FGridGenerationKey& Key = It.Key();
		const double Priority = It.Value();

		const FPCGGridDescriptor GridDescriptor = Key.GetGridDescriptor();

		const FIntVector GridCoords = Key.GetGridCoords();
		IPCGGraphExecutionSource* OriginalSource = Key.GetOriginalSource();
		if (!OriginalSource)
		{
			UE_LOGF(LogPCG, Verbose, "TickScheduleGeneration: Invalid original component in an InOutSourcesToGenerate entry, generation will be skipped.");
			continue;
		}

		IPCGGraphExecutionState& OriginalExecutionState = OriginalSource->GetExecutionState();

		// If the Grid is unbounded, we have a non-partitioned or unbounded execution source.
		if (GridDescriptor.IsUnboundedGrid())
		{
			if (!OriginalExecutionState.IsGenerating())
			{
				if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread())
				{
					UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] GENERATE: '%ls' (priority %lf)", *OriginalExecutionState.GetDebugName(), Priority);
				}

#if PCG_RGS_ONSCREENDEBUGMESSAGES
				++PCGRuntimeGenSchedulerHelpers::Stats.GenerateCallCounter;
#endif

				IPCGGraphExecutionState::FGenerateParams GenerateParams
				{
					.bEvenIfAlreadyGenerated = true,
					.bGenerateLocalSources = false
				};

				OriginalSource->GetExecutionState().Generate(GenerateParams);
			}
		}
		// Otherwise we have a local execution source.
		else
		{
			IPCGGraphExecutionSource* LocalSource = OriginalExecutionState.GetLocalSource(GridDescriptor, GridCoords);

			if (!LocalSource)
			{
				// @todo_pcg: CreateLocalSource could be moved into the graph execution state interface and generalized for all execution sources.
				if (UPCGRuntimeGenExecutionSource* OriginalRuntimeGenSource = Cast<UPCGRuntimeGenExecutionSource>(OriginalSource))
				{
					LocalSource = OriginalRuntimeGenSource->CreateLocalSource(GridDescriptor.GetGridSize(), GridCoords);
				}
				else if (UPCGComponent* OriginalComponent = Cast<UPCGComponent>(OriginalSource))
				{
					// Grab local component and PA if they exist already.
					TObjectPtr<APCGPartitionActor> PartitionActor = LocalSource ? LocalSource->GetExecutionState().GetTypedTarget<APCGPartitionActor>() : nullptr;

					if (!LocalSource || !ensure(PartitionActor))
					{
						// Local component & PA do not exist, create them.
						if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnablePooling.GetValueOnAnyThread())
						{
							// Get RuntimeGenPA from pool.
							PartitionActor = GetPartitionActorFromPool(GridDescriptor, GridCoords);

							if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread())
							{
								UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] UNPOOL PARTITION ACTOR: '%ls' (priority %lf, %d remaining out of %d)",
									*APCGPartitionActor::GetPCGPartitionActorName(GridDescriptor, GridCoords),
									Priority,
									PartitionActorPool.Num(),
									PartitionActorPoolSize);
							}
						}
						else
						{
							if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread())
							{
								UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] CREATE PARTITION ACTOR: '%ls' (priority %lf)",
									*APCGPartitionActor::GetPCGPartitionActorName(GridDescriptor, GridCoords),
									Priority);
							}

							// Find or Create RuntimeGenPA.
							PartitionActor = Subsystem->FindOrCreatePCGPartitionActor(
								GridDescriptor,
								GridCoords,
								/*bCanCreateActor=*/true,
								PCGRuntimeGenSchedulerHelpers::CVarHideActorsFromOutliner.GetValueOnAnyThread());
						}

						if (!ensure(PartitionActor))
						{
							continue;
						}

						// Update component mapping for this PA (add local component).
						{
							PCG::TUniqueScopeLock WriteLock(TrackingManager->ComponentToPartitionActorsMapLock);
							TSet<TObjectPtr<APCGPartitionActor>>* PartitionActorsPtr = TrackingManager->ComponentToPartitionActorsMap.Find(OriginalComponent);

							if (!PartitionActorsPtr)
							{
								PartitionActorsPtr = &TrackingManager->ComponentToPartitionActorsMap.Emplace(OriginalComponent);
							}

							// Log this original component before setting up the PA, so that we early out from RefreshExecutionSource if it gets called
							// in the AddGraphInstance call below.
							OriginalSourceBeingRegistered = OriginalComponent;

							PartitionActor->AddGraphInstance(OriginalComponent);

							OriginalSourceBeingRegistered = nullptr;

							PartitionActorsPtr->Add(PartitionActor);
						}

						// Create local component.
						LocalSource = PartitionActor->GetLocalComponent(OriginalComponent);
					}
				}
			}

			if (ensure(LocalSource) && !LocalSource->GetExecutionState().IsGenerating())
			{
				if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread())
				{
					UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] GENERATE: '%ls' (priority %lf)", *LocalSource->GetExecutionState().GetDebugName(), Priority);
				}

				// Higen graphs may have data links from original execution source to local execution sources. The original execution source will be given a higher priority than local
				// execution sources and will start generating first. If it is currently generating, local execution source needs to take a dependency to ensure execution completes.
				TArray<FPCGTaskId> Dependencies;
				if (OriginalExecutionState.IsGenerating() && OriginalExecutionState.GetGraph() && OriginalExecutionState.GetGraph()->IsHierarchicalGenerationEnabled())
				{
					const FPCGTaskId TaskId = OriginalExecutionState.GetGenerationTaskId();

					if (TaskId != InvalidPCGTaskId)
					{
						Dependencies.Add(TaskId);
					}
				}

#if PCG_RGS_ONSCREENDEBUGMESSAGES
				++PCGRuntimeGenSchedulerHelpers::Stats.GenerateCallCounter;
#endif

				// Force to refresh if the execution source is already generated.
				IPCGGraphExecutionState::FGenerateParams GenerateParams;
				GenerateParams.bEvenIfAlreadyGenerated = true;
				GenerateParams.Dependencies = MoveTemp(Dependencies);

				LocalSource->GetExecutionState().Generate(GenerateParams);

				Key.SetCachedLocalSource(LocalSource);
			}
		}

		GeneratedSources.Add(Key);
		++NumGenerating;
	}

	// Set any missed execution sources to retry next frame (or clear the pending execution sources if no execution sources were missed).
	InOutSourcesToGenerate = MoveTemp(MissedSources);

#if PCG_RGS_ONSCREENDEBUGMESSAGES
	PCGRuntimeGenSchedulerHelpers::Stats.NumGeneratingComponents = NumGenerating;
#endif
}

void FPCGRuntimeGenScheduler::TickRequestVirtualTexturePriming(const TSet<IPCGGenSourceBase*>& InGenSources)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::TickRequestVirtualTexturePriming);

	TMap<TSoftObjectPtr<URuntimeVirtualTexture>, TArray<URuntimeVirtualTextureComponent*>> VirtualTextureToComponents;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FindVirtualTextureComponents);

		// @todo_pcg: We could avoid polling for VT components every frame if they were registered somewhere instead.
		TArray<UObject*> FoundComponents;
		GetObjectsOfClass(URuntimeVirtualTextureComponent::StaticClass(), FoundComponents, /*bIncludeDerivedClasses=*/false, RF_ClassDefaultObject, EInternalObjectFlags::Garbage);

		for (UObject* FoundComponent : FoundComponents)
		{
			if (FoundComponent && FoundComponent->GetWorld() == Context.World)
			{
				URuntimeVirtualTextureComponent* VirtualTextureComponent = Cast<URuntimeVirtualTextureComponent>(FoundComponent);
				URuntimeVirtualTexture* VirtualTexture = VirtualTextureComponent ? VirtualTextureComponent->GetVirtualTexture() : nullptr;

				if (VirtualTexture)
				{
					TArray<URuntimeVirtualTextureComponent*>& VirtualTextureComponents = VirtualTextureToComponents.FindOrAdd(VirtualTexture);
					VirtualTextureComponents.Add(VirtualTextureComponent);
				}
			}
		}
	}

	if (VirtualTextureToComponents.IsEmpty())
	{
		return;
	}

	// Clear out any entries for components that no longer resolve.
	for (auto It = CachedPrimingInfos.CreateIterator(); It; ++It)
	{
		if (!It.Key().ResolveObjectPtr())
		{
			It.RemoveCurrent();
		}
	}

	for (const auto& [OriginalComponentKey, PrimingInfos] : CachedPrimingInfos)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RequestVirtualTexturePrimingForComponent);

		const UPCGComponent* OriginalComponent = OriginalComponentKey.ResolveObjectPtr();
		check(OriginalComponent);
		const UPCGGraph* Graph = OriginalComponent->GetGraph();

		const FBox OriginalComponentBounds = OriginalComponent->GetGridBounds();

		for (const FPCGVirtualTexturePrimingInfo& PrimingInfo : PrimingInfos)
		{
			if (!PrimingInfo.VirtualTexture || PrimingInfo.WorldTexelSize < PCGRuntimeGenSchedulerConstants::MinWorldVirtualTextureTexelSize)
			{
				continue;
			}

			TArray<URuntimeVirtualTextureComponent*>* VirtualTextureComponents = VirtualTextureToComponents.Find(PrimingInfo.VirtualTexture);

			if (!VirtualTextureComponents)
			{
				continue;
			}

			for (URuntimeVirtualTextureComponent* VirtualTextureComponent : *VirtualTextureComponents)
			{
				check(VirtualTextureComponent);

				const uint32 GridSize = PCGHiGenGrid::GridToGridSize(PrimingInfo.Grid) * (Graph ? Graph->GetGridSizeMultiplier() : 1.0);

				const double PrimingRadius = OriginalComponent->GetGenerationRadiusFromGrid(GridSize) + GridSize;

				for (const IPCGGenSourceBase* GenSource : InGenSources)
				{
					const TOptional<FVector> GenSourcePositionOptional = GenSource->GetPosition();

					if (!GenSourcePositionOptional.IsSet())
					{
						continue;
					}

					FVector GenSourcePosition = GenSourcePositionOptional.GetValue();

					if (OriginalComponent->Use2DGrid())
					{
						GenSourcePosition.Z = OriginalComponentBounds.GetCenter().Z;
					}

					const FSphere PrimingBounds = FSphere(GenSourcePosition, PrimingRadius);

					if (OriginalComponentBounds.Intersect(FBox(PrimingBounds)))
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(RequestVirtualTexturePreload);

						const double BoundsMaxExtent = FMath::Max(VirtualTextureComponent->Bounds.BoxExtent.X, VirtualTextureComponent->Bounds.BoxExtent.Y);
						const int32 VirtualTextureSizeTexels = FMath::Max(1, PrimingInfo.VirtualTexture->GetSize());
						const int32 SizeTexelsLog2 = FMath::FloorLog2(VirtualTextureSizeTexels);
						const int32 RequestedNumTexels = FMath::Max(1, BoundsMaxExtent / PrimingInfo.WorldTexelSize);
						const int32 RequestedTexelsLog2 = FMath::FloorLog2(RequestedNumTexels);
						const int32 MipLevel = FMath::Max(0, SizeTexelsLog2 - RequestedTexelsLog2);

						VirtualTextureComponent->RequestPreload(PrimingBounds, MipLevel);

#if PCG_RGS_ONSCREENDEBUGMESSAGES
						++PCGRuntimeGenSchedulerHelpers::Stats.VTPreloadCounter;
#endif

						if (PCGRuntimeGenSchedulerHelpers::CVarDebugDrawTexturePrimingBounds.GetValueOnAnyThread())
						{
							check(Context.World);

							DrawDebugCylinder(
								Context.World,
								FVector(GenSourcePosition.X, GenSourcePosition.Y, OriginalComponentBounds.Min.Z),
								FVector(GenSourcePosition.X, GenSourcePosition.Y, OriginalComponentBounds.Max.Z),
								/*Radius=*/PrimingBounds.W,
								/*Segments=*/8,
								/*Color=*/FColor::Red,
								/*bPersistentLines=*/false,
								/*LifeTime=*/0.02f);
						}
					}
				}
			}
		}
	}
}

void FPCGRuntimeGenScheduler::TickCVars(const APCGWorldActor* InPCGWorldActor)
{
	if (bActorFlushRequested && Subsystem && Subsystem->GetPCGWorldActor())
	{
		CleanupLocalSources(Subsystem->GetPCGWorldActor());
		ResetPartitionActorPoolToSize(PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationBasePoolSize.GetValueOnAnyThread());
	}
	bActorFlushRequested = false;

	// If pooling has been disabled since last frame, we should destroy the pool.
	const bool bPoolingEnabled = PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnablePooling.GetValueOnAnyThread();

	if (bPoolingWasEnabledLastFrame && !bPoolingEnabled)
	{
		CleanupLocalSources(InPCGWorldActor);
		ResetPartitionActorPoolToSize(/*NewPoolSize=*/0);
	}

	bPoolingWasEnabledLastFrame = bPoolingEnabled;

	// Handle when the base PA PoolSize is modified. Cleanup all local execution sources and reset the pool with the correct number of PAs.
	if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnablePooling.GetValueOnAnyThread())
	{
		// Don't allow a pool size <= 0
		const uint32 BasePoolSize = FMath::Max(1, PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationBasePoolSize.GetValueOnAnyThread());

		if (BasePoolSizeLastFrame != BasePoolSize)
		{
			BasePoolSizeLastFrame = BasePoolSize;

			CleanupLocalSources(InPCGWorldActor);
			ResetPartitionActorPoolToSize(BasePoolSize);
		}
	}

#if WITH_EDITOR
	const bool bTreatEditorViewportAsGenSource = IPCGEditorModule::ShouldTreatViewportAsGenerationSource(InPCGWorldActor);
	
	if (bTreatEditorViewportAsGenSource != bTreatEditorViewportAsGenSourcePreviousFrame && Context.World && Context.World->IsEditorWorld())
	{
		UE_LOGF(LogPCG, Verbose, "Flushed change detection status after bTreatEditorViewportAsGenerationSource value change.");
		ChangeDetector.FlushCachedState();
	}

	bTreatEditorViewportAsGenSourcePreviousFrame = bTreatEditorViewportAsGenSource;
#endif // WITH_EDITOR
}

void FPCGRuntimeGenScheduler::OnOriginalComponentRegistered(UPCGComponent* InOriginalComponent)
{
	// Ensure we are non-local runtime managed component.
	if (!InOriginalComponent || !InOriginalComponent->IsManagedByRuntimeGenSystem() || InOriginalComponent->IsLocalComponent())
	{
		return;
	}

	// Enable the RT UV cache so it's warm before the first GPU ray trace dispatch.
	PCGRayTracingUVCache::RequestEnable_GameThread(Context.World ? Context.World->Scene : nullptr);

	// When an original/non-partitioned component is registered, we need to dirty the state.
	bContextDirty = true;

	TimeToNextTick = 0.0f;

#if WITH_EDITOR
	RegisterGraphParameterChangedEvent(InOriginalComponent);
#endif

	CacheVirtualTexturePrimingInfos(InOriginalComponent);

	// Try to register as an RG execution source. If this fails, we will treat the original PCG component as the execution source.
	RegisterOriginalRuntimeGenExecutionSource(InOriginalComponent);
}

void FPCGRuntimeGenScheduler::OnOriginalComponentUnregistered(UPCGComponent* InOriginalComponent)
{
	if (!InOriginalComponent || InOriginalComponent->IsLocalComponent())
	{
		return;
	}

	// When an original/non-partitioned component is unregistered, we need to dirty the state.
	bContextDirty = true;

	TimeToNextTick = 0.0f;

	// Make sure we perform a scan for cells that need cleanup on next tick.
	bCleanupScanRequired = true;

#if WITH_EDITOR
	UnregisterGraphParameterChangedEvent(InOriginalComponent);
#endif

	CachedPrimingInfos.Remove(InOriginalComponent);

	// Gather all generated components which originated from this original component.
	TSet<FGridGenerationKey> KeysToCleanup;
	for (FGridGenerationKey GenerationKey : GeneratedSources)
	{
		if (GenerationKey.GetOriginalSource() == InOriginalComponent)
		{
			KeysToCleanup.Add(GenerationKey);
		}
	}

	TArray<FGridGenerationKey, TInlineAllocator<16>> InvalidKeys;

	for (const FGridGenerationKey& GenerationKey : KeysToCleanup)
	{
		if (!GenerationKey.IsValid())
		{
			InvalidKeys.Add(GenerationKey);
			continue;
		}

		const FPCGGridDescriptor GridDescriptor = GenerationKey.GetGridDescriptor();
		const bool bIsOriginalSource = GridDescriptor.GetGridSize() == PCGHiGenGrid::UnboundedGridSize();

		// Get the generated execution source for this key (might be a local execution source).
		IPCGGraphExecutionSource* SourceToCleanup = bIsOriginalSource ? InOriginalComponent : InOriginalComponent->GetExecutionState().GetLocalSource(GridDescriptor, GenerationKey.GetGridCoords());

		// It is possible for a PartitionActor's LocalSource to have been cleaned up by the APCGPartitionActor::EndPlay call depending on the order in which actors get called
		if (SourceToCleanup)
		{
			CleanupSource(GenerationKey, SourceToCleanup);
		}
	}

	for (const FGridGenerationKey& InvalidKey : InvalidKeys)
	{
		GeneratedSources.Remove(InvalidKey);
	}

	CleanupRemainingSources(InOriginalComponent);

	UnregisterOriginalRuntimeGenExecutionSource(InOriginalComponent);
}

void FPCGRuntimeGenScheduler::OnOriginalComponentReplaced(const UPCGComponent* InOldComponent, UPCGComponent* InNewComponent)
{
	if (!InOldComponent || !InNewComponent)
	{
		return;
	}

	for (FGridGenerationKey& GenerationKey : GeneratedSources)
	{
		if (GenerationKey.GetOriginalSourceEvenIfGarbage() == InOldComponent)
		{
			GenerationKey.SetOriginalSource(InNewComponent);
		}
	}

	TObjectPtr<UPCGRuntimeGenExecutionSource> OriginalExecutionSource = nullptr;
	if (OriginalExecutionSources.RemoveAndCopyValue(InOldComponent, OriginalExecutionSource))
	{
		OriginalExecutionSources.Add(InNewComponent, OriginalExecutionSource);

		// Make sure the runtime gen execution source is using the new PCG component.
		OriginalExecutionSource->SetOwningSource(InNewComponent);
	}

	TArray<FPCGVirtualTexturePrimingInfo> PrimingInfos;
	if (CachedPrimingInfos.RemoveAndCopyValue(InOldComponent, PrimingInfos))
	{
		CachedPrimingInfos.Add(InNewComponent, PrimingInfos);
	}

#if WITH_EDITOR
	UnregisterGraphParameterChangedEvent(InOldComponent);
	RegisterGraphParameterChangedEvent(InNewComponent);
#endif
}

#if WITH_EDITOR
void FPCGRuntimeGenScheduler::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances)
{
	for (const TPair<UObject*, UObject*>& OldToNew : InOldToNewInstances)
	{
		UPCGComponent* OldComponent = Cast<UPCGComponent>(OldToNew.Key);
		UPCGComponent* NewComponent = Cast<UPCGComponent>(OldToNew.Value);

		// Replace all usage of OldComponent with NewComponent.
		OnOriginalComponentReplaced(OldComponent, NewComponent);
	}
}
#endif

FPCGRuntimeGenScheduler::FGridGenerationKey::FGridGenerationKey(uint32 InGridSize, const FIntVector& InGridCoords, IPCGGraphExecutionSource* InOriginalSource, IPCGGraphExecutionSource* InLocalSource)
	: GridSize(InGridSize)
	, GridCoords(InGridCoords)
	, OriginalSource(Cast<UObject>(InOriginalSource))
	, CachedLocalSource(InLocalSource)
{
	bUse2DGrid = InOriginalSource ? InOriginalSource->GetExecutionState().Use2DGrid() : true;

	if (InGridSize == PCGHiGenGrid::UnboundedGridSize())
	{
		const AActor* Owner = InOriginalSource ? InOriginalSource->GetExecutionState().GetTypedTarget<AActor>() : nullptr;
		const USceneComponent* RootComponent = Owner ? Owner->GetRootComponent() : nullptr;

		if (RootComponent && RootComponent->GetMobility() != EComponentMobility::Movable)
		{
			CachedBounds = InOriginalSource->GetExecutionState().GetBounds();
		}
	}
	else if (InLocalSource)
	{
		CachedBounds = InLocalSource->GetExecutionState().GetBounds();
	}
}

void FPCGRuntimeGenScheduler::CleanupRemainingSources(IPCGGraphExecutionSource* InOriginalSource)
{
	// Check for remaining PAs to cleanup
	// There are some cases when on Refresh of the original component that GeneratedSources doesn't contain all PAs anymore.
	// If the PA is not far enough to be cleaned up but no longer is considered within the Gen Source Radius for Generation.
	if (UPCGComponent* OriginalComponent = Cast<UPCGComponent>(InOriginalSource); OriginalComponent && InOriginalSource->GetExecutionState().IsManagedByRuntimeGenSystem())
	{
		TSet<TObjectPtr<APCGPartitionActor>> PartitionActors = TrackingManager->GetPCGComponentPartitionActorMappings(OriginalComponent);
		for (TObjectPtr<APCGPartitionActor> PartitionActor : PartitionActors)
		{
			if (IPCGGraphExecutionSource* ComponentToCleanup = PartitionActor->GetLocalComponent(OriginalComponent))
			{
				CleanupLocalSource(PartitionActor, ComponentToCleanup);
			}
		}
	}
}

void FPCGRuntimeGenScheduler::CleanupLocalSources(const APCGWorldActor* InPCGWorldActor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::CleanupLocalSources);

	check(InPCGWorldActor);

	// Generated Entry Key, Local/Generated Source
	using ExecutionSourceToClean = TTuple<FGridGenerationKey, IPCGGraphExecutionSource*>;
	TArray<ExecutionSourceToClean, TInlineAllocator<16>> SourcesToClean;

	TSet<IPCGGraphExecutionSource*> OriginalSources;
	OriginalSources.Reserve(16);

	TArray<FGridGenerationKey, TInlineAllocator<16>> InvalidKeys;

	// Find all generated local execution sources.
	for (const FGridGenerationKey& GenerationKey : GeneratedSources)
	{
		if (!GenerationKey.IsValid())
		{
			InvalidKeys.Add(GenerationKey);
			continue;
		}

		const FPCGGridDescriptor GridDescriptor = GenerationKey.GetGridDescriptor();
		const FIntVector& GridCoords = GenerationKey.GetGridCoords();
		IPCGGraphExecutionSource* OriginalSource = GenerationKey.GetOriginalSource();
		check(OriginalSource);
		OriginalSources.Add(OriginalSource);

		// Only operate on LocalSources.
		if (!GridDescriptor.IsUnboundedGrid())
		{
			IPCGGraphExecutionSource* LocalSource = OriginalSource->GetExecutionState().GetLocalSource(GridDescriptor, GridCoords);
			SourcesToClean.Add({ GenerationKey, LocalSource });
		}
	}

	for (const FGridGenerationKey& InvalidKey : InvalidKeys)
	{
		GeneratedSources.Remove(InvalidKey);
	}

	for (const ExecutionSourceToClean& SourceToClean : SourcesToClean)
	{
		CleanupSource(SourceToClean.Get<0>(), SourceToClean.Get<1>());
	}

	for (IPCGGraphExecutionSource* OriginalSource : OriginalSources)
	{
		CleanupRemainingSources(OriginalSource);
	}
}

void FPCGRuntimeGenScheduler::OnLevelStreamingStateChanged(UWorld* InWorld, const ULevelStreaming* InLevelStreaming, ULevel* InLevelIfLoaded, ELevelStreamingState InPreviousState, ELevelStreamingState InNewState)
{
	if (Context.World == InWorld && (InPreviousState == ELevelStreamingState::LoadedVisible || InNewState == ELevelStreamingState::LoadedVisible))
	{
		// todo_pcg: Fine-grained invalidation based on bounds overlap tests did not trivially work on first attempt, improve debug tools/vis and retry.
		CachedStreamingQueryResults.Empty(CachedStreamingQueryResults.Num());
	}
}

void FPCGRuntimeGenScheduler::OnNetworkReplayScrub(UWorld* InWorld)
{
	if (Context.World != InWorld || !Subsystem)
	{
		return;
	}

	bSuspendedForReplayScrub = true;

	UE_LOGF(LogPCG, Verbose, "RuntimeGen scheduler suspended for network replay scrub.");

	// During a network replay scrub, all partition actors will be destroyed. Walk the proper unregistration
	// path for every runtime-gen execution source so that the scheduler's internal state is fully cleaned up.
	for (IPCGGraphExecutionSource* ExecutionSource : Subsystem->GetAllRegisteredExecutionSources())
	{
		if (ExecutionSource && ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			Subsystem->OnOriginalExecutionSourceUnregistered(ExecutionSource);
		}
	}

	// The scrub will destroy all transient actors including pooled PAs. Clear our references so we don't hold stale pointers.
	// The pool will be re-created on the next tick after the scrub completes. 
	ResetPartitionActorPoolToSize(0);
}

void FPCGRuntimeGenScheduler::OnNetworkReplayScrubComplete(UWorld* InWorld)
{
	if (Context.World == InWorld)
	{
		bSuspendedForReplayScrub = false;
	
		UE_LOGF(LogPCG, Verbose, "RuntimeGen scheduler resumed after network replay scrub.");
	}
}

void FPCGRuntimeGenScheduler::CleanupLocalSource(APCGPartitionActor* InPartitionActor, IPCGGraphExecutionSource* InLocalSource)
{
	if (InLocalSource && InLocalSource->GetExecutionState().IsLocalSource())
	{
		IPCGGraphExecutionState& ExecutionState = InLocalSource->GetExecutionState();

		if (IPCGGraphExecutionSource* OriginalExecutionSource = ExecutionState.GetOriginalSource())
		{
			if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread())
			{
				UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] CLEANUP: '%ls'", *ExecutionState.GetDebugName());
			}

#if PCG_RGS_ONSCREENDEBUGMESSAGES
			++PCGRuntimeGenSchedulerHelpers::Stats.CleanupCallCounter;
#endif

			// @todo_pcg: DestroyLocalSource could be moved into the graph execution state interface and generalized for all execution sources.
			if (UPCGRuntimeGenExecutionSource* RuntimeExecutionSource = Cast<UPCGRuntimeGenExecutionSource>(OriginalExecutionSource))
			{
				ensure(!InPartitionActor);
				RuntimeExecutionSource->DestroyLocalSource(Cast<UPCGRuntimeGenExecutionSource>(InLocalSource));
			}
			else if (UPCGComponent* OriginalComponent = Cast<UPCGComponent>(OriginalExecutionSource); InPartitionActor && ensure(OriginalComponent))
			{
				// This performs a CleanupLocalImmediate for us, no need to clean up ourselves.
				InPartitionActor->RemoveGraphInstance(OriginalComponent);
	
				// Remove component mapping.
				PCG::TUniqueScopeLock WriteLock(TrackingManager->ComponentToPartitionActorsMapLock);
				TSet<TObjectPtr<APCGPartitionActor>>* PartitionActorsPtr = TrackingManager->ComponentToPartitionActorsMap.Find(OriginalComponent);
	
				if (PartitionActorsPtr)
				{
					PartitionActorsPtr->Remove(InPartitionActor);
	
					if (PartitionActorsPtr->IsEmpty())
					{
						TrackingManager->ComponentToPartitionActorsMap.Remove(OriginalComponent);
					}
				}
			}
		}
	}

	// Cleanup the PA if it no longer has any components (return to pool or destroy).
	if (InPartitionActor && !InPartitionActor->HasLocalPCGComponents())
	{
		InPartitionActor->UnregisterPCG();

		if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnablePooling.GetValueOnAnyThread())
		{
			if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread())
			{
				UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] RETURNING PARTITION ACTOR TO POOL: '%ls' (%d remaining out of %d)", *InPartitionActor->GetActorNameOrLabel(), PartitionActorPool.Num() + 1, PartitionActorPoolSize);
			}

#if WITH_EDITOR
			InPartitionActor->SetActorLabel(*PCGRuntimeGenSchedulerConstants::PooledPartitionActorName);
#endif
			PartitionActorPool.Push(InPartitionActor);
		}
		else
		{
			if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread())
			{
				UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] DESTROY PARTITION ACTOR: '%ls'", *InPartitionActor->GetActorNameOrLabel());
			}

			Context.World->DestroyActor(InPartitionActor);
		}
	}
}

void FPCGRuntimeGenScheduler::CleanupSource(const FGridGenerationKey& GenerationKey, IPCGGraphExecutionSource* InGeneratedSource)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::CleanupSource);

	check(TrackingManager);

	const FPCGGridDescriptor GridDescriptor = GenerationKey.GetGridDescriptor();
	const FIntVector GridCoords = GenerationKey.GetGridCoords();

	APCGPartitionActor* PartitionActor = nullptr;

	if (!InGeneratedSource)
	{
		UE_LOGF(LogPCG, Warning, "Runtime generated component could not be recovered on grid %d at (%d, %d, %d). It has been lost or destroyed.", GridDescriptor.GetGridSize(), GridCoords.X, GridCoords.Y, GridCoords.Z);

		// If the GeneratedSource has been lost for some reason, get the PA directly from the TrackingManager.
		PartitionActor = TrackingManager->GetPartitionActor(GridDescriptor, GridCoords);
	}
	else // If the GeneratedSource does exist, we can clean it up.
	{
		PartitionActor = InGeneratedSource->GetExecutionState().GetTypedTarget<APCGPartitionActor>();

		if (GridDescriptor.IsUnboundedGrid())
		{
			if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread())
			{
				UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] CLEANUP: '%ls'", *InGeneratedSource->GetExecutionState().GetDebugName());
			}

			// If an OC tries to clean up but it's been generated via RG execution source instead, make sure to clean that up properly.
			if (UPCGComponent* OriginalComponent = Cast<UPCGComponent>(InGeneratedSource))
			{
				if (TObjectPtr<UPCGRuntimeGenExecutionSource> FoundOriginalSource = GetOriginalRuntimeGenExecutionSource(OriginalComponent))
				{
					InGeneratedSource = FoundOriginalSource;
				}
			}

#if PCG_RGS_ONSCREENDEBUGMESSAGES
			++PCGRuntimeGenSchedulerHelpers::Stats.CleanupCallCounter;
#endif

			IPCGGraphExecutionState::FCleanupParams CleanupParams;
			CleanupParams.bCleanupLocalSources = false; // Allow local sources to clean up themselves.
			CleanupParams.bReleaseManagedResources = true;
			CleanupParams.bImmediate = true;

			InGeneratedSource->GetExecutionState().Cleanup(CleanupParams);
		}
	}

	if (PartitionActor || (InGeneratedSource && InGeneratedSource->GetExecutionState().IsLocalSource()))
	{
		CleanupLocalSource(PartitionActor, InGeneratedSource);
	}
	
	GeneratedSources.Remove(GenerationKey);
}

void FPCGRuntimeGenScheduler::CleanupDelayedRefreshSources()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::CleanupDelayedRefreshSources);

	// Move to local to prevent crash if CleanupSource() re-entrantly adds to GeneratedSourcesToRemove.
	TSet<FGridGenerationKey> SourcesToRemoveLocal = MoveTemp(GeneratedSourcesToRemove);

	// Check that each refreshed local source is still intersecting its original source.
	// If it is not, it would be leaked instead of refreshed, so we should force a full cleanup.
	for (const FGridGenerationKey& GenerationKey : SourcesToRemoveLocal)
	{
		if (!GenerationKey.IsValid())
		{
			continue;
		}

		const FPCGGridDescriptor GridDescriptor = GenerationKey.GetGridDescriptor();
		IPCGGraphExecutionSource* OriginalSource = GenerationKey.GetOriginalSource();
		const FIntVector& GridCoords = GenerationKey.GetGridCoords();

		// The unbounded grid level will always lie inside the original source, so we can skip it.
		if (GridDescriptor.IsUnboundedGrid())
		{
			if (OriginalSource && !OriginalSource->GetExecutionState().IsActive())
			{
				CleanupSource(GenerationKey, OriginalSource);
			}

			continue;
		}

		IPCGGraphExecutionSource* LocalSource = OriginalSource ? OriginalSource->GetExecutionState().GetLocalSource(GridDescriptor, GridCoords) : nullptr;

		if (LocalSource)
		{
			const FBox OriginalBounds = OriginalSource->GetExecutionState().GetBounds();
			const FBox LocalBounds = LocalSource->GetExecutionState().GetBounds();

			if (!OriginalBounds.Intersect(LocalBounds) || !OriginalSource->GetExecutionState().IsActive())
			{
				CleanupSource(GenerationKey, LocalSource);
			}
		}
		else
		{
			// If the local source could not be recovered, just clean up.
			CleanupSource(GenerationKey, /*LocalSource=*/nullptr);
		}
	}

	// Remove cleaned sources from GeneratedSources.
	if (!SourcesToRemoveLocal.IsEmpty())
	{
		GeneratedSources = GeneratedSources.Difference(SourcesToRemoveLocal);
	}
}

void FPCGRuntimeGenScheduler::RefreshExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bRemovePartitionActors)
{
	if (!InExecutionSource || !ensure(IsInGameThread()))
	{
		return;
	}

	IPCGGraphExecutionState& ExecutionState = InExecutionSource->GetExecutionState();
	IPCGGraphExecutionSource* OriginalSource = ExecutionState.GetOriginalSource();
	UPCGComponent* OriginalComponent = Cast<UPCGComponent>(OriginalSource);

	// If we are refreshing a PCG Component that has already been generated as a RuntimeGen Execution Source, then we should make sure to use the correct original source (which is the RG exec source).
	if (TObjectPtr<UPCGRuntimeGenExecutionSource> FoundOriginalSource = GetOriginalRuntimeGenExecutionSource(OriginalComponent))
	{
		OriginalSource = FoundOriginalSource;
		OriginalComponent = Cast<UPCGComponent>(FoundOriginalSource->GetOwningSource());

		// If the original component was trying to refresh, we should update the registered execution source's properties in case the original component's properties changed.
		FoundOriginalSource->SetOwningSource(OriginalComponent);
	}
	else if (UPCGRuntimeGenExecutionSource* OriginalRuntimeGenSource = Cast<UPCGRuntimeGenExecutionSource>(OriginalSource))
	{
		OriginalComponent = Cast<UPCGComponent>(OriginalRuntimeGenSource->GetOwningSource());
	}

	if (!OriginalSource || !OriginalComponent)
	{
		return;
	}

	// If we are mid way through setting up an original source, early out from this refresh. This guards against refreshes triggered during execution source registration.
	// Compare against OriginalComponent (UPCGComponent*) rather than OriginalSource, since OriginalSourceBeingRegistered is always a UPCGComponent* and OriginalSource
	// may not be (e.g. when InExecutionSource is not a UPCGComponent, i.e. a runtime gen execution source, which resolves to its owning component via OriginalComponent).
	if (OriginalComponent == OriginalSourceBeingRegistered)
	{
		return;
	}

	const bool bUseActorComponentlessGeneration = OriginalComponent->UseActorComponentlessGeneration();
	const bool bHasExistingRuntimeGenExecutionSource = OriginalExecutionSources.Contains(OriginalComponent);

	if (bUseActorComponentlessGeneration != bHasExistingRuntimeGenExecutionSource)
	{
		// We always need to do a deep refresh if we've detected that bUseActorComponentlessGeneration has changed. We will update the original execution source registration after performing the cleanup.
		bRemovePartitionActors = true;
	}

	// Trigger a rescan of generation cells for this execution source.
	ChangeDetector.FlushCachedState(OriginalSource);

	TimeToNextTick = 0.0;

	const bool bLoggingEnabled = PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread();

	// Useful because we can run into generation order issues if execution sources are left to continue generating.
	if (ExecutionState.IsGenerating())
	{
		ExecutionState.Cancel();
	}

	IPCGGraphExecutionState& OriginalExecutionState = OriginalSource->GetExecutionState();

	if (!bRemovePartitionActors)
	{
		// Refresh path - mark execution source dirty and removed generated keys which will cause it to be scheduled for regeneration.

		// Register for deferred removal from generated execution sources set, execution source will be regenerated later (and in grid order
		// so that e.g. unbounded is generated first).
		if (ExecutionState.IsLocalSource())
		{
			if (bLoggingEnabled)
			{
				UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] SHALLOW REFRESH LOCAL COMPONENT: '%ls'", *ExecutionState.GetDebugName());
			}

			GeneratedSourcesToRemove.Emplace({ ExecutionState.GetGenerationGridSize(), ExecutionState.GetGenerationGridCoords(), OriginalSource, InExecutionSource });

			IPCGGraphExecutionState::FCleanupParams CleanupParams;
			CleanupParams.bReleaseManagedResources = false;
			CleanupParams.bImmediate = true;

			ExecutionState.Cleanup(CleanupParams);
		}
		else
		{
			// Register original execution source for deferred removal.
			GeneratedSourcesToRemove.Emplace({ PCGHiGenGrid::UnboundedGridSize(), FIntVector(0), OriginalSource });

			// Register local execution sources for deferred removal if they have not already registered themselves.
			for (const FGridGenerationKey& Key : GeneratedSources)
			{
				if (Key.GetOriginalSource() == OriginalSource && !GeneratedSourcesToRemove.Contains(Key))
				{
					if (IPCGGraphExecutionSource* LocalSource = OriginalExecutionState.GetLocalSource(Key.GetGridDescriptor(), Key.GetGridCoords()))
					{
						IPCGGraphExecutionState& LocalExecutionState = LocalSource->GetExecutionState();

						if (bLoggingEnabled)
						{
							UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] SHALLOW REFRESH LOCAL COMPONENT: '%ls'", *LocalExecutionState.GetDebugName());
						}

						IPCGGraphExecutionState::FCleanupParams CleanupParams;
						CleanupParams.bReleaseManagedResources = false;
						CleanupParams.bImmediate = true;

						LocalExecutionState.Cleanup(CleanupParams);

						// We need to make sure that the next time this is generated that it matches the original
						if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(LocalSource))
						{
							PCGComponent->SetPropertiesFromOriginal(OriginalComponent);
						}
					}

					GeneratedSourcesToRemove.Add(Key);
				}
			}

			if (bLoggingEnabled)
			{
				UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] SHALLOW REFRESH COMPONENT: '%ls' PARTITIONED: %d",
					*OriginalExecutionState.GetDebugName(),
					OriginalExecutionState.IsPartitioned() ? 1 : 0);
			}

			IPCGGraphExecutionState::FCleanupParams CleanupParams;
			CleanupParams.bReleaseManagedResources = false;
			CleanupParams.bImmediate = true;

			OriginalExecutionState.Cleanup(CleanupParams);
		}
	}
	else
	{
		// Full cleanout path - cleanup existing execution sources and return actors to the pool.

		auto RefreshLocalSource = [this, OriginalSource, bLoggingEnabled](IPCGGraphExecutionSource* LocalSource)
		{
			check(LocalSource);

			IPCGGraphExecutionState& LocalExecutionState = LocalSource->GetExecutionState();

			// Find the specific generation key for this execution source, if it exists, cleanup and generate.
			FGridGenerationKey LocalSourceKey(LocalExecutionState.GetGenerationGridSize(), LocalExecutionState.GetGenerationGridCoords(), OriginalSource, LocalSource);

			if (GeneratedSources.Find(LocalSourceKey))
			{
				if (bLoggingEnabled)
				{
					UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] DEEP REFRESH LOCAL COMPONENT: '%ls'", *LocalExecutionState.GetDebugName());
				}

				CleanupSource(LocalSourceKey, LocalSource);
			}
		};

		if (ExecutionState.IsLocalSource())
		{
			RefreshLocalSource(InExecutionSource);
		}
		else
		{
			TArray<FGridGenerationKey> GenerationKeys;

			for (FGridGenerationKey GenerationKey : GeneratedSources)
			{
				if (GenerationKey.GetOriginalSource() == OriginalSource)
				{
					GenerationKeys.Add(GenerationKey);
				}
			}

			// Gather all generated execution sources which originated from this original execution source.
			for (FGridGenerationKey GenerationKey : GenerationKeys)
			{
				const FPCGGridDescriptor GridDescriptor = GenerationKey.GetGridDescriptor();

				// If the grid is unbounded, we have a non-partitioned or unbounded execution source.
				if (GridDescriptor.IsUnboundedGrid())
				{
					if (bLoggingEnabled)
					{
						UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] DEEP REFRESH COMPONENT: '%ls' PARTITIONED: %d",
							*OriginalExecutionState.GetDebugName(),
							OriginalExecutionState.IsPartitioned() ? 1 : 0);
					}

					CleanupSource(GenerationKey, OriginalSource);
				}
				// Otherwise we have a local execution source.
				else
				{
					const FIntVector GridCoords = GenerationKey.GetGridCoords();

					if (IPCGGraphExecutionSource* LocalSource = OriginalExecutionState.GetLocalSource(GridDescriptor, GridCoords))
					{
						RefreshLocalSource(LocalSource);
					}
					else
					{
						// If the local execution source could not be recovered, cleanup its entry to avoid leaking resources/locking the grid cell.
						CleanupSource(GenerationKey, nullptr);
					}
				}
			}
		}
	}

	// Update the original execution source registration after the cleanup has been performed. That way we cleanup the existing source before moving to the new source.
	if (!ExecutionState.IsLocalSource() && bUseActorComponentlessGeneration != bHasExistingRuntimeGenExecutionSource)
	{
		if (bUseActorComponentlessGeneration)
		{
			RegisterOriginalRuntimeGenExecutionSource(OriginalComponent);
		}
		else
		{
			UnregisterOriginalRuntimeGenExecutionSource(OriginalComponent);
		}
	}

	if (!ExecutionState.IsLocalSource())
	{
		// When an original/non-partitioned execution source is refreshed, we need to dirty the state.
		bContextDirty = true;
		bCleanupScanRequired = true;
	}
}

APCGPartitionActor* FPCGRuntimeGenScheduler::GetPartitionActorFromPool(const FPCGGridDescriptor& GridDescriptor, const FIntVector& GridCoords)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::GetPartitionActorFromPool);

	check(TrackingManager);

	if (!Context.World)
	{
		UE_LOGF(LogPCG, Error, "[GetPartitionActorFromPool] World is null.");
		return nullptr;
	}

	// Attempt to find an existing RuntimeGen PA.
	if (APCGPartitionActor* ExistingActor = TrackingManager->GetPartitionActor(GridDescriptor, GridCoords))
	{
		return ExistingActor;
	}

	// Double size of the pool if it is empty.
	if (PartitionActorPool.IsEmpty())
	{
		// If PartitionActorPoolSize is zero, then we should use the CVarBasePoolSize instead. Result must always at least be >= 1.
		const uint32 CurrentPoolSize = FMath::Max(1, PartitionActorPoolSize > 0 ? PartitionActorPoolSize : PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationBasePoolSize.GetValueOnAnyThread());

		if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread())
		{
			UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] INCREASING TRANSIENT PARTITION ACTOR POOL SIZE BY (%d)", CurrentPoolSize);
		}

		// If pooling was enabled late, the editor world RuntimeGenScheduler will not have created the initial pool, so we should create it now.
		AddPartitionActorPoolCount(CurrentPoolSize);
	}

	check(!PartitionActorPool.IsEmpty());
	APCGPartitionActor* PartitionActor = PartitionActorPool.Pop();

#if WITH_EDITOR
	const FName ActorName = *APCGPartitionActor::GetPCGPartitionActorName(GridDescriptor, GridCoords);
	PartitionActor->SetActorLabel(ActorName.ToString());
#endif

	const FVector CellCenter(FVector(GridCoords.X + 0.5, GridCoords.Y + 0.5, GridCoords.Z + 0.5) * GridDescriptor.GetGridSize());
	if (!PartitionActor->Teleport(CellCenter))
	{
		UE_LOGF(LogPCG, Error, "[RUNTIMEGEN] Could not set the location of RuntimeGen partition actor '%ls'.", *PartitionActor->GetActorNameOrLabel());
	}

#if WITH_EDITOR
	PartitionActor->SetLockLocation(true);
#endif

	// Empty GUID, RuntimeGen PAs don't need one.
	PartitionActor->PostCreation(GridDescriptor);

	return PartitionActor;
}

void FPCGRuntimeGenScheduler::AddPartitionActorPoolCount(int32 Count)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGRuntimeGenScheduler::AddPartitionActorPoolCount);

	PartitionActorPoolSize += Count;

	FActorSpawnParameters SpawnParams;
#if WITH_EDITOR
	// Always hide pooled actors from outliner. Note that outliner tree view updates can incur significant costs in Slate code.
	SpawnParams.bHideFromSceneOutliner = PCGRuntimeGenSchedulerHelpers::CVarHideActorsFromOutliner.GetValueOnAnyThread();
#endif

	SpawnParams.ObjectFlags |= RF_Transient;
	SpawnParams.ObjectFlags &= ~RF_Transactional;

	// Important to override the level to make sure we spawn in the persistent level and not the editor's current editing level
	SpawnParams.OverrideLevel = Context.World->PersistentLevel;

	// Create RuntimeGen PA pool.
	for (int32 I = 0; I < Count; ++I)
	{
		// TODO: do these actors get networked automatically? do we want that or not?
		APCGPartitionActor* NewActor = Context.World->SpawnActor<APCGPartitionActor>(SpawnParams);
		check(NewActor);
		NewActor->SetToRuntimeGenerated();
		PartitionActorPool.Add(NewActor);
#if WITH_EDITOR
		NewActor->SetActorLabel(PCGRuntimeGenSchedulerConstants::PooledPartitionActorName);
#endif
	}
}

void FPCGRuntimeGenScheduler::ResetPartitionActorPoolToSize(uint32 NewPoolSize)
{
	for (APCGPartitionActor* PartitionActor : PartitionActorPool)
	{
		Context.World->DestroyActor(PartitionActor);
	}

	PartitionActorPool.Empty();
	PartitionActorPoolSize = 0;
	AddPartitionActorPoolCount(NewPoolSize);
}

void FPCGRuntimeGenScheduler::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (GenSourceManager)
	{
		GenSourceManager->AddReferencedObjects(Collector);
	}

	// The level should be keeping the pooled PAs visible to GC. This is just a tentative fix for a crash in GetPartitionActorFromPool(), to understand if the crash is happening because of unreferenced GCed actors.
	Collector.AddReferencedObjects(PartitionActorPool);

	// Keep runtime gen execution sources alive.
	for (auto& [OriginalComponent, ExecutionSource] : OriginalExecutionSources)
	{
		Collector.AddReferencedObject(ExecutionSource);
	}
}

void FPCGRuntimeGenScheduler::CacheVirtualTexturePrimingInfos(UPCGComponent* InOriginalComponent)
{
	CachedPrimingInfos.Remove(InOriginalComponent);

	const UPCGGraphInstance* Graph = InOriginalComponent ? InOriginalComponent->GetGraphInstance() : nullptr;
	const FInstancedPropertyBag* UserParametersStruct = Graph ? Graph->GetUserParametersStruct() : nullptr;
	const UPropertyBag* PropertyBag = UserParametersStruct ? UserParametersStruct->GetPropertyBagStruct() : nullptr;

	if (!PropertyBag)
	{
		return;
	}

	TArray<FPCGVirtualTexturePrimingInfo> PrimingInfos;

	for (const FPropertyBagPropertyDesc& PropertyDesc : PropertyBag->GetPropertyDescs())
	{
		if (PropertyDesc.ValueType == EPropertyBagPropertyType::Struct && PropertyDesc.ValueTypeObject == TBaseStructure<FPCGVirtualTexturePrimingInfo>::Get())
		{
			TValueOrError<FPCGVirtualTexturePrimingInfo*, EPropertyBagResult> Property = UserParametersStruct->GetValueStruct<FPCGVirtualTexturePrimingInfo>(PropertyDesc);

			if (Property.HasValue() && Property.GetValue() && !Property.GetValue()->VirtualTexture.IsNull())
			{
				PrimingInfos.Add(*Property.GetValue());
			}
		}
	}

	if (!PrimingInfos.IsEmpty())
	{
		CachedPrimingInfos.Add(InOriginalComponent, MoveTemp(PrimingInfos));
	}
}

void FPCGRuntimeGenScheduler::RefreshContext()
{
	const bool bAnyRuntimeGenSourcesBefore = Context.bAnyRuntimeGenSourcesExist;

	Context.bAnySourcesUse2DGrids = false;
	Context.bAnySourcesUse3DGrids = false;
	Context.bAnySourcesUseFrustumCulling = false;
	Context.bAnyRuntimeGenSourcesExist = false;

	auto ProcessExecutionSources = [&ContextInner=Context](const TSet<IPCGGraphExecutionSource*>& InExecutionSources)
	{
		for (IPCGGraphExecutionSource* ExecutionSource : InExecutionSources)
		{
			if (ExecutionSource && ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
			{
				ContextInner.bAnyRuntimeGenSourcesExist = true;

				const bool bUses2DGrid = ExecutionSource->GetExecutionState().Use2DGrid();
				ContextInner.bAnySourcesUse2DGrids |= bUses2DGrid;
				ContextInner.bAnySourcesUse3DGrids |= !bUses2DGrid;

				if (const UPCGSchedulingPolicyBase* Policy = ExecutionSource->GetExecutionState().GetRuntimeGenSchedulingPolicy())
				{
					ContextInner.bAnySourcesUseFrustumCulling |= Policy->CullsBasedOnDirection();
				}
			}
		}
	};

	ProcessExecutionSources(TrackingManager->GetAllRegisteredNonPartitionedExecutionSources());
	ProcessExecutionSources(TrackingManager->GetAllRegisteredPartitionedExecutionSources());

	if (PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableDebugging.GetValueOnAnyThread())
	{
		if (bAnyRuntimeGenSourcesBefore != Context.bAnyRuntimeGenSourcesExist)
		{
			if (Context.bAnyRuntimeGenSourcesExist)
			{
				UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] THERE ARE NOW RUNTIME EXECUTION SOURCES IN THE LEVEL. SCHEDULER WILL BEGIN TICKING.");
			}
			else
			{
				UE_LOGF(LogPCG, Warning, "[RUNTIMEGEN] THERE ARE NO MORE RUNTIME EXECUTION SOURCES. SCHEDULER WILL ONLY TICK TO CLEANUP.");
			}
		}
	}

	bContextDirty = false;
}

bool FPCGRuntimeGenScheduler::IsChangeDetectionEnabled() const
{
	return PCGRuntimeGenSchedulerHelpers::CVarRuntimeGenerationEnableChangeDetection.GetValueOnGameThread();
}

void FPCGRuntimeGenScheduler::RegisterOriginalRuntimeGenExecutionSource(UPCGComponent* InOriginalComponent)
{
	if (InOriginalComponent && InOriginalComponent->UseActorComponentlessGeneration())
	{
		if (TObjectPtr<UPCGRuntimeGenExecutionSource> FoundOriginalSource = GetOriginalRuntimeGenExecutionSource(InOriginalComponent))
		{
			// If we're already registered, just update the runtime gen execution source to stay up to date.
			FoundOriginalSource->SetOwningSource(InOriginalComponent);
		}
		else
		{
			// No existing source found, register a new runtime gen execution source.
			UPCGRuntimeGenExecutionSource* NewSource = NewObject<UPCGRuntimeGenExecutionSource>();
			NewSource->Register(InOriginalComponent);

			OriginalExecutionSources.Add(InOriginalComponent, NewSource);
		}
	}
}

void FPCGRuntimeGenScheduler::UnregisterOriginalRuntimeGenExecutionSource(UPCGComponent* InOriginalComponent)
{
	if (TObjectPtr<UPCGRuntimeGenExecutionSource>* FoundRuntimeGenSource = OriginalExecutionSources.Find(InOriginalComponent))
	{
		if (*FoundRuntimeGenSource)
		{
			IPCGGraphExecutionState::FCleanupParams CleanupParams;
			CleanupParams.bCleanupLocalSources = true;
			CleanupParams.bReleaseManagedResources = true;
			CleanupParams.bImmediate = true;

			(*FoundRuntimeGenSource)->Cleanup(CleanupParams);
			(*FoundRuntimeGenSource)->Unregister();
			(*FoundRuntimeGenSource)->MarkAsGarbage();
		}

		OriginalExecutionSources.Remove(InOriginalComponent);
	}
}

UPCGRuntimeGenExecutionSource* FPCGRuntimeGenScheduler::GetOriginalRuntimeGenExecutionSource(UPCGComponent* InOriginalComponent)
{
	if (TObjectPtr<UPCGRuntimeGenExecutionSource>* FoundOriginalSource = OriginalExecutionSources.Find(InOriginalComponent); FoundOriginalSource)
	{
		return *FoundOriginalSource;
	}

	return nullptr;
}

#if WITH_EDITOR
void FPCGRuntimeGenScheduler::RegisterGraphParameterChangedEvent(const UPCGComponent* InOriginalComponent)
{
	if (UPCGGraph* Graph = InOriginalComponent ? InOriginalComponent->GetGraph() : nullptr)
	{
		FDelegateHandle* FoundDelegateHandle = GraphParamChangedEventHandles.Find(InOriginalComponent);

		if (!FoundDelegateHandle)
		{
			GraphParamChangedEventHandles.Add(InOriginalComponent, Graph->OnGraphParametersChangedDelegate.AddRaw(this, &FPCGRuntimeGenScheduler::OnGraphParametersChanged));
		}
	}
}

void FPCGRuntimeGenScheduler::UnregisterGraphParameterChangedEvent(const UPCGComponent* InOriginalComponent)
{
	FDelegateHandle DelegateHandle;

	if (GraphParamChangedEventHandles.RemoveAndCopyValue(InOriginalComponent, DelegateHandle))
	{
		if (UPCGGraph* Graph = InOriginalComponent ? InOriginalComponent->GetGraph() : nullptr)
		{
			Graph->OnGraphParametersChangedDelegate.Remove(DelegateHandle);
		}
	}
}

void FPCGRuntimeGenScheduler::OnGraphParametersChanged(UPCGGraphInterface* InGraph, EPCGGraphParameterEvent InChangeType, FName InChangedPropertyName)
{
	TArray<UPCGComponent*> OriginalComponents;

	TSet<IPCGGraphExecutionSource*> PartitionedExecutionSources = TrackingManager->GetAllRegisteredPartitionedExecutionSources();
	TSet<IPCGGraphExecutionSource*> NonPartitionedExecutionSources = TrackingManager->GetAllRegisteredNonPartitionedExecutionSources();

	OriginalComponents.Reserve(PartitionedExecutionSources.Num() + NonPartitionedExecutionSources.Num());

	auto ShouldCachePrimingInfosForComponent = [InGraph](const UPCGComponent* OriginalComponent)
	{
		return OriginalComponent->bActivated && OriginalComponent->IsManagedByRuntimeGenSystem() && OriginalComponent->GetGraph() == InGraph;
	};

	// @todo_pcg: support execution source
	for (IPCGGraphExecutionSource* OriginalSource : PartitionedExecutionSources)
	{
		if (UPCGComponent* OriginalComponent = Cast<UPCGComponent>(OriginalSource))
		{
			if (ShouldCachePrimingInfosForComponent(OriginalComponent))
			{
				OriginalComponents.Add(OriginalComponent);
			}
		}
	}

	// @todo_pcg: support execution source
	for (IPCGGraphExecutionSource* OriginalSource : NonPartitionedExecutionSources)
	{
		if (UPCGComponent* OriginalComponent = Cast<UPCGComponent>(OriginalSource))
		{
			if (ShouldCachePrimingInfosForComponent(OriginalComponent))
			{
				OriginalComponents.Add(OriginalComponent);
			}
		}
	}

	for (UPCGComponent* Component : OriginalComponents)
	{
		CacheVirtualTexturePrimingInfos(Component);
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR || !UE_BUILD_SHIPPING
void FPCGRuntimeGenScheduler::OnCVarSinkFired()
{
	check(IsInGameThread());

	if (!TrackingManager)
	{
		return;
	}

	// Build a map from CVar name to the execution sources whose graph lists that CVar.
	TMap<FString, TArray<IPCGGraphExecutionSource*>> CVarToSources;

	auto GatherSources = [&](const TSet<IPCGGraphExecutionSource*>& Sources)
	{
		for (IPCGGraphExecutionSource* Source : Sources)
		{
			const UPCGGraph* Graph = Source ? Source->GetExecutionState().GetGraph() : nullptr;
			if (!Graph || Graph->ConsoleVariablesTriggeringRefresh.IsEmpty())
			{
				continue;
			}

			for (const FString& CVarName : Graph->ConsoleVariablesTriggeringRefresh)
			{
				CVarToSources.FindOrAdd(CVarName).Add(Source);
			}
		}
	};

	GatherSources(TrackingManager->GetAllRegisteredPartitionedExecutionSources());
	GatherSources(TrackingManager->GetAllRegisteredNonPartitionedExecutionSources());

	// Note: Cvar access in PCG is never cached, fresh values are always pulled.
	if (CVarToSources.IsEmpty())
	{
		WatchedCVarLastValues.Empty();
		return;
	}

	// Check which watched CVars actually changed value and collect the affected sources.
	TSet<IPCGGraphExecutionSource*> SourcesToRefresh;

	for (auto& [CVarName, Sources] : CVarToSources)
	{
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
		if (!CVar)
		{
			continue;
		}

		const FString CurrentValue = CVar->GetString();
		FString& CachedValue = WatchedCVarLastValues.FindOrAdd(CVarName);

		if (CachedValue != CurrentValue)
		{
			CachedValue = CurrentValue;
			SourcesToRefresh.Append(Sources);
		}
	}

	// Remove cached entries for CVars no longer watched by any registered source.
	for (auto It = WatchedCVarLastValues.CreateIterator(); It; ++It)
	{
		if (!CVarToSources.Contains(It->Key))
		{
			It.RemoveCurrent();
		}
	}

	for (IPCGGraphExecutionSource* Source : SourcesToRefresh)
	{
		RefreshExecutionSource(Source, /*bRemovePartitionActors=*/true);
	}
}
#endif // WITH_EDITOR || !UE_BUILD_SHIPPING

#undef PCG_RGS_ONSCREENDEBUGMESSAGES
