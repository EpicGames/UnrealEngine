// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/SchedulingPolicies/PCGSchedulingPolicyBase.h"

#include "PCGCommon.h"

#include "Containers/Set.h"
#include "Hash/xxhash.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSchedulingPolicyBase)

void UPCGSchedulingPolicyBase::PostLoad()
{
	Super::PostLoad();
	UpdateWorldPartitionTargetGridsHash();
}

#if WITH_EDITOR
void UPCGSchedulingPolicyBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UPCGSchedulingPolicyBase, WorldPartitionTargetGrids))
	{
		UpdateWorldPartitionTargetGridsHash();
	}
}
#endif

void UPCGSchedulingPolicyBase::UpdateWorldPartitionTargetGridsHash()
{
	if (WorldPartitionTargetGrids.IsEmpty())
	{
		WorldPartitionTargetGridsHash = 0;
		return;
	}

	// FName::ToUnstableInt gives a process-stable 64-bit identity per name.
	// Sorting these produces a deterministic byte sequence for any ordering of the same set, which XXH3 then hashes to 64 bits.
	TArray<uint64, TInlineAllocator<8>> Packed;
	Packed.Reserve(WorldPartitionTargetGrids.Num());
	for (const FName& Name : WorldPartitionTargetGrids)
	{
		Packed.Add(Name.ToUnstableInt());
	}
	Packed.Sort();

	WorldPartitionTargetGridsHash = FXxHash64::HashBuffer(Packed.GetData(), Packed.Num() * sizeof(uint64)).Hash;
}

bool UPCGSchedulingPolicyBase::IsEquivalent(const UPCGSchedulingPolicyBase* OtherSchedulingPolicy) const
{
	if (this == OtherSchedulingPolicy)
	{
		return true;
	}

	if (!OtherSchedulingPolicy)
	{
		return false;
	}

	if (GridStreamingDependencyMode != OtherSchedulingPolicy->GridStreamingDependencyMode)
	{
		return false;
	}

	// Order-independent compare for GridsDependentOnWorldStreaming (treated as a set).
	if (GridsDependentOnWorldStreaming.Num() != OtherSchedulingPolicy->GridsDependentOnWorldStreaming.Num())
	{
		return false;
	}

	for (EPCGHiGenGrid Grid : GridsDependentOnWorldStreaming)
	{
		if (!OtherSchedulingPolicy->GridsDependentOnWorldStreaming.Contains(Grid))
		{
			return false;
		}
	}

	if (WorldPartitionTargetGrids.Num() != OtherSchedulingPolicy->WorldPartitionTargetGrids.Num())
	{
		return false;
	}

	for (const FName& GridName : WorldPartitionTargetGrids)
	{
		if (!OtherSchedulingPolicy->WorldPartitionTargetGrids.Contains(GridName))
		{
			return false;
		}
	}

	return true;
}
