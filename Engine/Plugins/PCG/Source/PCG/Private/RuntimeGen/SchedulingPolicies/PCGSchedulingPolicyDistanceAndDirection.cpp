// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeGen/SchedulingPolicies/PCGSchedulingPolicyDistanceAndDirection.h"

#include "PCGCommon.h"
#include "RuntimeGen/GenSources/PCGGenSourceBase.h"

#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSchedulingPolicyDistanceAndDirection)

double UPCGSchedulingPolicyDistanceAndDirection::CalculatePriority(const IPCGGenSourceBase* InGenSource, const FBox& GenerationBounds, bool bUse2DGrid) const
{
	check(InGenSource);

	double Priority = 0.0;
	double MaxPriority = 0.0;

	if (bUseDistance || bUseDirection)
	{
		TOptional<FVector> GenSourcePositionOptional = InGenSource->GetPosition();

		if (ensure(GenSourcePositionOptional.IsSet()))
		{
			FVector GenSourcePosition = GenSourcePositionOptional.GetValue();
			if (bUse2DGrid)
			{
				GenSourcePosition.Z = 0.0;
			}

			const FVector GenSourceToComponent = GenerationBounds.GetClosestPointTo(GenSourcePosition) - GenSourcePosition;

			if (GenSourceToComponent.IsNearlyZero())
			{
				// If the gen source is inside the generation bounds then always give maximum priority.
				return 1.0;
			}

			if (bUseDistance)
			{
				// Normalize via grid size so that priority calculations are well behaved across different grid sizes.
				const double GridSize = FMath::Max(GenerationBounds.GetSize().X, GenerationBounds.GetSize().Y);
				const double DistInCells = GenSourceToComponent.Length() / GridSize;
				// No major significance of reciprocal choice here, just a weight curve that is monotonic and always in [0, 1].
				const double DistPriority = 1.0 / FMath::Max(DistInCells, 1.0);

				Priority += DistPriority;
				MaxPriority += 1.0;
			}

			if (bUseDirection)
			{
				TOptional<FVector> GenSourceDirectionOptional = InGenSource->GetDirection();

				if ((GenSourceDirectionOptional.IsSet()))
				{
					// Directional weight is simply dot product scaled to [0, 1], then weighted by user.
					const double DirectionDotProd = FVector::DotProduct(GenSourceToComponent.GetSafeNormal(), GenSourceDirectionOptional.GetValue());
					Priority += DirectionWeight * FMath::Lerp(0.5, 1.0, DirectionDotProd);
					MaxPriority += DirectionWeight;
				}
			}
		}
	}

	// Normalize.
	if (MaxPriority > 0.0)
	{
		Priority /= MaxPriority;
	}

	ensure(Priority >= 0.0 && Priority <= 1.0);

	return Priority;
}

bool UPCGSchedulingPolicyDistanceAndDirection::ShouldGenerate(const IPCGGenSourceBase* InGenSource, const FBox& GenerationBounds, bool bUse2DGrid) const
{
	// Filter out Gen sources based on network mode
	const bool bIsLocal = InGenSource->IsLocal();
	if ((NetworkMode == EPCGSchedulingPolicyNetworkMode::Client && !bIsLocal) || (NetworkMode == EPCGSchedulingPolicyNetworkMode::Server && bIsLocal))
	{
		return false;
	}

	if (!bUseFrustumCulling)
	{
		return true;
	}

	const TOptional<FConvexVolume> ViewFrustumOptional = InGenSource->GetViewFrustum(bUse2DGrid);

	if (!ViewFrustumOptional.IsSet())
	{
		return true;
	}

	FVector Center, Extents;
	GenerationBounds.GetCenterAndExtents(Center, Extents);

	// @todo_pcg: Instead of having a modifier on the bounds, it would probably be better to increase the FOV of the frustum's projection matrix.
	// Another idea would be to adjust scaling based on distance instead, which would end up most likely cheaper than recomputing the FOV.
	const FConvexVolume ViewFrustum = ViewFrustumOptional.GetValue();
	return ViewFrustum.IntersectBox(Center, Extents * GenerateBoundsModifier);
}

bool UPCGSchedulingPolicyDistanceAndDirection::ShouldCull(const IPCGGenSourceBase* InGenSource, const FBox& GenerationBounds, bool bUse2DGrid) const
{
	// Filter out Gen sources based on network mode
	const bool bIsLocal = InGenSource->IsLocal();
	if ((NetworkMode == EPCGSchedulingPolicyNetworkMode::Client && !bIsLocal) || (NetworkMode == EPCGSchedulingPolicyNetworkMode::Server && bIsLocal))
	{
		return true;
	}

	if (!bUseFrustumCulling)
	{
		// If frustum culling is not enabled, don't try cleaning up volumes within the cleanup radius.
		return false;
	}

	const TOptional<FConvexVolume> ViewFrustumOptional = InGenSource->GetViewFrustum(bUse2DGrid);

	if (!ViewFrustumOptional.IsSet())
	{
		return false;
	}

	FVector Center, Extents;
	GenerationBounds.GetCenterAndExtents(Center, Extents);

	// If the volume does not intersect the view frustum, allow it to be cleaned up.
	const FConvexVolume ViewFrustum = ViewFrustumOptional.GetValue();

	const float CleanupModifier = FMath::Max(GenerateBoundsModifier + 0.1, CleanupBoundsModifier);
	return !ViewFrustum.IntersectBox(Center, Extents * CleanupModifier);
}

bool UPCGSchedulingPolicyDistanceAndDirection::IsEquivalent(const UPCGSchedulingPolicyBase* OtherSchedulingPolicy) const
{
	if (this == OtherSchedulingPolicy)
	{
		return true;
	}

	if (!Super::IsEquivalent(OtherSchedulingPolicy))
	{
		return false;
	}

	if (const UPCGSchedulingPolicyDistanceAndDirection* Other = Cast<UPCGSchedulingPolicyDistanceAndDirection>(OtherSchedulingPolicy))
	{
		return bUseDistance == Other->bUseDistance
			&& bUseDirection == Other->bUseDirection
			&& DirectionWeight == Other->DirectionWeight
			&& bUseFrustumCulling == Other->bUseFrustumCulling
			&& GenerateBoundsModifier == Other->GenerateBoundsModifier
			&& CleanupBoundsModifier == Other->CleanupBoundsModifier
			&& NetworkMode == Other->NetworkMode;
	}

	return false;
}
