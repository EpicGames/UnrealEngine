// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialAlgo/PCGAStar.h"

#include "Helpers/PCGHelpers.h"
#include "SpatialAlgo/PCGOctreeQueries.h"

namespace PCGSpatialAlgo::AStar
{
	struct FNode
	{
		explicit FNode(const FPointDescription* InPointDescription, const int32 InPreviousNodeIndex, const double InLocalCost, const double InEstimatedGoalCost)
			: PointDescription(InPointDescription)
			, PreviousNodeIndex(InPreviousNodeIndex)
			, LocalCost(InLocalCost)
			, EstimatedGoalCost(InEstimatedGoalCost)
		{}

		const FVector& GetLocation() const
		{
			return PointDescription->GetLocation();
		}
		
		const FPointDescription* PointDescription = nullptr;

		/** The currently assigned parent node with the least cost. */
		int32 PreviousNodeIndex = INDEX_NONE;
		/** Concrete cost from start to this node. */
		double LocalCost = 0.0;
		/** Current total cost when heuristic is added to the local cost. */
		double EstimatedGoalCost = 0.0;
	};
	
	struct FSearchState::FInternalSearchData
	{
		static uint32 constexpr PreAllocNodeCount = 1024u;
		using FNodeList = TArray<FNode, TInlineAllocator<PreAllocNodeCount>>;
		/** Storage of the current known node. Source of truth for the nodes, so it should not be resized after its initial reserve. */
		FNodeList NodeList;
		/** Used as a priority list of nodes, sorted by cost. */
		TArray<int32, TInlineAllocator<PreAllocNodeCount>> OpenIndexList;
		/** Nodes that are no longer viable. */
		TArray<int32, TInlineAllocator<PreAllocNodeCount>> ClosedIndexList;
		/** To track nodes to their corresponding points. */
		TMap<const FPointDescription*, int32> PointDescriptionToNodeIndexMap;

		TArray<FPointDescription> InternalOriginatingPointsDescriptions;
		FPointDescription InternalStartPointDescription;
		TArray<FPointDescription> InternalGoalPointsDescriptions; 
	};
	
	FSearchState::FSearchState():
		InternalSearchData{MakeUnique<FInternalSearchData>()}
	{}

	FSearchState::~FSearchState() = default;

namespace Helpers
{
	bool CompareNodes(const FNode& Node1, const FNode& Node2)
	{
		return Node1.EstimatedGoalCost < Node2.EstimatedGoalCost;
	}

	void BuildFinalPath(const FSearchState& SearchState, const FNode& FinalNode, const bool bCopyOriginatingPoints, TArray<FPointDescription>& OutPath)
	{
		check(&FinalNode);

		OutPath.Reset();

		const FNode* Node = &FinalNode;
		do // Build path from the goal backwards.
		{
			OutPath.Emplace(*Node->PointDescription);

			Node = Node->PreviousNodeIndex != INDEX_NONE ? &SearchState.InternalSearchData->NodeList[Node->PreviousNodeIndex] : nullptr;
		}
		while (Node);

		// Reverse to get it in the correct travel order.
		Algo::Reverse(OutPath);
	}

	TTuple<int32, double> GetNearestGoalToLocation(const FSearchState& SearchState, const FPointDescription& FromPosition)
	{
		check(SearchState.InternalSearchData->InternalGoalPointsDescriptions.Num() > 0);

		double MinCostToGoal = std::numeric_limits<double>::max();
		int32 CurrentGoalIndex = INDEX_NONE;
		for (int32 GoalIndex = 0; GoalIndex < SearchState.InternalSearchData->InternalGoalPointsDescriptions.Num(); ++GoalIndex)
		{
			const FPointDescription& CurGoal = SearchState.InternalSearchData->InternalGoalPointsDescriptions[GoalIndex];
			const double CurDistToGoalSqr = FVector::DistSquared(FromPosition.GetLocation(), CurGoal.GetLocation());
			const double CurCostToGoal = SearchState.CostFunction(FromPosition, CurDistToGoalSqr, CurGoal);
			if (CurCostToGoal < MinCostToGoal)
			{
				MinCostToGoal = CurCostToGoal;
				CurrentGoalIndex = GoalIndex;
			}
		}
		check(CurrentGoalIndex != INDEX_NONE);

		return MakeTuple(CurrentGoalIndex, MinCostToGoal);
	}
}

namespace Cost
{
	double CalculateCost_EuclideanDistance(const FPointDescription& PreviousNodePoint, const double DistanceToPreviousNodeSquared, const FPointDescription& CurrentPoint)
	{
		return FMath::Sqrt(DistanceToPreviousNodeSquared);
	}
}

/** Note: In order for the path to be optimal, the heuristic cost must always be less than or equal to the actual cost. */
namespace Heuristic
{
	double CalculateHeuristic_EuclideanDistance(const FVector& CurrentLocation, const FVector& GoalLocation)
	{
		return FVector::Dist(CurrentLocation, GoalLocation);
	}
}


/** Initialize the search state for AStar. Must be called before ExecuteSearchIteration. */
void Initialize(const FPCGPointInputRange& StartPoint, const FPCGPointInputRange& GoalPoints, FSearchState& OutSearchState)
{
	OutSearchState.GoalPoints = GoalPoints;
	OutSearchState.InternalSearchData->InternalStartPointDescription = FPointDescription{StartPoint,0};
	// Emplace the starting node on the open list. The costs will be 0 at the starting point.
	OutSearchState.InternalSearchData->NodeList.Reset();
	OutSearchState.InternalSearchData->NodeList.Emplace(&OutSearchState.InternalSearchData->InternalStartPointDescription, /*InParent=*/INDEX_NONE, /*InCost=*/0.0, /*HeuristicCost=*/0.0);
	OutSearchState.InternalSearchData->OpenIndexList.Reset();
	// Okay to add the first index this way, since it will be "heapified" by default with one element.
	OutSearchState.InternalSearchData->OpenIndexList.Add(0);
	OutSearchState.InternalSearchData->ClosedIndexList.Reset();
	OutSearchState.InternalSearchData->PointDescriptionToNodeIndexMap.Reset();
	
	// Since we will use pointer to PontDescription, we should create them all in the array so their ptr is fixed.
	int32 OrigNumPoint = OutSearchState.OriginatingPointData->GetNumPoints();
	OutSearchState.InternalSearchData->InternalOriginatingPointsDescriptions.Empty();
	OutSearchState.InternalSearchData->InternalOriginatingPointsDescriptions.Reserve(OrigNumPoint);
	for (int32 Index = 0; Index < OrigNumPoint; ++Index)
	{
		OutSearchState.InternalSearchData->InternalOriginatingPointsDescriptions.Emplace(FPointDescription{{OutSearchState.OriginatingPointData, 0, OrigNumPoint}, Index});
	}
	
	int32 GoalNumPoint = OutSearchState.GoalPoints.RangeSize;
	OutSearchState.InternalSearchData->InternalGoalPointsDescriptions.Empty();
	OutSearchState.InternalSearchData->InternalGoalPointsDescriptions.Reserve(GoalNumPoint);
	for (int32 Index = 0; Index < GoalNumPoint; ++Index)
	{
		OutSearchState.InternalSearchData->InternalGoalPointsDescriptions.Emplace(FPointDescription{OutSearchState.GoalPoints, Index});
	}
}

/** Runs a single iteration of the A* algorithm. Intended to be called multiple times, whereupon it will return true when the algorithm is finished. */
ESearchResult ExecuteSearchIteration(const FSearchSettings& SearchSettings, FSearchState& SearchState, TArray<FPointDescription>& OutPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPathfindingElement::Algorithm::ExecuteSearchIteration);
	check(SearchState.OriginatingPointData);

	// The goal points should be guaranteed by the caller. Must call Initialize first to create the open index list.
	if ((SearchState.GoalPoints.RangeSize==0) || !ensure(!SearchState.InternalSearchData->OpenIndexList.IsEmpty()))
	{
		return ESearchResult::Invalid;
	}

	FSearchState::FInternalSearchData::FNodeList& NodeList = SearchState.InternalSearchData->NodeList;

	auto HeapPredicate = [&SearchState](int32 Index1, int32 Index2) { return Helpers::CompareNodes(SearchState.InternalSearchData->NodeList[Index1], SearchState.InternalSearchData->NodeList[Index2]); };

	// Get the lowest cost point on the list, which has been binary heap sorted.
	int32 CurrentNodeIndex;
	SearchState.InternalSearchData->OpenIndexList.HeapPop(CurrentNodeIndex, HeapPredicate, EAllowShrinking::No);

	TTuple<int32, double> GoalInfo = Helpers::GetNearestGoalToLocation(SearchState, *NodeList[CurrentNodeIndex].PointDescription);
	const int32 CurrentGoalIndex = (GoalInfo.Get<0>());
	const double MinDistanceToGoal = GoalInfo.Get<1>();
	const FVector& CurrentGoal = SearchState.InternalSearchData->InternalGoalPointsDescriptions[CurrentGoalIndex].GetLocation();

	auto PerPointProcessing = [&NodeList, CurrentNodeIndex, &CurrentGoal, &HeapPredicate, &SearchSettings, &SearchState](const FPointDescription* PointDescription, const double DistanceToPointSquared)
	{
		const bool bTrackedPoint = SearchState.InternalSearchData->PointDescriptionToNodeIndexMap.Contains(PointDescription);
		// Node has already been ruled out.
		if (bTrackedPoint && SearchState.InternalSearchData->ClosedIndexList.Contains(SearchState.InternalSearchData->PointDescriptionToNodeIndexMap[PointDescription]))
		{
			return;
		}

		const double TentativeNewLocalCost = NodeList[CurrentNodeIndex].LocalCost + SearchState.CostFunction(*NodeList[CurrentNodeIndex].PointDescription, DistanceToPointSquared, *PointDescription);

		// Blocked paths don't need expansion here, and serve no purpose at this point (at least from that direction) from being visited.
		if (TentativeNewLocalCost == std::numeric_limits<double>::max())
		{
			return;
		}

		/** Implementation Note: If there are multiple goals, a heuristic to a specific goal is not valid. The goal
		 * can not be changed during the pathfinding process, as neighbor evaluation is tied to the heuristic to the
		 * current goal at any point. Nodes may be ruled out and added to the Closed List
		 */
		auto GetHeuristicCost = [&SearchSettings, &SearchState](const FVector& CurrentPoint, const FVector& GoalPoint) -> double
		{
			return (SearchState.GoalPoints.RangeSize > 1) ? 0.0 : SearchSettings.HeuristicWeight * SearchState.HeuristicFunction(CurrentPoint, GoalPoint);
		};

		// Not tracking this point yet--add it to the node list and map it.
		if (!bTrackedPoint)
		{
			const double HeuristicCost = GetHeuristicCost(PointDescription->GetLocation(), CurrentGoal); 
			NodeList.Emplace(PointDescription, CurrentNodeIndex, TentativeNewLocalCost, TentativeNewLocalCost + HeuristicCost);

			const int32 NewNodeIndex = NodeList.Num() - 1;
			SearchState.InternalSearchData->OpenIndexList.HeapPush(NewNodeIndex, HeapPredicate);
			SearchState.InternalSearchData->PointDescriptionToNodeIndexMap.Emplace(PointDescription, NewNodeIndex);

			return;
		}

		// Check if the path to this neighbor is a better path than its current one. If so, update accordingly.
		const int32 NeighborIndex = SearchState.InternalSearchData->PointDescriptionToNodeIndexMap[PointDescription];
		FNode& Neighbor = SearchState.InternalSearchData->NodeList[NeighborIndex];
		if (TentativeNewLocalCost < Neighbor.LocalCost)
		{
			const int32 CurrentOpenListIndex = SearchState.InternalSearchData->OpenIndexList.Find(NeighborIndex);
			check(CurrentOpenListIndex != INDEX_NONE);

			// The priority may have changed, so remove the index and add it again.
			SearchState.InternalSearchData->OpenIndexList.HeapRemoveAt(CurrentOpenListIndex, HeapPredicate, EAllowShrinking::No);

			const double HeuristicCost = GetHeuristicCost(PointDescription->GetLocation(), CurrentGoal);
			Neighbor.PreviousNodeIndex = CurrentNodeIndex;
			Neighbor.LocalCost = TentativeNewLocalCost;
			Neighbor.EstimatedGoalCost = TentativeNewLocalCost + HeuristicCost;

			SearchState.InternalSearchData->OpenIndexList.HeapPush(NeighborIndex, HeapPredicate);
		}
	};

	// Arrived at the goal.
	if (FMath::IsNearlyZero(MinDistanceToGoal))
	{
		Helpers::BuildFinalPath(SearchState, SearchState.InternalSearchData->NodeList[CurrentNodeIndex], SearchSettings.bCopyOriginatingPoints, OutPath);
		return ESearchResult::Complete;
	}
	// Close enough to the goal we need to add it as a possibility.
	else if (MinDistanceToGoal <= SearchSettings.SearchDistance)
	{
		PerPointProcessing(&SearchState.InternalSearchData->InternalGoalPointsDescriptions[CurrentGoalIndex], MinDistanceToGoal*MinDistanceToGoal);
	}

	// Gather neighbors within the search radius.
	UPCGOctreeQueries::ForEachPointInsideSphere(
		SearchState.OriginatingPointData,
		NodeList[CurrentNodeIndex].GetLocation(),
 		SearchSettings.SearchDistance,
		[&PerPointProcessing, &OriginatingPointsDescriptions=SearchState.InternalSearchData->InternalOriginatingPointsDescriptions](const UPCGBasePointData* BasePointData, int32 PointIndex, const double DistanceToPointSquared)
		{
			PerPointProcessing(&(OriginatingPointsDescriptions[PointIndex]), DistanceToPointSquared);
		});

	// This node has been completely evaluated.
	SearchState.InternalSearchData->ClosedIndexList.Add(CurrentNodeIndex);

	// Final point to check, no path can be found.
	if (SearchState.InternalSearchData->OpenIndexList.IsEmpty())
	{
		OutPath.Reset();

		if (SearchSettings.bAcceptPartialPath)
		{
			// Find the closest point to any goal and walk back the path from there.
			int32 ClosestNodeIndex = INDEX_NONE;
			double ClosestDistance = std::numeric_limits<double>::max();
			for (const int32 NodeIndex : SearchState.InternalSearchData->ClosedIndexList)
			{
				TTuple<int32, double> ClosestGoalInfo = Helpers::GetNearestGoalToLocation(SearchState, *SearchState.InternalSearchData->NodeList[NodeIndex].PointDescription);
				const double CurGoalDistance = ClosestGoalInfo.Get<1>();
				if (CurGoalDistance < ClosestDistance)
				{
					ClosestDistance = CurGoalDistance;
					ClosestNodeIndex = NodeIndex;
				}
			}

			Helpers::BuildFinalPath(SearchState, SearchState.InternalSearchData->NodeList[ClosestNodeIndex], SearchSettings.bCopyOriginatingPoints, OutPath);
			return ESearchResult::Partial;
		}
		else
		{
			return ESearchResult::Invalid;
		}
	}

	return ESearchResult::Processing;
}
} // namespace PCGSpatialAlgo::AStar