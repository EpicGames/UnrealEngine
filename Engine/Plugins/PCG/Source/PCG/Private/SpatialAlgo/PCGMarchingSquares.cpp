// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialAlgo/PCGMarchingSquares.h"

#include "Containers/Map.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Tuple.h"

namespace PCGSpatialAlgo
{
namespace MarchingSquaresAlgo
{
	/**
	* Edge Ids
	* *---0---*
	* |       |
	* 3       1
	* |       |
	* *---2---*
	*/
	enum class Edge : uint8
	{
		Id_0 = 1 << 0,
		Id_1 = 1 << 1,
		Id_2 = 1 << 2,
		Id_3 = 1 << 3,
		Invalid = 1 << 7,

		AllEdges = Id_0 | Id_1 | Id_2 | Id_3
	};

	// Define it as flags so that we can use it as a bitfield for masking seen edges.
	ENUM_CLASS_FLAGS(Edge);

	struct Params
	{
		int32 CellCountX;
		int32 CellCountY;

		void SplitIndex(int32 Index, int32& OutX, int32& OutY) const
		{
			OutX = Index / (CellCountY - 1);
			OutY = Index % (CellCountY - 1);
		}

		/**
		* Return the next index of the grid according to the edge Id
		* If we go out of the grid, return INDEX_NONE.
		*/
		int32 GetNextIndex(int32 X, int32 Y, Edge EdgeId) const
		{
			switch (EdgeId)
			{
			case Edge::Id_0:
				--X;
				break;
			case Edge::Id_1:
				++Y;
				break;
			case Edge::Id_2:
				++X;
				break;
			case Edge::Id_3:
				--Y;
				break;
			default:
				checkNoEntry();
				return INDEX_NONE;
			}

			if (X < 0 || X >= (CellCountX - 1) || Y < 0 || Y >= (CellCountY - 1))
			{
				return INDEX_NONE;
			}
			else
			{
				return X * (CellCountY - 1) + Y;
			}
		}
	};

	/** Returns the matching edge on the neighbor cell */
	Edge GetNextEdge(Edge EdgeId)
	{
		// The edge Id for 0 will be 2 and vice versa and for 3 it will be 1 and vice versa
		switch (EdgeId)
		{
		case Edge::Id_0:
			return Edge::Id_2;
		case Edge::Id_1:
			return Edge::Id_3;
		case Edge::Id_2:
			return Edge::Id_0;
		case Edge::Id_3:
			return Edge::Id_1;
		default:
			checkNoEntry();
			return Edge::Invalid;
		}
	}

	/**
	* Return the edge ID corresponding to the input edge id.
	* cf. header for the different types
	*/
	Edge GetEdgeId(uint8 Type, Edge InputEdge)
	{
		switch (Type)
		{
		case 1: // fall-through
		case 14:
			return InputEdge == Edge::Id_2 ? Edge::Id_3 : Edge::Id_2;
		case 2: // fall-through
		case 13:
			return InputEdge == Edge::Id_1 ? Edge::Id_2 : Edge::Id_1;
		case 3: // fall-through
		case 12:
			return InputEdge == Edge::Id_1 ? Edge::Id_3 : Edge::Id_1;
		case 4: // fall-through
		case 11:
			return InputEdge == Edge::Id_0 ? Edge::Id_1 : Edge::Id_0;
		case 5:
		{
			switch (InputEdge)
			{
			case Edge::Id_1:
				return Edge::Id_2;
			case Edge::Id_2:
				return Edge::Id_1;
			case Edge::Id_3:
				return Edge::Id_0;
			case Edge::Id_0: // fall-through
			default:
				return Edge::Id_3;
			}
		}
		case 6: // fall-through
		case 9:
			return InputEdge == Edge::Id_0 ? Edge::Id_2 : Edge::Id_0;
		case 7: // fall-through
		case 8:
			return InputEdge == Edge::Id_3 ? Edge::Id_0 : Edge::Id_3;
		case 10:
		{
			switch (InputEdge)
			{
			case Edge::Id_1:
				return Edge::Id_0;
			case Edge::Id_2:
				return Edge::Id_3;
			case Edge::Id_3:
				return Edge::Id_2;
			case Edge::Id_0: // fall-through
			default:
				return Edge::Id_1;
			}
		}
		default:
			checkNoEntry();
			return Edge::Invalid;
		}
	}

	/**
	* Return the starting edges for a given type, with the first one being the tail, and the second the head.
	* Since all types have their mirror (like type 1 and 14), we switch the order of the edges, depending if Type < 8 or not,
	* since it switches the gradient direction.
	*/
	TTuple<Edge, Edge> GetStartingEdges(uint8 InType, Edge InEdgeMask)
	{
		// For special case 5 and 10. If Edge 0 has not yet been seen, start with 0. Otherwise, start with 2 (in the right order).
		if (InType == 5)
		{
			return EnumHasAnyFlags(InEdgeMask, Edge::Id_0) ? TTuple<Edge, Edge>{ Edge::Id_3, Edge::Id_0 } : TTuple<Edge, Edge>{ Edge::Id_1, Edge::Id_2 };
		}
		else if (InType == 10)
		{
			return EnumHasAnyFlags(InEdgeMask, Edge::Id_0) ? TTuple<Edge, Edge>{ Edge::Id_0, Edge::Id_1 } : TTuple<Edge, Edge>{ Edge::Id_2, Edge::Id_3 };
		}
		else
		{
			// Bootstrap with Invalid edge
			const Edge FirstEdge = GetEdgeId(InType, Edge::Invalid);
			const Edge SecondEdge = GetEdgeId(InType, FirstEdge);

			return InType >= 8 ? TTuple<Edge, Edge>{ FirstEdge, SecondEdge } : TTuple<Edge, Edge>{ SecondEdge, FirstEdge };
		}
	}

	/**
	* For a given edge id, at coordinate X,Y, return it's actual "real" coordinate on the grid.
	* If there is no linear interpolation, it will be the middle of the edge.
	* If there is linear interpolation, we interpolate linearly where the isovalue would sit on the edge.
	*/
	FVector2D GetEdgeCoordinate(int32 X, int32 Y, Edge EdgeId, const double& Isovalue, TFunctionRef<double(int32,int32)> ValueQuery, bool bUseLinearInterpolation)
	{
		FVector2D Result(X, Y);

		double Ratio = 0.5;
		if (bUseLinearInterpolation)
		{
			double Value1{};
			double Value2{};
			switch (EdgeId)
			{
			case Edge::Id_0:
				Value1 = ValueQuery(X, Y);
				Value2 = ValueQuery(X, Y + 1);
				break;
			case Edge::Id_1:
				Value1 = ValueQuery(X, Y + 1);
				Value2 = ValueQuery(X + 1, Y + 1);
				break;
			case Edge::Id_2:
				Value1 = ValueQuery(X + 1, Y);
				Value2 = ValueQuery(X + 1, Y + 1);
				break;
			case Edge::Id_3:
				Value1 = ValueQuery(X, Y);
				Value2 = ValueQuery(X + 1, Y);
				break;
			default:
				checkNoEntry();
				break;
			}

			// If Value1 ~= Value2, we expect 0.5 to be a correct approximation, so keep the default value.
			if (!FMath::IsNearlyEqual(Value1, Value2))
			{
				Ratio = (Isovalue - Value1) / (Value2 - Value1);
			}
		}

		check(Ratio >= 0.0 && Ratio <= 1.0);

		switch (EdgeId)
		{
		case Edge::Id_0:
			Result.Y += Ratio;
			break;
		case Edge::Id_1:
			Result.Y += 1;
			Result.X += Ratio;
			break;
		case Edge::Id_2:
			Result.X += 1;
			Result.Y += Ratio;
			break;
		case Edge::Id_3:
			Result.X += Ratio;
			break;
		default:
			checkNoEntry();
			break;
		}

		return Result;
	};

	struct MarchingArgs
	{
		TArray<uint8>& Types;
		TMap<int32, Edge>& Candidates;
		Params IndexParams;
		double Isovalue;
		TFunctionRef<double(int32, int32)> ValueQuery;
		bool bUseLinearInterpolation;
		TArray<FVector2D>* CurrentLine = nullptr;
	};

	/** Algorithm that will start at a given index and will march following the contour defined by the types until it reaches the beginning (loop) or the edge of the map (not closed loop) */
	void Marching(MarchingArgs& Args, int32 StartingIndex, Edge StartingEdgeId)
	{
		auto MarkEdgeSeenAndRemove = [&Candidates = Args.Candidates](int32 Index, Edge EdgeId)
		{
			check(Candidates.Contains(Index));

			EnumRemoveFlags(Candidates[Index], EdgeId);

			// If we have seen all the edges, remove it.
			if (!Candidates[Index])
			{
				Candidates.Remove(Index);
			}
		};

		int32 Index = StartingIndex;
		Edge EdgeId = StartingEdgeId;

		check(Args.CurrentLine);

		while (true)
		{
			Edge NextEdgeId = GetEdgeId(Args.Types[Index], EdgeId);

			int32 X, Y;
			Args.IndexParams.SplitIndex(Index, X, Y);

			Args.CurrentLine->Add(GetEdgeCoordinate(X, Y, NextEdgeId, Args.Isovalue, Args.ValueQuery, Args.bUseLinearInterpolation));

			MarkEdgeSeenAndRemove(Index, NextEdgeId);

			const int32 NextIndex = Args.IndexParams.GetNextIndex(X, Y, NextEdgeId);
			if (NextIndex == INDEX_NONE || !Args.Candidates.Contains(NextIndex))
			{
				// Getting out of the grid or we reached the beginning (so we are in a loop), stop there on that path
				return;
			}

			NextEdgeId = GetNextEdge(NextEdgeId);

			// In case 5 and 10, we can still have the cell in the candidates. But if we've already seen that edge, it means we are in a loop, so we need to stop there.
			if (!EnumHasAnyFlags(Args.Candidates[NextIndex], NextEdgeId))
			{
				break;
			}

			// Also mark that one as seen
			MarkEdgeSeenAndRemove(NextIndex, NextEdgeId);

			// If the next index was removed, it means we closed the line, so we can stop there
			if (!Args.Candidates.Contains(NextIndex))
			{
				break;
			}

			// Otherwise continue marching!
			Index = NextIndex;
			EdgeId = NextEdgeId;
		}
	};
}
}

TArray<PCGSpatialAlgo::FPCGMarchingSquareResult> PCGSpatialAlgo::MarchingSquares(const int32 CellCountX, const int32 CellCountY, const double Isovalue, TFunctionRef<double(int32 X, int32 Y)> ValueQuery, bool bUseLinearInterpolation)
{
	if (CellCountX < 1 || CellCountY < 1)
	{
		return {};
	}

	using Edge = MarchingSquaresAlgo::Edge;

	MarchingSquaresAlgo::Params Params{ CellCountX, CellCountY };

	// First generate all the types for each cell. We compute the type depending on the value of the 4 corners of our base map
	// so the "types" grid is smaller by 1 one on each axis (hence the -1 on cell count)
	TArray<uint8> Types;
	Types.SetNumUninitialized((Params.CellCountX - 1) * (Params.CellCountY - 1));

	// Will hold all the indexes where we have an edge, and their seen edge ids stored as a bitfield, as an EdgeMask.
	// We'll use the map to determine the candidates that have not yet been processed, and also
	// handle the case 5 and 10 where we have two edges in one cell (so we need to know if one edge was processed but not the other)
	TMap<int32, Edge> Candidates;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMarchingSquares::GatheringCandidates);

		for (int32 X = 0; X < Params.CellCountX - 1; ++X)
		{
			for (int32 Y = 0; Y < Params.CellCountY - 1; ++Y)
			{
				const int32 Index = X * (Params.CellCountY - 1) + Y;
				uint8& CurrentType = Types[Index];
				CurrentType = 0;

				// We store our type on a uint8, where we only use 4 bits.
				// For the list of types, cf. the header.
				if (ValueQuery(X, Y) >= Isovalue) { CurrentType |= (1 << 3); } // TopLeft, bit 3
				if (ValueQuery(X, Y + 1) >= Isovalue) { CurrentType |= (1 << 2); } // TopRight, bit 2
				if (ValueQuery(X + 1, Y + 1) >= Isovalue) { CurrentType |= (1 << 1); } // BottomRight, bit 1
				if (ValueQuery(X + 1, Y) >= Isovalue) { CurrentType |= (1 << 0); } // BottomLeft, bit 0

				// Types 0 and 15 mean that we have no edges.
				if (CurrentType == 0 || CurrentType == 15)
				{
					continue;
				}

				if (CurrentType == 5 || CurrentType == 10)
				{
					// All edges on
					Candidates.Add(Index, Edge::AllEdges);
				}
				else
				{
					// Bootstrapping with Invalid edge, this will give us the first edge
					// We can then re-use that first edge to get the second edge.
					const Edge FirstEdge = MarchingSquaresAlgo::GetEdgeId(CurrentType, Edge::Invalid);
					const Edge SecondEdge = MarchingSquaresAlgo::GetEdgeId(CurrentType, FirstEdge);
					check(FirstEdge != Edge::Invalid && SecondEdge != Edge::Invalid);

					Candidates.Emplace(Index, FirstEdge | SecondEdge);
				}
			}
		}
	}

	// Then we will pick up an index at random (first value of the candidate set)
	// and will go in both directions to connect all the edges.
	// We will fill 2 arrays, one for tail for for head.
	// We make sure to have tail going clockwise according to the value gradient (and the head counterclockwise)
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMarchingSquares::ReconnectTheDots);

	TArray<FPCGMarchingSquareResult> FinalResult;
	while (!Candidates.IsEmpty())
	{
		int32 StartingIndex = Candidates.FindArbitraryElement()->Key;
		Edge StartingEdgeMask = Candidates.FindArbitraryElement()->Value;
		check(!!StartingEdgeMask);

		TArray<FVector2D> Head;
		TArray<FVector2D> Tail;

		const TTuple<Edge, Edge> StartingEdges = MarchingSquaresAlgo::GetStartingEdges(Types[StartingIndex], StartingEdgeMask);

		const Edge FirstEdgeId = StartingEdges.Get<0>();
		const Edge SecondEdgeId = StartingEdges.Get<1>();

		MarchingSquaresAlgo::MarchingArgs Args{ Types, Candidates, Params, Isovalue, ValueQuery, bUseLinearInterpolation };

		// Run it first as the head, on the first edge.
		Args.CurrentLine = &Head;
		MarchingSquaresAlgo::Marching(Args, StartingIndex, FirstEdgeId);

		// Then we check if starting index is not in the candidates or if the first edge was seen, if so it means it is closed (head reached the tail).
		const bool bIsClosed = !Candidates.Contains(StartingIndex) || !EnumHasAnyFlags(Candidates[StartingIndex], FirstEdgeId);
		if (bIsClosed)
		{
			// Move the head, and pop the last point as it was duplicated.
			FinalResult.Add_GetRef({ std::move(Head), true }).LinkedGridCoordinates.Pop();
		}
		else
		{
			// Otherwise, we start again with the second edge, going backward on the tail.
			Args.CurrentLine = &Tail;
			MarchingSquaresAlgo::Marching(Args, StartingIndex, SecondEdgeId);

			// Since we went backward, we reverse the tail.
			Algo::Reverse(Tail);
			// And add the head.
			Tail.Append(std::move(Head));
			FinalResult.Add({ std::move(Tail), false });
		}
	}

	return FinalResult;
}