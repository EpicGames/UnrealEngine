// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialAlgo/PCGOctreeQueries.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGOctreeQueries)

namespace PCGOctreeQueries
{
	/** Factorize the logic for all GetClosest/GetFarthest, as the only thing that changes is the StartingDistance and the Condition. */
	template <typename Func = TFunctionRef<bool(double ChosenPointDistance, const UPCGBasePointData* BasePointData, int32 PointIndex, double SquaredDistance)>>
	const int32 GetPointIndexInSphereUnderCondition(const UPCGBasePointData* InPointData, const FVector& InCenter, const double InSearchDistance, const double InStartingDistance, Func Condition)
	{
		if (!InPointData)
		{
			return INDEX_NONE;
		}

		int32 ChosenPointIndex = INDEX_NONE;
		const double SearchDistance = FMath::Max(InSearchDistance, 0.0);

		double ChosenPointDistance = InStartingDistance;
		UPCGOctreeQueries::ForEachPointInsideSphere(InPointData, InCenter, SearchDistance, [&ChosenPointDistance, &ChosenPointIndex, Condition = std::move(Condition)](const UPCGBasePointData* BasePointData, int32 PointIndex, double SquaredDistance)
		{
			if (Condition(ChosenPointDistance, BasePointData, PointIndex, SquaredDistance))
			{
				ChosenPointDistance = SquaredDistance;
				ChosenPointIndex = PointIndex;
			}
		});

		return ChosenPointIndex;
	}
}

TArray<FPCGPoint> UPCGOctreeQueries::GetPointsInsideBounds(const UPCGPointData* InPointData, const FBox& InBounds)
{
	if (!InPointData)
	{
		return {};
	}

	TArray<FPCGPoint> Result;
	const TArray<FPCGPoint>& Points = InPointData->GetPoints();

	InPointData->GetPointOctree().FindElementsWithBoundsTest(InBounds, [&Points, &Result](const PCGPointOctree::FPointRef& PointRef)
	{
		if (Points.IsValidIndex(PointRef.Index))
		{
			Result.Add(Points[PointRef.Index]);
		}
	});

	return Result;
}

TArray<FPCGPoint> UPCGOctreeQueries::GetPointsInsideSphere(const UPCGPointData* InPointData, const FVector& InCenter, const double InRadius)
{
	if (!InPointData)
	{
		return {};
	}

	TArray<FPCGPoint> Result;
	ForEachPointInsideSphere(InPointData, InCenter, InRadius, [&Result](const FPCGPoint& Point, double)
	{
		Result.Add(Point);
	});

	return Result;
}

void UPCGOctreeQueries::GetClosestPoint(const UPCGPointData* InPointData, const FVector& InCenter, const bool bInDiscardCenter, bool& bOutFound, FPCGPoint& OutPoint, const double InSearchDistance)
{
	bOutFound = false;
	if (const FPCGPoint* ClosestPoint = GetClosestPoint(InPointData, InCenter, bInDiscardCenter, InSearchDistance))
	{
		bOutFound = true;
		OutPoint = *ClosestPoint;
	}
}

const FPCGPoint* UPCGOctreeQueries::GetClosestPoint(const UPCGPointData* InPointData, const FVector& InCenter, const bool bInDiscardCenter, const double InSearchDistance)
{
	const int32 PointIndex = GetClosestPointIndex(InPointData, InCenter, bInDiscardCenter, InSearchDistance);
	return PointIndex != INDEX_NONE ? &InPointData->GetPoints()[PointIndex] : nullptr;
}

int32 UPCGOctreeQueries::GetClosestPointIndex(const UPCGBasePointData* InPointData, const FVector& InCenter, const bool bInDiscardCenter, const double InSearchDistance)
{
	return PCGOctreeQueries::GetPointIndexInSphereUnderCondition(InPointData, InCenter, InSearchDistance, std::numeric_limits<double>::max(),
		[bInDiscardCenter](double ChosenPointDistance, const UPCGBasePointData* BasePointData, int32 PointIndex, double SquaredDistance)
		{
			return SquaredDistance <= ChosenPointDistance && (!bInDiscardCenter || SquaredDistance > UE_DOUBLE_SMALL_NUMBER);
		});
}

void UPCGOctreeQueries::GetClosestPointFromOtherPoint(const UPCGPointData* InPointData, const int32 InPointIndex, bool& bOutFound, FPCGPoint& OutPoint, const double InSearchDistance)
{
	bOutFound = false;
	if (!InPointData)
	{
		return;
	}

	if (!InPointData->GetPoints().IsValidIndex(InPointIndex))
	{
		return;
	}

	if (const FPCGPoint* ClosestPoint = GetClosestPointFromOtherPoint(InPointData, InPointData->GetPoints()[InPointIndex], InSearchDistance))
	{
		bOutFound = true;
		OutPoint = *ClosestPoint;
	}
}

const FPCGPoint* UPCGOctreeQueries::GetClosestPointFromOtherPoint(const UPCGPointData* InPointData, const FPCGPoint& InPoint, const double InSearchDistance)
{
	const int32 PointIndex = PCGOctreeQueries::GetPointIndexInSphereUnderCondition(InPointData, InPoint.Transform.GetLocation(), InSearchDistance, std::numeric_limits<double>::max(),
		[&InPoint](double ChosenPointDistance, const UPCGBasePointData* BasePointData, int32 PointIndex, double SquaredDistance)
		{
			const UPCGPointData* PointData = CastChecked<UPCGPointData>(BasePointData);
			return SquaredDistance <= ChosenPointDistance && &InPoint != &PointData->GetPoints()[PointIndex];
		});
	return PointIndex != INDEX_NONE ? &InPointData->GetPoints()[PointIndex] : nullptr;
}

int32 UPCGOctreeQueries::GetClosestPointIndexFromOtherPointIndex(const UPCGBasePointData* InPointData, int32 OtherPointIndex, const double InSearchDistance)
{
	const TConstPCGValueRange<FTransform> TransformRange = InPointData->GetConstTransformValueRange();
	if (!TransformRange.IsValidIndex(OtherPointIndex))
	{
		return INDEX_NONE;
	}

	const int32 PointIndex = PCGOctreeQueries::GetPointIndexInSphereUnderCondition(InPointData, TransformRange[OtherPointIndex].GetLocation(), InSearchDistance, std::numeric_limits<double>::max(),
		[OtherPointIndex](double ChosenPointDistance, const UPCGBasePointData* BasePointData, int32 PointIndex, double SquaredDistance)
		{
			return SquaredDistance <= ChosenPointDistance && PointIndex != OtherPointIndex;
		});
	return PointIndex;
}

void UPCGOctreeQueries::GetFarthestPoint(const UPCGPointData* InPointData, const FVector& InCenter, bool& bOutFound, FPCGPoint& OutPoint, const double InSearchDistance)
{
	bOutFound = false;
	if (const FPCGPoint* FarthestPoint = GetFarthestPoint(InPointData, InCenter, InSearchDistance))
	{
		bOutFound = true;
		OutPoint = *FarthestPoint;
	}
}

const FPCGPoint* UPCGOctreeQueries::GetFarthestPoint(const UPCGPointData* InPointData, const FVector& InCenter, const double InSearchDistance)
{
	const int32 PointIndex = GetFarthestPointIndex(InPointData, InCenter, InSearchDistance);
	return PointIndex != INDEX_NONE ? &InPointData->GetPoints()[PointIndex] : nullptr;
}

int32 UPCGOctreeQueries::GetFarthestPointIndex(const UPCGBasePointData* InPointData, const FVector& InCenter, const double InSearchDistance)
{
	return PCGOctreeQueries::GetPointIndexInSphereUnderCondition(InPointData, InCenter, InSearchDistance, std::numeric_limits<double>::min(),
		[](double ChosenPointDistance, const UPCGBasePointData* BasePointData, int32 PointIndex, double SquaredDistance)
		{
			return SquaredDistance >= ChosenPointDistance;
		});
}

void UPCGOctreeQueries::GetFarthestPointFromOtherPoint(const UPCGPointData* InPointData, const int32 InPointIndex, bool& bOutFound, FPCGPoint& OutPoint, const double InSearchDistance)
{
	bOutFound = false;
	if (!InPointData)
	{
		return;
	}

	if (!InPointData->GetPoints().IsValidIndex(InPointIndex))
	{
		return;
	}

	if (const FPCGPoint* ClosestPoint = GetFarthestPointFromOtherPoint(InPointData, InPointData->GetPoints()[InPointIndex], InSearchDistance))
	{
		bOutFound = true;
		OutPoint = *ClosestPoint;
	}
}

const FPCGPoint* UPCGOctreeQueries::GetFarthestPointFromOtherPoint(const UPCGPointData* InPointData, const FPCGPoint& InPoint, const double InSearchDistance)
{
	const int32 PointIndex = PCGOctreeQueries::GetPointIndexInSphereUnderCondition(InPointData, InPoint.Transform.GetLocation(), InSearchDistance, std::numeric_limits<double>::min(),
		[&InPoint](double ChosenPointDistance, const UPCGBasePointData* BasePointData, int32 PointIndex, double SquaredDistance)
		{
			const UPCGPointData* PointData = CastChecked<UPCGPointData>(BasePointData);
			return SquaredDistance >= ChosenPointDistance && &InPoint != &PointData->GetPoints()[PointIndex];
		});
	return PointIndex != INDEX_NONE ? &InPointData->GetPoints()[PointIndex] : nullptr;
}

int32 UPCGOctreeQueries::GetFarthestPointIndexFromOtherPointIndex(const UPCGBasePointData* InPointData, int32 OtherPointIndex, const double InSearchDistance)
{
	const TConstPCGValueRange<FTransform> TransformRange = InPointData->GetConstTransformValueRange();
	if (!TransformRange.IsValidIndex(OtherPointIndex))
	{
		return INDEX_NONE;
	}

	const int32 PointIndex = PCGOctreeQueries::GetPointIndexInSphereUnderCondition(InPointData, TransformRange[OtherPointIndex].GetLocation(), InSearchDistance, std::numeric_limits<double>::min(),
		[OtherPointIndex](double ChosenPointDistance, const UPCGBasePointData* BasePointData, int32 PointIndex, double SquaredDistance)
		{
			return SquaredDistance >= ChosenPointDistance && PointIndex != OtherPointIndex;
		});
	return PointIndex;
}
