// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/IPCGBaseSubsystem.h"

#include "PCGComponent.h"
#include "PCGDefaultExecutionSource.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "PCGProfilingLog.h"
#include "PCGTrackingManager.h"
#include "Editor/IPCGEditorModule.h"
#include "Graph/PCGGraphCache.h"
#include "Graph/PCGGraphCompiler.h"
#include "Graph/PCGGraphExecutor.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "RuntimeGen/PCGGenSourceManager.h"

#if WITH_EDITOR
#include "Editor.h"
#include "PackageSourceControlHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(IPCGBaseSubsystem)

void IPCGBaseSubsystem::InitializeBaseSubsystem()
{
	check(!GraphExecutor);
	GraphExecutor = MakeShared<FPCGGraphExecutor>(GetSubsystemWorld());

#if WITH_EDITOR
	FPCGModule::GetPCGModuleChecked().OnGraphChanged().AddRaw(this, &IPCGBaseSubsystem::NotifyGraphChanged);
#endif
}

void IPCGBaseSubsystem::DeinitializeBaseSubsystem()
{
#if WITH_EDITOR
	FPCGModule::GetPCGModuleChecked().OnGraphChanged().RemoveAll(this);
#endif

	GraphExecutor = nullptr;
}

#if WITH_EDITOR
void IPCGBaseSubsystem::OnScheduleGraph(const FPCGStackContext& StackContext)
{
	// nothing to do for now
}
#endif

FPCGPerExecutionCache* IPCGBaseSubsystem::GetCacheInternal() const
{
	return GraphExecutor ? &GraphExecutor->PerExecutionCache : nullptr;
}

FPCGTaskId IPCGBaseSubsystem::ScheduleGraph(const FPCGScheduleGraphParams& InParams)
{
	if (InParams.ExecutionSource)
	{
		return GraphExecutor->ScheduleGraph(InParams);
	}
	else
	{
		return InvalidPCGTaskId;
	}
}

FPCGTaskId IPCGBaseSubsystem::ScheduleGeneric(const FPCGScheduleGenericParams& InParams)
{
	check(GraphExecutor);
	return GraphExecutor->ScheduleGeneric(InParams);
}

void IPCGBaseSubsystem::CancelGeneration(IPCGGraphExecutionSource* Source)
{
	CancelGeneration(Source, /*bCleanupUnusedResources=*/true);
}

double IPCGBaseSubsystem::GetTickEndTime() const
{
	if (GraphExecutor)
	{
		return FPlatformTime::Seconds() + GraphExecutor->GetTickBudgetInSeconds();
	}

	// No Graph Executor no budget
	return FPlatformTime::Seconds();
}

double IPCGBaseSubsystem::Tick()
{
	double EndTime = GetTickEndTime();

	// If we have any tasks to execute, schedule some
	if (GraphExecutor)
	{
		GraphExecutor->Execute(EndTime);
	}

	return EndTime;
}

void IPCGBaseSubsystem::CancelGeneration(IPCGGraphExecutionSource* Source, bool bCleanupUnusedResources)
{
	check(GraphExecutor && IsInGameThread());
	if (!Source || !Source->GetExecutionState().IsGenerating())
	{
		return;
	}

	CancelGenerationInternal(Source, bCleanupUnusedResources);

	TArray<IPCGGraphExecutionSource*> CancelledExecutionSources = GraphExecutor->Cancel(Source);
	for (IPCGGraphExecutionSource* CancelledExecutionSource : CancelledExecutionSources)
	{
		if (CancelledExecutionSource)
		{
			CancelledExecutionSource->GetExecutionState().OnGraphExecutionAborted(/*bQuiet=*/true, bCleanupUnusedResources);
		}
	}	
}

void IPCGBaseSubsystem::CancelGeneration(UPCGGraph* Graph)
{
	check(GraphExecutor);

	if (!Graph)
	{
		return;
	}

	TArray<IPCGGraphExecutionSource*> CancelledExecutionSources = GraphExecutor->Cancel(Graph);
	for (IPCGGraphExecutionSource* CancelledExecutionSource : CancelledExecutionSources)
	{
		if (ensure(CancelledExecutionSource))
		{
			CancelledExecutionSource->GetExecutionState().OnGraphExecutionAborted(/*bQuiet=*/true);
		}
	}
}

void IPCGBaseSubsystem::CancelAllGeneration()
{
	check(GraphExecutor);

	TArray<IPCGGraphExecutionSource*> CancelledExecutionSources = GraphExecutor->CancelAll();
	for (IPCGGraphExecutionSource* CancelledExecutionSource : CancelledExecutionSources)
	{
		if (ensure(CancelledExecutionSource))
		{
			CancelledExecutionSource->GetExecutionState().OnGraphExecutionAborted(/*bQuiet=*/true);
		}
	}
}

bool IPCGBaseSubsystem::IsGraphCurrentlyExecuting(UPCGGraph* Graph)
{
	check(GraphExecutor);

	if (!Graph)
	{
		return false;
	}

	return GraphExecutor->IsGraphCurrentlyExecuting(Graph);
}

bool IPCGBaseSubsystem::IsAnyGraphCurrentlyExecuting() const
{
	return GraphExecutor && GraphExecutor->IsAnyGraphCurrentlyExecuting();
}

bool IPCGBaseSubsystem::IsGraphCacheDebuggingEnabled() const
{
	return GraphExecutor && GraphExecutor->IsGraphCacheDebuggingEnabled();
}

FPCGGraphCompiler* IPCGBaseSubsystem::GetGraphCompiler()
{
	return GraphExecutor ? GraphExecutor->GetCompiler() : nullptr;
}

// deprecated (5.8)
UPCGComputeGraph* IPCGBaseSubsystem::GetComputeGraph(const UPCGGraph* InGraph, uint32 GridSize, uint32 ComputeGraphIndex)
{
	if (FPCGGraphCompiler* GraphCompiler = GetGraphCompiler())
	{
		return GraphCompiler->GetComputeGraph(InGraph, GridSize, ComputeGraphIndex);
	}

	return nullptr;
}

bool IPCGBaseSubsystem::GetOutputData(FPCGTaskId TaskId, FPCGDataCollection& OutData)
{
	check(GraphExecutor);
	return GraphExecutor->GetOutputData(TaskId, OutData);
}

void IPCGBaseSubsystem::ClearOutputData(FPCGTaskId TaskId)
{
	check(GraphExecutor);
	GraphExecutor->ClearOutputData(TaskId);
}

#if WITH_EDITOR
void IPCGBaseSubsystem::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (FGraphExecution& GraphExecution : GraphExecutions)
	{
		Collector.AddReferencedObject(GraphExecution.ExecutionSource);
	}
}

void IPCGBaseSubsystem::NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType)
{
	if (GraphExecutor)
	{
		GraphExecutor->NotifyGraphChanged(InGraph, ChangeType);
	}
}

void IPCGBaseSubsystem::CleanFromCache(const IPCGElement* InElement, const UPCGSettings* InSettings/*= nullptr*/)
{
	if (GraphExecutor)
	{
		GraphExecutor->GetCache().CleanFromCache(InElement, InSettings);
	}
}

bool IPCGBaseSubsystem::GetStackContext(UPCGGraph* InGraph, uint32 InGridSize, bool bIsPartitioned, FPCGStackContext& OutStackContext)
{
	if (!InGraph)
	{
		return false;
	}

	// A non-partitioned component generally executes (original component or local component).
	if(bIsPartitioned)
	{
		// A partitioned higen original component will execute if the graph has UB grid level.
		if (InGraph->IsHierarchicalGenerationEnabled())
		{
			PCGHiGenGrid::FSizeArray GridSizes;
			bool bHasUnbounded = false;
			InGraph->GetGridSizes(GridSizes, bHasUnbounded);

			if (!bHasUnbounded)
			{
				return false;
			}
		}
		else
		{
			return false;
		}
	}

	if (FPCGGraphCompiler* GraphCompiler = GetGraphCompiler())
	{
		GraphCompiler->GetCompiledTasks(InGraph, InGridSize, OutStackContext, /*bIsTopGraph=*/false);
		return true;
	}
	else
	{
		return false;
	}
}

bool IPCGBaseSubsystem::GetStackContext(const IPCGGraphExecutionSource* InSource, FPCGStackContext& OutStackContext)
{
	const uint32 GridSize = InSource && InSource->GetExecutionState().IsLocalSource() ? InSource->GetExecutionState().GetGenerationGridSize() : PCGHiGenGrid::UnboundedGridSize();
	
	return InSource && GetStackContext(InSource->GetExecutionState().GetGraph(),GridSize, InSource->GetExecutionState().IsPartitioned(), OutStackContext);
}

uint32 IPCGBaseSubsystem::GetGraphCacheEntryCount(IPCGElement* InElement) const
{
	return GraphExecutor ? GraphExecutor->GetGraphCacheEntryCount(InElement) : 0;
}
#endif // WITH_EDITOR

#if PCG_PROFILING_ENABLED
void IPCGBaseSubsystem::OnPCGSourceGenerationDone(IPCGGraphExecutionSource* InExecutionSource, EPCGGenerationStatus InStatus)
{
	if (PCGProfilingLog::IsEnabled())
	{
		PCGProfilingLog::LogProfilingData(InExecutionSource, InStatus);
	}

#if WITH_EDITOR
	check(IsInGameThread());
	OnPCGSourceGenerationDoneDelegate.Broadcast(this, InExecutionSource, InStatus);

	UPCGDefaultExecutionSource* ExecutionSource = Cast<UPCGDefaultExecutionSource>(InExecutionSource);
	if (!ExecutionSource)
	{
		return;
	}

	const int32 Index = GraphExecutions.IndexOfByPredicate([ExecutionSource](const FGraphExecution& GraphExecution) { return ExecutionSource == GraphExecution.ExecutionSource.Get(); });

	if (Index != INDEX_NONE)
	{
		FGraphExecution& GraphExecution = GraphExecutions[Index];
		GraphExecution.GenerationCallback.Broadcast(this, ExecutionSource, InStatus);
		GraphExecutions.RemoveAtSwap(Index);

		if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
		{
			PCGEditorModule->ClearExecutionMetadata(ExecutionSource);
		}

		// Since the execution source is setup to be deleted after this execution, we need to forget all the managed resources of this source.
		FPCGManagedResourceContainerHelper ManagedResourceContainerHelper(ExecutionSource);
		if (ManagedResourceContainerHelper.IsValid())
		{
			ManagedResourceContainerHelper.ForgetAll();
		}

		// Marks the source in such a way that it will be removed from the PCG editor.
		ExecutionSource->MarkAsGarbage();
	}
#endif // WITH_EDITOR
}
#endif // PCG_PROFILING_ENABLED

#if WITH_EDITOR
void IPCGBaseSubsystem::SetDisableClearResults(bool bInDisableClearResults)
{
	if (GraphExecutor)
	{
		GraphExecutor->SetDisableClearResults(bInDisableClearResults);
	}
}

#endif // WITH_EDITOR

IPCGGraphCache* IPCGBaseSubsystem::GetCache()
{
	return GraphExecutor ? &(GraphExecutor->GetCache()) : nullptr;
}

void IPCGBaseSubsystem::FlushCache()
{
	if (GraphExecutor && GraphExecutor->GetCompiler())
	{
		GraphExecutor->GetCache().ClearCache();
		GraphExecutor->GetCompiler()->ClearCache();
	}

#if WITH_EDITOR
	// Garbage collection is very seldom run in the editor, but we currently can consume a lot of memory in the cache.
	// When running the GC while it is in a reconstruction script we actually destroy the old components that are being
	// reconstructed before the on object replaced broadcast is even called. Therefore it is not safe to run the GC while we are running a construction
	// script.
	const UWorld* World = GetSubsystemWorld();
	const bool bIsRunningConstructionScript = GIsReconstructingBlueprintInstances || (World && World->bIsRunningConstructionScript);
	if (GEngine && bIsRunningConstructionScript)
	{
		GEngine->ForceGarbageCollection(/*bPerformFullPurge=*/true);
	}
	else if (!bIsRunningConstructionScript && (!World || !World->IsGameWorld()))
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, /*bPerformFullPurge=*/true);
	}
#endif
}
