// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/PCGEngineSubsystem.h"

#include "PCGDefaultExecutionSource.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "Editor/IPCGEditorModule.h"
#include "Graph/PCGGraphExecutor.h"

#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGEngineSubsystem)

UPCGEngineSubsystem* UPCGEngineSubsystem::Get()
{
	return GEngine ? GEngine->GetEngineSubsystem<UPCGEngineSubsystem>() : nullptr;
}

#if WITH_EDITOR

void UPCGEngineSubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UPCGEngineSubsystem* This = Cast<UPCGEngineSubsystem>(InThis);
	check(This);
	This->IPCGBaseSubsystem::AddReferencedObjects(Collector);
}

void UPCGEngineSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	IPCGBaseSubsystem::InitializeBaseSubsystem();

	// Show notifications even if executor has no world 
	// @todo_pcg: for 5.8 have a better GraphExecutor initialization (this new member is private so can be easily removed)
	if (GraphExecutor)
	{
		GraphExecutor->bIsStandaloneGraphExecutor = true;
	}
}

void UPCGEngineSubsystem::Deinitialize()
{
	IPCGBaseSubsystem::DeinitializeBaseSubsystem();
	Super::Deinitialize();
}

void UPCGEngineSubsystem::NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType)
{
	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		PCGEditorModule->NotifyGraphChanged(InGraph, ChangeType);
	}

	IPCGBaseSubsystem::NotifyGraphChanged(InGraph, ChangeType);
}

void UPCGEngineSubsystem::GenerateGraph(const FPCGGenerateGraphParams& InParams)
{
	if (!InParams.Graph)
	{
		UE_LOGF(LogPCG, Error, "GenerateGraph called without valid graph");
		InParams.GenerationCallback.ExecuteIfBound(nullptr, EPCGGenerationStatus::Aborted);
		return;
	}

	if (!InParams.Graph->IsStandaloneGraph())
	{
		UE_LOGF(LogPCG, Error, "GenerateGraph called with non standalone graph");
		InParams.GenerationCallback.ExecuteIfBound(nullptr, EPCGGenerationStatus::Aborted);
		return;
	}

	IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultExecutionSource>({ InParams.Graph, InParams.Seed, /*bFireAndForget=*/true, InParams.GenerationCallback });
}

// deprecated
UPCGDefaultExecutionSource* UPCGEngineSubsystem::CreateExecutionSource(const FPCGDefaultExecutionSourceParams& InParams)
{
	return IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultExecutionSource>(InParams);
}

#endif // WITH_EDITOR

void UPCGEngineSubsystem::Tick(float DeltaTime)
{
	IPCGBaseSubsystem::Tick();
}

ETickableTickType UPCGEngineSubsystem::GetTickableTickType() const
{
#if WITH_EDITOR
	return IsTemplate() ? ETickableTickType::Never : ETickableTickType::Always;
#else
	return ETickableTickType::Never;
#endif
}

TStatId UPCGEngineSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FPCGEngineSubsystemTickFunction, STATGROUP_Tickables);
}
