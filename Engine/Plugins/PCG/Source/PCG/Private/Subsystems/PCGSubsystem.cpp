// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGSubsystem.h"

#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGManagedResource.h"
#include "PCGModule.h"
#include "PCGTrackingManager.h"
#include "PCGWorldActor.h"
#include "Graph/PCGGraphCache.h"
#include "Graph/PCGGraphCompiler.h"
#include "Graph/PCGGraphExecutor.h"
#include "Graph/PCGStackContext.h"
#include "Grid/PCGLandscapeCache.h"
#include "Grid/PCGPartitionActor.h"
#include "Helpers/PCGActorHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "RuntimeGen/PCGGenSourceManager.h"
#include "RuntimeGen/PCGRuntimeGenScheduler.h"

#include "ComponentReregisterContext.h"
#include "UnrealClient.h"
#include "Algo/Transform.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Level.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSubsystem)

#if WITH_EDITOR
#include "Editor/IPCGEditorModule.h"

#include "Editor.h"
#include "PackageSourceControlHelper.h"
#include "ObjectTools.h"
#include "Misc/ScopedSlowTask.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"
#else
#include "Engine/Engine.h"
#include "Engine/World.h"
#endif

namespace PCGSubsystemConsole
{
	static FAutoConsoleCommand CommandFlushCache(
		TEXT("pcg.FlushCache"),
		TEXT("Clears the PCG results cache and compiled graph cache."),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
				{
					PCGSubsystem->FlushCache();
				}
			}));

	TAutoConsoleVariable<int32> CVarPCGQuality(
		TEXT("pcg.Quality"), 2,
		TEXT("Selects the quality permutation of PCG which impacts Runtime Quality Branch/Select nodes.\n")
		TEXT(" 0: Low\n")
		TEXT(" 1: Medium\n")
		TEXT(" 2: High\n")
		TEXT(" 3: Epic\n")
		TEXT(" 4: Cinematic\n"),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* CVar)
		{
			if (UPCGSubsystem* Subsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
			{
				Subsystem->OnPCGQualityLevelChanged();
			}
		}),
		ECVF_Scalability);

	static FAutoConsoleCommand CommandRefreshRuntimeGen(
		TEXT("pcg.RuntimeGeneration.Refresh"),
		TEXT("Cleans up and re-generates all GenerateAtRuntime PCG components, including their partition actors."),
		FConsoleCommandDelegate::CreateLambda([]()
		{
			if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
			{
				PCGSubsystem->RefreshAllRuntimeGenExecutionSources(EPCGChangeType::GenerationGrid);
			}
		}));

#if WITH_EDITOR
	static FAutoConsoleCommand CommandBuildLandscapeCache(
		TEXT("pcg.BuildLandscapeCache"),
		TEXT("Builds the landscape cache in the current world."),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
				{
					PCGSubsystem->BuildLandscapeCache();
				}
			}));

	static FAutoConsoleCommand CommandClearLandscapeCache(
		TEXT("pcg.ClearLandscapeCache"),
		TEXT("Clear the landscape cache in the current world."),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
				{
					PCGSubsystem->ClearLandscapeCache();
				}
			}));

	static TAutoConsoleVariable<bool> CVarRebuildLandscapeOnPIE(
		TEXT("pcg.PIE.RebuildLandscapeOnPIE"),
		true,
		TEXT("Controls whether the landscape cache will be rebuilt on PIE"));

	static FAutoConsoleCommand CommandDeleteCurrentPCGWorldActor(
		TEXT("pcg.DeleteCurrentPCGWorldActor"),
		TEXT("Deletes the PCG World Actor currently registered to the PCG Subsystem."),
		FConsoleCommandDelegate::CreateLambda([]()
		{
			if(UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
			{
				PCGSubsystem->DestroyCurrentPCGWorldActor();
			}
		}));

	static FAutoConsoleCommand CommandDeleteAllPCGWorldActors(
		TEXT("pcg.DeleteAllPCGWorldActors"),
		TEXT("Deletes all PCG World Actors in current World.."),
		FConsoleCommandDelegate::CreateLambda([]()
		{
			if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
			{
				PCGSubsystem->DestroyAllPCGWorldActors();
			}
		}));
#endif
}

#if WITH_EDITOR
TSet<UWorld*> UPCGSubsystem::DisablePartitionActorCreationForWorld;

struct FPCGPartitionActorLoaderAdapter : public FLoaderAdapterShape
{
public:
	FPCGPartitionActorLoaderAdapter(UWorld* InWorld, const FBox& InBoundingBox, const FString& InLabel)
		: FLoaderAdapterShape(InWorld, InBoundingBox, InLabel)
	{ }

	virtual bool PassActorDescFilter(const FWorldPartitionHandle& Actor) const override
	{
		return FLoaderAdapterShape::PassActorDescFilter(Actor) && Actor->GetActorNativeClass() && Actor->GetActorNativeClass()->IsChildOf<APCGPartitionActor>();
	}
};
#endif

UPCGSubsystem::UPCGSubsystem()
	: UTickableWorldSubsystem()
	, IPCGBaseSubsystem()
	, TrackingManager(new FPCGTrackingManager(this)) // Can't use MakeUnique because the ctor of FPCGTrackingManager is private
{
}

void UPCGSubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UPCGSubsystem* PCGSubsystem = CastChecked<UPCGSubsystem>(InThis);
#if WITH_EDITOR
	PCGSubsystem->IPCGBaseSubsystem::AddReferencedObjects(Collector);
#endif // WITH_EDITOR
	
	if (FPCGRuntimeGenScheduler* RuntimeGenScheduler = PCGSubsystem->GetRuntimeGenScheduler())
	{
		RuntimeGenScheduler->AddReferencedObjects(Collector);
	}
}

UPCGSubsystem* UPCGSubsystem::GetSubsystemForCurrentWorld()
{
	UWorld* World = nullptr;

#if WITH_EDITOR
	if (GEditor)
	{
		if (GEditor->PlayWorld)
		{
			World = GEditor->PlayWorld;
		}
		else
		{
			World = GEditor->GetEditorWorldContext().World();
		}
	}
	else
#endif
	if (GEngine)
	{
		World = GEngine->GetCurrentPlayWorld();
	}

	return UPCGSubsystem::GetInstance(World);
}

void UPCGSubsystem::Deinitialize()
{
	// Cancel all tasks
	// TODO
	IPCGBaseSubsystem::DeinitializeBaseSubsystem();

	delete RuntimeGenScheduler;
	RuntimeGenScheduler = nullptr;

	PCGWorldActor = nullptr;
	bHasTickedOnce = false;

	TrackingManager->Deinitialize();

	Scalability::OnScalabilitySettingsChanged.RemoveAll(this);
	FGlobalComponentReregisterContext::OnGlobalComponentReregisterContextDestroyed().RemoveAll(this);

	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
	{
		GEngine->GameViewport->Viewport->ViewportResizedEvent.RemoveAll(this);
	}

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	if (GEditor)
	{
		GEditor->OnPreviewShaderPlatformChanged().RemoveAll(this);
	}

	if (UWorld* World = GetWorld(); World && !World->IsGameWorld())
	{
		// Unregister persistent level events
		OnLevelRemovedFromWorld(World->PersistentLevel, World);

		FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
		FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	}
#endif

	Super::Deinitialize();
}

void UPCGSubsystem::PostInitialize()
{
	Super::PostInitialize();

	LLM_SCOPE_BYTAG(PCG);

	// Gather world pcg actor if it exists
	if (!PCGWorldActor)
	{
		if (UWorld* World = GetWorld())
		{
			UPCGActorHelpers::ForEachActorInLevel<APCGWorldActor>(World->PersistentLevel, [this](AActor* InActor)
			{
				PCGWorldActor = Cast<APCGWorldActor>(InActor);
				return PCGWorldActor == nullptr;
			});
		}
	}

	TrackingManager->Initialize();

	IPCGBaseSubsystem::InitializeBaseSubsystem();

	// Initialize runtime generation scheduler
	check(!RuntimeGenScheduler);
	RuntimeGenScheduler = new FPCGRuntimeGenScheduler(GetWorld(), TrackingManager.Get());

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UPCGSubsystem::OnObjectsReplaced);

	if (GEditor)
	{
		GEditor->OnPreviewShaderPlatformChanged().AddUObject(this, &UPCGSubsystem::OnPreviewPlatformChanged);
	}

	if (UWorld* World = GetWorld(); World && !World->IsGameWorld())
	{
		// Register persistent level events
		OnLevelAddedToWorld(World->PersistentLevel, World);

		// Listen for sub-levels being added/removed to register per-level events (ex: Level Instances)
		FWorldDelegates::LevelAddedToWorld.AddUObject(this, &UPCGSubsystem::OnLevelAddedToWorld);
		FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &UPCGSubsystem::OnLevelRemovedFromWorld);
	}
#endif

	FGlobalComponentReregisterContext::OnGlobalComponentReregisterContextDestroyed().AddUObject(this, &UPCGSubsystem::OnGlobalComponentReregisterContextDestroyed);

	Scalability::OnScalabilitySettingsChanged.AddUObject(this, &UPCGSubsystem::OnScalabilitySettingsChanged);

	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
	{
		GEngine->GameViewport->Viewport->ViewportResizedEvent.AddUObject(this, &UPCGSubsystem::OnViewportResized);
	}
}

UPCGSubsystem* UPCGSubsystem::GetInstance(UWorld* World)
{
	if (World)
	{
		UPCGSubsystem* Subsystem = World->GetSubsystem<UPCGSubsystem>();
		return (Subsystem && Subsystem->IsInitialized()) ? Subsystem : nullptr;
	}
	else
	{
		return nullptr;
	}
}

void UPCGSubsystem::RegisterBeginTickAction(FTickAction&& Action)
{
	BeginTickActions.Emplace(Action);
}

#if WITH_EDITOR
UPCGSubsystem* UPCGSubsystem::GetActiveEditorInstance()
{
	if (GEditor)
	{
		return GEditor->PlayWorld ? UPCGSubsystem::GetInstance(GEditor->PlayWorld.Get()) : UPCGSubsystem::GetInstance(GEditor->GetEditorWorldContext().World());
	}

	return nullptr;
}

void UPCGSubsystem::SetConstructionScriptSourceComponent(UPCGComponent* InComponent)
{
	if (InComponent)
	{
		if (AActor* Owner = InComponent->GetOwner())
		{
			PerActorConstructionScriptSourceComponents.FindOrAdd(Owner).Add(InComponent->GetFName(), InComponent);
		}
	}
}

bool UPCGSubsystem::RemoveAndCopyConstructionScriptSourceComponent(AActor* InComponentOwner, FName InComponentName, UPCGComponent*& OutSourceComponent)
{
	OutSourceComponent = nullptr;
	if (FConstructionScriptSourceComponents* Found = PerActorConstructionScriptSourceComponents.Find(InComponentOwner))
	{
		TObjectKey<UPCGComponent> FoundComponent;
		if (Found->RemoveAndCopyValue(InComponentName, FoundComponent))
		{
			OutSourceComponent = FoundComponent.ResolveObjectPtrEvenIfGarbage();
			if (Found->IsEmpty())
			{
				PerActorConstructionScriptSourceComponents.Remove(InComponentOwner);
			}
		}
	}

	return OutSourceComponent != nullptr;
}
#endif

void UPCGSubsystem::Tick(float DeltaSeconds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSubsystem::Tick);
	LLM_SCOPE_BYTAG(PCG);

	// If we're in an AutoRTFM Transaction, exit the TickableObject Tick (which is top-level) and run again outside the transaction
	if (AutoRTFM::IsTransactional())
	{
		AutoRTFM::OnCommit([WeakThis = TWeakObjectPtr(this), DeltaSeconds]()
		{
			if (WeakThis.IsValid())
			{
				WeakThis->Tick(DeltaSeconds);
			}
		});

		return;
	}

	// Refresh dirty runtime gen sources before ticking the graph executor (in IPCGBaseSubsystem::Tick()) so that we're not processing a bunch of graphs/execution sources that will be cancelled immediately.
	if (bDirtyRuntimeGenExecutionSources)
	{
		RefreshAllRuntimeGenExecutionSources(DirtyRuntimeGenChangeType, DirtyRuntimeGenReason);

		bDirtyRuntimeGenExecutionSources = false;
		DirtyRuntimeGenChangeType = EPCGChangeType::None;
		DirtyRuntimeGenReason = ERuntimeGenRefreshReason::None;
	}

	Super::Tick(DeltaSeconds);

	ExecuteBeginTickActions();

#if WITH_EDITOR
	PerActorConstructionScriptSourceComponents.Empty();
#endif

	if (!bHasTickedOnce)
	{
#if WITH_EDITOR
		if (PCGSubsystemConsole::CVarRebuildLandscapeOnPIE.GetValueOnAnyThread() && PCGHelpers::IsRuntimeOrPIE())
		{
			BuildLandscapeCache(/*bQuiet=*/true, /*bForceBuild=*/false);
		}
#endif

		bHasTickedOnce = true;
	}

	// Lose references to landscape cache as needed
	// This will also initialize the cache if it isn't already so needs to happen before the GraphExecutor->Executor call
	if (PCGWorldActor && GetLandscapeCache())
	{
		GetLandscapeCache()->Tick(DeltaSeconds);
	}

	// Base subsystem can modify end time
	const double EndTime = IPCGBaseSubsystem::Tick();
	
	TrackingManager->Tick();

	if (PCGWorldActor)
	{
		RuntimeGenScheduler->Tick(PCGWorldActor, DeltaSeconds, EndTime);
	}
}

APCGWorldActor* UPCGSubsystem::GetPCGWorldActor()
{
#if WITH_EDITOR
	if (!PCGWorldActor && !PCGHelpers::IsRuntimeOrPIE())
	{
		PCG::TScopeLock Lock(PCGWorldActorLock);

		if (!PCGWorldActor)
		{
			PCGWorldActor = APCGWorldActor::CreatePCGWorldActor(GetWorld());
		}
	}
#endif

	return PCGWorldActor;
}

APCGWorldActor* UPCGSubsystem::GetPCGWorldActorForPartitionActor(APCGPartitionActor* InActor)
{
	APCGWorldActor* FoundWorldActor = GetPCGWorldActor();
	if (FoundWorldActor)
	{
		return FoundWorldActor;
	}
	
	// We're at runtime and we didn't find a World Actor. This can happen if the partition actor is not in the same level than the persistent
	// level of the world. In that case, look for the PCG World Actor in the same level as the Partition Actor. If we find one, register it.
	if (ensure(PCGHelpers::IsRuntimeOrPIE()) && ensure(InActor))
	{
		UPCGActorHelpers::ForEachActorInLevel<APCGWorldActor>(InActor->GetLevel(), [this](AActor* InActor)
		{
			PCG::TScopeLock Lock(PCGWorldActorLock);
			if (!PCGWorldActor)
			{
				PCGWorldActor = Cast<APCGWorldActor>(InActor);
			}

			return PCGWorldActor == nullptr;
		});
	}

	return PCGWorldActor;
}

APCGWorldActor* UPCGSubsystem::FindPCGWorldActor()
{
	return PCGWorldActor;
}

int32 UPCGSubsystem::GetPCGQualityLevel()
{
	return PCGSubsystemConsole::CVarPCGQuality.GetValueOnAnyThread();
}

void UPCGSubsystem::OnPCGQualityLevelChanged()
{
#if WITH_EDITOR
	// Notify every runtime-gen graph of the quality level change so its node-level state can update, regardless of whether the graph opts in to auto-refresh on system events.
	TrackingManager->ForAllOriginalExecutionSources([](IPCGGraphExecutionSource* InExecutionSource)
	{
		if (InExecutionSource && InExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			if (UPCGGraph* Graph = InExecutionSource->GetExecutionState().GetGraph())
			{
				Graph->OnPCGQualityLevelChanged();
			}
		}
	});
#endif

	RefreshAllRuntimeGenExecutionSources(EPCGChangeType::GenerationGrid, ERuntimeGenRefreshReason::QualitySettings);
}

void UPCGSubsystem::OnScalabilitySettingsChanged(const Scalability::FQualityLevels& QualityLevels)
{
	RefreshAllRuntimeGenExecutionSources(EPCGChangeType::GenerationGrid, ERuntimeGenRefreshReason::QualitySettings);
}

#if WITH_EDITOR
void UPCGSubsystem::DestroyCurrentPCGWorldActor()
{
	if (PCGWorldActor)
	{
		PCG::TScopeLock Lock(PCGWorldActorLock);
		PCGWorldActor->Destroy();
		PCGWorldActor = nullptr;
	}
}

void UPCGSubsystem::DestroyAllPCGWorldActors()
{
	// Delete all PAs first to avoid leaving orphans behind.
	DeleteSerializedPartitionActors(/*bOnlyDeleteUnused=*/false);

	// Get rid of current PCG world actor first
	DestroyCurrentPCGWorldActor();
	
	// Pick up any strays in the current world
	TArray<APCGWorldActor*> ActorsToDestroy;
	ForEachObjectWithOuter(GetWorld(), [&ActorsToDestroy](UObject* Object)
	{
		if (APCGWorldActor* WorldActor = Cast<APCGWorldActor>(Object))
		{
			if (IsValid(WorldActor))
			{
				ActorsToDestroy.Add(WorldActor);
			}
		}
	});

	for (APCGWorldActor* ActorToDestroy : ActorsToDestroy)
	{
		ActorToDestroy->Destroy();
	}
}

AActor* UPCGSubsystem::GetOrCreateDebugActor()
{
	if (!DebugActor.IsValid())
	{
		FActorSpawnParameters SpawnParams;
		SpawnParams.bCreateActorPackage = false;
		SpawnParams.bHideFromSceneOutliner = true;
		SpawnParams.Name = TEXT("PCG_Debug");
		SpawnParams.ObjectFlags = RF_Transient;
		UPCGActorHelpers::FSpawnDefaultActorParams Params(GetWorld(), AActor::StaticClass(), FTransform::Identity, SpawnParams);
		DebugActor = UPCGActorHelpers::SpawnDefaultActor(Params);
	}

	return DebugActor.Get();
}

void UPCGSubsystem::LogAbnormalComponentStates(bool bGroupByState) const
{
	TArray<UPCGComponent*> DeactivatedComponents;
	TArray<UPCGComponent*> NotGeneratedComponents;
	TArray<UPCGComponent*> DirtyGeneratedComponents;

	UE_LOGF(LogPCG, Log, "--- Logging Abnormal PCG Component States ---");

	UPCGActorHelpers::ForEachActorInWorld(GetWorld(), AActor::StaticClass(), [bGroupByState, &DeactivatedComponents, &NotGeneratedComponents, &DirtyGeneratedComponents](AActor* InActor)
	{
		if (!InActor || !IsValid(InActor))
		{
			return true;
		}

		TInlineComponentArray<UPCGComponent*, 1> PCGComponents;
		InActor->GetComponents(PCGComponents);

		for (UPCGComponent* PCGComponent : PCGComponents)
		{
			if (!PCGComponent->bActivated)
			{
				if (bGroupByState)
				{
					DeactivatedComponents.Add(PCGComponent);
				}
				else
				{
					UE_LOGF(LogPCG, Log, "%ls - %ls - Deactivated Component", *InActor->GetName(), *PCGComponent->GetName());
				}
			}
			else if (!PCGComponent->bGenerated && PCGComponent->GenerationTrigger != EPCGComponentGenerationTrigger::GenerateAtRuntime)
			{
				if (bGroupByState)
				{
					NotGeneratedComponents.Add(PCGComponent);
				}
				else
				{
					UE_LOGF(LogPCG, Log, "%ls - %ls - Not Generated Component", *InActor->GetName(), *PCGComponent->GetName());
				}
			}
			else if (PCGComponent->bDirtyGenerated)
			{
				if (bGroupByState)
				{
					DirtyGeneratedComponents.Add(PCGComponent);
				}
				else
				{
					UE_LOGF(LogPCG, Log, "%ls - %ls - Dirty Generated Component ", *InActor->GetName(), *PCGComponent->GetName());
				}
			}
		}

		return true;
	});

	if (bGroupByState)
	{
		UE_LOGF(LogPCG, Log, "--- Deactivated PCG Components ---");
		for (UPCGComponent* Component : DeactivatedComponents)
		{
			check(Component && Component->GetOwner());
			UE_LOGF(LogPCG, Log, "%ls - %ls", *Component->GetOwner()->GetName(), *Component->GetName());
		}

		UE_LOGF(LogPCG, Log, "--- Not Generated Components ---");
		for (UPCGComponent* Component : NotGeneratedComponents)
		{
			check(Component && Component->GetOwner());
			UE_LOGF(LogPCG, Log, "%ls - %ls", *Component->GetOwner()->GetName(), *Component->GetName());
		}

		UE_LOGF(LogPCG, Log, "--- Dirty Components ---");
		for (UPCGComponent* Component : DirtyGeneratedComponents)
		{
			check(Component && Component->GetOwner());
			UE_LOGF(LogPCG, Log, "%ls - %ls", *Component->GetOwner()->GetName(), *Component->GetName());
		}
	}
}

#endif // WITH_EDITOR

void UPCGSubsystem::RegisterPCGWorldActor(APCGWorldActor* InActor)
{
	// TODO: we should support merging or multi world actor support when relevant
	if (!PCGWorldActor)
	{
		PCGWorldActor = InActor;
	}
	else if (InActor != PCGWorldActor)
	{
		PCGWorldActor->MergeFrom(InActor);
	}
}

void UPCGSubsystem::UnregisterPCGWorldActor(APCGWorldActor* InActor)
{
	if (PCGWorldActor == InActor)
	{
		PCGWorldActor = nullptr;
	}
}

void UPCGSubsystem::OnOriginalExecutionSourceRegistered(IPCGGraphExecutionSource* InExecutionSource)
{
	// @todo_pcg: support execution source
	if (RuntimeGenScheduler)
	{
		if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource))
		{
			RuntimeGenScheduler->OnOriginalComponentRegistered(PCGComponent);
		}
	}
}

void UPCGSubsystem::OnOriginalExecutionSourceUnregistered(IPCGGraphExecutionSource* InExecutionSource)
{
	// @todo_pcg: support execution source
	if (RuntimeGenScheduler)
	{
		if (UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource))
		{
			RuntimeGenScheduler->OnOriginalComponentUnregistered(PCGComponent);
		}
	}

#if WITH_EDITOR
	if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
	{
		PCGEditorModule->ClearExecutionMetadata(InExecutionSource);
	}
#endif
}

void UPCGSubsystem::OnOriginalExecutionSourceReplaced(const IPCGGraphExecutionSource* InOldExecutionSource, IPCGGraphExecutionSource* InNewExecutionSource)
{
	// @todo_pcg: support execution source
	if (RuntimeGenScheduler)
	{
		const UPCGComponent* OldPCGComponent = Cast<UPCGComponent>(InOldExecutionSource);
		UPCGComponent* NewPCGComponent = Cast<UPCGComponent>(InNewExecutionSource);

		if (OldPCGComponent && NewPCGComponent)
		{
			RuntimeGenScheduler->OnOriginalComponentReplaced(OldPCGComponent, NewPCGComponent);
		}
	}
}

UPCGLandscapeCache* UPCGSubsystem::GetLandscapeCache()
{
	APCGWorldActor* LandscapeCacheOwner = GetPCGWorldActor();
	return LandscapeCacheOwner ? LandscapeCacheOwner->LandscapeCacheObject.Get() : nullptr;
}

FPCGTaskId UPCGSubsystem::ScheduleComponent(UPCGComponent* PCGComponent, uint32 Grid, bool bForce, const TArray<FPCGTaskId>& InDependencies)
{
	check(GraphExecutor);

	if (!PCGComponent)
	{
		return InvalidPCGTaskId;
	}

	bool bHasUnbounded = false;
	PCGHiGenGrid::FSizeArray GridSizes;
	ensure(PCGHelpers::GetGenerationGridSizes(PCGComponent->GetGraph(), GetPCGWorldActor(), GridSizes, bHasUnbounded));

	// Create the PartitionActors if necessary. Skip if this is a runtime managed component, PAs are handled manually by the RuntimeGenScheduler.
	// Editor only because we expect at runtime for PAs to already exist so they can properly be streamed in and out (creating them at runtime would leave them unmanaged and always loaded)
	if (PCGComponent->IsPartitioned() && !PCGComponent->IsManagedByRuntimeGenSystem())
	{
#if WITH_EDITOR
		if (!GridSizes.IsEmpty())
		{
			CreatePartitionActorsWithinBounds(PCGComponent, PCGComponent->GetGridBounds(), GridSizes);
		}
#endif // WITH_EDITOR

		TrackingManager->UpdateMappingPCGComponentPartitionActor(PCGComponent);
	}

	// Execution dependencies require a task to finish executing before the dependent task.
	TArray<FPCGTaskId> ExecutionDependencyTasks;

	// Data dependencies act as execution dependencies, but will also have their output consumed by the waiting task. For a component, this means
	// it will store the output data into its managed resources, which, for an original component, should not include the local component generation tasks,
	// since those resources should be managed locally.
	TArray<FPCGTaskId> DataDependencyTasks;

	// Schedule generation of original component if is is non-partitioned, or if it has nodes that will execute at the Unbounded level.
	FPCGTaskId OriginalComponentTask = InvalidPCGTaskId;

	bool bGeneratePCGComponent = false;
	if (!PCGComponent->IsPartitioned())
	{
		// This component is either an unpartitioned original component or a local component. Generate if grid size matches preference (if provided).
		bGeneratePCGComponent = (Grid == PCGHiGenGrid::UninitializedGridSize()) || (Grid == PCGComponent->GetGenerationGridSize());
	}
	else
	{
		// This component is a partitioned original component. Generate if the graph has unbounded nodes and if this grid matches preference (if provided).
		bGeneratePCGComponent = bHasUnbounded && (Grid == PCGHiGenGrid::UnboundedGridSize() || Grid == PCGHiGenGrid::UninitializedGridSize());
	}

	if (bGeneratePCGComponent)
	{
		OriginalComponentTask = PCGComponent->CreateGenerateTask(bForce, InDependencies);
		if (OriginalComponentTask != InvalidPCGTaskId)
		{
			DataDependencyTasks.Add(OriginalComponentTask);
		}
	}

	// If the component is partitioned, we will forward the calls to its registered PCG Partition actors
	if (PCGComponent->IsPartitioned() && PCGHiGenGrid::IsValidGridOrUninitialized(Grid))
	{
		// Local components depend on the original component (to ensure any data is available).
		TArray<FPCGTaskId> Dependencies = InDependencies;
		if (OriginalComponentTask != InvalidPCGTaskId)
		{
			Dependencies.Add(OriginalComponentTask);
		}

		auto LocalGenerateTask = [OriginalComponent = PCGComponent, Grid, &Dependencies, bForce, &GridSizes](UPCGComponent* LocalComponent, const TArray<FPCGTaskId>& ExtraDependencies)
		{
			if (!GridSizes.Contains(LocalComponent->GetGenerationGridSize()))
			{
				// Local component with invalid grid size. Grid sizes may have changed in graph.
				return LocalComponent->CleanupLocal(/*bRemoveComponents=*/true, Dependencies);
			}
			else if (Grid != PCGHiGenGrid::UninitializedGridSize() && Grid != LocalComponent->GetGenerationGridSize())
			{
				// Grid size does not match the given target grid, so skip.
				return InvalidPCGTaskId;
			}

			// If the local component is currently generating, it's probably because it was requested by a refresh.
			// Wait after this one instead
			if (LocalComponent->IsGenerating())
			{
				return LocalComponent->CurrentGenerationTask;
			}

			// Ensure that the PCG actor match our original
			LocalComponent->SetPropertiesFromOriginal(OriginalComponent);

			FPCGTaskId LocalCleanupTaskId = InvalidPCGTaskId;
			if (LocalComponent->bGenerated && !OriginalComponent->bGenerated)
			{
				// Detected a mismatch between the original component and the local component.
				// Request a cleanup first
				LocalCleanupTaskId = LocalComponent->CleanupLocal(/*bRemoveComponents=*/true, Dependencies);
			}

			TArray<FPCGTaskId> AdditionalDependencies;
			const TArray<FPCGTaskId>* AllDependencies = &Dependencies;
			if (LocalCleanupTaskId != InvalidPCGTaskId || !ExtraDependencies.IsEmpty())
			{
				AdditionalDependencies.Reserve(Dependencies.Num() + ExtraDependencies.Num() + 1);
				AdditionalDependencies.Append(Dependencies);

				if (LocalCleanupTaskId != InvalidPCGTaskId)
				{
					AdditionalDependencies.Add(LocalCleanupTaskId);
				}

				if (!ExtraDependencies.IsEmpty())
				{
					AdditionalDependencies.Append(ExtraDependencies);
				}

				AllDependencies = &AdditionalDependencies;
			}

			return LocalComponent->GenerateInternal(bForce, LocalComponent->GetGenerationGridSize(), EPCGComponentGenerationTrigger::GenerateOnDemand, *AllDependencies);
		};

		ExecutionDependencyTasks.Append(TrackingManager->DispatchToRegisteredLocalComponents(PCGComponent, LocalGenerateTask));
	}

	if (!ExecutionDependencyTasks.IsEmpty() || !DataDependencyTasks.IsEmpty())
	{
		TWeakObjectPtr<UPCGComponent> ComponentPtr(PCGComponent);

		return GraphExecutor->ScheduleGenericWithContext([ComponentPtr](FPCGContext* Context)
		{
			if (UPCGComponent* Component = ComponentPtr.Get())
			{
				// If the component is not valid anymore, just early out.
				if (!IsValid(Component))
				{
					return true;
				}

				const FBox NewBounds = Component->GetGridBounds();
				Component->PostProcessGraph(NewBounds, /*bGenerate=*/true, Context);
			}

			return true;
		}, PCGComponent, ExecutionDependencyTasks, DataDependencyTasks, /*bSupportBasePointDataInput=*/true);
	}
	else
	{
		UE_LOGF(LogPCG, Error, "[ScheduleComponent] Didn't schedule any task.");
		if (PCGComponent)
		{
			PCGComponent->OnProcessGraphAborted();
		}
		return InvalidPCGTaskId;
	}
}

// Deprecated 5.8
FPCGTaskId UPCGSubsystem::ScheduleComponent(UPCGComponent* PCGComponent, EPCGHiGenGrid Grid, bool bForce, const TArray<FPCGTaskId>& InDependencies)
{
	return ScheduleComponent(PCGComponent, PCGHiGenGrid::GridToGridSize(Grid), bForce, InDependencies);
}

FPCGTaskId UPCGSubsystem::ScheduleCleanup(UPCGComponent* PCGComponent, bool bRemoveComponents, const TArray<FPCGTaskId>& Dependencies)
{
	if (!PCGComponent)
	{
		return InvalidPCGTaskId;
	}

	bool bHasUnbounded = false;
	PCGHiGenGrid::FSizeArray GridSizes;
	ensure(PCGHelpers::GetGenerationGridSizes(PCGComponent->GetGraph(), GetPCGWorldActor(), GridSizes, bHasUnbounded));

	TArray<FPCGTaskId> AllTasks;

	// Schedule cleanup of original component if is is non-partitioned, or if it has nodes that will execute at the Unbounded level.
	if (!PCGComponent->IsPartitioned() || bHasUnbounded)
	{
		FPCGTaskId TaskId = PCGComponent->CreateCleanupTask(bRemoveComponents, Dependencies);
		if (TaskId != InvalidPCGTaskId)
		{
			AllTasks.Add(TaskId);
		}
	}

	// If the component is partitioned, we will forward the calls to its registered PCG Partition actors
	if (PCGComponent->IsPartitioned())
	{
		auto LocalCleanupTask = [bRemoveComponents, &Dependencies](UPCGComponent* LocalComponent, const TArray<FPCGTaskId>& ExtraDependencies)
		{
			// If the local component is currently cleaning up, it's probably because it was requested by a refresh.
			// Wait after this one instead
			if (LocalComponent->IsCleaningUp())
			{
				return LocalComponent->CurrentCleanupTask;
			}

			TArray<FPCGTaskId> AdditionalDependencies;
			const TArray<FPCGTaskId>* AllDependencies = &Dependencies;
			if (!ExtraDependencies.IsEmpty())
			{
				AdditionalDependencies.Reserve(Dependencies.Num() + ExtraDependencies.Num());
				AdditionalDependencies.Append(Dependencies);
				AdditionalDependencies.Append(ExtraDependencies);
				AllDependencies = &AdditionalDependencies;
			}

			// Always executes regardless of local component grid size - clean up as much as possible.
			return LocalComponent->CleanupLocal(bRemoveComponents, *AllDependencies);
		};

		AllTasks.Append(TrackingManager->DispatchToRegisteredLocalComponents(PCGComponent, LocalCleanupTask));
	}

	TWeakObjectPtr<UPCGComponent> ComponentPtr(PCGComponent);
	auto PostCleanupTask = [this, bRemoveComponents, ComponentPtr]()
	{
		if (UPCGComponent* Component = ComponentPtr.Get())
		{
			// If the component is not valid anymore, just early out
			if (!IsValid(Component))
			{
				return true;
			}

			Component->PostCleanupGraph(bRemoveComponents);

			// Remove the local component mappings if requested and the component is partitioned. If 'bRemoveComponents' is false, that indicates we are doing a refresh, so destroying
			// the component mappings is counterproductive.
			if (bRemoveComponents && Component->IsPartitioned())
			{
				TrackingManager->DeleteMappingPCGComponentPartitionActor(Component);
			}
		}

		return true;
	};

	FPCGTaskId PostCleanupTaskId = InvalidPCGTaskId;

	// If we have no tasks to do, just call PostCleanup immediately
	// otherwise wait for all the tasks to be done to call PostCleanup.
	if (AllTasks.IsEmpty())
	{
		PostCleanupTask();
	}
	else
	{
		PostCleanupTaskId = GraphExecutor->ScheduleGeneric(PostCleanupTask, PCGComponent, AllTasks);
	}

	return PostCleanupTaskId;
}

FPCGTaskId UPCGSubsystem::ScheduleGraph(IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& Dependencies)
{
	if (ExecutionSource)
	{
		return GraphExecutor->Schedule(ExecutionSource, Dependencies);
	}
	else
	{
		return InvalidPCGTaskId;
	}
}

FPCGTaskId UPCGSubsystem::ScheduleGraph(
	UPCGGraph* Graph,
	IPCGGraphExecutionSource* ExecutionSource,
	FPCGElementPtr PreGraphElement, 
	FPCGElementPtr InputElement, 
	const TArray<FPCGTaskId>& Dependencies, 
	const FPCGStack* InFromStack, 
	bool bAllowHierarchicalGeneration)
{
	return ScheduleGraph(FPCGScheduleGraphParams(Graph, ExecutionSource, PreGraphElement, InputElement, Dependencies, InFromStack, bAllowHierarchicalGeneration));
}

ETickableTickType UPCGSubsystem::GetTickableTickType() const
{
	return IsTemplate() ? ETickableTickType::Never : ETickableTickType::Always;
}

TStatId UPCGSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UPCGSubsystem, STATGROUP_Tickables);
}

FPCGTaskId UPCGSubsystem::ScheduleGeneric(TFunction<bool()> InOperation, IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies) const
{
	check(GraphExecutor);
	return GraphExecutor->ScheduleGeneric(InOperation, ExecutionSource, TaskExecutionDependencies);
}

FPCGTaskId UPCGSubsystem::ScheduleGeneric(TFunction<bool()> InOperation, TFunction<void()> InAbortOperation, IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies) const
{
	check(GraphExecutor);
	return GraphExecutor->ScheduleGeneric(InOperation, InAbortOperation, ExecutionSource, TaskExecutionDependencies);
}

FPCGTaskId UPCGSubsystem::ScheduleGenericWithContext(TFunction<bool(FPCGContext*)> InOperation, IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies, const TArray<FPCGTaskId>& TaskDataDependencies) const
{
	check(GraphExecutor);
	return GraphExecutor->ScheduleGenericWithContext(InOperation, ExecutionSource, TaskExecutionDependencies, TaskDataDependencies);
}

FPCGTaskId UPCGSubsystem::ScheduleGenericWithContext(TFunction<bool(FPCGContext*)> InOperation, TFunction<void(FPCGContext*)> InAbortOperation, IPCGGraphExecutionSource* ExecutionSource, const TArray<FPCGTaskId>& TaskExecutionDependencies, const TArray<FPCGTaskId>& TaskDataDependencies) const
{
	check(GraphExecutor);
	return GraphExecutor->ScheduleGenericWithContext(InOperation, InAbortOperation, ExecutionSource, TaskExecutionDependencies, TaskDataDependencies);
}


void UPCGSubsystem::CancelGenerationInternal(IPCGGraphExecutionSource* Source, bool bCleanupUnusedResources)
{
	check(GraphExecutor && IsInGameThread());
	UPCGComponent* Component = Cast<UPCGComponent>(Source);

	if (!Component || !Component->IsGenerating())
	{
		return;
	}

	if (Component->IsPartitioned())
	{
		auto LocalCancel = [this, bCleanupUnusedResources](UPCGComponent* LocalComponent, const TArray<FPCGTaskId>& ExtraDependencies)
		{
			ensure(ExtraDependencies.IsEmpty());
			if (LocalComponent->IsGenerating())
			{
				CancelGeneration(LocalComponent, bCleanupUnusedResources);
			}

			return InvalidPCGTaskId;
		};

		TrackingManager->DispatchToRegisteredLocalComponents(Component, LocalCancel);
	}
}

// DEPRECATED 5.8
void UPCGSubsystem::RefreshRuntimeGenComponent(UPCGComponent* RuntimeComponent, EPCGChangeType ChangeType)
{
	RefreshRuntimeGenExecutionSource(RuntimeComponent, ChangeType);
}

void UPCGSubsystem::RefreshRuntimeGenExecutionSource(IPCGGraphExecutionSource* ExecutionSource, EPCGChangeType ChangeType)
{
	if (ExecutionSource && ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem() && RuntimeGenScheduler)
	{
		// Only need to remove PAs if the grid sizes have changed.
		const bool bRemovePartitionActors = !!(ChangeType & EPCGChangeType::GenerationGrid);
		RuntimeGenScheduler->RefreshExecutionSource(ExecutionSource, bRemovePartitionActors);
	}
}

// DEPRECATED 5.8
void UPCGSubsystem::RefreshAllRuntimeGenComponents(EPCGChangeType ChangeType)
{
	RefreshAllRuntimeGenExecutionSources(ChangeType);
}

void UPCGSubsystem::RefreshAllRuntimeGenExecutionSources(EPCGChangeType ChangeType, ERuntimeGenRefreshReason Reason)
{
	const bool bForceAll = EnumHasAnyFlags(Reason, ERuntimeGenRefreshReason::UserRequested);

	for (IPCGGraphExecutionSource* ExecutionSource : GetAllRegisteredExecutionSources())
	{
		if (!ExecutionSource || !ExecutionSource->GetExecutionState().IsManagedByRuntimeGenSystem())
		{
			continue;
		}

		bool bShouldRefresh = bForceAll;
		if (!bShouldRefresh)
		{
			const UPCGGraph* Graph = ExecutionSource->GetExecutionState().GetGraph();
			if (Graph)
			{
				bShouldRefresh =
					(EnumHasAnyFlags(Reason, ERuntimeGenRefreshReason::RenderStateRefresh) && Graph->bAutoRefreshOnRenderStateRefresh)
					|| (EnumHasAnyFlags(Reason, ERuntimeGenRefreshReason::QualitySettings) && Graph->bAutoRefreshOnQualitySettingsChanged);
			}
		}

		if (bShouldRefresh)
		{
			RefreshRuntimeGenExecutionSource(ExecutionSource, ChangeType);
		}
	}
}

void UPCGSubsystem::DirtyRuntimeGenExecutionSources(EPCGChangeType ChangeType, ERuntimeGenRefreshReason Reason)
{
	bDirtyRuntimeGenExecutionSources = true;
	DirtyRuntimeGenChangeType |= ChangeType;
	DirtyRuntimeGenReason |= Reason;
}

#if WITH_EDITOR
void UPCGSubsystem::RefreshAllComponentsFiltered(const TFunction<bool(UPCGComponent*)>& ComponentFilter, EPCGChangeType ChangeType)
{
	// @todo_pcg: support execution source
	for (IPCGGraphExecutionSource* ExecutionSource : GetAllRegisteredExecutionSources())
	{
		if (UPCGComponent* Component = Cast<UPCGComponent>(ExecutionSource))
		{
			if (ComponentFilter(Component))
			{
				Component->Refresh(ChangeType);
			}
		}
	}
}
#endif

void UPCGSubsystem::ForAllRegisteredLocalComponents(UPCGComponent* InOriginalComponent, const TFunctionRef<void(UPCGComponent*)>& InFunc) const
{
	auto WrapperFunc = [&InFunc](UPCGComponent* Component, const TArray<FPCGTaskId>& ExtraDependencies) -> FPCGTaskId
	{
		ensure(ExtraDependencies.IsEmpty());
		InFunc(Component);
		return InvalidPCGTaskId;
	};

	TrackingManager->DispatchToRegisteredLocalComponents(InOriginalComponent, WrapperFunc);
}

void UPCGSubsystem::ForAllRegisteredIntersectingLocalComponents(UPCGComponent* InOriginalComponent, const FBoxCenterAndExtent& InBounds, const TFunctionRef<void(UPCGComponent*)>& InFunc) const
{
	check(InOriginalComponent);
	const FBox Overlap = InOriginalComponent->GetGridBounds().Overlap(InBounds.GetBox());

	// We reject overlaps with zero volume instead of simply checking Intersect(...) to avoid bounds which touch but do not overlap.
	if (Overlap.GetVolume() <= 0)
	{
		return;
	}

	TrackingManager->ForAllIntersectingPartitionActors(Overlap, [InOriginalComponent, &InFunc](APCGPartitionActor* Actor)
	{
		if (UPCGComponent* LocalComponent = Actor->GetLocalComponent(InOriginalComponent))
		{
			InFunc(LocalComponent);
		}
	});
}

void UPCGSubsystem::ForAllOverlappingComponentsInHierarchy(UPCGComponent* InComponent, const TFunctionRef<void(UPCGComponent*)>& InFunc) const
{
	UPCGComponent* OriginalComponent = InComponent->GetOriginalComponent();

	ForAllRegisteredLocalComponents(OriginalComponent, [InComponent, &InFunc](UPCGComponent* InLocalComponent)
	{
		const FBox OtherBounds = InLocalComponent->GetGridBounds();
		const FBox ThisBounds = InComponent->GetGridBounds();
		const FBox Overlap = OtherBounds.Overlap(ThisBounds);
		if (Overlap.GetVolume() > 0)
		{
			InFunc(InLocalComponent);
		}
	});
}

FPCGTaskId UPCGSubsystem::ForAllOverlappingCells(UPCGComponent* InComponent, const FBox& InBounds, const PCGHiGenGrid::FSizeArray& InGridSizes, bool bCanCreateActor, const TArray<FPCGTaskId>& Dependencies, TFunctionRef<FPCGTaskId(APCGPartitionActor*, const FBox&)> InFunc, TFunctionRef<FPCGTaskId(const FPCGGridDescriptor&, const FIntVector&, const FBox&)> InUnloadedFunc) const
{
	if (!GraphExecutor || !PCGWorldActor)
	{
		UE_LOGF(LogPCG, Error, "[ForAllOverlappingCells] GraphExecutor or PCGWorldActor is null.");
		return InvalidPCGTaskId;
	}

	PCGHiGenGrid::FSizeArray GridSizes = InGridSizes;

	// We have no use for unbounded grids as this is a grid-centric function. Also discard invalid grid sizes.
	GridSizes.Remove(PCGHiGenGrid::UnboundedGridSize());
	GridSizes.RemoveAll([](uint32 GridSize) { return !ensure(PCGHiGenGrid::IsValidGridSize(GridSize)); });

	if (GridSizes.IsEmpty())
	{
		return InvalidPCGTaskId;
	}

	TArray<FPCGTaskId> CellTasks;
	for (uint32 GridSize : GridSizes)
	{
		FPCGGridDescriptor Descriptor = InComponent->GetGridDescriptor(GridSize);
		check(!Descriptor.IsRuntime());

		// In case of 2D grid, we are clamping our bounds in Z to be within 0 and GridSize to create a 2D grid instead of 3D.
		FBox ModifiedInBounds = InBounds;
		if (Descriptor.Is2DGrid())
		{
			FVector MinBounds = InBounds.Min;
			FVector MaxBounds = InBounds.Max;

			MinBounds.Z = 0;
			MaxBounds.Z = GridSize;
			ModifiedInBounds = FBox(MinBounds, MaxBounds);
		}

		// Helper lambda to apply 'InFunc' to the specified grid cell. Only applies if the cell overlaps the modified bounds, and the actor can be found or created and is not unloaded.
		auto ApplyOnCell = [this, &CellTasks, ModifiedInBounds, bCanCreateActor, &Descriptor, &InFunc, InUnloadedFunc](const FIntVector& CellCoord, const FBox& CellBounds)
		{
			const FBox IntersectedBounds = ModifiedInBounds.Overlap(CellBounds);

			if (IntersectedBounds.IsValid)
			{
				APCGPartitionActor* Actor = FindOrCreatePCGPartitionActor(Descriptor, CellCoord, bCanCreateActor);
#if WITH_EDITOR
				if (!Actor && !GetWorld()->IsGameWorld() && TrackingManager->DoesPartitionActorRecordExist(Descriptor, CellCoord))
				{
					FPCGTaskId ExecuteTaskId = InUnloadedFunc(Descriptor, CellCoord, IntersectedBounds);
					
					if (ExecuteTaskId != InvalidPCGTaskId)
					{
						CellTasks.Add(ExecuteTaskId);
					}
				}
				else 
#endif
				if(Actor)
				{
					FPCGTaskId ExecuteTaskId = InFunc(Actor, IntersectedBounds);

					if (ExecuteTaskId != InvalidPCGTaskId)
					{
						CellTasks.Add(ExecuteTaskId);
					}
				}
			}
		};

		FIntVector MinCellCoords = UPCGActorHelpers::GetCellCoord(InBounds.Min, GridSize, Descriptor.Is2DGrid());
		FIntVector MaxCellCoords = UPCGActorHelpers::GetCellCoord(InBounds.Max, GridSize, Descriptor.Is2DGrid());

		// Apply 'InFunc' to all cells in the provided bounds.
		for (int32 Z = MinCellCoords.Z; Z <= MaxCellCoords.Z; ++Z)
		{
			for (int32 Y = MinCellCoords.Y; Y <= MaxCellCoords.Y; ++Y)
			{
				for (int32 X = MinCellCoords.X; X <= MaxCellCoords.X; ++X)
				{
					FIntVector CellCoords(X, Y, Z);

					const FVector Min = FVector(CellCoords) * GridSize;
					const FVector Max = Min + FVector(GridSize);
					FBox CellBounds(Min, Max);

					ApplyOnCell(MoveTemp(CellCoords), MoveTemp(CellBounds));
				}
			}
		}
	}

	// Create a dummy task to wait on dependencies, which creates a dummy task to wait on all cells.
	if (!CellTasks.IsEmpty())
	{
		TArray<FPCGTaskId> AllDependencies(Dependencies);
		AllDependencies.Append(CellTasks);
		return GraphExecutor->ScheduleGeneric([]() { return true; }, /*InSourceComponent=*/nullptr, AllDependencies);
	}
	else
	{
		return InvalidPCGTaskId;
	}
}

void UPCGSubsystem::CleanupLocalComponentsImmediate(UPCGComponent* InOriginalComponent, bool bRemoveComponents)
{
	if (!InOriginalComponent)
	{
		return;
	}

	auto LocalCleanupTask = [bRemoveComponents](UPCGComponent* LocalComponent, const TArray<FPCGTaskId>& ExtraDependencies)
	{
		ensure(ExtraDependencies.IsEmpty());
		if (ensure(LocalComponent) && !LocalComponent->IsCleaningUp())
		{
			LocalComponent->CleanupLocalImmediate(bRemoveComponents);
		}

		return InvalidPCGTaskId;
	};

	TrackingManager->DispatchToRegisteredLocalComponents(InOriginalComponent, LocalCleanupTask);

	// Remove the local component mappings if requested and the component is partitioned. If 'bRemoveComponents' is false, that indicates we are doing a refresh, so destroying
	// the component mappings is counterproductive.
	if (bRemoveComponents && InOriginalComponent->IsPartitioned())
	{
		TrackingManager->DeleteMappingPCGComponentPartitionActor(InOriginalComponent);
	}
}

UPCGComponent* UPCGSubsystem::GetLocalComponent(const FPCGGridDescriptor& GridDescriptor, const FIntVector& CellCoords, const UPCGComponent* InOriginalComponent) const
{
	check(InOriginalComponent);
	return TrackingManager->GetLocalComponent(GridDescriptor, CellCoords, InOriginalComponent);
}

APCGPartitionActor* UPCGSubsystem::GetRegisteredPCGPartitionActor(const FPCGGridDescriptor& GridDescriptor, const FIntVector& GridCoords) const
{
	return TrackingManager->GetPartitionActor(GridDescriptor, GridCoords);
}

APCGPartitionActor* UPCGSubsystem::FindOrCreatePCGPartitionActor(const FPCGGridDescriptor& GridDescriptor, const FIntVector& GridCoords, bool bCanCreateActor, bool bHideFromOutliner) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGActorHelpers::FindOrCreatePCGPartitionActor);

	UWorld* World = GetWorld();

	if (!World)
	{
		UE_LOGF(LogPCG, Error, "[FindOrCreatePCGPartitionActor] World is null.");
		return nullptr;
	}

	if (!PCGWorldActor)
	{
		UE_LOGF(LogPCG, Error, "[FindOrCreatePCGPartitionActor] PCGWorldActor is null.");
		return nullptr;
	}

	// Attempt to find an existing PA.
	if (APCGPartitionActor* ExistingActor = GetRegisteredPCGPartitionActor(GridDescriptor, GridCoords))
	{
		return ExistingActor;
	}
	else if (!GridDescriptor.IsRuntime())
	{
		// In a Game World PAs need to be Pre-Existing. 
		// The Original PCG Component will be marked as Generated on load see UPCGComponent::BeginPlay
		if (GetWorld()->IsGameWorld())
		{
			return nullptr;
		}

#if WITH_EDITOR
		// Check if there is already an unloaded actor for this cell. RuntimeGenerated PAs are never unloaded, so we ignore them.
		if (TrackingManager->DoesPartitionActorRecordExist(GridDescriptor, GridCoords))
		{
			return nullptr;
		}
#endif
	}

	// If FindOrCreatePCGPartitionActor is called on a Level while it is not fully registered.
	// We can't create the actor as it may already exist but not have been registered yet.
	if (!World->PersistentLevel->bAreComponentsCurrentlyRegistered)
	{
		return nullptr;
	}

#if WITH_EDITOR
	// Do not try and create while executing a Undo/Redo because actor might already be in the process of being re-created by the transaction
	if (GIsTransacting)
	{
		return nullptr;
	}
#endif
	
	if (!bCanCreateActor)
	{
		return nullptr;
	}

	FActorSpawnParameters SpawnParams;

	// Important to override the level to make sure we spawn in the persistent level and not the editor's current editing level
	SpawnParams.OverrideLevel = World->PersistentLevel;

#if WITH_EDITOR
	SpawnParams.Name = *APCGPartitionActor::GetPCGPartitionActorName(GridDescriptor, GridCoords);
	SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Required_Fatal;
	SpawnParams.bHideFromSceneOutliner = bHideFromOutliner;
#endif
	if (GridDescriptor.IsRuntime())
	{
		SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
		SpawnParams.ObjectFlags |= RF_Transient;
		SpawnParams.ObjectFlags &= ~RF_Transactional;
	}
		
#if WITH_EDITOR
	bool bPushedContext = false;
	const UExternalDataLayerAsset* ExternalDataLayerAsset = nullptr;
	
	// No need to do any DataLayer assignment in a game world
	if (!World->IsGameWorld())
	{
		TArray<TSoftObjectPtr<UDataLayerAsset>> DataLayerAssets;
		
		GridDescriptor.GetDataLayerAssets(DataLayerAssets, ExternalDataLayerAsset);

		// Avoid relying on the Editor Context at all
		UActorEditorContextSubsystem::Get()->PushContext();
		bPushedContext = true;
	}

	ON_SCOPE_EXIT
	{ 
		if (bPushedContext)
		{
			UActorEditorContextSubsystem::Get()->PopContext();
		}
	};

	// Specify EDL we want to use if any for spawning this actor
	FScopedOverrideSpawningLevelMountPointObject EDLScope(ExternalDataLayerAsset);

	// Handle the case where the actor already exists, but is in the undo stack (was deleted) (copied from ActorPartitionSubsystem, we should probably merge back into it at some point) 
	if (SpawnParams.NameMode == FActorSpawnParameters::ESpawnActorNameMode::Required_Fatal)
	{
		if (UObject* ExistingObject = StaticFindObject(nullptr, World->PersistentLevel, *SpawnParams.Name.ToString()))
		{
			AActor* ExistingActor = CastChecked<AActor>(ExistingObject);
			// This actor is expected to be invalid
			check(!IsValidChecked(ExistingActor));
			ExistingActor->Modify();

			// Don't go through AActor::Rename here because we aren't changing outers (the actor's level). We just want to rename that actor 
			// out of the way so we can spawn the new one in the exact same package, keeping the package name intact.
			ExistingActor->UObject::Rename(nullptr, nullptr, REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional);

			// Reuse ActorGuid so that ActorDesc can be updated on save
			SpawnParams.OverrideActorGuid = ExistingActor->GetActorGuid();
		}
	}
#endif

	const FVector CellCenter(FVector(GridCoords.X + 0.5, GridCoords.Y + 0.5, GridCoords.Z + 0.5) * GridDescriptor.GetGridSize());
	APCGPartitionActor* NewActor = CastChecked<APCGPartitionActor>(World->SpawnActor(APCGPartitionActor::StaticClass(), &CellCenter, nullptr, SpawnParams));

	if (GridDescriptor.IsRuntime())
	{
		NewActor->SetToRuntimeGenerated();
	}

#if WITH_EDITOR
	NewActor->SetLockLocation(true);
	NewActor->SetActorLabel(SpawnParams.Name.ToString());
#endif

	// Empty GUID if runtime generated, since transient PAs don't need one.
	NewActor->PostCreation(GridDescriptor);

	return NewActor;
}

TSet<TObjectPtr<APCGPartitionActor>> UPCGSubsystem::GetPCGComponentPartitionActorMappings(UPCGComponent* InComponent) const
{
	return TrackingManager->GetPCGComponentPartitionActorMappings(InComponent);
}

FPCGGenSourceManager* UPCGSubsystem::GetGenSourceManager() const
{
	return RuntimeGenScheduler ? RuntimeGenScheduler->GenSourceManager : nullptr;
}

void UPCGSubsystem::OnViewportResized(FViewport* Viewport, uint32 val)
{
	DirtyRuntimeGenExecutionSources(EPCGChangeType::None, ERuntimeGenRefreshReason::RenderStateRefresh);
}

void UPCGSubsystem::ResetCacheStats()
{
	if (GraphExecutor)
	{
		GraphExecutor->GetCache().ResetStats();
	}
}

void UPCGSubsystem::LogCacheStats()
{
	if (GraphExecutor)
	{
		GraphExecutor->GetCache().LogStats();
	}
}

#if WITH_EDITOR

FPCGTaskId UPCGSubsystem::ScheduleRefresh(UPCGComponent* Component, bool bForceRegen)
{
	check(Component && !Component->IsManagedByRuntimeGenSystem());

	TWeakObjectPtr<UPCGComponent> ComponentPtr(Component);

	auto RefreshTask = [ComponentPtr, bForceRegen]() {
		if (UPCGComponent* Component = ComponentPtr.Get())
		{
			Component->OnRefresh(bForceRegen);
		}
		return true;
	};

	return GraphExecutor->ScheduleGeneric(RefreshTask, Component, {});
}

void UPCGSubsystem::DirtyGraph(UPCGComponent* Component, const FBox& InBounds, EPCGComponentDirtyFlag DirtyFlag)
{
	check(Component);

	// Immediate operation
	auto DirtyTask = [DirtyFlag](UPCGComponent* LocalComponent, const TArray<FPCGTaskId>& ExtraDependencies)
	{
		ensure(ExtraDependencies.IsEmpty());
		LocalComponent->DirtyGenerated(DirtyFlag);
		return InvalidPCGTaskId;
	};

	TrackingManager->DispatchToRegisteredLocalComponents(Component, DirtyTask);
}

// deprecated
void UPCGSubsystem::ClearPCGLink(UPCGComponent* InComponent, const FBox& InBounds, AActor* InNewActor)
{
	FPCGMoveResourceParams Params
	{
		.Target = InNewActor
	};

	ClearPCGLink(InComponent, InBounds, Params);
}

void UPCGSubsystem::ClearPCGLink(IPCGGraphExecutionSource* InExecutionSource, const FBox& InBounds, const FPCGMoveResourceParams& InParams)
{
	UPCGComponent* InComponent = Cast<UPCGComponent>(InExecutionSource);

	if (!InComponent)
	{
		//@todo_pcg Implement once we support this on generic execution sources
		return;
	}

	//@todo_pcg - can make the target actor pointer a weak instead?
	TWeakObjectPtr<AActor> NewActorPtr(InParams.GetTarget<AActor>());
	TWeakObjectPtr<UPCGComponent> ComponentPtr(InComponent);

	auto MoveTask = [this, NewActorPtr, ComponentPtr, Params=InParams](APCGPartitionActor* PCGActor, const FBox& InIntersectedBounds)
	{
		check(NewActorPtr.IsValid() && ComponentPtr.IsValid() && PCGActor != nullptr);

		if (TObjectPtr<UPCGComponent> LocalComponent = PCGActor->GetLocalComponent(ComponentPtr.Get()))
		{
			LocalComponent->MoveResourcesToNewActor(Params, /*bCreateChild=*/true);
		}

		return true;
	};

	auto ScheduleTask = [this, ComponentPtr, MoveTask](APCGPartitionActor* InPCGActor, const FBox& InIntersectedBounds)
	{
		auto MoveTaskInternal = [MoveTask, InPCGActor, InIntersectedBounds]()
		{
			return MoveTask(InPCGActor, InIntersectedBounds);
		};

		return GraphExecutor->ScheduleGeneric(MoveTaskInternal, ComponentPtr.Get(), /*TaskExecutionDependencies=*/{});
	};

	auto ScheduleUnloadedTask = [this, ComponentPtr, MoveTask](const FPCGGridDescriptor& InGridDescriptor, const FIntVector& InGridCoord, const FBox& InIntersectedBounds)
	{
		auto MoveTaskInternal = [this, MoveTask, InGridDescriptor, InGridCoord, InIntersectedBounds]()
		{
			TUniquePtr<IWorldPartitionActorLoaderInterface::ILoaderAdapter> LoaderAdapter = MakeUnique<FPCGPartitionActorLoaderAdapter>(GetWorld(), InIntersectedBounds, TEXT("UPCGSubsystem::ClearPCGLink"));
			LoaderAdapter->Load();
			if (APCGPartitionActor* PCGActor = FindOrCreatePCGPartitionActor(InGridDescriptor, InGridCoord, /*bCanCreateActor=*/false))
			{
				return MoveTask(PCGActor, InIntersectedBounds);
			}

			return true;
		};

		return GraphExecutor->ScheduleGeneric(MoveTaskInternal, ComponentPtr.Get(), /*TaskExecutionDependencies=*/{});
	};

	bool bHasUnbounded = false;
	PCGHiGenGrid::FSizeArray GridSizes;
	ensure(PCGHelpers::GetGenerationGridSizes(InComponent->GetGraph(), GetPCGWorldActor(), GridSizes, bHasUnbounded));

	FPCGTaskId TaskId = InvalidPCGTaskId;
	if (!GridSizes.IsEmpty())
	{
		// Load unloaded PAs to make sure Clear PCG Link is not partial
		TaskId = ForAllOverlappingCells(InComponent, InBounds, GridSizes, /*bCanCreateActor=*/false, /*Dependencies=*/{}, ScheduleTask, ScheduleUnloadedTask);
	}

	// Verify if the NewActor has some components attached to its root or attached actors. If not, destroy it.
	// Return false if the new actor is not valid or destroyed.
	auto VerifyAndDestroyNewActor = [this, NewActorPtr]()
	{
		check(NewActorPtr.IsValid());

		USceneComponent* RootComponent = NewActorPtr->GetRootComponent();
		check(RootComponent);

		AActor* NewActor = NewActorPtr.Get();

		TArray<AActor*> AttachedActors;
		NewActor->GetAttachedActors(AttachedActors);

		if (RootComponent->GetNumChildrenComponents() == 0 && AttachedActors.IsEmpty())
		{
			GetWorld()->DestroyActor(NewActor);
			return false;
		}

		return true;
	};

	if (TaskId != InvalidPCGTaskId)
	{
		auto CleanupTask = [this, ComponentPtr, VerifyAndDestroyNewActor]()
		{

			// If the new actor is valid, clean up the original component.
			if (VerifyAndDestroyNewActor())
			{
				check(ComponentPtr.IsValid());
				ComponentPtr->Cleanup(/*bRemoveComponents=*/true);
			}

			return true;
		};

		GraphExecutor->ScheduleGeneric(CleanupTask, InComponent, { TaskId });
	}
	else
	{
		VerifyAndDestroyNewActor();
	}
}

void UPCGSubsystem::DeleteSerializedPartitionActors(bool bOnlyDeleteUnused, bool bOnlyChildren)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSubsystem::DeleteSerializedPartitionActors);

	TSet<UPackage*> PackagesToCleanup;
	UWorld* World = GetWorld();

	if (!World)
	{
		return;
	}

	UWorldPartition* WorldPartition = World->GetWorldPartition();
	TMap<FGuid, TArray<FGuid>> Attachments;

	auto GetAttachments = [WorldPartition](const FGuid& ParentActor, const TMap<FGuid, TArray<FGuid>>& Attachments, auto&& GetAttachmentsRecursive, TArray<FGuid>& OutAttachedActors) -> void
	{
		if (const TArray<FGuid>* Attached = Attachments.Find(ParentActor))
		{
			OutAttachedActors.Append(*Attached);
			for (const FGuid& AttachedGuid : *Attached)
			{
				GetAttachmentsRecursive(AttachedGuid, Attachments, GetAttachmentsRecursive, OutAttachedActors);
			}
		}
	};

	auto GatherAndDestroyActors = [this, &PackagesToCleanup, World, WorldPartition, bOnlyDeleteUnused, bOnlyChildren, &GetAttachments, &Attachments](AActor* Actor) -> bool
	{
		TObjectPtr<APCGPartitionActor> PartitionActor = CastChecked<APCGPartitionActor>(Actor);

		// Do not delete RuntimeGen PAs or PAs with graph instances if we are only deleting unused PAs.
		if (!PartitionActor->IsRuntimeGenerated() && (!bOnlyDeleteUnused || !PartitionActor->HasGraphInstances()))
		{
			TArray<AActor*> ActorsToDelete;

			TArray<FWorldPartitionReference> ActorReferences;

			// Load Generated Resources to delete them
			TArray<TSoftObjectPtr<AActor>> ManagedActors = UPCGComponent::GetManagedActorPaths(PartitionActor);
			for (const TSoftObjectPtr<AActor>& ManagedActorPath : ManagedActors)
			{
				// Test to see if actor is loaded first to support non World Partition worlds
				AActor* ManagedActor = ManagedActorPath.Get();
				if (!ManagedActor && WorldPartition)
				{
					if (const FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstanceByPath(ManagedActorPath.ToSoftObjectPath()))
					{
						FWorldPartitionReference& ActorReference = ActorReferences.Emplace_GetRef(FWorldPartitionReference(ActorDescInstance->GetContainerInstance(), ActorDescInstance->GetGuid()));
						ManagedActor = ActorReference.GetActor();
					}
				}

				if (ManagedActor)
				{
					ActorsToDelete.Add(ManagedActor);
				}
			}
			
			// Load Attachments before getting them in the next code block, since loading an actor doesn't load it's attachments (the reference is child to parent)
			if (WorldPartition)
			{
				TArray<FGuid> AttachedActors;
				GetAttachments(Actor->GetActorGuid(), Attachments, GetAttachments, AttachedActors);
				for (const FGuid& AttachedActor : AttachedActors)
				{
					ActorReferences.Add(FWorldPartitionReference(WorldPartition, AttachedActor));
				}
			}

			// We might have actors that weren't saved as managed resources that are attached and have the proper tag
			TArray<AActor*> AttachedActors;
			PartitionActor->GetAttachedActors(AttachedActors, /*bResetArray=*/true, /*bRecursivelyIncludeAttachedActors=*/ true);
			for (AActor* AttachedActor : AttachedActors)
			{
				if (AttachedActor->ActorHasTag(PCGHelpers::DefaultPCGActorTag))
				{
					if (!ActorsToDelete.Contains(AttachedActor))
					{
						ActorsToDelete.Add(AttachedActor);
					}
				}
				else if (WorldPartition && !bOnlyChildren)
				{
					// If actor isn't getting deleted but is an attached actor and it's Partition Actor parent 
					// will get deleted then Pin the actor so it stays loaded and modified for the user to save
					WorldPartition->PinActors({ AttachedActor->GetActorGuid() });
				}
			}

			if (!bOnlyChildren)
			{
				ActorsToDelete.Add(PartitionActor);
			}

			for (AActor* ActorToDelete : ActorsToDelete)
			{
				if (UPackage* ExternalPackage = ActorToDelete->GetExternalPackage())
				{
					// Since we aren't in a transaction (and this operation isnt undoable) make sure UPackage objects are no longer marked as RF_Standalone so it they can get properly GCed in (ObjectTools::CleanupAfterSuccessfulDelete)
					// Without this call we end up spamming in UWorldPartition::OnGCPostReachabilityAnalysis
					ForEachObjectWithPackage(ExternalPackage, [this, Actor](UObject* Object)
					{
						Object->ClearFlags(RF_Standalone);
						return true;
					}, EGetObjectsFlags::None);

					PackagesToCleanup.Add(ExternalPackage);
				}

				World->DestroyActor(ActorToDelete);
			}
		}

		return true;
	};

	// First, clear selection otherwise it might crash
	if (GEditor)
	{
		GEditor->SelectNone(true, true, false);
		// Any reference in the Transaction buffer to the deleted actors will prevent them from being properly GCed so here we reset the transaction buffer
		GEditor->ResetTransaction(NSLOCTEXT("PCGSubsystem", "DeletePartitionActorsResetTransaction", "Deleted PCG Actors"));
	}

	{
		FScopedSlowTask DeleteTask(0, NSLOCTEXT("PCGSubsystem", "DeletePartitionActors", "Deleting PCG Actors..."));
		DeleteTask.MakeDialog();

		FWorldPartitionHelpers::FForEachActorWithLoadingResult LoadingResult;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSubsystem::DeleteSerializedPartitionActors::ForEachActorInLevel);
			if (WorldPartition)
			{
				// Gather Attach Parent information
				FWorldPartitionHelpers::ForEachActorDescInstance(WorldPartition, [&Attachments](const FWorldPartitionActorDescInstance* ActorDescInstance)
				{
					if (ActorDescInstance->GetParentActor().IsValid())
					{
						Attachments.FindOrAdd(ActorDescInstance->GetParentActor()).Add(ActorDescInstance->GetGuid());
					}

					return true;
				});

				// Process Loaded Actors first (and unsaved actors that don't have an Actor Desc yet)
				// Do not use UPCGActorHelpers::ForEachActorInLevel as GatherAndDestroy can end up modifying the Actors array (by loading actors) in a WP World
				TSet<FGuid> ProcessedActors;
				const TArray<AActor*> ActorsCopy(World->PersistentLevel->Actors);
				for (AActor* Actor : ActorsCopy)
				{
					if (APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(Actor))
					{
						ProcessedActors.Add(Actor->GetActorGuid());
						GatherAndDestroyActors(Actor);
					}
				}
				
				FWorldPartitionHelpers::FForEachActorWithLoadingParams ForEachActorWithLoadingParams;
				ForEachActorWithLoadingParams.bKeepReferences = true;
				ForEachActorWithLoadingParams.ActorClasses = { APCGPartitionActor::StaticClass() };

				// Load and Process remaining actors
				FWorldPartitionHelpers::ForEachActorWithLoading(WorldPartition, [&GatherAndDestroyActors, &ProcessedActors](const FWorldPartitionActorDescInstance* ActorDescInstance)
				{
					if(AActor* Actor = ActorDescInstance->GetActor(); Actor && !ProcessedActors.Contains(Actor->GetActorGuid()))
					{
						GatherAndDestroyActors(Actor);
					}
					return true;
				},
				ForEachActorWithLoadingParams);
			}
			else
			{
				UPCGActorHelpers::ForEachActorInLevel<APCGPartitionActor>(World->PersistentLevel, GatherAndDestroyActors);
			}
		}

		if (PackagesToCleanup.Num() > 0)
		{
			ObjectTools::CleanupAfterSuccessfulDelete(PackagesToCleanup.Array(), /*bPerformanceReferenceCheck=*/true);
		}

		// Non World Partition Levels might have deleted actors without saving anything and we need to GC so that Partition Actors can be created again (avoid name clash)
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
}

void UPCGSubsystem::PropagateEditingModeToLocalComponents(UPCGComponent* InComponent, EPCGEditorDirtyMode EditingMode)
{
	if (ensure(InComponent && InComponent->IsPartitioned()))
	{
		FBox Bounds = TrackingManager->PartitionedOctree.GetBounds(InComponent);
		if (!Bounds.IsValid)
		{
			return;
		}

		TrackingManager->ForAllIntersectingPartitionActors(Bounds, [InComponent, EditingMode](APCGPartitionActor* Actor)
		{
			Actor->ChangeTransientState(InComponent, EditingMode);
		});
	}
}

void UPCGSubsystem::BuildLandscapeCache(bool bQuiet, bool bForceBuild)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGSubsystem::BuildLandscapeCache);
	if (UPCGLandscapeCache* LandscapeCache = GetLandscapeCache())
	{
		if (bForceBuild || LandscapeCache->SerializationMode != EPCGLandscapeCacheSerializationMode::NeverSerialize)
		{
			LandscapeCache->PrimeCache();
		}
	}
	else if(!bQuiet)
	{
		UE_LOGF(LogPCG, Error, "Unable to build landscape cache because either the world is null or there is no PCG world actor");
	}
}

void UPCGSubsystem::ClearLandscapeCache()
{
	if (UPCGLandscapeCache* LandscapeCache = GetLandscapeCache())
	{
		LandscapeCache->ClearCache();
	}
}

void UPCGSubsystem::GenerateAllPCGComponents(bool bForce) const
{
	TArray<FPCGTaskId> GenerateTasks;

	for (IPCGGraphExecutionSource* ExecutionSource : TrackingManager->GetAllRegisteredExecutionSources())
	{
		IPCGGraphExecutionState::FGenerateParams GenerateParams;
		GenerateParams.bEvenIfAlreadyGenerated = bForce;
		const FPCGTaskId TaskId = ExecutionSource->GetExecutionState().Generate(GenerateParams);
		if (TaskId != InvalidPCGTaskId)
		{
			GenerateTasks.Add(TaskId);
		}
	}

	auto BroadcastGenerated = [this]()
	{
		OnAllComponentsGenerated.Broadcast();
		return true;
	};

	if (!GenerateTasks.IsEmpty())
	{
		ScheduleGeneric(BroadcastGenerated, /*ExecutionSource=*/nullptr,	GenerateTasks);
	}
	else
	{
		// Broadcast immediately if nothing happened.
		BroadcastGenerated();
	}
}

void UPCGSubsystem::CleanupAllPCGComponents(bool bPurge) const
{
	TArray<FPCGTaskId> CleanupTasks;

	for (IPCGGraphExecutionSource* ExecutionSource : TrackingManager->GetAllRegisteredExecutionSources())
	{
		UPCGComponent* Component = Cast<UPCGComponent>(ExecutionSource);
		if (Component && bPurge)
		{
			Component->CleanupLocalDeleteAllGeneratedObjects({});
		}
		else
		{
			IPCGGraphExecutionState::FCleanupParams CleanupParams;
			CleanupParams.bReleaseManagedResources = true;
			const FPCGTaskId TaskId = ExecutionSource->GetExecutionState().Cleanup(CleanupParams);
			if (TaskId != InvalidPCGTaskId)
			{
				CleanupTasks.Add(TaskId);
			}
		}
	}

	auto BroadcastCleanup = [this]()
	{
		OnAllComponentsCleanedup.Broadcast();
		return true;
	};

	if (bPurge || CleanupTasks.IsEmpty())
	{
		// Broadcast immediately if nothing happened, or we purged (immediate cleanup)
		BroadcastCleanup();
	}
	else
	{
		ScheduleGeneric(BroadcastCleanup, /*ExecutionSource=*/nullptr, CleanupTasks);
	}
}

void UPCGSubsystem::ClearLinkForAllPCGComponents(const FPCGMoveResourceParams& InParams) const
{
	for (IPCGGraphExecutionSource* ExecutionSource : TrackingManager->GetAllRegisteredExecutionSources())
	{
		if (UPCGComponent* Component = Cast<UPCGComponent>(ExecutionSource))
		{
			Component->ClearPCGLink(InParams);
		}
	}

	OnAllComponentsClearedLink.Broadcast();
}

void UPCGSubsystem::NotifySelectionKeyChanged(const FPCGSelectionKey& InSelectionKey, const UObject* InOriginatingChangeObject, TArrayView<const FBox> InChangedBounds)
{
	TrackingManager->OnSelectionKeyChanged(InSelectionKey, InOriginatingChangeObject, InChangedBounds);
}

void UPCGSubsystem::CreateMissingPartitionActors()
{
	if (!PCGHelpers::IsRuntimeOrPIE())
	{
		TrackingManager->ForAllOriginalExecutionSources([this](IPCGGraphExecutionSource* InExecutionSource)
		{
			UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource);
			if (PCGComponent && PCGComponent->IsPartitioned() && !PCGComponent->IsManagedByRuntimeGenSystem())
			{
				bool bHasUnbounded = false;
				PCGHiGenGrid::FSizeArray GridSizes;
				ensure(PCGHelpers::GetGenerationGridSizes(PCGComponent->GetGraph(), GetPCGWorldActor(), GridSizes, bHasUnbounded));
				if (!GridSizes.IsEmpty())
				{
					CreatePartitionActorsWithinBounds(PCGComponent, PCGComponent->GetGridBounds(), GridSizes);
				}
				TrackingManager->UpdateMappingPCGComponentPartitionActor(PCGComponent);
			}
		});
	}
}

void UPCGSubsystem::OnPreviewPlatformChanged(EShaderPlatform /*NewShaderPlatform*/)
{
	FlushCache();

	RefreshAllRuntimeGenExecutionSources(EPCGChangeType::GenerationGrid, ERuntimeGenRefreshReason::RenderStateRefresh);
}

void UPCGSubsystem::CreatePartitionActorsWithinBounds(UPCGComponent* InComponent, const FBox& InBounds, const PCGHiGenGrid::FSizeArray& InGridSizes)
{
	UWorld* World = GetWorld();
	if (!PCGHelpers::IsRuntimeOrPIE() && !UE::GetIsEditorLoadingPackage() && !IsPartitionActorCreationDisabledForWorld(World))
	{
		// We can't spawn actors if we are running constructions scripts, asserting when we try to get the actor with the WP API.
		// We should never enter this if we are in a construction script. If the ensure is hit, we need to fix it.
		if (ensure(World && !World->bIsRunningConstructionScript))
		{
			ForAllOverlappingCells(InComponent, InBounds, InGridSizes, /*bCanCreateActor=*/true, {}, [](APCGPartitionActor*, const FBox&) { return InvalidPCGTaskId; });
		}
	}
}

void UPCGSubsystem::UpdateMappingPCGComponentPartitionActor(UPCGComponent* InComponent) 
{ 
	TrackingManager->UpdateMappingPCGComponentPartitionActor(InComponent); 
}

void UPCGSubsystem::SetChainedDispatchToLocalComponents(bool bInChainedDispatch)
{
	TrackingManager->SetChainedDispatchToLocalComponents(bInChainedDispatch);
}

void UPCGSubsystem::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances)
{
	if (RuntimeGenScheduler)
	{
		RuntimeGenScheduler->OnObjectsReplaced(InOldToNewInstances);
	}

	if (TrackingManager)
	{
		const TSet<IPCGGraphExecutionSource*> RegisteredSources = TrackingManager->GetAllRegisteredExecutionSources();

		for (const TPair<UObject*, UObject*>& OldToNew : InOldToNewInstances)
		{
			UPCGComponent* OldComponent = Cast<UPCGComponent>(OldToNew.Key);
			UPCGComponent* NewComponent = Cast<UPCGComponent>(OldToNew.Value);

			if (!OldComponent || !NewComponent)
			{
				continue;
			}

			if (RegisteredSources.Contains(OldComponent))
			{
				// Original component: remap the octree entry and partition-actor mappings.
				TrackingManager->RemapExecutionSource(OldComponent, NewComponent, /*bDoPartitionMapping=*/true);
			}
			else if (APCGPartitionActor* PartitionActor = Cast<APCGPartitionActor>(OldComponent->GetOwner()))
			{
				// Local component: not in the octree, but its owning partition actor keeps a mapping
				// keyed by the local component that must be repointed.
				PartitionActor->RemapLocalComponent(OldComponent, NewComponent);
			}
		}
	}
}

void UPCGSubsystem::OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld)
{
	if (InLevel && InWorld == GetWorld())
	{
		InLevel->OnLoadedActorRemovedFromLevelPreEvent.AddUObject(this, &UPCGSubsystem::OnLoadedActorRemovedFromLevelPreEvent);
		InLevel->OnLoadedActorRemovedFromLevelPostEvent.AddUObject(this, &UPCGSubsystem::OnLoadedActorRemovedFromLevelPostEvent);
	}
}

void UPCGSubsystem::OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	if (InLevel && InWorld == GetWorld())
	{
		InLevel->OnLoadedActorRemovedFromLevelPreEvent.RemoveAll(this);
		InLevel->OnLoadedActorRemovedFromLevelPostEvent.RemoveAll(this);
	}
}

void UPCGSubsystem::OnLoadedActorRemovedFromLevelPreEvent(const TArray<AActor*>& InActors)
{
	check(IsInGameThread());
	++NumLevelUnloadingActors;
}

void UPCGSubsystem::OnLoadedActorRemovedFromLevelPostEvent(const TArray<AActor*>& InActors)
{
	check(IsInGameThread());
	check(NumLevelUnloadingActors > 0);
	--NumLevelUnloadingActors;
}

bool UPCGSubsystem::IsComponentBeingUnloaded(UActorComponent* InComponent) const
{
	// World Partition actor unload
	if (NumLevelUnloadingActors > 0)
	{
		return true;
	}

	// Sub-level being removed 
	ULevel* OuterLevel = InComponent ? InComponent->GetTypedOuter<ULevel>() : nullptr;
	if (OuterLevel && OuterLevel->bIsBeingRemoved)
	{
		return true;
	}

	// World being unloaded (map change)
	if (UWorld* World = GetWorld(); World && World->IsBeingCleanedUp())
	{
		return true;
	}

	return false;
}

void UPCGSubsystem::OnPCGGraphCancelled(IPCGGraphExecutionSource* InExecutionSource)
{
	TrackingManager->OnPCGGraphCancelled(InExecutionSource);
}

void UPCGSubsystem::OnPCGGraphStartGenerating(IPCGGraphExecutionSource* InExecutionSource)
{
	TrackingManager->OnPCGGraphStartsGenerating(InExecutionSource);
}

void UPCGSubsystem::OnPCGGraphGenerated(IPCGGraphExecutionSource* InExecutionSource, TOptional<FBox> OptionalExecutionBounds)
{
	TrackingManager->OnPCGGraphGeneratedOrCleaned(InExecutionSource, MoveTemp(OptionalExecutionBounds));
}

void UPCGSubsystem::OnPCGGraphCleaned(IPCGGraphExecutionSource* InExecutionSource)
{
	TrackingManager->OnPCGGraphGeneratedOrCleaned(InExecutionSource);
}

void UPCGSubsystem::UpdateTracking(IPCGGraphExecutionSource* InExecutionSource, const TArray<FPCGSelectionKey>* OptionalChangedKeys)
{
	TrackingManager->UpdateTracking(InExecutionSource, OptionalChangedKeys);
}

void UPCGSubsystem::ResetPartitionActorsMap()
{
	TrackingManager->ResetPartitionActorsMap();
}

#endif // WITH_EDITOR

void UPCGSubsystem::ExecuteBeginTickActions()
{
	TArray<FTickAction> Actions = MoveTemp(BeginTickActions);
	BeginTickActions.Reset();

	for (FTickAction& Action : Actions)
	{
		Action();
	}
}

void UPCGSubsystem::OnGlobalComponentReregisterContextDestroyed()
{
	RefreshAllRuntimeGenExecutionSources(EPCGChangeType::None, ERuntimeGenRefreshReason::RenderStateRefresh);
}

void UPCGSubsystem::RegisterPartitionActor(APCGPartitionActor* InActor)
{
	TrackingManager->RegisterPartitionActor(InActor);
}

void UPCGSubsystem::UnregisterPartitionActor(APCGPartitionActor* InActor)
{
	TrackingManager->UnregisterPartitionActor(InActor);
}

bool UPCGSubsystem::RegisterOrUpdateExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bDoPartitionMapping)
{
	return TrackingManager->RegisterOrUpdateExecutionSource(InExecutionSource, bDoPartitionMapping);
}

bool UPCGSubsystem::RemapExecutionSource(const IPCGGraphExecutionSource* InOldExecutionSource, IPCGGraphExecutionSource* InNewExecutionSource, bool bDoPartitionMapping)
{
	return TrackingManager->RemapExecutionSource(InOldExecutionSource, InNewExecutionSource, bDoPartitionMapping);
}

void UPCGSubsystem::UnregisterExecutionSource(IPCGGraphExecutionSource* InExecutionSource, bool bForce)
{
#if WITH_EDITOR
	GetOnPCGSourceUnregistered().Broadcast(InExecutionSource);
#endif

	return TrackingManager->UnregisterExecutionSource(InExecutionSource, bForce);
}

TSet<IPCGGraphExecutionSource*> UPCGSubsystem::GetAllRegisteredPartitionedExecutionSources() const
{
	return TrackingManager->GetAllRegisteredPartitionedExecutionSources();
}

TSet<IPCGGraphExecutionSource*> UPCGSubsystem::GetAllRegisteredExecutionSources() const
{
	return TrackingManager->GetAllRegisteredExecutionSources();
}

void UPCGSubsystem::ForAllIntersectingPartitionedExecutionSources(const FBoxCenterAndExtent& InBounds, TFunctionRef<void(IPCGGraphExecutionSource*)> InFunc) const
{
	return TrackingManager->ForAllIntersectingPartitionedExecutionSources(InBounds, std::move(InFunc));
}

TArray<IPCGGraphExecutionSource*> UPCGSubsystem::GetAllIntersectingExecutionSources(const FBoxCenterAndExtent& InBounds) const
{
	return TrackingManager->GetAllIntersectingExecutionSources(InBounds);
}

// deprecated
bool UPCGSubsystem::RegisterOrUpdatePCGComponent(UPCGComponent* InComponent, bool bDoActorMapping)
{
	return RegisterOrUpdateExecutionSource(InComponent, bDoActorMapping);
}

// deprecated
bool UPCGSubsystem::RemapPCGComponent(const UPCGComponent* OldComponent, UPCGComponent* NewComponent, bool bDoActorMapping)
{
	return RemapExecutionSource(OldComponent, NewComponent, bDoActorMapping);
}

// deprecated
void UPCGSubsystem::UnregisterPCGComponent(UPCGComponent* InComponent, bool bForce)
{
	UnregisterExecutionSource(InComponent, bForce);
}

// deprecated
TSet<UPCGComponent*> UPCGSubsystem::GetAllRegisteredPartitionedComponents() const
{
	TSet<IPCGGraphExecutionSource*> ExecutionSources = TrackingManager->GetAllRegisteredPartitionedExecutionSources();

	TSet<UPCGComponent*> PCGComponents;
	PCGComponents.Reserve(ExecutionSources.Num());
	Algo::TransformIf(ExecutionSources, PCGComponents,
		[](IPCGGraphExecutionSource* InExecutionSource) { return Cast<UPCGComponent>(InExecutionSource) != nullptr; },
		[](IPCGGraphExecutionSource* InExecutionSource) { return Cast<UPCGComponent>(InExecutionSource); });

	return PCGComponents;
}

// deprecated
TSet<UPCGComponent*> UPCGSubsystem::GetAllRegisteredComponents() const
{
	TSet<IPCGGraphExecutionSource*> ExecutionSources = TrackingManager->GetAllRegisteredExecutionSources();

	TSet<UPCGComponent*> PCGComponents;
	PCGComponents.Reserve(ExecutionSources.Num());
	Algo::TransformIf(ExecutionSources, PCGComponents,
		[](IPCGGraphExecutionSource* InExecutionSource) { return Cast<UPCGComponent>(InExecutionSource) != nullptr; },
		[](IPCGGraphExecutionSource* InExecutionSource) { return Cast<UPCGComponent>(InExecutionSource); });

	return PCGComponents;
}

// deprecated
void UPCGSubsystem::ForAllIntersectingPartitionedComponents(const FBoxCenterAndExtent& InBounds, TFunctionRef<void(UPCGComponent*)> InFunc) const
{
	auto WrapperFunc = [&InFunc](IPCGGraphExecutionSource* InExecutionSource)
	{
		if(UPCGComponent* PCGComponent = Cast<UPCGComponent>(InExecutionSource))
		{
			InFunc(PCGComponent);
		}
	};

	return TrackingManager->ForAllIntersectingPartitionedExecutionSources(InBounds, WrapperFunc);
}

// deprecated
TArray<UPCGComponent*> UPCGSubsystem::GetAllIntersectingComponents(const FBoxCenterAndExtent& InBounds) const
{
	TArray<IPCGGraphExecutionSource*> IntersectingExecutionSources = TrackingManager->GetAllIntersectingExecutionSources(InBounds);
	
	TArray<UPCGComponent*> IntersectingPCGComponents;
	IntersectingPCGComponents.Reserve(IntersectingExecutionSources.Num());
	Algo::TransformIf(IntersectingExecutionSources, IntersectingPCGComponents,
		[](IPCGGraphExecutionSource* InExecutionSource) { return Cast<UPCGComponent>(InExecutionSource) != nullptr; },
		[](IPCGGraphExecutionSource* InExecutionSource) { return Cast<UPCGComponent>(InExecutionSource); });

	return IntersectingPCGComponents;
}

// deprecated
void UPCGSubsystem::OnOriginalComponentRegistered(UPCGComponent* InComponent)
{
	OnOriginalExecutionSourceRegistered(InComponent);
}

// deprecated
void UPCGSubsystem::OnOriginalComponentUnregistered(UPCGComponent* InComponent)
{
	OnOriginalExecutionSourceUnregistered(InComponent);
}
