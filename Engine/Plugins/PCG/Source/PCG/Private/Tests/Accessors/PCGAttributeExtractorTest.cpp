// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Accessor/PCGAttributeExtractorTest.h"

#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Metadata/Accessors/PCGAttributeExtractor.h"
#include "Metadata/PCGMetadataTypesConstantStruct.h"
#include "Tests/PCGTestsCommon.h"

#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAttributeExtractorTest)

#if WITH_EDITOR

namespace FPCGAttributeExtractorTestHelpers
{
	FPCGMetadataTypesConstantStruct CreateStruct(UObject* InObject = nullptr)
	{
		FPCGMetadataTypesConstantStruct Struct;

		Struct.FloatValue = 1.25f;
		Struct.DoubleValue = 2.0;
		Struct.Int32Value = 42;
		Struct.IntValue = 666;
		Struct.Vector2Value = FVector2D(1.23, 1.58);
		Struct.VectorValue = FVector(3.23, 5.58, 2.69);
		Struct.Vector4Value = FVector4(2.23, 9.58, 4.21, 8.01);
		Struct.BoolValue = true;
		Struct.NameValue = TEXT("Foo");
		Struct.StringValue = TEXT("Bar");
		Struct.RotatorValue = FRotator(0.5, 0.9, 0.4);
		Struct.QuatValue = FQuat(0.145, 0.254, 0.369, 0.478);
		Struct.TransformValue = FTransform(Struct.QuatValue, Struct.VectorValue, FVector(Struct.Vector2Value, 1.9874));
		Struct.SoftClassPathValue = FSoftClassPath(UPCGMetadata::StaticClass());
		if (InObject)
		{
			Struct.SoftObjectPathValue = FSoftObjectPath(InObject);
		}

		return Struct;
	}
}

#define PCG_STRUCT_NAME_CHECK(Name) GET_MEMBER_NAME_CHECKED(FPCGMetadataTypesConstantStruct, Name)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeSingleGetPropertyTest, FPCGTestBaseClass, "Plugins.PCG.Accessor.Property.SimpleGetProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeVectorPropertyExtractorTest, FPCGTestBaseClass, "Plugins.PCG.Accessor.Property.VectorExtractor", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeRotatorPropertyExtractorTest, FPCGTestBaseClass, "Plugins.PCG.Accessor.Property.RotatorExtractor", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeTransformPropertyExtractorTest, FPCGTestBaseClass, "Plugins.PCG.Accessor.Property.TransformExtractor", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeStringPropertyExtractorTest, FPCGTestBaseClass, "Plugins.PCG.Accessor.Property.StringExtractor", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeInvalidPropertyExtractorTest, FPCGTestBaseClass, "Plugins.PCG.Accessor.Property.InvalidExtractor", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertyMultipleDepthTest, FPCGTestBaseClass, "Plugins.PCG.Accessor.Property.MultipleDepth", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertyMultipleDepthRangeTest, FPCGTestBaseClass, "Plugins.PCG.Accessor.Property.MultipleDepthRange", PCGTestsCommon::TestFlags)

bool FPCGAttributeSingleGetPropertyTest::RunTest(const FString& Parameters)
{
	UPCGMetadata* TempMetadata = NewObject<UPCGMetadata>();

	FPCGMetadataTypesConstantStruct Struct = FPCGAttributeExtractorTestHelpers::CreateStruct(TempMetadata);

	auto Verify = [this, &Struct](const auto& ExpectedValue, const FName PropertyName) -> bool
	{
		using PropertyType = std::decay_t<decltype(ExpectedValue)>;

		UTEST_TRUE(TEXT("Property type supported"), PCGAttributeAccessorHelpers::IsPropertyAccessorSupported(PropertyName, FPCGMetadataTypesConstantStruct::StaticStruct()));

		TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(PropertyName, FPCGMetadataTypesConstantStruct::StaticStruct());
		UTEST_TRUE(FString::Printf(TEXT("Valid accessor for property %s"), *PropertyName.ToString()), Accessor.IsValid());
		UTEST_EQUAL(TEXT("Accessor type"), PCG::Private::MetadataTypes<PropertyType>::Id, Accessor->GetUnderlyingType());

		FPCGAttributeAccessorKeysSingleObjectPtr ObjectKey(&Struct);

		PropertyType Value{};
		UTEST_TRUE(TEXT("Getting a value in range"), Accessor->Get<PropertyType>(Value, ObjectKey));
		UTEST_EQUAL(TEXT("SingleGet: Value is equal to expected"), Value, ExpectedValue);

		PropertyType Value2{};
		UTEST_TRUE(TEXT("Getting a value outside range"), Accessor->Get<PropertyType>(Value2, 5, ObjectKey));
		UTEST_EQUAL(TEXT("SingleGet Outside Range: Value is equal to expected"), Value2, ExpectedValue);

		constexpr int NbValues = 3;
		PropertyType Values[NbValues];
		UTEST_TRUE(TEXT("Getting the same value thrice"), Accessor->GetRange<PropertyType>(MakeArrayView(Values, NbValues), 0, ObjectKey));
		for (int i = 0; i < NbValues; ++i)
		{
			UTEST_EQUAL(FString::Printf(TEXT("GetRange: Value %d is equal to expected"), i), Values[i], ExpectedValue);
		}

		PropertyType Values2[NbValues];
		UTEST_TRUE(TEXT("Getting the same value thrice outside range"), Accessor->GetRange<PropertyType>(MakeArrayView(Values2, NbValues), 5, ObjectKey));
		for (int i = 0; i < NbValues; ++i)
		{
			UTEST_EQUAL(FString::Printf(TEXT("GetRange Outside Range: Value %d is equal to expected"), i), Values2[i], ExpectedValue);
		}

		return true;
	};

	bool bTestPassed = true;

	bTestPassed &= Verify(double(Struct.FloatValue), PCG_STRUCT_NAME_CHECK(FloatValue));
	bTestPassed &= Verify(Struct.DoubleValue, PCG_STRUCT_NAME_CHECK(DoubleValue));
	bTestPassed &= Verify(Struct.Int32Value, PCG_STRUCT_NAME_CHECK(Int32Value));
	bTestPassed &= Verify(Struct.IntValue, PCG_STRUCT_NAME_CHECK(IntValue));
	bTestPassed &= Verify(Struct.Vector2Value, PCG_STRUCT_NAME_CHECK(Vector2Value));
	bTestPassed &= Verify(Struct.VectorValue, PCG_STRUCT_NAME_CHECK(VectorValue));
	bTestPassed &= Verify(Struct.Vector4Value, PCG_STRUCT_NAME_CHECK(Vector4Value));
	bTestPassed &= Verify(Struct.BoolValue, PCG_STRUCT_NAME_CHECK(BoolValue));
	bTestPassed &= Verify(Struct.NameValue, PCG_STRUCT_NAME_CHECK(NameValue));
	bTestPassed &= Verify(Struct.StringValue, PCG_STRUCT_NAME_CHECK(StringValue));
	bTestPassed &= Verify(Struct.RotatorValue, PCG_STRUCT_NAME_CHECK(RotatorValue));
	bTestPassed &= Verify(Struct.QuatValue, PCG_STRUCT_NAME_CHECK(QuatValue));
	bTestPassed &= Verify(Struct.TransformValue, PCG_STRUCT_NAME_CHECK(TransformValue));
	bTestPassed &= Verify(Struct.SoftClassPathValue, PCG_STRUCT_NAME_CHECK(SoftClassPathValue));
	bTestPassed &= Verify(Struct.SoftObjectPathValue, PCG_STRUCT_NAME_CHECK(SoftObjectPathValue));

	TempMetadata->MarkAsGarbage();

	return true;
}

bool FPCGAttributeVectorPropertyExtractorTest::RunTest(const FString& Parameters)
{
	FPCGMetadataTypesConstantStruct Struct = FPCGAttributeExtractorTestHelpers::CreateStruct();

	auto Verify = [this, &Struct](const auto& ExpectedValue, const FName PropertyName, const FName ExtractorName) -> bool
	{
		using PropertyType = std::decay_t<decltype(ExpectedValue)>;

		TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(PropertyName, FPCGMetadataTypesConstantStruct::StaticStruct());
		UTEST_TRUE(FString::Printf(TEXT("Valid accessor for property %s"), *PropertyName.ToString()), Accessor.IsValid());

		bool bSuccess = false;
		TUniquePtr<IPCGAttributeAccessor> ExtractorAccessor = PCGAttributeAccessorHelpers::CreateChainAccessor(std::move(Accessor), ExtractorName, bSuccess);
		UTEST_TRUE(TEXT("Valid extractor accessor"), (bSuccess && ExtractorAccessor.IsValid()));
		UTEST_TRUE(TEXT("Invalid accessor (ownership moved)"), !Accessor.IsValid());
		UTEST_EQUAL(TEXT("Extractor accessor type"), PCG::Private::MetadataTypes<PropertyType>::Id, ExtractorAccessor->GetUnderlyingType());

		FPCGAttributeAccessorKeysSingleObjectPtr ObjectKey(&Struct);

		PropertyType Value{};
		UTEST_TRUE(FString::Printf(TEXT("Getting a value for property %s and extractor %s"), *PropertyName.ToString(), *ExtractorName.ToString()), ExtractorAccessor->Get<PropertyType>(Value, ObjectKey));
		UTEST_EQUAL(FString::Printf(TEXT("Value for property %s and extractor %s is equal to expected"), *PropertyName.ToString(), *ExtractorName.ToString()), Value, ExpectedValue);

		return true;
	};

	bool bTestPassed = true;

	// Vector Lengths
	bTestPassed &= Verify(Struct.Vector2Value.Length(), PCG_STRUCT_NAME_CHECK(Vector2Value), PCGAttributeExtractorConstants::VectorLength);
	bTestPassed &= Verify(Struct.VectorValue.Length(), PCG_STRUCT_NAME_CHECK(VectorValue), PCGAttributeExtractorConstants::VectorLength);
	bTestPassed &= Verify(Struct.Vector4Value.Size(), PCG_STRUCT_NAME_CHECK(Vector4Value), PCGAttributeExtractorConstants::VectorLength);

	bTestPassed &= Verify(Struct.Vector2Value.Size(), PCG_STRUCT_NAME_CHECK(Vector2Value), PCGAttributeExtractorConstants::VectorSize);
	bTestPassed &= Verify(Struct.VectorValue.Size(), PCG_STRUCT_NAME_CHECK(VectorValue), PCGAttributeExtractorConstants::VectorSize);
	bTestPassed &= Verify(Struct.Vector4Value.Size(), PCG_STRUCT_NAME_CHECK(Vector4Value), PCGAttributeExtractorConstants::VectorSize);

	// Vector2 Composents
	auto Vec2Composents = [&Verify, &bTestPassed](const auto& Value, const FName PropertyName) -> bool
	{
		bTestPassed &= Verify(Value.X, PropertyName, FName(TEXT("X")));
		bTestPassed &= Verify(Value.Y, PropertyName, FName(TEXT("Y")));

		bTestPassed &= Verify(Value.X, PropertyName, FName(TEXT("R")));
		bTestPassed &= Verify(Value.Y, PropertyName, FName(TEXT("G")));

		bTestPassed &= Verify(FVector2D{ Value.X, Value.Y }, PropertyName, FName(TEXT("XY")));
		bTestPassed &= Verify(FVector2D{ Value.Y, Value.X }, PropertyName, FName(TEXT("YX")));
		bTestPassed &= Verify(FVector2D{ Value.Y, Value.Y }, PropertyName, FName(TEXT("YY")));

		bTestPassed &= Verify(FVector2D{ Value.X, Value.Y }, PropertyName, FName(TEXT("RG")));
		bTestPassed &= Verify(FVector2D{ Value.Y, Value.X }, PropertyName, FName(TEXT("GR")));
		bTestPassed &= Verify(FVector2D{ Value.Y, Value.Y }, PropertyName, FName(TEXT("GG")));

		return true;
	};

	Vec2Composents(Struct.Vector2Value, PCG_STRUCT_NAME_CHECK(Vector2Value));
	Vec2Composents(Struct.VectorValue, PCG_STRUCT_NAME_CHECK(VectorValue));
	Vec2Composents(Struct.Vector4Value, PCG_STRUCT_NAME_CHECK(Vector4Value));
	Vec2Composents(Struct.QuatValue, PCG_STRUCT_NAME_CHECK(QuatValue));

	// Vector3 Composents
	auto Vec3Composents = [&Verify, &bTestPassed](const auto& Value, const FName PropertyName) -> bool
	{
		bTestPassed &= Verify(Value.Z, PropertyName, FName(TEXT("Z")));
		bTestPassed &= Verify(Value.Z, PropertyName, FName(TEXT("B")));

		bTestPassed &= Verify(FVector2D{ Value.Z, Value.X }, PropertyName, FName(TEXT("ZX")));
		bTestPassed &= Verify(FVector2D{ Value.Y, Value.Z }, PropertyName, FName(TEXT("YZ")));

		bTestPassed &= Verify(FVector2D{ Value.Z, Value.X }, PropertyName, FName(TEXT("BR")));
		bTestPassed &= Verify(FVector2D{ Value.Y, Value.Z }, PropertyName, FName(TEXT("GB")));

		bTestPassed &= Verify(FVector{ Value.Z, Value.X, Value.Y }, PropertyName, FName(TEXT("ZXY")));
		bTestPassed &= Verify(FVector{ Value.Y, Value.Y, Value.Y }, PropertyName, FName(TEXT("YYY")));
		bTestPassed &= Verify(FVector{ Value.Y, Value.Y, Value.X }, PropertyName, FName(TEXT("YYX")));
		bTestPassed &= Verify(FVector{ Value.X, Value.Y, Value.Z }, PropertyName, FName(TEXT("XYZ")));

		bTestPassed &= Verify(FVector{ Value.Z, Value.X, Value.Y }, PropertyName, FName(TEXT("BRG")));
		bTestPassed &= Verify(FVector{ Value.Y, Value.Y, Value.Y }, PropertyName, FName(TEXT("GGG")));
		bTestPassed &= Verify(FVector{ Value.Y, Value.Y, Value.X }, PropertyName, FName(TEXT("GGR")));
		bTestPassed &= Verify(FVector{ Value.X, Value.Y, Value.Z }, PropertyName, FName(TEXT("RGB")));

		return true;
	};

	Vec3Composents(Struct.VectorValue, PCG_STRUCT_NAME_CHECK(VectorValue));
	Vec3Composents(Struct.Vector4Value, PCG_STRUCT_NAME_CHECK(Vector4Value));
	Vec3Composents(Struct.QuatValue, PCG_STRUCT_NAME_CHECK(QuatValue));

	// Vector4 Composents
	auto Vec4Composents = [&Verify, &bTestPassed](const auto& Value, const FName PropertyName) -> bool
	{
		bTestPassed &= Verify(Value.W, PropertyName, FName(TEXT("W")));

		bTestPassed &= Verify(Value.W, PropertyName, FName(TEXT("A")));

		bTestPassed &= Verify(FVector2D{ Value.Z, Value.W }, PropertyName, FName(TEXT("ZW")));
		bTestPassed &= Verify(FVector2D{ Value.W, Value.Y }, PropertyName, FName(TEXT("WY")));

		bTestPassed &= Verify(FVector2D{ Value.Z, Value.W }, PropertyName, FName(TEXT("BA")));
		bTestPassed &= Verify(FVector2D{ Value.W, Value.Y }, PropertyName, FName(TEXT("AG")));

		bTestPassed &= Verify(FVector{ Value.Z, Value.W, Value.Y }, PropertyName, FName(TEXT("ZWY")));
		bTestPassed &= Verify(FVector{ Value.W, Value.W, Value.W }, PropertyName, FName(TEXT("WWW")));
		bTestPassed &= Verify(FVector{ Value.Y, Value.W, Value.X }, PropertyName, FName(TEXT("YWX")));
		bTestPassed &= Verify(FVector{ Value.X, Value.Y, Value.W }, PropertyName, FName(TEXT("XYW")));

		bTestPassed &= Verify(FVector{ Value.Z, Value.W, Value.Y }, PropertyName, FName(TEXT("BAG")));
		bTestPassed &= Verify(FVector{ Value.W, Value.W, Value.W }, PropertyName, FName(TEXT("AAA")));
		bTestPassed &= Verify(FVector{ Value.Y, Value.W, Value.X }, PropertyName, FName(TEXT("GAR")));
		bTestPassed &= Verify(FVector{ Value.X, Value.Y, Value.W }, PropertyName, FName(TEXT("RGA")));

		bTestPassed &= Verify(FVector4{ Value.Z, Value.X, Value.Y, Value.W }, PropertyName, FName(TEXT("ZXYW")));
		bTestPassed &= Verify(FVector4{ Value.Z, Value.Z, Value.Z, Value.Z }, PropertyName, FName(TEXT("ZZZZ")));
		bTestPassed &= Verify(FVector4{ Value.X, Value.Y, Value.X, Value.Y }, PropertyName, FName(TEXT("XYXY")));
		bTestPassed &= Verify(FVector4{ Value.X, Value.Y, Value.Z, Value.W }, PropertyName, FName(TEXT("XYZW")));

		bTestPassed &= Verify(FVector4{ Value.Z, Value.X, Value.Y, Value.W }, PropertyName, FName(TEXT("BRGA")));
		bTestPassed &= Verify(FVector4{ Value.Z, Value.Z, Value.Z, Value.Z }, PropertyName, FName(TEXT("BBBB")));
		bTestPassed &= Verify(FVector4{ Value.X, Value.Y, Value.X, Value.Y }, PropertyName, FName(TEXT("RGRG")));
		bTestPassed &= Verify(FVector4{ Value.X, Value.Y, Value.Z, Value.W }, PropertyName, FName(TEXT("RGBA")));

		return true;
	};

	Vec4Composents(Struct.Vector4Value, PCG_STRUCT_NAME_CHECK(Vector4Value));
	Vec4Composents(Struct.QuatValue, PCG_STRUCT_NAME_CHECK(QuatValue));

	return bTestPassed;
}

bool FPCGAttributeRotatorPropertyExtractorTest::RunTest(const FString & Parameters)
{
	FPCGMetadataTypesConstantStruct Struct = FPCGAttributeExtractorTestHelpers::CreateStruct();

	auto Verify = [this, &Struct](auto ExpectedValue, const FName PropertyName, const FName ExtractorName) -> bool
	{
		using ExpectedType = decltype(ExpectedValue);

		TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(PropertyName, FPCGMetadataTypesConstantStruct::StaticStruct());
		UTEST_TRUE(FString::Printf(TEXT("Valid accessor for property %s"), *PropertyName.ToString()), Accessor.IsValid());

		bool bSuccess = false;
		TUniquePtr<IPCGAttributeAccessor> ExtractorAccessor = PCGAttributeAccessorHelpers::CreateChainAccessor(std::move(Accessor), ExtractorName, bSuccess);
		UTEST_TRUE(TEXT("Valid extractor accessor"), (bSuccess && ExtractorAccessor.IsValid()));
		UTEST_TRUE(TEXT("Invalid accessor (ownership moved)"), !Accessor.IsValid());
		UTEST_EQUAL(TEXT("Extractor accessor type"), PCG::Private::MetadataTypes<ExpectedType>::Id, ExtractorAccessor->GetUnderlyingType());

		FPCGAttributeAccessorKeysSingleObjectPtr ObjectKey(&Struct);

		ExpectedType Value{};
		UTEST_TRUE(FString::Printf(TEXT("Getting a value for property %s and extractor %s"), *PropertyName.ToString(), *ExtractorName.ToString()), ExtractorAccessor->Get<ExpectedType>(Value, ObjectKey));
		UTEST_EQUAL(FString::Printf(TEXT("Value for property %s and extractor %s is equal to expected"), *PropertyName.ToString(), *ExtractorName.ToString()), Value, ExpectedValue);

		return true;
	};

	bool bTestPassed = true;

	// Rotators
	bTestPassed &= Verify(Struct.RotatorValue.Pitch, PCG_STRUCT_NAME_CHECK(RotatorValue), PCGAttributeExtractorConstants::RotatorPitch);
	bTestPassed &= Verify(Struct.RotatorValue.Yaw, PCG_STRUCT_NAME_CHECK(RotatorValue), PCGAttributeExtractorConstants::RotatorYaw);
	bTestPassed &= Verify(Struct.RotatorValue.Roll, PCG_STRUCT_NAME_CHECK(RotatorValue), PCGAttributeExtractorConstants::RotatorRoll);
	bTestPassed &= Verify(Struct.RotatorValue.Quaternion().GetForwardVector(), PCG_STRUCT_NAME_CHECK(RotatorValue), PCGAttributeExtractorConstants::RotatorForward);
	bTestPassed &= Verify(Struct.RotatorValue.Quaternion().GetRightVector(), PCG_STRUCT_NAME_CHECK(RotatorValue), PCGAttributeExtractorConstants::RotatorRight);
	bTestPassed &= Verify(Struct.RotatorValue.Quaternion().GetUpVector(), PCG_STRUCT_NAME_CHECK(RotatorValue), PCGAttributeExtractorConstants::RotatorUp);
	bTestPassed &= Verify(Struct.RotatorValue.Euler(), PCG_STRUCT_NAME_CHECK(RotatorValue), PCGAttributeExtractorConstants::RotatorEulerAngles);

	// Quaternions
	bTestPassed &= Verify(Struct.QuatValue.Rotator().Pitch, PCG_STRUCT_NAME_CHECK(QuatValue), PCGAttributeExtractorConstants::RotatorPitch);
	bTestPassed &= Verify(Struct.QuatValue.Rotator().Yaw, PCG_STRUCT_NAME_CHECK(QuatValue), PCGAttributeExtractorConstants::RotatorYaw);
	bTestPassed &= Verify(Struct.QuatValue.Rotator().Roll, PCG_STRUCT_NAME_CHECK(QuatValue), PCGAttributeExtractorConstants::RotatorRoll);
	bTestPassed &= Verify(Struct.QuatValue.GetForwardVector(), PCG_STRUCT_NAME_CHECK(QuatValue), PCGAttributeExtractorConstants::RotatorForward);
	bTestPassed &= Verify(Struct.QuatValue.GetRightVector(), PCG_STRUCT_NAME_CHECK(QuatValue), PCGAttributeExtractorConstants::RotatorRight);
	bTestPassed &= Verify(Struct.QuatValue.GetUpVector(), PCG_STRUCT_NAME_CHECK(QuatValue), PCGAttributeExtractorConstants::RotatorUp);
	bTestPassed &= Verify(Struct.QuatValue.Euler(), PCG_STRUCT_NAME_CHECK(QuatValue), PCGAttributeExtractorConstants::RotatorEulerAngles);

	return bTestPassed;
}

bool FPCGAttributeTransformPropertyExtractorTest::RunTest(const FString & Parameters)
{
	FPCGMetadataTypesConstantStruct Struct = FPCGAttributeExtractorTestHelpers::CreateStruct();

	auto Verify = [this, &Struct](const auto& ExpectedValue, const FName PropertyName, const TArray<FName>& ExtractorNames) -> bool
	{
		using PropertyType = std::decay_t<decltype(ExpectedValue)>;

		TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(PropertyName, FPCGMetadataTypesConstantStruct::StaticStruct());
		UTEST_TRUE(FString::Printf(TEXT("Valid accessor for property %s"), *PropertyName.ToString()), Accessor.IsValid());

		bool bSuccess = false;
		TUniquePtr<IPCGAttributeAccessor> ExtractorAccessor = std::move(Accessor);
		for (const FName ExtractorName : ExtractorNames)
		{
			ExtractorAccessor = PCGAttributeAccessorHelpers::CreateChainAccessor(std::move(ExtractorAccessor), ExtractorName, bSuccess);
			UTEST_TRUE(FString::Printf(TEXT("Valid extractor accessor %s"), *ExtractorName.ToString()), (bSuccess && ExtractorAccessor.IsValid()));
		}

		UTEST_EQUAL("Extractror accessor type", PCG::Private::MetadataTypes<PropertyType>::Id, ExtractorAccessor->GetUnderlyingType());

		FPCGAttributeAccessorKeysSingleObjectPtr ObjectKey(&Struct);


		PropertyType Value{};
		FString ExtractorStr = FString::JoinBy(ExtractorNames, TEXT("."), [](const FName& Name) -> FString { return Name.ToString(); });
		UTEST_TRUE(FString::Printf(TEXT("Getting a value for property %s and extractor %s"), *PropertyName.ToString(), *ExtractorStr), ExtractorAccessor->Get<PropertyType>(Value, ObjectKey));
		UTEST_EQUAL(FString::Printf(TEXT("Value for property %s and extractor %s is equal to expected"), *PropertyName.ToString(), *ExtractorStr), Value, ExpectedValue);

		return true;
	};

	bool bTestPassed = true;

	bTestPassed &= Verify(Struct.TransformValue.GetLocation(), PCG_STRUCT_NAME_CHECK(TransformValue), { PCGAttributeExtractorConstants::TransformLocation });
	bTestPassed &= Verify(Struct.TransformValue.GetRotation(), PCG_STRUCT_NAME_CHECK(TransformValue), { PCGAttributeExtractorConstants::TransformRotation });
	bTestPassed &= Verify(Struct.TransformValue.GetScale3D(), PCG_STRUCT_NAME_CHECK(TransformValue), { PCGAttributeExtractorConstants::TransformScale });

	// Also testing double chain
	bTestPassed &= Verify(Struct.TransformValue.GetLocation().X, PCG_STRUCT_NAME_CHECK(TransformValue), { PCGAttributeExtractorConstants::TransformLocation, PCGAttributeExtractorConstants::VectorX });
	bTestPassed &= Verify(Struct.TransformValue.GetRotation().W, PCG_STRUCT_NAME_CHECK(TransformValue), { PCGAttributeExtractorConstants::TransformRotation, PCGAttributeExtractorConstants::VectorW });
	bTestPassed &= Verify(Struct.TransformValue.GetScale3D().Y, PCG_STRUCT_NAME_CHECK(TransformValue), { PCGAttributeExtractorConstants::TransformScale, PCGAttributeExtractorConstants::VectorY });

	return true;
}

bool FPCGAttributeStringPropertyExtractorTest::RunTest(const FString & Parameters)
{
	UPCGAttributeExtractorTestObject* Object = NewObject<UPCGAttributeExtractorTestObject>();
	FPCGMetadataTypesConstantStruct Struct = FPCGAttributeExtractorTestHelpers::CreateStruct(Object);

	auto Verify = [this, &Struct]<typename T>(const T& ExpectedValue, const FName PropertyName, const TArray<FName>& ExtractorNames) -> bool
	{
		using PropertyType = std::decay_t<T>;

		TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(PropertyName, FPCGMetadataTypesConstantStruct::StaticStruct());
		UTEST_TRUE(FString::Printf(TEXT("Valid accessor for property %s"), *PropertyName.ToString()), Accessor.IsValid());

		bool bSuccess = false;
		TUniquePtr<IPCGAttributeAccessor> ExtractorAccessor = std::move(Accessor);
		for (const FName ExtractorName : ExtractorNames)
		{
			ExtractorAccessor = PCGAttributeAccessorHelpers::CreateChainAccessor(std::move(ExtractorAccessor), ExtractorName, bSuccess);
			UTEST_TRUE(FString::Printf(TEXT("Valid extractor accessor %s"), *ExtractorName.ToString()), (bSuccess && ExtractorAccessor.IsValid()));
		}

		UTEST_EQUAL("Extractor accessor type", PCG::Private::MetadataTypes<PropertyType>::Id, ExtractorAccessor->GetUnderlyingType());

		FPCGAttributeAccessorKeysSingleObjectPtr ObjectKey(&Struct);

		PropertyType Value{};
		FString ExtractorStr = FString::JoinBy(ExtractorNames, TEXT("."), [](const FName& Name) -> FString { return Name.ToString(); });
		UTEST_TRUE(FString::Printf(TEXT("Getting a value for property %s and extractor %s"), *PropertyName.ToString(), *ExtractorStr), ExtractorAccessor->Get<PropertyType>(Value, ObjectKey));
		UTEST_EQUAL(FString::Printf(TEXT("Value for property %s and extractor %s is equal to expected"), *PropertyName.ToString(), *ExtractorStr), Value, ExpectedValue);

		return true;
	};

	bool bTestPassed = true;

	// "Foo" = 3 characters
	bTestPassed &= Verify(3, PCG_STRUCT_NAME_CHECK(StringValue), {PCGAttributeExtractorConstants::StringLength});
	// "Bar" = 3 characters
	bTestPassed &= Verify(3, PCG_STRUCT_NAME_CHECK(NameValue), {PCGAttributeExtractorConstants::StringLength});
	auto String = FSoftObjectPath(Object).ToString();
	// "/Engine/Transient.PCGAttributeExtractorTestObject_1" = 51 characters
	bTestPassed &= Verify(51, PCG_STRUCT_NAME_CHECK(SoftObjectPathValue), {PCGAttributeExtractorConstants::StringLength});
	// "/Script/PCG.PCGMetadata" = 23 characters
	bTestPassed &= Verify(23, PCG_STRUCT_NAME_CHECK(SoftClassPathValue), {PCGAttributeExtractorConstants::StringLength});

	return bTestPassed;
}

bool FPCGAttributeInvalidPropertyExtractorTest::RunTest(const FString& Parameters)
{
	const FName InvalidName = FName(TEXT("Invalid"));
	TUniquePtr<IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(InvalidName, FPCGMetadataTypesConstantStruct::StaticStruct());
	UTEST_TRUE(TEXT("Invalid accessor with name \"Invalid\""), !Accessor.IsValid());

	auto VerifyInvalidChain = [this](const FName PropertyName, const TArray<FName>& ExtractorNames)
	{
		TUniquePtr<IPCGAttributeAccessor> ExtractorAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(PropertyName, FPCGMetadataTypesConstantStruct::StaticStruct());
		UTEST_TRUE(TEXT("Valid accessor"), ExtractorAccessor.IsValid());

		bool bSuccess = false;

		for (int32 i = 0; i < ExtractorNames.Num(); ++i)
		{
			ExtractorAccessor = PCGAttributeAccessorHelpers::CreateChainAccessor(std::move(ExtractorAccessor), ExtractorNames[i], bSuccess);
			// All accessors valid until the last one
			if (i != ExtractorNames.Num() - 1)
			{
				UTEST_TRUE(FString::Printf(TEXT("Valid extractor accessor %s"), *ExtractorNames[i].ToString()), (bSuccess && ExtractorAccessor.IsValid()));
			}
			else
			{
				// ExtractorAccessor should still be valid (moving back the previous accessor), but success should be false.
				UTEST_TRUE(FString::Printf(TEXT("Failed to created extractor accessor %s"), *ExtractorNames[i].ToString()), (!bSuccess && ExtractorAccessor.IsValid()));
			}
		}

		return true;
	};

	bool bTestPassed = true;

	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(DoubleValue), { InvalidName });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(Vector2Value), { InvalidName });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(VectorValue), { InvalidName });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(Vector4Value), { InvalidName });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(QuatValue), { InvalidName });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(RotatorValue), { InvalidName });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(TransformValue), { InvalidName });

	// Special case for Vectors
	// Invalid letters for this type
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(Vector2Value), { "XYZ" });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(Vector2Value), { "RGB" });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(VectorValue), { "XYZW" });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(VectorValue), { "RGBA" });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(Vector4Value), { "XYZP" });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(Vector4Value), { "RGBT" });
	// Too long
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(Vector4Value), { "YYYYY" });
	// Mixing RGBA and XYZW
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(Vector2Value), { "RX" });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(VectorValue), { "BBYA" });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(Vector4Value), { "BYZA" });

	// Yaw will work for quaternions but not vector 4
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(Vector4Value), { "Yaw" });

	// Invalid double chain
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(TransformValue), { PCGAttributeExtractorConstants::TransformLocation, InvalidName });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(TransformValue), { PCGAttributeExtractorConstants::TransformRotation, InvalidName });
	VerifyInvalidChain(PCG_STRUCT_NAME_CHECK(TransformValue), { PCGAttributeExtractorConstants::TransformScale, InvalidName });

	return bTestPassed;
}

bool FPCGAttributePropertyMultipleDepthTest::RunTest(const FString& Parameters)
{
	FPCGAttributeExtractorTestStruct TestStruct{};
	TestStruct.Object = NewObject<UPCGAttributeExtractorTestObject>();
	TestStruct.Object->DoubleValue = 0.5;
	TestStruct.Object->SetFlags(RF_Transient);

	TestStruct.DepthStruct.FloatValue = 0.324f;
	TestStruct.DepthStruct.Depth2Struct.IntValue = 5;

	const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(FPCGAttributeExtractorTestStruct::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FPCGAttributeExtractorTestStruct, Object)));
	const FProperty* DepthStructProperty = FPCGAttributeExtractorTestStruct::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FPCGAttributeExtractorTestStruct, DepthStruct));
	const FProperty* Depth2StructProperty = FPCGAttributeExtractorTestStructDepth1::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FPCGAttributeExtractorTestStructDepth1, Depth2Struct));

	UTEST_NOT_NULL(TEXT("Valid object property"), ObjectProperty);
	UTEST_NOT_NULL(TEXT("Valid depth struct property"), DepthStructProperty);
	UTEST_NOT_NULL(TEXT("Valid depth struct 2 property"), Depth2StructProperty);

	// Static analysis caution
	if (!ObjectProperty || !DepthStructProperty || !Depth2StructProperty)
	{
		return false;
	}

	TUniquePtr<IPCGAttributeAccessor> ObjectAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(ObjectProperty);
	TUniquePtr<IPCGAttributeAccessor> ObjectDoubleAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(GET_MEMBER_NAME_CHECKED(UPCGAttributeExtractorTestObject, DoubleValue), UPCGAttributeExtractorTestObject::StaticClass());
	TUniquePtr<IPCGAttributeAccessor> DepthStructFloatAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(GET_MEMBER_NAME_CHECKED(FPCGAttributeExtractorTestStructDepth1, FloatValue), FPCGAttributeExtractorTestStructDepth1::StaticStruct());
	TUniquePtr<IPCGAttributeAccessor> Depth2StructIntAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(GET_MEMBER_NAME_CHECKED(FPCGAttributeExtractorTestStructDepth2, IntValue), FPCGAttributeExtractorTestStructDepth2::StaticStruct());

	UTEST_TRUE(TEXT("Valid object accessor"), ObjectAccessor.IsValid());
	UTEST_TRUE(TEXT("Valid object double accessor"), ObjectDoubleAccessor.IsValid());
	UTEST_TRUE(TEXT("Valid depth struct float accessor"), DepthStructFloatAccessor.IsValid());
	UTEST_TRUE(TEXT("Valid deptch struct 2 int accessor"), Depth2StructIntAccessor.IsValid());

	// Then create different keys, manually for now
	// For object it is a bit more work
	const FObjectPtr& ObjectPtr = (FObjectPtr&)ObjectProperty->GetPropertyValue_InContainer(&TestStruct);
	const UPCGAttributeExtractorTestObject* ObjectDataPtr = Cast<UPCGAttributeExtractorTestObject>(ObjectPtr.Get());
	UTEST_NOT_NULL(TEXT("Object is not null"), ObjectDataPtr);

	// Static analysis precaution
	if (!ObjectDataPtr)
	{
		TestStruct.Object->MarkAsGarbage();
		return false;
	}

	FPCGAttributeAccessorKeysSingleObjectPtr ObjectKey(ObjectDataPtr);

	const FPCGAttributeExtractorTestStructDepth1* DepthStructPtr = DepthStructProperty->ContainerPtrToValuePtr<FPCGAttributeExtractorTestStructDepth1>(&TestStruct);

	// Static analysis precaution
	if (!DepthStructPtr)
	{
		TestStruct.Object->MarkAsGarbage();
		return false;
	}

	FPCGAttributeAccessorKeysSingleObjectPtr DepthStructKey(DepthStructPtr);
	FPCGAttributeAccessorKeysSingleObjectPtr DepthStruct2Key(Depth2StructProperty->ContainerPtrToValuePtr<FPCGAttributeExtractorTestStructDepth2>(DepthStructPtr));

	double DoubleValue = 1.2;
	int IntValue = 9;
	float FloatValue = 2.14f;

	UTEST_TRUE(TEXT("Get double value on object"), ObjectDoubleAccessor->Get<double>(DoubleValue, ObjectKey));

	// Allow constructible for both, as property wrappers will create double/int64 accessors
	UTEST_TRUE(TEXT("Get float value on depth struct"), DepthStructFloatAccessor->Get<float>(FloatValue, DepthStructKey, EPCGAttributeAccessorFlags::AllowConstructible));
	UTEST_TRUE(TEXT("Get int value on depth 2 struct"), Depth2StructIntAccessor->Get<int>(IntValue, DepthStruct2Key, EPCGAttributeAccessorFlags::AllowConstructible));

	UTEST_EQUAL(TEXT("Double value the same"), DoubleValue, TestStruct.Object->DoubleValue);
	UTEST_EQUAL(TEXT("Float value the same"), FloatValue, TestStruct.DepthStruct.FloatValue);
	UTEST_EQUAL(TEXT("Int value the same"), IntValue, TestStruct.DepthStruct.Depth2Struct.IntValue);

	TestStruct.Object->MarkAsGarbage();
	return true;
}

bool FPCGAttributePropertyMultipleDepthRangeTest::RunTest(const FString& Parameters)
{
	constexpr int NbItems = 5;
	TArray<FPCGAttributeExtractorTestStruct, TInlineAllocator<NbItems>> TestStructs{};
	TestStructs.SetNum(NbItems);

	for (int i = 0; i < NbItems; ++i)
	{
		TestStructs[i].DepthStruct.Depth2Struct.IntValue = i + 1;
	}

	const FProperty* DepthStructProperty = FPCGAttributeExtractorTestStruct::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FPCGAttributeExtractorTestStruct, DepthStruct));
	const FProperty* Depth2StructProperty = FPCGAttributeExtractorTestStructDepth1::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FPCGAttributeExtractorTestStructDepth1, Depth2Struct));

	UTEST_NOT_NULL(TEXT("Valid depth struct property"), DepthStructProperty);
	UTEST_NOT_NULL(TEXT("Valid depth struct 2 property"), Depth2StructProperty);

	// Static analysis caution
	if (!DepthStructProperty || !Depth2StructProperty)
	{
		return false;
	}

	TUniquePtr<IPCGAttributeAccessor> Depth2StructIntAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(GET_MEMBER_NAME_CHECKED(FPCGAttributeExtractorTestStructDepth2, IntValue), FPCGAttributeExtractorTestStructDepth2::StaticStruct());
	UTEST_TRUE(TEXT("Valid depth struct 2 int accessor"), Depth2StructIntAccessor.IsValid());

	// We will erase the type of struct along the way.
	// We explicitly know the types, but in a runtime case, we won't be able to know (not known at compile time).
	// So test this usecase here.
	TArray<const void*, TInlineAllocator<NbItems>> Depth2StructPtrs;
	Depth2StructPtrs.SetNum(NbItems);

	for (int i = 0; i < NbItems; ++i)
	{
		const void* DepthStructPtr = DepthStructProperty->ContainerPtrToValuePtr<void>(&TestStructs[i]);

		// Static analysis precaution
		if (!DepthStructPtr)
		{
			return false;
		}

		Depth2StructPtrs[i] = Depth2StructProperty->ContainerPtrToValuePtr<void>(DepthStructPtr);
	}

	FPCGAttributeAccessorKeysGenericPtrs GenericKeys(Depth2StructPtrs);

	TArray<int, TInlineAllocator<NbItems>> IntValues;
	IntValues.SetNum(NbItems);

	UTEST_TRUE(TEXT("Get int values on depth 2 struct"), Depth2StructIntAccessor->GetRange<int>(IntValues, 0, GenericKeys, EPCGAttributeAccessorFlags::AllowConstructible));
	for (int i = 0; i < NbItems; ++i)
	{
		UTEST_EQUAL(FString::Printf(TEXT("Int value #%d is the same"), i), IntValues[i], TestStructs[i].DepthStruct.Depth2Struct.IntValue);
	}

	return true;
}

#undef PCG_STRUCT_NAME_CHECK

#endif // WITH_EDITOR
