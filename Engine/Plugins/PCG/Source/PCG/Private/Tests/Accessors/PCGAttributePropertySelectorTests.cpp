// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCGCustomVersion.h"
#include "Tests/PCGTestsCommon.h"

#include "Metadata/PCGAttributePropertySelector.h"

#if WITH_EDITOR
class FPCGAttributePropertySelectorTests : public FPCGTestBaseClass
{
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	bool FromStringComparison(const FString& InString, const FPCGAttributePropertySelector& InSelector)
	{
		FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateSelectorFromString(InString);
		UTEST_EQUAL("Selectors are equal", Selector, InSelector);
		return true;
	}

	bool ImportExport(const FString& ExpectedExport, const FPCGAttributePropertySelector& InSelector)
	{
		FString Export;
		// Only first parameter is useful.
		InSelector.ExportTextItem(Export, FPCGAttributePropertySelector{}, nullptr, 0, nullptr);
		UTEST_EQUAL("Export texts are equal", Export, ExpectedExport);
		
		FPCGAttributePropertySelector ImportedSelector;
		const TCHAR* Buffer = *Export;
		ImportedSelector.ImportTextItem(Buffer, 0, nullptr, nullptr);
		UTEST_EQUAL("Selectors are equal", ImportedSelector, InSelector);
		const bool bFullBufferConsumed = reinterpret_cast<const void*>(Buffer) == reinterpret_cast<const void*>(*Export + Export.Len());
		UTEST_TRUE("Full buffer consumed", bFullBufferConsumed);
		
		return true;
	}

	/**
	* Test that the selector serialized with an old version, with its selection set to PointProperty
	* are deserializing correctly (after the PostSerialize)
	*/
	bool ValidateDeprecatedSelector(const FPCGAttributePropertySelector& ExpectedSelector, EPCGPointProperties PointProperty, TArray<FString> ExtraNames)
	{
		// To be removed when deprecated point property no longer exists.
		
		FPCGAttributePropertySelector DeprecatedSelector{};
		DeprecatedSelector.GetExtraNamesMutable() = std::move(ExtraNames);
		
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DeprecatedSelector.Selection = EPCGAttributePropertySelection::PointProperty;
		DeprecatedSelector.PointProperty_DEPRECATED = PointProperty;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FArchive DummyAr;
		DummyAr.UsingCustomVersion(FPCGCustomVersion::GUID);
		DummyAr.SetCustomVersion(FPCGCustomVersion::GUID, static_cast<int32>(FPCGCustomVersion::AttributePropertySelectorDeprecatePointProperties) - 1, FName("Dummy"));
		DummyAr.SetIsLoading(true);
		DummyAr.SetIsPersistent(true);
		DeprecatedSelector.PostSerialize(DummyAr);
		
		UTEST_EQUAL("Selectors are equal", DeprecatedSelector, ExpectedSelector);
		return true;
	}

	/**
	* Test that the selector serialized with an old version, and set in the ctor as a point property (like in some PCGSettings constructors)
	* are deserializing correctly (after the PostSerialize)
	*/
	bool ValidateDeprecatedSelectorCDO(const FPCGAttributePropertySelector& ExpectedSelector, EPCGPointProperties PointProperty, TArray<FString> ExtraNames)
	{
		// To be removed when deprecated point property no longer exists.
		// Version with CDO where the default object call SetPointProperty.
		FPCGAttributePropertySelector DeprecatedSelector = FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Density, NAME_None, ExtraNames);
		
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DeprecatedSelector.PointProperty_DEPRECATED = PointProperty;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FArchive DummyAr;
		DummyAr.UsingCustomVersion(FPCGCustomVersion::GUID);
		DummyAr.SetCustomVersion(FPCGCustomVersion::GUID, static_cast<int32>(FPCGCustomVersion::AttributePropertySelectorDeprecatePointProperties) - 1, FName("Dummy"));
		DummyAr.SetIsLoading(true);
		DummyAr.SetIsPersistent(true);
		DeprecatedSelector.PostSerialize(DummyAr);
		
		UTEST_EQUAL("Selectors are equal", DeprecatedSelector, ExpectedSelector);
		return true;
	}

	/**
	* Test that the new selector serialized with the new version, and set in the ctor as a point property (like in some PCGSettings constructors)
	* are deserializing correctly (after the PostSerialize)
	*/
	bool ValidateSelectorCDOWithNewerVersion(const FPCGAttributePropertySelector& ExpectedSelector, EPCGPointProperties PointProperty, TArray<FString> ExtraNames)
	{
		// To be removed when deprecated point property no longer exists.
		// Version with CDO where the default object call SetPointProperty.
		FPCGAttributePropertySelector DeprecatedSelector = FPCGAttributePropertySelector::CreatePointPropertySelector(PointProperty, NAME_None, ExtraNames);

		FArchive DummyAr;
		DummyAr.UsingCustomVersion(FPCGCustomVersion::GUID);
		DummyAr.SetCustomVersion(FPCGCustomVersion::GUID, FPCGCustomVersion::AttributePropertySelectorDeprecatePointProperties, FName("Dummy"));
		DummyAr.SetIsLoading(true);
		DummyAr.SetIsPersistent(true);
		DeprecatedSelector.PostSerialize(DummyAr);
		
		UTEST_EQUAL("Selectors are equal", DeprecatedSelector, ExpectedSelector);
		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_Attribute, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.Attribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_Property, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.Property", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_ExtraProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.ExtraProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_AtLast, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.@Last", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_AtSource, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.@Source", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithDomain_Attribute, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithDomain.Attribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithDomain_Property, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithDomain.Property", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithDomain_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithDomain.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithDomain_ExtraProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithDomain.ExtraProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithDomain_AtLast, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithDomain.@Last", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithDomain_AtSource, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithDomain.@Source", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_Attribute, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.Attribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_Property, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.Property", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_ExtraProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.ExtraProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_AtLast, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.@Last", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_AtSource, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.@Source", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_Attribute, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.WithDomain.Attribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_Property, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.WithDomain.Property", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.WithDomain.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_ExtraProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.WithDomain.ExtraProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_AtLast, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.WithDomain.@Last", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_AtSource, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.FromString.WithExtraNames.WithDomain.@Source", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_Attribute, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.Attribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_Property, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.Property", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_ExtraProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.ExtraProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_AtLast, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.@Last", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_AtSource, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.@Source", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithDomain_Attribute, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithDomain.Attribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithDomain_Property, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithDomain.Property", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithDomain_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithDomain.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithDomain_ExtraProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithDomain.ExtraProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithExtraNames_Attribute, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithExtraNames.Attribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithExtraNames_Property, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithExtraNames.Property", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithExtraNames_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithExtraNames.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithExtraNames_ExtraProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithExtraNames.ExtraProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithExtraNames_WithDomain_Attribute, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithExtraNames.WithDomain.Attribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithExtraNames_WithDomain_Property, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithExtraNames.WithDomain.Property", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithExtraNames_WithDomain_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithExtraNames.WithDomain.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_ExportImport_WithExtraNames_WithDomain_ExtraProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.ExportImport.WithExtraNames.WithDomain.ExtraProperty", PCGTestsCommon::TestFlags)

// To be removed when deprecated point property no longer exists.
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_Deprecation_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.Deprecation.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_Deprecation_WithExtraNames_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.Deprecation.WithExtraNames.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_Deprecation_CDO_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.Deprecation.CDO.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_Deprecation_CDO_WithExtraNames_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.Deprecation.CDO.WithExtraNames.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_Deprecation_CDO_NewVersion_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.Deprecation.CDO.NewVersion.PointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_Deprecation_CDO_NewVersion_WithExtraNames_PointProperty, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.Deprecation.CDO.NewVersion.WithExtraNames.PointProperty", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_Invalid_JustAt, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.Invalid.JustAt", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_Invalid_InvalidSymbols, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.Invalid.InvalidSymbols", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_JustDomainName, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.JustDomainName", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributePropertySelector_UpdateDomainNameToAtLast, FPCGAttributePropertySelectorTests, "Plugins.PCG.AttributePropertySelector.UpdateDomainNameToAtLast", PCGTestsCommon::TestFlags)

bool FPCGAttributePropertySelector_FromString_Attribute::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("MyAttr"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("MyAttr")));
}

bool FPCGAttributePropertySelector_FromString_Property::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("$MyProperty"), FPCGAttributePropertySelector::CreatePropertySelector(TEXT("MyProperty")));
}

bool FPCGAttributePropertySelector_FromString_PointProperty::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("$Rotation"), FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Rotation));
}

bool FPCGAttributePropertySelector_FromString_ExtraProperty::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("$Index"), FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index));
}

bool FPCGAttributePropertySelector_FromString_AtLast::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@Last"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("@Last")));
}

bool FPCGAttributePropertySelector_FromString_AtSource::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@Source"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("@Source")));
}

bool FPCGAttributePropertySelector_FromString_WithDomain_Attribute::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.MyAttr"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("MyAttr"), TEXT("NewDomain")));
}

bool FPCGAttributePropertySelector_FromString_WithDomain_Property::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.$MyProperty"), FPCGAttributePropertySelector::CreatePropertySelector(TEXT("MyProperty"), TEXT("NewDomain")));
}

bool FPCGAttributePropertySelector_FromString_WithDomain_PointProperty::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.$Scale"), FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Scale, TEXT("NewDomain")));
}

bool FPCGAttributePropertySelector_FromString_WithDomain_ExtraProperty::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.$Index"), FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index, TEXT("NewDomain")));
}

bool FPCGAttributePropertySelector_FromString_WithDomain_AtLast::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.@Last"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("@Last"), TEXT("NewDomain")));
}

bool FPCGAttributePropertySelector_FromString_WithDomain_AtSource::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.@Source"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("@Source"), TEXT("NewDomain")));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_Attribute::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("MyAttr.X"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("MyAttr"), NAME_None, {TEXT("X")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_Property::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("$MyProperty.XY"), FPCGAttributePropertySelector::CreatePropertySelector(TEXT("MyProperty"), NAME_None, {TEXT("XY")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_PointProperty::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("$Density.Length"), FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Density, NAME_None, {TEXT("Length")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_ExtraProperty::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("$Index.Abs"), FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index, NAME_None, {TEXT("Abs")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_AtLast::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@Last.X"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("@Last"), NAME_None, {TEXT("X")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_AtSource::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@Source.X"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("@Source"), NAME_None, {TEXT("X")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_Attribute::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.MyAttr.Position.XY"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("MyAttr"), TEXT("NewDomain"), {TEXT("Position"), TEXT("XY")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_Property::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.$MyProperty.RGBA"), FPCGAttributePropertySelector::CreatePropertySelector(TEXT("MyProperty"), TEXT("NewDomain"), {TEXT("RGBA")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_PointProperty::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.$Steepness.Sign"), FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Steepness, TEXT("NewDomain"), {TEXT("Sign")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_ExtraProperty::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.$Index.Sign"), FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index, TEXT("NewDomain"), {TEXT("Sign")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_AtLast::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.@Last.Position.XY"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("@Last"), TEXT("NewDomain"), {TEXT("Position"), TEXT("XY")}));
}

bool FPCGAttributePropertySelector_FromString_WithExtraNames_WithDomain_AtSource::RunTest(const FString& Parameters)
{
	return FromStringComparison(TEXT("@NewDomain.@Source.Position.XY"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("@Source"), TEXT("NewDomain"), {TEXT("Position"), TEXT("XY")}));
}

bool FPCGAttributePropertySelector_ExportImport_Attribute::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(MyAttr)PCGEnd"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("MyAttr")));
}

bool FPCGAttributePropertySelector_ExportImport_Property::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin($MyProperty)PCGEnd"), FPCGAttributePropertySelector::CreatePropertySelector(TEXT("MyProperty")));
}

bool FPCGAttributePropertySelector_ExportImport_PointProperty::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin($Rotation)PCGEnd"), FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Rotation));
}

bool FPCGAttributePropertySelector_ExportImport_ExtraProperty::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin($Index)PCGEnd"), FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index));
}

bool FPCGAttributePropertySelector_ExportImport_AtLast::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(@Last)PCGEnd"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("@Last")));
}

bool FPCGAttributePropertySelector_ExportImport_AtSource::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(@Source)PCGEnd"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("@Source")));
}

bool FPCGAttributePropertySelector_ExportImport_WithDomain_Attribute::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(@NewDomain.MyAttr)PCGEnd"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("MyAttr"), TEXT("NewDomain")));
}

bool FPCGAttributePropertySelector_ExportImport_WithDomain_Property::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(@NewDomain.$MyProperty)PCGEnd"), FPCGAttributePropertySelector::CreatePropertySelector(TEXT("MyProperty"), TEXT("NewDomain")));
}

bool FPCGAttributePropertySelector_ExportImport_WithDomain_PointProperty::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(@NewDomain.$Scale)PCGEnd"), FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Scale, TEXT("NewDomain")));
}

bool FPCGAttributePropertySelector_ExportImport_WithDomain_ExtraProperty::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(@NewDomain.$Index)PCGEnd"), FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index, TEXT("NewDomain")));
}

bool FPCGAttributePropertySelector_ExportImport_WithExtraNames_Attribute::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(MyAttr.X)PCGEnd"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("MyAttr"), NAME_None, {TEXT("X")}));
}

bool FPCGAttributePropertySelector_ExportImport_WithExtraNames_Property::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin($MyProperty.XY)PCGEnd"), FPCGAttributePropertySelector::CreatePropertySelector(TEXT("MyProperty"), NAME_None, {TEXT("XY")}));
}

bool FPCGAttributePropertySelector_ExportImport_WithExtraNames_PointProperty::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin($Density.Length)PCGEnd"), FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Density, NAME_None, {TEXT("Length")}));
}

bool FPCGAttributePropertySelector_ExportImport_WithExtraNames_ExtraProperty::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin($Index.Abs)PCGEnd"), FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index, NAME_None, {TEXT("Abs")}));
}

bool FPCGAttributePropertySelector_ExportImport_WithExtraNames_WithDomain_Attribute::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(@NewDomain.MyAttr.Position.XY)PCGEnd"), FPCGAttributePropertySelector::CreateAttributeSelector(TEXT("MyAttr"), TEXT("NewDomain"), {TEXT("Position"), TEXT("XY")}));
}

bool FPCGAttributePropertySelector_ExportImport_WithExtraNames_WithDomain_Property::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(@NewDomain.$MyProperty.RGBA)PCGEnd"), FPCGAttributePropertySelector::CreatePropertySelector(TEXT("MyProperty"), TEXT("NewDomain"), {TEXT("RGBA")}));
}

bool FPCGAttributePropertySelector_ExportImport_WithExtraNames_WithDomain_PointProperty::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(@NewDomain.$Steepness.Sign)PCGEnd"), FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Steepness, TEXT("NewDomain"), {TEXT("Sign")}));
}

bool FPCGAttributePropertySelector_ExportImport_WithExtraNames_WithDomain_ExtraProperty::RunTest(const FString& Parameters)
{
	return ImportExport(TEXT("PCGBegin(@NewDomain.$Index.Sign)PCGEnd"), FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index, TEXT("NewDomain"), {TEXT("Sign")}));
}

bool FPCGAttributePropertySelector_Deprecation_PointProperty::RunTest(const FString& Parameters)
{
	// To be removed when deprecated point property no longer exists.
	return ValidateDeprecatedSelector(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Steepness), EPCGPointProperties::Steepness, {});
}

bool FPCGAttributePropertySelector_Deprecation_WithExtraNames_PointProperty::RunTest(const FString& Parameters)
{
	// To be removed when deprecated point property no longer exists.
	return ValidateDeprecatedSelector(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Rotation, NAME_None, {TEXT("Forward")}), EPCGPointProperties::Rotation, {TEXT("Forward")});
}

bool FPCGAttributePropertySelector_Deprecation_CDO_PointProperty::RunTest(const FString& Parameters)
{
	// To be removed when deprecated point property no longer exists.
	return ValidateDeprecatedSelectorCDO(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Steepness), EPCGPointProperties::Steepness, {});
}

bool FPCGAttributePropertySelector_Deprecation_CDO_WithExtraNames_PointProperty::RunTest(const FString& Parameters)
{
	// To be removed when deprecated point property no longer exists.
	return ValidateDeprecatedSelectorCDO(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Rotation, NAME_None, {TEXT("Forward")}), EPCGPointProperties::Rotation, {TEXT("Forward")});
}

bool FPCGAttributePropertySelector_Deprecation_CDO_NewVersion_PointProperty::RunTest(const FString& Parameters)
{
	// To be removed when deprecated point property no longer exists.
	return ValidateSelectorCDOWithNewerVersion(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Steepness), EPCGPointProperties::Steepness, {});
}

bool FPCGAttributePropertySelector_Deprecation_CDO_NewVersion_WithExtraNames_PointProperty::RunTest(const FString& Parameters)
{
	// To be removed when deprecated point property no longer exists.
	return ValidateSelectorCDOWithNewerVersion(FPCGAttributePropertySelector::CreatePointPropertySelector(EPCGPointProperties::Rotation, NAME_None, {TEXT("Forward")}), EPCGPointProperties::Rotation, {TEXT("Forward")});
}

bool FPCGAttributePropertySelector_Invalid_JustAt::RunTest(const FString& Parameters)
{
	FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateSelectorFromString(TEXT("@"));
	UTEST_EQUAL("Selector has no domain name", Selector.GetDomainName(), FName{NAME_None});
	UTEST_EQUAL("Selector is an attribute", Selector.GetSelection(), EPCGAttributePropertySelection::Attribute);
	UTEST_FALSE("Selector is invalid", Selector.IsValid());
	return true;
}

bool FPCGAttributePropertySelector_Invalid_InvalidSymbols::RunTest(const FString& Parameters)
{
	FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateSelectorFromString(TEXT(")(*&^"));
	UTEST_EQUAL("Selector has no domain name", Selector.GetDomainName(), FName{NAME_None});
	UTEST_EQUAL("Selector is an attribute", Selector.GetSelection(), EPCGAttributePropertySelection::Attribute);
	UTEST_FALSE("Selector is invalid", Selector.IsValid());
	return true;
}

bool FPCGAttributePropertySelector_JustDomainName::RunTest(const FString& Parameters)
{
	FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateSelectorFromString(TEXT("@MyDomain"));
	UTEST_EQUAL("Selector has the right domain name", Selector.GetDomainName(), FName{TEXT("MyDomain")});
	UTEST_EQUAL("Selector is an attribute", Selector.GetSelection(), EPCGAttributePropertySelection::Attribute);
	UTEST_EQUAL("Selector has an attribute name of None", Selector.GetAttributeName(), FName{NAME_None});
	return true;
}

bool FPCGAttributePropertySelector_UpdateDomainNameToAtLast::RunTest(const FString& Parameters)
{
	FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateSelectorFromString(TEXT("@Las"));
	Selector.Update(TEXT("@Last"));
	UTEST_EQUAL("Selector has no domain name", Selector.GetDomainName(), FName{NAME_None});
	UTEST_EQUAL("Selector is an attribute", Selector.GetSelection(), EPCGAttributePropertySelection::Attribute);
	UTEST_EQUAL("Selector has an attribute name of @Last", Selector.GetAttributeName(), FName{TEXT("@Last")});
	return true;
}

#endif // WITH_EDITOR
