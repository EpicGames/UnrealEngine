// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "Data/PCGSplineData.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataDomain.h"
#include "Metadata/PCGMetadataCommon.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplineAccessorControlPointsPropertyTest, FPCGTestBaseClass, "Plugins.PCG.Accessor.Splines.ControlPointProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplineAccessorControlPointsMetadataTest, FPCGTestBaseClass, "Plugins.PCG.Accessor.Splines.ControlPointMetadata", PCGTestsCommon::TestFlags)

bool FPCGSplineAccessorControlPointsPropertyTest::RunTest(const FString& Parameters)
{
	UPCGSplineData* SplineData = NewObject<UPCGSplineData>();

	constexpr int NumPoints = 10;
	TArray<FSplinePoint> SplinePoints;
	SplinePoints.Reserve(NumPoints);
	for (int i = 0; i < NumPoints; ++i)
	{
		SplinePoints.Emplace(
			/*InputKey=*/ static_cast<float>(i),
			/*InPosition=*/ FVector(i, i, i),
			/*InArriveTangent=*/ FVector(2*i, 2*i,2*i),
			/*InLeaveTangent=*/ FVector(3*i, 3*i, 3*i),
			/*InRotation=*/ FRotator(i, i, i),
			/*InScale=*/ FVector(i, i, i),
			/*InType=*/ESplinePointType::CurveCustomTangent);
	}

	SplineData->Initialize(SplinePoints, /*bInClosedLoop*/ false, FTransform::Identity);

	// Index needs to be const
	TUniquePtr<const IPCGAttributeAccessor> IndexAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(SplineData, FPCGAttributePropertySelector::CreateExtraPropertySelector(EPCGExtraProperties::Index));
	TUniquePtr<IPCGAttributeAccessor> PositionAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SplineData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("Position")));
	TUniquePtr<IPCGAttributeAccessor> ArriveTangentAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SplineData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("ArriveTangent")));
	TUniquePtr<IPCGAttributeAccessor> LeaveTangentAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SplineData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("LeaveTangent")));
	TUniquePtr<IPCGAttributeAccessor> RotationAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SplineData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("Rotation")));
	TUniquePtr<IPCGAttributeAccessor> ScaleAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SplineData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("Scale")));
	TUniquePtr<IPCGAttributeAccessor> InterpTypeAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SplineData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("InterpType")));

	// Keys can be used for any property, so only create one on the position.
	TUniquePtr<IPCGAttributeAccessorKeys> SplineKeys = PCGAttributeAccessorHelpers::CreateKeys(SplineData, FPCGAttributePropertySelector::CreatePropertySelector(TEXT("Position")));

	UTEST_TRUE("Index accessor is valid", IndexAccessor.IsValid());
	UTEST_TRUE("Position accessor is valid", PositionAccessor.IsValid());
	UTEST_TRUE("ArriveTangent accessor is valid", ArriveTangentAccessor.IsValid());
	UTEST_TRUE("LeaveTangent accessor is valid", LeaveTangentAccessor.IsValid());
	UTEST_TRUE("Rotation accessor is valid", RotationAccessor.IsValid());
	UTEST_TRUE("Scale accessor is valid", ScaleAccessor.IsValid());
	UTEST_TRUE("InterpType accessor is valid", InterpTypeAccessor.IsValid());
	
	UTEST_TRUE("Keys are valid", SplineKeys.IsValid());

	UTEST_EQUAL("Number of keys", SplineKeys->GetNum(), NumPoints);

	// Modify all the points by adding 1 to each numerical value
	auto ReadModifySetAndValidate = [this, NumPoints, &SplineKeys]<typename T>(TUniquePtr<IPCGAttributeAccessor>& Accessor, const T& ValueToAdd, const TCHAR* What) -> bool
	{
		TArray<T> Values;
		TArray<T> ModifiedValues;
		Values.SetNum(NumPoints);
		ModifiedValues.Reserve(NumPoints);
		
		UTEST_TRUE(FString::Printf(TEXT("GetRange on '%s' succeeded."), What), Accessor->GetRange<T>(Values, 0, *SplineKeys));
		
		for (T& Value : Values)
		{
			ModifiedValues.Emplace(Value + ValueToAdd);
		}
		
		UTEST_TRUE(FString::Printf(TEXT("SetRange on '%s' succeeded."), What), Accessor->SetRange<T>(ModifiedValues, 0, *SplineKeys));

		TArray<T> NewValues;
		NewValues.SetNum(NumPoints);
		UTEST_TRUE("Second GetRange succeeded.", Accessor->GetRange<T>(NewValues, 0, *SplineKeys));
		
		for (int i = 0; i < NumPoints; ++i)
		{
			UTEST_NOT_EQUAL(FString::Printf(TEXT("Value for '%s' at index '%d' was is different from the first get"), What, i), NewValues[i], Values[i]);
			UTEST_EQUAL(FString::Printf(TEXT("Value for '%s' at index '%d' was set correctly"), What, i), NewValues[i], ModifiedValues[i]);
		}

		return true;
	};

	ReadModifySetAndValidate(PositionAccessor, FVector::OneVector, TEXT("Position"));
	ReadModifySetAndValidate(ArriveTangentAccessor, FVector::OneVector, TEXT("ArriveTangent"));
	ReadModifySetAndValidate(LeaveTangentAccessor, FVector::OneVector, TEXT("LeaveTangent"));
	ReadModifySetAndValidate(RotationAccessor, FQuat::MakeFromEuler(FVector::OneVector), TEXT("Rotation"));
	ReadModifySetAndValidate(ScaleAccessor, FVector::OneVector, TEXT("Scale"));

	// Also validate that the index and interp type are also OK
	TArray<int32> IndexValues;
	TArray<int64> InterpTypeValues;
	IndexValues.SetNum(NumPoints);
	InterpTypeValues.SetNum(NumPoints);

	UTEST_TRUE("GetRange on Index succeeded", IndexAccessor->GetRange<int32>(IndexValues, 0, *SplineKeys));
	UTEST_TRUE("GetRange on InterpTypes succeeded", InterpTypeAccessor->GetRange<int64>(InterpTypeValues, 0, *SplineKeys));

	for (int i = 0; i < NumPoints; ++i)
	{
		UTEST_EQUAL("Index", IndexValues[i], i);
		UTEST_EQUAL("InterpType", InterpTypeValues[i], static_cast<int64>(ESplinePointType::CurveCustomTangent));
	}
	
	return true;
}

bool FPCGSplineAccessorControlPointsMetadataTest::RunTest(const FString& Parameters)
{
	UPCGSplineData* SplineData = NewObject<UPCGSplineData>();

	constexpr int NumPoints = 10;
	TArray<FSplinePoint> SplinePoints;
	SplinePoints.Reserve(NumPoints);
	for (int i = 0; i < NumPoints; ++i)
	{
		SplinePoints.Emplace(/*InputKey=*/ static_cast<float>(i), FVector::ZeroVector);
	}

	SplineData->Initialize(SplinePoints, /*bInClosedLoop*/ false, FTransform::Identity);

	UPCGMetadata* Metadata = SplineData->MutableMetadata();
	FPCGMetadataDomain* MetadataDomain = Metadata ? Metadata->GetMetadataDomain(PCGMetadataDomainID::Elements) : nullptr;

	UTEST_NOT_NULL("Metadata domain for control points exist", MetadataDomain);
	check(MetadataDomain);

	const FName IntAttributeName = TEXT("Int");
	FPCGMetadataAttribute<int32>* IntAttribute = MetadataDomain->CreateAttribute<int32>(IntAttributeName, -1, false, false);
	UTEST_NOT_NULL("Attribute was successfully created", IntAttribute);

	const FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(IntAttributeName, TEXT("ControlPoints"));
	TUniquePtr<IPCGAttributeAccessor> IntAccessor = PCGAttributeAccessorHelpers::CreateAccessor(SplineData, Selector);
	TUniquePtr<IPCGAttributeAccessorKeys> SplineKeys = PCGAttributeAccessorHelpers::CreateKeys(SplineData, Selector);

	UTEST_TRUE("Int accessor is valid", IntAccessor.IsValid());
	UTEST_TRUE("Keys are valid", SplineKeys.IsValid());
	UTEST_EQUAL("Number of keys", SplineKeys->GetNum(), NumPoints);

	TArray<int32> Values;
	Values.SetNum(NumPoints);
	UTEST_TRUE("GetRange succeeded.", IntAccessor->GetRange<int32>(Values, 0, *SplineKeys));
	UTEST_TRUE("All have the default value", Algo::AllOf(Values, [](int32 Value) { return Value == -1; }));

	for (int i = 0; i < NumPoints; ++i)
	{
		Values[i] = i;
	}

	UTEST_TRUE("SetRange succeeded.", IntAccessor->SetRange<int32>(Values, 0, *SplineKeys));
	TArray<int32> NewValues;
	NewValues.SetNum(NumPoints);

	UTEST_TRUE("Second GetRange succeeded.", IntAccessor->GetRange<int32>(NewValues, 0, *SplineKeys));
	for (int i = 0; i < NumPoints; ++i)
	{
		UTEST_EQUAL(FString::Printf(TEXT("Value at index '%d' was set correctly"), i), NewValues[i], i);
	}
	
	return true;
}