// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "Grid/PCGLandscapeCache.h"

#if WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGLandscapeCache_CalcSafeIndices, FPCGTestBaseClass, "Plugins.PCG.LandscapeCache.CalcSafeIndices", PCGTestsCommon::TestFlags)

bool FPCGLandscapeCache_CalcSafeIndices::RunTest(const FString& Parameters)
{
	PCGLandscapeCache::FSafeIndices TopLeft = PCGLandscapeCache::CalcSafeIndices(FVector2D(0.0), 32);

	UTEST_EQUAL("TopLeft.X0Y0", TopLeft.X0Y0, 0 + 0*32);
	UTEST_EQUAL("TopLeft.X1Y0", TopLeft.X1Y0, 1 + 0*32);
	UTEST_EQUAL("TopLeft.X0Y1", TopLeft.X0Y1, 0 + 1*32);
	UTEST_EQUAL("TopLeft.X1Y1", TopLeft.X1Y1, 1 + 1*32);
	UTEST_EQUAL("TopLeft.XFraction", TopLeft.XFraction, 0.0f);
	UTEST_EQUAL("TopLeft.YFraction", TopLeft.YFraction, 0.0f);

	PCGLandscapeCache::FSafeIndices ClampTopLeft = PCGLandscapeCache::CalcSafeIndices(FVector2D(-0.5), 32);

	UTEST_EQUAL("ClampTopLeft.X0Y0", ClampTopLeft.X0Y0, 0 + 0*32);
	UTEST_EQUAL("ClampTopLeft.X1Y0", ClampTopLeft.X1Y0, 1 + 0*32);
	UTEST_EQUAL("ClampTopLeft.X0Y1", ClampTopLeft.X0Y1, 0 + 1*32);
	UTEST_EQUAL("ClampTopLeft.X1Y1", ClampTopLeft.X1Y1, 1 + 1*32);
	UTEST_EQUAL("ClampTopLeft.XFraction", ClampTopLeft.XFraction, 0.0f);
	UTEST_EQUAL("ClampTopLeft.YFraction", ClampTopLeft.YFraction, 0.0f);

	PCGLandscapeCache::FSafeIndices FractionTopLeft = PCGLandscapeCache::CalcSafeIndices(FVector2D(0.5, 0.4), 32);

	UTEST_EQUAL("FractionTopLeft.X0Y0", FractionTopLeft.X0Y0, 0 + 0*32);
	UTEST_EQUAL("FractionTopLeft.X1Y0", FractionTopLeft.X1Y0, 1 + 0*32);
	UTEST_EQUAL("FractionTopLeft.X0Y1", FractionTopLeft.X0Y1, 0 + 1*32);
	UTEST_EQUAL("FractionTopLeft.X1Y1", FractionTopLeft.X1Y1, 1 + 1*32);
	UTEST_EQUAL("FractionTopLeft.XFraction", FractionTopLeft.XFraction, 0.5f);
	UTEST_EQUAL("FractionTopLeft.YFraction", FractionTopLeft.YFraction, 0.4f);

	PCGLandscapeCache::FSafeIndices MiddleFraction = PCGLandscapeCache::CalcSafeIndices(FVector2D(4.5, 10.4), 32);

	UTEST_EQUAL("MiddleFraction.X0Y0", MiddleFraction.X0Y0, 4 + 10*32);
	UTEST_EQUAL("MiddleFraction.X1Y0", MiddleFraction.X1Y0, 5 + 10*32);
	UTEST_EQUAL("MiddleFraction.X0Y1", MiddleFraction.X0Y1, 4 + 11*32);
	UTEST_EQUAL("MiddleFraction.X1Y1", MiddleFraction.X1Y1, 5 + 11*32);
	UTEST_EQUAL("MiddleFraction.XFraction", MiddleFraction.XFraction, 0.5f);
	UTEST_EQUAL("MiddleFraction.YFraction", MiddleFraction.YFraction, 0.4f);

	PCGLandscapeCache::FSafeIndices BottomRight = PCGLandscapeCache::CalcSafeIndices(FVector2D(31.0), 32);

	UTEST_EQUAL("BottomRight.X0Y0", BottomRight.X0Y0, 32*32-1);
	UTEST_EQUAL("BottomRight.X1Y0", BottomRight.X1Y0, 32*32-1);
	UTEST_EQUAL("BottomRight.X0Y1", BottomRight.X0Y1, 32*32-1);
	UTEST_EQUAL("BottomRight.X1Y1", BottomRight.X1Y1, 32*32-1);
	UTEST_EQUAL("BottomRight.XFraction", BottomRight.XFraction, 0.0f);
	UTEST_EQUAL("BottomRight.YFraction", BottomRight.YFraction, 0.0f);

	PCGLandscapeCache::FSafeIndices ClampBottomRight = PCGLandscapeCache::CalcSafeIndices(FVector2D(33.5), 32);

	UTEST_EQUAL("ClampBottomRight.X0Y0", ClampBottomRight.X0Y0, 32*32-1);
	UTEST_EQUAL("ClampBottomRight.X1Y0", ClampBottomRight.X1Y0, 32*32-1);
	UTEST_EQUAL("ClampBottomRight.X0Y1", ClampBottomRight.X0Y1, 32*32-1);
	UTEST_EQUAL("ClampBottomRight.X1Y1", ClampBottomRight.X1Y1, 32*32-1);
	UTEST_EQUAL("ClampBottomRight.XFraction", ClampBottomRight.XFraction, 0.0f);
	UTEST_EQUAL("ClampBottomRight.YFraction", ClampBottomRight.YFraction, 0.0f);

	PCGLandscapeCache::FSafeIndices FractionBottomRight = PCGLandscapeCache::CalcSafeIndices(FVector2D(30.4, 30.5), 32);

	UTEST_EQUAL("FractionBottomRight.X0Y0", FractionBottomRight.X0Y0, 30 + 30*32);
	UTEST_EQUAL("FractionBottomRight.X1Y0", FractionBottomRight.X1Y0, 31 + 30*32);
	UTEST_EQUAL("FractionBottomRight.X0Y1", FractionBottomRight.X0Y1, 30 + 31*32);
	UTEST_EQUAL("FractionBottomRight.X1Y1", FractionBottomRight.X1Y1, 31 + 31*32);
	UTEST_EQUAL("FractionBottomRight.XFraction", FractionBottomRight.XFraction, 0.4f);
	UTEST_EQUAL("FractionBottomRight.YFraction", FractionBottomRight.YFraction, 0.5f);

	return true;
}
#endif
