// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"
#include "Compute/PCGComputeCommon.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGSanitizePinLabelForHLSLTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.Common.SanitizePinLabel",
	PCGTestsCommon::TestFlags)

bool FPCGSanitizePinLabelForHLSLTest::RunTest(const FString& Parameters)
{
	// Valid identifiers pass through unchanged.
	UTEST_EQUAL("Simple alpha", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("Out"))), FString(TEXT("Out")));
	UTEST_EQUAL("AlphaNumeric", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("Pin1"))), FString(TEXT("Pin1")));
	UTEST_EQUAL("Underscore", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("My_Pin"))), FString(TEXT("My_Pin")));
	UTEST_EQUAL("Long valid name", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("Output_Data_3"))), FString(TEXT("Output_Data_3")));
	UTEST_EQUAL("NAME_None", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(NAME_None)), FString(TEXT("None")));

	// Spaces replaced with underscores.
	UTEST_EQUAL("Single space", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("End Points"))), FString(TEXT("End_Points")));
	UTEST_EQUAL("Multiple spaces", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("My Pin Label"))), FString(TEXT("My_Pin_Label")));

	// Special characters replaced with underscores.
	UTEST_EQUAL("Dot", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("My.Pin"))), FString(TEXT("My_Pin")));
	UTEST_EQUAL("Hyphen", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("My-Pin"))), FString(TEXT("My_Pin")));
	UTEST_EQUAL("Mixed special chars", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("A!B@C"))), FString(TEXT("A_B_C")));

	// Leading non-alpha characters trimmed (HLSL identifiers must start with a letter).
	UTEST_EQUAL("Leading digit", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("3D_Data"))), FString(TEXT("D_Data")));
	UTEST_EQUAL("Multiple leading digits", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("123abc"))), FString(TEXT("abc")));
	UTEST_EQUAL("Leading underscore", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("_foo"))), FString(TEXT("foo")));
	UTEST_EQUAL("Leading underscores and digits", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("__2x"))), FString(TEXT("x")));

	// All characters invalid → empty string.
	UTEST_EQUAL("All digits", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("123"))), FString(TEXT("")));
	UTEST_EQUAL("All special", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("!@#"))), FString(TEXT("")));

	// Combined edge cases.
	UTEST_EQUAL("Leading special + spaces", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("#My Pin"))), FString(TEXT("My_Pin")));
	UTEST_EQUAL("Single letter", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("A"))), FString(TEXT("A")));
	UTEST_EQUAL("Trailing special", PCGComputeHelpers::SanitizePinLabelForHLSL(FName(TEXT("Pin!"))), FString(TEXT("Pin_")));

	return true;
}

// ---- IsValidHLSLPinLabel ----

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGIsValidHLSLPinLabelTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.Common.IsValidHLSLPinLabel",
	PCGTestsCommon::TestFlags)

bool FPCGIsValidHLSLPinLabelTest::RunTest(const FString& Parameters)
{
	// Valid labels.
	UTEST_TRUE("Simple label is valid", PCGComputeHelpers::IsValidHLSLPinLabel(FName(TEXT("Out"))));
	UTEST_TRUE("Underscore label is valid", PCGComputeHelpers::IsValidHLSLPinLabel(FName(TEXT("My_Pin"))));
	UTEST_TRUE("Alphanumeric label is valid", PCGComputeHelpers::IsValidHLSLPinLabel(FName(TEXT("Pin1"))));

	// Invalid labels.
	UTEST_FALSE("Space is invalid", PCGComputeHelpers::IsValidHLSLPinLabel(FName(TEXT("End Points"))));
	UTEST_FALSE("Leading digit is invalid", PCGComputeHelpers::IsValidHLSLPinLabel(FName(TEXT("3D_Data"))));
	UTEST_FALSE("Dot is invalid", PCGComputeHelpers::IsValidHLSLPinLabel(FName(TEXT("My.Pin"))));
	UTEST_FALSE("Leading underscore is invalid", PCGComputeHelpers::IsValidHLSLPinLabel(FName(TEXT("_foo"))));

	return true;
}

#endif // WITH_EDITOR
