// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGCopyPoints.h"
#include "Elements/PCGGather.h"

namespace PCGCopyPointsGPUTestHelpers
{
	static const TArray<FTransform> SourceTransforms = {
		FTransform(FQuat::MakeFromEuler(FVector(10.0, 20.0, 30.0)), FVector(100.0, 200.0, 300.0), FVector(1.0, 2.0, 0.5)),
		FTransform(FQuat::MakeFromEuler(FVector(45.0, 0.0, 90.0)), FVector(-50.0, 75.0, 150.0), FVector(0.5, 1.0, 1.5)),
		FTransform(FQuat::Identity, FVector(0.0, 0.0, 0.0), FVector(1.0, 1.0, 1.0)),
	};

	static const TArray<FTransform> TargetTransforms = {
		FTransform(FQuat::MakeFromEuler(FVector(0.0, 0.0, 45.0)), FVector(500.0, 0.0, 0.0), FVector(2.0, 2.0, 2.0)),
		FTransform(FQuat::MakeFromEuler(FVector(90.0, 0.0, 0.0)), FVector(0.0, -300.0, 100.0), FVector(1.0, 0.5, 3.0)),
	};

	struct FCopyPointsGraphSetup
	{
		UPCGNode* GatherNode = nullptr;
		UPCGCopyPointsSettings* CopySettings = nullptr;
		UPCGNode* CopyNode = nullptr;
	};

	FCopyPointsGraphSetup BuildCopyPointsGraph(UPCGGraph* Graph, UPCGNode* SourceNode, UPCGNode* TargetNode)
	{
		FCopyPointsGraphSetup Setup;
		Setup.CopyNode = Graph->AddNodeOfType<UPCGCopyPointsSettings>(Setup.CopySettings);

		UPCGGatherSettings* GatherSettings = nullptr;
		Setup.GatherNode = Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

		Graph->AddEdge(SourceNode, PCGPinConstants::DefaultOutputLabel, Setup.CopyNode, PCGCopyPointsConstants::SourcePointsLabel);
		Graph->AddEdge(TargetNode, PCGPinConstants::DefaultOutputLabel, Setup.CopyNode, PCGCopyPointsConstants::TargetPointsLabel);
		Graph->AddEdge(Setup.CopyNode, PCGPinConstants::DefaultOutputLabel, Setup.GatherNode, PCGPinConstants::DefaultInputLabel);

		return Setup;
	}
}

/**
 * GPU test: CopyPoints CPU/GPU parity with default settings (all Relative).
 *
 *   CreatePoints(Source) -\
 *                          -> CopyPoints -> Gather
 *   CreatePoints(Target) -/
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCopyPointsGPU_DefaultTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CopyPoints.Default",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCopyPointsGPU_DefaultTest::RunTest(const FString& Parameters)
{
	using namespace PCGCopyPointsGPUTestHelpers;

	return PCGGPUTestCommon::RunCPUGPUParityTest(this,
		[](UPCGGraph* Graph, UWorld* /*World*/) -> PCGGPUTestCommon::FParityBuildResult
		{
			UPCGNode* SourceNode = PCGTestsCommon::AddCreatePointsNode(Graph, SourceTransforms);
			UPCGNode* TargetNode = PCGTestsCommon::AddCreatePointsNode(Graph, TargetTransforms);
			FCopyPointsGraphSetup Setup = BuildCopyPointsGraph(Graph, SourceNode, TargetNode);
			return { Setup.GatherNode, { Setup.CopySettings } };
		},
		[](FAutomationTestBase* Test, const FPCGDataCollection& CPUOutput, const FPCGDataCollection& GPUOutput) -> bool
		{
			return PCGTestsCommon::ComparePointDataCollections(Test, CPUOutput, GPUOutput);
		});
}

/**
 * GPU test: CopyPoints with all inheritance set to Source.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCopyPointsGPU_AllSourceTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CopyPoints.AllSource",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCopyPointsGPU_AllSourceTest::RunTest(const FString& Parameters)
{
	using namespace PCGCopyPointsGPUTestHelpers;

	return PCGGPUTestCommon::RunCPUGPUParityTest(this,
		[](UPCGGraph* Graph, UWorld* /*World*/) -> PCGGPUTestCommon::FParityBuildResult
		{
			UPCGNode* SourceNode = PCGTestsCommon::AddCreatePointsNode(Graph, SourceTransforms);
			UPCGNode* TargetNode = PCGTestsCommon::AddCreatePointsNode(Graph, TargetTransforms);
			FCopyPointsGraphSetup Setup = BuildCopyPointsGraph(Graph, SourceNode, TargetNode);

			Setup.CopySettings->RotationInheritance = EPCGCopyPointsInheritanceMode::Source;
			Setup.CopySettings->ScaleInheritance = EPCGCopyPointsInheritanceMode::Source;
			Setup.CopySettings->ColorInheritance = EPCGCopyPointsInheritanceMode::Source;
			Setup.CopySettings->SeedInheritance = EPCGCopyPointsInheritanceMode::Source;
			Setup.CopySettings->AttributeInheritance = EPCGCopyPointsMetadataInheritanceMode::SourceOnly;

			return { Setup.GatherNode, { Setup.CopySettings } };
		},
		[](FAutomationTestBase* Test, const FPCGDataCollection& CPUOutput, const FPCGDataCollection& GPUOutput) -> bool
		{
			return PCGTestsCommon::ComparePointDataCollections(Test, CPUOutput, GPUOutput);
		});
}

/**
 * GPU test: CopyPoints with all inheritance set to Target.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCopyPointsGPU_AllTargetTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CopyPoints.AllTarget",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCopyPointsGPU_AllTargetTest::RunTest(const FString& Parameters)
{
	using namespace PCGCopyPointsGPUTestHelpers;

	return PCGGPUTestCommon::RunCPUGPUParityTest(this,
		[](UPCGGraph* Graph, UWorld* /*World*/) -> PCGGPUTestCommon::FParityBuildResult
		{
			UPCGNode* SourceNode = PCGTestsCommon::AddCreatePointsNode(Graph, SourceTransforms);
			UPCGNode* TargetNode = PCGTestsCommon::AddCreatePointsNode(Graph, TargetTransforms);
			FCopyPointsGraphSetup Setup = BuildCopyPointsGraph(Graph, SourceNode, TargetNode);

			Setup.CopySettings->RotationInheritance = EPCGCopyPointsInheritanceMode::Target;
			Setup.CopySettings->ScaleInheritance = EPCGCopyPointsInheritanceMode::Target;
			Setup.CopySettings->ColorInheritance = EPCGCopyPointsInheritanceMode::Target;
			Setup.CopySettings->SeedInheritance = EPCGCopyPointsInheritanceMode::Target;
			Setup.CopySettings->AttributeInheritance = EPCGCopyPointsMetadataInheritanceMode::TargetOnly;

			return { Setup.GatherNode, { Setup.CopySettings } };
		},
		[](FAutomationTestBase* Test, const FPCGDataCollection& CPUOutput, const FPCGDataCollection& GPUOutput) -> bool
		{
			return PCGTestsCommon::ComparePointDataCollections(Test, CPUOutput, GPUOutput);
		});
}

/**
 * GPU test: CopyPoints with bCopyEachSourceOnEveryTarget=false (N:N pairing instead of cartesian product).
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCopyPointsGPU_NonCartesianTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CopyPoints.NonCartesian",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCopyPointsGPU_NonCartesianTest::RunTest(const FString& Parameters)
{
	using namespace PCGCopyPointsGPUTestHelpers;

	return PCGGPUTestCommon::RunCPUGPUParityTest(this,
		[](UPCGGraph* Graph, UWorld* /*World*/) -> PCGGPUTestCommon::FParityBuildResult
		{
			UPCGNode* SourceNode = PCGTestsCommon::AddCreatePointsNode(Graph, SourceTransforms);
			UPCGNode* TargetNode = PCGTestsCommon::AddCreatePointsNode(Graph, TargetTransforms);
			FCopyPointsGraphSetup Setup = BuildCopyPointsGraph(Graph, SourceNode, TargetNode);

			Setup.CopySettings->bCopyEachSourceOnEveryTarget = false;

			return { Setup.GatherNode, { Setup.CopySettings } };
		},
		[](FAutomationTestBase* Test, const FPCGDataCollection& CPUOutput, const FPCGDataCollection& GPUOutput) -> bool
		{
			return PCGTestsCommon::ComparePointDataCollections(Test, CPUOutput, GPUOutput);
		});
}

/**
 * GPU test: CopyPoints with all overridable params supplied via connected CPU override pins.
 *
 * Settings are left at defaults; CreateAttributeSet nodes connected to each override pin supply non-trivial values.
 * This exercises the GPU override system for every overridable parameter on CopyPoints.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCopyPointsGPU_OverrideAllParamsTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CopyPoints.OverrideAllParams",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCopyPointsGPU_OverrideAllParamsTest::RunTest(const FString& Parameters)
{
	using namespace PCGCopyPointsGPUTestHelpers;

	return PCGGPUTestCommon::RunCPUGPUParityTest(this,
		[](UPCGGraph* Graph, UWorld* /*World*/) -> PCGGPUTestCommon::FParityBuildResult
		{
			UPCGNode* SourceNode = PCGTestsCommon::AddCreatePointsNode(Graph, SourceTransforms);
			UPCGNode* TargetNode = PCGTestsCommon::AddCreatePointsNode(Graph, TargetTransforms);
			FCopyPointsGraphSetup Setup = BuildCopyPointsGraph(Graph, SourceNode, TargetNode);

			// Override all overridable params via connected CreateAttributeSet nodes.
			// Enum params are overridden as Int32.
			PCGGPUTestCommon::AddOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, RotationInheritance), static_cast<int32>(EPCGCopyPointsInheritanceMode::Source));
			PCGGPUTestCommon::AddOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bApplyTargetRotationToPositions), false);
			PCGGPUTestCommon::AddOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, ScaleInheritance), static_cast<int32>(EPCGCopyPointsInheritanceMode::Target));
			PCGGPUTestCommon::AddOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bApplyTargetScaleToPositions), false);
			PCGGPUTestCommon::AddOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, ColorInheritance), static_cast<int32>(EPCGCopyPointsInheritanceMode::Source));
			PCGGPUTestCommon::AddOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, SeedInheritance), static_cast<int32>(EPCGCopyPointsInheritanceMode::Target));
			PCGGPUTestCommon::AddOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, AttributeInheritance), static_cast<int32>(EPCGCopyPointsMetadataInheritanceMode::TargetFirst));
			PCGGPUTestCommon::AddOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, TagInheritance), static_cast<int32>(EPCGCopyPointsTagInheritanceMode::Target));
			PCGGPUTestCommon::AddOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bCopyEachSourceOnEveryTarget), true);

			return { Setup.GatherNode, { Setup.CopySettings } };
		},
		[](FAutomationTestBase* Test, const FPCGDataCollection& CPUOutput, const FPCGDataCollection& GPUOutput) -> bool
		{
			return PCGTestsCommon::ComparePointDataCollections(Test, CPUOutput, GPUOutput);
		});
}

/**
 * GPU test: CopyPoints with all overridable params supplied from the GPU via CreateAttributeSet -> CustomHLSL ASP chains.
 *
 * Each override is produced by a CreateAttributeSet feeding into an empty CustomHLSL AttributeSetProcessor
 * that passes the value through on the GPU. This validates that overrides arriving from GPU-resident data
 * are correctly read back and applied.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCopyPointsGPU_GPUOverridesTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CopyPoints.GPUOverrides",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCopyPointsGPU_GPUOverridesTest::RunTest(const FString& Parameters)
{
	using namespace PCGCopyPointsGPUTestHelpers;

	return PCGGPUTestCommon::RunCPUGPUParityTest(this,
		[](UPCGGraph* Graph, UWorld* /*World*/) -> PCGGPUTestCommon::FParityBuildResult
		{
			UPCGNode* SourceNode = PCGTestsCommon::AddCreatePointsNode(Graph, SourceTransforms);
			UPCGNode* TargetNode = PCGTestsCommon::AddCreatePointsNode(Graph, TargetTransforms);
			FCopyPointsGraphSetup Setup = BuildCopyPointsGraph(Graph, SourceNode, TargetNode);

			// Override params via GPU-produced attribute sets (CreateAttributeSet -> CustomHLSL passthrough).
			PCGGPUTestCommon::AddGPUOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, RotationInheritance), static_cast<int32>(EPCGCopyPointsInheritanceMode::Target));
			PCGGPUTestCommon::AddGPUOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bApplyTargetRotationToPositions), true);
			PCGGPUTestCommon::AddGPUOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, ScaleInheritance), static_cast<int32>(EPCGCopyPointsInheritanceMode::Source));
			PCGGPUTestCommon::AddGPUOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bApplyTargetScaleToPositions), true);
			PCGGPUTestCommon::AddGPUOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, ColorInheritance), static_cast<int32>(EPCGCopyPointsInheritanceMode::Relative));
			PCGGPUTestCommon::AddGPUOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, SeedInheritance), static_cast<int32>(EPCGCopyPointsInheritanceMode::Source));
			PCGGPUTestCommon::AddGPUOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, AttributeInheritance), static_cast<int32>(EPCGCopyPointsMetadataInheritanceMode::SourceFirst));
			PCGGPUTestCommon::AddGPUOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, TagInheritance), static_cast<int32>(EPCGCopyPointsTagInheritanceMode::Both));
			PCGGPUTestCommon::AddGPUOverride(Graph, Setup.CopyNode, GET_MEMBER_NAME_CHECKED(UPCGCopyPointsSettings, bCopyEachSourceOnEveryTarget), false);

			return { Setup.GatherNode, { Setup.CopySettings } };
		},
		[](FAutomationTestBase* Test, const FPCGDataCollection& CPUOutput, const FPCGDataCollection& GPUOutput) -> bool
		{
			return PCGTestsCommon::ComparePointDataCollections(Test, CPUOutput, GPUOutput);
		});
}

#endif // WITH_EDITOR
