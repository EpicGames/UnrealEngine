// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "PCGParamData.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGCreateAttribute.h"
#include "Elements/PCGCreatePoints.h"
#include "Elements/PCGGather.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

/**
 * GPU test: CustomHLSL AttributeSetProcessor doubles a float attribute.
 *
 *   CreateAttributeSet (Value=5) -> CustomHLSL (Value *= 2) -> Gather
 *
 * Verifies the output attribute set has Value == 10.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLAttributeSetDoubleValueTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.AttributeSetDoubleValue",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLAttributeSetDoubleValueTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    // CreateAttributeSet: one element with float attribute "Value" = 5.
    UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
    UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
    CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::Float;
    CreateAttrSettings->AttributeTypes.FloatValue = 5.0f;
    CreateAttrSettings->OutputTarget.SetAttributeName(FName(TEXT("Value")));

    // CustomHLSL AttributeSetProcessor: Value *= 2.
    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::AttributeSetProcessor);
    HLSLSettings->SetSourceText(
        TEXT("float v = In_GetFloat(In_DataIndex, ElementIndex, 'Value');\n")
        TEXT("Out_SetFloat(Out_DataIndex, ElementIndex, 'Value', v * 2.0f);\n")
    );

    // Useful to add gather node to grab output data from.
    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGParamData* OutParams = Cast<UPCGParamData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Readback data is attribute set data"), OutParams))
        {
            return false;
        }

        const UPCGMetadata* Metadata = OutParams->ConstMetadata();
        if (!TestNotNull(TEXT("Output has metadata"), Metadata))
        {
            return false;
        }

        const FPCGMetadataAttribute<float>* FloatAttr = Metadata->GetConstTypedAttribute<float>(FName(TEXT("Value")));
        if (!TestNotNull(TEXT("Output has 'Value' float attribute"), FloatAttr))
        {
            return false;
        }

        const double ActualValue = FloatAttr->GetValueFromItemKey(0);
        return TestNearlyEqual(*FString::Printf(TEXT("Value attribute doubled (expected 10.0, got %.1f)"), ActualValue), ActualValue, 10.0, 0.1);
    });
}

/**
 * GPU test: CustomHLSL AttributeSetProcessor passes a string attribute through unchanged.
 *
 *   CreateAttributeSet (StringAttr="Hello") -> CustomHLSL (read StringAttr, write StringAttr) -> Gather
 *
 * Verifies the output attribute set has StringAttr == "Hello".
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLAttributeSetStringAttributeTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.AttributeSetStringAttribute",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLAttributeSetStringAttributeTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    // CreateAttributeSet: one element with string attribute "StringAttr" = "Hello".
    UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
    UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
    CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::String;
    CreateAttrSettings->AttributeTypes.StringValue = TEXT("Hello");
    CreateAttrSettings->OutputTarget.SetAttributeName(FName(TEXT("StringAttr")));

    // CustomHLSL AttributeSetProcessor: read StringAttr and write it back unchanged.
    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::AttributeSetProcessor);
    HLSLSettings->SetSourceText(
        TEXT("int StringAttrVal = In_GetStringKey(In_DataIndex, ElementIndex, 'StringAttr');\n")
        TEXT("Out_SetStringKey(Out_DataIndex, ElementIndex, 'StringAttr', StringAttrVal);\n")
    );

    // Useful to add gather node to grab output data from.
    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGParamData* OutParams = Cast<UPCGParamData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Readback data is attribute set data"), OutParams))
        {
            return false;
        }

        const UPCGMetadata* Metadata = OutParams->ConstMetadata();
        if (!TestNotNull(TEXT("Output has metadata"), Metadata))
        {
            return false;
        }

        const FPCGMetadataAttribute<FString>* StringAttr = Metadata->GetConstTypedAttribute<FString>(FName(TEXT("StringAttr")));
        if (!TestNotNull(TEXT("Output has 'StringAttr' string attribute"), StringAttr))
        {
            return false;
        }

        const FString ActualValue = StringAttr->GetValueFromItemKey(0);
        return TestEqual(
            *FString::Printf(TEXT("'StringAttr' attribute passed through (expected 'Hello', got '%s')"), *ActualValue),
            ActualValue, FString(TEXT("Hello")));
    });
}

/**
 * GPU test: CustomHLSL infers created attributes from HLSL Set calls.
 *
 *   CreatePoints (2 points) -> CustomHLSL PointProcessor (writes one attribute of each type) -> Gather
 *
 * Verifies that attributes written via Out_Set* calls in HLSL are automatically inferred and present
 * on the output point data with the correct C++ metadata type. No CreatedKernelAttributeKeys are
 * configured — HLSL inference is the sole source.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLInferredAttributeTypesTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.InferredAttributeTypes",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLInferredAttributeTypesTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    // CreatePoints: 2 default points.
    UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
    UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
    CreatePointsSettings->PointsToCreate.Empty();
    CreatePointsSettings->PointsToCreate.AddDefaulted();
    CreatePointsSettings->PointsToCreate.AddDefaulted();
    CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;

    // CustomHLSL PointProcessor: write one attribute of each supported non-string type.
    // No CreatedKernelAttributeKeys — all attributes are inferred from the HLSL Set calls.
    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
    HLSLSettings->SetSourceText(
        TEXT("Out_SetBool(Out_DataIndex, ElementIndex, 'AttrBool', true);\n")
        TEXT("Out_SetInt(Out_DataIndex, ElementIndex, 'AttrInt', 7);\n")
        TEXT("Out_SetFloat(Out_DataIndex, ElementIndex, 'AttrFloat', 1.5f);\n")
        TEXT("Out_SetFloat2(Out_DataIndex, ElementIndex, 'AttrFloat2', float2(1.0f, 2.0f));\n")
        TEXT("Out_SetFloat3(Out_DataIndex, ElementIndex, 'AttrFloat3', float3(1.0f, 2.0f, 3.0f));\n")
        TEXT("Out_SetFloat4(Out_DataIndex, ElementIndex, 'AttrFloat4', float4(1.0f, 2.0f, 3.0f, 4.0f));\n")
        TEXT("Out_SetRotator(Out_DataIndex, ElementIndex, 'AttrRotator', float3(10.0f, 20.0f, 30.0f));\n")
        TEXT("Out_SetQuat(Out_DataIndex, ElementIndex, 'AttrQuat', float4(0.0f, 0.0f, 0.0f, 1.0f));\n")
        TEXT("Out_SetTransform(Out_DataIndex, ElementIndex, 'AttrTransform', float4x4(1,0,0,0, 0,1,0,0, 0,0,1,0, 0,0,0,1));\n")
    );

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
        {
            return false;
        }

        const UPCGMetadata* Metadata = OutPoints->ConstMetadata();
        if (!TestNotNull(TEXT("Output has metadata"), Metadata))
        {
            return false;
        }

        bool bResult = true;
        bResult &= TestNotNull(TEXT("Bool attribute inferred from HLSL"),      Metadata->GetConstTypedAttribute<bool>(FName("AttrBool")));
        bResult &= TestNotNull(TEXT("Int attribute inferred from HLSL"),        Metadata->GetConstTypedAttribute<int32>(FName("AttrInt")));
        bResult &= TestNotNull(TEXT("Float attribute inferred from HLSL"),      Metadata->GetConstTypedAttribute<float>(FName("AttrFloat")));
        bResult &= TestNotNull(TEXT("Float2 attribute inferred from HLSL"),     Metadata->GetConstTypedAttribute<FVector2D>(FName("AttrFloat2")));
        bResult &= TestNotNull(TEXT("Float3 attribute inferred from HLSL"),     Metadata->GetConstTypedAttribute<FVector>(FName("AttrFloat3")));
        bResult &= TestNotNull(TEXT("Float4 attribute inferred from HLSL"),     Metadata->GetConstTypedAttribute<FVector4>(FName("AttrFloat4")));
        bResult &= TestNotNull(TEXT("Rotator attribute inferred from HLSL"),    Metadata->GetConstTypedAttribute<FRotator>(FName("AttrRotator")));
        bResult &= TestNotNull(TEXT("Quat attribute inferred from HLSL"),       Metadata->GetConstTypedAttribute<FQuat>(FName("AttrQuat")));
        bResult &= TestNotNull(TEXT("Transform attribute inferred from HLSL"),  Metadata->GetConstTypedAttribute<FTransform>(FName("AttrTransform")));
        return bResult;
    });
}

/**
 * GPU test: CustomHLSL per-pin attribute creation settings (enabled).
 *
 *   CustomHLSL PointGenerator (empty HLSL, bPerPinAttributeCreationSettings=true, all types in CreatedKernelAttributeKeys) -> Gather
 *
 * Verifies that when bPerPinAttributeCreationSettings is enabled, attributes configured in
 * CreatedKernelAttributeKeys are present on the output even when HLSL writes no attributes.
 *
 * Note: EPCGKernelAttributeType::Name/StringKey is excluded because it requires input pins
 * to carry existing string-key attributes for the GPU string table. A PointGenerator has no
 * inputs, so including it would produce a "No incoming attributes to obtain string keys from"
 * warning and the attribute would not be usable.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLPerPinAttributeSettingsEnabledTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.PerPinAttributeSettingsEnabled",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLPerPinAttributeSettingsEnabledTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
    HLSLSettings->SetSourceText(TEXT(""));
    HLSLSettings->NumElements = 2;
    HLSLSettings->SetPerPinAttributeCreationSettings(true);

    // Add one attribute of each type to the default output pin.
    check(HLSLSettings->OutputPins.Num() == 1);
    TArray<FPCGKernelAttributeKey>& Keys = HLSLSettings->OutputPins[0].PropertiesGPU.CreatedKernelAttributeKeys;
    Keys.Add(FPCGKernelAttributeKey(FName("AttrBool"),      EPCGKernelAttributeType::Bool));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrInt"),       EPCGKernelAttributeType::Int));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrFloat"),     EPCGKernelAttributeType::Float));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrFloat2"),    EPCGKernelAttributeType::Float2));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrFloat3"),    EPCGKernelAttributeType::Float3));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrFloat4"),    EPCGKernelAttributeType::Float4));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrRotator"),   EPCGKernelAttributeType::Rotator));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrQuat"),      EPCGKernelAttributeType::Quat));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrTransform"), EPCGKernelAttributeType::Transform));

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
        {
            return false;
        }

        const UPCGMetadata* Metadata = OutPoints->ConstMetadata();
        if (!TestNotNull(TEXT("Output has metadata"), Metadata))
        {
            return false;
        }

        bool bResult = true;
        bResult &= TestNotNull(TEXT("Bool attribute created from per-pin settings"),      Metadata->GetConstTypedAttribute<bool>(FName("AttrBool")));
        bResult &= TestNotNull(TEXT("Int attribute created from per-pin settings"),        Metadata->GetConstTypedAttribute<int32>(FName("AttrInt")));
        bResult &= TestNotNull(TEXT("Float attribute created from per-pin settings"),      Metadata->GetConstTypedAttribute<float>(FName("AttrFloat")));
        bResult &= TestNotNull(TEXT("Float2 attribute created from per-pin settings"),     Metadata->GetConstTypedAttribute<FVector2D>(FName("AttrFloat2")));
        bResult &= TestNotNull(TEXT("Float3 attribute created from per-pin settings"),     Metadata->GetConstTypedAttribute<FVector>(FName("AttrFloat3")));
        bResult &= TestNotNull(TEXT("Float4 attribute created from per-pin settings"),     Metadata->GetConstTypedAttribute<FVector4>(FName("AttrFloat4")));
        bResult &= TestNotNull(TEXT("Rotator attribute created from per-pin settings"),    Metadata->GetConstTypedAttribute<FRotator>(FName("AttrRotator")));
        bResult &= TestNotNull(TEXT("Quat attribute created from per-pin settings"),       Metadata->GetConstTypedAttribute<FQuat>(FName("AttrQuat")));
        bResult &= TestNotNull(TEXT("Transform attribute created from per-pin settings"),  Metadata->GetConstTypedAttribute<FTransform>(FName("AttrTransform")));
        return bResult;
    });
}

/**
 * GPU test: CustomHLSL per-pin attribute creation settings (disabled).
 *
 *   CustomHLSL PointGenerator (empty HLSL, bPerPinAttributeCreationSettings=false, all types in CreatedKernelAttributeKeys) -> Gather
 *
 * Verifies that when bPerPinAttributeCreationSettings is disabled (the default), attributes
 * configured in CreatedKernelAttributeKeys are ignored — only HLSL inference applies (none here).
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLPerPinAttributeSettingsDisabledTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.PerPinAttributeSettingsDisabled",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLPerPinAttributeSettingsDisabledTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
    HLSLSettings->SetSourceText(TEXT(""));
    HLSLSettings->NumElements = 2;
    HLSLSettings->SetPerPinAttributeCreationSettings(false);  // default: authored keys are ignored

    // Same keys as the enabled test — none of these should appear in the output.
    check(HLSLSettings->OutputPins.Num() == 1);
    TArray<FPCGKernelAttributeKey>& Keys = HLSLSettings->OutputPins[0].PropertiesGPU.CreatedKernelAttributeKeys;
    Keys.Add(FPCGKernelAttributeKey(FName("AttrBool"),      EPCGKernelAttributeType::Bool));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrInt"),       EPCGKernelAttributeType::Int));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrFloat"),     EPCGKernelAttributeType::Float));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrFloat2"),    EPCGKernelAttributeType::Float2));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrFloat3"),    EPCGKernelAttributeType::Float3));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrFloat4"),    EPCGKernelAttributeType::Float4));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrRotator"),   EPCGKernelAttributeType::Rotator));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrQuat"),      EPCGKernelAttributeType::Quat));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrTransform"), EPCGKernelAttributeType::Transform));

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
        {
            return false;
        }

        const UPCGMetadata* Metadata = OutPoints->ConstMetadata();
        if (!TestNotNull(TEXT("Output has metadata"), Metadata))
        {
            return false;
        }

        bool bResult = true;
        bResult &= TestNull(TEXT("Bool attribute absent when per-pin settings disabled"),      Metadata->GetConstTypedAttribute<bool>(FName("AttrBool")));
        bResult &= TestNull(TEXT("Int attribute absent when per-pin settings disabled"),        Metadata->GetConstTypedAttribute<int32>(FName("AttrInt")));
        bResult &= TestNull(TEXT("Float attribute absent when per-pin settings disabled"),      Metadata->GetConstTypedAttribute<float>(FName("AttrFloat")));
        bResult &= TestNull(TEXT("Float2 attribute absent when per-pin settings disabled"),     Metadata->GetConstTypedAttribute<FVector2D>(FName("AttrFloat2")));
        bResult &= TestNull(TEXT("Float3 attribute absent when per-pin settings disabled"),     Metadata->GetConstTypedAttribute<FVector>(FName("AttrFloat3")));
        bResult &= TestNull(TEXT("Float4 attribute absent when per-pin settings disabled"),     Metadata->GetConstTypedAttribute<FVector4>(FName("AttrFloat4")));
        bResult &= TestNull(TEXT("Rotator attribute absent when per-pin settings disabled"),    Metadata->GetConstTypedAttribute<FRotator>(FName("AttrRotator")));
        bResult &= TestNull(TEXT("Quat attribute absent when per-pin settings disabled"),       Metadata->GetConstTypedAttribute<FQuat>(FName("AttrQuat")));
        bResult &= TestNull(TEXT("Transform attribute absent when per-pin settings disabled"),  Metadata->GetConstTypedAttribute<FTransform>(FName("AttrTransform")));
        return bResult;
    });
}

/**
 * GPU test: attribute created from both CreatedKernelAttributeKeys and HLSL Set call with no conflict.
 *
 *   CustomHLSL PointGenerator (bPerPinAttributeCreationSettings=true, AttrFloat authored + written in HLSL) -> Gather
 *
 * Verifies that when the same attribute (name + type) is declared in both CreatedKernelAttributeKeys and
 * written via HLSL, the attribute is present exactly once with the correct type. The deduplication in
 * PopulateAttributeKeys (AddUnique) should prevent duplicate entries.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLAttributeCreationOverlapTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.AttributeCreationOverlap",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLAttributeCreationOverlapTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
    HLSLSettings->NumElements = 2;
    HLSLSettings->SetPerPinAttributeCreationSettings(true);

    // SetSourceText MUST be called before adding keys to CreatedKernelAttributeKeys.
    // PostEditChangeProperty triggered by SetSourceText calls UpdateAttributeKeys() which
    // runs UpdateIdentifierFromSelector() on every key — keys constructed via FPCGKernelAttributeKey(FName, Type)
    // have an empty Name selector, so their MetadataDomain gets set to Invalid, making them fail IsValid().
    HLSLSettings->SetSourceText(
        TEXT("Out_SetFloat(Out_DataIndex, ElementIndex, 'AttrFloat', 3.0f);\n")
        TEXT("Out_SetBool(Out_DataIndex, ElementIndex, 'AttrBool', true);\n")
    );

    // Author AttrFloat and AttrInt in CreatedKernelAttributeKeys.
    check(HLSLSettings->OutputPins.Num() == 1);
    TArray<FPCGKernelAttributeKey>& Keys = HLSLSettings->OutputPins[0].PropertiesGPU.CreatedKernelAttributeKeys;
    Keys.Add(FPCGKernelAttributeKey(FName("AttrFloat"), EPCGKernelAttributeType::Float));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrInt"),   EPCGKernelAttributeType::Int));

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
        {
            return false;
        }

        const UPCGMetadata* Metadata = OutPoints->ConstMetadata();
        if (!TestNotNull(TEXT("Output has metadata"), Metadata))
        {
            return false;
        }

        bool bResult = true;
        // Authored only — should exist.
        bResult &= TestNotNull(TEXT("AttrInt authored only — present"),              Metadata->GetConstTypedAttribute<int32>(FName("AttrInt")));
        // Authored AND written in HLSL (same type) — should exist exactly once, no conflict.
        bResult &= TestNotNull(TEXT("AttrFloat authored + HLSL written — present"),  Metadata->GetConstTypedAttribute<float>(FName("AttrFloat")));
        // Inferred from HLSL only — should exist.
        bResult &= TestNotNull(TEXT("AttrBool inferred from HLSL — present"),        Metadata->GetConstTypedAttribute<bool>(FName("AttrBool")));
        return bResult;
    });
}

/**
 * GPU test: authored and inferred attributes with distinct names coexist without conflict.
 *
 *   CustomHLSL PointGenerator (bPerPinAttributeCreationSettings=true) -> Gather
 *
 * Authored in CreatedKernelAttributeKeys: "AttrA" as Float, "AttrB" as Int.
 * Written in HLSL:                        "AttrC" as Bool,  "AttrD" as Float3.
 *
 * Verifies that all four attributes are present on the output — attributes from
 * CreatedKernelAttributeKeys (pass 1) and HLSL inference (pass 2) are merged together.
 *
 * Note: same-name/different-type attributes are NOT supported by the runtime. Each attribute
 * name must map to exactly one type. This test uses four distinct names to avoid that constraint.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLAuthoredAndInferredCoexistTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.AuthoredAndInferredCoexist",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLAuthoredAndInferredCoexistTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
    HLSLSettings->NumElements = 2;
    HLSLSettings->SetPerPinAttributeCreationSettings(true);

    // SetSourceText MUST be called before adding keys (see AttributeCreationOverlap for explanation).
    HLSLSettings->SetSourceText(
        TEXT("Out_SetBool(Out_DataIndex, ElementIndex, 'AttrC', true);\n")
        TEXT("Out_SetFloat3(Out_DataIndex, ElementIndex, 'AttrD', float3(1.0f, 2.0f, 3.0f));\n")
    );

    // Author "AttrA" as Float and "AttrB" as Int.
    check(HLSLSettings->OutputPins.Num() == 1);
    TArray<FPCGKernelAttributeKey>& Keys = HLSLSettings->OutputPins[0].PropertiesGPU.CreatedKernelAttributeKeys;
    Keys.Add(FPCGKernelAttributeKey(FName("AttrA"), EPCGKernelAttributeType::Float));
    Keys.Add(FPCGKernelAttributeKey(FName("AttrB"), EPCGKernelAttributeType::Int));

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
        {
            return false;
        }

        const UPCGMetadata* Metadata = OutPoints->ConstMetadata();
        if (!TestNotNull(TEXT("Output has metadata"), Metadata))
        {
            return false;
        }

        bool bResult = true;
        bResult &= TestNotNull(TEXT("AttrA (Float, authored) — present"),   Metadata->GetConstTypedAttribute<float>(FName("AttrA")));
        bResult &= TestNotNull(TEXT("AttrB (Int, authored) — present"),     Metadata->GetConstTypedAttribute<int32>(FName("AttrB")));
        bResult &= TestNotNull(TEXT("AttrC (Bool, HLSL inferred) — present"),   Metadata->GetConstTypedAttribute<bool>(FName("AttrC")));
        bResult &= TestNotNull(TEXT("AttrD (Float3, HLSL inferred) — present"), Metadata->GetConstTypedAttribute<FVector>(FName("AttrD")));
        return bResult;
    });
}

#endif // WITH_EDITOR
