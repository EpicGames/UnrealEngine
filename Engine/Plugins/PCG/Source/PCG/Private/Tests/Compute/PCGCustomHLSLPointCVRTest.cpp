// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGCreatePoints.h"
#include "Elements/PCGGather.h"
#include "Helpers/PCGHelpers.h"

/**
 * GPU test: PointProcessor with a conditional constant-literal write to position must NOT incorrectly
 * demote Transform to CVR when forwarding from an allocated input.
 *
 *   CreatePoints (4 pts: X=0,100,200,300) -> PointProcessor (conditional constant SetPosition) -> Gather
 *
 * The setter value float3(0.0,0.0,0.0) is a constant literal, but the call is inside an if-block whose
 * condition is never true (ElementIndex > 9999 with only 4 elements). The static analysis in
 * PopulateAttributeKeys only inspects the VALUE tokens, not whether the call site is inside a
 * conditional - so it marks Transform/Position as constant-only in PinToPropertiesWithSingleConstantSetters.
 * DeallocatePropertiesForAllData then strips Transform from the output descriptor, producing a 1-slot
 * CVR buffer. The PointProcessor preamble copies element[0]'s position into that single slot, so all
 * four elements read (0,0,0) instead of their actual input positions.
 *
 * Expected (correct): Transform is NOT CVR; each element retains its per-element input position.
 * Failing (with bug):  Transform IS CVR; elements 1-3 return position (0,0,0) instead of (100,0,0) etc.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLConditionalConstantCVRForwardingTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.ConditionalConstantCVRForwarding",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLConditionalConstantCVRForwardingTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	// 4 points at distinct X positions so Transform is FullAlloc (allocated) in the input descriptor.
	UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
	UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
	CreatePointsSettings->PointsToCreate.Empty();
	CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;
	for (int32 i = 0; i < 4; ++i)
	{
		FPCGPoint& Pt = CreatePointsSettings->PointsToCreate.AddDefaulted_GetRef();
		Pt.Transform.SetLocation(FVector(i * 100.0, 0.0, 0.0));
	}

	// PointProcessor: conditional constant write to Position. The condition (ElementIndex > 9999)
	// is always false for 4 elements, so no element's position is ever overwritten. The static
	// analysis marks Transform as constant-only because the value argument is a constant literal -
	// it does not track whether the call is inside a branch. This triggers the bug path:
	// DeallocatePropertiesForAllData strips Transform from the descriptor even though the input had
	// it fully allocated with varying per-element values.
	UPCGCustomHLSLSettings* HLSLSettings = nullptr;
	UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
	HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
	HLSLSettings->SetSourceText(
		TEXT("if (ElementIndex > 9999u)\n")
		TEXT("{\n")
		TEXT("    Out_SetPosition(Out_DataIndex, ElementIndex, float3(0.0, 0.0, 0.0));\n")
		TEXT("}\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
		{
			const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
			if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
			{
				return false;
			}

			constexpr int32 ExpectedPointCount = 4;
			if (!TestEqual(TEXT("Output point count"), OutPoints->GetNumPoints(), ExpectedPointCount))
			{
				return false;
			}

			const FConstPCGPointValueRanges Ranges(OutPoints);
			bool bTestPassed = true;

			// Transform must NOT be CVR: the write was conditional and never triggered, so the input's
			// per-element positions should be forwarded unchanged. With the bug, the descriptor
			// incorrectly strips Transform to a 1-slot CVR, causing this assertion to fail.
			bTestPassed &= TestFalse(TEXT("Transform should NOT be CVR (conditional write never triggered)"),
				Ranges.TransformRange.GetSingleValue().IsSet());

			// Each element should retain its input position. With the bug all elements read element[0]'s
			// position (0,0,0) so assertions for i>0 fail (they return 0 instead of i*100).
			for (int32 i = 0; i < ExpectedPointCount; ++i)
			{
				const FString P = FString::Printf(TEXT("Point[%d]"), i);
				bTestPassed &= TestNearlyEqual(P + TEXT(" Position.X"), Ranges.TransformRange[i].GetLocation().X, (double)(i * 100), 0.01);
				bTestPassed &= TestNearlyEqual(P + TEXT(" Position.Y is 0"), Ranges.TransformRange[i].GetLocation().Y, 0.0, 0.01);
				bTestPassed &= TestNearlyEqual(P + TEXT(" Position.Z is 0"), Ranges.TransformRange[i].GetLocation().Z, 0.0, 0.01);
			}

			return bTestPassed;
		});
}

/**
 * GPU test: two parallel PointProcessors setting different property subsets, followed by an empty
 * passthrough processor, with CVR state verified at both gather points.
 *
 *   CreatePoints (4 points) -> Processor1 (TRS + BoundsMin + Color) -> Gather1 -> EmptyProcessor -> Gather2
 *                           \-> Processor2 (BoundsMax + Color + Steepness + Seed + Density) /
 *
 * Gather1 holds 2 data items (one per processor). EmptyProcessor receives both items simultaneously
 * (multi-data passthrough) and forwards them unchanged to Gather2.
 *
 * The same CVR assertions are run on both Gather1 and Gather2, confirming that CVR allocation
 * metadata is correctly preserved when a point processor passes through multi-data unchanged.
 *
 * Processor 1: constant TRS + BoundsMin (CVR), per-element Color (non-CVR), and no-setter CVR
 *              BoundsMax + Steepness + Seed (!bAutoInit, no setter -> CVR, uninitialized), Density (non-CVR - sentinel excluded).
 * Processor 2: constant BoundsMax + Steepness + Seed (CVR), per-element Color (non-CVR),
 *              Density (non-CVR - sentinel), and no-setter CVR Transform + BoundsMin (!bAutoInit, no setter -> CVR, uninitialized).
 *
 * The EmptyProcessor uses bAutoInitializeOutput=true (default) so it inherits each property's CVR/FullAlloc state
 * from its input unchanged. With bAutoInitializeOutput=false, all unwritten properties would be compressed to
 * CVR in the empty pass, destroying per-element Color and Density data.
 *
 * pcg.GPU.FuzzMemory=true fills GPU buffers with noise so no-setter CVR slots (uninitialized by design when
 * !bAutoInitializeOutput) contain garbage, not the input-default values. The test asserts that these CVR
 * vectors do NOT match the corresponding input defaults, verifying the slots were truly left alone.
 *
 * The two data items are distinguished by BoundsMax CVR value:
 *   Processor 2: explicit constant BoundsMax setter -> CVR with value (2, 0.75, 8).
 *   Processor 1: no BoundsMax setter -> CVR with undefined (garbage) value, identified by not matching Processor 2.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLConstantValueRangeTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.ConstantValueRange",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLConstantValueRangeTest::RunTest(const FString& Parameters)
{
	TGuardConsoleVariable<bool> FuzzMemoryGuard(IConsoleManager::Get().FindConsoleVariable(TEXT("pcg.GPU.FuzzMemory")), /*bNewValue=*/true);

	FPCGGPUGraphTestRunner Runner;

	// CreatePoints: 4 points with different positions. The distinct positions ensure the Transform
	// input is FullAlloc (non-CVR) for both processors before any no-setter CVR compression occurs.
	UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
	UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
	CreatePointsSettings->PointsToCreate.Empty();
	for (int32 i = 0; i < 4; ++i)
	{
		FPCGPoint& Pt = CreatePointsSettings->PointsToCreate.AddDefaulted_GetRef();
		Pt.Transform.SetLocation(FVector(i * 100.0, 0.0, 0.0));
	}
	CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;

	// Processor 1: TRS + BoundsMin as constant CVR setters; Color as non-CVR per-element setter.
	//   arithmetic:           float3(1.0 + 0.5, +2.0, (((3.0))))     for Position (unary-plus, triple-nested parens)
	//   embedded C comment:   float4(0.0, 0.0, 0.0, /* w */ 1.0)     for Rotation (identity quat xyzw)
	//   ternary:              float3(1 ? 2.0 : 3.0, 2.0, 2.0)        for Scale
	//   mixed-sign literals:  float3(-0.5, 0.5, -0.25)               for BoundsMin
	//   per-element variable: float4(ElementIndex * 0.25, ...)        for Color (non-CVR)
	UPCGCustomHLSLSettings* HLSLSettings1 = nullptr;
	UPCGNode* HLSLNode1 = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings1);
	HLSLSettings1->SetKernelType(EPCGKernelType::PointProcessor);
	check(HLSLSettings1->OutputPins.Num() == 1);
	HLSLSettings1->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = false;
	HLSLSettings1->SetSourceText(
		TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, float3(1.0 + 0.5, +2.0, (((3.0)))));\n")
		TEXT("Out_SetRotation(Out_DataIndex, ElementIndex, float4(0.0, 0.0, 0.0, /* w */ 1.0));\n")
		TEXT("Out_SetScale(Out_DataIndex, ElementIndex, float3(1 ? 2.0 : 3.0, 2.0, 2.0));\n")
		TEXT("Out_SetBoundsMin(Out_DataIndex, ElementIndex, float3(-0.5, 0.5, -0.25));\n")
		TEXT("Out_SetColor(Out_DataIndex, ElementIndex, float4((float)ElementIndex * 0.25, 0.0, 0.0, 1.0));\n")
	);

	// Processor 2: BoundsMax + Steepness + Seed as constant CVR setters; Color as non-CVR per-element;
	// Density always non-CVR (sentinel encoding).
	//   nested arithmetic:    float3(1.0*2.0, 3.0/4.0, (2.5*2.0)+3.0) for BoundsMax
	//   per-element variable: float4(ElementIndex * 0.25, ...)          for Color (non-CVR)
	//   unary plus on float:  +0.75                                      for Steepness
	//   hex integer literal:  0x2A (== 42)                              for Seed
	//   constant Density:     1.0 (still fully allocated - sentinel)
	UPCGCustomHLSLSettings* HLSLSettings2 = nullptr;
	UPCGNode* HLSLNode2 = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings2);
	HLSLSettings2->SetKernelType(EPCGKernelType::PointProcessor);
	check(HLSLSettings2->OutputPins.Num() == 1);
	HLSLSettings2->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = false;
	HLSLSettings2->SetSourceText(
		TEXT("Out_SetBoundsMax(Out_DataIndex, ElementIndex, float3(1.0*2.0, 3.0/4.0, (2.5*2.0)+3.0));\n")
		TEXT("Out_SetColor(Out_DataIndex, ElementIndex, float4((float)ElementIndex * 0.25, 0.0, 0.0, 1.0));\n")
		TEXT("Out_SetSteepness(Out_DataIndex, ElementIndex, +0.75);\n")
		TEXT("Out_SetSeed(Out_DataIndex, ElementIndex, 0x2A);\n")
		TEXT("Out_SetDensity(Out_DataIndex, ElementIndex, 1.0);\n")
	);

	// Gather1: collects both processor outputs as separate data items.
	UPCGGatherSettings* GatherSettings1 = nullptr;
	UPCGNode* GatherNode1 = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings1);

	// EmptyProcessor: no-op body - passes all data items through unchanged. bAutoInitializeOutput=true (default)
	// means each property inherits its CVR/FullAlloc state from the input unchanged. With bAutoInitializeOutput=false,
	// all unwritten properties would be compressed to CVR in this pass, destroying per-element Color and Density data.
	UPCGCustomHLSLSettings* EmptyProcSettings = nullptr;
	UPCGNode* EmptyProcNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(EmptyProcSettings);
	EmptyProcSettings->SetKernelType(EPCGKernelType::PointProcessor);
	check(EmptyProcSettings->OutputPins.Num() == 1);
	EmptyProcSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = true;
	EmptyProcSettings->SetSourceText(TEXT(""));

	// Gather2: collects the passthrough output for post-passthrough CVR verification.
	UPCGGatherSettings* GatherSettings2 = nullptr;
	UPCGNode* GatherNode2 = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings2);

	Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode1, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode2, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode1, PCGPinConstants::DefaultOutputLabel, GatherNode1, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode2, PCGPinConstants::DefaultOutputLabel, GatherNode1, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(GatherNode1, PCGPinConstants::DefaultOutputLabel, EmptyProcNode, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(EmptyProcNode, PCGPinConstants::DefaultOutputLabel, GatherNode2, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode1, GatherNode2 } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	// Validates a single data collection from a gather: expects 2 data items (one per source processor),
	// distinguishes them by BoundsMax CVR value, and verifies CVR flags and values for each.
	constexpr int32 ExpectedPointCount = 4;
	auto ValidateGatherOutput = [&](const FPCGDataCollection& GatherOutput, const TCHAR* GatherLabel) -> bool
		{
			if (!TestEqual(*FString::Printf(TEXT("%s: two data items (one per processor)"), GatherLabel), GatherOutput.TaggedData.Num(), 2))
			{
				return false;
			}

			bool bPassed = true;
			bool bFoundProc1 = false;
			bool bFoundProc2 = false;

			for (const FPCGTaggedData& TaggedData : GatherOutput.TaggedData)
			{
				const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
				if (!TestNotNull(*FString::Printf(TEXT("%s: output data is point data"), GatherLabel), OutPoints))
				{
					return false;
				}

				if (!TestEqual(*FString::Printf(TEXT("%s: output point count"), GatherLabel), OutPoints->GetNumPoints(), ExpectedPointCount))
				{
					return false;
				}

				const FConstPCGPointValueRanges Ranges(OutPoints);

				// Disambiguation: both processors produce CVR BoundsMax.
				// Processor 2 has an explicit constant setter -> CVR with value (2, 0.75, 8).
				// Processor 1 has no setter -> CVR with undefined (garbage) value; identified by not matching Processor 2's explicit value.
				const TOptional<const FVector> BoundsMaxCVR = Ranges.BoundsMaxRange.GetSingleValue();
				bPassed &= TestTrue(*FString::Printf(TEXT("%s: BoundsMax is CVR (both processors)"), GatherLabel), BoundsMaxCVR.IsSet());
				// Check all three components to avoid a false P2 match when fuzz happens to produce X~=2.0.
				const bool bIsProc2BoundsMax = BoundsMaxCVR.IsSet()
					&& FMath::IsNearlyEqual(BoundsMaxCVR.GetValue().X, 2.0, 0.1)
					&& FMath::IsNearlyEqual(BoundsMaxCVR.GetValue().Y, 0.75, 0.1)
					&& FMath::IsNearlyEqual(BoundsMaxCVR.GetValue().Z, 8.0, 0.1);
				if (!bIsProc2BoundsMax)
				{
					// Processor 1: no BoundsMax setter -> CVR with undefined (garbage) value, identified by elimination.
					bFoundProc1 = true;

					// CVR status: Processor 1 explicitly sets TRS+BoundsMin; no-setter properties (!bAutoInit) are compressed to CVR.
					bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc1: Transform is a CVR (explicit setter)"), GatherLabel), Ranges.TransformRange.GetSingleValue().IsSet());
					bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc1: BoundsMin is a CVR (explicit setter)"), GatherLabel), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					bPassed &= TestFalse(*FString::Printf(TEXT("%s Proc1: Color is NOT a CVR (per-element variable)"), GatherLabel), Ranges.ColorRange.GetSingleValue().IsSet());
					bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc1: BoundsMax is a CVR (no-setter, !bAutoInit)"), GatherLabel), Ranges.BoundsMaxRange.GetSingleValue().IsSet());
					bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc1: Steepness is a CVR (no-setter, !bAutoInit)"), GatherLabel), Ranges.SteepnessRange.GetSingleValue().IsSet());
					bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc1: Seed is a CVR (no-setter, !bAutoInit)"), GatherLabel), Ranges.SeedRange.GetSingleValue().IsSet());
					bPassed &= TestFalse(*FString::Printf(TEXT("%s Proc1: Density is NOT a CVR (sentinel-encoded)"), GatherLabel), Ranges.DensityRange.GetSingleValue().IsSet());

					// Per-element values for all properties and all 4 elements.
					for (int32 i = 0; i < ExpectedPointCount; ++i)
					{
						// Transform: constant pos=(1.5,2,3), identity rot, scale=(2,2,2)
						const FTransform& Ti = Ranges.TransformRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: Transform[%d] Pos.X (1.5)"), GatherLabel, i), Ti.GetLocation().X, 1.5, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: Transform[%d] Pos.Y (2.0)"), GatherLabel, i), Ti.GetLocation().Y, 2.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: Transform[%d] Pos.Z (3.0)"), GatherLabel, i), Ti.GetLocation().Z, 3.0, 0.01);
						bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc1: Transform[%d] Rot identity"), GatherLabel, i), Ti.GetRotation().Equals(FQuat::Identity, 0.01f));
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: Transform[%d] Scale.X (2.0)"), GatherLabel, i), Ti.GetScale3D().X, 2.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: Transform[%d] Scale.Y (2.0)"), GatherLabel, i), Ti.GetScale3D().Y, 2.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: Transform[%d] Scale.Z (2.0)"), GatherLabel, i), Ti.GetScale3D().Z, 2.0, 0.01);

						// BoundsMin: constant (-0.5, 0.5, -0.25)
						const FVector& BMin = Ranges.BoundsMinRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: BoundsMin[%d].X (-0.5)"), GatherLabel, i), BMin.X, -0.5, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: BoundsMin[%d].Y (0.5)"), GatherLabel, i), BMin.Y, 0.5, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: BoundsMin[%d].Z (-0.25)"), GatherLabel, i), BMin.Z, -0.25, 0.01);

						// Color: R = i*0.25 (per-element), G=0, B=0, A=1
						const FVector4& C = Ranges.ColorRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: Color[%d].R (%g)"), GatherLabel, i, i * 0.25), (double)C.X, i * 0.25, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: Color[%d].G (0.0)"), GatherLabel, i), (double)C.Y, 0.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: Color[%d].B (0.0)"), GatherLabel, i), (double)C.Z, 0.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc1: Color[%d].A (1.0)"), GatherLabel, i), (double)C.W, 1.0, 0.01);
					}
				}
				else
				{
					// Processor 2: explicit constant BoundsMax setter -> CVR (2, 0.75, 8).
					bFoundProc2 = true;

					// CVR status: Processor 2 explicitly sets BoundsMax/Steepness/Seed; no-setter (!bAutoInit) compresses Transform+BoundsMin to CVR.
					bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc2: BoundsMax is a CVR (explicit setter)"), GatherLabel), Ranges.BoundsMaxRange.GetSingleValue().IsSet());
					bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc2: Steepness is a CVR (explicit setter)"), GatherLabel), Ranges.SteepnessRange.GetSingleValue().IsSet());
					bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc2: Seed is a CVR (explicit setter)"), GatherLabel), Ranges.SeedRange.GetSingleValue().IsSet());
					bPassed &= TestFalse(*FString::Printf(TEXT("%s Proc2: Color is NOT a CVR (per-element variable)"), GatherLabel), Ranges.ColorRange.GetSingleValue().IsSet());
					bPassed &= TestFalse(*FString::Printf(TEXT("%s Proc2: Density is NOT a CVR (sentinel-encoded)"), GatherLabel), Ranges.DensityRange.GetSingleValue().IsSet());
					bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc2: Transform is a CVR (no-setter, !bAutoInit)"), GatherLabel), Ranges.TransformRange.GetSingleValue().IsSet());
					bPassed &= TestTrue(*FString::Printf(TEXT("%s Proc2: BoundsMin is a CVR (no-setter, !bAutoInit)"), GatherLabel), Ranges.BoundsMinRange.GetSingleValue().IsSet());

					// Per-element values for all properties and all 4 elements.
					for (int32 i = 0; i < ExpectedPointCount; ++i)
					{
						// BoundsMax: constant (2.0, 0.75, 8.0) for all elements - CVR
						const FVector& BMax = Ranges.BoundsMaxRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc2: BoundsMax[%d].X (2.0)"), GatherLabel, i), BMax.X, 2.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc2: BoundsMax[%d].Y (0.75)"), GatherLabel, i), BMax.Y, 0.75, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc2: BoundsMax[%d].Z (8.0)"), GatherLabel, i), BMax.Z, 8.0, 0.01);

						// Color: R = i*0.25 (per-element), G=0, B=0, A=1
						const FVector4& C = Ranges.ColorRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc2: Color[%d].R (%g)"), GatherLabel, i, i * 0.25), (double)C.X, i * 0.25, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc2: Color[%d].G (0.0)"), GatherLabel, i), (double)C.Y, 0.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc2: Color[%d].B (0.0)"), GatherLabel, i), (double)C.Z, 0.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc2: Color[%d].A (1.0)"), GatherLabel, i), (double)C.W, 1.0, 0.01);

						// Steepness: constant 0.75 for all elements - CVR
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc2: Steepness[%d] (0.75)"), GatherLabel, i), (double)Ranges.SteepnessRange[i], 0.75, 0.01);

						// Seed: constant 42 for all elements - CVR
						bPassed &= TestEqual(*FString::Printf(TEXT("%s Proc2: Seed[%d] (42)"), GatherLabel, i), Ranges.SeedRange[i], 42);

						// Density: sentinel-encoded (always FullAlloc), explicit setter; written to 1.0 per element
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s Proc2: Density[%d] (1.0)"), GatherLabel, i), (double)Ranges.DensityRange[i], 1.0, 0.01);
					}
				}
			}

			bPassed &= TestTrue(*FString::Printf(TEXT("%s: Found Processor 1 output (identified by elimination)"), GatherLabel), bFoundProc1);
			bPassed &= TestTrue(*FString::Printf(TEXT("%s: Found Processor 2 output (BoundsMax CVR ~2.0)"), GatherLabel), bFoundProc2);

			return bPassed;
		};

	bool bTestPassed = true;
	bTestPassed &= ValidateGatherOutput(OutNodeData[0], TEXT("Gather1"));
	bTestPassed &= ValidateGatherOutput(OutNodeData[1], TEXT("Gather2"));
	return bTestPassed;
}

/**
 * GPU tests covering every branch of following decision logic.
 *
 * if (bAutoInitializeOutput)
 *     if (NumSettersFound == 0)
 *         if (IsGeneratorKernel())
 *             return true; // Use CVR for point defaults, unless Density, but i assume that is special cased outside (if not, could be in this function at top).
 *         else if (IsProcessorKernel())
 *             return bInputIsCVR; // Processor - follow whatever we receive in input.
 *         else
 *             return false; // Don't assume we can use CVR for other kernel types.
 *     else
 *         return false; // Always full allocate, even if there is a single setter with const expression, because that setter might change a subset of elements.
 * else // !bAutoInitializeOutput
 *     if (NumSettersFound == 0)
 *         return true; // If property is truly uninitialized, or is set via InitializePoint(), then it should just use a single value.
 *     else if (NumSettersFound == 1 && bAllSettersUseConstExpressions)
 *         return true; // If there is just one setter present that sets a const value, and the data is not auto initialized, then just use
 *                      // a single value. Don't distuingish whether that setter is in control flow or not, as uninitialized values not valuable.
 *     else
 *         return false; // Always allocate, even if there is a single setter with const expression, because that setter might change a subset of elements.
 *
 * BoundsMin is used as the test property throughout (not special-cased like Density, easy to set/verify).
 *
 * Generator sub-tests use a generator as the kernel-under-test.
 * Processor sub-tests pair an upstream generator with a downstream processor
 * to independently control the input CVR state and the processor setter behavior.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLUseCVRTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.UseCVR",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLUseCVRTest::RunTest(const FString& Parameters)
{
	TGuardConsoleVariable<bool> FuzzMemoryGuard(IConsoleManager::Get().FindConsoleVariable(TEXT("pcg.GPU.FuzzMemory")), /*bNewValue=*/true);

	bool bAllPassed = true;

	// --- Generator, bAutoInitializeOutput=true, no setter -> CVR ---
	// Generator with no BoundsMin setter; auto-init from point defaults keeps it as a single slot.
	{
		FPCGGPUGraphTestRunner Runner;

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
		HLSLSettings->NumElements = 4;
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = true;
		HLSLSettings->SetSourceText(TEXT("// No BoundsMin setter.\n"));

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Gen/bAutoInit/no-setter: readback is point data"), OutPoints))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					return TestTrue(TEXT("Gen/bAutoInit/no-setter: BoundsMin is CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Processor, bAutoInitializeOutput=true, CVR input, no setter -> CVR ---
	// CPU-produced point data (CreatePoints) always has GetAllocatedProperties()==All, so it cannot
	// supply CVR input to a processor. Instead we chain a GPU generator (no BoundsMin setter) in front:
	// that generator produces CVR BoundsMin (AllocatedPointProperties starts at None for unset properties).
	// The processor with bAutoInitializeOutput=true and no setter must inherit: output is also CVR.
	{
		FPCGGPUGraphTestRunner Runner;

		// Generator: no BoundsMin setter -> CVR output (AllocatedPointProperties stays None for BoundsMin).
		UPCGCustomHLSLSettings* GeneratorSettings = nullptr;
		UPCGNode* GeneratorNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GeneratorSettings);
		GeneratorSettings->SetKernelType(EPCGKernelType::PointGenerator);
		GeneratorSettings->NumElements = 4;
		GeneratorSettings->SetSourceText(TEXT("// No BoundsMin setter.\n"));

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = true;
		HLSLSettings->SetSourceText(TEXT("// No BoundsMin setter.\n"));

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(GeneratorNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Proc/bAutoInit/no-setter/CVR-in: readback is point data"), OutPoints))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					return TestTrue(TEXT("Proc/bAutoInit/no-setter/CVR-in: BoundsMin is CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Processor, bAutoInitializeOutput=true, FullAlloc input, no setter -> NOT CVR ---
	// CreatePoints with distinct BoundsMin per point forces FullAlloc (non-CVR) in the GPU upload.
	// Processor with bAutoInitializeOutput=true and no BoundsMin setter must follow input: not CVR.
	{
		FPCGGPUGraphTestRunner Runner;

		UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
		UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
		CreatePointsSettings->PointsToCreate.Empty();
		CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;
		for (int32 i = 0; i < 4; ++i)
		{
			FPCGPoint& Pt = CreatePointsSettings->PointsToCreate.AddDefaulted_GetRef();
			Pt.BoundsMin = FVector(i * -10.0, 0.0, 0.0); // Distinct per point -> non-CVR in GPU upload
		}

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = true;
		HLSLSettings->SetSourceText(TEXT("// No BoundsMin setter.\n"));

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Proc/bAutoInit/no-setter/FullAlloc-in: readback is point data"), OutPoints))
					{
						return false;
					}
					if (!TestEqual(TEXT("Proc/bAutoInit/no-setter/FullAlloc-in: output point count"), OutPoints->GetNumPoints(), 4))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					bool bPassed = TestFalse(TEXT("Proc/bAutoInit/no-setter/FullAlloc-in: FullAlloc input, no setter -> BoundsMin is NOT CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					for (int32 i = 0; i < 4; ++i)
					{
						const FVector B = Ranges.BoundsMinRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/bAutoInit/no-setter/FullAlloc-in: BoundsMin[%d].X"), i), B.X, i * -10.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/bAutoInit/no-setter/FullAlloc-in: BoundsMin[%d].Y"), i), B.Y, 0.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/bAutoInit/no-setter/FullAlloc-in: BoundsMin[%d].Z"), i), B.Z, 0.0, 0.01);
					}
					return bPassed;
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Processor, bAutoInitializeOutput=true, constant setter -> NOT CVR ---
	// Even a single constant-expression setter must force full allocation when bAutoInitializeOutput=true.
	// The input is CVR (empty generator with no BoundsMin setter) to isolate the effect of bAutoInitializeOutput:
	// the const setter alone should force FullAlloc regardless of input state.
	{
		FPCGGPUGraphTestRunner Runner;

		// Generator: no BoundsMin setter -> CVR output for BoundsMin.
		UPCGCustomHLSLSettings* GeneratorSettings = nullptr;
		UPCGNode* GeneratorNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GeneratorSettings);
		GeneratorSettings->SetKernelType(EPCGKernelType::PointGenerator);
		GeneratorSettings->NumElements = 4;
		GeneratorSettings->SetSourceText(TEXT("// No BoundsMin setter.\n"));

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = true;
		// Constant-expression setter. With !bAutoInitializeOutput a single const setter would produce CVR,
		// but with bAutoInitializeOutput=true any setter forces full allocation.
		HLSLSettings->SetSourceText(TEXT("Out_SetBoundsMin(Out_DataIndex, ElementIndex, float3(-0.5, 0.5, -0.25));\n"));

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(GeneratorNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Proc/bAutoInit/const-setter: readback is point data"), OutPoints))
					{
						return false;
					}
					if (!TestEqual(TEXT("Proc/bAutoInit/const-setter: output point count"), OutPoints->GetNumPoints(), 4))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					bool bPassed = TestFalse(TEXT("Proc/bAutoInit/const-setter: const setter -> BoundsMin is NOT CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					for (int32 i = 0; i < 4; ++i)
					{
						const FVector B = Ranges.BoundsMinRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/bAutoInit/const-setter: BoundsMin[%d].X"), i), B.X, -0.5, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/bAutoInit/const-setter: BoundsMin[%d].Y"), i), B.Y, 0.5, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/bAutoInit/const-setter: BoundsMin[%d].Z"), i), B.Z, -0.25, 0.01);
					}
					return bPassed;
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Generator, bAutoInitializeOutput=false, no setter -> CVR (uninitialized) ---
	// No setter and no auto-init: the property is uninitialized or set via InitializePoint(),
	// so a single slot covers all elements.
	{
		FPCGGPUGraphTestRunner Runner;

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
		HLSLSettings->NumElements = 4;
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = false;
		HLSLSettings->SetSourceText(TEXT("// No BoundsMin setter.\n"));

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Gen/!bAutoInit/no-setter: readback is point data"), OutPoints))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					bool bPassed = TestTrue(TEXT("Gen/!bAutoInit/no-setter: BoundsMin is CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					return bPassed;
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Processor, bAutoInitializeOutput=false, FullAlloc input, no setter -> CVR (uninitialized) ---
	// Even with non-CVR input, !bAutoInit + no setter compresses output to CVR.
	// The preamble does not run (bAutoInitializeOutput=false), so the single CVR slot holds uninitialized GPU memory.
	// All elements read that one uninitialized slot - the key assertion is that the output is CVR (1 slot), not the value.
	{
		FPCGGPUGraphTestRunner Runner;

		UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
		UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
		CreatePointsSettings->PointsToCreate.Empty();
		CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;
		for (int32 i = 0; i < 4; ++i)
		{
			FPCGPoint& Pt = CreatePointsSettings->PointsToCreate.AddDefaulted_GetRef();
			Pt.BoundsMin = FVector((i + 1) * -10.0, 0.0, 0.0); // Distinct per point -> non-CVR
		}

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = false;
		HLSLSettings->SetSourceText(TEXT("// No BoundsMin setter.\n"));

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Proc/!bAutoInit/no-setter/FullAlloc-in: readback is point data"), OutPoints))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					// Output must be CVR: no setter and !bAutoInitializeOutput compresses FullAlloc input to 1 slot.
					bool bPassed = TestTrue(TEXT("Proc/!bAutoInit/no-setter/FullAlloc-in: FullAlloc input, no setter -> BoundsMin is CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					return bPassed;
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Generator, bAutoInitializeOutput=false, single constant setter -> CVR ---
	// A single setter with a constant-expression value compresses the output to a single slot.
	{
		FPCGGPUGraphTestRunner Runner;

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
		HLSLSettings->NumElements = 4;
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = false;
		HLSLSettings->SetSourceText(TEXT("Out_SetBoundsMin(Out_DataIndex, ElementIndex, float3(-0.5, 0.5, -0.25));\n"));

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Gen/!bAutoInit/1-const-setter: readback is point data"), OutPoints))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					bool bPassed = TestTrue(TEXT("Gen/!bAutoInit/1-const-setter: BoundsMin is CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					if (Ranges.BoundsMinRange.GetSingleValue().IsSet())
					{
						const FVector B = Ranges.BoundsMinRange.GetSingleValue().GetValue();
						bPassed &= TestNearlyEqual(TEXT("Gen/!bAutoInit/1-const-setter: BoundsMin.X"), B.X, -0.5, 0.01);
						bPassed &= TestNearlyEqual(TEXT("Gen/!bAutoInit/1-const-setter: BoundsMin.Y"), B.Y, 0.5, 0.01);
						bPassed &= TestNearlyEqual(TEXT("Gen/!bAutoInit/1-const-setter: BoundsMin.Z"), B.Z, -0.25, 0.01);
					}
					return bPassed;
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Processor, bAutoInitializeOutput=false, single constant setter, FullAlloc input -> CVR ---
	// A constant setter compresses FullAlloc input to CVR even when the input had the property fully allocated.
	// The constant value is the same for every element, so 1 slot suffices.
	{
		FPCGGPUGraphTestRunner Runner;

		UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
		UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
		CreatePointsSettings->PointsToCreate.Empty();
		CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;
		for (int32 i = 0; i < 4; ++i)
		{
			FPCGPoint& Pt = CreatePointsSettings->PointsToCreate.AddDefaulted_GetRef();
			Pt.BoundsMin = FVector(i * -10.0, 0.0, 0.0); // Distinct per point -> non-CVR in GPU upload
		}

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = false;
		HLSLSettings->SetSourceText(TEXT("Out_SetBoundsMin(Out_DataIndex, ElementIndex, float3(-0.5, 0.5, -0.25));\n"));

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Proc/!bAutoInit/1-const-setter/FullAlloc-in: readback is point data"), OutPoints))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					bool bPassed = TestTrue(TEXT("Proc/!bAutoInit/1-const-setter/FullAlloc-in: FullAlloc input, 1 const setter -> BoundsMin is CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					if (Ranges.BoundsMinRange.GetSingleValue().IsSet())
					{
						const FVector B = Ranges.BoundsMinRange.GetSingleValue().GetValue();
						bPassed &= TestNearlyEqual(TEXT("Proc/!bAutoInit/1-const-setter/FullAlloc-in: BoundsMin.X"), B.X, -0.5, 0.01);
						bPassed &= TestNearlyEqual(TEXT("Proc/!bAutoInit/1-const-setter/FullAlloc-in: BoundsMin.Y"), B.Y, 0.5, 0.01);
						bPassed &= TestNearlyEqual(TEXT("Proc/!bAutoInit/1-const-setter/FullAlloc-in: BoundsMin.Z"), B.Z, -0.25, 0.01);
					}
					return bPassed;
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Generator, bAutoInitializeOutput=false, single non-constant setter -> NOT CVR ---
	// A single setter using a non-constant expression (ElementIndex-dependent) forces full allocation.
	{
		FPCGGPUGraphTestRunner Runner;

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
		HLSLSettings->NumElements = 4;
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = false;
		HLSLSettings->SetSourceText(
			TEXT("Out_SetBoundsMin(Out_DataIndex, ElementIndex, float3((float)ElementIndex * -10.0, 0.0, 0.0));\n")
			TEXT("Out_SetDensity(Out_DataIndex, ElementIndex, 1.0);\n")
		);

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Gen/!bAutoInit/non-const-setter: readback is point data"), OutPoints))
					{
						return false;
					}
					if (!TestEqual(TEXT("Gen/!bAutoInit/non-const-setter: output point count"), OutPoints->GetNumPoints(), 4))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					bool bPassed = TestFalse(TEXT("Gen/!bAutoInit/non-const-setter: non-const setter -> BoundsMin is NOT CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					for (int32 i = 0; i < 4; ++i)
					{
						const FVector B = Ranges.BoundsMinRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Gen/!bAutoInit/non-const-setter: BoundsMin[%d].X"), i), B.X, i * -10.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Gen/!bAutoInit/non-const-setter: BoundsMin[%d].Y"), i), B.Y, 0.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Gen/!bAutoInit/non-const-setter: BoundsMin[%d].Z"), i), B.Z, 0.0, 0.01);
					}
					return bPassed;
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Processor, bAutoInitializeOutput=false, non-constant setter, CVR input -> NOT CVR ---
	// A non-constant setter promotes CVR input to FullAlloc output.
	{
		FPCGGPUGraphTestRunner Runner;

		// Generator: no BoundsMin setter -> CVR output for BoundsMin.
		UPCGCustomHLSLSettings* GeneratorSettings = nullptr;
		UPCGNode* GeneratorNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GeneratorSettings);
		GeneratorSettings->SetKernelType(EPCGKernelType::PointGenerator);
		GeneratorSettings->NumElements = 4;
		GeneratorSettings->SetSourceText(TEXT("// No BoundsMin setter.\n"));

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = false;
		HLSLSettings->SetSourceText(
			TEXT("Out_SetBoundsMin(Out_DataIndex, ElementIndex, float3((float)ElementIndex * -10.0, 0.0, 0.0));\n")
			TEXT("Out_SetDensity(Out_DataIndex, ElementIndex, 1.0);\n")
		);

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(GeneratorNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Proc/!bAutoInit/non-const-setter/CVR-in: readback is point data"), OutPoints))
					{
						return false;
					}
					if (!TestEqual(TEXT("Proc/!bAutoInit/non-const-setter/CVR-in: output point count"), OutPoints->GetNumPoints(), 4))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					bool bPassed = TestFalse(TEXT("Proc/!bAutoInit/non-const-setter/CVR-in: non-const setter, CVR input -> BoundsMin is NOT CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					for (int32 i = 0; i < 4; ++i)
					{
						const FVector B = Ranges.BoundsMinRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/!bAutoInit/non-const-setter/CVR-in: BoundsMin[%d].X"), i), B.X, i * -10.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/!bAutoInit/non-const-setter/CVR-in: BoundsMin[%d].Y"), i), B.Y, 0.0, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/!bAutoInit/non-const-setter/CVR-in: BoundsMin[%d].Z"), i), B.Z, 0.0, 0.01);
					}
					return bPassed;
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Generator, bAutoInitializeOutput=false, multiple setters -> NOT CVR ---
	// Multiple setter calls for the same property force full allocation even when all values are constant.
	{
		FPCGGPUGraphTestRunner Runner;

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
		HLSLSettings->NumElements = 4;
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = false;
		// Two Out_SetBoundsMin calls: parser counts both, NumSettersFound == 2 -> full allocation.
		HLSLSettings->SetSourceText(
			TEXT("if (ElementIndex == 0u)\n")
			TEXT("    Out_SetBoundsMin(Out_DataIndex, ElementIndex, float3(-0.5, 0.5, -0.25));\n")
			TEXT("else\n")
			TEXT("    Out_SetBoundsMin(Out_DataIndex, ElementIndex, float3(-1.0, 1.0, -0.5));\n")
			TEXT("Out_SetDensity(Out_DataIndex, ElementIndex, 1.0);\n")
		);

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Gen/!bAutoInit/multi-setter: readback is point data"), OutPoints))
					{
						return false;
					}
					if (!TestEqual(TEXT("Gen/!bAutoInit/multi-setter: output point count"), OutPoints->GetNumPoints(), 4))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					bool bPassed = TestFalse(TEXT("Gen/!bAutoInit/multi-setter: multiple setters -> BoundsMin is NOT CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					// Element 0 takes the if-branch (-0.5, 0.5, -0.25); elements 1-3 take the else-branch (-1.0, 1.0, -0.5).
					const FVector ExpectedB0(-0.5, 0.5, -0.25);
					const FVector ExpectedBRest(-1.0, 1.0, -0.5);
					for (int32 i = 0; i < 4; ++i)
					{
						const FVector Expected = (i == 0) ? ExpectedB0 : ExpectedBRest;
						const FVector B = Ranges.BoundsMinRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Gen/!bAutoInit/multi-setter: BoundsMin[%d].X"), i), B.X, Expected.X, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Gen/!bAutoInit/multi-setter: BoundsMin[%d].Y"), i), B.Y, Expected.Y, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Gen/!bAutoInit/multi-setter: BoundsMin[%d].Z"), i), B.Z, Expected.Z, 0.01);
					}
					return bPassed;
				});
		}
		bAllPassed &= bSubPassed;
	}

	// --- Processor, bAutoInitializeOutput=false, multiple setters, CVR input -> NOT CVR ---
	// Multiple setter calls promote CVR input to FullAlloc output.
	{
		FPCGGPUGraphTestRunner Runner;

		// Generator: no BoundsMin setter -> CVR output for BoundsMin.
		UPCGCustomHLSLSettings* GeneratorSettings = nullptr;
		UPCGNode* GeneratorNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GeneratorSettings);
		GeneratorSettings->SetKernelType(EPCGKernelType::PointGenerator);
		GeneratorSettings->NumElements = 4;
		GeneratorSettings->SetSourceText(TEXT("// No BoundsMin setter.\n"));

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
		check(HLSLSettings->OutputPins.Num() == 1);
		HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = false;
		// Two Out_SetBoundsMin calls: parser counts both, NumSettersFound == 2 -> full allocation.
		HLSLSettings->SetSourceText(
			TEXT("if (ElementIndex == 0u)\n")
			TEXT("    Out_SetBoundsMin(Out_DataIndex, ElementIndex, float3(-0.5, 0.5, -0.25));\n")
			TEXT("else\n")
			TEXT("    Out_SetBoundsMin(Out_DataIndex, ElementIndex, float3(-1.0, 1.0, -0.5));\n")
			TEXT("Out_SetDensity(Out_DataIndex, ElementIndex, 1.0);\n")
		);

		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
		Runner.Graph->AddEdge(GeneratorNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
		Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		TArray<FPCGDataCollection> OutNodeData;
		bool bSubPassed = Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
		if (bSubPassed)
		{
			bSubPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
				{
					const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
					if (!TestNotNull(TEXT("Proc/!bAutoInit/multi-setter/CVR-in: readback is point data"), OutPoints))
					{
						return false;
					}
					if (!TestEqual(TEXT("Proc/!bAutoInit/multi-setter/CVR-in: output point count"), OutPoints->GetNumPoints(), 4))
					{
						return false;
					}
					const FConstPCGPointValueRanges Ranges(OutPoints);
					bool bPassed = TestFalse(TEXT("Proc/!bAutoInit/multi-setter/CVR-in: multiple setters, CVR input -> BoundsMin is NOT CVR"), Ranges.BoundsMinRange.GetSingleValue().IsSet());
					// Element 0 takes the if-branch (-0.5, 0.5, -0.25); elements 1-3 take the else-branch (-1.0, 1.0, -0.5).
					const FVector ExpectedB0(-0.5, 0.5, -0.25);
					const FVector ExpectedBRest(-1.0, 1.0, -0.5);
					for (int32 i = 0; i < 4; ++i)
					{
						const FVector Expected = (i == 0) ? ExpectedB0 : ExpectedBRest;
						const FVector B = Ranges.BoundsMinRange[i];
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/!bAutoInit/multi-setter/CVR-in: BoundsMin[%d].X"), i), B.X, Expected.X, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/!bAutoInit/multi-setter/CVR-in: BoundsMin[%d].Y"), i), B.Y, Expected.Y, 0.01);
						bPassed &= TestNearlyEqual(*FString::Printf(TEXT("Proc/!bAutoInit/multi-setter/CVR-in: BoundsMin[%d].Z"), i), B.Z, Expected.Z, 0.01);
					}
					return bPassed;
				});
		}
		bAllPassed &= bSubPassed;
	}

	return bAllPassed;
}

#endif // WITH_EDITOR
