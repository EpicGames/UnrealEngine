// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "Compute/Data/PCGRawBufferData.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Elements/PCGGather.h"

namespace PCGCustomHLSLRawBufferTestHelpers
{
	/** Configure a CustomHLSL node as a Custom kernel with a single raw buffer output pin and fixed element count. */
	void SetupRawBufferOutputNode(UPCGCustomHLSLSettings* Settings, int32 NumElements)
	{
		Settings->SetKernelType(EPCGKernelType::Custom);
		Settings->DispatchThreadCount = EPCGDispatchThreadCount::Fixed;
		Settings->FixedThreadCount = NumElements;
		Settings->InputPins.Empty();
		Settings->OutputPins = { FPCGPinPropertiesGPU(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeInfoRawBuffer::AsId()) };
		Settings->OutputPins[0].PropertiesGPU.InitializationMode = EPCGPinInitMode::Custom;
		Settings->OutputPins[0].PropertiesGPU.ElementCount = NumElements;
	}

	/** Validate readback data as UPCGRawBufferData with expected uint32 values. */
	bool ValidateRawBufferReadback(FAutomationTestBase* Test, const FPCGDataCollection& DataCollection, const TArray<uint32>& ExpectedValues)
	{
		return PCGTestsCommon::ValidateSingleTaggedData(Test, DataCollection, [&](const FPCGTaggedData& TaggedData)
		{
			const UPCGRawBufferData* RawBuffer = Cast<UPCGRawBufferData>(TaggedData.Data);
			if (!Test->TestNotNull(TEXT("Readback data is raw buffer data"), RawBuffer))
			{
				return false;
			}

			const TArray<uint32>& ActualData = RawBuffer->GetConstData();

			if (!Test->TestEqual(TEXT("Raw buffer element count"), ActualData.Num(), ExpectedValues.Num()))
			{
				return false;
			}

			bool bPassed = true;
			for (int32 i = 0; i < ExpectedValues.Num(); ++i)
			{
				bPassed &= Test->TestEqual(
					*FString::Printf(TEXT("Element[%d] (expected %u, got %u)"), i, ExpectedValues[i], ActualData[i]),
					ActualData[i], ExpectedValues[i]);
			}
			return bPassed;
		});
	}
}

/**
 * GPU test: CustomHLSL Custom kernel writes a raw buffer, reads it back via Out_Load, increments, and writes again.
 *
 *   CustomHLSL (Store -> Out_Load -> +1 -> Store) -> Gather
 *
 * Dispatches 8 threads. Each thread N writes its own slot, reads it back, and writes again -
 * UAV access within a lane is program-ordered so no barrier is needed.
 * Exercises both write (Out_Store) and read (Out_Load) on a raw buffer output pin.
 * Verifies readback contains [2, 12, 22, 32, 42, 52, 62, 72].
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLRawBufferStoreLoadStoreTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.RawBuffer.StoreLoadStore",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLRawBufferStoreLoadStoreTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	constexpr int32 NumElements = 8;

	UPCGCustomHLSLSettings* HLSLSettings = nullptr;
	UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
	PCGCustomHLSLRawBufferTestHelpers::SetupRawBufferOutputNode(HLSLSettings, NumElements);
	HLSLSettings->SetSourceText(
		TEXT("Out_Store(ThreadIndex, ThreadIndex * 10u + 1u);\n")
		TEXT("uint Value = Out_Load(ThreadIndex);\n")
		TEXT("Out_Store(ThreadIndex, Value + 1u);\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	TArray<uint32> ExpectedValues;
	for (int32 i = 0; i < NumElements; ++i)
	{
		ExpectedValues.Add(i * 10 + 2);
	}

	return PCGCustomHLSLRawBufferTestHelpers::ValidateRawBufferReadback(this, OutNodeData[0], ExpectedValues);
}

/**
 * GPU test: CustomHLSL Custom kernel writes via Store4 and reads back.
 *
 *   CustomHLSL (Store4: 4 sequential values per thread) -> Gather
 *
 * Dispatches 2 threads, each writing 4 uint32s via Store4. Thread 0 writes [100,101,102,103]
 * at offset 0; thread 1 writes [200,201,202,203] at offset 4.
 * Verifies readback contains [100, 101, 102, 103, 200, 201, 202, 203].
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLRawBufferStore4Test,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.RawBuffer.Store4",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLRawBufferStore4Test::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	// 2 threads, each writes 4 elements = 8 total elements in the buffer.
	constexpr int32 NumThreads = 2;
	constexpr int32 NumElements = NumThreads * 4;

	UPCGCustomHLSLSettings* HLSLSettings = nullptr;
	UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
	PCGCustomHLSLRawBufferTestHelpers::SetupRawBufferOutputNode(HLSLSettings, NumElements);
	HLSLSettings->FixedThreadCount = NumThreads;
	HLSLSettings->SetSourceText(
		TEXT("uint Base = (ThreadIndex + 1u) * 100u;\n")
		TEXT("Out_Store4(ThreadIndex * 4u, uint4(Base, Base + 1u, Base + 2u, Base + 3u));\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	const TArray<uint32> ExpectedValues = { 100, 101, 102, 103, 200, 201, 202, 203 };
	return PCGCustomHLSLRawBufferTestHelpers::ValidateRawBufferReadback(this, OutNodeData[0], ExpectedValues);
}

/**
 * GPU test: Two chained CustomHLSL nodes pass raw buffer data from output to input.
 *
 *   CustomHLSL_A (Store known values) -> CustomHLSL_B (Load, double, Store) -> Gather
 *
 * Node A writes [10, 20, 30, 40] to a raw buffer. Node B reads each value via Load,
 * doubles it, and writes to its own raw buffer output via Store.
 * Verifies readback from node B contains [20, 40, 60, 80].
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLRawBufferChainTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.RawBuffer.Chain",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLRawBufferChainTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	constexpr int32 NumElements = 4;

	// Node A: write known values to raw buffer output.
	UPCGCustomHLSLSettings* SettingsA = nullptr;
	UPCGNode* NodeA = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(SettingsA);
	PCGCustomHLSLRawBufferTestHelpers::SetupRawBufferOutputNode(SettingsA, NumElements);
	SettingsA->SetSourceText(
		TEXT("Out_Store(ThreadIndex, (ThreadIndex + 1u) * 10u);\n")
	);

	// Node B: read from raw buffer input, double the value, write to raw buffer output.
	UPCGCustomHLSLSettings* SettingsB = nullptr;
	UPCGNode* NodeB = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(SettingsB);
	SettingsB->SetKernelType(EPCGKernelType::Custom);
	SettingsB->DispatchThreadCount = EPCGDispatchThreadCount::Fixed;
	SettingsB->FixedThreadCount = NumElements;

	// Input pin: raw buffer.
	SettingsB->InputPins = { FPCGPinProperties(PCGPinConstants::DefaultInputLabel, FPCGDataTypeInfoRawBuffer::AsId()) };

	// Output pin: raw buffer, initialized from the input pin.
	SettingsB->OutputPins = { FPCGPinPropertiesGPU(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeInfoRawBuffer::AsId()) };
	SettingsB->OutputPins[0].PropertiesGPU.InitializationMode = EPCGPinInitMode::FromInputPins;
	SettingsB->OutputPins[0].PropertiesGPU.PinsToInititalizeFrom = { PCGPinConstants::DefaultInputLabel };

	SettingsB->SetSourceText(
		TEXT("uint Value = In_Load(ThreadIndex);\n")
		TEXT("Out_Store(ThreadIndex, Value * 2u);\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(NodeA, PCGPinConstants::DefaultOutputLabel, NodeB, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(NodeB, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	const TArray<uint32> ExpectedValues = { 20, 40, 60, 80 };
	return PCGCustomHLSLRawBufferTestHelpers::ValidateRawBufferReadback(this, OutNodeData[0], ExpectedValues);
}

/**
 * GPU test: storing to a raw buffer INPUT pin must produce an error. The raw buffer DI
 * registers Store only on its output (UAV) side, so `In_Store` is never wrapped and
 * appears in the user's HLSL as an undeclared identifier - the shader compiler must reject it.
 *
 *   CustomHLSL_A (Store known values) -> CustomHLSL_B (Out_Store + In_Store: shader compile error expected) -> Gather
 *
 * Node B writes to its output via Out_Store (so PCG's "output uninitialized" static check
 * does NOT fire) AND attempts to write to its input via In_Store. The Out_Store keeps the
 * static analysis happy; the In_Store then has to be caught by shader compilation.
 *
 * If no error fires, In_Store is being silently dropped somewhere in the pipeline - that
 * itself is a bug that this test must catch.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLRawBufferInputStoreErrorTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.RawBuffer.InputStoreError",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLRawBufferInputStoreErrorTest::RunTest(const FString& Parameters)
{
	// HLSL compiler must reject In_Store as undeclared. Match the substring common to DXC ("use of undeclared identifier 'X'")
	// and FXC ("X3004: undeclared identifier 'X'").
	AddExpectedError(TEXT("undeclared identifier 'In_Store'"), EAutomationExpectedErrorFlags::Contains, /*Occurrences=*/0, /*bIsRegex=*/false);

	FPCGGPUGraphTestRunner Runner;

	constexpr int32 NumElements = 4;

	// Node A: produce raw buffer data to feed Node B's input pin.
	UPCGCustomHLSLSettings* SettingsA = nullptr;
	UPCGNode* NodeA = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(SettingsA);
	PCGCustomHLSLRawBufferTestHelpers::SetupRawBufferOutputNode(SettingsA, NumElements);
	SettingsA->SetSourceText(
		TEXT("Out_Store(ThreadIndex, ThreadIndex);\n")
	);

	// Node B: writes to its output (keeps the "uninitialized output" static check happy)
	// AND attempts to write to its input via In_Store - the latter must error out.
	UPCGCustomHLSLSettings* SettingsB = nullptr;
	UPCGNode* NodeB = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(SettingsB);
	SettingsB->SetKernelType(EPCGKernelType::Custom);
	SettingsB->DispatchThreadCount = EPCGDispatchThreadCount::Fixed;
	SettingsB->FixedThreadCount = NumElements;
	SettingsB->InputPins = { FPCGPinProperties(PCGPinConstants::DefaultInputLabel, FPCGDataTypeInfoRawBuffer::AsId()) };
	SettingsB->OutputPins = { FPCGPinPropertiesGPU(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeInfoRawBuffer::AsId()) };
	SettingsB->OutputPins[0].PropertiesGPU.InitializationMode = EPCGPinInitMode::FromInputPins;
	SettingsB->OutputPins[0].PropertiesGPU.PinsToInititalizeFrom = { PCGPinConstants::DefaultInputLabel };
	SettingsB->SetSourceText(
		TEXT("Out_Store(ThreadIndex, 100u);\n")
		TEXT("In_Store(ThreadIndex, 42u);\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(NodeA, PCGPinConstants::DefaultOutputLabel, NodeB, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(NodeB, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	// Execute with the error expected - return value intentionally ignored. The test framework
	// passes if the registered expected error fires; it fails if the error never appears.
	TArray<FPCGDataCollection> OutNodeData;
	Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);

	return true;
}

#endif // WITH_EDITOR
