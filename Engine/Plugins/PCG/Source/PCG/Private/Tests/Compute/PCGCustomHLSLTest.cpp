// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGCreatePoints.h"
#include "Elements/PCGFilterByIndex.h"
#include "Elements/PCGGather.h"
#include "Helpers/PCGHelpers.h"

/**
 * GPU test: CustomHLSL PointProcessor translates point positions.
 *
 *   CreatePoints (4 known positions) -> CustomHLSL (X += 10, Y += 25, Z += 50) -> Gather
 *
 * Verifies each output point's position == input position + offset, using a different
 * offset per axis to confirm all three components are written correctly.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLPointTranslateTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.PointTranslate",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLPointTranslateTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	// CreatePoints: 4 points at known positions.
	UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
	UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
	CreatePointsSettings->PointsToCreate.Empty();

	const TArray<FVector> InputPositions = {
		FVector(  0.f,   0.f,   0.f),
		FVector(100.f, 200.f, 300.f),
		FVector(-50.f,  75.f, 250.f),
		FVector( 10.f, -30.f, -80.f),
	};
	for (const FVector& Pos : InputPositions)
	{
		FPCGPoint& Pt = CreatePointsSettings->PointsToCreate.AddDefaulted_GetRef();
		Pt.Transform.SetLocation(Pos);
	}
	CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;

	// CustomHLSL PointProcessor: translate by a different amount per axis.
	UPCGCustomHLSLSettings* HLSLSettings = nullptr;
	UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
	HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
	HLSLSettings->SetSourceText(
		TEXT("float3 Pos = In_GetPosition(In_DataIndex, ElementIndex);\n")
		TEXT("Pos.x += 10.0f;\n")
		TEXT("Pos.y += 25.0f;\n")
		TEXT("Pos.z += 50.0f;\n")
		TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, Pos);\n")
	);

	// Useful to add gather node to grab output data from.
	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
	{
		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
		{
			return false;
		}

		bool bTestPassed = TestEqual(TEXT("Output point count matches input"), OutPoints->GetNumPoints(), InputPositions.Num());

		if (OutPoints->GetNumPoints() == InputPositions.Num())
		{
			const FConstPCGPointValueRanges Ranges(OutPoints);
			for (int32 i = 0; i < InputPositions.Num(); ++i)
			{
				const FVector ExpectedPos = InputPositions[i] + FVector(10.0, 25.0, 50.0);
				const FVector ActualPos = Ranges.TransformRange[i].GetLocation();
				bTestPassed &= TestNearlyEqual(*FString::Printf(TEXT("Point[%d] X (expected %.1f, got %.1f)"), i, ExpectedPos.X, ActualPos.X), ActualPos.X, ExpectedPos.X, 0.1);
				bTestPassed &= TestNearlyEqual(*FString::Printf(TEXT("Point[%d] Y (expected %.1f, got %.1f)"), i, ExpectedPos.Y, ActualPos.Y), ActualPos.Y, ExpectedPos.Y, 0.1);
				bTestPassed &= TestNearlyEqual(*FString::Printf(TEXT("Point[%d] Z (expected %.1f, got %.1f)"), i, ExpectedPos.Z, ActualPos.Z), ActualPos.Z, ExpectedPos.Z, 0.1);
			}
		}

		return bTestPassed;
	});
}

/**
 * GPU test: Two chained CustomHLSL PointProcessor nodes apply different transforms.
 *
 *   CreatePoints (4 points) -> CustomHLSL (set scale to (2,3,4)) -> CustomHLSL (rotate i*90 deg around Z) -> Gather
 *
 * Verifies the output scale (set by node 1 and carried through node 2) and per-point rotation
 * (0, 90, 180, 270 deg around Z set by node 2), confirming each node in the chain sees the
 * previous node's output data.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLChainedPointProcessorsTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.ChainedPointProcessors",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLChainedPointProcessorsTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	// CreatePoints: 4 default points.
	UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
	UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
	CreatePointsSettings->PointsToCreate.Empty();
	for (int32 i = 0; i < 4; ++i)
	{
		CreatePointsSettings->PointsToCreate.AddDefaulted();
	}
	CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;

	// First HLSL node: set non-uniform scale.
	UPCGCustomHLSLSettings* HLSLSettings1 = nullptr;
	UPCGNode* HLSLNode1 = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings1);
	HLSLSettings1->SetKernelType(EPCGKernelType::PointProcessor);
	HLSLSettings1->SetSourceText(
		TEXT("Out_SetScale(Out_DataIndex, ElementIndex, float3(2.0f, 3.0f, 4.0f));\n")
	);

	// Second HLSL node: rotate each point by a different angle around Z (0, 90, 180, 270 deg).
	// Scale from node 1 should be carried through.
	UPCGCustomHLSLSettings* HLSLSettings2 = nullptr;
	UPCGNode* HLSLNode2 = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings2);
	HLSLSettings2->SetKernelType(EPCGKernelType::PointProcessor);
	HLSLSettings2->SetSourceText(
		TEXT("FQuat Rot = Quat_FromAxisAngle(half3(0.0f, 0.0f, 1.0f), ElementIndex * 1.5707963f);\n")
		TEXT("Out_SetRotation(Out_DataIndex, ElementIndex, Rot);\n")
	);

	// Useful to add gather node to grab output data from.
	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode1, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode1,        PCGPinConstants::DefaultOutputLabel, HLSLNode2, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode2,        PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
	{
		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
		{
			return false;
		}

		constexpr int32 ExpectedPointCount = 4;
		bool bTestPassed = TestEqual(TEXT("Output point count matches input"), OutPoints->GetNumPoints(), ExpectedPointCount);

		if (OutPoints->GetNumPoints() == ExpectedPointCount)
		{
			const FVector ExpectedScale(2.f, 3.f, 4.f);

			const FConstPCGPointValueRanges Ranges(OutPoints);
			for (int32 i = 0; i < ExpectedPointCount; ++i)
			{
				const FVector ActualScale = Ranges.TransformRange[i].GetScale3D();
				bTestPassed &= TestNearlyEqual(*FString::Printf(TEXT("Point[%d] Scale X (expected %.1f, got %.1f)"), i, ExpectedScale.X, ActualScale.X), ActualScale.X, ExpectedScale.X, 0.01);
				bTestPassed &= TestNearlyEqual(*FString::Printf(TEXT("Point[%d] Scale Y (expected %.1f, got %.1f)"), i, ExpectedScale.Y, ActualScale.Y), ActualScale.Y, ExpectedScale.Y, 0.01);
				bTestPassed &= TestNearlyEqual(*FString::Printf(TEXT("Point[%d] Scale Z (expected %.1f, got %.1f)"), i, ExpectedScale.Z, ActualScale.Z), ActualScale.Z, ExpectedScale.Z, 0.01);

				const FQuat ExpectedRot(FVector::ZAxisVector, FMath::DegreesToRadians(90.f * i));
				const FQuat ActualRot = Ranges.TransformRange[i].GetRotation();
				bTestPassed &= TestTrue(*FString::Printf(TEXT("Point[%d] rotation matches expected %d deg around Z"), i, 90 * i), ActualRot.Equals(ExpectedRot, 0.01f));
			}
		}

		return bTestPassed;
	});
}

/**
 * GPU test: CustomHLSL PointProcessor with pin labels containing spaces.
 *
 *   CreatePoints (4 known positions) -> CustomHLSL (X += 100) -> Gather
 *
 * The CustomHLSL node uses "Input Points" as input pin label and "Output Points" as output pin label.
 * The graph compiler sanitizes these to "Input_Points" and "Output_Points" for HLSL code generation.
 * The HLSL source references the sanitized names. Verifies the end-to-end pipeline produces
 * correct output, confirming that pin label sanitization works in the compiler.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLSanitizedPinLabelsTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.SanitizedPinLabels",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLSanitizedPinLabelsTest::RunTest(const FString& Parameters)
{
	// Verify our test labels actually need sanitization.
	const FName OriginalInputLabel(TEXT("Input Points"));
	const FName OriginalOutputLabel(TEXT("Output Points"));
	TestFalse(TEXT("Input label requires sanitization"), PCGComputeHelpers::IsValidHLSLPinLabel(OriginalInputLabel));
	TestFalse(TEXT("Output label requires sanitization"), PCGComputeHelpers::IsValidHLSLPinLabel(OriginalOutputLabel));

	// Sanitize - mimics what PostEditChangeProperty does in the editor when a user types a label with spaces.
	const FName SanitizedInputLabel(PCGComputeHelpers::SanitizePinLabelForHLSL(OriginalInputLabel));
	const FName SanitizedOutputLabel(PCGComputeHelpers::SanitizePinLabelForHLSL(OriginalOutputLabel));
	TestEqual(TEXT("Input label sanitized"), SanitizedInputLabel, FName(TEXT("Input_Points")));
	TestEqual(TEXT("Output label sanitized"), SanitizedOutputLabel, FName(TEXT("Output_Points")));

	FPCGGPUGraphTestRunner Runner;

	// CreatePoints: 4 points at known positions.
	UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
	UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
	CreatePointsSettings->PointsToCreate.Empty();

	const TArray<FVector> InputPositions = {
		FVector(  0.f,   0.f,   0.f),
		FVector(100.f, 200.f, 300.f),
		FVector(-50.f,  75.f, 250.f),
		FVector( 10.f, -30.f, -80.f),
	};
	for (const FVector& Pos : InputPositions)
	{
		FPCGPoint& Pt = CreatePointsSettings->PointsToCreate.AddDefaulted_GetRef();
		Pt.Transform.SetLocation(Pos);
	}
	CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;

	// CustomHLSL PointProcessor with sanitized pin labels (spaces replaced with underscores).
	UPCGCustomHLSLSettings* HLSLSettings = nullptr;
	UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
	HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);

	// Use the sanitized labels on the pins, same as the editor would after PostEditChangeProperty.
	HLSLSettings->InputPins = { FPCGPinProperties(SanitizedInputLabel, FPCGDataTypeIdentifier{EPCGDataType::Point}) };
	HLSLSettings->OutputPins = { FPCGPinPropertiesGPU(SanitizedOutputLabel, FPCGDataTypeIdentifier{EPCGDataType::Point}) };

	HLSLSettings->SetSourceText(
		TEXT("float3 Pos = Input_Points_GetPosition(Input_Points_DataIndex, ElementIndex);\n")
		TEXT("Pos.x += 100.0f;\n")
		TEXT("Output_Points_SetPosition(Output_Points_DataIndex, ElementIndex, Pos);\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, SanitizedInputLabel);
	Runner.Graph->AddEdge(HLSLNode, SanitizedOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
	{
		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
		{
			return false;
		}

		bool bTestPassed = TestEqual(TEXT("Output point count matches input"), OutPoints->GetNumPoints(), InputPositions.Num());

		if (OutPoints->GetNumPoints() == InputPositions.Num())
		{
			const FConstPCGPointValueRanges Ranges(OutPoints);
			for (int32 i = 0; i < InputPositions.Num(); ++i)
			{
				const FVector ExpectedPos = InputPositions[i] + FVector(100.0, 0.0, 0.0);
				const FVector ActualPos = Ranges.TransformRange[i].GetLocation();
				bTestPassed &= TestNearlyEqual(*FString::Printf(TEXT("Point[%d] X (expected %.1f, got %.1f)"), i, ExpectedPos.X, ActualPos.X), ActualPos.X, ExpectedPos.X, 0.1);
				bTestPassed &= TestNearlyEqual(*FString::Printf(TEXT("Point[%d] Y (expected %.1f, got %.1f)"), i, ExpectedPos.Y, ActualPos.Y), ActualPos.Y, ExpectedPos.Y, 0.1);
				bTestPassed &= TestNearlyEqual(*FString::Printf(TEXT("Point[%d] Z (expected %.1f, got %.1f)"), i, ExpectedPos.Z, ActualPos.Z), ActualPos.Z, ExpectedPos.Z, 0.1);
			}
		}

		return bTestPassed;
	});
}

/**
 * GPU test: CustomHLSL PointGenerator with no property writes produces correct defaults.
 *
 *   CustomHLSL PointGenerator (4 points, empty body) -> Gather
 *
 * pcg.GPU.FuzzMemory=true pre-fills GPU buffers with random noise so the test catches any property
 * that the preamble fails to initialise (a zero-filled buffer would pass silently).
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLGeneratorDefaultInitTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.GeneratorDefaultInit",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLGeneratorDefaultInitTest::RunTest(const FString& Parameters)
{
    TGuardConsoleVariable<bool> FuzzMemoryGuard(IConsoleManager::Get().FindConsoleVariable(TEXT("pcg.GPU.FuzzMemory")), /*bNewValue=*/true);

    bool bAllPassed = true;

    {
        FPCGGPUGraphTestRunner Runner;

        UPCGCustomHLSLSettings* HLSLSettings = nullptr;
        UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
        // Use SetKernelType (not direct assignment) - it empties InputPins for generators so that
        // IsInputPinRequiredByExecution=true doesn't block graph execution on the dangling default pin.
        HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
        HLSLSettings->NumElements = 4;
        HLSLSettings->SetSourceText(TEXT("// No property writes - init preamble handles all defaults.\n"));

        UPCGGatherSettings* GatherSettings = nullptr;
        UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
        Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

        TArray<FPCGDataCollection> OutNodeData;
        bAllPassed &= Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
        if (bAllPassed)
        {
            bAllPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
            {
                const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
                if (!TestNotNull(TEXT("SmartInit: readback data is point data"), OutPoints))
                {
                    return false;
                }

                constexpr int32 ExpectedPointCount = 4;
                if (!TestEqual(TEXT("SmartInit: output point count"), OutPoints->GetNumPoints(), ExpectedPointCount))
                {
                    return false;
                }

                const FConstPCGPointValueRanges Ranges(OutPoints);
                bool bPassed = true;

                // Every unwritten property should be CVR (backed by 1 slot, not N).
                bPassed &= TestTrue(TEXT("DefaultInit: Transform is a constant value range"),  Ranges.TransformRange.GetSingleValue().IsSet());
                bPassed &= TestTrue(TEXT("DefaultInit: BoundsMin is a constant value range"),  Ranges.BoundsMinRange.GetSingleValue().IsSet());
                bPassed &= TestTrue(TEXT("DefaultInit: BoundsMax is a constant value range"),  Ranges.BoundsMaxRange.GetSingleValue().IsSet());
                bPassed &= TestTrue(TEXT("DefaultInit: Color is a constant value range"),      Ranges.ColorRange.GetSingleValue().IsSet());
                // Density is always fully allocated (never a CVR), so no CVR assertion here.
                bPassed &= TestTrue(TEXT("DefaultInit: Steepness is a constant value range"),  Ranges.SteepnessRange.GetSingleValue().IsSet());
                bPassed &= TestTrue(TEXT("DefaultInit: Seed is a constant value range"),       Ranges.SeedRange.GetSingleValue().IsSet());

                // Verify default values. Most properties use FPCGPoint{} defaults directly.
                // BoundsMin/BoundsMax are hardcoded: the GPU preamble initialises them to -50/50,
                // which differs from the FPCGPoint C++ defaults.
                const FPCGPoint DefaultPoint;
                if (Ranges.TransformRange.GetSingleValue().IsSet())
                {
                    const FTransform& T = Ranges.TransformRange.GetSingleValue().GetValue();
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default position X"), T.GetLocation().X, DefaultPoint.Transform.GetLocation().X, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default position Y"), T.GetLocation().Y, DefaultPoint.Transform.GetLocation().Y, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default position Z"), T.GetLocation().Z, DefaultPoint.Transform.GetLocation().Z, 0.01);
                    bPassed &= TestTrue(TEXT("DefaultInit: default rotation is identity"), T.GetRotation().Equals(DefaultPoint.Transform.GetRotation(), 0.01f));
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default scale X"), T.GetScale3D().X, DefaultPoint.Transform.GetScale3D().X, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default scale Y"), T.GetScale3D().Y, DefaultPoint.Transform.GetScale3D().Y, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default scale Z"), T.GetScale3D().Z, DefaultPoint.Transform.GetScale3D().Z, 0.01);
                }
                if (Ranges.BoundsMinRange.GetSingleValue().IsSet())
                {
                    const FVector& V = Ranges.BoundsMinRange.GetSingleValue().GetValue();
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default BoundsMin X is -50"), V.X, -50.0, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default BoundsMin Y is -50"), V.Y, -50.0, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default BoundsMin Z is -50"), V.Z, -50.0, 0.01);
                }
                if (Ranges.BoundsMaxRange.GetSingleValue().IsSet())
                {
                    const FVector& V = Ranges.BoundsMaxRange.GetSingleValue().GetValue();
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default BoundsMax X is 50"), V.X, 50.0, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default BoundsMax Y is 50"), V.Y, 50.0, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default BoundsMax Z is 50"), V.Z, 50.0, 0.01);
                }
                if (Ranges.ColorRange.GetSingleValue().IsSet())
                {
                    const FVector4& C = Ranges.ColorRange.GetSingleValue().GetValue();
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default Color R"), (double)C.X, (double)DefaultPoint.Color.X, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default Color G"), (double)C.Y, (double)DefaultPoint.Color.Y, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default Color B"), (double)C.Z, (double)DefaultPoint.Color.Z, 0.01);
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default Color A"), (double)C.W, (double)DefaultPoint.Color.W, 0.01);
                }
                if (Ranges.DensityRange.GetSingleValue().IsSet())
                {
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default Density"), (double)Ranges.DensityRange.GetSingleValue().GetValue(), (double)DefaultPoint.Density, 0.01);
                }
                if (Ranges.SteepnessRange.GetSingleValue().IsSet())
                {
                    bPassed &= TestNearlyEqual(TEXT("DefaultInit: default Steepness"), (double)Ranges.SteepnessRange.GetSingleValue().GetValue(), (double)DefaultPoint.Steepness, 0.01);
                }
                if (Ranges.SeedRange.GetSingleValue().IsSet())
                {
                    bPassed &= TestEqual(TEXT("DefaultInit: default Seed"), Ranges.SeedRange.GetSingleValue().GetValue(), PCGValueConstants::DefaultSeed);
                }
                return bPassed;
            });
        }
    }

    return bAllPassed;
}

/**
 * GPU test: CustomHLSL PointGenerator writing only position - rotation and scale receive defaults.
 *
 *   CustomHLSL PointGenerator (4 points, sets only SetPosition) -> Gather
 *
 * Covers two scenarios to exercise TRS partial-write paths:
 *   - Constant position (fully allocated): position is a constant expression but bAutoInitializeOutput=true
 *     forces a fully-allocated (N-slot) Transform buffer. Rotation and scale must be initialised for ALL
 *     elements by the preamble. Regression: preamble must include constant-setter bits in AllocProps so
 *     R/S are written per-element, not just at element 0.
 *   - Per-element position (fully allocated): position varies per element so the Transform buffer
 *     is fully allocated (N slots). Rotation and scale must be initialised for ALL elements, not
 *     just element 0. Regression test - element-0-only writes are not enough here.
 *
 * pcg.GPU.FuzzMemory=true ensures garbage noise in the buffer surfaces any missing initialisation.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLGeneratorPartialTRSInitTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.GeneratorPartialTRSInit",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLGeneratorPartialTRSInitTest::RunTest(const FString& Parameters)
{
    TGuardConsoleVariable<bool> FuzzMemoryGuard(IConsoleManager::Get().FindConsoleVariable(TEXT("pcg.GPU.FuzzMemory")), /*bNewValue=*/true);

    // Expected TRS: position (1,2,3) from user HLSL; rotation identity and scale (1,1,1) from preamble init.
    // Shared by both sub-cases; both verify all 4 elements.
    auto CheckTRS = [&](const FTransform& T, const FString& Prefix) -> bool
    {
        bool bPassed = true;
        bPassed &= TestNearlyEqual(Prefix + TEXT(" position X is 1"), T.GetLocation().X,  1.0, 0.01);
        bPassed &= TestNearlyEqual(Prefix + TEXT(" position Y is 2"), T.GetLocation().Y,  2.0, 0.01);
        bPassed &= TestNearlyEqual(Prefix + TEXT(" position Z is 3"), T.GetLocation().Z,  3.0, 0.01);
        bPassed &= TestTrue(Prefix + TEXT(" rotation is identity"),   T.GetRotation().Equals(FQuat::Identity, 0.01f));
        bPassed &= TestNearlyEqual(Prefix + TEXT(" scale X is 1"),    T.GetScale3D().X,   1.0, 0.01);
        bPassed &= TestNearlyEqual(Prefix + TEXT(" scale Y is 1"),    T.GetScale3D().Y,   1.0, 0.01);
        bPassed &= TestNearlyEqual(Prefix + TEXT(" scale Z is 1"),    T.GetScale3D().Z,   1.0, 0.01);
        return bPassed;
    };

    static const TCHAR* KernelSource = TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, float3(1.0f, 2.0f, 3.0f));\n");
    bool bAllPassed = true;

    // --- Constant position: Transform fully allocated, R/S initialized for ALL elements ---
    // With bAutoInitializeOutput=true a constant-expression setter still forces a fully-allocated (N-slot)
    // buffer. The preamble must write R/S per-element (not just at element 0), otherwise elements 1+ would
    // have garbage rotation and scale with FuzzMemory=true.
    {
        FPCGGPUGraphTestRunner Runner;

        UPCGCustomHLSLSettings* HLSLSettings = nullptr;
        UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
        HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
        HLSLSettings->NumElements = 4;
        HLSLSettings->SetSourceText(KernelSource);

        UPCGGatherSettings* GatherSettings = nullptr;
        UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
        Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

        TArray<FPCGDataCollection> OutNodeData;
        bAllPassed &= Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
        if (bAllPassed)
        {
            bAllPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
            {
                const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
                if (!TestNotNull(TEXT("ConstPos: readback data is point data"), OutPoints))
                {
                    return false;
                }
                if (!TestEqual(TEXT("ConstPos: output point count"), OutPoints->GetNumPoints(), 4))
                {
                    return false;
                }

                const FConstPCGPointValueRanges Ranges(OutPoints);
                bool bPassed = true;

                // All 4 elements must have correct P/R/S. R and S come from the preamble (not user code).
                for (int32 i = 0; i < 4; ++i)
                {
                    bPassed &= CheckTRS(Ranges.GetPoint(i).Transform, FString::Printf(TEXT("ConstPos[%d]"), i));
                }
                return bPassed;
            });
        }
    }

    // --- Per-element position: Transform fully allocated, R/S must be initialized for all elements ---
    {
        FPCGGPUGraphTestRunner Runner;

        UPCGCustomHLSLSettings* HLSLSettings = nullptr;
        UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
        HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
        HLSLSettings->NumElements = 4;
        // Non-constant position: each element gets a different X value, which forces the Transform
        // buffer to be fully allocated (N slots). Rotation and scale must receive defaults for every
        // element - an element-0-only write would leave elements 1+ with garbage values.
        HLSLSettings->SetSourceText(TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, float3((float)ElementIndex * 100.0f, 0.0f, 0.0f));\n"));

        UPCGGatherSettings* GatherSettings = nullptr;
        UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
        Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

        TArray<FPCGDataCollection> OutNodeData;
        bAllPassed &= Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);
        if (bAllPassed)
        {
            bAllPassed &= PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
            {
                const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
                if (!TestNotNull(TEXT("PerElemPos: readback data is point data"), OutPoints))
                {
                    return false;
                }
                if (!TestEqual(TEXT("PerElemPos: output point count"), OutPoints->GetNumPoints(), 4))
                {
                    return false;
                }

                const FConstPCGPointValueRanges Ranges(OutPoints);
                bool bPassed = true;
                for (int32 i = 0; i < 4; ++i)
                {
                    const FPCGPoint Point = Ranges.GetPoint(i);
                    const FString P = FString::Printf(TEXT("PerElemPos: Point[%d]"), i);
                    bPassed &= TestNearlyEqual(P + TEXT(" position X"),     Point.Transform.GetLocation().X, (double)(i * 100), 0.01);
                    bPassed &= TestNearlyEqual(P + TEXT(" position Y is 0"), Point.Transform.GetLocation().Y, 0.0, 0.01);
                    bPassed &= TestNearlyEqual(P + TEXT(" position Z is 0"), Point.Transform.GetLocation().Z, 0.0, 0.01);
                    // Rotation and scale must be defaults for ALL elements, not just element 0.
                    bPassed &= TestTrue(P + TEXT(" rotation is identity"), Point.Transform.GetRotation().Equals(FQuat::Identity, 0.01f));
                    bPassed &= TestNearlyEqual(P + TEXT(" scale X is 1"),  Point.Transform.GetScale3D().X, 1.0, 0.01);
                    bPassed &= TestNearlyEqual(P + TEXT(" scale Y is 1"),  Point.Transform.GetScale3D().Y, 1.0, 0.01);
                    bPassed &= TestNearlyEqual(P + TEXT(" scale Z is 1"),  Point.Transform.GetScale3D().Z, 1.0, 0.01);
                }
                return bPassed;
            });
        }
    }

    return bAllPassed;
}

/**
 * GPU test: PointProcessor with bAutoInitializeOutput=true correctly passes through per-element input
 * values for unwritten properties.
 *
 *   CreatePoints (4 points, distinct per-point Color and Seed) -> CustomHLSL (write only Position) -> Gather
 *
 * With bAutoInitializeOutput=false, unwritten properties are compressed to CVR (single slot, uninitialized),
 * so unwritten Color and Seed would be squashed to one slot rather than copied per-element.
 * With bAutoInitializeOutput=true (default), unwritten properties inherit the input's CVR/N-slot state,
 * so N-slot input Color and Seed remain N-slot in the output and the preamble copies them per-element.
 *
 * pcg.GPU.FuzzMemory=true ensures any slot the preamble fails to write contains garbage, making
 * the missing per-element copy immediately visible as a test failure.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLProcessorPassthroughTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.ProcessorPassthrough",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLProcessorPassthroughTest::RunTest(const FString& Parameters)
{
    TGuardConsoleVariable<bool> FuzzMemoryGuard(IConsoleManager::Get().FindConsoleVariable(TEXT("pcg.GPU.FuzzMemory")), /*bNewValue=*/true);

    FPCGGPUGraphTestRunner Runner;

    UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
    UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
    CreatePointsSettings->PointsToCreate.Empty();
    CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;

    // 4 points with distinct per-point Colors and Seeds. Having distinct values forces N-slot
    // (fully allocated) buffers for Color and Seed in the GPU data collection - not CVR (1-slot).
    const TArray<FVector4> InputColors = {
        FVector4(1.f, 0.f, 0.f, 1.f),
        FVector4(0.f, 1.f, 0.f, 1.f),
        FVector4(0.f, 0.f, 1.f, 1.f),
        FVector4(1.f, 1.f, 0.f, 1.f),
    };
    const TArray<int32> InputSeeds = { 10, 20, 30, 40 };

    for (int32 i = 0; i < 4; ++i)
    {
        FPCGPoint& Pt = CreatePointsSettings->PointsToCreate.AddDefaulted_GetRef();
        Pt.Color = InputColors[i];
        Pt.Seed = InputSeeds[i];
    }

    // Processor: write only Position (non-constant expression so Transform is N-slot).
    // bAutoInitializeOutput=true so unwritten Color and Seed inherit the input's N-slot state and are copied per-element.
    // With bAutoInitializeOutput=false, unwritten properties would be compressed to CVR (element-0 value only).
    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
    check(HLSLSettings->OutputPins.Num() == 1);
    HLSLSettings->OutputPins[0].PropertiesGPU.bAutoInitializeOutput = true;
    HLSLSettings->SetSourceText(
        TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, float3((float)ElementIndex * 100.f, 0.f, 0.f));\n")
    );

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
        {
            return false;
        }

        constexpr int32 ExpectedPointCount = 4;
        if (!TestEqual(TEXT("Output point count"), OutPoints->GetNumPoints(), ExpectedPointCount))
        {
            return false;
        }

        bool bTestPassed = true;

        const FConstPCGPointValueRanges Ranges(OutPoints);
        for (int32 i = 0; i < ExpectedPointCount; ++i)
        {
            const FPCGPoint Pt = Ranges.GetPoint(i);
            const FString P = FString::Printf(TEXT("Point[%d]"), i);

            // Color: not written by kernel, must be copied per-element from input.
            bTestPassed &= TestNearlyEqual(P + TEXT(" Color R"), (double)Pt.Color.X, (double)InputColors[i].X, 0.01);
            bTestPassed &= TestNearlyEqual(P + TEXT(" Color G"), (double)Pt.Color.Y, (double)InputColors[i].Y, 0.01);
            bTestPassed &= TestNearlyEqual(P + TEXT(" Color B"), (double)Pt.Color.Z, (double)InputColors[i].Z, 0.01);
            bTestPassed &= TestNearlyEqual(P + TEXT(" Color A"), (double)Pt.Color.W, (double)InputColors[i].W, 0.01);

            // Seed: not written by kernel, must be copied per-element from input.
            bTestPassed &= TestEqual(P + TEXT(" Seed"), Pt.Seed, InputSeeds[i]);
        }

        return bTestPassed;
    });
}

/**
 * GPU test: PointGenerator with an attribute name token ('AttrName') appearing as a nested function argument.
 *
 *   CreateAttributeSet (Scale=(2,3,4)) -> AttributeSetProcessor -> CustomHLSL PointGenerator -> Gather
 *
 * The kernel calls:
 *   Out_SetScale(Out_DataIndex, ElementIndex,
 *       CalculateScale(LayerData_GetFloat3(LayerData_DataIndex, 0u, 'Scale')));
 *
 * Regression: the parser failed to recognise 'Scale' as an attribute name when it appeared inside
 * a nested argument list. Moving the LayerData_GetFloat3 call to a separate line worked around the
 * bug. This test catches regressions where the nested form fails to parse or compile.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLNestedAttributeArgTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.NestedAttributeArg",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLNestedAttributeArgTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    const FName LayerDataPinLabel(TEXT("LayerData"));
    const FVector ExpectedScale(2.f, 3.f, 4.f);

    // Create an attribute set with a single FVector attribute 'Scale', then push it through an
    // empty AttributeSetProcessor to ensure it reaches the generator as GPU data.
    UPCGNode* CreateAttrNode = PCGTestsCommon::AddCreateAttributeSetNode(Runner.Graph, FName(TEXT("Scale")), ExpectedScale);

    UPCGCustomHLSLSettings* ASPSettings = nullptr;
    UPCGNode* ASPNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(ASPSettings);
    ASPSettings->SetKernelType(EPCGKernelType::AttributeSetProcessor);
    // Empty body - just forces the attribute set onto the GPU.

    Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, ASPNode, PCGPinConstants::DefaultInputLabel);

    // PointGenerator with an extra LayerData attribute set input pin.
    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
    HLSLSettings->NumElements = 2;
    HLSLSettings->InputPins.Add(FPCGPinProperties(LayerDataPinLabel, FPCGDataTypeIdentifier{EPCGDataType::Param}));

    // User-defined function that wraps its argument. The key is that 'Scale' is passed as an
    // attribute name inside a nested call, which is the case the parser was failing on.
    HLSLSettings->SetFunctionsText(TEXT("float3 CalculateScale(float3 BaseScale) { return BaseScale; }\n"));
    // Use 0u for both DataIndex and ElementIndex - the attribute set has exactly one data item
    // with one entry, so index 0 is always correct. (Generators don't auto-declare _DataIndex
    // for attribute set input pins the way processors do for point pins.)
    HLSLSettings->SetSourceText(
        TEXT("Out_SetScale(Out_DataIndex, ElementIndex, CalculateScale(LayerData_GetFloat3(0u, 0u, 'Scale')));\n")
    );

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(ASPNode,   PCGPinConstants::DefaultOutputLabel, HLSLNode,  LayerDataPinLabel);
    Runner.Graph->AddEdge(HLSLNode,  PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Readback data is point data"), OutPoints))
        {
            return false;
        }

        constexpr int32 ExpectedPointCount = 2;
        if (!TestEqual(TEXT("Output point count"), OutPoints->GetNumPoints(), ExpectedPointCount))
        {
            return false;
        }

        bool bTestPassed = true;
        const FConstPCGPointValueRanges Ranges(OutPoints);
        for (int32 i = 0; i < ExpectedPointCount; ++i)
        {
            const FVector ActualScale = Ranges.GetPoint(i).Transform.GetScale3D();
            const FString P = FString::Printf(TEXT("Point[%d]"), i);
            bTestPassed &= TestNearlyEqual(P + TEXT(" Scale X"), ActualScale.X, (double)ExpectedScale.X, 0.01);
            bTestPassed &= TestNearlyEqual(P + TEXT(" Scale Y"), ActualScale.Y, (double)ExpectedScale.Y, 0.01);
            bTestPassed &= TestNearlyEqual(P + TEXT(" Scale Z"), ActualScale.Z, (double)ExpectedScale.Z, 0.01);
        }
        return bTestPassed;
    });
}

/**
 * GPU test: AttributeSetProcessor receives a data collection with zero data items.
 *
 *   CreateAttributeSet (1 item) -> FilterDataByIndex (drops index 0 via OutsideFilter pin) ->
 *   CustomHLSL AttributeSetProcessor (empty body) -> Gather
 *
 * Regression for CL 53569560: when an attribute set processor's input/output has zero data items,
 * the data interface previously hit `check(SizeBytes > 0)` because ComputePackedSizeBytes() returned 0
 * for an empty data collection (the AnyOf guard in PCGDataCollectionPacking.cpp skipped the header).
 * The fix splits the size into HeaderSizeBytes/DataSizeBytes and clamps the data buffer allocation
 * to a 4-byte minimum.
 *
 * Expected: graph completes with no crash, gather output has zero tagged data items.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLAttributeSetProcessorEmptyOutputTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.AttributeSetProcessorEmptyOutput",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLAttributeSetProcessorEmptyOutputTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    // CreateAttributeSet: produces 1 data item with one attribute.
    UPCGNode* CreateAttrNode = PCGTestsCommon::AddCreateAttributeSetNode(Runner.Graph, FName(TEXT("Value")), 42);

    // FilterDataByIndex: select index 0 -> InsideFilter receives the only data item, OutsideFilter
    // receives an empty data collection (zero data items). Wiring downstream nodes from OutsideFilter
    // gives the GPU pipeline a zero-data-item input - the bug repro condition.
    UPCGFilterByIndexSettings* FilterSettings = nullptr;
    UPCGNode* FilterNode = Runner.Graph->AddNodeOfType<UPCGFilterByIndexSettings>(FilterSettings);
    FilterSettings->SelectedIndices = TEXT("0");
    FilterSettings->bInvertFilter = false;

    // Empty AttributeSetProcessor: receives the zero-data-item collection, runs on GPU, produces
    // zero-data-item output. Allocates the GPU data collection buffers - this is where the previous
    // assert fired.
    UPCGCustomHLSLSettings* ASPSettings = nullptr;
    UPCGNode* ASPNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(ASPSettings);
    ASPSettings->SetKernelType(EPCGKernelType::AttributeSetProcessor);

    // Gather output collected by the test runner - readback should yield no tagged data items.
    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel,    FilterNode, PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(FilterNode,     PCGPinConstants::DefaultOutFilterLabel, ASPNode,    PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(ASPNode,        PCGPinConstants::DefaultOutputLabel,    GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return TestEqual(TEXT("Gather produced zero data items"), OutNodeData[0].TaggedData.Num(), 0);
}

/**
 * Test helper that flips the protected bUseLegacyDataCollectionAPI flag on UPCGCustomHLSLSettings.
 * Friend'd in PCGCustomHLSL.h so tests can force the kernel into legacy mode without exposing a
 * production setter on the settings class.
 */
struct FPCGCustomHLSLSettingsTestHelper
{
    static void SetUseLegacyDataCollectionAPI(UPCGCustomHLSLSettings* Settings, bool bValue)
    {
        if (Settings)
        {
            Settings->bUseLegacyDataCollectionAPI = bValue;
            // Refresh declarations so any cached editor-side state derived from the flag is in sync. Not
            // strictly required for kernel compilation - the OR-rule pass in the graph compiler reads the
            // flag at compile time - but keeps the settings object consistent if anything inspects it.
            Settings->UpdateDeclarations();
        }
    }
};

/**
 * GPU test: data collection input pins are read-only - a setter call on an input pin must fail to compile.
 *
 * The kernel calls In_SetPosition (a writer) on the input pin. With non-legacy mode (the default for new
 * kernels), the data interface's GetSupportedInputs returns only readers, so the binding system never emits
 * an In_SetPosition wrapper and the kernel HLSL fails to compile.
 *
 * AddExpectedError swallows the resulting compile error log; if the read-only-input behavior ever regressed
 * and the call compiled, the expected error would be missing and the test would fail.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLInputDataCollectionIsReadOnlyTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.InputDataCollectionIsReadOnly",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLInputDataCollectionIsReadOnlyTest::RunTest(const FString& Parameters)
{
    // Match the substring common to DXC ("use of undeclared identifier 'X'") and FXC ("X3004: undeclared identifier 'X'").
    // Occurrences=0 means "at least one" - tolerates multi-permutation builds that log the same error more than once.
    AddExpectedError(TEXT("undeclared identifier 'In_SetPosition'"), EAutomationExpectedErrorFlags::Contains, /*Occurrences=*/0, /*bIsRegex=*/false);

    FPCGGPUGraphTestRunner Runner;

    UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
    UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
    CreatePointsSettings->PointsToCreate.AddDefaulted();

    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);

    // Sanity: PostInitProperties + IsNewObjectAndNotDefault should have flipped a freshly-created kernel to non-legacy.
    if (!TestFalse(TEXT("Newly created kernel defaults to non-legacy"), HLSLSettings->GetUseLegacyDataCollectionAPI()))
    {
        return false;
    }

    HLSLSettings->SetSourceText(
        TEXT("// Drive the output normally so the rest of the kernel is well-formed.\n")
        TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, In_GetPosition(In_DataIndex, ElementIndex));\n")
        TEXT("// Input pins are read-only - In_SetPosition is not advertised on input pins, so this fails to bind.\n")
        TEXT("In_SetPosition(In_DataIndex, ElementIndex, float3(0.0f, 0.0f, 0.0f));\n")
    );

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    // Run the graph. The runner reports completion regardless of shader compile success - the actual
    // assertion lives in AddExpectedError above, which validates that the read-only compile error was logged.
    TArray<FPCGDataCollection> OutNodeData;
    Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData);

    return true;
}

/**
 * GPU test: a kernel forced into legacy mode treats every data collection pin as read-write -
 * getters work on output pins and setters work on input pins.
 *
 * Forces bUseLegacyDataCollectionAPI=true via the test helper, then runs a kernel that calls
 * Out_GetPosition (reader on an output pin) and In_SetPosition (writer on an input pin). Both are
 * resolved because the OR-rule pass in the graph compiler propagates the kernel's legacy flag to
 * the touching UPCGDataCollectionDataInterface, which then exposes the union of read + write
 * functions on input pins (matching pre-split behavior).
 *
 * Test passes iff the kernel compiles and executes successfully.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLLegacyDataCollectionAlwaysReadWriteTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.LegacyDataCollectionAlwaysReadWrite",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLLegacyDataCollectionAlwaysReadWriteTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
    UPCGNode* CreatePointsNode = Runner.Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
    // PointsToCreate ships with one default entry; clear it so AddDefaulted produces exactly one point.
    CreatePointsSettings->PointsToCreate.Empty();
    CreatePointsSettings->PointsToCreate.AddDefaulted();

    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);

    // Force legacy mode. The OR-rule pass in the graph compiler reads this flag, marks the touching DIs
    // as legacy (bLegacyAllowSettersOnInputs), and the union of read + write functions becomes available on input pins.
    FPCGCustomHLSLSettingsTestHelper::SetUseLegacyDataCollectionAPI(HLSLSettings, true);
    if (!TestTrue(TEXT("Legacy flag is set on the kernel settings"), HLSLSettings->GetUseLegacyDataCollectionAPI()))
    {
        return false;
    }

    HLSLSettings->SetSourceText(
        TEXT("// Legacy mode treats every data collection pin as read-write: getters work on output pins,\n")
        TEXT("// setters work on input pins.\n")
        TEXT("float3 OutPos = Out_GetPosition(Out_DataIndex, ElementIndex);\n")
        TEXT("In_SetPosition(In_DataIndex, ElementIndex, OutPos);\n")
        TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, In_GetPosition(In_DataIndex, ElementIndex));\n")
    );

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!TestTrue(
        TEXT("Legacy kernel reads from output pin and writes to input pin"),
        Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData)))
    {
        return false;
    }

    // Validate that the kernel actually produced output - guards against a silent regression where the
    // kernel compiles+executes successfully but the legacy-only binding path no-ops (e.g. In_SetPosition
    // resolves to a wrapper but the underlying body is dropped). CreatePoints emitted one default point,
    // so the legacy kernel should pass it through unchanged.
    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [this](const FPCGTaggedData& TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        return TestNotNull(TEXT("Readback data is point data"), OutPoints)
            && TestEqual(TEXT("Legacy kernel produced one output point"), OutPoints->GetNumPoints(), 1);
    });
}

#endif // WITH_EDITOR
