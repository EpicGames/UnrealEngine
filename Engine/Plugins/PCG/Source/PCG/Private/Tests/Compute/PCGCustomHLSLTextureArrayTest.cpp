// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGTexture2DArrayData.h"
#include "Elements/PCGGather.h"

#include "Misc/AutomationTest.h"

/**
 * GPU test: TextureArrayGenerator writes a unique solid color into each slice of a 1x1x3 array.
 *
 *   CustomHLSL TextureArrayGenerator (1x1, 3 slices: slice 0=red, 1=green, 2=blue)
 *     -> CustomHLSL PointGenerator (3 pts, samples slice ElementIndex of the array)
 *     -> Gather
 *
 * Validates that each point's color matches the slice color, confirming the 3D dispatch
 * runs the kernel for every (x,y,slice) and writes through the array DI's RWTexture2DArray UAV.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLTextureArrayGeneratorTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.TextureArray.Generator",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLTextureArrayGeneratorTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	// Generator: 1x1x3 texture array. Each slice gets a different solid color, keyed off ElementIndex.z.
	UPCGCustomHLSLSettings* GenSettings = nullptr;
	UPCGNode* GenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GenSettings);
	GenSettings->SetKernelType(EPCGKernelType::TextureArrayGenerator);
	GenSettings->NumElementsX = 1;
	GenSettings->NumElementsY = 1;
	GenSettings->NumElementsZ = 3;
	GenSettings->SetSourceText(
		TEXT("float4 Color = float4(0.0f, 0.0f, 0.0f, 1.0f);\n")
		TEXT("if (ElementIndex.z == 0u) Color = float4(1.0f, 0.0f, 0.0f, 1.0f);\n")
		TEXT("else if (ElementIndex.z == 1u) Color = float4(0.0f, 1.0f, 0.0f, 1.0f);\n")
		TEXT("else if (ElementIndex.z == 2u) Color = float4(0.0f, 0.0f, 1.0f, 1.0f);\n")
		TEXT("Out_Store(Out_DataIndex, ElementIndex, Color);\n")
	);

	// PointGenerator: 3 points, each samples a different slice of the array via the texture array DI's Load(uint, uint3).
	const FName TexPinLabel(TEXT("Tex"));
	UPCGCustomHLSLSettings* PointGenSettings = nullptr;
	UPCGNode* PointGenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(PointGenSettings);
	PointGenSettings->SetKernelType(EPCGKernelType::PointGenerator);
	PointGenSettings->NumElements = 3;
	PointGenSettings->InputPins.Add(FPCGPinProperties(TexPinLabel, FPCGDataTypeInfoTexture2DArray::AsId(), /*bInAllowMultipleData=*/false));
	PointGenSettings->SetSourceText(
		TEXT("float4 TexColor = Tex_Load(0u, uint3(0u, 0u, ElementIndex));\n")
		TEXT("Out_SetColor(Out_DataIndex, ElementIndex, TexColor);\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(GenNode,      PCGPinConstants::DefaultOutputLabel, PointGenNode, TexPinLabel);
	Runner.Graph->AddEdge(PointGenNode, PCGPinConstants::DefaultOutputLabel, GatherNode,   PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
	{
		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Output is point data"), OutPoints))
		{
			return false;
		}

		if (!TestEqual(TEXT("Output point count is 3"), OutPoints->GetNumPoints(), 3))
		{
			return false;
		}

		bool bPassed = true;
		const FConstPCGPointValueRanges Ranges(OutPoints);

		// Point i samples slice i: expected color (R=delta(i==0), G=delta(i==1), B=delta(i==2), A=1).
		const FVector4 ExpectedColors[] = {
			FVector4(1.0, 0.0, 0.0, 1.0),
			FVector4(0.0, 1.0, 0.0, 1.0),
			FVector4(0.0, 0.0, 1.0, 1.0),
		};

		for (int32 SliceIndex = 0; SliceIndex < 3; ++SliceIndex)
		{
			const FVector4 Color = Ranges.ColorRange[SliceIndex];
			const FString P = FString::Printf(TEXT("Slice[%d]"), SliceIndex);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s R"), *P), (double)Color.X, ExpectedColors[SliceIndex].X, 0.01);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s G"), *P), (double)Color.Y, ExpectedColors[SliceIndex].Y, 0.01);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s B"), *P), (double)Color.Z, ExpectedColors[SliceIndex].Z, 0.01);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s A"), *P), (double)Color.W, ExpectedColors[SliceIndex].W, 0.01);
		}

		return bPassed;
	});
}

/**
 * GPU test: TextureArrayProcessor reads each texel of an upstream array and adds 0.25 to its blue channel.
 *
 *   TextureArrayGenerator (1x1x3, per-slice solid: red/green/blue)
 *     -> TextureArrayProcessor (B += 0.25)
 *     -> PointGenerator (3 pts, samples each slice)
 *     -> Gather
 *
 * Validates R/G are unchanged and B = source + 0.25 for every slice. Exercises the processor preamble's
 * uint3 ElementIndex emission, the auto-init of the output via InitializeOutputDataTexture, and the
 * processor branch of ComputeOutputBindingDataDesc inheriting input array dimensions.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLTextureArrayProcessorTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.TextureArray.Processor",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLTextureArrayProcessorTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	// Generator: 1x1x3 array, slice 0=red, slice 1=green, slice 2=blue.
	UPCGCustomHLSLSettings* GenSettings = nullptr;
	UPCGNode* GenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GenSettings);
	GenSettings->SetKernelType(EPCGKernelType::TextureArrayGenerator);
	GenSettings->NumElementsX = 1;
	GenSettings->NumElementsY = 1;
	GenSettings->NumElementsZ = 3;
	GenSettings->SetSourceText(
		TEXT("float4 Color = float4(0.0f, 0.0f, 0.0f, 1.0f);\n")
		TEXT("if (ElementIndex.z == 0u) Color = float4(1.0f, 0.0f, 0.0f, 1.0f);\n")
		TEXT("else if (ElementIndex.z == 1u) Color = float4(0.0f, 1.0f, 0.0f, 1.0f);\n")
		TEXT("else if (ElementIndex.z == 2u) Color = float4(0.0f, 0.0f, 1.0f, 1.0f);\n")
		TEXT("Out_Store(Out_DataIndex, ElementIndex, Color);\n")
	);

	// Processor: load each texel from input, add 0.25 to blue, store.
	// Pin RGBA16f so blue=1.25 round-trips unclamped (validated below).
	UPCGCustomHLSLSettings* ProcSettings = nullptr;
	UPCGNode* ProcNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(ProcSettings);
	ProcSettings->SetKernelType(EPCGKernelType::TextureArrayProcessor);
	ProcSettings->TextureFormat = EPCGRenderTargetFormat::RGBA16f;
	ProcSettings->SetSourceText(
		TEXT("float4 Texel = In_Load(In_DataIndex, ElementIndex);\n")
		TEXT("Texel.b += 0.25f;\n")
		TEXT("Out_Store(Out_DataIndex, ElementIndex, Texel);\n")
	);

	// Sample each slice from the processed array.
	const FName TexPinLabel(TEXT("Tex"));
	UPCGCustomHLSLSettings* PointGenSettings = nullptr;
	UPCGNode* PointGenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(PointGenSettings);
	PointGenSettings->SetKernelType(EPCGKernelType::PointGenerator);
	PointGenSettings->NumElements = 3;
	PointGenSettings->InputPins.Add(FPCGPinProperties(TexPinLabel, FPCGDataTypeInfoTexture2DArray::AsId(), /*bInAllowMultipleData=*/false));
	PointGenSettings->SetSourceText(
		TEXT("float4 TexColor = Tex_Load(0u, uint3(0u, 0u, ElementIndex));\n")
		TEXT("Out_SetColor(Out_DataIndex, ElementIndex, TexColor);\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(GenNode,      PCGPinConstants::DefaultOutputLabel, ProcNode,     PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(ProcNode,     PCGPinConstants::DefaultOutputLabel, PointGenNode, TexPinLabel);
	Runner.Graph->AddEdge(PointGenNode, PCGPinConstants::DefaultOutputLabel, GatherNode,   PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
	{
		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Output is point data"), OutPoints))
		{
			return false;
		}

		if (!TestEqual(TEXT("Output point count is 3"), OutPoints->GetNumPoints(), 3))
		{
			return false;
		}

		bool bPassed = true;
		const FConstPCGPointValueRanges Ranges(OutPoints);

		// Source slice colors plus +0.25 blue: (1, 0, 0.25, 1), (0, 1, 0.25, 1), (0, 0, 1.25, 1).
		// Note: 1.25 stays at 1.25 only if format is RGBA16f (no clamping). Default format is RGBA16f so this is fine.
		const FVector4 ExpectedColors[] = {
			FVector4(1.0, 0.0, 0.25, 1.0),
			FVector4(0.0, 1.0, 0.25, 1.0),
			FVector4(0.0, 0.0, 1.25, 1.0),
		};

		for (int32 SliceIndex = 0; SliceIndex < 3; ++SliceIndex)
		{
			const FVector4 Color = Ranges.ColorRange[SliceIndex];
			const FString P = FString::Printf(TEXT("Slice[%d]"), SliceIndex);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s R unchanged"), *P), (double)Color.X, ExpectedColors[SliceIndex].X, 0.01);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s G unchanged"), *P), (double)Color.Y, ExpectedColors[SliceIndex].Y, 0.01);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s B = src + 0.25"), *P), (double)Color.Z, ExpectedColors[SliceIndex].Z, 0.01);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s A unchanged"), *P), (double)Color.W, ExpectedColors[SliceIndex].W, 0.01);
		}

		return bPassed;
	});
}

/**
 * GPU test: NumElementsX/Y/Z all flow through the override pin path for a 3D generator.
 *
 *   AttributeSet overrides (NumElementsX = 64, NumElementsY = 64, NumElementsZ = 5)
 *     -> CustomHLSL TextureArrayGenerator (asset defaults 1x1x1, overridden to 64x64x5; slice s writes (s/(Z-1), 0, 0, 1))
 *     -> PointGenerator (5 pts, samples (0,0) of each slice)
 *     -> Gather
 *
 * Asset values are deliberately set to 1x1x1 so the test fails unless every override actually
 * lands. Failure modes caught: any of the X/Y/Z override pins not wired, or thread count
 * ignoring a dimension.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGCustomHLSLTextureArraySizeOverrideTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.CustomHLSL.TextureArray.SizeOverride",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLTextureArraySizeOverrideTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	constexpr int32 OverriddenWidth = 64;
	constexpr int32 OverriddenHeight = 64;
	constexpr int32 OverriddenSliceCount = 5;

	UPCGCustomHLSLSettings* GenSettings = nullptr;
	UPCGNode* GenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GenSettings);
	GenSettings->SetKernelType(EPCGKernelType::TextureArrayGenerator);
	GenSettings->SetSourceText(
		TEXT("const float Red = float(ElementIndex.z) / float(NumElements.z - 1u);\n")
		TEXT("Out_Store(Out_DataIndex, ElementIndex, float4(Red, 0.0f, 0.0f, 1.0f));\n")
	);

	PCGGPUTestCommon::AddOverride(Runner.Graph, GenNode, GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsX), OverriddenWidth);
	PCGGPUTestCommon::AddOverride(Runner.Graph, GenNode, GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsY), OverriddenHeight);
	PCGGPUTestCommon::AddOverride(Runner.Graph, GenNode, GET_MEMBER_NAME_CHECKED(UPCGCustomHLSLSettings, NumElementsZ), OverriddenSliceCount);

	const FName TexPinLabel(TEXT("Tex"));
	UPCGCustomHLSLSettings* PointGenSettings = nullptr;
	UPCGNode* PointGenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(PointGenSettings);
	PointGenSettings->SetKernelType(EPCGKernelType::PointGenerator);
	PointGenSettings->NumElements = OverriddenSliceCount;
	PointGenSettings->InputPins.Add(FPCGPinProperties(TexPinLabel, FPCGDataTypeInfoTexture2DArray::AsId(), /*bInAllowMultipleData=*/false));
	PointGenSettings->SetSourceText(
		TEXT("float4 TexColor = Tex_Load(0u, uint3(0u, 0u, ElementIndex));\n")
		TEXT("Out_SetColor(Out_DataIndex, ElementIndex, TexColor);\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(GenNode,      PCGPinConstants::DefaultOutputLabel, PointGenNode, TexPinLabel);
	Runner.Graph->AddEdge(PointGenNode, PCGPinConstants::DefaultOutputLabel, GatherNode,   PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
	{
		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Output is point data"), OutPoints))
		{
			return false;
		}

		if (!TestEqual(TEXT("Output point count matches overridden slice count"), OutPoints->GetNumPoints(), OverriddenSliceCount))
		{
			return false;
		}

		bool bPassed = true;
		const FConstPCGPointValueRanges Ranges(OutPoints);

		for (int32 SliceIndex = 0; SliceIndex < OverriddenSliceCount; ++SliceIndex)
		{
			const FVector4 Color = Ranges.ColorRange[SliceIndex];
			const float ExpectedR = float(SliceIndex) / float(OverriddenSliceCount - 1);
			const FString P = FString::Printf(TEXT("Slice[%d]"), SliceIndex);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s R = %.3f"), *P, ExpectedR), (double)Color.X, (double)ExpectedR, 0.02);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s G = 0"), *P), (double)Color.Y, 0.0, 0.02);
			bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s B = 0"), *P), (double)Color.Z, 0.0, 0.02);
		}

		return bPassed;
	});
}

#endif // WITH_EDITOR
