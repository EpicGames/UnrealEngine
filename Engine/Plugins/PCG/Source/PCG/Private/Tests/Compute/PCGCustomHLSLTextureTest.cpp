// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGPinPropertiesGPU.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGTextureData.h"
#include "Elements/PCGGather.h"
#include "Helpers/PCGHelpers.h"

#include "HAL/PlatformProcess.h"
#include "Misc/AutomationTest.h"
#include "RenderingThread.h"

namespace PCGCustomHLSLTextureTestHelpers
{
    /**
     * Polls RequestCPUReadback() on every UPCGTexture2DSingleBaseData in the collection until all complete.
     * Returns false if any readback does not complete within the polling limit.
     */
    bool WaitForTextureReadbacks(FAutomationTestBase* Test, const FPCGDataCollection& NodeData)
    {
        constexpr int32 MaxIter = 10000;

        for (const FPCGTaggedData& TaggedData : NodeData.TaggedData)
        {
            const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(TaggedData.Data);
            if (!TextureData)
            {
                continue;
            }

            bool bComplete = false;
            for (int32 i = 0; i < MaxIter; ++i)
            {
                bComplete = TextureData->RequestCPUReadback();
                if (bComplete)
                {
                    break;
                }
                FPlatformProcess::SleepNoStats(0.001f);
                FlushRenderingCommands();
            }

            if (!Test->TestTrue(TEXT("Texture CPU readback completed"), bComplete))
            {
                return false;
            }
        }

        return true;
    }
}

/**
 * GPU test: TextureGenerator writes a UV gradient into a 4x4 texture.
 *
 *   CustomHLSL TextureGenerator (4x4, R=u, G=v, B=0.5, A=1) -> Gather
 *
 * After readback, converts each texture to point data (one point per texel) and validates
 * that R increases left-to-right, G increases top-to-bottom, B=0.5, A=1 for all texels.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLTextureGeneratorTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.Texture.Generator",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLTextureGeneratorTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::TextureGenerator);
    HLSLSettings->NumElementsX = 4;
    HLSLSettings->NumElementsY = 4;
    HLSLSettings->SetSourceText(
        TEXT("uint2 Resolution = Out_GetNumElements(Out_DataIndex);\n")
        TEXT("float u = float(ElementIndex.x) / float(Resolution.x - 1u);\n")
        TEXT("float v = float(ElementIndex.y) / float(Resolution.y - 1u);\n")
        TEXT("Out_Store(Out_DataIndex, ElementIndex, float4(u, v, 0.5f, 1.0f));\n")
    );

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);
    Runner.Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    if (!PCGCustomHLSLTextureTestHelpers::WaitForTextureReadbacks(this, OutNodeData[0]))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Output is texture data"), TextureData))
        {
            return false;
        }

        const FIntPoint Resolution = TextureData->GetResolution();
        if (!TestEqual(TEXT("Texture width is 4"), Resolution.X, 4) ||
            !TestEqual(TEXT("Texture height is 4"), Resolution.Y, 4))
        {
            return false;
        }

        // CreatePointData gives one point per texel in row-major order: index = y * Width + x
        const UPCGBasePointData* PointData = TextureData->CreatePointData(nullptr);
        if (!TestNotNull(TEXT("CreatePointData returned data"), PointData))
        {
            return false;
        }

        const int32 Width = Resolution.X;
        const int32 Height = Resolution.Y;
        if (!TestEqual(TEXT("Point count matches texel count"), PointData->GetNumPoints(), Width * Height))
        {
            return false;
        }

        bool bPassed = true;
        const FConstPCGPointValueRanges Ranges(PointData);

        for (int32 Y = 0; Y < Height; ++Y)
        {
            for (int32 X = 0; X < Width; ++X)
            {
                const int32 Idx = Y * Width + X;
                const FVector4 Color = Ranges.ColorRange[Idx];
                const float ExpectedR = float(X) / float(Width - 1);
                const float ExpectedG = float(Y) / float(Height - 1);
                const FString P = FString::Printf(TEXT("Texel(%d,%d)"), X, Y);
                bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s R (u=%.3f)"), *P, ExpectedR), (double)Color.X, (double)ExpectedR, 0.01);
                bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s G (v=%.3f)"), *P, ExpectedG), (double)Color.Y, (double)ExpectedG, 0.01);
                bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s B=0.5"), *P), (double)Color.Z, 0.5, 0.01);
                bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s A=1.0"), *P), (double)Color.W, 1.0, 0.01);
            }
        }

        return bPassed;
    });
}

/**
 * GPU test: TextureProcessor reads an upstream gradient texture and adds 0.25 to the blue channel.
 *
 *   CustomHLSL TextureGenerator (4x4, R=u, G=v, B=0.5, A=1) -> CustomHLSL TextureProcessor (B += 0.25) -> Gather
 *
 * Validates that R and G are unchanged from the generator and B = 0.75 for all texels.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLTextureProcessorTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.Texture.Processor",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLTextureProcessorTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    // Generator: 4x4 UV gradient.
    UPCGCustomHLSLSettings* GenSettings = nullptr;
    UPCGNode* GenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GenSettings);
    GenSettings->SetKernelType(EPCGKernelType::TextureGenerator);
    GenSettings->NumElementsX = 4;
    GenSettings->NumElementsY = 4;
    GenSettings->SetSourceText(
        TEXT("uint2 Resolution = Out_GetNumElements(Out_DataIndex);\n")
        TEXT("float u = float(ElementIndex.x) / float(Resolution.x - 1u);\n")
        TEXT("float v = float(ElementIndex.y) / float(Resolution.y - 1u);\n")
        TEXT("Out_Store(Out_DataIndex, ElementIndex, float4(u, v, 0.5f, 1.0f));\n")
    );

    // Processor: load each texel from the input and add 0.25 to blue.
    UPCGCustomHLSLSettings* ProcSettings = nullptr;
    UPCGNode* ProcNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(ProcSettings);
    ProcSettings->SetKernelType(EPCGKernelType::TextureProcessor);
    ProcSettings->SetSourceText(
        TEXT("float4 Texel = In_Load(In_DataIndex, ElementIndex, 0u);\n")
        TEXT("Texel.b += 0.25f;\n")
        TEXT("Out_Store(Out_DataIndex, ElementIndex, Texel);\n")
    );

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(GenNode,  PCGPinConstants::DefaultOutputLabel, ProcNode,   PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(ProcNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    if (!PCGCustomHLSLTextureTestHelpers::WaitForTextureReadbacks(this, OutNodeData[0]))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Output is texture data"), TextureData))
        {
            return false;
        }

        const FIntPoint Resolution = TextureData->GetResolution();
        if (!TestEqual(TEXT("Texture width is 4"), Resolution.X, 4) ||
            !TestEqual(TEXT("Texture height is 4"), Resolution.Y, 4))
        {
            return false;
        }

        const UPCGBasePointData* PointData = TextureData->CreatePointData(nullptr);
        if (!TestNotNull(TEXT("CreatePointData returned data"), PointData))
        {
            return false;
        }

        const int32 Width = Resolution.X;
        const int32 Height = Resolution.Y;
        if (!TestEqual(TEXT("Point count matches texel count"), PointData->GetNumPoints(), Width * Height))
        {
            return false;
        }

        bool bPassed = true;
        const FConstPCGPointValueRanges Ranges(PointData);

        for (int32 Y = 0; Y < Height; ++Y)
        {
            for (int32 X = 0; X < Width; ++X)
            {
                const int32 Idx = Y * Width + X;
                const FVector4 Color = Ranges.ColorRange[Idx];
                const float ExpectedR = float(X) / float(Width - 1);
                const float ExpectedG = float(Y) / float(Height - 1);
                const FString P = FString::Printf(TEXT("Texel(%d,%d)"), X, Y);
                bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s R unchanged (%.3f)"), *P, ExpectedR), (double)Color.X, (double)ExpectedR, 0.01);
                bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s G unchanged (%.3f)"), *P, ExpectedG), (double)Color.Y, (double)ExpectedG, 0.01);
                bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s B = 0.75 (0.5 + 0.25)"), *P), (double)Color.Z, 0.75, 0.01);
                bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s A = 1.0"), *P), (double)Color.W, 1.0, 0.01);
            }
        }

        return bPassed;
    });
}

/**
 * GPU test: Texture from a generator is wired to a PointGenerator that has a multi-data texture input pin.
 *
 *   CustomHLSL TextureGenerator (4x4, R=u, G=0, B=0, A=1) -> CustomHLSL PointGenerator (4 pts, samples texture) -> Gather
 *
 * The PointGenerator samples the texture at u = ElementIndex / 3, v = 0 and stores the color on each output point.
 * Validates that each point's Red channel matches the expected u value, confirming the texture data is accessible
 * from a PointGenerator with a multi-data texture input pin.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLTextureSingleToMultiDataTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.Texture.SingleToMultiData",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLTextureSingleToMultiDataTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    const FName TexPinLabel(TEXT("Tex"));

    // Generator: 4x4 texture with R=u gradient.
    UPCGCustomHLSLSettings* GenSettings = nullptr;
    UPCGNode* GenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GenSettings);
    GenSettings->SetKernelType(EPCGKernelType::TextureGenerator);
    GenSettings->NumElementsX = 4;
    GenSettings->NumElementsY = 4;
    GenSettings->SetSourceText(
        TEXT("uint2 Resolution = Out_GetNumElements(Out_DataIndex);\n")
        TEXT("float u = float(ElementIndex.x) / float(Resolution.x - 1u);\n")
        TEXT("Out_Store(Out_DataIndex, ElementIndex, float4(u, 0.0f, 0.0f, 1.0f));\n")
    );

    // PointGenerator: 4 points, each samples the texture at a different u coordinate and stores the color.
    UPCGCustomHLSLSettings* PointGenSettings = nullptr;
    UPCGNode* PointGenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(PointGenSettings);
    PointGenSettings->SetKernelType(EPCGKernelType::PointGenerator);
    PointGenSettings->NumElements = 4;
    // Add a multi-data texture input pin.
    PointGenSettings->InputPins.Add(FPCGPinProperties(TexPinLabel, FPCGDataTypeIdentifier{EPCGDataType::BaseTexture}, /*bInAllowMultipleData=*/true));
    // Load the texel at (ElementIndex, 0) from the texture and store its color on the output point.
    // Tex_Load is used instead of Tex_Sample to get an exact per-texel value without bilinear interpolation.
    PointGenSettings->SetSourceText(
        TEXT("float4 TexColor = Tex_Load(0u, uint2(ElementIndex, 0u), 0u);\n")
        TEXT("Out_SetColor(Out_DataIndex, ElementIndex, TexColor);\n")
    );

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(GenNode,      PCGPinConstants::DefaultOutputLabel, PointGenNode, TexPinLabel);
    Runner.Graph->AddEdge(PointGenNode, PCGPinConstants::DefaultOutputLabel, GatherNode,   PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Output is point data"), OutPoints))
        {
            return false;
        }

        if (!TestEqual(TEXT("Output point count is 4"), OutPoints->GetNumPoints(), 4))
        {
            return false;
        }

        bool bPassed = true;
        const FConstPCGPointValueRanges Ranges(OutPoints);

        for (int32 i = 0; i < 4; ++i)
        {
            const FVector4 Color = Ranges.ColorRange[i];
            const float ExpectedR = float(i) / 3.0f;
            const FString P = FString::Printf(TEXT("Point[%d]"), i);
            bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s R (u=%.3f)"), *P, ExpectedR), (double)Color.X, (double)ExpectedR, 0.02);
            bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s G = 0"), *P), (double)Color.Y, 0.0, 0.02);
            bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s B = 0"), *P), (double)Color.Z, 0.0, 0.02);
        }

        return bPassed;
    });
}

/**
 * GPU test: Two textures arrive at a TextureProcessor whose input pin is single-data (bAllowMultipleData=false).
 *
 *   TextureGenerator Red (4x4, R=1,G=0,B=0,A=1) -> Gather
 *   TextureGenerator Green (4x4, R=0,G=1,B=0,A=1) -> same Gather
 *   Gather -> TextureProcessor (single-data input, pass-through) -> Gather
 *
 * Only the first texture should be processed (the DI caps at 1 binding when single-data). Red is added to GatherBoth
 * first, so edge-addition order places it at binding index 0. Validates that all output texels are red (R~1, G~0, B~0),
 * confirming the single-data constraint picks the first-added texture and discards the second.
 *
 * Note: the graph may emit a warning about too many data items arriving on a single-data pin.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLTextureMultiToSingleDataTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.Texture.MultiToSingleData",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLTextureMultiToSingleDataTest::RunTest(const FString& Parameters)
{
    AddExpectedError(TEXT("too many data items arriving on single data pin"), EAutomationExpectedErrorFlags::Contains, /*Occurrences=*/1, /*bIsRegex=*/false);

    FPCGGPUGraphTestRunner Runner;

    // Generator Red: all texels are (1,0,0,1).
    UPCGCustomHLSLSettings* GenRedSettings = nullptr;
    UPCGNode* GenRedNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GenRedSettings);
    GenRedSettings->SetKernelType(EPCGKernelType::TextureGenerator);
    GenRedSettings->NumElementsX = 4;
    GenRedSettings->NumElementsY = 4;
    GenRedSettings->SetSourceText(TEXT("Out_Store(Out_DataIndex, ElementIndex, float4(1.0f, 0.0f, 0.0f, 1.0f));\n"));

    // Generator Green: all texels are (0,1,0,1).
    UPCGCustomHLSLSettings* GenGreenSettings = nullptr;
    UPCGNode* GenGreenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GenGreenSettings);
    GenGreenSettings->SetKernelType(EPCGKernelType::TextureGenerator);
    GenGreenSettings->NumElementsX = 4;
    GenGreenSettings->NumElementsY = 4;
    GenGreenSettings->SetSourceText(TEXT("Out_Store(Out_DataIndex, ElementIndex, float4(0.0f, 1.0f, 0.0f, 1.0f));\n"));

    // Gather both textures together before feeding to the processor.
    UPCGGatherSettings* GatherBothSettings = nullptr;
    UPCGNode* GatherBothNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherBothSettings);

    // TextureProcessor: pass-through. Its default input pin already has bAllowMultipleData=false (set by UpdatePinSettings).
    UPCGCustomHLSLSettings* ProcSettings = nullptr;
    UPCGNode* ProcNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(ProcSettings);
    ProcSettings->SetKernelType(EPCGKernelType::TextureProcessor);
    ProcSettings->SetSourceText(
        TEXT("float4 Texel = In_Load(In_DataIndex, ElementIndex, 0u);\n")
        TEXT("Out_Store(Out_DataIndex, ElementIndex, Texel);\n")
    );

    UPCGGatherSettings* GatherOutSettings = nullptr;
    UPCGNode* GatherOutNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherOutSettings);

    Runner.Graph->AddEdge(GenRedNode,    PCGPinConstants::DefaultOutputLabel, GatherBothNode, PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(GenGreenNode,  PCGPinConstants::DefaultOutputLabel, GatherBothNode, PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(GatherBothNode, PCGPinConstants::DefaultOutputLabel, ProcNode,      PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(ProcNode,      PCGPinConstants::DefaultOutputLabel, GatherOutNode,  PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherOutNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    if (!PCGCustomHLSLTextureTestHelpers::WaitForTextureReadbacks(this, OutNodeData[0]))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGTexture2DSingleBaseData* TextureData = Cast<UPCGTexture2DSingleBaseData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Output is texture data"), TextureData))
        {
            return false;
        }

        const FIntPoint Resolution = TextureData->GetResolution();
        if (!TestEqual(TEXT("Texture width is 4"), Resolution.X, 4) ||
            !TestEqual(TEXT("Texture height is 4"), Resolution.Y, 4))
        {
            return false;
        }

        const UPCGBasePointData* PointData = TextureData->CreatePointData(nullptr);
        if (!TestNotNull(TEXT("CreatePointData returned data"), PointData))
        {
            return false;
        }

        const int32 Width = Resolution.X;
        const int32 Height = Resolution.Y;
        if (!TestEqual(TEXT("Point count matches texel count"), PointData->GetNumPoints(), Width * Height))
        {
            return false;
        }

        // All texels should be uniformly red: Red was added to GatherBoth first (edge-addition order is preserved), so it ends up at
        // binding index 0 and the single-data DI selects it while the Green texture is discarded.
        bool bPassed = true;
        const FConstPCGPointValueRanges Ranges(PointData);

        for (int32 Idx = 0; Idx < Width * Height; ++Idx)
        {
            const FVector4 Color = Ranges.ColorRange[Idx];
            const FString P = FString::Printf(TEXT("Texel[%d]"), Idx);
            bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s R = 1 (red texture selected)"), *P), (double)Color.X, 1.0, 0.01);
            bPassed &= TestNearlyEqual(*FString::Printf(TEXT("%s G = 0 (red texture selected)"), *P), (double)Color.Y, 0.0, 0.01);
        }

        return bPassed;
    });
}

/**
 * GPU test: sampling an out-of-bounds texture data index returns black, not data from index 0.
 *
 *   CustomHLSL TextureGenerator (4x4, solid red R=1,G=0,B=0,A=1) -> CustomHLSL PointGenerator (2 pts, multi-data Tex pin) -> Gather
 *
 * Only 1 texture is fed to the PointGenerator. Point 0 samples Tex index 0 (valid, red).
 * Point 1 samples Tex index 1 (out-of-bounds) and must return black (not a leak of index 0's data).
 * Regression test for GetTextureInfoInternal_ returning BindingIndex=0 / Dimension=Texture2D
 * on an invalid index, which caused the real texture to be sampled instead of returning zero.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGCustomHLSLTextureOutOfBoundsIndexTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.CustomHLSL.Texture.OutOfBoundsIndex",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGCustomHLSLTextureOutOfBoundsIndexTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    const FName TexPinLabel(TEXT("Tex"));

    // Generator: solid red texture.
    UPCGCustomHLSLSettings* GenSettings = nullptr;
    UPCGNode* GenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GenSettings);
    GenSettings->SetKernelType(EPCGKernelType::TextureGenerator);
    GenSettings->NumElementsX = 4;
    GenSettings->NumElementsY = 4;
    GenSettings->SetSourceText(TEXT("Out_Store(Out_DataIndex, ElementIndex, float4(1.0f, 0.0f, 0.0f, 1.0f));\n"));

    // PointGenerator: 2 points.
    // Point 0 samples texture index 0 (valid) -> red.
    // Point 1 samples texture index 1 (out-of-bounds, only 1 texture fed) -> black.
    UPCGCustomHLSLSettings* PointGenSettings = nullptr;
    UPCGNode* PointGenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(PointGenSettings);
    PointGenSettings->SetKernelType(EPCGKernelType::PointGenerator);
    PointGenSettings->NumElements = 2;
    PointGenSettings->InputPins.Add(FPCGPinProperties(TexPinLabel, FPCGDataTypeIdentifier{EPCGDataType::BaseTexture}, /*bInAllowMultipleData=*/true));
    PointGenSettings->SetSourceText(
        TEXT("float4 TexColor = Tex_Load(uint(ElementIndex), uint2(0u, 0u), 0u);\n")
        TEXT("Out_SetColor(Out_DataIndex, ElementIndex, TexColor);\n")
    );

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    Runner.Graph->AddEdge(GenNode,      PCGPinConstants::DefaultOutputLabel, PointGenNode, TexPinLabel);
    Runner.Graph->AddEdge(PointGenNode, PCGPinConstants::DefaultOutputLabel, GatherNode,   PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
    {
        return false;
    }

    return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Output is point data"), OutPoints))
        {
            return false;
        }

        if (!TestEqual(TEXT("Output point count is 2"), OutPoints->GetNumPoints(), 2))
        {
            return false;
        }

        bool bPassed = true;
        const FConstPCGPointValueRanges Ranges(OutPoints);

        // Point 0: sampled texture index 0 (valid) -> red.
        const FVector4 Color0 = Ranges.ColorRange[0];
        bPassed &= TestNearlyEqual(TEXT("Point[0] R = 1 (valid index 0, red texture)"), (double)Color0.X, 1.0, 0.01);
        bPassed &= TestNearlyEqual(TEXT("Point[0] G = 0"), (double)Color0.Y, 0.0, 0.01);
        bPassed &= TestNearlyEqual(TEXT("Point[0] B = 0"), (double)Color0.Z, 0.0, 0.01);

        // Point 1: sampled texture index 1 (out-of-bounds) -> black.
        const FVector4 Color1 = Ranges.ColorRange[1];
        bPassed &= TestNearlyEqual(TEXT("Point[1] R = 0 (out-of-bounds index 1 -> black)"), (double)Color1.X, 0.0, 0.01);
        bPassed &= TestNearlyEqual(TEXT("Point[1] G = 0 (out-of-bounds index 1 -> black)"), (double)Color1.Y, 0.0, 0.01);
        bPassed &= TestNearlyEqual(TEXT("Point[1] B = 0 (out-of-bounds index 1 -> black)"), (double)Color1.Z, 0.0, 0.01);
        bPassed &= TestNearlyEqual(TEXT("Point[1] A = 0 (out-of-bounds index 1 -> black)"), (double)Color1.W, 0.0, 0.01);

        return bPassed;
    });
}

#endif // WITH_EDITOR
