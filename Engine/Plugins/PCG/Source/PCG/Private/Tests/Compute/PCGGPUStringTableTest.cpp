// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "PCGParamData.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Elements/PCGCreateAttribute.h"
#include "Elements/PCGGather.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

/**
 * GPU test: FName attribute passes through an AttributeSetProcessor unchanged.
 *
 *   CreateAttributeSet (NameAttr=FName("TestName")) -> CustomHLSL (read NameAttr, write NameAttr) -> Gather
 *
 * FName uses CompressData=false (UsesValueKeys()=false), so AddInputDataStringsToTable takes the
 * slow path (iterates all N entries via ApplyOnAccessor). Verifies the FName survives the roundtrip.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGStringTableFNamePassthroughTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.StringTable.FNamePassthrough",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGStringTableFNamePassthroughTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	// CreateAttributeSet: one element with Name attribute "NameAttr" = FName("TestName").
	UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
	UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
	CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::Name;
	CreateAttrSettings->AttributeTypes.NameValue = FName(TEXT("TestName"));
	CreateAttrSettings->OutputTarget.SetAttributeName(FName(TEXT("NameAttr")));

	// CustomHLSL AttributeSetProcessor: read NameAttr and write it back unchanged.
	// FName is represented by the 'GetName'/'SetName' HLSL intrinsics (int key into the string table).
	UPCGCustomHLSLSettings* HLSLSettings = nullptr;
	UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
	HLSLSettings->SetKernelType(EPCGKernelType::AttributeSetProcessor);
	HLSLSettings->SetSourceText(
		TEXT("int NameVal = In_GetName(In_DataIndex, ElementIndex, 'NameAttr');\n")
		TEXT("Out_SetName(Out_DataIndex, ElementIndex, 'NameAttr', NameVal);\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, HLSLNode,   PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode,       PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
	{
		const UPCGParamData* OutParams = Cast<UPCGParamData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Readback data is attribute set data"), OutParams))
		{
			return false;
		}

		const UPCGMetadata* Metadata = OutParams->ConstMetadata();
		if (!TestNotNull(TEXT("Output has metadata"), Metadata))
		{
			return false;
		}

		const FPCGMetadataAttribute<FName>* NameAttr = Metadata->GetConstTypedAttribute<FName>(FName(TEXT("NameAttr")));
		if (!TestNotNull(TEXT("Output has 'NameAttr' Name attribute"), NameAttr))
		{
			return false;
		}

		const FName ActualValue = NameAttr->GetValueFromItemKey(0);
		return TestEqual(
			*FString::Printf(TEXT("'NameAttr' passed through (expected 'TestName', got '%s')"), *ActualValue.ToString()),
			ActualValue, FName(TEXT("TestName")));
	});
}

/**
 * GPU test: FString attribute passes through two chained AttributeSetProcessor nodes unchanged.
 *
 *   CreateAttributeSet (StringAttr="ChainedValue") -> CustomHLSL1 (passthrough) -> CustomHLSL2 (passthrough) -> Gather
 *
 * When GPU proxy data from kernel 1 feeds kernel 2, SourceBufferToGraphStringKey must remap string
 * keys correctly. Verifies the string survives both hops.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGStringTableChainedGPUKernelsTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.StringTable.ChainedGPUKernelsStringPassthrough",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGStringTableChainedGPUKernelsTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	// CreateAttributeSet: "ChainedValue" on StringAttr.
	UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
	UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
	CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::String;
	CreateAttrSettings->AttributeTypes.StringValue = TEXT("ChainedValue");
	CreateAttrSettings->OutputTarget.SetAttributeName(FName(TEXT("StringAttr")));

	// First HLSL node: passthrough StringAttr.
	UPCGCustomHLSLSettings* HLSLSettings1 = nullptr;
	UPCGNode* HLSLNode1 = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings1);
	HLSLSettings1->SetKernelType(EPCGKernelType::AttributeSetProcessor);
	HLSLSettings1->SetSourceText(
		TEXT("int k = In_GetStringKey(In_DataIndex, ElementIndex, 'StringAttr');\n")
		TEXT("Out_SetStringKey(Out_DataIndex, ElementIndex, 'StringAttr', k);\n")
	);

	// Second HLSL node: passthrough StringAttr again.
	UPCGCustomHLSLSettings* HLSLSettings2 = nullptr;
	UPCGNode* HLSLNode2 = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings2);
	HLSLSettings2->SetKernelType(EPCGKernelType::AttributeSetProcessor);
	HLSLSettings2->SetSourceText(
		TEXT("int k = In_GetStringKey(In_DataIndex, ElementIndex, 'StringAttr');\n")
		TEXT("Out_SetStringKey(Out_DataIndex, ElementIndex, 'StringAttr', k);\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, HLSLNode1,  PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode1,      PCGPinConstants::DefaultOutputLabel, HLSLNode2,  PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode2,      PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
	{
		const UPCGParamData* OutParams = Cast<UPCGParamData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Readback data is attribute set data"), OutParams))
		{
			return false;
		}

		const UPCGMetadata* Metadata = OutParams->ConstMetadata();
		if (!TestNotNull(TEXT("Output has metadata"), Metadata))
		{
			return false;
		}

		const FPCGMetadataAttribute<FString>* StringAttr = Metadata->GetConstTypedAttribute<FString>(FName(TEXT("StringAttr")));
		if (!TestNotNull(TEXT("Output has 'StringAttr' string attribute"), StringAttr))
		{
			return false;
		}

		const FString ActualValue = StringAttr->GetValueFromItemKey(0);
		return TestEqual(
			*FString::Printf(TEXT("'StringAttr' survived two HLSL hops (expected 'ChainedValue', got '%s')"), *ActualValue),
			ActualValue, FString(TEXT("ChainedValue")));
	});
}

/**
 * GPU test: Two distinct FString attributes on the same attribute set both survive the GPU roundtrip.
 *
 *   CreateAttributeSet(AttrA="Foo") -> AddAttribute(AttrB="Bar") -> CustomHLSL (passthrough both) -> Gather
 *
 * Tests that multiple string attributes from the same data item all land in the string table and their
 * keys round-trip independently.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGStringTableMultipleStringAttributesTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.StringTable.MultipleStringAttributesPassthrough",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGStringTableMultipleStringAttributesTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	// CreateAttributeSet: AttrA = "Foo".
	UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
	UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
	CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::String;
	CreateAttrSettings->AttributeTypes.StringValue = TEXT("Foo");
	CreateAttrSettings->OutputTarget.SetAttributeName(FName(TEXT("AttrA")));

	// AddAttribute: add AttrB = "Bar" to the same param data (constant, no source pin wired).
	UPCGAddAttributeSettings* AddAttrSettings = nullptr;
	UPCGNode* AddAttrNode = Runner.Graph->AddNodeOfType<UPCGAddAttributeSettings>(AddAttrSettings);
	AddAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::String;
	AddAttrSettings->AttributeTypes.StringValue = TEXT("Bar");
	AddAttrSettings->OutputTarget.SetAttributeName(FName(TEXT("AttrB")));

	// CustomHLSL AttributeSetProcessor: passthrough both string attributes.
	UPCGCustomHLSLSettings* HLSLSettings = nullptr;
	UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
	HLSLSettings->SetKernelType(EPCGKernelType::AttributeSetProcessor);
	HLSLSettings->SetSourceText(
		TEXT("int ka = In_GetStringKey(In_DataIndex, ElementIndex, 'AttrA');\n")
		TEXT("int kb = In_GetStringKey(In_DataIndex, ElementIndex, 'AttrB');\n")
		TEXT("Out_SetStringKey(Out_DataIndex, ElementIndex, 'AttrA', ka);\n")
		TEXT("Out_SetStringKey(Out_DataIndex, ElementIndex, 'AttrB', kb);\n")
	);

	UPCGGatherSettings* GatherSettings = nullptr;
	UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

	Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, AddAttrNode, PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(AddAttrNode,    PCGPinConstants::DefaultOutputLabel, HLSLNode,   PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode,       PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0], [&](const FPCGTaggedData& TaggedData)
	{
		const UPCGParamData* OutParams = Cast<UPCGParamData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Readback data is attribute set data"), OutParams))
		{
			return false;
		}

		const UPCGMetadata* Metadata = OutParams->ConstMetadata();
		if (!TestNotNull(TEXT("Output has metadata"), Metadata))
		{
			return false;
		}

		const FPCGMetadataAttribute<FString>* AttrA = Metadata->GetConstTypedAttribute<FString>(FName(TEXT("AttrA")));
		if (!TestNotNull(TEXT("Output has 'AttrA' string attribute"), AttrA))
		{
			return false;
		}

		const FPCGMetadataAttribute<FString>* AttrB = Metadata->GetConstTypedAttribute<FString>(FName(TEXT("AttrB")));
		if (!TestNotNull(TEXT("Output has 'AttrB' string attribute"), AttrB))
		{
			return false;
		}

		bool bTestPassed = true;
		const FString ActualA = AttrA->GetValueFromItemKey(0);
		bTestPassed &= TestEqual(
			*FString::Printf(TEXT("'AttrA' preserved (expected 'Foo', got '%s')"), *ActualA),
			ActualA, FString(TEXT("Foo")));

		const FString ActualB = AttrB->GetValueFromItemKey(0);
		bTestPassed &= TestEqual(
			*FString::Printf(TEXT("'AttrB' preserved (expected 'Bar', got '%s')"), *ActualB),
			ActualB, FString(TEXT("Bar")));

		return bTestPassed;
	});
}

/**
 * GPU test: Multiple distinct FString values all survive the GPU roundtrip.
 *
 *   CreateAttributeSet("Val_0") --\
 *   CreateAttributeSet("Val_1") --+
 *                                  Gather -> CustomHLSL (passthrough) -> Gather
 *   CreateAttributeSet("Val_2") --+
 *   CreateAttributeSet("Val_3") --/
 *
 * Tests string table growth with K==N (all unique values). Exercises AddUnique for several distinct keys
 * and verifies none are lost or corrupted.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGStringTableManyUniqueStringsTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.StringTable.ManyUniqueStrings",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGStringTableManyUniqueStringsTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	constexpr int32 NumUniqueStrings = 4;

	// Create N attribute sets each with a unique "Val" string.
	UPCGGatherSettings* GatherSettings1 = nullptr;
	UPCGNode* GatherNode1 = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings1);

	for (int32 i = 0; i < NumUniqueStrings; ++i)
	{
		UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
		UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
		CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::String;
		CreateAttrSettings->AttributeTypes.StringValue = FString::Printf(TEXT("Val_%d"), i);
		CreateAttrSettings->OutputTarget.SetAttributeName(FName(TEXT("Val")));

		Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, GatherNode1, PCGPinConstants::DefaultInputLabel);
	}

	UPCGCustomHLSLSettings* HLSLSettings = nullptr;
	UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
	HLSLSettings->SetKernelType(EPCGKernelType::AttributeSetProcessor);
	HLSLSettings->SetSourceText(
		TEXT("int k = In_GetStringKey(In_DataIndex, ElementIndex, 'Val');\n")
		TEXT("Out_SetStringKey(Out_DataIndex, ElementIndex, 'Val', k);\n")
	);

	UPCGGatherSettings* GatherSettings2 = nullptr;
	UPCGNode* GatherNode2 = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings2);

	Runner.Graph->AddEdge(GatherNode1, PCGPinConstants::DefaultOutputLabel, HLSLNode,    PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode,    PCGPinConstants::DefaultOutputLabel, GatherNode2, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode2 } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	const FPCGDataCollection& OutputCollection = OutNodeData[0];
	if (!TestEqual(TEXT("Output has exactly 4 data items"), OutputCollection.TaggedData.Num(), NumUniqueStrings))
	{
		return false;
	}

	// Collect all output string values and verify all unique values are present.
	TArray<FString> FoundStrings;
	for (const FPCGTaggedData& TaggedData : OutputCollection.TaggedData)
	{
		const UPCGParamData* OutParams = Cast<UPCGParamData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Output item is param data"), OutParams))
		{
			return false;
		}

		const FPCGMetadataAttribute<FString>* ValAttr =
			OutParams->ConstMetadata()->GetConstTypedAttribute<FString>(FName(TEXT("Val")));
		if (!TestNotNull(TEXT("Output item has 'Val' attribute"), ValAttr))
		{
			return false;
		}

		FoundStrings.AddUnique(ValAttr->GetValueFromItemKey(0));
	}

	bool bTestPassed = TestEqual(TEXT("All 4 output strings are unique"), FoundStrings.Num(), NumUniqueStrings);
	for (int32 i = 0; i < NumUniqueStrings; ++i)
	{
		const FString Expected = FString::Printf(TEXT("Val_%d"), i);
		bTestPassed &= TestTrue(*FString::Printf(TEXT("'%s' is in output"), *Expected), FoundStrings.Contains(Expected));
	}
	return bTestPassed;
}

/**
 * GPU test: Duplicate string values are deduplicated in the string table and all entries round-trip correctly.
 *
 *   CreateAttributeSet("Alpha") --\
 *   CreateAttributeSet("Alpha") --+
 *                                  Gather -> CustomHLSL (passthrough) -> Gather
 *   CreateAttributeSet("Beta")  --+
 *   CreateAttributeSet("Beta")  --/
 *
 * Tests the K<<N scenario: 4 data items map to K=2 unique string values. Exercises StringTable.AddUnique
 * deduplication and verifies every item reads back the correct value.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGStringTableDuplicateStringCompressionTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.StringTable.DuplicateStringCompression",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGStringTableDuplicateStringCompressionTest::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;

	// 4 attribute sets: 2 x "Alpha", 2 x "Beta".
	const TArray<FString> InputValues = { TEXT("Alpha"), TEXT("Alpha"), TEXT("Beta"), TEXT("Beta") };

	UPCGGatherSettings* GatherSettings1 = nullptr;
	UPCGNode* GatherNode1 = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings1);

	for (const FString& Value : InputValues)
	{
		UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
		UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
		CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::String;
		CreateAttrSettings->AttributeTypes.StringValue = Value;
		CreateAttrSettings->OutputTarget.SetAttributeName(FName(TEXT("Category")));

		Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, GatherNode1, PCGPinConstants::DefaultInputLabel);
	}

	UPCGCustomHLSLSettings* HLSLSettings = nullptr;
	UPCGNode* HLSLNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
	HLSLSettings->SetKernelType(EPCGKernelType::AttributeSetProcessor);
	HLSLSettings->SetSourceText(
		TEXT("int k = In_GetStringKey(In_DataIndex, ElementIndex, 'Category');\n")
		TEXT("Out_SetStringKey(Out_DataIndex, ElementIndex, 'Category', k);\n")
	);

	UPCGGatherSettings* GatherSettings2 = nullptr;
	UPCGNode* GatherNode2 = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings2);

	Runner.Graph->AddEdge(GatherNode1, PCGPinConstants::DefaultOutputLabel, HLSLNode,    PCGPinConstants::DefaultInputLabel);
	Runner.Graph->AddEdge(HLSLNode,    PCGPinConstants::DefaultOutputLabel, GatherNode2, PCGPinConstants::DefaultInputLabel);

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode2 } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	const FPCGDataCollection& OutputCollection = OutNodeData[0];
	if (!TestEqual(TEXT("Output has exactly 4 data items"), OutputCollection.TaggedData.Num(), 4))
	{
		return false;
	}

	// Count occurrences of each string in the output.
	int32 AlphaCount = 0;
	int32 BetaCount  = 0;
	for (const FPCGTaggedData& TaggedData : OutputCollection.TaggedData)
	{
		const UPCGParamData* OutParams = Cast<UPCGParamData>(TaggedData.Data);
		if (!TestNotNull(TEXT("Output item is param data"), OutParams))
		{
			return false;
		}

		const FPCGMetadataAttribute<FString>* CategoryAttr =
			OutParams->ConstMetadata()->GetConstTypedAttribute<FString>(FName(TEXT("Category")));
		if (!TestNotNull(TEXT("Output item has 'Category' attribute"), CategoryAttr))
		{
			return false;
		}

		const FString Value = CategoryAttr->GetValueFromItemKey(0);
		if (Value == TEXT("Alpha"))      { ++AlphaCount; }
		else if (Value == TEXT("Beta"))  { ++BetaCount;  }
		else
		{
			AddError(*FString::Printf(TEXT("Unexpected value '%s' in output"), *Value));
			return false;
		}
	}

	bool bTestPassed = TestEqual(TEXT("'Alpha' appears exactly 2 times"), AlphaCount, 2);
	bTestPassed     &= TestEqual(TEXT("'Beta' appears exactly 2 times"),  BetaCount,  2);
	return bTestPassed;
}

#endif // WITH_EDITOR
