// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGSettings.h"
#include "Compute/Data/PCGProxyForGPUData.h"
#include "Graph/PCGGraphExecutor.h"

#include "ComputeWorkerInterface.h"
#include "CustomResourcePool.h"
#include "EngineUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderingThread.h"
#include "RenderTargetPool.h"
#include "Components/BoxComponent.h"
#include "ComputeFramework/ComputeFramework.h"
#include "Containers/Ticker.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AutomationTest.h"
#include "UObject/UObjectGlobals.h"

bool PCGGPUTestCommon::RunCPUGPUParityTest(FAutomationTestBase* Test, FBuildGraphFunc BuildGraph, FValidateFunc Validate, const FPCGGPUGraphTestRunnerConfig& RunnerConfig)
{
	// CPU execution
	FPCGGPUGraphTestRunner CPURunner(RunnerConfig);
	FParityBuildResult CPUSetup = BuildGraph(CPURunner.Graph, CPURunner.TestWorld);

	for (UPCGSettings* Settings : CPUSetup.GPUSettings)
	{
		// Explicitly disable GPU mode on the settings.
		Settings->SetExecuteOnGPU(false);
	}

	TArray<FPCGDataCollection> CPUOutput;
	if (!CPURunner.Execute({ .Test = Test, .CollectOutputDataForNodes = { CPUSetup.CollectNode } }, /*OutNodeData=*/CPUOutput))
	{
		return false;
	}

	// GPU execution
	FPCGGPUGraphTestRunner GPURunner(RunnerConfig);
	FParityBuildResult GPUSetup = BuildGraph(GPURunner.Graph, GPURunner.TestWorld);

	for (UPCGSettings* Settings : GPUSetup.GPUSettings)
	{
		Settings->SetExecuteOnGPU(true);
	}

	TArray<FPCGDataCollection> GPUOutput;
	if (!GPURunner.Execute({ .Test = Test, .CollectOutputDataForNodes = { GPUSetup.CollectNode } }, /*OutNodeData=*/GPUOutput))
	{
		return false;
	}

	return Validate(Test, CPUOutput[0], GPUOutput[0]);
}

FPCGGPUGraphTestRunner::FPCGGPUGraphTestRunner(const FPCGGPUGraphTestRunnerConfig& InConfig)
{
	// Create an isolated game world so test artifacts (actors, landscapes, etc.) don't pollute
	// the editor world. The world gets a renderer Scene (needed for ComputeFramework::FlushWork).
	const FName WorldName = MakeUniqueObjectName(GetTransientPackage(), UWorld::StaticClass(), FName(TEXT("PCGGPUTestWorld")));
	FWorldContext& WorldContext = GEngine->CreateNewWorldContext(EWorldType::Game);
	TestWorld = UWorld::CreateWorld(EWorldType::Game, /*bInformEngineOfWorld=*/false, WorldName, GetTransientPackage());
	check(TestWorld);
	TestWorld->AddToRoot();
	WorldContext.SetCurrentWorld(TestWorld);

	// Spawn test actor with valid bounds. The bounds box must exist before the PCG component
	// registers, because registration queries actor bounds to set up the execution source.
	FActorSpawnParameters SpawnParams;
	SpawnParams.ObjectFlags = RF_Transient;
	TestActor = TestWorld->SpawnActor<AActor>(SpawnParams);
	check(TestActor);
	TestActor->SetActorLocation(InConfig.ActorLocation);

	UBoxComponent* BoundsBox = NewObject<UBoxComponent>(TestActor, FName(TEXT("BoundsBox")), RF_Transient);
	BoundsBox->SetBoxExtent(InConfig.BoundsExtent);
	BoundsBox->RegisterComponent();
	TestActor->AddInstanceComponent(BoundsBox);
	TestActor->SetRootComponent(BoundsBox);

	TestPCGComponent = NewObject<UPCGComponent>(TestActor, FName(TEXT("TestPCGComponent")), RF_Transient);
	check(TestPCGComponent);
	TestActor->AddInstanceComponent(TestPCGComponent);
	TestPCGComponent->RegisterComponent();
	TestPCGComponent->SetIsPartitioned(false);

	Graph = NewObject<UPCGGraph>(GetTransientPackage(), NAME_None, RF_Transient);
}

FPCGGPUGraphTestRunner::~FPCGGPUGraphTestRunner()
{
	if (TestWorld)
	{
		// End-play all actors so components unregister cleanly.
		for (AActor* Actor : FActorRange(TestWorld))
		{
			if (Actor)
			{
				Actor->RouteEndPlay(EEndPlayReason::LevelTransition);
			}
		}

		GEngine->ShutdownWorldNetDriver(TestWorld);
		TestWorld->DestroyWorld(/*bInformEngineOfWorld=*/true);
		TestWorld->SetPhysicsScene(nullptr);
		GEngine->DestroyWorldContext(TestWorld);
		TestWorld->RemoveFromRoot();
		TestWorld = nullptr;

		// UWorld::DestroyWorld does not release the FScene -- only UWorld::FinishDestroy() does, and that only runs during GC. Collect garbage now so FinishDestroy fires synchronously,
		// which calls Scene->Release() and enqueues the FScene deletion on the render thread. The pool tick below is enqueued after it, so the deletion runs first (FIFO), dropping
		// FGPUScene's TRefCountPtrs on the reserved-resource RDG buffers (refcount 2->1) before the pool tick sees them.
		CollectGarbage(RF_NoFlags, true);

		// Tick render resource pools to release reserved-resource allocations (e.g. GPUScene buffers, render targets). Without a real frame pump these pools never tick, so reserved VA builds up
		// across test runs and trips the rhi.ReservedResources.VirtualSizeWarningGB budget. FRDGBufferPool::kFramesUntilRelease = 30, so 31 ticks are required to release GPUScene's
		// 4 x 2 GB reserved buffers that are allocated per-scene. GRenderTargetPool only needs 3 (releases when UnusedForNFrames > 2).
		ENQUEUE_RENDER_COMMAND(PCGGPUTest_TickRenderPools)([](FRHICommandListImmediate& RHICmdList)
		{
			for (int32 i = 0; i < 31; ++i)
			{
				GRenderTargetPool.TickPoolElements();
				FRDGBuilder::TickPoolElements();
			}

			ICustomResourcePool::TickPoolElements(RHICmdList);
		});
		FlushRenderingCommands();
	}
}

bool FPCGGPUGraphTestRunner::Execute(const FPCGGPUGraphTestRunnerParams& Params, TArray<FPCGDataCollection>& OutNodeData)
{
	check(Params.Test);

	TestPCGComponent->SetGraph(Graph);

	TSharedPtr<FPCGGraphExecutor> Executor = MakeShared<FPCGGraphExecutor>();
	Executor->SetKeepIntermediateResults(true);

	TArray<FPCGDataCollection> CapturedOutputs;
	CapturedOutputs.SetNum(Params.CollectOutputDataForNodes.Num());
	TArray<bool> NodeOutputCaptured;
	NodeOutputCaptured.Init(false, Params.CollectOutputDataForNodes.Num());
	TArray<bool> CulledNodeExecuted;
	CulledNodeExecuted.Init(false, Params.ExpectCulledNodes.Num());

	// This will execute the graph and run the lambda on each node, which we use to capture output data.
	const FPCGTaskId FinalTaskId = Executor->ScheduleDebugWithTaskCallback(
		TestPCGComponent,
		[&](FPCGTaskId /*TaskId*/, const UPCGNode* Node, const FPCGDataCollection& Output)
		{
			for (int32 i = 0; i < Params.CollectOutputDataForNodes.Num(); ++i)
			{
				if (Node == Params.CollectOutputDataForNodes[i])
				{
					CapturedOutputs[i] = Output;
					NodeOutputCaptured[i] = true;
					break;
				}
			}

			for (int32 i = 0; i < Params.ExpectCulledNodes.Num(); ++i)
			{
				if (Node == Params.ExpectCulledNodes[i])
				{
					CulledNodeExecuted[i] = true;
					break;
				}
			}
		}
	);

	// Sentinel for execution.
	volatile bool bExecutionComplete = false;
	Executor->ScheduleGeneric(
		[&bExecutionComplete]() { bExecutionComplete = true; return true; },
		TestPCGComponent,
		{ FinalTaskId }
	);

	constexpr float FakeDeltaTime = 1.f / 30.f;
	constexpr int32 MaxExecuteIterations = 10000;
	for (int32 i = 0; i < MaxExecuteIterations && !bExecutionComplete; ++i)
	{
		// Fires FPCGModule::Tick (flushing SleepUntilNextFrame unpause callbacks) AND any ExecuteOnGameThread
		// tickers added by the previous iteration.
		FTSTicker::GetCoreTicker().Tick(FakeDeltaTime);

		// Advances any now-unpaused tasks, may enqueue GPU compute work.
		Executor->Execute();

		// Manually dispatches any queued EndOfFrameUpdate compute work for the scene, since without a real render
		// frame the engine never does this automatically.
		if (TestWorld && TestWorld->Scene)
		{
			// todo_pcg: Using this execution group is fine for now, but in the future this will be dictated by the requirements of the compute work.
			ComputeFramework::FlushWork(TestWorld->Scene, ComputeTaskExecutionGroup::EndOfFrameUpdate);
		}

		// Render thread executes the RDG, then calls ExecuteOnGameThread() which registers export callbacks as
		// one-shot core tickers for the next iteration.
		FlushRenderingCommands();
	}

	if (!Params.Test->TestTrue(TEXT("Graph execution completed within iteration limit"), bExecutionComplete))
	{
		Executor->CancelAll();
		return false;
	}

	bool bAllNodeOutputsCaptured = true;
	for (int32 i = 0; i < Params.CollectOutputDataForNodes.Num(); ++i)
	{
		bAllNodeOutputsCaptured &= Params.Test->TestTrue(
			*FString::Printf(TEXT("Output for node %d was captured by debug callback"), i),
			NodeOutputCaptured[i]);
	}
	if (!bAllNodeOutputsCaptured)
	{
		return false;
	}

	bool bAllCulledNodesWereCulled = true;
	for (int32 i = 0; i < Params.ExpectCulledNodes.Num(); ++i)
	{
		bAllCulledNodesWereCulled &= Params.Test->TestFalse(*FString::Printf(TEXT("Node '%s' expected to be culled was not executed"), *Params.ExpectCulledNodes[i]->GetNodeTitle(EPCGNodeTitleType::ListView).ToString()), CulledNodeExecuted[i]);
	}
	if (!bAllCulledNodesWereCulled)
	{
		return false;
	}

	if (Params.bReadbackGPUData)
	{
		FlushRenderingCommands();
	}

	OutNodeData.SetNum(Params.CollectOutputDataForNodes.Num());

	// Gather all the output data collections for the requested nodes.
	for (int32 i = 0; i < CapturedOutputs.Num(); ++i)
	{
		for (const FPCGTaggedData& TaggedData : CapturedOutputs[i].TaggedData)
		{
			if (Params.bReadbackGPUData)
			{
				if (const UPCGProxyForGPUData* Proxy = Cast<UPCGProxyForGPUData>(TaggedData.Data))
				{
					UPCGProxyForGPUData::FReadbackResult Result;
					constexpr int32 MaxReadbackIterations = 10000;
					for (int32 ReadbackIter = 0; ReadbackIter < MaxReadbackIterations; ++ReadbackIter)
					{
						Result = Proxy->GetCPUData(nullptr);
						if (Result.bComplete)
						{
							break;
						}

						FPlatformProcess::SleepNoStats(0.001f);
						FlushRenderingCommands();
					}

					if (!Params.Test->TestTrue(TEXT("GPU->CPU readback completed"), Result.bComplete))
					{
						return false;
					}

					if (!Params.Test->TestNotNull(TEXT("Readback produced a data object"), Result.TaggedData.Data.Get()))
					{
						return false;
					}

					OutNodeData[i].TaggedData.Add(Result.TaggedData);
					continue;
				}
			}

			OutNodeData[i].TaggedData.Add(TaggedData);
		}
	}

	if (!Params.Test->TestFalse(TEXT("All generation should be complete"), Executor->IsAnyGraphCurrentlyExecuting()))
	{
		Executor->CancelAll();
		return false;
	}

	return true;
}

#endif // WITH_EDITOR
