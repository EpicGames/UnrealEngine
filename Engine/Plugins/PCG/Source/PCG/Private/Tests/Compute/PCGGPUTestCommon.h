// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"

#include "PCGData.h"
#include "Compute/Elements/PCGCustomHLSL.h"

#include "Templates/Function.h"

class AActor;
class FAutomationTestBase;
class UPCGComponent;
class UPCGGraph;
class UPCGNode;
class UPCGSettings;
class UWorld;

struct FPCGGPUGraphTestRunnerConfig
{
	/** World-space location of the test actor. */
	FVector ActorLocation = FVector::ZeroVector;

	/** Extent of the box component attached to the test actor. Controls the PCG component's grid bounds. */
	FVector BoundsExtent = FVector(2500.f);
};

namespace PCGGPUTestCommon
{
	/** Add a CreateAttributeSet override node and wire it to a target node's override pin. */
	template <typename T>
	void AddOverride(UPCGGraph* Graph, UPCGNode* TargetNode, FName PinName, const T& Value)
	{
		UPCGNode* Node = PCGTestsCommon::AddCreateAttributeSetNode(Graph, PinName, Value);
		Graph->AddEdge(Node, PCGPinConstants::DefaultOutputLabel, TargetNode, PinName);
	}

	/** Add a CreateAttributeSet -> empty CustomHLSL ASP chain (forces GPU execution) and wire to a target node's override pin. */
	template <typename T>
	void AddGPUOverride(UPCGGraph* Graph, UPCGNode* TargetNode, FName PinName, const T& Value)
	{
		UPCGNode* CreateNode = PCGTestsCommon::AddCreateAttributeSetNode(Graph, PinName, Value);

		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::AttributeSetProcessor);

		Graph->AddEdge(CreateNode, PCGPinConstants::DefaultOutputLabel, HLSLNode, PCGPinConstants::DefaultInputLabel);
		Graph->AddEdge(HLSLNode, PCGPinConstants::DefaultOutputLabel, TargetNode, PinName);
	}

	/**
	 * Run the same graph on CPU and GPU (separate runners) and validate the outputs.
	 * BuildGraph is called twice — it should build the graph and return the node to collect output from
	 * plus the settings whose bExecuteOnGPU flag should be toggled for the GPU run.
	 * Validate receives the two output collections and should return true if they match.
	 */
	struct FParityBuildResult
	{
		UPCGNode* CollectNode = nullptr;
		TArray<UPCGSettings*> GPUSettings;
	};
	using FBuildGraphFunc = TFunctionRef<FParityBuildResult(UPCGGraph* Graph, UWorld* World)>;
	using FValidateFunc = TFunctionRef<bool(FAutomationTestBase* Test, const FPCGDataCollection& CPUOutput, const FPCGDataCollection& GPUOutput)>;
	bool RunCPUGPUParityTest(FAutomationTestBase* Test, FBuildGraphFunc BuildGraph, FValidateFunc Validate, const FPCGGPUGraphTestRunnerConfig& RunnerConfig = {});
}

/** Parameters for FPCGGPUGraphTestRunner::Execute(). */
struct FPCGGPUGraphTestRunnerParams
{
	FAutomationTestBase* Test = nullptr;
	TArray<UPCGNode*> CollectOutputDataForNodes;

	/** If true (default), UPCGProxyForGPUData outputs are read back to CPU before being added to OutNodeData. */
	bool bReadbackGPUData = true;

	/** Nodes that should be culled (statically or dynamically) and therefore never execute. Execute() asserts that the debug callback was
	* NOT fired for each of these nodes. */
	TArray<UPCGNode*> ExpectCulledNodes;
};

/**
 * Helper for writing GPU PCG automation tests.
 *
 * Handles all boilerplate: actor/component setup, bounds, graph creation, executor scheduling, tick/execute/FlushWork loop,
 * and GPU->CPU readback.
 *
 * Usage:
 *   1. Construct FPCGGPUGraphTestRunner.
 *   2. Add nodes to FPCGGPUGraphTestRunner::Graph and wire them.
 *   3. Specify for which nodes the test runner should collect the output data collections from (FPCGGPUGraphTestRunnerParams::CollectOutputDataForNodes).
 *   4. Call Runner.Execute(), if it returns true it completed.
 *   5. Verify OutNodeData in the test body.
 */
struct FPCGGPUGraphTestRunner
{
	UWorld* TestWorld = nullptr;
	AActor* TestActor = nullptr;
	UPCGComponent* TestPCGComponent = nullptr;
	UPCGGraph* Graph = nullptr;

	/** Creates an isolated game world with a test actor, PCG component, and graph. */
	explicit FPCGGPUGraphTestRunner(const FPCGGPUGraphTestRunnerConfig& InConfig = {});

	/** Tears down the isolated world and all actors within it. */
	~FPCGGPUGraphTestRunner();

	FPCGGPUGraphTestRunner(const FPCGGPUGraphTestRunner&) = delete;
	FPCGGPUGraphTestRunner& operator=(const FPCGGPUGraphTestRunner&) = delete;

	/**
	 * Runs the graph to completion, reads back (if Params.bReadbackGPUData) all
	 * UPCGProxyForGPUData from each specified node's output to CPU data, and fills OutNodeData.
	 * Logs test failures via Params.Test on error. Returns true if everything succeeded.
	 */
	bool Execute(const FPCGGPUGraphTestRunnerParams& Params, TArray<FPCGDataCollection>& OutNodeData);
};

#endif // WITH_EDITOR
