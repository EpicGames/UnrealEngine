// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGGraph.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGCreateAttribute.h"
#include "Elements/PCGGather.h"
#include "Elements/Metadata/PCGMetadataPartition.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include "Misc/AutomationTest.h"

/**
 * GPU Compiler ordering test: two PointGenerator GPU nodes (separate islands) feeding one CPU
 * Gather must arrive in edge-addition order.
 *
 * Graph:
 *   PointGenerator A (1 point, X=100) --+
 *                                        +--> CPU Gather (collect here)
 *   PointGenerator B (1 point, X=200) --+
 *
 * GenA's edge to Gather is added BEFORE GenB's, so edge-addition order is [A, B].
 * The expected output is therefore:
 *   TaggedData[0] -> from GenA, point.X == 100
 *   TaggedData[1] -> from GenB, point.X == 200
 *
 * Root cause of the current failure: WireComputeGraphTask() APPENDS the compute-graph-task
 * edge for each GPU island to the downstream CPU node's Inputs array, rather than replacing
 * the existing kernel-task edge in-place. The append order follows island/task-ID order (which
 * reverses edge-addition order here), so GenB ends up at index 0 after the original kernel-task
 * edges are culled.
 *
 * Fix: replace in-place at Inputs[SuccessorInputIndex] instead of Inputs.Add(), so the entry
 * stays at the same position it held when the edge was first added.
 *
 * This test is expected to FAIL until that fix is applied.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGGPUCompilerInputOrderTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.GraphCompilerGPU.InputOrderMatchesEdgeAdditionOrder",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGGPUCompilerInputOrderTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    // Generator A: one point at X=100. Added to Gather first.
    UPCGCustomHLSLSettings* GenASettings = nullptr;
    UPCGNode* GenANode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GenASettings);
    GenASettings->SetKernelType(EPCGKernelType::PointGenerator);
    GenASettings->NumElements = 1;
    GenASettings->SetSourceText(TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, float3(100.0, 0.0, 0.0));\n"));

    // Generator B: one point at X=200. Added to Gather second.
    UPCGCustomHLSLSettings* GenBSettings = nullptr;
    UPCGNode* GenBNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(GenBSettings);
    GenBSettings->SetKernelType(EPCGKernelType::PointGenerator);
    GenBSettings->NumElements = 1;
    GenBSettings->SetSourceText(TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, float3(200.0, 0.0, 0.0));\n"));

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    // Edge-addition order determines expected data ordering.
    Runner.Graph->AddEdge(GenANode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(GenBNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, OutNodeData))
    {
        return false;
    }

    const FPCGDataCollection& GatherOutput = OutNodeData[0];

    if (!TestEqual(TEXT("Gather has exactly 2 tagged data items"), GatherOutput.TaggedData.Num(), 2))
    {
        return false;
    }

    // Expected: TaggedData[0] = GenA (X=100), TaggedData[1] = GenB (X=200).
    // With the WireComputeGraphTask ordering bug: TaggedData[0] = GenB (X=200), TaggedData[1] = GenA (X=100).
    bool bPassed = true;

    const float ExpectedX[2] = { 100.f, 200.f };
    const TCHAR* const DataLabel[2] = { TEXT("GenA"), TEXT("GenB") };

    for (int32 DataIdx = 0; DataIdx < 2; ++DataIdx)
    {
        const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(GatherOutput.TaggedData[DataIdx].Data);
        if (!TestNotNull(*FString::Printf(TEXT("TaggedData[%d] is point data"), DataIdx), PointData))
        {
            bPassed = false;
            continue;
        }

        if (!TestEqual(*FString::Printf(TEXT("TaggedData[%d] has 1 point"), DataIdx), PointData->GetNumPoints(), 1))
        {
            bPassed = false;
            continue;
        }

        const FConstPCGPointValueRanges Ranges(PointData);
        const float ActualX = static_cast<float>(Ranges.TransformRange[0].GetLocation().X);
        bPassed &= TestNearlyEqual(
            *FString::Printf(TEXT("TaggedData[%d] (%s) position.X == %.0f"), DataIdx, DataLabel[DataIdx], (double)ExpectedX[DataIdx]),
            (double)ActualX,
            (double)ExpectedX[DataIdx],
            0.1);
    }

    return bPassed;
}

/**
 * GPU compiler test: ExternalBufferReuse header uses the producer's attribute layout, not the consumer's remapped view.
 *
 * When a consumer kernel reuses an upstream GPU buffer (ExternalBufferReuse path), the header written in
 * GetHeaderBuffer_RenderThread must be based on ExternalBufferDesc (the producer's actual attribute layout).
 * If it uses TargetDataCollectionDesc instead, the FirstElementAddress values are scattered into the wrong
 * slots, and the shader reads one attribute's data as if it were another after the ID remap.
 *
 * The test forces this mismatch by declaring 'Value' before 'Category' in the PointGenerator HLSL, which
 * gives Value the lower attribute ID (NUM_RESERVED+0) in the producer graph. MetadataPartition's static
 * table only contains Category, so in the consumer graph Category gets NUM_RESERVED+0 - a different mapping.
 * Without the fix, Category is read as Value's data (all zeros), partitioning fails silently, and both
 * Alpha and Beta partitions contain 0 points. With the fix, Alpha and Beta each contain 2 points.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGGPUCompilerExternalBufferReuseHeaderTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.GraphCompilerGPU.ExternalBufferReuseHeader",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGGPUCompilerExternalBufferReuseHeaderTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    // 4 CreateAttributeSet nodes produce [Alpha, Alpha, Beta, Beta] in the 'Category' string attribute.
    const TArray<FString> CategoryValues = { TEXT("Alpha"), TEXT("Alpha"), TEXT("Beta"), TEXT("Beta") };

    UPCGGatherSettings* GatherSettings = nullptr;
    UPCGNode* GatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

    for (const FString& Value : CategoryValues)
    {
        UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
        UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
        CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::String;
        CreateAttrSettings->AttributeTypes.StringValue = Value;
        CreateAttrSettings->OutputTarget.SetAttributeName(FName(TEXT("Category")));
        Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);
    }

    // PointGenerator that copies Category from input AND sets an unused 'Value' int attribute. 'Value' appears BEFORE 'Category' in the HLSL source so that ParseShaderSource
    // assigns Value the lower attribute ID (NUM_RESERVED+0) in the producer graph. Category then gets NUM_RESERVED+1. MetadataPartition's consumer graph has Category at NUM_RESERVED+0,
    // resulting in the ID mismatch that exposes the header bug.
    UPCGCustomHLSLSettings* HLSLSettings = nullptr;
    UPCGNode* PointGenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
    HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
    HLSLSettings->NumElements = CategoryValues.Num();
    // Input pin must be added AFTER SetKernelType, which clears InputPins for generators.
    HLSLSettings->InputPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Param);
    HLSLSettings->SetSourceText(
        TEXT("Out_SetInt(Out_DataIndex, ElementIndex, 'Value', 0);\n")
        TEXT("Out_SetStringKey(Out_DataIndex, ElementIndex, 'Category', In_GetStringKey(ElementIndex, 0, 'Category'));\n")
    );

    UPCGMetadataPartitionSettings* MPSettings = nullptr;
    UPCGNode* PartitionNode = Runner.Graph->AddNodeOfType<UPCGMetadataPartitionSettings>(MPSettings);
    MPSettings->PartitionAttributeSelectors[0].SetAttributeName(FName(TEXT("Category")));
    MPSettings->bDoNotPartition = false;
    MPSettings->SetExecuteOnGPU(true);

    UPCGGatherSettings* OutputGatherSettings = nullptr;
    UPCGNode* OutputGatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(OutputGatherSettings);

    Runner.Graph->AddEdge(GatherNode,    PCGPinConstants::DefaultOutputLabel, PointGenNode,     PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(PointGenNode,  PCGPinConstants::DefaultOutputLabel, PartitionNode,    PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(PartitionNode, PCGPinConstants::DefaultOutputLabel, OutputGatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { OutputGatherNode } }, OutNodeData))
    {
        return false;
    }

    const FPCGDataCollection& OutputCollection = OutNodeData[0];

    // Expect 2 output partitions: Alpha (2 points) and Beta (2 points).
    if (!TestEqual(TEXT("Output data item count"), OutputCollection.TaggedData.Num(), 2))
    {
        return false;
    }

    TMap<FString, int32> ObservedCounts;
    for (const FPCGTaggedData& TaggedData : OutputCollection.TaggedData)
    {
        const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
        if (!TestNotNull(TEXT("Partition output item is point data"), OutPoints))
        {
            return false;
        }

        const FPCGMetadataAttribute<FString>* CategoryAttr =
            OutPoints->ConstMetadata()->GetConstTypedAttribute<FString>(FName(TEXT("Category")));
        if (!TestNotNull(TEXT("Partition output has 'Category' attribute"), CategoryAttr))
        {
            return false;
        }

        // GPU-readback point data uses sequential metadata keys starting at 0.
        const FString CategoryValue = CategoryAttr->GetValueFromItemKey(0);
        ObservedCounts.FindOrAdd(CategoryValue) += OutPoints->GetNumPoints();
    }

    bool bPassed = true;
    const TArray<TPair<FString, int32>> Expected = { { TEXT("Alpha"), 2 }, { TEXT("Beta"), 2 } };
    for (const TPair<FString, int32>& Pair : Expected)
    {
        const int32* Observed = ObservedCounts.Find(Pair.Key);
        bPassed &= TestNotNull(*FString::Printf(TEXT("Partition '%s' present in output"), *Pair.Key), Observed);
        if (Observed)
        {
            bPassed &= TestEqual(*FString::Printf(TEXT("Partition '%s' point count"), *Pair.Key), *Observed, Pair.Value);
        }
    }
    return bPassed;
}

#endif // WITH_EDITOR
