// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "Compute/Elements/PCGCustomHLSL.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGCreateAttribute.h"
#include "Elements/PCGGather.h"
#include "Elements/Metadata/PCGMetadataPartition.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

namespace PCGMetadataPartitionGPUTestHelpers
{
    /**
     * Validates that OutputCollection has the expected total number of data items, and that the
     * summed point count per Category value across all items matches ExpectedCategoryTotals.
     * Order of data items in the collection is not assumed.
     */
    static bool ValidatePartitionOutput(
        FAutomationTestBase* Test,
        const FPCGDataCollection& OutputCollection,
        int32 ExpectedDataItemCount,
        TArrayView<const TPair<FString, int32>> ExpectedCategoryTotals)
    {
        if (!Test->TestEqual(TEXT("Output data item count"),
            OutputCollection.TaggedData.Num(), ExpectedDataItemCount))
        {
            return false;
        }

        // Sum observed point counts per Category value across all output data items.
        TMap<FString, int32> ObservedCounts;
        for (const FPCGTaggedData& TaggedData : OutputCollection.TaggedData)
        {
            const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
            if (!Test->TestNotNull(TEXT("Partition output item is point data"), OutPoints))
            {
                return false;
            }

            const FPCGMetadataAttribute<FString>* CategoryAttr =
                OutPoints->ConstMetadata()->GetConstTypedAttribute<FString>(FName(TEXT("Category")));
            if (!Test->TestNotNull(TEXT("Partition output point data has 'Category' attribute"), CategoryAttr))
            {
                return false;
            }

            // GPU-readback point data uses sequential metadata keys starting at 0.
            const FString CategoryValue = CategoryAttr->GetValueFromItemKey(0);

            // Every element in this partition must carry the same Category value.
            const int32 NumPoints = OutPoints->GetNumPoints();
            for (int64 Key = 1; Key < NumPoints; ++Key)
            {
                Test->TestEqual(
                    *FString::Printf(TEXT("Partition '%s' element %lld has correct value"), *CategoryValue, Key),
                    CategoryAttr->GetValueFromItemKey(Key), CategoryValue);
            }

            ObservedCounts.FindOrAdd(CategoryValue) += NumPoints;
        }

        bool bTestPassed = true;
        for (const TPair<FString, int32>& Pair : ExpectedCategoryTotals)
        {
            const int32* Observed = ObservedCounts.Find(Pair.Key);
            bTestPassed &= Test->TestNotNull(
                *FString::Printf(TEXT("Partition '%s' present in output"), *Pair.Key), Observed);
            if (Observed)
            {
                bTestPassed &= Test->TestEqual(
                    *FString::Printf(TEXT("Partition '%s' total point count"), *Pair.Key),
                    *Observed, Pair.Value);
            }
        }
        return bTestPassed;
    }
}

/**
 * GPU test: MetadataPartition correctly partitions multiple input data objects, each with interleaved
 * Category values.
 *
 * Two groups feed two PointGenerators, each producing one input data object for MetadataPartition:
 *   Group 0 (4 points): Alpha, Gamma, Beta, Gamma  -->  1 Alpha, 1 Beta, 2 Gamma
 *   Group 1 (5 points): Alpha, Beta, Gamma, Beta, Gamma  -->  1 Alpha, 2 Beta, 2 Gamma
 *
 * MetadataPartition partitions each input data object independently (bEmitPerDataCounts=true),
 * producing 3 output data items per group = 6 total. Total counts: Alpha=2, Beta=3, Gamma=4.
 * 
 * todo_pcg: Currently output order of data from GPU Metadata Partition is not guaranteed to match encountered value order in input.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGMetadataPartitionGPU_PartitionTest,
    FPCGTestBaseClass,
    "Plugins.PCG.Compute.MetadataPartition.Partition",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGMetadataPartitionGPU_PartitionTest::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;

    const TArray<TArray<FString>> Groups = {
        { TEXT("Alpha"), TEXT("Gamma"), TEXT("Beta"),  TEXT("Gamma") },
        { TEXT("Alpha"), TEXT("Beta"),  TEXT("Gamma"), TEXT("Beta"),  TEXT("Gamma") },
    };

    UPCGGatherSettings* PrePartitionGatherSettings = nullptr;
    UPCGNode* PrePartitionGatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(PrePartitionGatherSettings);

    for (const TArray<FString>& GroupValues : Groups)
    {
        UPCGGatherSettings* GroupGatherSettings = nullptr;
        UPCGNode* GroupGatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(GroupGatherSettings);

        for (const FString& Value : GroupValues)
        {
            UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
            UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
            CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::String;
            CreateAttrSettings->AttributeTypes.StringValue = Value;
            CreateAttrSettings->OutputTarget.SetAttributeName(FName(TEXT("Category")));
            Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, GroupGatherNode, PCGPinConstants::DefaultInputLabel);
        }

        UPCGCustomHLSLSettings* HLSLSettings = nullptr;
        UPCGNode* PointGenNode = Runner.Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
        HLSLSettings->SetKernelType(EPCGKernelType::PointGenerator);
        HLSLSettings->NumElements = GroupValues.Num();
        // Input pin must be added AFTER SetKernelType, which clears InputPins for generators.
        HLSLSettings->InputPins.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Param);
        HLSLSettings->SetSourceText(
            TEXT("Out_SetStringKey(Out_DataIndex, ElementIndex, 'Category', In_GetStringKey(ElementIndex, 0, 'Category'));\n")
        );

        Runner.Graph->AddEdge(GroupGatherNode, PCGPinConstants::DefaultOutputLabel, PointGenNode,          PCGPinConstants::DefaultInputLabel);
        Runner.Graph->AddEdge(PointGenNode,    PCGPinConstants::DefaultOutputLabel, PrePartitionGatherNode, PCGPinConstants::DefaultInputLabel);
    }

    UPCGMetadataPartitionSettings* MPSettings = nullptr;
    UPCGNode* PartitionNode = Runner.Graph->AddNodeOfType<UPCGMetadataPartitionSettings>(MPSettings);
    MPSettings->PartitionAttributeSelectors[0].SetAttributeName(FName(TEXT("Category")));
    MPSettings->bDoNotPartition = false;
    MPSettings->SetExecuteOnGPU(true);

    UPCGGatherSettings* OutputGatherSettings = nullptr;
    UPCGNode* OutputGatherNode = Runner.Graph->AddNodeOfType<UPCGGatherSettings>(OutputGatherSettings);

    Runner.Graph->AddEdge(PrePartitionGatherNode, PCGPinConstants::DefaultOutputLabel, PartitionNode,   PCGPinConstants::DefaultInputLabel);
    Runner.Graph->AddEdge(PartitionNode,          PCGPinConstants::DefaultOutputLabel, OutputGatherNode, PCGPinConstants::DefaultInputLabel);

    TArray<FPCGDataCollection> OutNodeData;
    if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { OutputGatherNode } }, OutNodeData))
    {
        return false;
    }

    // 2 input data objects x 3 unique values = 6 output data items. Totals: Alpha=2, Beta=3, Gamma=4.
    const TArray<TPair<FString, int32>> ExpectedCategoryTotals = {
        { TEXT("Alpha"), 2 },
        { TEXT("Beta"),  3 },
        { TEXT("Gamma"), 4 },
    };
    return PCGMetadataPartitionGPUTestHelpers::ValidatePartitionOutput(this, OutNodeData[0], /*ExpectedDataItemCount=*/6, ExpectedCategoryTotals);
}

#endif // WITH_EDITOR
