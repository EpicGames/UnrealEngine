// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/Compute/PCGGPUTestCommon.h"

#include "PCGCommon.h"
#include "PCGGraph.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGGather.h"
#include "Elements/PCGTransformPoints.h"

namespace PCGTransformPointsGPUTestHelpers
{
	static const TArray<FTransform> InputTransforms = {
		FTransform(FQuat::Identity, FVector(  0.0,   0.0,   0.0), FVector(1.0, 2.0, 3.0)),
		FTransform(FQuat::Identity, FVector(100.0, 200.0, 300.0), FVector(0.5, 1.5, 2.5)),
		FTransform(FQuat::Identity, FVector(-50.0,  75.0, 250.0), FVector(1.0, 1.0, 1.0)),
		FTransform(FQuat::Identity, FVector( 10.0, -30.0, -80.0), FVector(2.0, 0.5, 1.5)),
	};

	/** Create a TransformPoints node with varied settings that exercise all params. */
	UPCGNode* AddTransformPointsNode(UPCGGraph* Graph, UPCGTransformPointsSettings*& OutSettings)
	{
		OutSettings = nullptr;
		UPCGNode* Node = Graph->AddNodeOfType<UPCGTransformPointsSettings>(OutSettings);
		OutSettings->OffsetMin = FVector(10.0, -5.0, 20.0);
		OutSettings->OffsetMax = FVector(30.0, 15.0, 50.0);
		OutSettings->bAbsoluteOffset = true;
		OutSettings->RotationMin = FRotator(10.0, 20.0, 0.0);
		OutSettings->RotationMax = FRotator(45.0, 90.0, 30.0);
		OutSettings->bAbsoluteRotation = false;
		OutSettings->ScaleMin = FVector(0.5, 1.0, 0.8);
		OutSettings->ScaleMax = FVector(2.0, 3.0, 1.5);
		OutSettings->bAbsoluteScale = false;
		OutSettings->bUniformScale = false;
		OutSettings->bRecomputeSeed = true;
		return Node;
	}
}

/**
 * GPU test: TransformPoints CPU/GPU parity with varied offset, rotation, and scale ranges.
 *
 *   CreatePoints -> TransformPoints -> Gather
 *
 * All settings use distinct min/max ranges to exercise seed-based randomness on both paths.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGTransformPointsGPU_CPUParityTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.TransformPoints.CPUGPUParity",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGTransformPointsGPU_CPUParityTest::RunTest(const FString& Parameters)
{
	using namespace PCGTransformPointsGPUTestHelpers;

	return PCGGPUTestCommon::RunCPUGPUParityTest(this,
		[](UPCGGraph* Graph, UWorld* /*World*/) -> PCGGPUTestCommon::FParityBuildResult
		{
			UPCGNode* CreateNode = PCGTestsCommon::AddCreatePointsNode(Graph, InputTransforms);

			UPCGTransformPointsSettings* TransformSettings = nullptr;
			UPCGNode* TransformNode = AddTransformPointsNode(Graph, TransformSettings);

			UPCGGatherSettings* GatherSettings = nullptr;
			UPCGNode* GatherNode = Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

			Graph->AddEdge(CreateNode, PCGPinConstants::DefaultOutputLabel, TransformNode, PCGPinConstants::DefaultInputLabel);
			Graph->AddEdge(TransformNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

			return { GatherNode, { TransformSettings } };
		},
		[](FAutomationTestBase* Test, const FPCGDataCollection& CPUOutput, const FPCGDataCollection& GPUOutput) -> bool
		{
			return PCGTestsCommon::ComparePointDataCollections(Test, CPUOutput, GPUOutput);
		});
}

/**
 * GPU test: TransformPoints with all overridable params supplied via connected override pins.
 *
 *   CreatePoints -> TransformPoints (all settings default) -> Gather
 *                   ^ CreateAttributeSet nodes connected to each override pin
 *
 * Settings are left at defaults; override pins supply non-trivial values for every overridable param.
 * CPU/GPU parity validates the GPU override system processes all params correctly.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGTransformPointsGPU_OverrideAllParamsTest,
	FPCGTestBaseClass,
	"Plugins.PCG.Compute.TransformPoints.OverrideAllParams",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGTransformPointsGPU_OverrideAllParamsTest::RunTest(const FString& Parameters)
{
	using namespace PCGTransformPointsGPUTestHelpers;

	return PCGGPUTestCommon::RunCPUGPUParityTest(this,
		[](UPCGGraph* Graph, UWorld* /*World*/) -> PCGGPUTestCommon::FParityBuildResult
		{
			UPCGNode* CreateNode = PCGTestsCommon::AddCreatePointsNode(Graph, InputTransforms);

			// TransformPoints with default settings — all overrides come from connected pins.
			UPCGTransformPointsSettings* TransformSettings = nullptr;
			UPCGNode* TransformNode = Graph->AddNodeOfType<UPCGTransformPointsSettings>(TransformSettings);

			// Override all overridable params via connected CreateAttributeSet nodes.
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, OffsetMin), FVector(10.0, -5.0, 20.0));
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, OffsetMax), FVector(30.0, 15.0, 50.0));
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, bAbsoluteOffset), true);
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, RotationMin), FRotator(10.0, 20.0, 0.0));
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, RotationMax), FRotator(45.0, 90.0, 30.0));
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, bAbsoluteRotation), false);
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, ScaleMin), FVector(0.5, 1.0, 0.8));
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, ScaleMax), FVector(2.0, 3.0, 1.5));
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, bAbsoluteScale), true);
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, bUniformScale), false);
			PCGGPUTestCommon::AddOverride(Graph, TransformNode, GET_MEMBER_NAME_CHECKED(UPCGTransformPointsSettings, bRecomputeSeed), true);

			UPCGGatherSettings* GatherSettings = nullptr;
			UPCGNode* GatherNode = Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

			Graph->AddEdge(CreateNode, PCGPinConstants::DefaultOutputLabel, TransformNode, PCGPinConstants::DefaultInputLabel);
			Graph->AddEdge(TransformNode, PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

			return { GatherNode, { TransformSettings } };
		},
		[](FAutomationTestBase* Test, const FPCGDataCollection& CPUOutput, const FPCGDataCollection& GPUOutput) -> bool
		{
			return PCGTestsCommon::ComparePointDataCollections(Test, CPUOutput, GPUOutput);
		});
}

#endif // WITH_EDITOR
