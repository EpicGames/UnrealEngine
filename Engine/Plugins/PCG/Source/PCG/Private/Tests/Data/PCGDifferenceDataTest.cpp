// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"
#include "Data/PCGDifferenceData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGVolumeData.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDifferenceDataTest, FPCGTestBaseClass, "Plugins.PCG.Difference.Data", PCGTestsCommon::TestFlags)

bool FPCGDifferenceDataTest::RunTest(const FString& Parameters)
{
	const UPCGVolumeData* FirstVolume = PCGTestsCommon::CreateVolumeData(FBox::BuildAABB(FVector::OneVector * 250, FVector::OneVector * 500));

	const UPCGVolumeData* SecondVolume = PCGTestsCommon::CreateVolumeData(FBox::BuildAABB(FVector::OneVector * -250, FVector::OneVector * 500));

	const UPCGVolumeData* ThirdVolume = PCGTestsCommon::CreateVolumeData(FBox::BuildAABB(FVector::OneVector * 10000, FVector::OneVector * 500));

	// Inside first volume
	UPCGBasePointData* FirstVolumeTestPoint = PCGTestsCommon::CreateBasePointData(FVector::OneVector * 500);
	// Inside second volume
	UPCGBasePointData* SecondVolumeTestPoint = PCGTestsCommon::CreateBasePointData(FVector::OneVector * -500);
	// Point at origin
	UPCGBasePointData* OriginTestPoint = PCGTestsCommon::CreateBasePointData();
	// Point far from anything
	UPCGBasePointData* FarOffTestPoint = PCGTestsCommon::CreateBasePointData(FVector::OneVector * -10000);

	// Tests for the DifferenceData itself
	auto ValidateVolumeDifferenceData = [this](const UPCGVolumeData* OriginalVolume, const UPCGVolumeData* DifferenceVolume) -> const UPCGDifferenceData*
	{
		// Basic data validation
		if (!TestNotNull("OriginalVolume", OriginalVolume) || !TestNotNull("DifferenceVolume", DifferenceVolume))
		{
			return nullptr;
		}

		// Create difference data with Subtract
		const UPCGDifferenceData* DifferenceData = OriginalVolume->Subtract(nullptr, DifferenceVolume);

		if (!TestNotNull("DifferenceData", DifferenceData))
		{
			return nullptr;
		}

		// Dimension of a volume overlapping another should be a 3D volume
		TestEqual("DifferenceData dimension", DifferenceData->GetDimension(), 3);

		// Validate the bounds
		const FBox Bounds = DifferenceData->GetBounds();
		TestTrue("DifferenceData bounds is valid", !!Bounds.IsValid); // !! to convert uint8 to bool
		TestEqual("DifferenceData Bounds", Bounds, OriginalVolume->GetBounds());

		return DifferenceData;
	};

	// Tests points in the various volumes
	auto ValidatePointsShouldExist = [this](const UPCGDifferenceData* DifferenceData, const UPCGBasePointData* PointData, bool PointShouldExist)
	{
		check(PointData);

		// Basic data validation
		if (!TestNotNull("DifferenceData", DifferenceData))
		{
			return;
		}

		check(PointData->GetNumPoints() == 1);
		// Validate that we're able to sample a point
		const FConstPCGPointValueRanges ValueRanges(PointData);
		FPCGPoint Point = ValueRanges.GetPoint(0);

		FPCGPoint SampledPoint;
		bool PointWasSampled = DifferenceData->SamplePoint(Point.Transform, Point.GetLocalBounds(), SampledPoint, nullptr);

		// If the sampled point should exist, and if so, compare it to the original point
		TestTrue("Point sampled", PointWasSampled == PointShouldExist);
		if (PointWasSampled && PointShouldExist)
		{
			TestTrue("Sampled point identical", PCGTestsCommon::PointsAreIdentical(Point, SampledPoint));
		}

		// Generate PointData
		const UPCGBasePointData* OutputPointData = DifferenceData->ToBasePointData(nullptr);
		if (TestNotNull("PointData generated", OutputPointData))
		{
			// If the point should be generated
			TestTrue("PointData exists", OutputPointData->GetNumPoints() > 0);
		}
	};

	// Create and validate difference data with subtraction
	const UPCGDifferenceData* FirstSubSecond = ValidateVolumeDifferenceData(FirstVolume, SecondVolume); // Should exist
	const UPCGDifferenceData* SecondSubFirst = ValidateVolumeDifferenceData(SecondVolume, FirstVolume); // Should exist
	const UPCGDifferenceData* FirstSubThird = ValidateVolumeDifferenceData(FirstVolume, ThirdVolume);   // Should not exist
	const UPCGDifferenceData* ThirdSubFirst = ValidateVolumeDifferenceData(ThirdVolume, FirstVolume);   // Should not exist

	// Run lambda tests
	ValidatePointsShouldExist(FirstSubSecond, FirstVolumeTestPoint, true);   // First point inside first volume only
	ValidatePointsShouldExist(FirstSubSecond, SecondVolumeTestPoint, false); // Second point inside second volume only
	ValidatePointsShouldExist(FirstSubSecond, OriginTestPoint, false);       // Origin in subtracted volume

	ValidatePointsShouldExist(SecondSubFirst, FirstVolumeTestPoint, false);  // First point in subtracted volume
	ValidatePointsShouldExist(SecondSubFirst, SecondVolumeTestPoint, true);  // Second point in second volume only
	ValidatePointsShouldExist(SecondSubFirst, OriginTestPoint, false);       // Origin in subtracted volume

	ValidatePointsShouldExist(FirstSubThird, FirstVolumeTestPoint, true);    // First point in first volume only
	ValidatePointsShouldExist(FirstSubThird, SecondVolumeTestPoint, false);  // Second point not in either volume
	ValidatePointsShouldExist(FirstSubThird, OriginTestPoint, true);         // Origin in first volume, no subtracted volume

	ValidatePointsShouldExist(ThirdSubFirst, FirstVolumeTestPoint, false);   // No points in this volume
	ValidatePointsShouldExist(ThirdSubFirst, SecondVolumeTestPoint, false);
	ValidatePointsShouldExist(ThirdSubFirst, OriginTestPoint, false);

	ValidatePointsShouldExist(FirstSubSecond, FarOffTestPoint, false); // Far off point shouldn't be in any volume
	ValidatePointsShouldExist(SecondSubFirst, FarOffTestPoint, false);
	ValidatePointsShouldExist(FirstSubThird, FarOffTestPoint, false);
	ValidatePointsShouldExist(ThirdSubFirst, FarOffTestPoint, false);

	return true;
}