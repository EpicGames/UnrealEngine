// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"
#include "Data/PCGIntersectionData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGVolumeData.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGIntersectionDataTest, FPCGTestBaseClass, "Plugins.PCG.Intersection.Data", PCGTestsCommon::TestFlags)

bool FPCGIntersectionDataTest::RunTest(const FString& Parameters)
{
	UPCGBasePointData* InsidePoint = PCGTestsCommon::CreateBasePointData();
	check(InsidePoint->GetNumPoints() == 1);

	UPCGBasePointData* OutsidePoint = PCGTestsCommon::CreateBasePointData(FVector::OneVector * 10000);
	check(OutsidePoint->GetNumPoints() == 1);

	UPCGVolumeData* Volume = PCGTestsCommon::CreateVolumeData(FBox::BuildAABB(FVector::ZeroVector, FVector::OneVector * 100));

	// Create intersections
	UPCGIntersectionData* InsideVolume = InsidePoint->IntersectWith(nullptr, Volume);
	UPCGIntersectionData* VolumeInside = Volume->IntersectWith(nullptr, InsidePoint);
	UPCGIntersectionData* OutsideVolume = OutsidePoint->IntersectWith(nullptr, Volume);
	UPCGIntersectionData* VolumeOutside = Volume->IntersectWith(nullptr, OutsidePoint);

	auto ValidateInsideIntersection = [this, InsidePoint](UPCGIntersectionData* Intersection)
	{
		// Basic data validations
		TestTrue("Valid intersection", Intersection != nullptr);

		if (!Intersection)
		{
			return;
		}

		TestTrue("Valid dimension", Intersection->GetDimension() == 0);
		TestTrue("Valid bounds", Intersection->GetBounds() == InsidePoint->GetBounds());
		TestTrue("Valid strict bounds", Intersection->GetStrictBounds() == InsidePoint->GetStrictBounds());

		// Validate sample point
		FConstPCGPointValueRanges ValueRanges(InsidePoint);
		const FPCGPoint Point = ValueRanges.GetPoint(0);

		FPCGPoint SampledPoint;
		TestTrue("Successful point sampling", Intersection->SamplePoint(Point.Transform, Point.GetLocalBounds(), SampledPoint, nullptr));
		TestTrue("Correct sampled point", PCGTestsCommon::PointsAreIdentical(Point, SampledPoint));

		// Validate create point data
		const UPCGBasePointData* OutputPointData = Intersection->ToBasePointData(nullptr);
		TestTrue("Successful ToPoint", OutputPointData != nullptr);
		
		if (OutputPointData)
		{
			TestTrue("Valid number of points in ToPoint", OutputPointData->GetNumPoints() == 1);
			if (OutputPointData->GetNumPoints() == 1)
			{
				const FConstPCGPointValueRanges OutValueRanges(OutputPointData);
				TestTrue("Correct point in ToPoint", PCGTestsCommon::PointsAreIdentical(Point, OutValueRanges.GetPoint(0)));
			}
		}
	};

	ValidateInsideIntersection(InsideVolume);
	ValidateInsideIntersection(VolumeInside);

	auto ValidateOutsideIntersection = [this, OutsidePoint](UPCGIntersectionData* Intersection)
	{
		TestTrue("Valid intersection", Intersection != nullptr);

		if (!Intersection)
		{
			return;
		}

		TestTrue("Valid dimension", Intersection->GetDimension() == 0);
		TestTrue("Null bounds", !Intersection->GetBounds().IsValid);
		TestTrue("Null strict bounds", !Intersection->GetStrictBounds().IsValid);

		// Validate that we're not able to sample a point
		FConstPCGPointValueRanges ValueRanges(OutsidePoint);
		const FPCGPoint Point = ValueRanges.GetPoint(0);

		FPCGPoint SampledPoint;
		TestTrue("Unsuccessful point sampling", !Intersection->SamplePoint(Point.Transform, Point.GetLocalBounds(), SampledPoint, nullptr));

		// Validate empty point data
		const UPCGBasePointData* OutputPointData = Intersection->ToBasePointData(nullptr);
		TestTrue("Successful ToPoint", OutputPointData != nullptr);

		if (OutputPointData)
		{
			TestTrue("Empty point data", OutputPointData->GetNumPoints() == 0);
		}
	};

	ValidateOutsideIntersection(OutsideVolume);
	ValidateOutsideIntersection(VolumeOutside);

	return true;
}

//TOODs:
// Test with one/two data that do not have a trivial transformation (e.g. projection, surfaces, ...)