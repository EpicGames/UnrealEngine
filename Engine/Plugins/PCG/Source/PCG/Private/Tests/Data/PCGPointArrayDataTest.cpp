// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"
#include "Data/PCGPointArrayData.h"

#define PCG_FMT(Text, ...) FString::Printf(TEXT(Text), __VA_ARGS__)

class FPCGPointArrayDataTestBaseClass : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;

	template <EPCGPointNativeProperties NativeProperty>
	bool TestEqualRange(const UPCGBasePointData* PointDataA, const UPCGBasePointData* PointDataB)
	{		
		const typename TPCGPointNativeProperty<NativeProperty>::ConstValueRange RangeA = PointDataA->GetConstValueRange<NativeProperty>();
		const typename TPCGPointNativeProperty<NativeProperty>::ConstValueRange RangeB = PointDataB->GetConstValueRange<NativeProperty>();

		const FString PropertyName = StaticEnum<EPCGPointNativeProperties>()->GetNameStringByValue((int64)TPCGPointNativeProperty<NativeProperty>::EnumValue);
		UTEST_EQUAL(PCG_FMT("Property %s : RangeA.Num() == RangeB.Num()", *PropertyName), RangeA.Num(), RangeB.Num());

		for (int32 i = 0; i < RangeA.Num(); ++i)
		{
			UTEST_EQUAL(PCG_FMT("Property % s : RangeA[%d] == RangeB[%d]", *PropertyName, i, i), RangeA[i], RangeB[i]);
		}

		return true;
	};

	bool TestEqualData(const UPCGBasePointData* PointDataA, const UPCGBasePointData* PointDataB)
	{
		if (!TestEqualRange<EPCGPointNativeProperties::Transform>(PointDataA, PointDataB))
		{
			return false;
		}

		if (!TestEqualRange<EPCGPointNativeProperties::Density>(PointDataA, PointDataB))
		{
			return false;
		}

		if (!TestEqualRange<EPCGPointNativeProperties::BoundsMin>(PointDataA, PointDataB))
		{
			return false;
		}

		if (!TestEqualRange<EPCGPointNativeProperties::BoundsMax>(PointDataA, PointDataB))
		{
			return false;
		}

		if (!TestEqualRange<EPCGPointNativeProperties::Color>(PointDataA, PointDataB))
		{
			return false;
		}

		if (!TestEqualRange<EPCGPointNativeProperties::Steepness>(PointDataA, PointDataB))
		{
			return false;
		}

		if (!TestEqualRange<EPCGPointNativeProperties::Seed>(PointDataA, PointDataB))
		{
			return false;
		}

		if (!TestEqualRange<EPCGPointNativeProperties::MetadataEntry>(PointDataA, PointDataB))
		{
			return false;
		}

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataInitializeFromDataTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.InitializeFromData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataDuplicateDataTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.DuplicateData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataFlattenTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.Flatten", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataMultiLevelInheritanceTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.MultiLevelInheritance", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataToPointDataTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.ToPointData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataToPointArrayDataTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.ToPointArray", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataDisabledParentingTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.DisabledParenting", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataCopyPointsToFromPointArrayDataToPointDataTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.CopyPointsToFromPointArrayDataToPointData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataCopyPointsToFromPointArrayDataToPointArrayDataTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.CopyPointsFromPointArrayDataToPointArrayData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataCopyPointsToFromPointDataToPointDataTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.CopyPointsToFromPointDataToPointData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataCopyPointsToFromPointDataToPointArrayDataTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.CopyPointsFromPointDataToPointArrayData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointArrayDataCopyPropertyOverInheritedPropertyTest, FPCGPointArrayDataTestBaseClass, "Plugins.PCG.PointArrayData.CopyPropertyOverInheritedProperty", PCGTestsCommon::TestFlags)

bool FPCGPointArrayDataInitializeFromDataTest::RunTest(const FString& Parameters)
{
	// Enable Parenting
	const bool bPreviousCVarValue = CVarPCGEnablePointArrayDataParenting.GetValueOnAnyThread();
	CVarPCGEnablePointArrayDataParenting->Set(true, ECVF_SetByCode);
	ON_SCOPE_EXIT
	{
		CVarPCGEnablePointArrayDataParenting->Set(bPreviousCVarValue, ECVF_SetByCode);
	};

	UPCGPointArrayData* ParentData = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(100, 42);
	EPCGPointNativeProperties ParentAllocatedProperties = EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Color | EPCGPointNativeProperties::Density;

	// CreateRandomPointData only allocates those properties
	TestEqual("Valid ParentAllocatedProperties", (int32)ParentAllocatedProperties, (int32)ParentData->GetAllocatedProperties());
		
	UPCGPointArrayData* ChildData = PCGTestsCommon::CreateEmptyPointData<UPCGPointArrayData>();
	EPCGPointNativeProperties ChildAllocatedProperties = EPCGPointNativeProperties::None;

	// CreateEmptyPointData should not allocate any properties
	TestEqual("Valid ChildAllocatedProperties", (int32)ChildAllocatedProperties, (int32)ChildData->GetAllocatedProperties());
	TestTrue("ChildData has 0 points", ChildData->GetNumPoints() == 0);

	ChildData->InitializeFromData(ParentData);
	TestEqual("ChildData->GetAllocatedProperties() == ParentData->GetAllocatedProperties()", (int32)ChildData->GetAllocatedProperties(), (int32)ParentData->GetAllocatedProperties());
	TestTrue("ChildData->GetNumPoints() == ParentData->GetNumPoints()", ChildData->GetNumPoints() == ParentData->GetNumPoints());
	TestTrue("ChildData has parent", ChildData->HasSpatialDataParent());
	TestFalse("ParentData has no parent", ParentData->HasSpatialDataParent());

	// Compare data
	return TestEqualData(ParentData, ChildData);
}

bool FPCGPointArrayDataDuplicateDataTest::RunTest(const FString& Parameters)
{
	// Enable Parenting
	const bool bPreviousCVarValue = CVarPCGEnablePointArrayDataParenting.GetValueOnAnyThread();
	CVarPCGEnablePointArrayDataParenting->Set(true, ECVF_SetByCode);
	ON_SCOPE_EXIT
	{
		CVarPCGEnablePointArrayDataParenting->Set(bPreviousCVarValue, ECVF_SetByCode);
	};

	UPCGPointArrayData* ParentData = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(100, 42);
	EPCGPointNativeProperties ParentAllocatedProperties = EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Color | EPCGPointNativeProperties::Density;

	// CreateRandomPointData only allocates those properties
	TestEqual("Valid ParentAllocatedProperties", (int32)ParentAllocatedProperties, (int32)ParentData->GetAllocatedProperties());

	UPCGPointArrayData* ChildData = CastChecked<UPCGPointArrayData>(ParentData->DuplicateData(nullptr));
		
	TestEqual("ChildData->GetAllocatedProperties() == ParentData->GetAllocatedProperties()", (int32)ChildData->GetAllocatedProperties(), (int32)ParentData->GetAllocatedProperties());
	TestTrue("ChildData->GetNumPoints() == ParentData->GetNumPoints()", ChildData->GetNumPoints() == ParentData->GetNumPoints());
	TestTrue("ChildData has parent", ChildData->HasSpatialDataParent());
	TestFalse("ParentData has no parent", ParentData->HasSpatialDataParent());

	// Compare data
	return TestEqualData(ParentData, ChildData);
}

bool FPCGPointArrayDataFlattenTest::RunTest(const FString& Parameters)
{
	// Enable Parenting
	const bool bPreviousCVarValue = CVarPCGEnablePointArrayDataParenting.GetValueOnAnyThread();
	CVarPCGEnablePointArrayDataParenting->Set(true, ECVF_SetByCode);
	ON_SCOPE_EXIT
	{
		CVarPCGEnablePointArrayDataParenting->Set(bPreviousCVarValue, ECVF_SetByCode);
	};

	UPCGPointArrayData* ParentData = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(100, 42);
	EPCGPointNativeProperties ParentAllocatedProperties = EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Color | EPCGPointNativeProperties::Density;

	// CreateRandomPointData only allocates those properties
	TestEqual("Valid ParentAllocatedProperties", (int32)ParentAllocatedProperties, (int32)ParentData->GetAllocatedProperties());

	UPCGPointArrayData* ChildData = PCGTestsCommon::CreateEmptyPointData<UPCGPointArrayData>();
	EPCGPointNativeProperties ChildAllocatedProperties = EPCGPointNativeProperties::None;

	// CreateEmptyPointData should not allocate any properties
	TestEqual("Valid ChildAllocatedProperties", (int32)ChildAllocatedProperties, (int32)ChildData->GetAllocatedProperties());
	TestTrue("ChildData has 0 points", ChildData->GetNumPoints() == 0);

	ChildData->InitializeFromData(ParentData);
	TestEqual("ChildData->GetAllocatedProperties() == ParentData->GetAllocatedProperties()", (int32)ChildData->GetAllocatedProperties(), (int32)ParentData->GetAllocatedProperties());
	TestTrue("ChildData->GetNumPoints() == ParentData->GetNumPoints()", ChildData->GetNumPoints() == ParentData->GetNumPoints());
	TestTrue("ChildData has parent", ChildData->HasSpatialDataParent());
	TestFalse("ParentData has no parent", ParentData->HasSpatialDataParent());

	// This will copy parent properties into child memory, at this point we should no longer inherit from parent
	ChildData->Flatten();
	TestFalse("ChildData has parent", ChildData->HasSpatialDataParent());

	// Compare data
	return TestEqualData(ParentData, ChildData);
}

bool FPCGPointArrayDataMultiLevelInheritanceTest::RunTest(const FString& Parameters)
{
	// Enable Parenting
	const bool bPreviousCVarValue = CVarPCGEnablePointArrayDataParenting.GetValueOnAnyThread();
	CVarPCGEnablePointArrayDataParenting->Set(true, ECVF_SetByCode);
	ON_SCOPE_EXIT
	{
		CVarPCGEnablePointArrayDataParenting->Set(bPreviousCVarValue, ECVF_SetByCode);
	};

	UPCGPointArrayData* ParentData = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(100, 42);
	EPCGPointNativeProperties ParentAllocatedProperties = EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Color | EPCGPointNativeProperties::Density;

	// CreateRandomPointData only allocates those properties
	TestEqual("Valid ParentAllocatedProperties", (int32)ParentAllocatedProperties, (int32)ParentData->GetAllocatedProperties());

	UPCGPointArrayData* ChildData = PCGTestsCommon::CreateEmptyPointData<UPCGPointArrayData>();
	EPCGPointNativeProperties ChildAllocatedProperties = EPCGPointNativeProperties::None;

	// CreateEmptyPointData should not allocate any properties
	TestEqual("Valid ChildAllocatedProperties", (int32)ChildAllocatedProperties, (int32)ChildData->GetAllocatedProperties());
	TestTrue("ChildData has 0 points", ChildData->GetNumPoints() == 0);

	ChildData->InitializeFromData(ParentData);
	TestEqual("ChildData->GetAllocatedProperties() == ParentData->GetAllocatedProperties()", (int32)ChildData->GetAllocatedProperties(), (int32)ParentData->GetAllocatedProperties());
	TestTrue("ChildData->GetNumPoints() == ParentData->GetNumPoints()", ChildData->GetNumPoints() == ParentData->GetNumPoints());
	TestTrue("ChildData has parent", ChildData->HasSpatialDataParent());
	TestFalse("ParentData has no parent", ParentData->HasSpatialDataParent());
	
	UPCGPointArrayData* GrandChildData = CastChecked<UPCGPointArrayData>(ChildData->DuplicateData(nullptr));
	
	TestEqual("GrandChildData->GetAllocatedProperties() == ChildData->GetAllocatedProperties()", (int32)GrandChildData->GetAllocatedProperties(), (int32)ChildData->GetAllocatedProperties());
	TestTrue("GrandChildData->GetNumPoints() == ChildData->GetNumPoints()", GrandChildData->GetNumPoints() == ChildData->GetNumPoints());
	TestTrue("GrandChildData has parent", GrandChildData->HasSpatialDataParent());
	TestTrue("ChildData has parent", ChildData->HasSpatialDataParent());

	// Compare data
	if (!TestEqualData(ParentData, ChildData))
	{
		return false;
	}

	if (!TestEqualData(ParentData, GrandChildData))
	{
		return false;
	}

	return TestEqualData(ChildData, GrandChildData);
}

bool FPCGPointArrayDataToPointDataTest::RunTest(const FString& Parameters)
{
	UPCGPointArrayData* PointArrayData = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(100, 42);
	const UPCGPointData* PointData = PointArrayData->ToPointData(nullptr);

	return TestEqualData(PointArrayData, PointData);
}

bool FPCGPointArrayDataToPointArrayDataTest::RunTest(const FString& Parameters)
{
	UPCGPointData* PointData = PCGTestsCommon::CreateRandomPointData<UPCGPointData>(100, 42);
	const UPCGPointArrayData* PointArrayData = PointData->ToPointArrayData(nullptr);

	return TestEqualData(PointArrayData, PointData);
}

bool FPCGPointArrayDataDisabledParentingTest::RunTest(const FString& Parameters)
{
	// Disable Parenting
	const bool bPreviousCVarValue = CVarPCGEnablePointArrayDataParenting.GetValueOnAnyThread();
	CVarPCGEnablePointArrayDataParenting->Set(false, ECVF_SetByCode);
	ON_SCOPE_EXIT
	{
		CVarPCGEnablePointArrayDataParenting->Set(bPreviousCVarValue, ECVF_SetByCode);
	};

	UPCGPointArrayData* ParentData = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(100, 42);
	EPCGPointNativeProperties ParentAllocatedProperties = EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Color | EPCGPointNativeProperties::Density;

	// CreateRandomPointData only allocates those properties
	TestEqual("Valid ParentAllocatedProperties", (int32)ParentAllocatedProperties, (int32)ParentData->GetAllocatedProperties());

	UPCGPointArrayData* ChildDataA = PCGTestsCommon::CreateEmptyPointData<UPCGPointArrayData>();
	EPCGPointNativeProperties ChildAllocatedProperties = EPCGPointNativeProperties::None;

	// CreateEmptyPointData should not allocate any properties
	TestEqual("Valid ChildDataA ChildAllocatedProperties", (int32)ChildAllocatedProperties, (int32)ChildDataA->GetAllocatedProperties());
	TestTrue("ChildDataA has 0 points", ChildDataA->GetNumPoints() == 0);

	ChildDataA->InitializeFromData(ParentData);
	TestTrue("ChildDataA has 0 points (After InitializeFromData)", ChildDataA->GetNumPoints() == 0);
	TestFalse("ChildDataA has no parent", ChildDataA->HasSpatialDataParent());
	
	UPCGPointArrayData* ChildDataB = CastChecked<UPCGPointArrayData>(ParentData->DuplicateData(nullptr));
	
	TestEqual("ChildDataB->GetAllocatedProperties() == ParentData->GetAllocatedProperties()", (int32)ChildDataB->GetAllocatedProperties(), (int32)ParentData->GetAllocatedProperties());
	TestTrue("ChildDataB->GetNumPoints() == ParentData->GetNumPoints()", ChildDataB->GetNumPoints() == ParentData->GetNumPoints());
	TestFalse("ChildDataB has no parent", ChildDataB->HasSpatialDataParent());
	
	// Compare data
	return TestEqualData(ParentData, ChildDataB);
}

bool FPCGPointArrayDataCopyPointsToFromPointArrayDataToPointDataTest::RunTest(const FString& Parameters)
{
	UPCGPointArrayData* PointArrayData = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(100, 42);
	UPCGPointData* PointData = PCGTestsCommon::CreateEmptyPointData<UPCGPointData>();

	PointData->SetNumPoints(PointArrayData->GetNumPoints());
	PointArrayData->CopyPointsTo(PointData, 0, 0, PointArrayData->GetNumPoints());

	return TestEqualData(PointArrayData, PointData);
}

bool FPCGPointArrayDataCopyPointsToFromPointArrayDataToPointArrayDataTest::RunTest(const FString& Parameters)
{
	UPCGPointArrayData* PointArrayData = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(100, 42);
	UPCGPointArrayData* PointArrayData2 = PCGTestsCommon::CreateEmptyPointData<UPCGPointArrayData>();

	PointArrayData2->SetNumPoints(PointArrayData->GetNumPoints());
	PointArrayData->CopyPointsTo(PointArrayData2, 0, 0, PointArrayData->GetNumPoints());

	return TestEqualData(PointArrayData, PointArrayData2);
}

bool FPCGPointArrayDataCopyPointsToFromPointDataToPointArrayDataTest::RunTest(const FString& Parameters)
{
	UPCGPointData* PointData = PCGTestsCommon::CreateRandomPointData<UPCGPointData>(100, 42);
	UPCGPointArrayData* PointArrayData = PCGTestsCommon::CreateEmptyPointData<UPCGPointArrayData>();

	PointArrayData->SetNumPoints(PointData->GetNumPoints());
	PointData->CopyPointsTo(PointArrayData, 0, 0, PointData->GetNumPoints());

	return TestEqualData(PointArrayData, PointData);
}

bool FPCGPointArrayDataCopyPointsToFromPointDataToPointDataTest::RunTest(const FString& Parameters)
{
	UPCGPointData* PointData = PCGTestsCommon::CreateRandomPointData<UPCGPointData>(100, 42);
	UPCGPointData* PointData2 = PCGTestsCommon::CreateEmptyPointData<UPCGPointData>();

	PointData2->SetNumPoints(PointData->GetNumPoints());
	PointData->CopyPointsTo(PointData2, 0, 0, PointData->GetNumPoints());

	return TestEqualData(PointData2, PointData);
}

bool FPCGPointArrayDataCopyPropertyOverInheritedPropertyTest::RunTest(const FString& Parameters)
{
	// Enable Parenting
	const bool bPreviousCVarValue = CVarPCGEnablePointArrayDataParenting.GetValueOnAnyThread();
	CVarPCGEnablePointArrayDataParenting->Set(true, ECVF_SetByCode);
	ON_SCOPE_EXIT
	{
		CVarPCGEnablePointArrayDataParenting->Set(bPreviousCVarValue, ECVF_SetByCode);
	};

	UPCGPointArrayData* ParentData = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(100, 42);
	EPCGPointNativeProperties ParentAllocatedProperties = EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Color | EPCGPointNativeProperties::Density;

	// CreateRandomPointData only allocates those properties
	TestEqual("Valid ParentAllocatedProperties", (int32)ParentAllocatedProperties, (int32)ParentData->GetAllocatedProperties());

	UPCGPointArrayData* ChildData = PCGTestsCommon::CreateEmptyPointData<UPCGPointArrayData>();
	EPCGPointNativeProperties ChildAllocatedProperties = EPCGPointNativeProperties::None;

	// CreateEmptyPointData should not allocate any properties
	TestEqual("Valid ChildAllocatedProperties", (int32)ChildAllocatedProperties, (int32)ChildData->GetAllocatedProperties());
	TestTrue("ChildData has 0 points", ChildData->GetNumPoints() == 0);

	ChildData->InitializeFromData(ParentData);
	TestEqual("ChildData->GetAllocatedProperties() == ParentData->GetAllocatedProperties()", (int32)ChildData->GetAllocatedProperties(), (int32)ParentData->GetAllocatedProperties());
	TestTrue("ChildData->GetNumPoints() == ParentData->GetNumPoints()", ChildData->GetNumPoints() == ParentData->GetNumPoints());
	TestTrue("ChildData has parent", ChildData->HasSpatialDataParent());
	TestFalse("ParentData has no parent", ParentData->HasSpatialDataParent());

	if (!TestEqualData(ParentData, ChildData))
	{
		return false;
	}

	UPCGPointArrayData* OtherData = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(100, 57);

	OtherData->CopyPropertiesTo(ChildData, 0, 0, OtherData->GetNumPoints(), EPCGPointNativeProperties::Color | EPCGPointNativeProperties::Seed);
	TestTrue("ChildData has parent (after CopyPropertiesTo)", ChildData->HasSpatialDataParent());

	if (!TestEqualRange<EPCGPointNativeProperties::Color>(ChildData, OtherData))
	{
		return false;
	}

	if (!TestEqualRange<EPCGPointNativeProperties::Seed>(ChildData, OtherData))
	{
		return false;
	}

	if (!TestEqualRange<EPCGPointNativeProperties::Transform>(ChildData, ParentData))
	{
		return false;
	}

	if (!TestEqualRange<EPCGPointNativeProperties::Density>(ChildData, ParentData))
	{
		return false;
	}

	OtherData->CopyPropertiesTo(ChildData, 0, 0, OtherData->GetNumPoints(), EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Density);
	ChildData->Flatten(); // Make sure we don't have inherited properties anymore (non allocated ones)
	TestFalse("ChildData has no parent (after CopyPropertiesTo)", ChildData->HasSpatialDataParent());
		
	return TestEqualData(ChildData, OtherData);
}

#undef PCG_FMT