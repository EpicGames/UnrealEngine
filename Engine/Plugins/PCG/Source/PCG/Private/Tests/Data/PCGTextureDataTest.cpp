// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "Data/PCGTextureData.h"

#include "Engine/Texture2D.h"
#include "TextureResource.h"

#if WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGTextureDataOffsetTilingRotation, FPCGTestBaseClass, "Plugins.PCG.Texture.OffsetTilingRotation", PCGTestsCommon::TestFlags)

bool FPCGTextureDataOffsetTilingRotation::RunTest(const FString& Parameters)
{
	const int32 TextureSize = 128;
	const int32 WhitePixelX = 50;
	const int32 WhitePixelY = 70;

	TArray<FColor> Pixels;
	Pixels.Init(FColor::Black, TextureSize * TextureSize);
	Pixels[WhitePixelY * TextureSize + WhitePixelX] = FColor::White;

	UTexture2D* Texture2D = UTexture2D::CreateTransient(TextureSize, TextureSize, EPixelFormat::PF_B8G8R8A8);
	Texture2D->CompressionSettings = TextureCompressionSettings::TC_VectorDisplacementmap;
	Texture2D->SRGB = 0;
	Texture2D->MipGenSettings = TMGS_NoMipmaps;
	Texture2D->UpdateResource();

	void* RawTextureData = Texture2D->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE);

	FMemory::Memcpy(RawTextureData, Pixels.GetData(), Pixels.Num() * sizeof(FColor));

	Texture2D->GetPlatformData()->Mips[0].BulkData.Unlock();
	Texture2D->UpdateResource();

	UPCGTextureData* TextureData = NewObject<UPCGTextureData>();
	TextureData->bUseAdvancedTiling = true;

	bool bTestPassed = true;

	FTransform Transform;
	while (!TextureData->Initialize(Texture2D, /*TextureIndex=*/0, Transform)) {}

	// CPU readback is deferred; poll until ColorData is populated before sampling.
	while (!TextureData->RequestCPUReadback()) {}

	bTestPassed &= TestTrue("Texture data successfully initialized", TextureData->IsSuccessfullyInitialized());

	if (TextureData->IsSuccessfullyInitialized())
	{
		FRandomStream RandomStream;
		FPCGPoint OutPoint;

		const float PixelCenterX = WhitePixelX + 0.5f;
		const float PixelCenterY = WhitePixelY + 0.5f;

		const float TextureSpacePixelX = PixelCenterX / TextureSize;
		const float ScaledPixelX = (2.0f * TextureSpacePixelX) - 1.0f;
		const float TextureSpacePixelY = PixelCenterY / TextureSize;
		const float ScaledPixelY = (2.0f * TextureSpacePixelY) - 1.0f;

		{
			TextureData->Rotation = 0.f;

			// sampling with no offset
			TextureData->CenterOffset = FVector2D::ZeroVector;
			TextureData->SamplePoint(FTransform(), FBox(), OutPoint, nullptr);
			bTestPassed &= TestEqual("Valid color sampled for no offset at (0, 0)", OutPoint.Color, static_cast<FVector>(FColor::Black));

			// sampling at position, with no offset
			TextureData->CenterOffset = FVector2D::ZeroVector;
			TextureData->SamplePoint(FTransform(FVector(0.5, 0.5, 0.0)), FBox(), OutPoint, nullptr);
			bTestPassed &= TestEqual("Valid color sampled for no offset at position", OutPoint.Color, static_cast<FVector>(FColor::Black));

			// sampling at expected position
			TextureData->CenterOffset = FVector2D::ZeroVector;
			TextureData->SamplePoint(FTransform(FVector(ScaledPixelX, ScaledPixelY, 0)), FBox(), OutPoint, nullptr);
			bTestPassed &= TestEqual("Valid color sampled at expected position", OutPoint.Color, static_cast<FVector>(FColor::White));

			// sampling with offset
			TextureData->CenterOffset = FVector2D(0.5 - TextureSpacePixelX, 0.5 - TextureSpacePixelY);
			TextureData->SamplePoint(FTransform(), FBox(), OutPoint, nullptr);
			bTestPassed &= TestEqual("Valid color sampled for offset from (0, 0)", OutPoint.Color, static_cast<FVector>(FColor::White));

			// sampling at position, with offset
			TextureData->CenterOffset = FVector2D(1.0 - TextureSpacePixelX, 1.0 - TextureSpacePixelY);
			TextureData->SamplePoint(FTransform(FVector(1.0, 1.0, 0.0)), FBox(), OutPoint, nullptr);
			bTestPassed &= TestEqual("Valid color sampled for offset from position", OutPoint.Color, static_cast<FVector>(FColor::White));

			for (float Rotation = -360.f; Rotation < 360.f; Rotation += 10.f)
			{
				const float Theta = FMath::DegreesToRadians(Rotation);
				const float CosTheta = FMath::Cos(Theta);
				const float SinTheta = FMath::Sin(Theta);
				const FVector::FReal X = (ScaledPixelX * CosTheta) - (ScaledPixelY * SinTheta);
				const FVector::FReal Y = (ScaledPixelY * CosTheta) + (ScaledPixelX * SinTheta);

				FTransform RotatedTransform(FVector(X, Y, 0.f));

				// sampling with rotation at black position
				TextureData->Rotation = -Rotation;
				TextureData->CenterOffset = FVector2D::ZeroVector;
				TextureData->SamplePoint(FTransform(FVector(0.5, 0.5, 0.0)), FBox(), OutPoint, nullptr);
				bTestPassed &= TestEqual("Valid color sampled for rotated off-position", OutPoint.Color, static_cast<FVector>(FColor::Black));

				// sampling with rotation at white position
				TextureData->Rotation = -Rotation;
				TextureData->CenterOffset = FVector2D::ZeroVector;
				TextureData->SamplePoint(RotatedTransform, FBox(), OutPoint, nullptr);
				bTestPassed &= TestEqual("Valid color sampled for rotated position", OutPoint.Color, static_cast<FVector>(FColor::White));
			}
		}
	}

	return bTestPassed;
}
#endif
