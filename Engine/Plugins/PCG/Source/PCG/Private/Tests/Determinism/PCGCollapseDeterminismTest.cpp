// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Elements/PCGCollapseElement.h"
#include "Tests/Determinism/PCGDeterminismTestsCommon.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCollapseDeterminismSingleSameDataTest, FPCGTestBaseClass, "Plugins.PCG.Collapse.Determinism.SingleSameData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCollapseDeterminismSingleIdenticalDataTest, FPCGTestBaseClass, "Plugins.PCG.Collapse.Determinism.SingleIdenticalData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCollapseDeterminismMultipleSameDataTest, FPCGTestBaseClass, "Plugins.PCG.Collapse.Determinism.MultipleSameData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCollapseDeterminismMultipleIdenticalDataTest, FPCGTestBaseClass, "Plugins.PCG.Collapse.Determinism.MultipleIdenticalData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCollapseDeterminismOrderIndependenceTest, FPCGTestBaseClass, "Plugins.PCG.Collapse.Determinism.OrderIndependence", PCGTestsCommon::TestFlags)

namespace
{
	constexpr static int32 NumMultipleSpatialSetsToAdd = 3;
	constexpr static int32 NumPointsToAddDuringMultipleCases = 20;
	constexpr static int32 NumTestableSpatialDataTypes = 5;
	constexpr static EPCGDataType TestableSpatialDataTypes[NumTestableSpatialDataTypes] =
	{
		EPCGDataType::Point,
		EPCGDataType::Volume,
		EPCGDataType::Surface,
		EPCGDataType::PolyLine,
		EPCGDataType::Primitive
	};

	void CreateNewRandomizedSpatialDataSet(TArray<PCGTestsCommon::FTestData>& TestDataSet,
		int32 NumSpatialSetsToAdd = 1,
		int32 NumPointsToCreatePerSet = PCGDeterminismTests::Defaults::NumPointsToGenerate,
		int32 NumPolyLinePointsToCreatePerSet = PCGDeterminismTests::Defaults::NumPolyLinePointsToGenerate)
	{
		check(NumSpatialSetsToAdd > 0 && NumPointsToCreatePerSet > 0 && NumPolyLinePointsToCreatePerSet > 0);

		TestDataSet.Empty();
		PCGTestsCommon::FTestData TestData;
		TestDataSet.Init(TestData, NumTestableSpatialDataTypes);

		for (PCGTestsCommon::FTestData& Data : TestDataSet)
		{
			PCGTestsCommon::GenerateSettings<UPCGCollapseSettings>(Data);
		}

		for (int32 I = 0; I < NumSpatialSetsToAdd; ++I)
		{
			PCGDeterminismTests::AddRandomizedMultiplePointInputData(TestDataSet[0], NumPointsToCreatePerSet);
			PCGDeterminismTests::AddRandomizedVolumeInputData(TestDataSet[1]);
			PCGDeterminismTests::AddRandomizedSurfaceInputData(TestDataSet[2]);
			PCGDeterminismTests::AddRandomizedPolyLineInputData(TestDataSet[3], NumPolyLinePointsToCreatePerSet);
			PCGDeterminismTests::AddRandomizedPrimitiveInputData(TestDataSet[4]);
		}
	}
}

bool FPCGCollapseDeterminismSingleSameDataTest::RunTest(const FString& Parameters)
{
	// Test single same data
	TArray<PCGTestsCommon::FTestData> TestDataSet;
	CreateNewRandomizedSpatialDataSet(TestDataSet);

	for (int32 I = 0; I < TestDataSet.Num(); ++I)
	{
		bool bExecutionIsDeterministic = PCGDeterminismTests::ExecutionIsDeterministicSameData(TestDataSet[I]);
		TestTrue(FString::Printf(TEXT("[%s] Same single input and settings, same output"), *UEnum::GetValueAsString(TestableSpatialDataTypes[I])), bExecutionIsDeterministic);
	}

	return true;
}

bool FPCGCollapseDeterminismSingleIdenticalDataTest::RunTest(const FString& Parameters)
{
	// Test single identical data
	TArray<PCGTestsCommon::FTestData> FirstTestDataSet;
	TArray<PCGTestsCommon::FTestData> SecondTestDataSet;
	CreateNewRandomizedSpatialDataSet(FirstTestDataSet);
	CreateNewRandomizedSpatialDataSet(SecondTestDataSet);

	for (int32 I = 0; I < FirstTestDataSet.Num(); ++I)
	{
		bool bExecutionIsDeterministic = PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataSet[I], SecondTestDataSet[I]);
		TestTrue(FString::Printf(TEXT("[%s] Identical single input and settings, same output"), *UEnum::GetValueAsString(TestableSpatialDataTypes[I])), bExecutionIsDeterministic);
	}

	return true;
}

bool FPCGCollapseDeterminismMultipleSameDataTest::RunTest(const FString& Parameters)
{
	// Test multiple same data
	TArray<PCGTestsCommon::FTestData> TestDataSet;
	CreateNewRandomizedSpatialDataSet(TestDataSet, NumMultipleSpatialSetsToAdd, NumPointsToAddDuringMultipleCases);

	for (int32 I = 0; I < TestDataSet.Num(); ++I)
	{
		bool bExecutionIsDeterministic = PCGDeterminismTests::ExecutionIsDeterministicSameData(TestDataSet[I]);
		TestTrue(FString::Printf(TEXT("[%s] Same multiple input, same output"), *UEnum::GetValueAsString(TestableSpatialDataTypes[I])), bExecutionIsDeterministic);
	}

	return true;
}

bool FPCGCollapseDeterminismMultipleIdenticalDataTest::RunTest(const FString& Parameters)
{
	// Test multiple identical data
	TArray<PCGTestsCommon::FTestData> FirstTestDataSet;
	TArray<PCGTestsCommon::FTestData> SecondTestDataSet;
	CreateNewRandomizedSpatialDataSet(FirstTestDataSet, NumMultipleSpatialSetsToAdd);
	CreateNewRandomizedSpatialDataSet(SecondTestDataSet, NumMultipleSpatialSetsToAdd);

	for (int32 I = 0; I < FirstTestDataSet.Num(); ++I)
	{
		bool bExecutionIsDeterministic = PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataSet[I], SecondTestDataSet[I]);
		TestTrue(FString::Printf(TEXT("[%s] Identical multiple input, same output"), *UEnum::GetValueAsString(TestableSpatialDataTypes[I])), bExecutionIsDeterministic);
	}

	return true;
}

bool FPCGCollapseDeterminismOrderIndependenceTest::RunTest(const FString& Parameters)
{
	// Test shuffled input data
	TArray<PCGTestsCommon::FTestData> FirstTestDataSet;
	TArray<PCGTestsCommon::FTestData> SecondTestDataSet;
	CreateNewRandomizedSpatialDataSet(FirstTestDataSet, NumMultipleSpatialSetsToAdd);
	CreateNewRandomizedSpatialDataSet(SecondTestDataSet, NumMultipleSpatialSetsToAdd);

	for (int32 I = 0; I < FirstTestDataSet.Num(); ++I)
	{
		PCGDeterminismTests::ShuffleInputOrder(SecondTestDataSet[I]);
		bool bExecutionIsDeterministic = PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataSet[I], SecondTestDataSet[I]);
		TestTrue(FString::Printf(TEXT("[%s] Shuffled input order, same output"), *UEnum::GetValueAsString(TestableSpatialDataTypes[I])), bExecutionIsDeterministic);
	}

	return true;
}

#endif