// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_EDITOR
#include "Tests/Determinism/PCGDeterminismNativeTests.h"

#include "PCGSettings.h"

namespace PCGDeterminismTests
{
	FNativeTestRegistry* FNativeTestRegistry::RegistryPtr = nullptr;

	void FNativeTestRegistry::Create()
	{
		check(!RegistryPtr);
		RegistryPtr = new FNativeTestRegistry;
	}

	void FNativeTestRegistry::Destroy()
	{
		check(RegistryPtr);
		delete RegistryPtr;
		RegistryPtr = nullptr;
	}

	void FNativeTestRegistry::RegisterTestFunction(UClass* SettingsStaticClass, TFunction<bool()> TestFunction)
	{
		check(RegistryPtr && SettingsStaticClass);
		RegistryPtr->NativeTestMapping.Emplace(SettingsStaticClass, TestFunction);
	}

	void FNativeTestRegistry::DeregisterTestFunction(const UClass* SettingsStaticClass)
	{
		check(RegistryPtr && SettingsStaticClass);
		if (RegistryPtr->NativeTestMapping.Contains(SettingsStaticClass))
		{
			RegistryPtr->NativeTestMapping.Remove(SettingsStaticClass);
		}
	}

	TFunction<bool()> FNativeTestRegistry::GetNativeTestFunction(const UPCGSettings* PCGSettings)
	{
		check(RegistryPtr);
		if (TFunction<bool()>* TestFunction = RegistryPtr->NativeTestMapping.Find(PCGSettings->GetClass()))
		{
			return *TestFunction;
		}

		return {};
	}
}
#endif