// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Determinism/PCGDeterminismTestBlueprintBase.h"
#include "PCGNode.h"
#include "Tests/Determinism/PCGDeterminismTestsCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDeterminismTestBlueprintBase)

void UPCGDeterminismTestBlueprintBase::ExecuteTest_Implementation(const UPCGNode* InPCGNode, FDeterminismTestResult& InOutTestResult)
{
	// The default case if the user tries to run the blueprint test unimplemented
	InOutTestResult.TestResultTitle = FName(*InPCGNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString());
	InOutTestResult.TestResultName = InPCGNode->GetName();
	InOutTestResult.Seed = InPCGNode->GetSettings()->GetSeed();
	PCGDeterminismTests::RunDeterminismTest(InPCGNode, InOutTestResult, PCGDeterminismTests::Defaults::DeterminismBasicTestInfo);
}
