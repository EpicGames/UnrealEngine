// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tests/Determinism/PCGDeterminismTestsCommon.h"

#include "Data/PCGPolyLineData.h"
#include "PCGComponent.h"
#include "Data/PCGSpatialData.h"
#include "PCGGraph.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPrimitiveData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGSurfaceData.h"
#include "Data/PCGVolumeData.h"
#include "Graph/PCGGraphCache.h"
#include "Graph/PCGGraphExecutor.h"
#include "Helpers/PCGHelpers.h"

#include "Components/SplineComponent.h"
#include "Components/StaticMeshComponent.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDeterminismTestsCommon)

#define LOCTEXT_NAMESPACE "PCGDeterminism"

static TAutoConsoleVariable<int32> CVarDeterminismGraphTestCycleLimit(
	TEXT("pcg.DeterminismGraphTestCycleLimit"),
	10000,
	TEXT("How many execution attempts before discarding a graph test."));

static TAutoConsoleVariable<int32> CVarDeterminismPermutationLimit(
	TEXT("pcg.DeterminismPermutationLimit"),
	10000,
	TEXT("A limit for the maximum amount of input permutations before auto-failing a basic determinism test."));

namespace PCGDeterminismTests
{
	bool LogInvalidTest(const UPCGNode* InPCGNode, FDeterminismTestResult& OutResult)
	{
		UE_LOGF(LogPCG, Warning, "Attempting to run an invalid determinism test");
		OutResult.AdditionalDetails.Add(TEXT("Invalid test"));
		return false;
	}

	void RunDeterminismTest(const UPCGNode* InPCGNode, FDeterminismTestResult& OutResult, const FNodeTestInfo& TestToRun)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::RunDeterminismTest::Node);

		if (!InPCGNode || !InPCGNode->GetSettings())
		{
			OutResult.DataTypesTested = EPCGDataType::None;
			OutResult.bFlagRaised = true;
			OutResult.AdditionalDetails.Add(TEXT("Invalid Node or Settings"));
			return;
		}

		bool bTestWasSuccessful = TestToRun.TestDelegate(InPCGNode, TestToRun.TestName, OutResult);
		OutResult.bFlagRaised |= !bTestWasSuccessful;

		// If the test did not set its own result, base it off the success/fail
		if (!OutResult.TestResults.Find(TestToRun.TestName))
		{
			UpdateTestResults(TestToRun.TestName,
				OutResult,
				bTestWasSuccessful ? EDeterminismLevel::Basic : EDeterminismLevel::NoDeterminism);
		}
	}

#if WITH_EDITOR
	void RunDeterminismTest(const UPCGGraph* InPCGGraph, const UPCGComponent* InPCGComponent, FDeterminismTestResult& OutResult)
	{
		RunDeterminismTest(InPCGGraph, static_cast<const IPCGGraphExecutionSource*>(InPCGComponent), OutResult);
	}
	
	void RunDeterminismTest(const UPCGGraph* InPCGGraph, const IPCGGraphExecutionSource* InSource, FDeterminismTestResult& OutResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::RunDeterminismTest::Graph);

		// @todo_pcg: Support Determinism test on all sources.  
		const UPCGComponent* InPCGComponent = Cast<const UPCGComponent>(InSource);
		
		if (!InPCGGraph || !InPCGComponent)
		{
			OutResult.DataTypesTested = EPCGDataType::None;
			OutResult.bFlagRaised = true;
			OutResult.AdditionalDetails.Add(TEXT("Invalid Graph/Component"));
			return;
		}

		// Update the data types tested
		const UPCGData* InputPCGData = InPCGComponent->GetInputPCGData();
		check(InputPCGData);
		OutResult.DataTypesTested |= InputPCGData->GetDataTypeId();

		// Clone the actor and component
		AActor* PCGActor = InPCGComponent->GetOwner();
		check(PCGActor);

		FObjectDuplicationParameters DuplicateParams(PCGActor, PCGActor->GetOuter());
		DuplicateParams.ApplyFlags = RF_Transient;

		AActor* PCGActorCopy = nullptr;
		{
			FGCScopeGuard Scope;
			PCGActorCopy = Cast<AActor>(StaticDuplicateObjectEx(DuplicateParams));
		}

		PCGActorCopy->AddToRoot();

		UPCGComponent* PCGComponentCopy = PCGActorCopy->FindComponentByClass<UPCGComponent>();
		check(PCGComponentCopy);
		PCGComponentCopy->SetIsPartitioned(false);
		if (PCGActor->GetWorld())
		{
			PCGComponentCopy->RegisterComponentWithWorld(PCGActor->GetWorld());
		}

		TSharedPtr<FPCGGraphExecutor> Executor = MakeShared<FPCGGraphExecutor>();
		Executor->SetKeepIntermediateResults(true);

		ON_SCOPE_EXIT
		{
			if (Executor->IsGraphCurrentlyExecuting(InPCGGraph))
			{
				Executor->CancelAll();
			}

			// Clean up anything generated
			PCGComponentCopy->Cleanup();

			// Clean up the debug actor
			PCGActorCopy->UnregisterAllComponents();
			PCGActorCopy->MarkComponentsAsGarbage();
			PCGActorCopy->RemoveFromRoot();
			PCGActorCopy->MarkAsGarbage();
		};

		auto ScheduleAndWaitForExecution = [&Executor, PCGComponentCopy](FPCGTaskId FinalTaskID) -> bool
		{
			// TODO: Consider randomizing/iterating through possible input orders
			volatile bool bTasksComplete = false;

			// Clear the cache between runs
			Executor->GetCache().ClearCache();

			// Schedule final task for waiting
			Executor->ScheduleGeneric([&bTasksComplete]()
			{
				bTasksComplete = true;
				return true;
			}, PCGComponentCopy, {FinalTaskID});

			// TODO: Reshape the test framework to run a graph independently and use the callback events from the context.
			/* Implementor's Note: Async loading or other frame bound activities can cause this to stall out, so the
			 * cycle limit will stop early if needed.
			 **/
			int32 ExecutionCount = 0;
			while (!bTasksComplete)
			{
				if (ExecutionCount++ < CVarDeterminismGraphTestCycleLimit.GetValueOnAnyThread())
				{
					Executor->Execute();
				}
				else
				{
					Executor->CancelAll();
					return false;
				}
			}

			return true;
		};

		TMap<FPCGTaskId, TTuple<const UPCGNode*, const FPCGDataCollection>> IntermediateOutputArray;

		// TODO: TaskId may not remain robust in the future
		auto RecordMappedOutput = [&IntermediateOutputArray](FPCGTaskId TaskId, const UPCGNode* Node, const FPCGDataCollection& NodeOutput)
		{
			check(Node);
			IntermediateOutputArray.Add(TaskId, MakeTuple(Node, NodeOutput));
		};

		// Schedule the graph execution
		FPCGTaskId FinalTaskID = Executor->ScheduleDebugWithTaskCallback(PCGComponentCopy, RecordMappedOutput);

		TMap<FPCGTaskId, TTuple<const UPCGNode*, const FPCGDataCollection>> FirstIntermediateOutputArray;
		bool bExecutionCompleted = ScheduleAndWaitForExecution(FinalTaskID);
		if (bExecutionCompleted)
		{
			// Copy the first array over, so it can be used again
			FirstIntermediateOutputArray = IntermediateOutputArray;

			// Clean up generated content
			IntermediateOutputArray.Empty();
			PCGComponentCopy->Cleanup();

			// Run again
			FinalTaskID = Executor->ScheduleDebugWithTaskCallback(PCGComponentCopy, RecordMappedOutput);
			bExecutionCompleted &= ScheduleAndWaitForExecution(FinalTaskID);
		}

		// Early out if one of the executions was unable to complete
		if (!bExecutionCompleted)
		{
			OutResult.bFlagRaised = true;
			UpdateTestResults(Defaults::GraphResultName, OutResult, EDeterminismLevel::NoDeterminism);
			OutResult.AdditionalDetails.Add(TEXT("Required async or timed out"));
			return;
		}

		if (IntermediateOutputArray.IsEmpty() || FirstIntermediateOutputArray.IsEmpty())
		{
			OutResult.bFlagRaised = true;
			UpdateTestResults(Defaults::GraphResultName, OutResult, EDeterminismLevel::NoDeterminism);
			OutResult.AdditionalDetails.Add(TEXT("No output was generated"));
			return;
		}

		// Early out for mismatched outputs
		if (IntermediateOutputArray.Num() != FirstIntermediateOutputArray.Num())
		{
			OutResult.bFlagRaised = true;
			UpdateTestResults(Defaults::GraphResultName, OutResult, EDeterminismLevel::NoDeterminism);
			OutResult.AdditionalDetails.Add(TEXT("Number of outputs differ"));
			return;
		}

		// Begin output evaluation...
		EDeterminismLevel HighestDeterminismLevel = EDeterminismLevel::OrderIndependent;

		for (const TTuple<FPCGTaskId, TTuple<const UPCGNode*, const FPCGDataCollection>>& TaskOutputMapping : FirstIntermediateOutputArray)
		{
			const FPCGTaskId TaskIdKey = TaskOutputMapping.Key;
			const TTuple<const UPCGNode*, const FPCGDataCollection> NodeDataTuple = TaskOutputMapping.Value;
			const UPCGNode* NodePtr = NodeDataTuple.Get<0>();
			const FString NodeNameString = NodePtr->GetName();

			UE_LOGF(LogPCG, Log, "[%ls] Evaluating Node [%ls]...", *InPCGGraph->GetName(), *NodeNameString);

			const FPCGDataCollection& FirstOutput = NodeDataTuple.Get<1>();
			const FPCGDataCollection& SecondOutput = IntermediateOutputArray[TaskIdKey].Get<1>();

			// Check for identical outputs
			if (HighestDeterminismLevel == EDeterminismLevel::OrderIndependent)
			{
				if (!DataCollectionsAreIdentical(FirstOutput, SecondOutput))
				{
					// Not identical, downgrade to consistent
					HighestDeterminismLevel = EDeterminismLevel::OrderConsistent;
					UE_LOGF(LogPCG, Log, "[%ls] Node failed \"independent\" test. Downgrading to \"consistent\"", *NodeNameString);
					OutResult.AdditionalDetails.Emplace(TEXT("Downgraded to order consistent at node: ") + NodeNameString);
				}
			}

			// Check for consistent outputs
			if (HighestDeterminismLevel == EDeterminismLevel::OrderConsistent)
			{
				if (!DataCollectionsAreConsistent(FirstOutput, SecondOutput, FirstOutput.TaggedData.Num()))
				{
					// Not consistent, downgrade to orthogonal
					HighestDeterminismLevel = EDeterminismLevel::OrderOrthogonal;
					UE_LOGF(LogPCG, Log, "[%ls] Node failed \"consistent\" test. Downgrading to \"orthogonal\"", *NodeNameString);
					OutResult.AdditionalDetails.Emplace(TEXT("Downgraded to order orthogonal at node: ") + NodeNameString);
				}
			}

			// Highest possible is orthogonal, if not, then fail test
			if (HighestDeterminismLevel == EDeterminismLevel::OrderOrthogonal)
			{
				if (!DataCollectionsContainSameData(FirstOutput, SecondOutput))
				{
					// Not orthogonal, fail test
					HighestDeterminismLevel = EDeterminismLevel::NoDeterminism;
					UE_LOGF(LogPCG, Log, "[%ls] Node failed \"orthogonal\" test. Test failed.", *NodeNameString);
					OutResult.AdditionalDetails.Emplace(TEXT("Determinism test failed at node: ") + NodeNameString);
					break;
				}
			}
		}

		// Finalize the results
		UpdateTestResults(Defaults::GraphResultName, OutResult, HighestDeterminismLevel);
	}
#endif

	bool RunBasicTestSuite(const UPCGNode* InPCGNode, const FName& TestName, FDeterminismTestResult& OutResult)
	{
		// Get Pins
		const TArray<TObjectPtr<UPCGPin>>& InputPins = InPCGNode->GetInputPins();

		// Base Options Per Pin
		FNodeAndOptions NodeAndOptions(InPCGNode, OutResult.Seed, /*bMultipleOptionsPerPin=*/true);
		RetrieveBaseOptionsPerPin(NodeAndOptions.BaseOptionsByPin, InputPins, OutResult.DataTypesTested, Defaults::NumTestInputsPerPin);

		if (GetNumPermutations(NodeAndOptions.BaseOptionsByPin) > CVarDeterminismPermutationLimit.GetValueOnGameThread())
		{
			EDeterminismLevel SimpleTestDeterminismLevel = GetHighestDeterminismLevelSimpleShuffleInput(NodeAndOptions);
			// Downgrading the level here, because the Basic Test is intended to be Pass/Fail
			if (SimpleTestDeterminismLevel != EDeterminismLevel::NoDeterminism)
			{
				SimpleTestDeterminismLevel = EDeterminismLevel::Basic;
			}

			UpdateTestResults(TestName, OutResult, SimpleTestDeterminismLevel);
			UpdateTestResultForOverPermutationLimitError(OutResult);
			return false;
		}

		return RunBasicSelfTest(NodeAndOptions) && RunBasicCopiedSelfTest(NodeAndOptions);
	}

	bool RunOrderIndependenceSuite(const UPCGNode* InPCGNode, const FName& TestName, FDeterminismTestResult& OutResult)
	{
		// Start with not deterministic
		UpdateTestResults(TestName, OutResult, EDeterminismLevel::NoDeterminism);

		// Get Pins
		const TArray<TObjectPtr<UPCGPin>>& InputPins = InPCGNode->GetInputPins();

		// Base Options Per Pin
		FNodeAndOptions NodeAndOptions(InPCGNode, OutResult.Seed, /*bMultipleOptionsPerPin=*/true);
		// TODO: Multiple inputs per pin might lead to redundant tests. Filter these out
		RetrieveBaseOptionsPerPin(NodeAndOptions.BaseOptionsByPin, InputPins, OutResult.DataTypesTested, Defaults::NumTestInputsPerPin);

		if (GetNumPermutations(NodeAndOptions.BaseOptionsByPin) > CVarDeterminismPermutationLimit.GetValueOnGameThread())
		{
			EDeterminismLevel SimpleTestDeterminismLevel = GetHighestDeterminismLevelSimpleShuffleInput(NodeAndOptions);
			// We can still test the levels of determinism with the simple test
			UpdateTestResults(TestName, OutResult, SimpleTestDeterminismLevel);
			// However, the result will have additional details highlighting that a full test wasn't conducted
			UpdateTestResultForOverPermutationLimitError(OutResult);
			return false;
		}

		// No inputs, so just test determinism against itself
		if (InputPins.Num() == 0 && RunBasicCopiedSelfTest(NodeAndOptions))
		{
			UpdateTestResults(TestName, OutResult, EDeterminismLevel::OrderIndependent);
			return true;
		}

		EDeterminismLevel MaxLevel = GetHighestDeterminismLevel(NodeAndOptions);
		UpdateTestResults(TestName, OutResult, MaxLevel);

		return MaxLevel != EDeterminismLevel::None;
	}

	bool RunBasicSelfTest(const FNodeAndOptions& NodeAndOptions)
	{
		const UPCGNode* PCGNode = NodeAndOptions.PCGNode;
		const TArray<TArray<FPCGDataTypeIdentifier>>& BaseOptionsByPin = NodeAndOptions.BaseOptionsByPin;

		PCGTestsCommon::FTestData TestData(NodeAndOptions.Seed, PCGNode->GetSettings());

		// For all permutations
		const int32 NumPermutations = GetNumPermutations(BaseOptionsByPin);
		for (int32 PermutationIndex = 0; PermutationIndex < NumPermutations; ++PermutationIndex)
		{
			// Reset test data
			TestData.Reset(PCGNode->GetSettings());

			// Add an input for each pin
			for (int32 PinIndex = 0; PinIndex < BaseOptionsByPin.Num(); ++PinIndex)
			{
				FPCGDataTypeIdentifier DataType = GetPermutation(PermutationIndex, PinIndex, BaseOptionsByPin);
				AddRandomizedInputData(TestData, DataType);
			}

			if (!ExecutionIsDeterministicSameData(TestData, PCGNode))
			{
				return false;
			}
		}

		return true;
	}

	bool RunBasicCopiedSelfTest(const FNodeAndOptions& NodeAndOptions)
	{
		const UPCGNode* PCGNode = NodeAndOptions.PCGNode;
		const TArray<TArray<FPCGDataTypeIdentifier>>& BaseOptionsByPin = NodeAndOptions.BaseOptionsByPin;

		PCGTestsCommon::FTestData FirstTestData(NodeAndOptions.Seed, PCGNode->GetSettings());
		PCGTestsCommon::FTestData SecondTestData(NodeAndOptions.Seed, PCGNode->GetSettings());

		const int32 NumPermutations = GetNumPermutations(BaseOptionsByPin);
		// For all permutations
		for (int32 PermutationIndex = 0; PermutationIndex < NumPermutations; ++PermutationIndex)
		{
			// Reset test data
			FirstTestData.Reset(PCGNode->GetSettings());
			SecondTestData.Reset(PCGNode->GetSettings());

			// Add an input for each pin
			for (int32 PinIndex = 0; PinIndex < BaseOptionsByPin.Num(); ++PinIndex)
			{
				FPCGDataTypeIdentifier DataType = GetPermutation(PermutationIndex, PinIndex, BaseOptionsByPin);
				AddRandomizedInputData(FirstTestData, DataType);
				AddRandomizedInputData(SecondTestData, DataType);
			}

			if (!ExecutionIsDeterministic(FirstTestData, SecondTestData, PCGNode))
			{
				return false;
			}
		}

		return true;
	}

	EDeterminismLevel GetHighestDeterminismLevel(const FNodeAndOptions& NodeAndOptions,
		int32 NumInputsPerPin,
		EDeterminismLevel MaxLevel)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::GetHighestDeterminismLevel);

		const UPCGNode* PCGNode = NodeAndOptions.PCGNode;
		const TArray<TArray<FPCGDataTypeIdentifier>>& BaseOptionsByPin = NodeAndOptions.BaseOptionsByPin;

		PCGTestsCommon::FTestData FirstTestData(NodeAndOptions.Seed, PCGNode->GetSettings());
		PCGTestsCommon::FTestData SecondTestData(NodeAndOptions.Seed, PCGNode->GetSettings());

		int32 NumInputs = NodeAndOptions.bMultipleOptionsPerPin ? NumInputsPerPin : 1;
		const int32 NumPermutations = GetNumPermutations(BaseOptionsByPin);
		// For each rotation of the inputs
		for (int32 RotationIndex = 0; RotationIndex < BaseOptionsByPin.Num(); ++RotationIndex)
		{
			// For all permutations
			for (int32 PermutationIndex = 0; PermutationIndex < NumPermutations;)
			{
				// Reset test data
				FirstTestData.Reset(PCGNode->GetSettings());
				SecondTestData.Reset(PCGNode->GetSettings());

				// Add an input for each pin
				for (int32 PinIndex = 0; PinIndex < BaseOptionsByPin.Num(); ++PinIndex)
				{
					for (int32 I = 0; I < NumInputs; ++I)
					{
						FPCGDataTypeIdentifier DataType = GetPermutation(PermutationIndex++, PinIndex, BaseOptionsByPin);
						AddRandomizedInputData(FirstTestData, DataType);
						AddRandomizedInputData(SecondTestData, DataType);
					}
				}

				ShiftInputOrder(SecondTestData, RotationIndex + 1);

				ExecuteWithTestData(FirstTestData, PCGNode);
				ExecuteWithTestData(SecondTestData, PCGNode);

				// Always execute at the minimum level. If that level fails, drop a level and start over
				switch (MaxLevel)
				{
				case EDeterminismLevel::OrderIndependent:
					if (DataCollectionsAreIdentical(FirstTestData.OutputData, SecondTestData.OutputData))
					{
						break;
					}
					MaxLevel = EDeterminismLevel::OrderConsistent;
					PermutationIndex = 0;
					// Falls through
				case EDeterminismLevel::OrderConsistent:
					if (DataCollectionsAreConsistent(FirstTestData.OutputData, SecondTestData.OutputData, NumInputs))
					{
						// With only 1 input, orthogonal == consistent
						if (BaseOptionsByPin.Num() < 2)
						{
							if (DataCollectionsContainSameData(FirstTestData.OutputData, SecondTestData.OutputData))
							{
								break;
							}
						}
						break;
					}
					MaxLevel = EDeterminismLevel::OrderOrthogonal;
					PermutationIndex = 0;
					// Falls through
				case EDeterminismLevel::OrderOrthogonal:
					if (DataCollectionsContainSameData(FirstTestData.OutputData, SecondTestData.OutputData))
					{
						break;
					}
					// Falls through
				default:
					return EDeterminismLevel::NoDeterminism;
				}
			}
		}

		return MaxLevel;
	}

	EDeterminismLevel GetHighestDeterminismLevelSimpleShuffleInput(const FNodeAndOptions& NodeAndOptions, int32 NumInputsPerPin, EDeterminismLevel MaxLevel)
	{
		const UPCGNode* PCGNode = NodeAndOptions.PCGNode;
		const TArray<TArray<FPCGDataTypeIdentifier>>& BaseOptionsByPin = NodeAndOptions.BaseOptionsByPin;

		PCGTestsCommon::FTestData FirstTestData(NodeAndOptions.Seed, PCGNode->GetSettings());
		PCGTestsCommon::FTestData SecondTestData(NodeAndOptions.Seed, PCGNode->GetSettings());

		int32 NumInputs = NodeAndOptions.bMultipleOptionsPerPin ? NumInputsPerPin : 1;

		// Add inputs for each pin. Unlike the full test, this will just shuffle the input order
		for (int32 PinIndex = 0; PinIndex < BaseOptionsByPin.Num(); ++PinIndex)
		{
			for (int32 I = 0; I < NumInputs; ++I)
			{
				FPCGDataTypeIdentifier DataType = BaseOptionsByPin[PinIndex][I];
				AddRandomizedInputData(FirstTestData, DataType);
				AddRandomizedInputData(SecondTestData, DataType);
			}
		}

		ShuffleInputOrder(SecondTestData);

		ExecuteWithTestData(FirstTestData, PCGNode);
		ExecuteWithTestData(SecondTestData, PCGNode);

		// Gauge the highest level
		switch (MaxLevel)
		{
		case EDeterminismLevel::OrderIndependent:
			if (DataCollectionsAreIdentical(FirstTestData.OutputData, SecondTestData.OutputData))
			{
				break;
			}
			MaxLevel = EDeterminismLevel::OrderConsistent;
			// Falls through
		case EDeterminismLevel::OrderConsistent:
			if (DataCollectionsAreConsistent(FirstTestData.OutputData, SecondTestData.OutputData, NumInputs))
			{
				// With only 1 input, orthogonal == consistent
				if (BaseOptionsByPin.Num() < 2)
				{
					if (DataCollectionsContainSameData(FirstTestData.OutputData, SecondTestData.OutputData))
					{
						break;
					}
				}
				break;
			}
			MaxLevel = EDeterminismLevel::OrderOrthogonal;
			// Falls through
		case EDeterminismLevel::OrderOrthogonal:
			if (DataCollectionsContainSameData(FirstTestData.OutputData, SecondTestData.OutputData))
			{
				break;
			}
			// Falls through
		default:
			MaxLevel = EDeterminismLevel::NoDeterminism;
		}

		return MaxLevel;
	}

	void AddRandomizedInputData(PCGTestsCommon::FTestData& TestData, FPCGDataTypeIdentifier DataType, const FName& PinName)
	{
		if (DataType == EPCGDataType::Point)
		{
			AddRandomizedMultiplePointInputData(TestData, Defaults::NumTestPointsToGenerate, PinName);
			return;
		}
		else if (DataType == EPCGDataType::Volume)
		{
			AddRandomizedVolumeInputData(TestData, PinName);
			return;
		}
		else if (DataType == EPCGDataType::PolyLine)
		{
			AddRandomizedPolyLineInputData(TestData, Defaults::NumTestPolyLinePointsToGenerate, PinName);
			return;
		}
		else if (DataType == EPCGDataType::Primitive)
		{
			AddRandomizedPrimitiveInputData(TestData, PinName);
			return;
		}
		else if (DataType == EPCGDataType::Surface)
		{
			AddRandomizedSurfaceInputData(TestData, PinName);
			return;
		}
		else if (DataType == EPCGDataType::Landscape)
		{
			AddRandomizedLandscapeInputData(TestData, PinName);
			return;
		}
		// Just return to represent "empty" data
		else if (!DataType.IsValid())
		{
			return;
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Specific type required to add to test data");
		}
	}

	void AddSinglePointInputData(FPCGDataCollection& InputData, const FVector& Location, const FName& PinName)
	{
		UPCGBasePointData* PointData = PCGTestsCommon::CreateBasePointData(Location);

		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = PointData;
		TaggedData.Pin = PinName;
	}

	void AddMultiplePointsInputData(FPCGDataCollection& InputData, const TArray<FPCGPoint>& Points, const FName& PinName)
	{
		UPCGBasePointData* PointData = PCGTestsCommon::CreateEmptyBasePointData();
		PointData->SetNumPoints(Points.Num());
		PointData->AllocateProperties(EPCGPointNativeProperties::All);

		FPCGPointValueRanges ValueRanges(PointData, /*bAllocate=*/false);
		for (int32 Index = 0; Index < Points.Num(); ++Index)
		{
			ValueRanges.SetFromPoint(Index, Points[Index]);
		}

		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = PointData;
		TaggedData.Pin = PinName;
	}

	void AddVolumeInputData(FPCGDataCollection& InputData, const FVector& Location, const FVector& HalfSize, const FVector& VoxelSize, const FName& PinName)
	{
		UPCGVolumeData* VolumeData = PCGTestsCommon::CreateVolumeData(FBox::BuildAABB(Location, HalfSize));
		VolumeData->VoxelSize = VoxelSize;

		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = VolumeData;
		TaggedData.Pin = PinName;
	}

	void AddPolyLineInputData(FPCGDataCollection& InputData, USplineComponent* SplineComponent, const FName& PinName)
	{
		UPCGSplineData* SplineData = NewObject<UPCGSplineData>();
		SplineData->Initialize(SplineComponent);

		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = SplineData;
		TaggedData.Pin = PinName;
	}

	void AddPrimitiveInputData(FPCGDataCollection& InputData, UPrimitiveComponent* PrimitiveComponent, const FVector& VoxelSize, const FName& PinName)
	{
		UPCGPrimitiveData* PrimitiveData = NewObject<UPCGPrimitiveData>();
		PrimitiveData->Initialize(PrimitiveComponent);
		PrimitiveData->VoxelSize = VoxelSize;

		FPCGTaggedData& TaggedData = InputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = PrimitiveData;
		TaggedData.Pin = PinName;
	}

	void AddLandscapeInputData(FPCGDataCollection& InputData)
	{
		// TODO: Implement this in the future as needed
	}

	void AddRandomizedSinglePointInputData(PCGTestsCommon::FTestData& TestData, int32 PointNum, const FName& PinName)
	{
		check(PointNum > 0);
		for (int32 I = 0; I < PointNum; ++I)
		{
			AddSinglePointInputData(TestData.InputData, TestData.RandomStream.VRand() * Defaults::LargeDistance, PinName);
		}
	}

	void AddRandomizedMultiplePointInputData(PCGTestsCommon::FTestData& TestData, int32 PointNum, const FName& PinName)
	{
		check(PointNum > 0);

		TArray<FPCGPoint> Points;
		Points.SetNumUninitialized(PointNum);
		for (int32 I = 0; I < PointNum; ++I)
		{
			FVector NewLocation = TestData.RandomStream.VRand() * Defaults::LargeDistance;
			FTransform NewTransform(FRotator::ZeroRotator, NewLocation, FVector::OneVector * TestData.RandomStream.FRandRange(0.5, 1.5));
			int PointSeed = PCGHelpers::ComputeSeed(static_cast<int>(NewLocation.X), static_cast<int>(NewLocation.Y), static_cast<int>(NewLocation.Z));
			Points[I] = FPCGPoint(NewTransform, 1.f, PCGHelpers::ComputeSeed(PointSeed, TestData.Seed));
		}

		AddMultiplePointsInputData(TestData.InputData, Points, PinName);
	}

	void AddRandomizedVolumeInputData(PCGTestsCommon::FTestData& TestData, const FName& PinName)
	{
		AddVolumeInputData(TestData.InputData,
			TestData.RandomStream.VRand() * Defaults::MediumDistance,
			Defaults::MediumVector + TestData.RandomStream.VRand() * 0.5f * Defaults::MediumDistance,
			Defaults::SmallVector + TestData.RandomStream.VRand() * 0.5f * Defaults::SmallDistance,
			PinName);
	}

	void AddRandomizedSurfaceInputData(PCGTestsCommon::FTestData& TestData, const FName& PinName)
	{
		// TODO: PCG doesn't currently generate Surface data; function remains for future scalability
	}

	void AddRandomizedPolyLineInputData(PCGTestsCommon::FTestData& TestData, int32 PointNum, const FName& PinName)
	{
		check(TestData.TestActor);
		USplineComponent* TestSplineComponent = Cast<USplineComponent>(TestData.TestActor->GetComponentByClass(USplineComponent::StaticClass()));

		if (TestSplineComponent == nullptr)
		{
			TestSplineComponent = NewObject<USplineComponent>(TestData.TestActor, FName(TEXT("Test Spline Component")), RF_Transient);
		}

		check(PointNum > 1);
		for (int32 I = 0; I < PointNum; ++I)
		{
			TestSplineComponent->AddSplinePoint(TestData.RandomStream.VRand() * Defaults::LargeDistance, ESplineCoordinateSpace::Type::World, false);
			TestSplineComponent->AddRelativeRotation(FRotator(
				TestData.RandomStream.FRandRange(-90.0, 90.0),
				TestData.RandomStream.FRandRange(-90.0, 90.0),
				TestData.RandomStream.FRandRange(-90.0, 90.0)));
		}
		TestSplineComponent->UpdateSpline();

		AddPolyLineInputData(TestData.InputData, TestSplineComponent, PinName);
	}

	void AddRandomizedPrimitiveInputData(PCGTestsCommon::FTestData& TestData, const FName& PinName)
	{
		check(TestData.TestActor);
		UPrimitiveComponent* TestPrimitiveComponent = Cast<UPrimitiveComponent>(TestData.TestActor->GetComponentByClass(UPrimitiveComponent::StaticClass()));

		if (TestPrimitiveComponent == nullptr)
		{
			// TODO: If it reaches here, this will break, as it has no bounds. Please suggest the best way to give it some bounds?
			TestPrimitiveComponent = NewObject<UStaticMeshComponent>(TestData.TestActor, FName(TEXT("Test Primitive Component")), RF_Transient);
		}

		TestPrimitiveComponent->SetWorldTransform(FTransform(
			FRotator(TestData.RandomStream.FRandRange(0.0, 90.0), TestData.RandomStream.FRandRange(0.0, 90.0), TestData.RandomStream.FRandRange(0.0, 90.0)),
			TestData.RandomStream.VRand() * Defaults::LargeDistance,
			FVector::OneVector * TestData.RandomStream.FRandRange(0.5, 1.5)));

		// TODO: Probably more varieties to add in the future
		AddPrimitiveInputData(TestData.InputData, TestPrimitiveComponent, Defaults::MediumVector + TestData.RandomStream.VRand() * 0.5f * Defaults::MediumDistance, PinName);
	}

	void AddRandomizedLandscapeInputData(PCGTestsCommon::FTestData& TestData, const FName& PinName)
	{
		// TODO: Implement this in the future as needed
	}

	bool DataCollectionsAreIdentical(const FPCGDataCollection& FirstCollection, const FPCGDataCollection& SecondCollection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::DataCollectionsAreIdentical);

		if (FirstCollection.TaggedData.Num() != SecondCollection.TaggedData.Num())
		{
			return false;
		}

		// Find comparable data from first collection
		for (int32 I = 0; I < FirstCollection.TaggedData.Num(); ++I)
		{
			FPCGDataTypeIdentifier FirstDataType = FirstCollection.TaggedData[I].Data->GetDataTypeId();
			FPCGDataTypeIdentifier SecondDataType = SecondCollection.TaggedData[I].Data->GetDataTypeId();

			if (!DataTypeIsComparable(FirstDataType))
			{
				continue;
			}

			// Only compare if they are the same type and same pin label
			if (FirstDataType != SecondDataType || FirstCollection.TaggedData[I].Pin != SecondCollection.TaggedData[I].Pin)
			{
				return false;
			}

			auto CompareFunction = GetDataCompareFunction(FirstDataType, EDeterminismLevel::OrderIndependent);
			if (!CompareFunction(FirstCollection.TaggedData[I].Data, SecondCollection.TaggedData[I].Data))
			{
				return false;
			}
		}

		return true;
	}

	bool DataCollectionsAreConsistent(const FPCGDataCollection& FirstCollection, const FPCGDataCollection& SecondCollection, int32 NumInputs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::DataCollectionsAreConsistent);

		TArray<int32> IndexOffsets;
		if (!DataCollectionsMatch(FirstCollection, SecondCollection, IndexOffsets))
		{
			return false;
		}

		// Short circuit null set
		if (IndexOffsets.Num() == 0)
		{
			return true;
		}

		// All of the offset correlations should zero sum, if all output elements are matching
		int32 Sum = 0;
		// Data collection elements should be consistent in their groupings based on the ordering of the input data.
		// The offset of each grouping should be the same, so when it switches, it indicates a different grouping. There
		// should only be at most one grouping per input
		int32 NumSwitches = 0;

		int32 LastOffset = IndexOffsets[0];
		for (int32 Offset : IndexOffsets)
		{
			if (Offset != LastOffset)
			{
				// Number of switches must be less than number of inputs
				if (++NumSwitches >= NumInputs)
				{
					return false;
				}
			}
			Sum += Offset;
		}

		// Should be zero summed
		return Sum == 0;
	}

	bool DataCollectionsContainSameData(const FPCGDataCollection& FirstCollection, const FPCGDataCollection& SecondCollection)
	{
		TArray<int32> NullArray;
		return DataCollectionsMatch(FirstCollection, SecondCollection, NullArray);
	}

	bool DataCollectionsMatch(const FPCGDataCollection& FirstCollection,
		const FPCGDataCollection& SecondCollection,
		TArray<int32>& OutIndexOffsets)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::DataCollectionsMatch);

		OutIndexOffsets.Empty();

		if (FirstCollection.TaggedData.Num() != SecondCollection.TaggedData.Num())
		{
			return false;
		}

		// Short circuit if there's nothing to compare
		if (FirstCollection.TaggedData.Num() == 0)
		{
			return true;
		}

		TArray<int32> FirstCollectionComparableIndices;
		TArray<int32> SecondCollectionComparableIndices;

		// Find comparable data from first collection
		for (int32 I = 0; I < FirstCollection.TaggedData.Num(); ++I)
		{
			if (DataIsComparable(FirstCollection.TaggedData[I].Data))
			{
				FirstCollectionComparableIndices.Emplace(I);
			}
		}

		// Find comparable data from second collection
		for (int32 I = 0; I < SecondCollection.TaggedData.Num(); ++I)
		{
			// Check if its a type we care about comparing
			if (DataIsComparable(SecondCollection.TaggedData[I].Data))
			{
				SecondCollectionComparableIndices.Emplace(I);
			}
		}

		if (FirstCollectionComparableIndices.Num() != SecondCollectionComparableIndices.Num())
		{
			return false;
		}

		// Nothing to compare, so its a null match
		if (FirstCollectionComparableIndices.Num() == 0)
		{
			return true;
		}

		// Offsets of one data's index from the other collection's index
		TSet<int32> MatchedIndices;
		MatchedIndices.Reserve(FirstCollectionComparableIndices.Num());
		OutIndexOffsets.SetNumUninitialized(FirstCollectionComparableIndices.Num());

		for(int32 FirstIndex = 0; FirstIndex < FirstCollectionComparableIndices.Num(); ++FirstIndex)
		{
			int32 I = FirstCollectionComparableIndices[FirstIndex];

			bool bFound = false;
			for(int32 SecondIndex = 0; SecondIndex < SecondCollectionComparableIndices.Num(); ++SecondIndex)
			{
				int32 J = SecondCollectionComparableIndices[SecondIndex];

				if (MatchedIndices.Contains(J))
				{
					continue;
				}

				FPCGDataTypeIdentifier FirstDataType = FirstCollection.TaggedData[I].Data->GetDataTypeId();
				FPCGDataTypeIdentifier SecondDataType = SecondCollection.TaggedData[J].Data->GetDataTypeId();

				// Only compare if they are the same type and same pin label
				if (FirstDataType == SecondDataType && FirstCollection.TaggedData[I].Pin == SecondCollection.TaggedData[J].Pin)
				{
					auto CompareFunction = GetDataCollectionCompareFunction(FirstDataType);
					if (CompareFunction(FirstCollection.TaggedData[I].Data, SecondCollection.TaggedData[J].Data))
					{
						MatchedIndices.Emplace(J);
						OutIndexOffsets[FirstIndex] = J - I;
						bFound = true;
						break;
					}
				}
			}

			if (!bFound)
			{
				OutIndexOffsets.Empty();
				return false;
			}
		}

		return true;
	}

	bool InternalDataMatches(const UPCGData* FirstData, const UPCGData* SecondData, TArray<int32>& OutIndexOffsets)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::InternalDataMatches);

		check(FirstData->GetDataTypeId() == SecondData->GetDataTypeId());

		// Filter output differences
		auto CompareFunction = GetDataCompareFunction(FirstData->GetDataTypeId(), EDeterminismLevel::OrderConsistent);
		return CompareFunction(FirstData, SecondData);
	}

	bool SpatialDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::SpatialDataIsIdentical);

		const UPCGSpatialData* FirstSpatialData = CastChecked<const UPCGSpatialData>(FirstData);
		const UPCGSpatialData* SecondSpatialData = CastChecked<const UPCGSpatialData>(SecondData);

		if (!SpatialBasicsAreIdentical(FirstSpatialData, SecondSpatialData))
		{
			return false;
		}
		else if (BothDataCastsToDataType<const UPCGBasePointData>(FirstData, SecondData))
		{
			return PointDataIsIdentical(FirstData, SecondData);
		}
		else if (BothDataCastsToDataType<const UPCGVolumeData>(FirstData, SecondData))
		{
			return VolumeDataIsIdentical(FirstData, SecondData);
		}
		else if (BothDataCastsToDataType<const UPCGSurfaceData>(FirstData, SecondData))
		{
			return SurfaceDataIsIdentical(FirstData, SecondData);
		}
		else if (BothDataCastsToDataType<const UPCGPolyLineData>(FirstData, SecondData))
		{
			return PolyLineDataIsIdentical(FirstData, SecondData);
		}
		else if (BothDataCastsToDataType<const UPCGPrimitiveData>(FirstData, SecondData))
		{
			return PrimitiveDataIsIdentical(FirstData, SecondData);
		}
		// Default to sampling
		else if (BothDataCastsToDataType<const UPCGSpatialData>(FirstData, SecondData))
		{
			return SampledSpatialDataIsIdentical(Cast<const UPCGSpatialData>(FirstData), Cast<const UPCGSpatialData>(SecondData));
		}

		return false;
	}

	template<EPCGPointNativeProperties Property>
	bool CompareProperty(const UPCGBasePointData* FirstPointData, const UPCGBasePointData* SecondPointData)
	{
		const typename TPCGPointNativeProperty<Property>::ConstValueRange FirstRange = FirstPointData->GetConstValueRange<Property>();
		const typename TPCGPointNativeProperty<Property>::ConstValueRange SecondRange = SecondPointData->GetConstValueRange<Property>();
		check(FirstRange.Num() == SecondRange.Num());

		for (int32 Index = 0; Index < FirstRange.Num(); ++Index)
		{
			if (!PCG::Private::MetadataTraits<typename TPCGPointNativeProperty<Property>::Type>::Equal(FirstRange[Index], SecondRange[Index]))
			{
				return false;
			}
		}
		
		return true;
	}

	bool PointDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::PointDataIsIdentical);

		const UPCGBasePointData* FirstPointData = CastChecked<const UPCGBasePointData>(FirstData);
		const UPCGBasePointData* SecondPointData = CastChecked<const UPCGBasePointData>(SecondData);

		if (!SpatialBasicsAreIdentical(FirstPointData, SecondPointData))
		{
			return false;
		}
				
		if (FirstPointData->GetNumPoints() != SecondPointData->GetNumPoints())
		{
			return false;
		}

		if (!CompareProperty<EPCGPointNativeProperties::Steepness>(FirstPointData, SecondPointData))
		{
			return false;
		}

		if (!CompareProperty<EPCGPointNativeProperties::Density>(FirstPointData, SecondPointData))
		{
			return false;
		}
		
		if (!CompareProperty<EPCGPointNativeProperties::BoundsMin>(FirstPointData, SecondPointData))
		{
			return false;
		}
		
		if (!CompareProperty<EPCGPointNativeProperties::BoundsMax>(FirstPointData, SecondPointData))
		{
			return false;
		}
		
		if (!CompareProperty<EPCGPointNativeProperties::Color>(FirstPointData, SecondPointData))
		{
			return false;
		}
		
		if (!CompareProperty<EPCGPointNativeProperties::Transform>(FirstPointData, SecondPointData))
		{
			return false;
		}

		return true;
	}

	bool VolumeDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::VolumeDataIsIdentical);

		const UPCGVolumeData* FirstVolumeData = CastChecked<const UPCGVolumeData>(FirstData);
		const UPCGVolumeData* SecondVolumeData = CastChecked<const UPCGVolumeData>(SecondData);

		return FirstVolumeData->VoxelSize == SecondVolumeData->VoxelSize;
	}

	bool SurfaceDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		const UPCGSurfaceData* FirstSurfaceData = CastChecked<const UPCGSurfaceData>(FirstData);
		const UPCGSurfaceData* SecondSurfaceData = CastChecked<const UPCGSurfaceData>(SecondData);

		// TODO: Implement Surface Data comparison as needed in the future
		// In the meantime, at least check if they are the same object
		if (FirstSurfaceData == SecondSurfaceData)
		{
			return true;
		}

		UE_LOGF(LogPCG, Warning, "Surface comparison not fully implemented.");

		return false;
	}

	bool PolyLineDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		const UPCGPolyLineData* FirstPolyLineData = CastChecked<const UPCGPolyLineData>(FirstData);
		const UPCGPolyLineData* SecondPolyLineData = CastChecked<const UPCGPolyLineData>(SecondData);

		int NumSegments = FirstPolyLineData->GetNumSegments();
		for (int32 I = 0; I < NumSegments; ++I)
		{
			// TODO: Needs more robust checking for straight line vs spline tangents, etc
			if (FirstPolyLineData->GetSegmentLength(I) != SecondPolyLineData->GetSegmentLength(I) ||
				!FirstPolyLineData->GetTransformAtDistance(I, 0.0).Equals(SecondPolyLineData->GetTransformAtDistance(I, 0.0)))
			{
				return false;
			}
		}

		return true;
	}

	bool PrimitiveDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		const UPCGPrimitiveData* FirstPrimitiveData = CastChecked<const UPCGPrimitiveData>(FirstData);
		const UPCGPrimitiveData* SecondPrimitiveData = CastChecked<const UPCGPrimitiveData>(SecondData);

		if (FirstPrimitiveData->VoxelSize != SecondPrimitiveData->VoxelSize)
		{
			return false;
		}

		// TODO: Compare the ToPointData, which will require the Context
		return true;
	}

	bool SampledSpatialDataIsIdentical(const UPCGSpatialData* FirstSpatialData, const UPCGSpatialData* SecondSpatialData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::SampledSpatialDataIsIdentical);

		check(FirstSpatialData && SecondSpatialData);

		// The bounds should have already been checked by now
		check(FirstSpatialData->GetBounds() == SecondSpatialData->GetBounds());

		// Combine the bounds of both spatial data and expand by a little to get the overall bounds
		FBox SampleBounds = FirstSpatialData->GetBounds() + SecondSpatialData->GetBounds();
		SampleBounds = SampleBounds.ExpandBy(SampleBounds.GetExtent() * Defaults::TestingVolumeExpandByFactor);
		const FVector SampleExtent = SampleBounds.GetExtent();

		FPCGPoint FirstPoint;
		FPCGPoint SecondPoint;
		FVector StepInterval = SampleExtent * 2.0 / FMath::Max(Defaults::NumSamplingStepsPerDimension, 1);
		FVector StartingOffset = SampleBounds.Min + StepInterval * 0.5;

		// Sample points across the 3D volume
		for (FVector::FReal X = StartingOffset.X; X < SampleBounds.Max.X; X += StepInterval.X)
		{
			for (FVector::FReal Y = StartingOffset.Y; Y < SampleBounds.Max.Y; Y += StepInterval.Y)
			{
				for (FVector::FReal Z = StartingOffset.Z; Z < SampleBounds.Max.Z; Z += StepInterval.Z)
				{
					FTransform PointTransform(FVector(X, Y, Z));

					bool bFirstPointWasSampled = FirstSpatialData->SamplePoint(PointTransform, SampleBounds, FirstPoint, nullptr);
					bool bSecondPointWasSampled = SecondSpatialData->SamplePoint(PointTransform, SampleBounds, SecondPoint, nullptr);

					if (bFirstPointWasSampled != bSecondPointWasSampled)
					{
						return false;
					}

					// Only compare if both points were sampled
					if (bFirstPointWasSampled && bSecondPointWasSampled && !PCGTestsCommon::PointsAreIdentical(FirstPoint, SecondPoint))
					{
						return false;
					}
				}
			}
		}

		return true;
	}

	bool SpatialBasicsAreIdentical(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		const UPCGSpatialData* FirstSpatialData = CastChecked<const UPCGSpatialData>(FirstData);
		const UPCGSpatialData* SecondSpatialData = CastChecked<const UPCGSpatialData>(SecondData);

		return FirstSpatialData->GetDataType() == SecondSpatialData->GetDataType() &&
			FirstSpatialData->GetDimension() == SecondSpatialData->GetDimension() &&
			FirstSpatialData->GetBounds() == SecondSpatialData->GetBounds() &&
			FirstSpatialData->GetStrictBounds() == SecondSpatialData->GetStrictBounds();
	}

	bool SpatialDataIsConsistent(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		const UPCGSpatialData* FirstSpatialData = CastChecked<const UPCGSpatialData>(FirstData);
		const UPCGSpatialData* SecondSpatialData = CastChecked<const UPCGSpatialData>(SecondData);

		// Even for consistent, these should be identical
		if (!SpatialBasicsAreIdentical(FirstSpatialData, SecondSpatialData))
		{
			return false;
		}

		if (BothDataCastsToDataType<const UPCGPointData>(FirstSpatialData, SecondSpatialData))
		{
			return PointDataIsConsistent(FirstSpatialData, SecondSpatialData);
		}

		// TODO: Implement other data types as they come
		return SpatialDataIsIdentical(FirstData, SecondData);
	}

	bool PointDataIsConsistent(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		const UPCGPointData* FirstPointData = CastChecked<const UPCGPointData>(FirstData);
		const UPCGPointData* SecondPointData = CastChecked<const UPCGPointData>(SecondData);

		const TArray<FPCGPoint>& FirstPoints = FirstPointData->GetPoints();
		const TArray<FPCGPoint>& SecondPoints = SecondPointData->GetPoints();

		if (FirstPoints.Num() != SecondPoints.Num())
		{
			return false;
		}

		// Short circuit, because a null set is deterministic
		if (FirstPoints.Num() == 0)
		{
			return true;
		}

		// TODO: To be used in the future for series comparisons
		TArray<int32> IndexOffsets;
		IndexOffsets.SetNum(FirstPoints.Num());

		for (int32 I = 0; I < FirstPoints.Num(); ++I)
		{
			bool bFound = false;

			for (int32 J = 0; J < SecondPoints.Num(); ++J)
			{
				if (PCGTestsCommon::PointsAreIdentical(FirstPoints[I], SecondPoints[J]))
				{
					IndexOffsets[I] = J - I;
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				return false;
			}
		}

		return true;
	}

	bool SpatialDataIsOrthogonal(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		const UPCGSpatialData* FirstSpatialData = CastChecked<const UPCGSpatialData>(FirstData);
		const UPCGSpatialData* SecondSpatialData = CastChecked<const UPCGSpatialData>(SecondData);

		if (!SpatialBasicsAreIdentical(FirstSpatialData, SecondSpatialData))
		{
			return false;
		}

		if (BothDataCastsToDataType<const UPCGPointData>(FirstSpatialData, SecondSpatialData))
		{
			return PointDataIsOrthogonal(FirstSpatialData, SecondSpatialData);
		}

		// TODO: Implement other data types as they come
		return SpatialDataIsIdentical(FirstData, SecondData);
	}

	bool PointDataIsOrthogonal(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		const UPCGPointData* FirstPointData = CastChecked<const UPCGPointData>(FirstData);
		const UPCGPointData* SecondPointData = CastChecked<const UPCGPointData>(SecondData);

		const TArray<FPCGPoint>& FirstPoints = FirstPointData->GetPoints();
		const TArray<FPCGPoint>& SecondPoints = SecondPointData->GetPoints();

		if (FirstPoints.Num() != SecondPoints.Num())
		{
			return false;
		}

		// Match points based on indices
		TSet<int32> RemainingIndices;
		RemainingIndices.Reserve(SecondPoints.Num());

		for (int32 I = 0; I < SecondPoints.Num(); ++I)
		{
			RemainingIndices.Add(I);
		}

		for (int32 I = 0; I < FirstPoints.Num(); ++I)
		{
			bool bFound = false;

			for (int32 J : RemainingIndices)
			{
				if (PCGTestsCommon::PointsAreIdentical(FirstPoints[I], SecondPoints[J]))
				{
					RemainingIndices.Remove(J);
					bFound = true;
					break;
				}
			}

			// Couldn't find the matching point
			if (!bFound)
			{
				return false;
			}
		}

		return true;
	}

	bool ParamDataIsIdentical(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		const UPCGParamData* FirstParamData = CastChecked<const UPCGParamData>(FirstData);
		const UPCGParamData* SecondParamData = CastChecked<const UPCGParamData>(SecondData);

		return MetadataIsIdentical(FirstParamData->Metadata, SecondParamData->Metadata);
	}

	bool MetadataIsIdentical(const UPCGMetadata* FirstMetadata, const UPCGMetadata* SecondMetadata)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGDeterminismTests::MetadataIsIdentical);

		check(FirstMetadata && SecondMetadata);

		// Early out for different counts
		if (FirstMetadata->GetAttributeCount() != SecondMetadata->GetAttributeCount())
		{
			return false;
		}

		TArray<FName> FirstAttributeNames;
		TArray<EPCGMetadataTypes> FirstAttributeTypes;
		FirstMetadata->GetAttributes(FirstAttributeNames, FirstAttributeTypes);

		TArray<FName> SecondAttributeNames;
		TArray<EPCGMetadataTypes> SecondAttributeTypes;
		SecondMetadata->GetAttributes(SecondAttributeNames, SecondAttributeTypes);

		for (const FName& AttributeName : FirstAttributeNames)
		{
			const FPCGMetadataAttributeBase* FirstAttributeBase = FirstMetadata->GetConstAttribute(AttributeName);
			const FPCGMetadataAttributeBase* SecondAttributeBase = SecondMetadata->GetConstAttribute(AttributeName);

			// Early out if it isn't found
			if (!FirstAttributeBase || !SecondAttributeBase)
			{
				return false;
			}

			// Check the Entry:Value key mappings
			const PCGMetadataEntryKey EntryKeyNum = FirstMetadata->GetItemCountForChild();
			for (PCGMetadataEntryKey EntryKey = 0; EntryKey < EntryKeyNum; ++EntryKey)
			{
				if (FirstAttributeBase->GetValueKey(EntryKey) != SecondAttributeBase->GetValueKey(EntryKey))
				{
					return false;
				}
			}

			// Compare the values with the value keys
			const PCGMetadataValueKey FirstNumValueKeys = FirstAttributeBase->GetValueKeyOffsetForChild();
			const PCGMetadataValueKey SecondNumValueKeys = SecondAttributeBase->GetValueKeyOffsetForChild();

			if (FirstNumValueKeys != SecondNumValueKeys)
			{
				return false;
			}

			bool bAttributesAreEqual = false;

			for (PCGMetadataValueKey ValueKey = 0; ValueKey < FirstNumValueKeys; ++ValueKey)
			{
				// Note: these must all have operator==() defined
				switch (FirstAttributeBase->GetTypeId())
				{
				case PCG::Private::MetadataTypes<float>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<float>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<double>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<double>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<bool>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<bool>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<FVector>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<FVector>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<FVector4>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<FVector4>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<int32>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<int32>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<int64>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<int64>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<FString>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<FString>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<FName>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<FName>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<FQuat>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<FQuat>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<FRotator>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<FRotator>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				case PCG::Private::MetadataTypes<FTransform>::Id:
					bAttributesAreEqual = MetadataAttributesAreEqual<FTransform>(FirstAttributeBase, SecondAttributeBase, ValueKey);
					break;
				default:
					UE_LOGF(LogPCG, Warning, "Invalid type to compare for metadata being identical: %d", FirstAttributeBase->GetTypeId());
					break;
				}

				if (!bAttributesAreEqual)
				{
					return false;
				}
			}
		}

		return true;
	}

	bool ComparisonIsUnimplemented(const UPCGData* FirstData, const UPCGData* SecondData)
	{
		return false;
	}

	void UpdateTestResultForOverPermutationLimitError(FDeterminismTestResult& OutResult)
	{
		OutResult.AdditionalDetails.Emplace(TEXT("A full test was not conducted, due to a high number of permutations. Adjust 'pcg.DeterminismPermutationLimit' to alter this limit."));
	}

	bool ConsistencyComparisonIsUnimplemented(const UPCGData* FirstData, const UPCGData* SecondData, TArray<int32>& OutOutIndexOffsets)
	{
		return ComparisonIsUnimplemented(FirstData, SecondData);
	}

	bool DataTypeIsComparable(FPCGDataTypeIdentifier DataType)
	{
		// Comparable data types (Param and Spatial)
		const FPCGDataTypeIdentifier ValidTypes = FPCGDataTypeIdentifier::Construct(FPCGDataTypeInfoParam::AsId(), FPCGDataTypeInfoSpatial::AsId());
		if (DataType.Intersects(ValidTypes))
		{
			return true;
		}

		UE_LOGF(LogPCG, Warning, "Unknown determinism data comparison type: %ls", *DataType.ToString());
		return false;
	}

	bool DataIsComparable(const UPCGData* Data)
	{
		return Data && DataTypeIsComparable(Data->GetDataTypeId());
	}

	bool DataCanBeShuffled(const UPCGData* Data)
	{
		// TODO: Revisit for other data types that can be shuffled
		return Data && Data->IsA<UPCGPointData>();
	}

	void ShuffleInputOrder(PCGTestsCommon::FTestData& TestData)
	{
		PCGHelpers::ShuffleArray<FPCGTaggedData>(TestData.RandomStream, TestData.InputData.TaggedData);
	}

	void ShuffleOutputOrder(PCGTestsCommon::FTestData& TestData)
	{
		PCGHelpers::ShuffleArray<FPCGTaggedData>(TestData.RandomStream, TestData.OutputData.TaggedData);
	}

	void ShuffleAllInternalData(PCGTestsCommon::FTestData& TestData)
	{
		for (const FPCGTaggedData& TaggedData : TestData.InputData.TaggedData)
		{
			if (DataCanBeShuffled(TaggedData.Data))
			{
				UPCGPointData* PointData = const_cast<UPCGPointData*>(CastChecked<UPCGPointData>(TaggedData.Data));
				PCGHelpers::ShuffleArray<FPCGPoint>(TestData.RandomStream, PointData->GetMutablePoints());
			}
		}
	}

	void ShiftInputOrder(PCGTestsCommon::FTestData& TestData, int32 NumShifts)
	{
		PCGHelpers::ShiftArrayElements<FPCGTaggedData>(TestData.InputData.TaggedData, NumShifts);
	}

	TArray<FPCGDataTypeIdentifier> FilterTestableDataTypes(FPCGDataTypeIdentifier AllowedTypes, int32 NumMultipleInputs)
	{
		TArray<FPCGDataTypeIdentifier> RemainingDataTypes;
		if (!(AllowedTypes & EPCGDataType::Spatial))
		{
			// TODO: Expand this to other data types
			UE_LOGF(LogPCG, Error, "Only spatial data types are currently supported");
			return RemainingDataTypes;
		}

		// Add every possible combination of testable data types
		for (const FPCGDataTypeIdentifier& DataType : TestableDataTypes)
		{
			if (!DataType.IsValid() || !!(AllowedTypes & DataType))
			{
				for (int32 I = 0; I < NumMultipleInputs; ++I)
				{
					RemainingDataTypes.Emplace(DataType);
				}
			}
		}

		return RemainingDataTypes;
	}

	void RetrieveBaseOptionsPerPin(TArray<TArray<FPCGDataTypeIdentifier>>& BaseOptionsArray,
		const TArray<TObjectPtr<UPCGPin>>& InputPins,
		FPCGDataTypeIdentifier& OutDataTypesTested,
		int32 NumMultipleInputs)
	{
		BaseOptionsArray.Empty();

		for (int32 I = 0; I < InputPins.Num(); ++I)
		{
			FPCGDataTypeIdentifier AllowedDataTypes = InputPins[I]->Properties.AllowedTypes;
			if (!DataTypeIsComparable(AllowedDataTypes))
			{
				continue;
			}

			TArray<FPCGDataTypeIdentifier> TestableDataTypesForPin = FilterTestableDataTypes(AllowedDataTypes, NumMultipleInputs);
			if (!TestableDataTypesForPin.IsEmpty())
			{
				BaseOptionsArray.Emplace(TestableDataTypesForPin);
				OutDataTypesTested |= AllowedDataTypes;
			}
		}
	}

	int32 GetNumPermutations(const TArray<TArray<FPCGDataTypeIdentifier>>& BaseOptionsArray)
	{
		int32 NumPermutations = 1;

		for (int32 I = 0; I < BaseOptionsArray.Num(); ++I)
		{
			NumPermutations *= FMath::Max(BaseOptionsArray[I].Num(), 1);
		}

		return NumPermutations;
	}

	FPCGDataTypeIdentifier GetPermutation(int32 PermutationIteration, int32 PinIndex, const TArray<TArray<FPCGDataTypeIdentifier>>& BaseOptionsPerPin)
	{
		check(PermutationIteration >= 0);
		check(BaseOptionsPerPin.IsValidIndex(PinIndex));

		int32 CardinalityProduct = 1;
		for (int I = 0; I < PinIndex; ++I)
		{
			CardinalityProduct *= BaseOptionsPerPin[I].Num();
		}
		check(CardinalityProduct > 0);

		int32 PermutationIndex = (PermutationIteration / CardinalityProduct) % BaseOptionsPerPin[PinIndex].Num();
		return BaseOptionsPerPin[PinIndex][PermutationIndex];
	}

	void UpdateTestResults(FName TestName, FDeterminismTestResult& OutResult, EDeterminismLevel DeterminismLevel)
	{
		EDeterminismLevel& Result = OutResult.TestResults.FindOrAdd(TestName);
		Result = DeterminismLevel;
	}

	TFunction<bool(const UPCGData*, const UPCGData*)> GetDataCompareFunction(FPCGDataTypeIdentifier DataType, EDeterminismLevel DeterminismLevel)
	{
		if (!DataTypeIsComparable(DataType))
		{
			// Should never reach here
			UE_LOGF(LogPCG, Warning, "Attempting to compare incomparable data.");
			return ComparisonIsUnimplemented;
		}

		if (DataType.IsSameType(FPCGDataTypeInfoParam::AsId()))
		{
			return ParamDataIsIdentical;
		}

		if (DataType.IsChildOf(FPCGDataTypeInfoSpatial::AsId()))
		{
			switch (DeterminismLevel)
			{
				case EDeterminismLevel::OrderOrthogonal:
					return SpatialDataIsOrthogonal;
				case EDeterminismLevel::OrderConsistent:
					return SpatialDataIsConsistent;
				case EDeterminismLevel::OrderIndependent:
					return SpatialDataIsIdentical;
				default:
					UE_LOGF(LogPCG, Warning, "Comparable PCG DataType has no 'data comparison function'.");
					return ComparisonIsUnimplemented;
			}
		}

		UE_LOGF(LogPCG, Warning, "Can not compare non-spatial or non-param data: %ls", *DataType.ToString());
		return ComparisonIsUnimplemented;
	}

	TFunction<bool(const UPCGData*, const UPCGData*)> GetDataCollectionCompareFunction(FPCGDataTypeIdentifier DataType)
	{
		// TODO: Currently only spatial data is comparable. Other types to be added.
		if (!DataTypeIsComparable(DataType) || !DataType.IsChildOf(FPCGDataTypeInfoSpatial::AsId()))
		{
			// Should never reach here
			UE_LOGF(LogPCG, Warning, "Attempting to compare incomparable data.");
			return ComparisonIsUnimplemented;
		}

		return SpatialBasicsAreIdentical;
	}

	void ExecuteWithTestData(PCGTestsCommon::FTestData& TestData, const UPCGNode* PCGNode)
	{
		FPCGElementPtr Element = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext(PCGNode);

		// Execute both elements until complete
		while (!Element->Execute(Context.Get()))
		{
		}

		TestData.OutputData = MoveTemp(Context->OutputData);

		FPCGContext::Release(Context.Release());
	}

	void ExecuteWithSameTestData(const PCGTestsCommon::FTestData& TestData, const UPCGNode* PCGNode, FPCGDataCollection& OutFirstOutputData, FPCGDataCollection& OutSecondOutputData)
	{
		FPCGElementPtr FirstElement = TestData.Settings->GetElement();
		FPCGElementPtr SecondElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> FirstContext = PCGTestsCommon::InitializeTestContext(FirstElement.Get(), TestData.InputData, TestData.TestPCGComponent, PCGNode);
		TUniquePtr<FPCGContext> SecondContext = PCGTestsCommon::InitializeTestContext(SecondElement.Get(), TestData.InputData, TestData.TestPCGComponent, PCGNode);

		// Execute both elements until complete
		while (!FirstElement->Execute(FirstContext.Get()))
		{
		}

		// Execute both elements until complete
		while (!SecondElement->Execute(SecondContext.Get()))
		{
		}

		OutFirstOutputData = FirstContext->OutputData;
		OutSecondOutputData = SecondContext->OutputData;

		FPCGContext::Release(FirstContext.Release());
		FPCGContext::Release(SecondContext.Release());
	}

	void ExecuteWithSameTestDataSameElement(const PCGTestsCommon::FTestData& TestData, const UPCGNode* PCGNode, FPCGDataCollection& OutFirstOutputData, FPCGDataCollection& OutSecondOutputData)
	{
		FPCGElementPtr Element = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> FirstContext = TestData.InitializeTestContext(PCGNode);
		TUniquePtr<FPCGContext> SecondContext = TestData.InitializeTestContext(PCGNode);

		// Execute both elements until complete
		while (!Element->Execute(FirstContext.Get()))
		{
		}

		// Execute both elements until complete
		while (!Element->Execute(SecondContext.Get()))
		{
		}

		OutFirstOutputData = FirstContext->OutputData;
		OutSecondOutputData = SecondContext->OutputData;

		FPCGContext::Release(FirstContext.Release());
		FPCGContext::Release(SecondContext.Release());
	}

	bool ExecutionIsDeterministic(PCGTestsCommon::FTestData& FirstTestData, PCGTestsCommon::FTestData& SecondTestData, const UPCGNode* PCGNode)
	{
		ExecuteWithTestData(FirstTestData, PCGNode);
		ExecuteWithTestData(SecondTestData, PCGNode);

		return DataCollectionsContainSameData(FirstTestData.OutputData, SecondTestData.OutputData);
	}

	bool ExecutionIsConcretelyDeterministic(PCGTestsCommon::FTestData& FirstTestData, PCGTestsCommon::FTestData& SecondTestData, const UPCGNode* PCGNode)
	{
		ExecuteWithTestData(FirstTestData, PCGNode);
		ExecuteWithTestData(SecondTestData, PCGNode);

		for (FPCGTaggedData& TaggedData : FirstTestData.OutputData.TaggedData)
		{
			TaggedData.Data = CastChecked<UPCGSpatialData>(TaggedData.Data)->ToPointData();
		}

		for (FPCGTaggedData& TaggedData : SecondTestData.OutputData.TaggedData)
		{
			TaggedData.Data = CastChecked<UPCGSpatialData>(TaggedData.Data)->ToPointData();
		}

		return DataCollectionsContainSameData(FirstTestData.OutputData, SecondTestData.OutputData);
	}

	bool ExecutionIsDeterministicSameData(const PCGTestsCommon::FTestData& TestData, const UPCGNode* PCGNode)
	{
		FPCGDataCollection FirstOutput;
		FPCGDataCollection SecondOutput;
		ExecuteWithSameTestData(TestData, PCGNode, FirstOutput, SecondOutput);

		FPCGDataCollection ThirdOutput;
		FPCGDataCollection FourthOutput;
		ExecuteWithSameTestDataSameElement(TestData, PCGNode, ThirdOutput, FourthOutput);

		// Same data, so it ought to be exact
		return DataCollectionsAreIdentical(FirstOutput, SecondOutput) && DataCollectionsAreIdentical(ThirdOutput, FourthOutput);
	}
}

#undef LOCTEXT_NAMESPACE
