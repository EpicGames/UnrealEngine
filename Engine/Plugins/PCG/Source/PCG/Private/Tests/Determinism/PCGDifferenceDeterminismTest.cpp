// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_EDITOR
#include "Tests/Determinism/PCGDifferenceDeterminismTest.h"

#include "Tests/Determinism/PCGDeterminismTestsCommon.h"

#include "Elements/PCGDifferenceElement.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDifferenceDeterminismSingleSameDataTest, FPCGTestBaseClass, "Plugins.PCG.Difference.Determinism.SingleSameData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDifferenceDeterminismSingleIdenticalDataTest, FPCGTestBaseClass, "Plugins.PCG.Difference.Determinism.SingleMultipleData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDifferenceDeterminismMultipleSameDataTest, FPCGTestBaseClass, "Plugins.PCG.Difference.Determinism.MultipleSameData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDifferenceDeterminismMultipleIdenticalDataTest, FPCGTestBaseClass, "Plugins.PCG.Difference.Determinism.MultipleIdenticalData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDifferenceDeterminismOrderIndependenceTest, FPCGTestBaseClass, "Plugins.PCG.Difference.Determinism.OrderIndependence", PCGTestsCommon::TestFlags)

namespace
{
	void SetSettingsModeInferred(PCGTestsCommon::FTestData& TestData)
	{
		UPCGDifferenceSettings* Settings = CastChecked<UPCGDifferenceSettings>(TestData.Settings);
		Settings->Mode = EPCGDifferenceMode::Inferred;
	}

	void SetSettingsModeContinuous(PCGTestsCommon::FTestData& TestData)
	{
		UPCGDifferenceSettings* Settings = CastChecked<UPCGDifferenceSettings>(TestData.Settings);
		Settings->Mode = EPCGDifferenceMode::Continuous;
	}

	void SetSettingsModeDiscrete(PCGTestsCommon::FTestData& TestData)
	{
		UPCGDifferenceSettings* Settings = CastChecked<UPCGDifferenceSettings>(TestData.Settings);
		Settings->Mode = EPCGDifferenceMode::Discrete;
	}

	void DifferenceTestBase(PCGTestsCommon::FTestData& TestData, EPCGDifferenceMode DifferenceMode)
	{
		TFunction<void(PCGTestsCommon::FTestData&)> AdditionalSettingsDelegate;

		switch (DifferenceMode)
		{
		case EPCGDifferenceMode::Inferred: // To PointData if points in source and difference inputs
			AdditionalSettingsDelegate = SetSettingsModeInferred;
			break;
		case EPCGDifferenceMode::Continuous: // Stays DifferenceData
			AdditionalSettingsDelegate = SetSettingsModeContinuous;
			break;
		case EPCGDifferenceMode::Discrete: // To PointData
			AdditionalSettingsDelegate = SetSettingsModeDiscrete;
			break;
		default:
			break;
		}

		PCGTestsCommon::GenerateSettings<UPCGDifferenceSettings>(TestData, AdditionalSettingsDelegate);
		// Source
		PCGDeterminismTests::AddVolumeInputData(TestData.InputData, FVector::ZeroVector, PCGDeterminismTests::Defaults::LargeVector, PCGDeterminismTests::Defaults::SmallVector, PCGDifferenceConstants::SourceLabel);

		// Difference
		PCGDeterminismTests::AddVolumeInputData(TestData.InputData, FVector::ZeroVector, PCGDeterminismTests::Defaults::MediumVector, PCGDeterminismTests::Defaults::SmallVector, PCGDifferenceConstants::DifferencesLabel);
	}

	void DifferenceTestMultiplePoint(PCGTestsCommon::FTestData& TestData, EPCGDifferenceMode DifferenceMode)
	{
		DifferenceTestBase(TestData, DifferenceMode);

		// Randomized Sources
		PCGDeterminismTests::AddRandomizedMultiplePointInputData(TestData, 10, PCGDifferenceConstants::SourceLabel);
		PCGDeterminismTests::AddRandomizedMultiplePointInputData(TestData, 20, PCGDifferenceConstants::SourceLabel);

		// Randomized Differences
		PCGDeterminismTests::AddRandomizedMultiplePointInputData(TestData, 10, PCGDifferenceConstants::DifferencesLabel);
		PCGDeterminismTests::AddRandomizedMultiplePointInputData(TestData, 20, PCGDifferenceConstants::DifferencesLabel);
	}

	void DifferenceTestMultipleVolume(PCGTestsCommon::FTestData& TestData, EPCGDifferenceMode DifferenceMode)
	{
		DifferenceTestBase(TestData, DifferenceMode);

		// Randomized Sources
		PCGDeterminismTests::AddRandomizedVolumeInputData(TestData, PCGDifferenceConstants::SourceLabel);
		PCGDeterminismTests::AddRandomizedVolumeInputData(TestData, PCGDifferenceConstants::SourceLabel);

		// Randomized Differences
		PCGDeterminismTests::AddRandomizedVolumeInputData(TestData, PCGDifferenceConstants::DifferencesLabel);
		PCGDeterminismTests::AddRandomizedVolumeInputData(TestData, PCGDifferenceConstants::DifferencesLabel);
	}

	void DifferenceTestMultiplePointAndVolume(PCGTestsCommon::FTestData& TestData, EPCGDifferenceMode DifferenceMode)
	{
		DifferenceTestBase(TestData, DifferenceMode);

		// Randomized Sources
		PCGDeterminismTests::AddRandomizedVolumeInputData(TestData, PCGDifferenceConstants::SourceLabel);
		PCGDeterminismTests::AddRandomizedMultiplePointInputData(TestData, 20, PCGDifferenceConstants::SourceLabel);

		// Randomized Differences
		PCGDeterminismTests::AddRandomizedVolumeInputData(TestData, PCGDifferenceConstants::DifferencesLabel);
		PCGDeterminismTests::AddRandomizedMultiplePointInputData(TestData, 20, PCGDifferenceConstants::DifferencesLabel);
	}
}

namespace PCGDeterminismTests
{
	namespace DifferenceElement
	{
		bool RunTestSuite()
		{
			FString EmptyParameters;
			FPCGDifferenceDeterminismSingleSameDataTest DifferenceDeterminismSingleSameDataTest(EmptyParameters);
			FPCGDifferenceDeterminismSingleIdenticalDataTest DifferenceDeterminismSingleIdenticalDataTest(EmptyParameters);
			FPCGDifferenceDeterminismMultipleSameDataTest DifferenceDeterminismMultipleSameDataTest(EmptyParameters);
			FPCGDifferenceDeterminismMultipleIdenticalDataTest DifferenceDeterminismMultipleIdenticalDataTest(EmptyParameters);
			FPCGDifferenceDeterminismOrderIndependenceTest DifferenceDeterminismOrderIndependenceTest(EmptyParameters);

			bool bSuccess = true;
			bSuccess &= DifferenceDeterminismSingleSameDataTest.RunPCGTest(EmptyParameters);
			bSuccess &= DifferenceDeterminismSingleIdenticalDataTest.RunPCGTest(EmptyParameters);
			bSuccess &= DifferenceDeterminismMultipleSameDataTest.RunPCGTest(EmptyParameters);
			bSuccess &= DifferenceDeterminismMultipleIdenticalDataTest.RunPCGTest(EmptyParameters);
			bSuccess &= DifferenceDeterminismOrderIndependenceTest.RunPCGTest(EmptyParameters);
			return bSuccess;
		}
	}
}

bool FPCGDifferenceDeterminismSingleSameDataTest::RunTest(const FString& Parameters)
{
	// Test single same data
	PCGTestsCommon::FTestData TestDataInferred;
	PCGTestsCommon::FTestData TestDataContinuous;
	PCGTestsCommon::FTestData TestDataDiscrete;

	DifferenceTestBase(TestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestBase(TestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestBase(TestDataDiscrete, EPCGDifferenceMode::Discrete);

	TestTrue("Same single input, same output, 'Inferred' mode", PCGDeterminismTests::ExecutionIsDeterministicSameData(TestDataInferred));
	TestTrue("Same single input, same output, 'Continuous' mode", PCGDeterminismTests::ExecutionIsDeterministicSameData(TestDataContinuous));
	TestTrue("Same single input, same output, 'Discrete' mode", PCGDeterminismTests::ExecutionIsDeterministicSameData(TestDataDiscrete));

	return true;
}

bool FPCGDifferenceDeterminismSingleIdenticalDataTest::RunTest(const FString& Parameters)
{
	// Test single identical data
	PCGTestsCommon::FTestData FirstTestDataInferred;
	PCGTestsCommon::FTestData FirstTestDataContinuous;
	PCGTestsCommon::FTestData FirstTestDataDiscrete;
	PCGTestsCommon::FTestData SecondTestDataInferred;
	PCGTestsCommon::FTestData SecondTestDataContinuous;
	PCGTestsCommon::FTestData SecondTestDataDiscrete;

	DifferenceTestBase(FirstTestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestBase(FirstTestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestBase(FirstTestDataDiscrete, EPCGDifferenceMode::Discrete);
	DifferenceTestBase(SecondTestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestBase(SecondTestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestBase(SecondTestDataDiscrete, EPCGDifferenceMode::Discrete);

	TestTrue("Identical single input, same output, 'Inferred' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataInferred, SecondTestDataInferred));
	TestTrue("Identical single input, same output, 'Continuous' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataContinuous, SecondTestDataContinuous));
	TestTrue("Identical single input, same output, 'Discrete' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataDiscrete, SecondTestDataDiscrete));

	return true;
}

bool FPCGDifferenceDeterminismMultipleSameDataTest::RunTest(const FString& Parameters)
{
	// Test multiple same data
	PCGTestsCommon::FTestData TestDataInferred;
	PCGTestsCommon::FTestData TestDataContinuous;
	PCGTestsCommon::FTestData TestDataDiscrete;

	DifferenceTestMultiplePointAndVolume(TestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestMultiplePointAndVolume(TestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestMultiplePointAndVolume(TestDataDiscrete, EPCGDifferenceMode::Discrete);

	TestTrue("Same single input, same output, 'Inferred' mode", PCGDeterminismTests::ExecutionIsDeterministicSameData(TestDataInferred));
	TestTrue("Same single input, same output, 'Continuous' mode", PCGDeterminismTests::ExecutionIsDeterministicSameData(TestDataContinuous));
	TestTrue("Same single input, same output, 'Discrete' mode", PCGDeterminismTests::ExecutionIsDeterministicSameData(TestDataDiscrete));

	return true;
}

bool FPCGDifferenceDeterminismMultipleIdenticalDataTest::RunTest(const FString& Parameters)
{
	// Test multiple identical data
	PCGTestsCommon::FTestData FirstTestDataInferred;
	PCGTestsCommon::FTestData FirstTestDataContinuous;
	PCGTestsCommon::FTestData FirstTestDataDiscrete;
	PCGTestsCommon::FTestData SecondTestDataInferred;
	PCGTestsCommon::FTestData SecondTestDataContinuous;
	PCGTestsCommon::FTestData SecondTestDataDiscrete;

	DifferenceTestMultiplePointAndVolume(FirstTestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestMultiplePointAndVolume(FirstTestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestMultiplePointAndVolume(FirstTestDataDiscrete, EPCGDifferenceMode::Discrete);
	DifferenceTestMultiplePointAndVolume(SecondTestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestMultiplePointAndVolume(SecondTestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestMultiplePointAndVolume(SecondTestDataDiscrete, EPCGDifferenceMode::Discrete);

	TestTrue("Identical single input, same output, 'Inferred' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataInferred, SecondTestDataInferred));
	TestTrue("Identical single input, same output, 'Continuous' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataContinuous, SecondTestDataContinuous));
	TestTrue("Identical single input, same output, 'Discrete' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataDiscrete, SecondTestDataDiscrete));

	return true;
}

bool FPCGDifferenceDeterminismOrderIndependenceTest::RunTest(const FString& Parameters)
{
	// Test multiple identical data, shuffled
	PCGTestsCommon::FTestData FirstTestDataInferred;
	PCGTestsCommon::FTestData FirstTestDataContinuous;
	PCGTestsCommon::FTestData FirstTestDataDiscrete;
	PCGTestsCommon::FTestData SecondTestDataInferred;
	PCGTestsCommon::FTestData SecondTestDataContinuous;
	PCGTestsCommon::FTestData SecondTestDataDiscrete;

	DifferenceTestMultipleVolume(FirstTestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestMultipleVolume(FirstTestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestMultipleVolume(FirstTestDataDiscrete, EPCGDifferenceMode::Discrete);
	DifferenceTestMultipleVolume(SecondTestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestMultipleVolume(SecondTestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestMultipleVolume(SecondTestDataDiscrete, EPCGDifferenceMode::Discrete);

	PCGDeterminismTests::ShuffleInputOrder(SecondTestDataInferred);
	PCGDeterminismTests::ShuffleInputOrder(SecondTestDataContinuous);
	PCGDeterminismTests::ShuffleInputOrder(SecondTestDataDiscrete);

	TestTrue("Volume Only - Shuffled input order, same output, 'Inferred' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataInferred, SecondTestDataInferred));
	TestTrue("Volume Only - Shuffled input order, same output, 'Continuous' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataContinuous, SecondTestDataContinuous));
	TestTrue("Volume Only - Shuffled input order, same output, 'Discrete' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataDiscrete, SecondTestDataDiscrete));

	FirstTestDataInferred.Reset();
	FirstTestDataContinuous.Reset();
	FirstTestDataDiscrete.Reset();
	SecondTestDataInferred.Reset();
	SecondTestDataContinuous.Reset();
	SecondTestDataDiscrete.Reset();

	DifferenceTestMultiplePoint(FirstTestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestMultiplePoint(FirstTestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestMultiplePoint(FirstTestDataDiscrete, EPCGDifferenceMode::Discrete);
	DifferenceTestMultiplePoint(SecondTestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestMultiplePoint(SecondTestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestMultiplePoint(SecondTestDataDiscrete, EPCGDifferenceMode::Discrete);

	PCGDeterminismTests::ShuffleInputOrder(SecondTestDataInferred);
	PCGDeterminismTests::ShuffleInputOrder(SecondTestDataContinuous);
	PCGDeterminismTests::ShuffleInputOrder(SecondTestDataDiscrete);

	TestTrue("Point Only - Shuffled input order, same output, 'Inferred' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataInferred, SecondTestDataInferred));
	TestTrue("Point Only - Shuffled input order, same output, 'Continuous' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataContinuous, SecondTestDataContinuous));
	TestTrue("Point Only - Shuffled input order, same output, 'Discrete' mode", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestDataDiscrete, SecondTestDataDiscrete));

	FirstTestDataInferred.Reset();
	FirstTestDataContinuous.Reset();
	FirstTestDataDiscrete.Reset();
	SecondTestDataInferred.Reset();
	SecondTestDataContinuous.Reset();
	SecondTestDataDiscrete.Reset();

	DifferenceTestMultiplePointAndVolume(FirstTestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestMultiplePointAndVolume(FirstTestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestMultiplePointAndVolume(FirstTestDataDiscrete, EPCGDifferenceMode::Discrete);
	DifferenceTestMultiplePointAndVolume(SecondTestDataInferred, EPCGDifferenceMode::Inferred);
	DifferenceTestMultiplePointAndVolume(SecondTestDataContinuous, EPCGDifferenceMode::Continuous);
	DifferenceTestMultiplePointAndVolume(SecondTestDataDiscrete, EPCGDifferenceMode::Discrete);

	PCGDeterminismTests::ShuffleInputOrder(SecondTestDataInferred);
	PCGDeterminismTests::ShuffleInputOrder(SecondTestDataContinuous);
	PCGDeterminismTests::ShuffleInputOrder(SecondTestDataDiscrete);

	// Since the difference node will return PointData - rather than DifferenceData during a PointData to PointData comparison, just convert all output into PointData
	TestTrue("Point and Volume - Shuffled input order, same output, 'Inferred' mode", PCGDeterminismTests::ExecutionIsConcretelyDeterministic(FirstTestDataInferred, SecondTestDataInferred));
	TestTrue("Point and Volume - Shuffled input order, same output, 'Continuous' mode", PCGDeterminismTests::ExecutionIsConcretelyDeterministic(FirstTestDataContinuous, SecondTestDataContinuous));
	TestTrue("Point and Volume - Shuffled input order, same output, 'Discrete' mode", PCGDeterminismTests::ExecutionIsConcretelyDeterministic(FirstTestDataDiscrete, SecondTestDataDiscrete));

	return true;
}

#endif