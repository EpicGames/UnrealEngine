// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Elements/PCGInnerIntersectionElement.h"
#include "Tests/Determinism/PCGDeterminismTestsCommon.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGIntersectionDeterminismSingleSameDataTest, FPCGTestBaseClass, "Plugins.PCG.Intersection.Determinism.SingleSameData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGIntersectionDeterminismSingleIdenticalDataTest, FPCGTestBaseClass, "Plugins.PCG.Intersection.Determinism.SingleMultipleData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGIntersectionDeterminismMultipleSameDataTest, FPCGTestBaseClass, "Plugins.PCG.Intersection.Determinism.MultipleSameData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGIntersectionDeterminismMultipleIdenticalDataTest, FPCGTestBaseClass, "Plugins.PCG.Intersection.Determinism.MultipleIdenticalData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGIntersectionDeterminismOrderIndependenceTest, FPCGTestBaseClass, "Plugins.PCG.Intersection.Determinism.OrderIndependence", PCGTestsCommon::TestFlags)

namespace
{
	void IntersectionTestBase(PCGTestsCommon::FTestData& TestData)
	{
		PCGTestsCommon::GenerateSettings<UPCGInnerIntersectionSettings>(TestData);
		// Source Volumes
		PCGDeterminismTests::AddVolumeInputData(TestData.InputData, PCGDeterminismTests::Defaults::SmallVector, PCGDeterminismTests::Defaults::MediumVector, PCGDeterminismTests::Defaults::MediumVector);
		PCGDeterminismTests::AddVolumeInputData(TestData.InputData, PCGDeterminismTests::Defaults::SmallVector * -1.f, PCGDeterminismTests::Defaults::MediumVector, PCGDeterminismTests::Defaults::MediumVector);
	}

	void IntersectionTestMultiple(PCGTestsCommon::FTestData& TestData)
	{
		IntersectionTestBase(TestData);

		// Randomized Sources
		PCGDeterminismTests::AddRandomizedVolumeInputData(TestData);
	}
}

bool FPCGIntersectionDeterminismSingleSameDataTest::RunTest(const FString& Parameters)
{
	// Test single same data
	PCGTestsCommon::FTestData TestData;

	IntersectionTestBase(TestData);

	return TestTrue(TEXT("Same single input and settings, same output"), PCGDeterminismTests::ExecutionIsDeterministicSameData(TestData));
}

bool FPCGIntersectionDeterminismSingleIdenticalDataTest::RunTest(const FString& Parameters)
{
	// Test single identical data
	PCGTestsCommon::FTestData FirstTestData;
	PCGTestsCommon::FTestData SecondTestData;

	IntersectionTestBase(FirstTestData);
	IntersectionTestBase(SecondTestData);

	return TestTrue(TEXT("Identical single input and settings, same output"), PCGDeterminismTests::ExecutionIsDeterministic(FirstTestData, SecondTestData));
}

bool FPCGIntersectionDeterminismMultipleSameDataTest::RunTest(const FString& Parameters)
{
	// Test multiple same data
	PCGTestsCommon::FTestData TestData;

	IntersectionTestMultiple(TestData);

	return TestTrue(TEXT("Identical multiple input, same output"), PCGDeterminismTests::ExecutionIsDeterministicSameData(TestData));
}

bool FPCGIntersectionDeterminismMultipleIdenticalDataTest::RunTest(const FString& Parameters)
{
	// Test multiple identical data
	PCGTestsCommon::FTestData FirstTestData;
	PCGTestsCommon::FTestData SecondTestData;

	IntersectionTestMultiple(FirstTestData);
	IntersectionTestMultiple(SecondTestData);

	return TestTrue(TEXT("Identical single input and settings, same output"), PCGDeterminismTests::ExecutionIsDeterministic(FirstTestData, SecondTestData));
}

bool FPCGIntersectionDeterminismOrderIndependenceTest::RunTest(const FString& Parameters)
{
	// Test multiple identical, shuffled data
	PCGTestsCommon::FTestData FirstTestData;
	PCGTestsCommon::FTestData SecondTestData;

	IntersectionTestMultiple(FirstTestData);
	IntersectionTestMultiple(SecondTestData);

	PCGDeterminismTests::ShuffleInputOrder(SecondTestData);

	return TestTrue(TEXT("Shuffled input order, same output"), PCGDeterminismTests::ExecutionIsDeterministic(FirstTestData, SecondTestData));
}

#endif