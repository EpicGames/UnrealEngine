// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Elements/PCGVolumeSampler.h"
#include "Tests/Determinism/PCGDeterminismTestsCommon.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGVolumeSamplerDeterminismSingleSameDataTest, FPCGTestBaseClass, "Plugins.PCG.VolumeSampler.Determinism.SingleSameData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGVolumeSamplerDeterminismSingleIdenticalDataTest, FPCGTestBaseClass, "Plugins.PCG.VolumeSampler.Determinism.SingleIdenticalData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGVolumeSamplerDeterminismMultipleSameDataTest, FPCGTestBaseClass, "Plugins.PCG.VolumeSampler.Determinism.MultipleSameData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGVolumeSamplerDeterminismMultipleIdenticalDataTest, FPCGTestBaseClass, "Plugins.PCG.VolumeSampler.Determinism.MultipleIdenticalData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGVolumeSamplerDeterminismOrderIndependenceTest, FPCGTestBaseClass, "Plugins.PCG.VolumeSampler.Determinism.OrderIndependence", PCGTestsCommon::TestFlags)

namespace
{
	constexpr static int32 NumInputVolumesToAdd = 10;

	void RandomizeVolumeSettingsVoxelSize(PCGTestsCommon::FTestData& TestData)
	{
		UPCGVolumeSamplerSettings* Settings = CastChecked<UPCGVolumeSamplerSettings>(TestData.Settings);
		Settings->VoxelSize = PCGDeterminismTests::Defaults::MediumVector + TestData.RandomStream.VRand() * 0.5 * PCGDeterminismTests::Defaults::MediumDistance;
	}
}

bool FPCGVolumeSamplerDeterminismSingleSameDataTest::RunTest(const FString& Parameters)
{
	// Test single same data
	PCGTestsCommon::FTestData TestData;

	PCGTestsCommon::GenerateSettings<UPCGVolumeSamplerSettings>(TestData, RandomizeVolumeSettingsVoxelSize);
	PCGDeterminismTests::AddRandomizedVolumeInputData(TestData);

	TestTrue("Same single input and settings, same output", PCGDeterminismTests::ExecutionIsDeterministicSameData(TestData));

	return true;
}

bool FPCGVolumeSamplerDeterminismSingleIdenticalDataTest::RunTest(const FString& Parameters)
{
	// Test single identical data
	PCGTestsCommon::FTestData FirstTestData;
	PCGTestsCommon::FTestData SecondTestData;

	PCGTestsCommon::GenerateSettings<UPCGVolumeSamplerSettings>(FirstTestData, RandomizeVolumeSettingsVoxelSize);
	PCGTestsCommon::GenerateSettings<UPCGVolumeSamplerSettings>(SecondTestData, RandomizeVolumeSettingsVoxelSize);

	PCGDeterminismTests::AddRandomizedVolumeInputData(FirstTestData);
	PCGDeterminismTests::AddRandomizedVolumeInputData(SecondTestData);

	TestTrue("Identical single input and settings, same output", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestData, SecondTestData));

	return true;
}

bool FPCGVolumeSamplerDeterminismMultipleSameDataTest::RunTest(const FString& Parameters)
{
	// Test multiple same data
	PCGTestsCommon::FTestData TestData;

	PCGTestsCommon::GenerateSettings<UPCGVolumeSamplerSettings>(TestData, RandomizeVolumeSettingsVoxelSize);

	// Add many random volumes
	for (int32 I = 0; I < NumInputVolumesToAdd; ++I)
	{
		PCGDeterminismTests::AddRandomizedVolumeInputData(TestData);
	}

	TestTrue("Same multiple input, same output", PCGDeterminismTests::ExecutionIsDeterministicSameData(TestData));

	return true;
}

bool FPCGVolumeSamplerDeterminismMultipleIdenticalDataTest::RunTest(const FString& Parameters)
{
	// Test multiple identical data
	PCGTestsCommon::FTestData FirstTestData;
	PCGTestsCommon::FTestData SecondTestData;

	PCGTestsCommon::GenerateSettings<UPCGVolumeSamplerSettings>(FirstTestData, RandomizeVolumeSettingsVoxelSize);
	PCGTestsCommon::GenerateSettings<UPCGVolumeSamplerSettings>(SecondTestData, RandomizeVolumeSettingsVoxelSize);

	// Add many random volumes
	for (int32 I = 0; I < NumInputVolumesToAdd; ++I)
	{
		PCGDeterminismTests::AddRandomizedVolumeInputData(FirstTestData);
		PCGDeterminismTests::AddRandomizedVolumeInputData(SecondTestData);
	}

	TestTrue("Identical multiple input, same output", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestData, SecondTestData));

	return true;
}

bool FPCGVolumeSamplerDeterminismOrderIndependenceTest::RunTest(const FString& Parameters)
{
	// Test shuffled input data
	PCGTestsCommon::FTestData FirstTestData;
	PCGTestsCommon::FTestData SecondTestData;

	PCGTestsCommon::GenerateSettings<UPCGVolumeSamplerSettings>(FirstTestData, RandomizeVolumeSettingsVoxelSize);
	PCGTestsCommon::GenerateSettings<UPCGVolumeSamplerSettings>(SecondTestData, RandomizeVolumeSettingsVoxelSize);

	// Add many random volumes
	for (int32 I = 0; I < NumInputVolumesToAdd; ++I)
	{
		PCGDeterminismTests::AddRandomizedVolumeInputData(FirstTestData);
		PCGDeterminismTests::AddRandomizedVolumeInputData(SecondTestData);
	}

	PCGDeterminismTests::ShuffleInputOrder(SecondTestData);

	TestTrue("Shuffled input order, same output", PCGDeterminismTests::ExecutionIsDeterministic(FirstTestData, SecondTestData));

	return true;
}

#endif