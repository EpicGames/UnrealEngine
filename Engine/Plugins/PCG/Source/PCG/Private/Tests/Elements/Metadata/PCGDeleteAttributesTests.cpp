// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGContext.h"
#include "Elements/PCGDeleteAttributesElement.h"
#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"

#include "PCGParamData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGPointData.h"
#include "Metadata/PCGMetadata.h"

template <typename DataType>
class FPCGDeleteAttributesTests : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;
	
protected:
	static inline const FName AttributeName1 = TEXT("MyAttr123");
	static inline const FName AttributeName2 = TEXT("MyAttr23");
	static inline const FName AttributeName3 = TEXT("MyAttr3");
	static inline const TStaticArray<FName, 3> DefaultAttributeNames = {AttributeName1, AttributeName2, AttributeName3};
	
	static UPCGData* CreateData()
	{
		if constexpr (std::is_same_v<DataType, UPCGBasePointData>)
		{
			return FPCGContext::NewPointData_AnyThread(nullptr);
		}
		else
		{
			return NewObject<DataType>();
		}
	}

	static void CreateAttributes(UPCGData* Data, const FPCGMetadataDomainID DomainID, TArrayView<const FName> OptionalNames = {})
	{
		check(Data);
		UPCGMetadata* Metadata = Data->MutableMetadata();
		check(Metadata);
		FPCGMetadataDomain* MetadataDomain = Metadata->GetMetadataDomain(DomainID);
		check(MetadataDomain);

		TArrayView<const FName> AttributeNames = OptionalNames.IsEmpty() ? DefaultAttributeNames : OptionalNames;

		for (int32 i = 0; i < AttributeNames.Num(); ++i)
		{
			MetadataDomain->CreateAttribute<int32>(AttributeNames[i], i, false, false);
		}
	}
	
	const UPCGData* ExecuteElement(const UPCGData* InputData, UPCGDeleteAttributesSettings* Settings)
	{
		PCGTestsCommon::FTestData TestData(42, Settings);
		
		// Also push the settings
		TestData.InputData.TaggedData.Emplace_GetRef().Data = TestData.Settings;
		TestData.InputData.TaggedData.Last().Pin = TEXT("Settings");
		
		FPCGElementPtr TestElement = TestData.Settings->GetElement();
		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		FPCGTaggedData& NewTaggedData = Context->InputData.TaggedData.Emplace_GetRef();
		NewTaggedData.Data = InputData;
		NewTaggedData.Pin = PCGPinConstants::DefaultInputLabel;


		while (!TestElement->Execute(Context.Get())) {}

		if (TestTrue(TEXT("Element execution has 1 output"), Context->OutputData.TaggedData.Num() == 1)
			&& TestNotNull(TEXT("Element execution output data is valid"), Context->OutputData.TaggedData[0].Data.Get()))
		{
			return Context->OutputData.TaggedData[0].Data.Get();
		}
		else
		{
			return nullptr;
		}
	}

	bool ValidateAttributes(const UPCGData* Data, const FPCGMetadataDomainID DomainID, TArrayView<const FName> ExpectedNames)
	{
		if (!Data)
		{
			return false;
		}

		const UPCGMetadata* Metadata = Data->ConstMetadata();
		check(Metadata);
		const FPCGMetadataDomain* MetadataDomain = Metadata->GetConstMetadataDomain(DomainID);
		check(MetadataDomain);

		TArray<FName> AttributeNames;
		TArray<EPCGMetadataTypes> AttributeTypes;
		MetadataDomain->GetAttributes(AttributeNames, AttributeTypes);

		for (const FName ExpectedName : ExpectedNames)
		{
			UTEST_TRUE(*FString::Printf(TEXT("Attribute %s should exist in domain %s"), *ExpectedName.ToString(), *DomainID.DebugName.ToString()), AttributeNames.Contains(ExpectedName));
		}

		for (const FName AttributeName : AttributeNames)
		{
			UTEST_TRUE(*FString::Printf(TEXT("Attribute %s is expected in domain %s"), *AttributeName.ToString(), *DomainID.DebugName.ToString()), ExpectedNames.Contains(AttributeName));
		}

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDeleteAttributesTests_Points_DeleteSelected, FPCGDeleteAttributesTests<UPCGBasePointData>, "Plugins.PCG.DeleteAttributes.Points.DeleteSelected", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDeleteAttributesTests_Points_KeepSelected, FPCGDeleteAttributesTests<UPCGBasePointData>, "Plugins.PCG.DeleteAttributes.Points.KeepSelected", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDeleteAttributesTests_Points_KeepSelectedEmpty, FPCGDeleteAttributesTests<UPCGBasePointData>, "Plugins.PCG.DeleteAttributes.Points.KeepSelectedEmpty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDeleteAttributesTests_Points_DeleteSelectedSubstring, FPCGDeleteAttributesTests<UPCGBasePointData>, "Plugins.PCG.DeleteAttributes.Points.DeleteSelectedSubstring", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDeleteAttributesTests_Points_DeleteSelectedMatches, FPCGDeleteAttributesTests<UPCGBasePointData>, "Plugins.PCG.DeleteAttributes.Points.DeleteSelectedMatches", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDeleteAttributesTests_Points_KeepSelectedSubstring, FPCGDeleteAttributesTests<UPCGBasePointData>, "Plugins.PCG.DeleteAttributes.Points.KeepSelectedSubstring", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDeleteAttributesTests_Points_KeepSelectedMatches, FPCGDeleteAttributesTests<UPCGBasePointData>, "Plugins.PCG.DeleteAttributes.Points.KeepSelectedMatches", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDeleteAttributesTests_Points_DeleteSelectedData, FPCGDeleteAttributesTests<UPCGBasePointData>, "Plugins.PCG.DeleteAttributes.Points.DeleteSelectedData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDeleteAttributesTests_Points_DeleteSelectedElements, FPCGDeleteAttributesTests<UPCGBasePointData>, "Plugins.PCG.DeleteAttributes.Points.DeleteSelectedElements", PCGTestsCommon::TestFlags)

/**
 * Delete
 * Attributes In: MyAttr123, MyAttr23, MyAttr3
 * Selection: "MyAttr123,MyAttr23"
 * Expected out: MyAttr3
 */
bool FPCGDeleteAttributesTests_Points_DeleteSelected::RunTest(const FString& Parameters)
{
	UPCGData* InputData = CreateData();
	CreateAttributes(InputData, PCGMetadataDomainID::Default);

	UPCGDeleteAttributesSettings* Settings = NewObject<UPCGDeleteAttributesSettings>();
	Settings->Operation = EPCGAttributeFilterOperation::DeleteSelectedAttributes;
	Settings->Operator = EPCGStringMatchingOperator::Equal;
	Settings->MetadataDomain = PCGDataConstants::DefaultDomainName;
	Settings->SelectedAttributes = AttributeName1.ToString() + TEXT(",") + AttributeName2.ToString();
	
	return ValidateAttributes(ExecuteElement(InputData, Settings), PCGMetadataDomainID::Default, {AttributeName3});
}

/**
 * Keep
 * Attributes In: MyAttr123, MyAttr23, MyAttr3
 * Selection: "MyAttr123,MyAttr23"
 * Expected out: MyAttr123,MyAttr23
 */
bool FPCGDeleteAttributesTests_Points_KeepSelected::RunTest(const FString& Parameters)
{
	UPCGData* InputData = CreateData();
	CreateAttributes(InputData, PCGMetadataDomainID::Default);

	UPCGDeleteAttributesSettings* Settings = NewObject<UPCGDeleteAttributesSettings>();
	Settings->Operation = EPCGAttributeFilterOperation::KeepSelectedAttributes;
	Settings->Operator = EPCGStringMatchingOperator::Equal;
	Settings->MetadataDomain = PCGDataConstants::DefaultDomainName;
	Settings->SelectedAttributes = AttributeName1.ToString() + TEXT(",") + AttributeName2.ToString();
	
	return ValidateAttributes(ExecuteElement(InputData, Settings), PCGMetadataDomainID::Default, {AttributeName1, AttributeName2});
}

/**
 * Keep
 * Attributes In: MyAttr123, MyAttr23, MyAttr3
 * Selection: ""
 * Expected out: Nothing
 */
bool FPCGDeleteAttributesTests_Points_KeepSelectedEmpty::RunTest(const FString& Parameters)
{
	UPCGData* InputData = CreateData();
	CreateAttributes(InputData, PCGMetadataDomainID::Default);

	UPCGDeleteAttributesSettings* Settings = NewObject<UPCGDeleteAttributesSettings>();
	Settings->Operation = EPCGAttributeFilterOperation::KeepSelectedAttributes;
	Settings->Operator = EPCGStringMatchingOperator::Equal;
	Settings->MetadataDomain = PCGDataConstants::DefaultDomainName;
	Settings->SelectedAttributes = FString();
	
	return ValidateAttributes(ExecuteElement(InputData, Settings), PCGMetadataDomainID::Default, {});
}

/**
 * Delete substring
 * Attributes In: MyAttr123, MyAttr23, MyAttr3
 * Selection: "MyAttr1"
 * Expected out: MyAttr23, MyAttr3
 */
bool FPCGDeleteAttributesTests_Points_DeleteSelectedSubstring::RunTest(const FString& Parameters)
{
	UPCGData* InputData = CreateData();
	CreateAttributes(InputData, PCGMetadataDomainID::Default);

	UPCGDeleteAttributesSettings* Settings = NewObject<UPCGDeleteAttributesSettings>();
	Settings->Operation = EPCGAttributeFilterOperation::DeleteSelectedAttributes;
	Settings->Operator = EPCGStringMatchingOperator::Substring;
	Settings->MetadataDomain = PCGDataConstants::DefaultDomainName;
	Settings->SelectedAttributes = TEXT("MyAttr1");
	
	return ValidateAttributes(ExecuteElement(InputData, Settings), PCGMetadataDomainID::Default, {AttributeName2, AttributeName3});
}

/**
 * Delete matches
 * Attributes In: MyAttr123, MyAttr23, MyAttr3
 * Selection: "MyAttr*3"
 * Expected out:  Nothing
 */
bool FPCGDeleteAttributesTests_Points_DeleteSelectedMatches::RunTest(const FString& Parameters)
{
	UPCGData* InputData = CreateData();
	CreateAttributes(InputData, PCGMetadataDomainID::Default);

	UPCGDeleteAttributesSettings* Settings = NewObject<UPCGDeleteAttributesSettings>();
	Settings->Operation = EPCGAttributeFilterOperation::DeleteSelectedAttributes;
	Settings->Operator = EPCGStringMatchingOperator::Matches;
	Settings->MetadataDomain = PCGDataConstants::DefaultDomainName;
	Settings->SelectedAttributes = TEXT("MyAttr*3");
	
	return ValidateAttributes(ExecuteElement(InputData, Settings), PCGMetadataDomainID::Default, {});
}

/**
 * Kepp substring
 * Attributes In: MyAttr123, MyAttr23, MyAttr3
 * Selection: "MyAttr1"
 * Expected out: MyAttr123
 */
bool FPCGDeleteAttributesTests_Points_KeepSelectedSubstring::RunTest(const FString& Parameters)
{
	UPCGData* InputData = CreateData();
	CreateAttributes(InputData, PCGMetadataDomainID::Default);

	UPCGDeleteAttributesSettings* Settings = NewObject<UPCGDeleteAttributesSettings>();
	Settings->Operation = EPCGAttributeFilterOperation::KeepSelectedAttributes;
	Settings->Operator = EPCGStringMatchingOperator::Substring;
	Settings->MetadataDomain = PCGDataConstants::DefaultDomainName;
	Settings->SelectedAttributes = TEXT("MyAttr1");
	
	return ValidateAttributes(ExecuteElement(InputData, Settings), PCGMetadataDomainID::Default, {AttributeName1});
}

/**
 * Keep matches
 * Attributes In: MyAttr123, MyAttr23, MyAttr3
 * Selection: "MyAttr*3"
 * Expected out:  MyAttr123, MyAttr23, MyAttr3
 */
bool FPCGDeleteAttributesTests_Points_KeepSelectedMatches::RunTest(const FString& Parameters)
{
	UPCGData* InputData = CreateData();
	CreateAttributes(InputData, PCGMetadataDomainID::Default);

	UPCGDeleteAttributesSettings* Settings = NewObject<UPCGDeleteAttributesSettings>();
	Settings->Operation = EPCGAttributeFilterOperation::KeepSelectedAttributes;
	Settings->Operator = EPCGStringMatchingOperator::Matches;
	Settings->MetadataDomain = PCGDataConstants::DefaultDomainName;
	Settings->SelectedAttributes = TEXT("MyAttr*3");
	
	return ValidateAttributes(ExecuteElement(InputData, Settings), PCGMetadataDomainID::Default, {AttributeName1, AttributeName2, AttributeName3});
}

/**
 * Delete Data
 * Attributes In:
 *    - Elements: MyAttr123, MyAttr23, MyAttr3
 *    - Data: MyAttr123, MyAttr23, MyAttr3
 * Selection: "MyAttr123"
 * Expected out: 
 *    - Elements: MyAttr123, MyAttr23, MyAttr3
 *    - Data: MyAttr23, MyAttr3
 */
bool FPCGDeleteAttributesTests_Points_DeleteSelectedData::RunTest(const FString& Parameters)
{
	UPCGData* InputData = CreateData();
	CreateAttributes(InputData, PCGMetadataDomainID::Elements);
	CreateAttributes(InputData, PCGMetadataDomainID::Data);

	UPCGDeleteAttributesSettings* Settings = NewObject<UPCGDeleteAttributesSettings>();
	Settings->Operation = EPCGAttributeFilterOperation::DeleteSelectedAttributes;
	Settings->Operator = EPCGStringMatchingOperator::Equal;
	Settings->MetadataDomain = PCGDataConstants::DataDomainName;
	Settings->SelectedAttributes = AttributeName1.ToString();

	const UPCGData* OutputData = ExecuteElement(InputData, Settings);

	return ValidateAttributes(OutputData, PCGMetadataDomainID::Data, {AttributeName2, AttributeName3})
		&& ValidateAttributes(OutputData, PCGMetadataDomainID::Elements, {AttributeName1, AttributeName2, AttributeName3});
}

/**
 * Delete Elements
 * Attributes In:
 *    - Elements: MyAttr123, MyAttr23, MyAttr3
 *    - Data: MyAttr123, MyAttr23, MyAttr3
 * Selection: "MyAttr123,MyAttr23"
 * Expected out: 
 *    - Elements: MyAttr3
 *    - Data: MyAttr123, MyAttr23, MyAttr3
 */
bool FPCGDeleteAttributesTests_Points_DeleteSelectedElements::RunTest(const FString& Parameters)
{
	UPCGData* InputData = CreateData();
	CreateAttributes(InputData, PCGMetadataDomainID::Elements);
	CreateAttributes(InputData, PCGMetadataDomainID::Data);

	UPCGDeleteAttributesSettings* Settings = NewObject<UPCGDeleteAttributesSettings>();
	Settings->Operation = EPCGAttributeFilterOperation::DeleteSelectedAttributes;
	Settings->Operator = EPCGStringMatchingOperator::Equal;
	Settings->MetadataDomain = PCGPointDataConstants::ElementsDomainName;
	Settings->SelectedAttributes = AttributeName1.ToString() + TEXT(",") + AttributeName2.ToString();

	const UPCGData* OutputData = ExecuteElement(InputData, Settings);

	return ValidateAttributes(OutputData, PCGMetadataDomainID::Data, {AttributeName1, AttributeName2, AttributeName3})
		&& ValidateAttributes(OutputData, PCGMetadataDomainID::Elements, {AttributeName3});
}

#endif // WITH_EDITOR