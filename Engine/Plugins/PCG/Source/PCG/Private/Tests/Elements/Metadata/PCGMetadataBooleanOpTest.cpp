// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"

#include "Elements/Metadata/PCGMetadataBooleanOpElement.h"

#if WITH_EDITOR

struct FPCGMetadataBooleanOpTestState
{
	UPCGMetadataBooleanSettings* Settings = nullptr;
	UPCGParamData* ParamData1 = nullptr;
	UPCGParamData* ParamData2 = nullptr;
};

class FPCGMetadataBooleanOpTest : public FPCGTestBaseClass
{
public:
	const FName TrueAttribute = TEXT("True");
	const FName FalseAttribute = TEXT("False");
	const FName Int0Attribute = TEXT("Int0");
	const FName Int1Attribute = TEXT("Int1");
	const FName InvalidAttribute = TEXT("Invalid");
	const FName OutputAttributeName = TEXT("Output");

	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	static FPCGMetadataBooleanOpTestState Init()
	{
		FPCGMetadataBooleanOpTestState State;

		State.ParamData1 = PCGTestsCommon::CreateEmptyParamData();
		State.ParamData2 = PCGTestsCommon::CreateEmptyParamData();

		constexpr bool bAllowInterpolation = false;
		constexpr bool bOverrideParent = false;

		State.ParamData1->Metadata->CreateAttribute<bool>(TEXT("True"), /*DefaultValue=*/ true, bAllowInterpolation, bOverrideParent);
		State.ParamData1->Metadata->CreateAttribute<bool>(TEXT("False"), /*DefaultValue=*/ false, bAllowInterpolation, bOverrideParent);
		State.ParamData1->Metadata->CreateAttribute<int64>(TEXT("Int0"), /*DefaultValue=*/ 0, bAllowInterpolation, bOverrideParent);
		State.ParamData1->Metadata->CreateAttribute<int64>(TEXT("Int1"), /*DefaultValue=*/ 1, bAllowInterpolation, bOverrideParent);
		State.ParamData1->Metadata->CreateAttribute<FVector>(TEXT("Invalid"), /*DefaultValue=*/ FVector::ZeroVector, bAllowInterpolation, bOverrideParent);
		State.ParamData1->Metadata->AddEntry();

		State.ParamData2->Metadata->CreateAttribute<bool>(TEXT("True"), /*DefaultValue=*/ true, bAllowInterpolation, bOverrideParent);
		State.ParamData2->Metadata->CreateAttribute<bool>(TEXT("False"), /*DefaultValue=*/ false, bAllowInterpolation, bOverrideParent);
		State.ParamData2->Metadata->CreateAttribute<int64>(TEXT("Int0"), /*DefaultValue=*/ 0, bAllowInterpolation, bOverrideParent);
		State.ParamData2->Metadata->CreateAttribute<int64>(TEXT("Int1"), /*DefaultValue=*/ 1, bAllowInterpolation, bOverrideParent);
		State.ParamData2->Metadata->CreateAttribute<FVector>(TEXT("Invalid"), /*DefaultValue=*/ FVector::ZeroVector, bAllowInterpolation, bOverrideParent);
		State.ParamData2->Metadata->AddEntry();

		State.Settings = NewObject<UPCGMetadataBooleanSettings>();

		return State;
	}

	bool RunInternal(const EPCGMetadataBooleanOperation Op, const FName Attr1, const FName Attr2, const bool bExpectedResult, bool bIsValid = true)
	{
		FPCGMetadataBooleanOpTestState State = Init();

		PCGTestsCommon::FTestData TestData;
		TestData.Reset(State.Settings);
		
		FPCGElementPtr MetadataBooleanElement = TestData.Settings->GetElement();

		FPCGTaggedData& ParamTaggedData1 = TestData.InputData.TaggedData.Emplace_GetRef();
		ParamTaggedData1.Data = State.ParamData1;
		ParamTaggedData1.Pin = Op == EPCGMetadataBooleanOperation::Not ? PCGPinConstants::DefaultInputLabel : PCGMetadataSettingsBaseConstants::DoubleInputFirstLabel;

		FPCGTaggedData& ParamTaggedData2 = TestData.InputData.TaggedData.Emplace_GetRef();
		ParamTaggedData2.Data = State.ParamData2;
		ParamTaggedData2.Pin = PCGMetadataSettingsBaseConstants::DoubleInputSecondLabel;

		State.Settings->Operation = Op;
		State.Settings->InputSource1.SetAttributeName(Attr1);
		State.Settings->InputSource2.SetAttributeName(Attr2);
		State.Settings->OutputTarget.SetAttributeName(OutputAttributeName);
		State.Settings->ForceOutputConnections[0] = true;

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!MetadataBooleanElement->Execute(Context.Get())){}

		const TArray<FPCGTaggedData> Outputs = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);

		// If we expect an invalid op, check that we have no output and exit immediately.
		if (!bIsValid)
		{
			return TestTrue(TEXT("Invalid op == no output"), Outputs.IsEmpty());
		}

		UTEST_EQUAL("Number of outputs", Outputs.Num(), 1)

		const UPCGData* OutputData = Outputs[0].Data;
		UTEST_TRUE("Output data is param data", OutputData->IsA<UPCGParamData>())
		
		const UPCGMetadata* OutMetadata = OutputData->ConstMetadata();
		UTEST_NOT_NULL("Output metadata exists", OutMetadata)

		const FPCGMetadataAttribute<bool>* OutAttribute = OutMetadata->GetConstTypedAttribute<bool>(State.Settings->OutputTarget.GetName());
		UTEST_NOT_NULL("Output attribute exists", OutAttribute);
		UTEST_EQUAL("Output attribute has an entry", OutAttribute->GetNumberOfEntries(), 1);

		const bool Out = OutAttribute->GetValueFromItemKey(PCGInvalidEntryKey);
		UTEST_EQUAL("Op has the expected result", Out, bExpectedResult);
		
		return true;
	}

	bool Run(const EPCGMetadataBooleanOperation Op, const FName Attr1, const FName Attr2, const bool bExpectedResult, bool bIsValid = true)
	{
		if (!RunInternal(Op, Attr1, Attr2, bExpectedResult, bIsValid))
		{
			const UEnum* Enum = StaticEnum<EPCGMetadataBooleanOperation>();
			check(Enum);
			const FName OpName(Enum->GetNameStringByValue(static_cast<int64>(Op)));
			
			UE_LOGF(LogPCG, Error, "%ls %ls %ls == %ls failed", *Attr1.ToString(), *OpName.ToString(), *Attr2.ToString(), bExpectedResult ? TEXT("true") : TEXT("false"));
			return false;
		}
		else
		{
			return true;
		}
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBooleanNotTest, FPCGMetadataBooleanOpTest, "Plugins.PCG.Metadata.BooleanOp.Not", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBooleanAndTest, FPCGMetadataBooleanOpTest, "Plugins.PCG.Metadata.BooleanOp.And", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBooleanOrTest, FPCGMetadataBooleanOpTest, "Plugins.PCG.Metadata.BooleanOp.Or", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBooleanXorTest, FPCGMetadataBooleanOpTest, "Plugins.PCG.Metadata.BooleanOp.Xor", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBooleanNandTest, FPCGMetadataBooleanOpTest, "Plugins.PCG.Metadata.BooleanOp.Nand", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBooleanNorTest, FPCGMetadataBooleanOpTest, "Plugins.PCG.Metadata.BooleanOp.Nor", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBooleanXnorTest, FPCGMetadataBooleanOpTest, "Plugins.PCG.Metadata.BooleanOp.Xnor", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBooleanImplyTest, FPCGMetadataBooleanOpTest, "Plugins.PCG.Metadata.BooleanOp.Imply", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBooleanNimplyTest, FPCGMetadataBooleanOpTest, "Plugins.PCG.Metadata.BooleanOp.Nimply", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBooleanInvalidTest, FPCGMetadataBooleanOpTest, "Plugins.PCG.Metadata.BooleanOp.Invalid", PCGTestsCommon::TestFlags)

bool FPCGMetadataBooleanNotTest::RunTest(const FString& Parameters)
{
	// Attr1, bExpectedResult
	TArray<TTuple<FName, bool>> TestValues =
	{
		{TrueAttribute, false},
		{FalseAttribute, true},
		{Int0Attribute, true},
		{Int1Attribute, false}
	};

	for (const auto [Attr1, bExpectedResult] : TestValues)
	{
		if (!Run(EPCGMetadataBooleanOperation::Not, Attr1, NAME_None, bExpectedResult))
		{
			return false;
		}
	}

	return true;
}

bool FPCGMetadataBooleanOrTest::RunTest(const FString& Parameters)
{	
	// Attr1, Attr2, bExpectedResult
	TArray<TTuple<FName, FName, bool>> TestValues =
	{
		{TrueAttribute, FalseAttribute, true},
		{TrueAttribute, TrueAttribute, true},
		{TrueAttribute, Int0Attribute, true},
		{TrueAttribute, Int1Attribute, true},
		
		{FalseAttribute, FalseAttribute, false},
		{FalseAttribute, TrueAttribute, true},
		{FalseAttribute, Int0Attribute, false},
		{FalseAttribute, Int1Attribute, true},

		{Int0Attribute, FalseAttribute, false},
		{Int0Attribute, TrueAttribute, true},
		{Int0Attribute, Int0Attribute, false},
		{Int0Attribute, Int1Attribute, true},

		{Int1Attribute, FalseAttribute, true},
		{Int1Attribute, TrueAttribute, true},
		{Int1Attribute, Int0Attribute, true},
		{Int1Attribute, Int1Attribute, true},
	};

	for (const auto [Attr1, Attr2, bExpectedResult] : TestValues)
	{
		if (!Run(EPCGMetadataBooleanOperation::Or, Attr1, Attr2, bExpectedResult))
		{
			return false;
		}
	}

	return true;
}

bool FPCGMetadataBooleanAndTest::RunTest(const FString& Parameters)
{
	// Attr1, Attr2, bExpectedResult
	TArray<TTuple<FName, FName, bool>> TestValues
	{
		{TrueAttribute, FalseAttribute, false},
		{TrueAttribute, TrueAttribute, true},
		{TrueAttribute, Int0Attribute, false},
		{TrueAttribute, Int1Attribute, true},
				
		{FalseAttribute, FalseAttribute, false},
		{FalseAttribute, TrueAttribute, false},
		{FalseAttribute, Int0Attribute, false},
		{FalseAttribute, Int1Attribute, false},

		{Int0Attribute, FalseAttribute, false},
		{Int0Attribute, TrueAttribute, false},
		{Int0Attribute, Int0Attribute, false},
		{Int0Attribute, Int1Attribute, false},

		{Int1Attribute, FalseAttribute, false},
		{Int1Attribute, TrueAttribute, true},
		{Int1Attribute, Int0Attribute, false},
		{Int1Attribute, Int1Attribute, true},
	};

	for (const auto [Attr1, Attr2, bExpectedResult] : TestValues)
	{
		if (!Run(EPCGMetadataBooleanOperation::And, Attr1, Attr2, bExpectedResult))
		{
			return false;
		}
	}

	return true;
}

bool FPCGMetadataBooleanXorTest::RunTest(const FString& Parameters)
{
	// Attr1, Attr2, bExpectedResult
	TArray<TTuple<FName, FName, bool>> TestValues
	{
		{TrueAttribute, FalseAttribute, true},
		{TrueAttribute, TrueAttribute, false},
		{TrueAttribute, Int0Attribute, true},
		{TrueAttribute, Int1Attribute, false},
			
		{FalseAttribute, FalseAttribute, false},
		{FalseAttribute, TrueAttribute, true},
		{FalseAttribute, Int0Attribute, false},
		{FalseAttribute, Int1Attribute, true},

		{Int0Attribute, FalseAttribute, false},
		{Int0Attribute, TrueAttribute, true},
		{Int0Attribute, Int0Attribute, false},
		{Int0Attribute, Int1Attribute, true},

		{Int1Attribute, FalseAttribute, true},
		{Int1Attribute, TrueAttribute, false},
		{Int1Attribute, Int0Attribute, true},
		{Int1Attribute, Int1Attribute, false},
	};

	for (const auto [Attr1, Attr2, bExpectedResult] : TestValues)
	{
		if (!Run(EPCGMetadataBooleanOperation::Xor, Attr1, Attr2, bExpectedResult))
		{
			return false;
		}
	}

	return true;
}

bool FPCGMetadataBooleanNandTest::RunTest(const FString& Parameters)
{
	// Attr1, Attr2, bExpectedResult
	TArray<TTuple<FName, FName, bool>> TestValues
	{
		{TrueAttribute, FalseAttribute, true},
		{TrueAttribute, TrueAttribute, false},
		{TrueAttribute, Int0Attribute, true},
		{TrueAttribute, Int1Attribute, false},
		
		{FalseAttribute, FalseAttribute, true},
		{FalseAttribute, TrueAttribute, true},
		{FalseAttribute, Int0Attribute, true},
		{FalseAttribute, Int1Attribute, true},

		{Int0Attribute, FalseAttribute, true},
		{Int0Attribute, TrueAttribute, true},
		{Int0Attribute, Int0Attribute, true},
		{Int0Attribute, Int1Attribute, true},

		{Int1Attribute, FalseAttribute, true},
		{Int1Attribute, TrueAttribute, false},
		{Int1Attribute, Int0Attribute, true},
		{Int1Attribute, Int1Attribute, false},
	};

	for (const auto [Attr1, Attr2, bExpectedResult] : TestValues)
	{
		if (!Run(EPCGMetadataBooleanOperation::Nand, Attr1, Attr2, bExpectedResult))
		{
			return false;
		}
	}

	return true;
}

bool FPCGMetadataBooleanNorTest::RunTest(const FString& Parameters)
{
	// Attr1, Attr2, bExpectedResult
	TArray<TTuple<FName, FName, bool>> TestValues =
	{
		{TrueAttribute, FalseAttribute, false},
		{TrueAttribute, TrueAttribute, false},
		{TrueAttribute, Int0Attribute, false},
		{TrueAttribute, Int1Attribute, false},
		
		{FalseAttribute, FalseAttribute, true},
		{FalseAttribute, TrueAttribute, false},
		{FalseAttribute, Int0Attribute, true},
		{FalseAttribute, Int1Attribute, false},

		{Int0Attribute, FalseAttribute, true},
		{Int0Attribute, TrueAttribute, false},
		{Int0Attribute, Int0Attribute, true},
		{Int0Attribute, Int1Attribute, false},

		{Int1Attribute, FalseAttribute, false},
		{Int1Attribute, TrueAttribute, false},
		{Int1Attribute, Int0Attribute, false},
		{Int1Attribute, Int1Attribute, false},
	};

	for (const auto [Attr1, Attr2, bExpectedResult] : TestValues)
	{
		if (!Run(EPCGMetadataBooleanOperation::Nor, Attr1, Attr2, bExpectedResult))
		{
			return false;
		}
	}

	return true;
}

bool FPCGMetadataBooleanXnorTest::RunTest(const FString& Parameters)
{
	// Attr1, Attr2, bExpectedResult
	TArray<TTuple<FName, FName, bool>> TestValues
	{
		{TrueAttribute, FalseAttribute, false},
		{TrueAttribute, TrueAttribute, true},
		{TrueAttribute, Int0Attribute, false},
		{TrueAttribute, Int1Attribute, true},
		
		{FalseAttribute, FalseAttribute, true},
		{FalseAttribute, TrueAttribute, false},
		{FalseAttribute, Int0Attribute, true},
		{FalseAttribute, Int1Attribute, false},

		{Int0Attribute, FalseAttribute, true},
		{Int0Attribute, TrueAttribute, false},
		{Int0Attribute, Int0Attribute, true},
		{Int0Attribute, Int1Attribute, false},

		{Int1Attribute, FalseAttribute, false},
		{Int1Attribute, TrueAttribute, true},
		{Int1Attribute, Int0Attribute, false},
		{Int1Attribute, Int1Attribute, true},
	};

	for (const auto [Attr1, Attr2, bExpectedResult] : TestValues)
	{
		if (!Run(EPCGMetadataBooleanOperation::Xnor, Attr1, Attr2, bExpectedResult))
		{
			return false;
		}
	}

	return true;
}

bool FPCGMetadataBooleanImplyTest::RunTest(const FString& Parameters)
{
	// Attr1, Attr2, bExpectedResult
	TArray<TTuple<FName, FName, bool>> TestValues =
	{
		{TrueAttribute, FalseAttribute, false},
		{TrueAttribute, TrueAttribute, true},
		{TrueAttribute, Int0Attribute, false},
		{TrueAttribute, Int1Attribute, true},
		
		{FalseAttribute, FalseAttribute, true},
		{FalseAttribute, TrueAttribute, true},
		{FalseAttribute, Int0Attribute, true},
		{FalseAttribute, Int1Attribute, true},

		{Int0Attribute, FalseAttribute, true},
		{Int0Attribute, TrueAttribute, true},
		{Int0Attribute, Int0Attribute, true},
		{Int0Attribute, Int1Attribute, true},

		{Int1Attribute, FalseAttribute, false},
		{Int1Attribute, TrueAttribute, true},
		{Int1Attribute, Int0Attribute, false},
		{Int1Attribute, Int1Attribute, true},
	};

	for (const auto [Attr1, Attr2, bExpectedResult] : TestValues)
	{
		if (!Run(EPCGMetadataBooleanOperation::Imply, Attr1, Attr2, bExpectedResult))
		{
			return false;
		}
	}

	return true;
}

bool FPCGMetadataBooleanNimplyTest::RunTest(const FString& Parameters)
{
	// Attr1, Attr2, bExpectedResult
	TArray<TTuple<FName, FName, bool>> TestValues
	{
		{TrueAttribute, FalseAttribute, true},
		{TrueAttribute, TrueAttribute, false},
		{TrueAttribute, Int0Attribute, true},
		{TrueAttribute, Int1Attribute, false},
		
		{FalseAttribute, FalseAttribute, false},
		{FalseAttribute, TrueAttribute, false},
		{FalseAttribute, Int0Attribute, false},
		{FalseAttribute, Int1Attribute, false},

		{Int0Attribute, FalseAttribute, false},
		{Int0Attribute, TrueAttribute, false},
		{Int0Attribute, Int0Attribute, false},
		{Int0Attribute, Int1Attribute, false},

		{Int1Attribute, FalseAttribute, true},
		{Int1Attribute, TrueAttribute, false},
		{Int1Attribute, Int0Attribute, true},
		{Int1Attribute, Int1Attribute, false},
	};

	for (const auto [Attr1, Attr2, bExpectedResult] : TestValues)
	{
		if (!Run(EPCGMetadataBooleanOperation::Nimply, Attr1, Attr2, bExpectedResult))
		{
			return false;
		}
	}

	return true;
}

bool FPCGMetadataBooleanInvalidTest::RunTest(const FString& Parameters)
{
	AddExpectedError(TEXT("Attribute/Property 'Invalid' from pin InA is not a supported type ('Vector')"), EAutomationExpectedErrorFlags::Contains, 2, /*bIsRegex=*/false);
	AddExpectedError(TEXT("Attribute/Property 'Invalid' from pin InB is not a supported type ('Vector')"), EAutomationExpectedErrorFlags::Contains, 1, /*bIsRegex=*/false);
	AddExpectedError(TEXT("Attribute/Property 'Invalid' from pin In is not a supported type ('Vector')"), EAutomationExpectedErrorFlags::Contains, 1, /*bIsRegex=*/false);
	
	if (!Run(EPCGMetadataBooleanOperation::Not, InvalidAttribute, InvalidAttribute, /*bExpectedResult*/ false, /*bIsValid=*/ false))
	{
		return false;
	}
	
	if (!Run(EPCGMetadataBooleanOperation::Or, InvalidAttribute, FalseAttribute, /*bExpectedResult*/ false, /*bIsValid=*/ false))
	{
		return false;
	}

	if (!Run(EPCGMetadataBooleanOperation::Or, FalseAttribute, InvalidAttribute, /*bExpectedResult*/ false, /*bIsValid=*/ false))
	{
		return false;
	}

	if (!Run(EPCGMetadataBooleanOperation::Or, InvalidAttribute, InvalidAttribute, /*bExpectedResult*/ false, /*bIsValid=*/ false))
	{
		return false;
	}
	
	return true;
}

#endif // WITH_EDITOR
