// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "Elements/Metadata/PCGMetadataMakeTransform.h"
#include "PCGComponent.h"
#include "Metadata/PCGMetadataAccessor.h"
#include "PCGParamData.h"

#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataBreakTransform.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "PCGContext.h"

#if WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBreakTransformTest, FPCGTestBaseClass, "Plugins.PCG.Metadata.BreakTransform", PCGTestsCommon::TestFlags)

namespace PCGBreakTransformTest
{
	enum class EPCGComponentToCheck
	{
		Translation,
		Rotation,
		Scale
	};

	const FName TransformAttribute = TEXT("Transform");
	const FName InvalidAttribute = TEXT("Float");
	
	FTransform RandomTransform(FRandomStream& RandomSource)
	{
		FVector RandomTranslation = RandomSource.VRand();
		FQuat RandomRotation = FRotator(RandomSource.FRand(), RandomSource.FRand(), RandomSource.FRand()).Quaternion();
		FVector RandomScale = RandomSource.VRand();

		return FTransform(RandomRotation, RandomTranslation, RandomScale);
	}

	void GenerateSpatialData(PCGTestsCommon::FTestData& TestData)
	{
		FPCGTaggedData& SpatialTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateRandomBasePointData(10, TestData.Seed);
		SpatialTaggedData.Data = PointData;
		SpatialTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		const bool bAllowsInterpolation = false;
		const bool bOverrideParent = false;

		PointData->Metadata->CreateTransformAttribute(TransformAttribute, FTransform::Identity, bAllowsInterpolation, bOverrideParent);
		PointData->Metadata->CreateFloatAttribute(InvalidAttribute, 0.0f, bAllowsInterpolation, bOverrideParent);

		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();

		// Randomize attribute values, and leave the second half to default value
		FRandomStream RandomSource(TestData.Seed);
		for (int I = 0; I < PointData->GetNumPoints() / 2; ++I)
		{
			UPCGMetadataAccessorHelpers::SetTransformAttributeByMetadataKey(MetadataEntryRange[I], PointData->Metadata, TransformAttribute, RandomTransform(RandomSource));
			UPCGMetadataAccessorHelpers::SetFloatAttributeByMetadataKey(MetadataEntryRange[I], PointData->Metadata, InvalidAttribute, RandomSource.FRand());
		}
	}

	void GenerateParamData(PCGTestsCommon::FTestData& TestData)
	{
		FPCGTaggedData& ParamTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		TObjectPtr<UPCGParamData> ParamData = PCGTestsCommon::CreateEmptyParamData();
		ParamTaggedData.Data = ParamData;
		ParamTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();
		const bool bAllowsInterpolation = false;
		const bool bOverrideParent = false;

		ParamData->Metadata->CreateTransformAttribute(TransformAttribute, FTransform::Identity, bAllowsInterpolation, bOverrideParent);
		ParamData->Metadata->CreateFloatAttribute(InvalidAttribute, 0.0f, bAllowsInterpolation, bOverrideParent);

		// Randomize attribute values
		FRandomStream RandomSource(TestData.Seed);
		if (FPCGMetadataAttribute<FTransform>* Attribute = static_cast<FPCGMetadataAttribute<FTransform>*>(ParamData->Metadata->GetMutableAttribute(TransformAttribute)))
		{
			Attribute->SetValue(EntryKey, RandomTransform(RandomSource));
		}

		if (FPCGMetadataAttribute<float>* Attribute = static_cast<FPCGMetadataAttribute<float>*>(ParamData->Metadata->GetMutableAttribute(InvalidAttribute)))
		{
			Attribute->SetValue(EntryKey, RandomSource.FRand());
		}
	}
}


bool FPCGMetadataBreakTransformTest::RunTest(const FString& Parameters)
{	
	auto ValidateMetadataBreakVector = [this](PCGTestsCommon::FTestData& TestData, bool bIsValid = true) -> bool
	{
		const FName OutputAttributeName = TEXT("Output");

		UPCGMetadataBreakTransformSettings* Settings = CastChecked<UPCGMetadataBreakTransformSettings>(TestData.Settings);
		Settings->OutputTarget.SetAttributeName(OutputAttributeName);
		FPCGElementPtr MetadataBreakTransformElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!MetadataBreakTransformElement->Execute(Context.Get()))
		{}

		// If the test is not valid, just check that we have no output and early out
		if (!bIsValid)
		{
			return TestTrue(TEXT("Invalid operation, no outputs"), Context->OutputData.GetAllInputs().IsEmpty());
		}

		const TArray<FPCGTaggedData>& Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

		if (!TestEqual("Right number of inputs", Inputs.Num(), 1))
		{
			return false;
		}

		bool bTestPassed = true;

		const FPCGTaggedData& Input = Inputs[0];

		TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, Settings->InputSource);
		TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, Settings->InputSource);

		if (!TestTrue("InputSource was found", InputAccessor.IsValid() && InputKeys.IsValid()))
		{
			return false;
		}

		auto ValidateComponentOutput = [this, &bTestPassed, &InputKeys, &InputAccessor](const FPCGTaggedData& Output, const FPCGAttributePropertySelector& OutSelector, PCGBreakTransformTest::EPCGComponentToCheck ComponentToCheck, auto DummyValue)
		{
			using OutType = decltype(DummyValue);

			if (!TestTrue("Valid output data", Output.Data != nullptr))
			{
				bTestPassed = false;
				return;
			}

			TUniquePtr<const IPCGAttributeAccessor> OutputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Output.Data, OutSelector);
			TUniquePtr<const IPCGAttributeAccessorKeys> OutputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(Output.Data, OutSelector);

			if (!TestTrue("OutputTarget was found", OutputAccessor.IsValid() && OutputKeys.IsValid()))
			{
				bTestPassed = false;
				return;
			}

			if (!TestEqual("Output has a valid type", OutputAccessor->GetUnderlyingType(), PCG::Private::MetadataTypes<OutType>::Id))
			{
				bTestPassed = false;
				return;
			}

			if (!TestEqual("Identical EntryKeys count", InputKeys->GetNum(), OutputKeys->GetNum()))
			{
				bTestPassed = false;
				return;
			}

			for (int32 i = 0; i < OutputKeys->GetNum(); ++i)
			{
				FTransform SourceValue{};
				if (!InputAccessor->Get<FTransform>(SourceValue, i, *InputKeys))
				{
					bTestPassed = false;
					return;
				}

				OutType OutValue{};
				if (!OutputAccessor->Get<OutType>(OutValue, i, *OutputKeys))
				{
					bTestPassed = false;
					return;
				}

				if constexpr (std::is_same_v<FVector, OutType>)
				{
					if (ComponentToCheck == PCGBreakTransformTest::EPCGComponentToCheck::Translation)
					{
						bTestPassed &= TestEqual("Translation", SourceValue.GetTranslation(), OutValue);
					}
					else if (ComponentToCheck == PCGBreakTransformTest::EPCGComponentToCheck::Scale)
					{
						bTestPassed &= TestEqual("Scale", SourceValue.GetScale3D(), OutValue);
					}
					else
					{
						bTestPassed = false;
					}
				}
				else
				{
					if (ComponentToCheck == PCGBreakTransformTest::EPCGComponentToCheck::Rotation)
					{
						bTestPassed &= TestEqual("Rotation", SourceValue.GetRotation(), OutValue);
					}
					else
					{
						bTestPassed = false;
					}
				}
			}
		};

		const TArray<FPCGTaggedData> OutputsTranslation = Context->OutputData.GetInputsByPin(PCGMetadataTransformConstants::Translation);
		const TArray<FPCGTaggedData> OutputsRotation = Context->OutputData.GetInputsByPin(PCGMetadataTransformConstants::Rotation);
		const TArray<FPCGTaggedData> OutputsScale = Context->OutputData.GetInputsByPin(PCGMetadataTransformConstants::Scale);

		if (TestEqual("Appropriate number of outputs generated for Translation", OutputsTranslation.Num(), 1))
		{
			ValidateComponentOutput(OutputsTranslation[0], Settings->OutputTarget, PCGBreakTransformTest::EPCGComponentToCheck::Translation, FVector{});
		}
		else
		{
			bTestPassed = false;
		}

		if (TestEqual("Appropriate number of outputs generated for Rotation", OutputsRotation.Num(), 1))
		{
			ValidateComponentOutput(OutputsRotation[0], Settings->OutputTarget, PCGBreakTransformTest::EPCGComponentToCheck::Rotation, FQuat{});
		}
		else
		{
			bTestPassed = false;
		}

		if (TestEqual("Appropriate number of outputs generated for Scale", OutputsScale.Num(), 1))
		{
			ValidateComponentOutput(OutputsScale[0], Settings->OutputTarget, PCGBreakTransformTest::EPCGComponentToCheck::Scale, FVector{});
		}
		else
		{
			bTestPassed = false;
		}

		return bTestPassed;
	};

	bool bTestPassed = true;

	PCGTestsCommon::FTestData TestDataSpatial;
	PCGTestsCommon::GenerateSettings<UPCGMetadataBreakTransformSettings>(TestDataSpatial);
	PCGBreakTransformTest::GenerateSpatialData(TestDataSpatial);

	PCGTestsCommon::FTestData TestDataParams;
	PCGTestsCommon::GenerateSettings<UPCGMetadataBreakTransformSettings>(TestDataParams);
	PCGBreakTransformTest::GenerateParamData(TestDataParams);

	using PairWhatData = TPair<FString, PCGTestsCommon::FTestData*>;

	TStaticArray<PairWhatData, 2> AllTestData;
	AllTestData[0] = PairWhatData("Testing with point data as input", &TestDataSpatial);
	AllTestData[1] = PairWhatData("Testing with param data as input", &TestDataParams);

	// Setup error catching. This error should happen only twice (invalid type)
	AddExpectedError(TEXT("Attribute/Property 'Float' from pin"), EAutomationExpectedErrorFlags::Contains, 2);

	for (PairWhatData& PairTestData : AllTestData)
	{
		AddInfo(PairTestData.Key);
		PCGTestsCommon::FTestData* TestData = PairTestData.Value;

		UPCGMetadataBreakTransformSettings* Settings = CastChecked<UPCGMetadataBreakTransformSettings>(TestData->Settings);

		Settings->ForceOutputConnections[0] = true;
		Settings->ForceOutputConnections[1] = true;
		Settings->ForceOutputConnections[2] = true;

		{
			AddInfo(TEXT("Test with Transform as input attribute"));
			Settings->InputSource.SetAttributeName(PCGBreakTransformTest::TransformAttribute);
			bTestPassed &= ValidateMetadataBreakVector(*TestData);
		}

		{
			AddInfo(TEXT("Test with Float as input attribute (invalid type)"));
			Settings->InputSource.SetAttributeName(PCGBreakTransformTest::InvalidAttribute);

			bTestPassed &= ValidateMetadataBreakVector(*TestData, /*bIsValid=*/false);
		}
	}

	return bTestPassed;
}

#endif // WITH_EDITOR
