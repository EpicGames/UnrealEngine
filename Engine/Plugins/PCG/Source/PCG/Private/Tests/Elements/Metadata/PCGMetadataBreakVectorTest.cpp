// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "Metadata/PCGMetadataAccessor.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"

#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataBreakVector.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#if WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataBreakVectorTest, FPCGTestBaseClass, "Plugins.PCG.Metadata.BreakVector", PCGTestsCommon::TestFlags)

namespace PCGBreakVectorTest
{
	enum class EPCGComponentToCheck : uint8
	{
		X,
		Z,
		Y,
		W
	};

	const FName Vec2Attribute = TEXT("Vec2");
	const FName Vec3Attribute = TEXT("Vec3");
	const FName Vec4Attribute = TEXT("Vec4");
	const FName RotatorAttribute = TEXT("Rotator");
	const FName FloatAttribute = TEXT("Float");

	void GenerateSpatialData(PCGTestsCommon::FTestData& TestData)
	{
		FPCGTaggedData& SpatialTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateRandomBasePointData(10, TestData.Seed);
		SpatialTaggedData.Data = PointData;
		SpatialTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		const bool bAllowsInterpolation = false;
		const bool bOverrideParent = false;

		PointData->Metadata->CreateVector2Attribute(Vec2Attribute, FVector2D::ZeroVector, bAllowsInterpolation, bOverrideParent);
		PointData->Metadata->CreateVectorAttribute(Vec3Attribute, FVector::Zero(), bAllowsInterpolation, bOverrideParent);
		PointData->Metadata->CreateVector4Attribute(Vec4Attribute, FVector4::Zero(), bAllowsInterpolation, bOverrideParent);
		PointData->Metadata->CreateRotatorAttribute(RotatorAttribute, FRotator::ZeroRotator, bAllowsInterpolation, bOverrideParent);
		PointData->Metadata->CreateFloatAttribute(FloatAttribute, 0, bAllowsInterpolation, bOverrideParent);

		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();

		// Randomize attribute values, and leave the second half to default value
		FRandomStream RandomSource(TestData.Seed);
		for (int I = 0; I < PointData->GetNumPoints() / 2; ++I)
		{
			UPCGMetadataAccessorHelpers::SetVector2AttributeByMetadataKey(MetadataEntryRange[I], PointData->Metadata, Vec2Attribute, FVector2D(RandomSource.VRand()));
			UPCGMetadataAccessorHelpers::SetVectorAttributeByMetadataKey(MetadataEntryRange[I], PointData->Metadata, Vec3Attribute, RandomSource.VRand());
			UPCGMetadataAccessorHelpers::SetVector4AttributeByMetadataKey(MetadataEntryRange[I], PointData->Metadata, Vec4Attribute, FVector4(RandomSource.VRand(), RandomSource.FRand()));
			UPCGMetadataAccessorHelpers::SetRotatorAttributeByMetadataKey(MetadataEntryRange[I], PointData->Metadata, RotatorAttribute, FRotator(RandomSource.FRand(), RandomSource.FRand(), RandomSource.FRand()));
			UPCGMetadataAccessorHelpers::SetFloatAttributeByMetadataKey(MetadataEntryRange[I], PointData->Metadata, FloatAttribute, RandomSource.FRand());
		}
	}

	void GenerateParamData(PCGTestsCommon::FTestData& TestData)
	{
		FPCGTaggedData& ParamTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		TObjectPtr<UPCGParamData> ParamData = PCGTestsCommon::CreateEmptyParamData();
		ParamTaggedData.Data = ParamData;
		ParamTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();
		const bool bAllowsInterpolation = false;
		const bool bOverrideParent = false;

		ParamData->Metadata->CreateVector2Attribute(Vec2Attribute, FVector2D::ZeroVector, bAllowsInterpolation, bOverrideParent);
		ParamData->Metadata->CreateVectorAttribute(Vec3Attribute, FVector::Zero(), bAllowsInterpolation, bOverrideParent);
		ParamData->Metadata->CreateVector4Attribute(Vec4Attribute, FVector4::Zero(), bAllowsInterpolation, bOverrideParent);
		ParamData->Metadata->CreateRotatorAttribute(RotatorAttribute, FRotator::ZeroRotator, bAllowsInterpolation, bOverrideParent);
		ParamData->Metadata->CreateFloatAttribute(FloatAttribute, 0, bAllowsInterpolation, bOverrideParent);

		// Randomize attribute values
		FRandomStream RandomSource(TestData.Seed);
		if (FPCGMetadataAttribute<FVector2D>* Attribute = static_cast<FPCGMetadataAttribute<FVector2D>*>(ParamData->Metadata->GetMutableAttribute(Vec2Attribute)))
		{
			Attribute->SetValue(EntryKey, FVector2D(RandomSource.VRand()));
		}

		if (FPCGMetadataAttribute<FVector>* Attribute = static_cast<FPCGMetadataAttribute<FVector>*>(ParamData->Metadata->GetMutableAttribute(Vec3Attribute)))
		{
			Attribute->SetValue(EntryKey, RandomSource.VRand());
		}

		if (FPCGMetadataAttribute<FVector4>* Attribute = static_cast<FPCGMetadataAttribute<FVector4>*>(ParamData->Metadata->GetMutableAttribute(Vec4Attribute)))
		{
			Attribute->SetValue(EntryKey, FVector4(RandomSource.VRand(), RandomSource.FRand()));
		}

		if (FPCGMetadataAttribute<FRotator>* Attribute = static_cast<FPCGMetadataAttribute<FRotator>*>(ParamData->Metadata->GetMutableAttribute(RotatorAttribute)))
		{
			Attribute->SetValue(EntryKey, FRotator(RandomSource.FRand(), RandomSource.FRand(), RandomSource.FRand()));
		}

		if (FPCGMetadataAttribute<float>* Attribute = static_cast<FPCGMetadataAttribute<float>*>(ParamData->Metadata->GetMutableAttribute(FloatAttribute)))
		{
			Attribute->SetValue(EntryKey, RandomSource.FRand());
		}
	}
}


bool FPCGMetadataBreakVectorTest::RunTest(const FString& Parameters)
{	
	auto ValidateMetadataBreakVector = [this](PCGTestsCommon::FTestData& TestData, bool bIsValid = true) -> bool
	{
		const FName OutputAttributeName = TEXT("Output");

		UPCGMetadataBreakVectorSettings* Settings = CastChecked<UPCGMetadataBreakVectorSettings>(TestData.Settings);
		Settings->OutputTarget.SetAttributeName(OutputAttributeName);
		FPCGElementPtr MetadataBreakVectorElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!MetadataBreakVectorElement->Execute(Context.Get()))
		{}

		// If the test is not valid, just check that we have no output and early out
		if (!bIsValid)
		{
			return TestTrue(TEXT("Invalid operation, no outputs"), Context->OutputData.GetAllInputs().IsEmpty());
		}

		const TArray<FPCGTaggedData>& Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

		if (!TestEqual("Right number of inputs", Inputs.Num(), 1))
		{
			return false;
		}

		bool bTestPassed = true;

		const FPCGTaggedData& Input = Inputs[0];

		TUniquePtr<const IPCGAttributeAccessor> InputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, Settings->InputSource);
		TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, Settings->InputSource);

		if (!TestTrue("InputSource was found", InputAccessor.IsValid() && InputKeys.IsValid()))
		{
			return false;
		}

		auto ValidateComponentOutput = [this, &bTestPassed, &InputKeys, &InputAccessor](const FPCGTaggedData& Output, const FPCGAttributePropertySelector& OutSelector, PCGBreakVectorTest::EPCGComponentToCheck ComponentToCheck)
		{
			if (!TestTrue("Valid output data", Output.Data != nullptr))
			{
				bTestPassed = false;
				return;
			}

			TUniquePtr<const IPCGAttributeAccessor> OutputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Output.Data, OutSelector);
			TUniquePtr<const IPCGAttributeAccessorKeys> OutputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(Output.Data, OutSelector);

			if (!TestTrue("OutputTarget was found", OutputAccessor.IsValid() && OutputKeys.IsValid()))
			{
				bTestPassed = false;
				return;
			}

			if (!TestEqual("Output has a valid type (double)", OutputAccessor->GetUnderlyingType(), PCG::Private::MetadataTypes<double>::Id))
			{
				bTestPassed = false;
				return;
			}

			if (!TestEqual("Identical EntryKeys count", InputKeys->GetNum(), OutputKeys->GetNum()))
			{
				bTestPassed = false;
				return;
			}

			for (int32 i = 0; i < OutputKeys->GetNum(); ++i)
			{
				FVector4 SourceValue{};
				if (InputAccessor->GetUnderlyingType() == PCG::Private::MetadataTypes<FVector>::Id)
				{
					FVector TempValue{};
					if (!InputAccessor->Get<FVector>(TempValue, i, *InputKeys))
					{
						bTestPassed = false;
						return;
					}

					SourceValue = FVector4(TempValue);
				}
				else if (InputAccessor->GetUnderlyingType() == PCG::Private::MetadataTypes<FVector2D>::Id)
				{
					FVector2D TempValue{};
					if (!InputAccessor->Get<FVector2D>(TempValue, i, *InputKeys))
					{
						bTestPassed = false;
						return;
					}

					SourceValue = FVector4(TempValue.X, TempValue.Y);
				}
				else if (InputAccessor->GetUnderlyingType() == PCG::Private::MetadataTypes<FVector4>::Id)
				{
					if (!InputAccessor->Get<FVector4>(SourceValue, i, *InputKeys))
					{
						bTestPassed = false;
						return;
					}
				}
				else //if (InputAccessor->GetUnderlyingType() == PCG::Private::MetadataTypes<FRotator>::Id)
				{
					FRotator TempValue{};
					if (!InputAccessor->Get<FRotator>(TempValue, i, *InputKeys))
					{
						bTestPassed = false;
						return;
					}

					SourceValue.X = TempValue.Roll;
					SourceValue.Y = TempValue.Pitch;
					SourceValue.Z = TempValue.Yaw;
				}

				double OutValue{};
				if (!OutputAccessor->Get<double>(OutValue, i, *OutputKeys))
				{
					bTestPassed = false;
					return;
				}

				if (ComponentToCheck == PCGBreakVectorTest::EPCGComponentToCheck::X)
				{
					bTestPassed &= TestEqual("X", SourceValue.X, OutValue);
				}
				else if (ComponentToCheck == PCGBreakVectorTest::EPCGComponentToCheck::Y)
				{
					bTestPassed &= TestEqual("Y", SourceValue.Y, OutValue);
				}
				else if (ComponentToCheck == PCGBreakVectorTest::EPCGComponentToCheck::Z)
				{
					bTestPassed &= TestEqual("Z", SourceValue.Z, OutValue);
				}
				else if (ComponentToCheck == PCGBreakVectorTest::EPCGComponentToCheck::W)
				{
					bTestPassed &= TestEqual("W", SourceValue.W, OutValue);
				}
			}
		};

		const TArray<FPCGTaggedData> OutputsX = Context->OutputData.GetInputsByPin(PCGMetadataBreakVectorConstants::XLabel);
		const TArray<FPCGTaggedData> OutputsY = Context->OutputData.GetInputsByPin(PCGMetadataBreakVectorConstants::YLabel);
		const TArray<FPCGTaggedData> OutputsZ = Context->OutputData.GetInputsByPin(PCGMetadataBreakVectorConstants::ZLabel);
		const TArray<FPCGTaggedData> OutputsW = Context->OutputData.GetInputsByPin(PCGMetadataBreakVectorConstants::WLabel);

		if (Settings->ForceOutputConnections[0])
		{
			if (TestEqual("Appropriate number of outputs generated fox X", OutputsX.Num(), 1))
			{
				ValidateComponentOutput(OutputsX[0], Settings->OutputTarget, PCGBreakVectorTest::EPCGComponentToCheck::X);
			}
			else
			{
				bTestPassed = false;
			}
		}

		if (Settings->ForceOutputConnections[1])
		{
			if (TestEqual("Appropriate number of outputs generated for Y", OutputsY.Num(), 1))
			{
				ValidateComponentOutput(OutputsY[0], Settings->OutputTarget, PCGBreakVectorTest::EPCGComponentToCheck::Y);
			}
			else
			{
				bTestPassed = false;
			}
		}

		if (Settings->ForceOutputConnections[2])
		{
			if (TestEqual("Appropriate number of outputs generated for Z", OutputsZ.Num(), 1))
			{
				ValidateComponentOutput(OutputsZ[0], Settings->OutputTarget, PCGBreakVectorTest::EPCGComponentToCheck::Z);
			}
			else
			{
				bTestPassed = false;
			}
		}

		if (Settings->ForceOutputConnections[3])
		{
			if (TestEqual("Appropriate number of outputs generated for W", OutputsW.Num(), 1))
			{
				ValidateComponentOutput(OutputsW[0], Settings->OutputTarget, PCGBreakVectorTest::EPCGComponentToCheck::W);
			}
			else
			{
				bTestPassed = false;
			}
		}

		return bTestPassed;
	};

	bool bTestPassed = true;

	PCGTestsCommon::FTestData TestDataSpatial;
	PCGTestsCommon::GenerateSettings<UPCGMetadataBreakVectorSettings>(TestDataSpatial);
	PCGBreakVectorTest::GenerateSpatialData(TestDataSpatial);

	PCGTestsCommon::FTestData TestDataParams;
	PCGTestsCommon::GenerateSettings<UPCGMetadataBreakVectorSettings>(TestDataParams);
	PCGBreakVectorTest::GenerateParamData(TestDataParams);

	using PairWhatData = TPair<FString, PCGTestsCommon::FTestData*>;

	TStaticArray<PairWhatData, 2> AllTestData;
	AllTestData[0] = PairWhatData("Testing with point data as input", &TestDataSpatial);
	AllTestData[1] = PairWhatData("Testing with param data as input", &TestDataParams);

	// Setup error catching. This error should happen only twice (invalid type)
	AddExpectedError(TEXT("Attribute/Property 'Float' from pin"), EAutomationExpectedErrorFlags::Contains, 2);

	for (PairWhatData& PairTestData : AllTestData)
	{
		AddInfo(PairTestData.Key);
		PCGTestsCommon::FTestData* TestData = PairTestData.Value;

		UPCGMetadataBreakVectorSettings* Settings = CastChecked<UPCGMetadataBreakVectorSettings>(TestData->Settings);

		Settings->ForceOutputConnections[0] = true;
		Settings->ForceOutputConnections[1] = true;
		Settings->ForceOutputConnections[2] = false;
		Settings->ForceOutputConnections[3] = false;

		{
			AddInfo(TEXT("Test with FVector2D as input attribute"));
			Settings->InputSource.SetAttributeName(PCGBreakVectorTest::Vec2Attribute);
			bTestPassed &= ValidateMetadataBreakVector(*TestData);
		}

		Settings->ForceOutputConnections[2] = true;

		{
			AddInfo(TEXT("Test with FVector as input attribute"));
			Settings->InputSource.SetAttributeName(PCGBreakVectorTest::Vec3Attribute);
			bTestPassed &= ValidateMetadataBreakVector(*TestData);
		}

		{
			AddInfo(TEXT("Test with FVector4 as input attribute"));
			Settings->InputSource.SetAttributeName(PCGBreakVectorTest::Vec4Attribute);
			Settings->ForceOutputConnections[3] = true;
			bTestPassed &= ValidateMetadataBreakVector(*TestData);
			Settings->ForceOutputConnections[3] = false;
		}

		{
			AddInfo(TEXT("Test with FRotator as input attribute"));
			Settings->InputSource.SetAttributeName(PCGBreakVectorTest::RotatorAttribute);
			bTestPassed &= ValidateMetadataBreakVector(*TestData);
		}

		{
			AddInfo(TEXT("Test with Float as input attribute (invalid type)"));
			Settings->InputSource.SetAttributeName(PCGBreakVectorTest::FloatAttribute);

			bTestPassed &= ValidateMetadataBreakVector(*TestData, /*bIsValid=*/false);
		}
	}

	return bTestPassed;
}

#endif // WITH_EDITOR
