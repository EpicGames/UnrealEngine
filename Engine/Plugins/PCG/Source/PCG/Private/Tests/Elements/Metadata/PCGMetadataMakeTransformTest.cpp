// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "Metadata/PCGMetadataAccessor.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"

#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataMakeTransform.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#if WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMakeTransformTest, FPCGTestBaseClass, "Plugins.PCG.Metadata.MakeTransform", PCGTestsCommon::TestFlags)

namespace PCGMetadataMakeTransformTest
{
	const FName Vec3Attribute = TEXT("Vec3");
	const FName QuatAttribute = TEXT("Quat");

	void ConnectData(FPCGDataCollection& InputData, TObjectPtr<UPCGData> Data)
	{
		FPCGTaggedData& TaggedData1 = InputData.TaggedData.Emplace_GetRef();
		TaggedData1.Data = Data;
		TaggedData1.Pin = PCGMetadataTransformConstants::Translation;

		FPCGTaggedData& TaggedData2 = InputData.TaggedData.Emplace_GetRef();
		TaggedData2.Data = Data;
		TaggedData2.Pin = PCGMetadataTransformConstants::Rotation;

		FPCGTaggedData& TaggedData3 = InputData.TaggedData.Emplace_GetRef();
		TaggedData3.Data = Data;
		TaggedData3.Pin = PCGMetadataTransformConstants::Scale;
	}

	void GenerateSpatialData(PCGTestsCommon::FTestData& TestData)
	{
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateRandomBasePointData(10, TestData.Seed);
		
		ConnectData(TestData.InputData, PointData);

		const bool bAllowsInterpolation = false;
		const bool bOverrideParent = false;

		PointData->Metadata->CreateVectorAttribute(Vec3Attribute, FVector::ZeroVector, bAllowsInterpolation, bOverrideParent);
		PointData->Metadata->CreateQuatAttribute(QuatAttribute, FQuat::Identity, bAllowsInterpolation, bOverrideParent);

		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();

		// Randomize attribute values, and leave the second half to default value
		FRandomStream RandomSource(TestData.Seed);
		for (int I = 0; I < PointData->GetNumPoints() / 2; ++I)
		{
			UPCGMetadataAccessorHelpers::SetVectorAttributeByMetadataKey(MetadataEntryRange[I], PointData->Metadata, Vec3Attribute, RandomSource.VRand());
			UPCGMetadataAccessorHelpers::SetQuatAttributeByMetadataKey(MetadataEntryRange[I], PointData->Metadata, QuatAttribute, FRotator(RandomSource.FRand(), RandomSource.FRand(), RandomSource.FRand()).Quaternion());
		}
	}

	void GenerateParamData(PCGTestsCommon::FTestData& TestData)
	{
		TObjectPtr<UPCGParamData> ParamData = PCGTestsCommon::CreateEmptyParamData();

		ConnectData(TestData.InputData, ParamData);

		PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();
		const bool bAllowsInterpolation = false;
		const bool bOverrideParent = false;

		ParamData->Metadata->CreateVectorAttribute(Vec3Attribute, FVector::ZeroVector, bAllowsInterpolation, bOverrideParent);
		ParamData->Metadata->CreateQuatAttribute(QuatAttribute, FQuat::Identity, bAllowsInterpolation, bOverrideParent);

		// Randomize attribute values
		FRandomStream RandomSource(TestData.Seed);
		if (FPCGMetadataAttribute<FVector>* Attribute = static_cast<FPCGMetadataAttribute<FVector>*>(ParamData->Metadata->GetMutableAttribute(Vec3Attribute)))
		{
			Attribute->SetValue(EntryKey, RandomSource.VRand());
		}

		if (FPCGMetadataAttribute<FQuat>* Attribute = static_cast<FPCGMetadataAttribute<FQuat>*>(ParamData->Metadata->GetMutableAttribute(QuatAttribute)))
		{
			Attribute->SetValue(EntryKey, FRotator(RandomSource.FRand(), RandomSource.FRand(), RandomSource.FRand()).Quaternion());
		}
	}
}


bool FPCGMetadataMakeTransformTest::RunTest(const FString& Parameters)
{	
	auto ValidateMetadataMakeVector = [this](PCGTestsCommon::FTestData& TestData) -> bool
	{
		const FName OutputAttributeName = TEXT("Output");

		UPCGMetadataMakeTransformSettings* Settings = CastChecked<UPCGMetadataMakeTransformSettings>(TestData.Settings);
		Settings->OutputTarget.SetAttributeName(OutputAttributeName);
		FPCGElementPtr MetadataBreakTransformElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!MetadataBreakTransformElement->Execute(Context.Get()))
		{}

		// We have the same metadata to all pins, so just check the first one
		const TArray<FPCGTaggedData>& TranslationInputs = Context->InputData.GetInputsByPin(PCGMetadataTransformConstants::Translation);

		if (!TestEqual("Right number of translation inputs", TranslationInputs.Num(), 1))
		{
			return false;
		}

		const FPCGTaggedData& Input = TranslationInputs[0];

		TUniquePtr<const IPCGAttributeAccessor> InputAccessor1 = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, Settings->InputSource1);
		TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys1 = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, Settings->InputSource1);

		TUniquePtr<const IPCGAttributeAccessor> InputAccessor2 = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, Settings->InputSource2);
		TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys2 = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, Settings->InputSource2);

		TUniquePtr<const IPCGAttributeAccessor> InputAccessor3 = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, Settings->InputSource3);
		TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys3 = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, Settings->InputSource3);

		if (!TestTrue("InputSource1 was found", InputAccessor1.IsValid() && InputKeys1.IsValid()))
		{
			return false;
		}

		if (!TestTrue("InputSource2 was found", InputAccessor2.IsValid() && InputKeys2.IsValid()))
		{
			return false;
		}

		if (!TestTrue("InputSource3 was found", InputAccessor3.IsValid() && InputKeys3.IsValid()))
		{
			return false;
		}

		if (!TestEqual("Same number of input keys between source 1 and 2", InputKeys1->GetNum(), InputKeys2->GetNum()))
		{
			return false;
		}

		if (!TestEqual("Same number of input keys between source 1 and 3", InputKeys1->GetNum(), InputKeys3->GetNum()))
		{
			return false;
		}

		const TArray<FPCGTaggedData> Outputs = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
		if (!TestEqual("Right number of outputs", Outputs.Num(), 1))
		{
			return false;
		}

		const FPCGTaggedData& Output = Outputs[0];

		if (!TestTrue("Valid output data", Output.Data != nullptr))
		{
			return false;
		}

		TUniquePtr<const IPCGAttributeAccessor> OutputAccessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Output.Data, Settings->OutputTarget);
		TUniquePtr<const IPCGAttributeAccessorKeys> OutputKeys = PCGAttributeAccessorHelpers::CreateConstKeys(Output.Data, Settings->OutputTarget);

		if (!TestTrue("OutputTarget was found", OutputAccessor.IsValid() && OutputKeys.IsValid()))
		{
			return false;
		}

		if (!TestEqual("Output has a valid type (Transform)", OutputAccessor->GetUnderlyingType(), PCG::Private::MetadataTypes<FTransform>::Id))
		{
			return false;
		}

		if (!TestEqual("Identical EntryKeys count", InputKeys1->GetNum(), OutputKeys->GetNum()))
		{
			return false;
		}

		for (int32 i = 0; i < OutputKeys->GetNum(); ++i)
		{
			FVector TranslationValue{};
			FQuat QuatValue{};
			FVector ScaleValue{};
			FTransform OutValue{};

			if (!InputAccessor1->Get<FVector>(TranslationValue, i, *InputKeys1)
				|| !InputAccessor2->Get<FQuat>(QuatValue, i, *InputKeys2)
				|| !InputAccessor3->Get<FVector>(ScaleValue, i, *InputKeys3)
				|| !OutputAccessor->Get<FTransform>(OutValue, i, *OutputKeys))
			{
				return false;
			}
			
			if (!TestEqual("Mismatch between input and output for translation", OutValue.GetTranslation(), TranslationValue))
			{
				return false;
			}

			if (!TestEqual("Mismatch between input and output for rotation", OutValue.GetRotation(), QuatValue))
			{
				return false;
			}

			if (!TestEqual("Mismatch between input and output for scale", OutValue.GetScale3D(), ScaleValue))
			{
				return false;
			}
		}

		return true;
	};

	bool bTestPassed = true;

	PCGTestsCommon::FTestData TestDataSpatial;
	PCGTestsCommon::GenerateSettings<UPCGMetadataMakeTransformSettings>(TestDataSpatial);
	PCGMetadataMakeTransformTest::GenerateSpatialData(TestDataSpatial);

	PCGTestsCommon::FTestData TestDataParams;
	PCGTestsCommon::GenerateSettings<UPCGMetadataMakeTransformSettings>(TestDataParams);
	PCGMetadataMakeTransformTest::GenerateParamData(TestDataParams);

	using PairWhatData = TPair<FString, PCGTestsCommon::FTestData*>;

	TStaticArray<PairWhatData, 2> AllTestData;
	AllTestData[0] = PairWhatData("Testing with point data as input", &TestDataSpatial);
	AllTestData[1] = PairWhatData("Testing with param data as input", &TestDataParams);

	for (PairWhatData& PairTestData : AllTestData)
	{
		AddInfo(PairTestData.Key);
		PCGTestsCommon::FTestData* TestData = PairTestData.Value;

		UPCGMetadataMakeTransformSettings* Settings = CastChecked<UPCGMetadataMakeTransformSettings>(TestData->Settings);

		Settings->ForceOutputConnections[0] = true;

		Settings->InputSource1.SetAttributeName(PCGMetadataMakeTransformTest::Vec3Attribute);
		Settings->InputSource2.SetAttributeName(PCGMetadataMakeTransformTest::QuatAttribute);
		Settings->InputSource3.SetAttributeName(PCGMetadataMakeTransformTest::Vec3Attribute);

		bTestPassed &= ValidateMetadataMakeVector(*TestData);
	}

	return bTestPassed;
}

#endif // WITH_EDITOR
