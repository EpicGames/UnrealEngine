// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGMetadataMathsOpElement.h"
#include "Helpers/PCGPropertyHelpers.h"

#if WITH_EDITOR

class FPCGMetadataMathsOpTest : public FPCGTestBaseClass
{
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	inline static const FName DefaultAttributeName = TEXT("TestAttribute");
	inline static const FVector DefaultTranslation = {1.0, 10.0, 100.0};
	inline static const FPCGAttributePropertyInputSelector DefaultSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(DefaultAttributeName);

	template <typename T>
	struct TestParams
	{
		explicit TestParams(const EPCGMetadataMathsOperation Operation, TArray<TArray<FPCGTaggedData>>&& InputDataByPin, TArray<T>&& ExpectedDefaultValues, TArray<T>&& ExpectedResultValues, const int32 PinToForward = 0) :
			Operation(Operation),
			InputDataByPin(std::move(InputDataByPin)),
			ExpectedDefaultValues(ExpectedDefaultValues),
			ExpectedResultValues(ExpectedResultValues),
			PinToForward(PinToForward)
		{
		}

		EPCGMetadataMathsOperation Operation;
		TArray<TArray<FPCGTaggedData>> InputDataByPin;
		TArray<T> ExpectedDefaultValues;
		TArray<T> ExpectedResultValues;
		int32 PinToForward = 0;
	};

	// Expects Input Data to be an array of Tagged Data on each index per pin and a single value in each Input Data (1 attribute or 1 point data)
	template <typename T>
	bool ExecuteTest(TestParams<T>&& Params, const FPCGAttributePropertyInputSelector InputSelector = DefaultSelector)
	{
		const EPCGMetadataMathsOperation& Operation = Params.Operation;
		const TArray<TArray<FPCGTaggedData>>& InputDataByPin = Params.InputDataByPin;
		const TArray<T>& ExpectedDefaults = Params.ExpectedDefaultValues;
		const TArray<T>& ExpectedValues = Params.ExpectedResultValues;
		const int32 PinToForward = Params.PinToForward;

		check(!InputDataByPin.IsEmpty() && !ExpectedDefaults.IsEmpty() && !ExpectedValues.IsEmpty());

		const UEnum* EnumClass = StaticEnum<EPCGMetadataMathsOperation>();
		auto FormatText = [Operation, PinToForward, EnumClass](const FString& Text) -> FString
		{
			return FString::Printf(TEXT("[Pin %d] %s (%s): %s."), PinToForward, *EnumClass->GetDisplayNameTextByValue(static_cast<int64>(Operation)).ToString(), *PCG::Private::GetTypeName<T>(), *Text);
		};

		bool bSuccess = true;

		PCGTestsCommon::FTestData TestData;
		UPCGMetadataMathsSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGMetadataMathsSettings>(TestData);
		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		// Assign the element settings here. Use the same selector for all input sources, since the user can only select one
		Settings->Operation = Operation;
		Settings->InputSource1 = InputSelector;
		Settings->InputSource2 = InputSelector;
		Settings->InputSource3 = InputSelector;
		Settings->OutputTarget.SetAttributeName(DefaultAttributeName);
		Settings->OutputDataFromPin = Settings->GetInputPinLabel(PinToForward);

		// Assign pin labels based on incoming data for the execution
		for (int PinIndex = 0; PinIndex < InputDataByPin.Num(); ++PinIndex)
		{
			TArray<FPCGTaggedData> InputData = InputDataByPin[PinIndex];
			for (FPCGTaggedData& Data : InputData)
			{
				Data.Pin = Settings->GetInputPinLabel(PinIndex);
			}

			TestData.InputData.TaggedData.Append(std::move(InputData));
		}

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get()))
		{
		}

		UTEST_TRUE(*FormatText("Output created"), !Context->OutputData.TaggedData.IsEmpty());

		UTEST_EQUAL(*FormatText("Correct number of outputs"), Context->OutputData.TaggedData.Num(), ExpectedValues.Num());

		// Cycle through expected results and validate
		for (int I = 0; I < ExpectedValues.Num(); ++I)
		{
			const UPCGData* OutputData = Context->OutputData.TaggedData[I].Data;
			UTEST_TRUE("Valid output data", OutputData != nullptr);

			FPCGAttributePropertySelector Selector = FPCGAttributePropertySelector::CreateAttributeSelector(DefaultAttributeName);
			const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(OutputData, Selector);
			const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(OutputData, Selector);

			UTEST_TRUE(*FormatText("Target attribute or property was found"), Accessor.IsValid() && Keys.IsValid());

			UTEST_EQUAL(*FormatText("Output has the correct type"), Accessor->GetUnderlyingType(), PCG::Private::MetadataTypes<T>::Id);

			T DefaultValue{};
			T OutputValue{};

			FPCGAttributeAccessorKeysEntries DefaultKey(PCGInvalidEntryKey);
			bSuccess &= TestTrue(*FormatText("Output default value attribute accessible"), Accessor->Get(DefaultValue, 0, DefaultKey));

			// We're only testing one attribute for each iteration
			bSuccess &= TestTrue(*FormatText("Output value attribute accessible"), Accessor->Get(OutputValue, 0, *Keys));

			// Vec2 and Vec4 don't have "nearly equal" in the test framework
			if constexpr (std::is_same_v<T, FVector2D> || std::is_same_v<T, FVector4>)
			{
				bSuccess &= TestTrue(*FormatText("Default value is correct"), DefaultValue.Equals(ExpectedDefaults[I]));
				bSuccess &= TestTrue(*FormatText("Result is correct"), OutputValue.Equals(ExpectedValues[I]));
			}
			else
			{
				bSuccess &= TestEqual(*FormatText("Default value is correct"), DefaultValue, ExpectedDefaults[I]);
				bSuccess &= TestEqual(*FormatText("Result is correct"), OutputValue, ExpectedValues[I]);
			}
		}

		return bSuccess;
	}

	// Generate points with a default location, density, and seed based on index
	TArray<TArray<FPCGTaggedData>> GeneratePointData(const int32 NumPins = 1, const int32 NumData = 1)
	{
		TArray<TArray<FPCGTaggedData>> GeneratedData;

		for (int PinIndex = 0; PinIndex < NumPins; ++PinIndex)
		{
			TArray<FPCGTaggedData>& TaggedData = GeneratedData.Emplace_GetRef();

			for (int32 InputIndex = 0; InputIndex < NumData; ++InputIndex)
			{
				int32 M = NumData * PinIndex + InputIndex + 1;
				TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateEmptyBasePointData();
				PointData->SetNumPoints(1);
				PointData->SetTransform(FTransform(M * DefaultTranslation));
				PointData->SetDensity(static_cast<double>(NumData * NumPins) / M);
				PointData->SetSeed(M);
				TaggedData.Emplace_GetRef().Data = PointData;
			}
		}

		return GeneratedData;
	}

	// Generate param data of the tested type
	template <typename T>
	TArray<TArray<FPCGTaggedData>> GenerateParamData(const TArray<TArray<T>> ValuesByPin)
	{
		TArray<TArray<FPCGTaggedData>> GeneratedData;

		for (TArray<T> Values : ValuesByPin)
		{
			TArray<FPCGTaggedData>& TaggedData = GeneratedData.Emplace_GetRef();

			for (const T& Value : Values)
			{
				TObjectPtr<UPCGParamData> ParamData = PCGTestsCommon::CreateEmptyParamData();
				TaggedData.Emplace_GetRef().Data = ParamData;

				PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();
				constexpr bool bAllowsInterpolation = true;
				constexpr bool bOverrideParent = false;

				FPCGMetadataAttribute<T>* Attribute = ParamData->Metadata->FindOrCreateAttribute<T>(DefaultAttributeName, PCG::Private::MetadataTraits<T>::ZeroValue(), bAllowsInterpolation, bOverrideParent);
				check(Attribute);

				Attribute->SetValue(EntryKey, Value);
			}
		}

		return GeneratedData;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_OneMinus, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.OneMinus", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_OneMinus::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::OneMinus;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(OneMinus, GenerateParamData<int32>({{5}}), TArray({1}), TArray({-4})));
	bSuccess &= ExecuteTest(TestParams(OneMinus, GenerateParamData<long long>({{6ll}}), TArray({1ll}), TArray({-5ll})));
	bSuccess &= ExecuteTest(TestParams(OneMinus, GenerateParamData<float>({{0.1f}}), TArray({1.f}), TArray({0.9f})));
	bSuccess &= ExecuteTest(TestParams(OneMinus, GenerateParamData<double>({{0.2}}), TArray({1.0}), TArray({0.8})));
	bSuccess &= ExecuteTest(TestParams(OneMinus, GenerateParamData<FVector2D>({{FVector2D(0.3, 1.4)}}), TArray({FVector2D::One()}), TArray({FVector2D(0.7, -0.4)})));
	bSuccess &= ExecuteTest(TestParams(OneMinus, GenerateParamData<FVector>({{FVector(0.5, 1.6, 0.7)}}), TArray({FVector::One()}), TArray({FVector(0.5, -0.6, 0.3)})));
	bSuccess &= ExecuteTest(TestParams(OneMinus, GenerateParamData<FVector4>({{FVector4(0.5, 1.6, 0.7, 0.8)}}), TArray({FVector4::One()}), TArray({FVector4(0.5, -0.6, 0.3, 0.2)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(OneMinus, GeneratePointData(), TArray<int32>({0}), TArray<int32>({0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(OneMinus, GeneratePointData(), TArray({0.0}), TArray({0.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(OneMinus, GeneratePointData(), TArray({FVector(0.0, 0.0, 0.0)}), TArray({FVector(0.0, -9.0, -99.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Sign, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Sign", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Sign::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Sign;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(Sign, GenerateParamData<int32>({{0}}), TArray({0}), TArray({0})));
	bSuccess &= ExecuteTest(TestParams(Sign, GenerateParamData<int32>({{5}}), TArray({0}), TArray({1})));
	bSuccess &= ExecuteTest(TestParams(Sign, GenerateParamData<int32>({{-5}}), TArray({0}), TArray({-1})));
	bSuccess &= ExecuteTest(TestParams(Sign, GenerateParamData<long long>({{6ll}}), TArray({0ll}), TArray({1ll})));
	bSuccess &= ExecuteTest(TestParams(Sign, GenerateParamData<float>({{-0.1f}}), TArray({0.f}), TArray({-1.f})));
	bSuccess &= ExecuteTest(TestParams(Sign, GenerateParamData<double>({{-4.2}}), TArray({0.0}), TArray({-1.0})));
	bSuccess &= ExecuteTest(TestParams(Sign, GenerateParamData<FVector2D>({{FVector2D(0.3, -1.4)}}), TArray({FVector2D::Zero()}), TArray({FVector2D(1.0, -1.0)})));
	bSuccess &= ExecuteTest(TestParams(Sign, GenerateParamData<FVector>({{FVector(0.5, -1.6, 0.0)}}), TArray({FVector::Zero()}), TArray({FVector(1.0, -1.0, 0.0)})));
	bSuccess &= ExecuteTest(TestParams(Sign, GenerateParamData<FVector4>({{FVector4(0.5, -1.6, 0.0, -0.0)}}), TArray({FVector4::Zero()}), TArray({FVector4(1.0, -1.0, 0.0, 0.0)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Sign, GeneratePointData(), TArray<int32>({0}), TArray<int32>({1})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Sign, GeneratePointData(), TArray({0.0}), TArray({1.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Sign, GeneratePointData(), TArray({FVector(0.0, 0.0, 0.0)}), TArray({FVector(1.0, 1.0, 1.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Frac, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Frac", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Frac::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Frac;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(Frac, GenerateParamData<int32>({{0}}), TArray({0}), TArray({0})));
	bSuccess &= ExecuteTest(TestParams(Frac, GenerateParamData<long long>({{6ll}}), TArray({0ll}), TArray({0ll})));
	bSuccess &= ExecuteTest(TestParams(Frac, GenerateParamData<float>({{-0.1f}}), TArray({0.f}), TArray({.9f})));
	bSuccess &= ExecuteTest(TestParams(Frac, GenerateParamData<double>({{-0.5}}), TArray({0.0}), TArray({0.5})));
	bSuccess &= ExecuteTest(TestParams(Frac, GenerateParamData<FVector2D>({{FVector2D(0.3, -1.4)}}), TArray({FVector2D::Zero()}), TArray({FVector2D(.3, .6)})));
	bSuccess &= ExecuteTest(TestParams(Frac, GenerateParamData<FVector>({{FVector(0.5, -1.6, 0.0)}}), TArray({FVector::Zero()}), TArray({FVector(.5, .4, .0)})));
	bSuccess &= ExecuteTest(TestParams(Frac, GenerateParamData<FVector4>({{FVector4(0.5, -1.6, 0.0, -0.0)}}), TArray({FVector4::Zero()}), TArray({FVector4(.5, .4, 0.0, 0.0)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Frac, GeneratePointData(), TArray<int32>({0}), TArray<int32>({0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Frac, GeneratePointData(), TArray({0.0}), TArray({0.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Frac, GeneratePointData(), TArray({FVector(0.0, 0.0, 0.0)}), TArray({FVector(0.0, 0.0, 0.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Truncate, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Truncate", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Truncate::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Truncate;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(Truncate, GenerateParamData<int32>({{0}}), TArray({0}), TArray({0})));
	bSuccess &= ExecuteTest(TestParams(Truncate, GenerateParamData<long long>({{6ll}}), TArray({0ll}), TArray({6ll})));
	bSuccess &= ExecuteTest(TestParams(Truncate, GenerateParamData<float>({{-0.1f}}), TArray({0.f}), TArray({0.0f})));
	bSuccess &= ExecuteTest(TestParams(Truncate, GenerateParamData<double>({{-0.3}}), TArray({0.0}), TArray({0.0})));
	bSuccess &= ExecuteTest(TestParams(Truncate, GenerateParamData<FVector2D>({{FVector2D(0.3, -1.4)}}), TArray({FVector2D::Zero()}), TArray({FVector2D(0.0, -1.0)})));
	bSuccess &= ExecuteTest(TestParams(Truncate, GenerateParamData<FVector>({{FVector(0.5, -1.6, 0.0)}}), TArray({FVector::Zero()}), TArray({FVector(0.0, -1.0, .0)})));
	bSuccess &= ExecuteTest(TestParams(Truncate, GenerateParamData<FVector4>({{FVector4(0.5, -1.6, 0.0, -0.0)}}), TArray({FVector4::Zero()}), TArray({FVector4(0.0, -1.0, 0.0, 0.0)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Truncate, GeneratePointData(), TArray<int32>({0}), TArray<int32>({1})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Truncate, GeneratePointData(), TArray({0.0}), TArray({1.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Truncate, GeneratePointData(), TArray({FVector(0.0, 0.0, 0.0)}), TArray({FVector(1.0, 10.0, 100.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Add, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Add", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Add::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Add;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<int32>({{5}, {1}}), TArray({0}), TArray({6})));
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<long long>({{6ll}, {-1ll}}), TArray({0ll}), TArray({5ll})));
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<float>({{0.1f}, {0.4f}}), TArray({0.f}), TArray({0.5f})));
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<double>({{0.0}, {-2.4}}), TArray({0.0}), TArray({-2.4})));
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<FVector2D>({{FVector2D(0.3, 1.4)}, {FVector2D(-1.2, 1.4)}}), TArray({FVector2D::Zero()}), TArray({FVector2D(-0.9, 2.8)})));
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<FVector>({{FVector(0.5, 1.6, 0.7)}, {FVector(-0.5, 4.2, -0.7)}}), TArray({FVector::Zero()}), TArray({FVector(0.0, 5.8, 0.0)})));
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<FVector4>({{FVector4(0.5, 1.6, 0.7, 0.8)}, {FVector4(-0.5, 4.2, -0.7, 0.3)}}), TArray({FVector4::Zero()}), TArray({FVector4(0.0, 5.8, 0.0, 1.1)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Add, GeneratePointData(2), TArray<int32>({0}), TArray<int32>({3})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Add, GeneratePointData(2), TArray({0.0}), TArray({3.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Add, GeneratePointData(2), TArray({FVector::Zero()}), TArray({FVector(3.0, 30.0, 300.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Subtract, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Subtract", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Subtract::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Subtract;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(Subtract, GenerateParamData<int32>({{5}, {1}}), TArray({0}), TArray({4})));
	bSuccess &= ExecuteTest(TestParams(Subtract, GenerateParamData<long long>({{6ll}, {-1ll}}), TArray({0ll}), TArray({7ll})));
	bSuccess &= ExecuteTest(TestParams(Subtract, GenerateParamData<float>({{0.1f}, {0.4f}}), TArray({0.f}), TArray({-.3f})));
	bSuccess &= ExecuteTest(TestParams(Subtract, GenerateParamData<double>({{0.0}, {-2.4}}), TArray({0.0}), TArray({2.4})));
	bSuccess &= ExecuteTest(TestParams(Subtract, GenerateParamData<FVector2D>({{FVector2D(0.3, 1.4)}, {FVector2D(-1.2, 1.4)}}), TArray({FVector2D::Zero()}), TArray({FVector2D(1.5, 0.0)})));
	bSuccess &= ExecuteTest(TestParams(Subtract, GenerateParamData<FVector>({{FVector(0.5, 1.6, 0.7)}, {FVector(-0.5, 4.2, -0.7)}}), TArray({FVector::Zero()}), TArray({FVector(1.0, -2.6, 1.4)})));
	bSuccess &= ExecuteTest(TestParams(Subtract, GenerateParamData<FVector4>({{FVector4(0.5, 1.6, 0.7, 0.8)}, {FVector4(-0.5, 4.2, -0.7, 0.3)}}), TArray({FVector4::Zero()}), TArray({FVector4(1.0, -2.6, 1.4, 0.5)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Subtract, GeneratePointData(2), TArray<int32>({0}), TArray<int32>({-1})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Subtract, GeneratePointData(2), TArray({0.0}), TArray({1.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Subtract, GeneratePointData(2), TArray({FVector::Zero()}), TArray({FVector(-1.0, -10.0, -100.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Multiply, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Multiply", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Multiply::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Multiply;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(Multiply, GenerateParamData<int32>({{5}, {1}}), TArray({0}), TArray({5})));
	bSuccess &= ExecuteTest(TestParams(Multiply, GenerateParamData<long long>({{6ll}, {-1ll}}), TArray({0ll}), TArray({-6ll})));
	bSuccess &= ExecuteTest(TestParams(Multiply, GenerateParamData<float>({{0.1f}, {0.4f}}), TArray({0.f}), TArray({0.04f})));
	bSuccess &= ExecuteTest(TestParams(Multiply, GenerateParamData<double>({{0.0}, {-2.4}}), TArray({0.0}), TArray({0.0})));
	bSuccess &= ExecuteTest(TestParams(Multiply, GenerateParamData<FVector2D>({{FVector2D(0.3, 1.4)}, {FVector2D(-1.2, 1.4)}}), TArray({FVector2D::Zero()}), TArray({FVector2D(-0.36, 1.96)})));
	bSuccess &= ExecuteTest(TestParams(Multiply, GenerateParamData<FVector>({{FVector(0.5, 1.6, 0.7)}, {FVector(-0.5, 4.2, -0.7)}}), TArray({FVector::Zero()}), TArray({FVector(-0.25, 6.72, -0.49)})));
	bSuccess &= ExecuteTest(TestParams(Multiply, GenerateParamData<FVector4>({{FVector4(0.5, 1.6, 0.7, 0.8)}, {FVector4(-0.5, 4.2, -0.7, 0.3)}}), TArray({FVector4::Zero()}), TArray({FVector4(-0.25, 6.72, -0.49, 0.24)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Multiply, GeneratePointData(2), TArray<int32>({0}), TArray<int32>({2})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Multiply, GeneratePointData(2), TArray({0.0}), TArray({2.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Multiply, GeneratePointData(2), TArray({FVector::Zero()}), TArray({FVector(2.0, 200.0, 20000.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Divide, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Divide", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Divide::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Divide;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(Divide, GenerateParamData<int32>({{32}, {8}}), TArray({0}), TArray({4})));
	bSuccess &= ExecuteTest(TestParams(Divide, GenerateParamData<long long>({{33ll}, {3ll}}), TArray({0ll}), TArray({11ll})));
	bSuccess &= ExecuteTest(TestParams(Divide, GenerateParamData<float>({{0.1f}, {0.4f}}), TArray({0.f}), TArray({0.25f})));
	bSuccess &= ExecuteTest(TestParams(Divide, GenerateParamData<double>({{0.0}, {-2.4}}), TArray({0.0}), TArray({0.0})));
	bSuccess &= ExecuteTest(TestParams(Divide, GenerateParamData<FVector2D>({{FVector2D(0.3, 1.4)}, {FVector2D(-1.2, 1.4)}}), TArray({FVector2D::Zero()}), TArray({FVector2D(-0.25, 1.0)})));
	bSuccess &= ExecuteTest(TestParams(Divide, GenerateParamData<FVector>({{FVector(0.5, 1.6, 0.7)}, {FVector(-0.5, 4.2, -0.7)}}), TArray({FVector::Zero()}), TArray({FVector(-1.0, 0.381, -1.0)})));
	bSuccess &= ExecuteTest(TestParams(Divide, GenerateParamData<FVector4>({{FVector4(0.5, 1.6, 0.7, 0.8)}, {FVector4(-0.5, 4.2, -0.7, 0.3)}}), TArray({FVector4::Zero()}), TArray({FVector4(-1.0, 0.381, -1.0, 2.666667)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Divide, GeneratePointData(2), TArray<int32>({0}), TArray<int32>({0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Divide, GeneratePointData(2), TArray({0.0}), TArray({2.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Divide, GeneratePointData(2), TArray({FVector::Zero()}), TArray({FVector(0.5, 0.5, 0.5)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_MultipleInput, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.MultipleInput", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_MultipleInput::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Add;

	// 2 x 2 - N:N
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<int32>({{4, 5}, {6, 7}}), TArray({0, 0}), TArray({10, 12})));

	// 3 x 3 - N:N
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<int32>({{4, 5, 6}, {7, 8, 9}}), TArray({0, 0, 0}), TArray({11, 13, 15})));

	// 2 x 1 - N:1
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<int32>({{4, 5}, {6}}), TArray({0, 0}), TArray({10, 11})));

	// 3 x 1 - N:1
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<int32>({{4, 5, 6}, {7}}), TArray({0, 0, 0}), TArray({11, 12, 13})));

	// 1 x 3 - N:1 with second pin
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<int32>({{4}, {5, 6, 7}}), TArray({0, 0, 0}), TArray({9, 10, 11}), /*PinToForward=*/ 1));

	// 1 x 3 - 1:N
	bSuccess &= ExecuteTest(TestParams(Add, GenerateParamData<int32>({{4}, {5, 6, 7}}), TArray({0, 0, 0}), TArray({9, 10, 11})));

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Abs, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Abs", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Abs::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Abs;

	// Param Data
	int32 P1 = 5;
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<int32>({{P1}}), TArray({0}), TArray({P1})));
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<int32>({{-1 * P1}}), TArray({0}), TArray({P1})));

	int64 P2 = 5ll;
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<int64>({{P2}}), TArray({0ll}), TArray({P2})));
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<int64>({{-1ll * P2}}), TArray({0ll}), TArray({P2})));

	float P3 = 5.2f;
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<float>({{P3}}), TArray({0.f}), TArray({P3})));
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<float>({{-1.0f * P3}}), TArray({0.f}), TArray({P3})));

	double V1[] = {0.3, 512.2, 0.0, 15.0};
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<double>({{V1[0]}}), TArray({.0}), TArray({V1[0]})));
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<double>({{-1.0 * V1[0]}}), TArray({.0}), TArray({V1[0]})));
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<FVector2D>({{FVector2D(V1[0], V1[1])}}), TArray({FVector2D::Zero()}), TArray({FVector2D(V1[0], V1[1])})));
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<FVector2D>({{-1.0 * FVector2D(V1[0], V1[1])}}), TArray({FVector2D::Zero()}), TArray({FVector2D(V1[0], V1[1])})));
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<FVector>({{FVector(V1[0], V1[1], V1[2])}}), TArray({FVector::Zero()}), TArray({FVector(V1[0], V1[1], V1[2])})));
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<FVector>({{-1.0 * FVector(V1[0], V1[1], V1[2])}}), TArray({FVector::Zero()}), TArray({FVector(V1[0], V1[1], V1[2])})));
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<FVector4>({{FVector4(V1[0], V1[1], V1[2], V1[3])}}), TArray({FVector4::Zero()}), TArray({FVector4(V1[0], V1[1], V1[2], V1[3])})));
	bSuccess &= ExecuteTest(TestParams(Abs, GenerateParamData<FVector4>({{-1.0 * FVector4(V1[0], V1[1], V1[2], V1[3])}}), TArray({FVector4::Zero()}), TArray({FVector4(V1[0], V1[1], V1[2], V1[3])})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Abs, GeneratePointData(), TArray<int32>({0}), TArray<int32>({1})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Abs, GeneratePointData(), TArray({0.0}), TArray({1.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Abs, GeneratePointData(), TArray({FVector::Zero()}), TArray({FVector(1.0, 10.0, 100.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Inc, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Inc", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Inc::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Inc;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(Inc, GenerateParamData<int32>({{5}}), TArray({1}), TArray({6})));
	bSuccess &= ExecuteTest(TestParams(Inc, GenerateParamData<int64>({{6ll}}), TArray({1ll}), TArray({7ll})));
	bSuccess &= ExecuteTest(TestParams(Inc, GenerateParamData<float>({{0.1f}}), TArray({1.f}), TArray({1.1f})));
	bSuccess &= ExecuteTest(TestParams(Inc, GenerateParamData<double>({{0.2}}), TArray({1.0}), TArray({1.2})));
	bSuccess &= ExecuteTest(TestParams(Inc, GenerateParamData<FVector2D>({{FVector2D(0.3, 1.4)}}), TArray({FVector2D::One()}), TArray({FVector2D(1.3, 2.4)})));
	bSuccess &= ExecuteTest(TestParams(Inc, GenerateParamData<FVector>({{FVector(0.5, 1.6, 0.7)}}), TArray({FVector::One()}), TArray({FVector(1.5, 2.6, 1.7)})));
	bSuccess &= ExecuteTest(TestParams(Inc, GenerateParamData<FVector4>({{FVector4(0.5, 1.6, 0.7, 0.8)}}), TArray({FVector4::One()}), TArray({FVector4(1.5, 2.6, 1.7, 1.8)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Inc, GeneratePointData(), TArray<int32>({0}), TArray<int32>({2})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Inc, GeneratePointData(), TArray({0.0}), TArray({2.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Inc, GeneratePointData(), TArray({FVector(0.0, 0.0, 0.0)}), TArray({FVector(2.0, 11.0, 101.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Dec, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Dec", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Dec::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Dec;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(Dec, GenerateParamData<int32>({{5}}), TArray({-1}), TArray({4})));
	bSuccess &= ExecuteTest(TestParams(Dec, GenerateParamData<int64>({{6ll}}), TArray({-1ll}), TArray({5ll})));
	bSuccess &= ExecuteTest(TestParams(Dec, GenerateParamData<float>({{0.1f}}), TArray({-1.f}), TArray({-0.9f})));
	bSuccess &= ExecuteTest(TestParams(Dec, GenerateParamData<double>({{0.2}}), TArray({-1.0}), TArray({-0.8})));
	bSuccess &= ExecuteTest(TestParams(Dec, GenerateParamData<FVector2D>({{FVector2D(0.3, 1.4)}}), TArray({-FVector2D::One()}), TArray({FVector2D(-0.7, 0.4)})));
	bSuccess &= ExecuteTest(TestParams(Dec, GenerateParamData<FVector>({{FVector(0.5, 1.6, 0.7)}}), TArray({-FVector::One()}), TArray({FVector(-0.5, 0.6, -0.3)})));
	bSuccess &= ExecuteTest(TestParams(Dec, GenerateParamData<FVector4>({{FVector4(0.5, 1.6, 0.7, 0.8)}}), TArray({-FVector4::One()}), TArray({FVector4(-0.5, 0.6, -0.3, -0.2)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Dec, GeneratePointData(), TArray<int32>({0}), TArray<int32>({0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Dec, GeneratePointData(), TArray({0.0}), TArray({0.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Dec, GeneratePointData(), TArray({FVector(0.0, 0.0, 0.0)}), TArray({FVector(0.0, 9.0, 99.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_Negate, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.Negate", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_Negate::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::Negate;

	// Param Data
	bSuccess &= ExecuteTest(TestParams(Negate, GenerateParamData<int32>({{5}}), TArray({0}), TArray({-5})));
	bSuccess &= ExecuteTest(TestParams(Negate, GenerateParamData<int64>({{6ll}}), TArray({0ll}), TArray({-6ll})));
	bSuccess &= ExecuteTest(TestParams(Negate, GenerateParamData<float>({{0.1f}}), TArray({0.f}), TArray({-0.1f})));
	bSuccess &= ExecuteTest(TestParams(Negate, GenerateParamData<double>({{-0.2}}), TArray({0.0}), TArray({0.2})));
	bSuccess &= ExecuteTest(TestParams(Negate, GenerateParamData<FVector2D>({{FVector2D(0.3, -1.4)}}), TArray({FVector2D::Zero()}), TArray({FVector2D(-0.3, 1.4)})));
	bSuccess &= ExecuteTest(TestParams(Negate, GenerateParamData<FVector>({{FVector(0.5, -1.6, 0.7)}}), TArray({FVector::Zero()}), TArray({FVector(-0.5, 1.6, -0.7)})));
	bSuccess &= ExecuteTest(TestParams(Negate, GenerateParamData<FVector4>({{FVector4(0.5, -1.6, 0.7, -0.8)}}), TArray({FVector4::Zero()}), TArray({FVector4(-0.5, 1.6, -0.7, 0.8)})));

	// Point Data
	FPCGAttributePropertyInputSelector Selector;
	Selector.SetPointProperty(EPCGPointProperties::Seed);
	bSuccess &= ExecuteTest(TestParams(Negate, GeneratePointData(), TArray<int32>({0}), TArray<int32>({-1})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Density);
	bSuccess &= ExecuteTest(TestParams(Negate, GeneratePointData(), TArray({0.0}), TArray({-1.0})), Selector);
	Selector.SetPointProperty(EPCGPointProperties::Position);
	bSuccess &= ExecuteTest(TestParams(Negate, GeneratePointData(), TArray({FVector(0.0, 0.0, 0.0)}), TArray({FVector(-1.0, -10.0, -100.0)})), Selector);

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataMathsOp_AddModulo, FPCGMetadataMathsOpTest, "Plugins.PCG.Metadata.MathsOp.AddModulo", PCGTestsCommon::TestFlags)

bool FPCGMetadataMathsOp_AddModulo::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	using EPCGMetadataMathsOperation::AddModulo;
	
	// ({0, 1, 2} + 1) % 3 -> {1, 2, 0}
	bSuccess &= ExecuteTest(TestParams(AddModulo, GenerateParamData<int32>({{0, 1, 2}, {1}, {3}}), TArray({0, 0, 0}), TArray({1, 2, 0})));
	
	// ({0, 1, 2} - 1) % 3 -> {2, 0, 1}
	bSuccess &= ExecuteTest(TestParams(AddModulo, GenerateParamData<int64>({{0ll, 1ll, 2ll}, {-1ll}, {3ll}}), TArray({0ll, 0ll, 0ll}), TArray({2ll, 0ll, 1ll})));
	
	return bSuccess;
}

// TODO: Round, Sqrt, Floor, Ceil, Max, Min, Pow, ClampMin, ClampMax, Modulo, Set, Clamp, Lerp

#endif // WITH_EDITOR
