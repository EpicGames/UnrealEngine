// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGCustomVersion.h"
#include "PCGParamData.h"
#include "Algo/Count.h"

#include "Elements/Metadata/PCGMetadataRenameElement.h"

#if WITH_EDITOR

struct FPCGMetadataRenameTestState
{
	TArray<FName> ExistingAttributes = {TEXT(""), TEXT("Attr1"), TEXT("Attr2")};
	FPCGAttributePropertyInputSelector InSelector;
	FName ExpectedAttributeToBeRenamed;
	FPCGMetadataDomainID ExpectedDomain = PCGMetadataDomainID::Default;
	FName OutName = TEXT("NewAttr");
	bool bDeprecation = false;
	bool bValid = true;
};

class FPCGMetadataRenameTest : public FPCGTestBaseClass
{
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	bool Run(const FPCGMetadataRenameTestState& State)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPCGMetadataRenameSettings>(TestData);
		UPCGMetadataRenameSettings* Settings = CastChecked<UPCGMetadataRenameSettings>(TestData.Settings);

		Settings->AttributeToRename = State.InSelector;
		Settings->NewAttributeName = State.OutName;

		if (State.bDeprecation)
		{
			// Apply the deprecation on the settings, by forcing a previous version and calling postload
			Settings->DataVersion = FPCGCustomVersion::AttributePropertySelectorDeprecatePointProperties;
			Settings->PostLoad();
		}
		
		FPCGElementPtr MetadataBooleanElement = TestData.Settings->GetElement();

		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		check(ParamData);
		UPCGMetadata* Metadata = ParamData->MutableMetadata();
		FPCGMetadataDomain* MetadataDomain = Metadata ? Metadata->GetMetadataDomain(State.ExpectedDomain) : nullptr;
		check(MetadataDomain || State.ExpectedDomain == PCGMetadataDomainID::Invalid);

		if (State.ExpectedDomain != PCGMetadataDomainID::Invalid)
		{
			for (const FName Attribute : State.ExistingAttributes)
			{
				MetadataDomain->CreateAttribute<float>(Attribute, 0.0f, /*bAllowInterpolation=*/true, /*bOverrideParent=*/true);
			}
		}

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
		check(Context.IsValid());
		
		FPCGTaggedData& InputData = Context->InputData.TaggedData.Emplace_GetRef();
		InputData.Data = ParamData;
		InputData.Pin = PCGPinConstants::DefaultInputLabel;

		while (!MetadataBooleanElement->Execute(Context.Get())){}

		UTEST_EQUAL("Output data is present", Context->OutputData.TaggedData.Num(), 1);
		const UPCGParamData* OutputParamData = Cast<UPCGParamData>(Context->OutputData.TaggedData[0].Data);
		
		UTEST_NOT_NULL("Output data is a param data", OutputParamData);
		check(OutputParamData);

		const UPCGMetadata* OutputMetadata = OutputParamData->ConstMetadata();
		const FPCGMetadataDomain* OutputMetadataDomain = OutputMetadata ? OutputMetadata->GetConstMetadataDomain(State.ExpectedDomain) : nullptr;

		if (!OutputMetadataDomain && State.ExpectedDomain == PCGMetadataDomainID::Invalid)
		{
			return true;
		}
		
		check(OutputMetadataDomain);
		
		TArray<FName> AttributeNames;
		TArray<EPCGMetadataTypes> AttributeTypes;
		OutputMetadataDomain->GetAttributes(AttributeNames, AttributeTypes);

		UTEST_EQUAL(*FString::Printf(TEXT("Metadata has %d attributes"), State.ExistingAttributes.Num()), AttributeNames.Num(), State.ExistingAttributes.Num())
		const int32 NumAttributesMatchExisting = Algo::CountIf(AttributeNames, [&State](const FName AttributeName) { return State.ExistingAttributes.Contains(AttributeName); });

		if (State.bValid)
		{
			UTEST_TRUE("Attribute was renamed correctly", OutputMetadataDomain->HasAttribute(State.OutName) && !OutputMetadataDomain->HasAttribute(State.ExpectedAttributeToBeRenamed));
			UTEST_EQUAL("Other attributes are untouched", NumAttributesMatchExisting, State.ExistingAttributes.Num() - 1);
		}
		else
		{
			UTEST_EQUAL("Attributes are untouched", NumAttributesMatchExisting, State.ExistingAttributes.Num());
		}

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataRenameNoneTest, FPCGMetadataRenameTest, "Plugins.PCG.Metadata.Rename.None", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataRenameAtLastTest, FPCGMetadataRenameTest, "Plugins.PCG.Metadata.Rename.AtLast", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataRenameNormalTest, FPCGMetadataRenameTest, "Plugins.PCG.Metadata.Rename.Normal", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataRenameInvalidTest, FPCGMetadataRenameTest, "Plugins.PCG.Metadata.Rename.Invalid", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataRenameDataDomainTest, FPCGMetadataRenameTest, "Plugins.PCG.Metadata.Rename.DataDomain", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataRenameInvalidDataDomainTest, FPCGMetadataRenameTest, "Plugins.PCG.Metadata.Rename.InvalidDataDomain", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMetadataRenameDeprecationTest, FPCGMetadataRenameTest, "Plugins.PCG.Metadata.Rename.Deprecation", PCGTestsCommon::TestFlags)

bool FPCGMetadataRenameNoneTest::RunTest(const FString& Parameters)
{
	FPCGMetadataRenameTestState State;
	State.InSelector.SetAttributeName(NAME_None);
	State.ExpectedAttributeToBeRenamed = NAME_None;

	return Run(State);
}

bool FPCGMetadataRenameAtLastTest::RunTest(const FString& Parameters)
{
	FPCGMetadataRenameTestState State;
	State.InSelector.SetAttributeName(TEXT("@Last"));
	State.ExpectedAttributeToBeRenamed = TEXT("Attr2");

	return Run(State);
}

bool FPCGMetadataRenameNormalTest::RunTest(const FString& Parameters)
{
	FPCGMetadataRenameTestState State;
	State.InSelector.SetAttributeName(TEXT("Attr1"));
	State.ExpectedAttributeToBeRenamed = TEXT("Attr1");

	return Run(State);
}

bool FPCGMetadataRenameInvalidTest::RunTest(const FString& Parameters)
{
	FPCGMetadataRenameTestState State;
	State.bValid = false;

	// Property are not supported
	AddInfo(TEXT("Testing property in selector is not supported."));
	State.InSelector.SetPropertyName("Position");

	AddExpectedError(TEXT("Attribute to rename '$Position' is not an attribute."), EAutomationExpectedMessageFlags::Contains, 1, /*bIsRegex=*/false);

	if (!Run(State))
	{
		return false;
	}

	// Extra names are not supported
	AddInfo(TEXT("Testing extra names in selector is not supported."));
	State.InSelector.SetAttributeName("Attr1");
	State.InSelector.GetExtraNamesMutable().Emplace(TEXT("Foo"));

	AddExpectedError(TEXT("Attribute to rename 'Attr1.Foo' is not an attribute."), EAutomationExpectedMessageFlags::Contains, 1, /*bIsRegex=*/false);

	if (!Run(State))
	{
		return false;
	}

	// If the attribute doesn't exist, nothing is done.
	AddInfo(TEXT("Testing non-existing attribute is ignored."));
	State.InSelector.SetAttributeName("Foo");
	if (!Run(State))
	{
		return false;
	}

	// If the output attribute name is invalid, it fails
	AddInfo(TEXT("Testing non-valid attribute is not supported."));
	
	AddExpectedError(TEXT("Failed to rename attribute from 'Attr1' to 'Foo#'"), EAutomationExpectedMessageFlags::Contains, 1, /*bIsRegex=*/false);
	AddExpectedError(TEXT("New attribute name Foo# is not valid"), EAutomationExpectedMessageFlags::Contains, 1, /*bIsRegex=*/false);
	State.InSelector.SetAttributeName("Attr1");
	State.OutName = TEXT("Foo#");
	return Run(State);
}

bool FPCGMetadataRenameDataDomainTest::RunTest(const FString& Parameters)
{
	FPCGMetadataRenameTestState State;
	State.InSelector.Update(TEXT("@Data.Attr1"));
	State.ExpectedAttributeToBeRenamed = TEXT("Attr1");
	State.ExpectedDomain = PCGMetadataDomainID::Data;

	return Run(State);
}

bool FPCGMetadataRenameInvalidDataDomainTest::RunTest(const FString& Parameters)
{
	FPCGMetadataRenameTestState State;
	State.InSelector.Update(TEXT("@Foo.Attr1"));
	State.ExpectedAttributeToBeRenamed = TEXT("Attr1");
	State.ExpectedDomain = PCGMetadataDomainID::Invalid;
	State.bValid = false;

	AddExpectedError(TEXT("Metadata domain Foo is invalid for this data."), EAutomationExpectedMessageFlags::Contains, 1, /*bIsRegex=*/false);

	return Run(State);
}

bool FPCGMetadataRenameDeprecationTest::RunTest(const FString& Parameters)
{
	FPCGMetadataRenameTestState State;
	State.bDeprecation = true;

	// If the attribute to rename is not None, it should work as-is
	AddInfo(TEXT("Testing deprecation with multiple attribute and not None as input."));
	State.InSelector.SetAttributeName(TEXT("Attr1"));
	State.ExpectedAttributeToBeRenamed = TEXT("Attr1");

	if (!Run(State))
	{
		return false;
	}
	
	// If there are multiple attributes, InSelector is None and deprecation is on, it acts like @LastCreated
	State.InSelector.SetAttributeName(NAME_None);
	State.ExpectedAttributeToBeRenamed = TEXT("Attr2");

	AddInfo(TEXT("Testing deprecation with multiple attributes and None as input."));
	if (!Run(State))
	{
		return false;
	}

	// If there is just None, it should be None
	State.ExistingAttributes = {NAME_None};
	State.ExpectedAttributeToBeRenamed = NAME_None;

	AddInfo(TEXT("Testing deprecation with single attribute and None as input."));
	return Run(State);
}

#endif // WITH_EDITOR