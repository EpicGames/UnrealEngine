// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"

#include "Elements/PCGAddTag.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAddTagTest_ZeroTags, FPCGTestBaseClass, "Plugins.PCG.AddTag.ZeroTags", PCGTestsCommon::TestFlags)

bool FPCGAddTagTest_ZeroTags::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;

	//auto generate settings for this node
	PCGTestsCommon::GenerateSettings<UPCGAddTagSettings>(TestData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	// At least one input data is required to add tags and get an output
	FPCGTaggedData& NewTaggedData = Context->InputData.TaggedData.Emplace_GetRef();
	NewTaggedData.Data = PCGTestsCommon::CreateBasePointData();
	NewTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	while (!TestElement->Execute(Context.Get())) {}

	//testing inputs and outputs
	const TArray<FPCGTaggedData>& Inputs = Context->InputData.TaggedData;

	UTEST_EQUAL("Number of Inputs", Inputs.Num(), 2);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Number of Outputs", Outputs.Num(), 1);

	//testing tag info
	const TSet<FString> OutputTagSet = Context->OutputData.TaggedData[0].Tags;

	UTEST_EQUAL("Tag count", OutputTagSet.Num(), 0);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAddTagTest_OneTag, FPCGTestBaseClass, "Plugins.PCG.AddTag.OneTag", PCGTestsCommon::TestFlags)

bool FPCGAddTagTest_OneTag::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;

	//auto generate settings for this node
	PCGTestsCommon::GenerateSettings<UPCGAddTagSettings>(TestData);
	UPCGAddTagSettings* Settings = CastChecked<UPCGAddTagSettings>(TestData.Settings);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	Settings->TagsToAdd = "Feature";

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	// At least one input data is required to add tags and get an output
	FPCGTaggedData& NewTaggedData = Context->InputData.TaggedData.Emplace_GetRef();
	NewTaggedData.Data = PCGTestsCommon::CreateBasePointData();
	NewTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	while (!TestElement->Execute(Context.Get())) {}

	//testing inputs and outputs
	const TArray<FPCGTaggedData>& Inputs = Context->InputData.TaggedData;

	UTEST_EQUAL("Number of Inputs", Inputs.Num(), 2);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Number of Outputs", Outputs.Num(), 1);

	//testing tag info
	const TSet<FString>& OutputTagSet = Outputs[0].Tags;

	UTEST_EQUAL("Tag count", OutputTagSet.Num(), 1);

	if (!OutputTagSet.IsEmpty())
	{
		FString OutTag = OutputTagSet.Array()[0];

		UTEST_TRUE("Tags contains Feature", OutputTagSet.Contains("Feature"));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAddTagTest_MultipleTags, FPCGTestBaseClass, "Plugins.PCG.AddTag.MultipleTags", PCGTestsCommon::TestFlags)

bool FPCGAddTagTest_MultipleTags::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;

	//auto generate settings for this node
	PCGTestsCommon::GenerateSettings<UPCGAddTagSettings>(TestData);
	UPCGAddTagSettings* Settings = CastChecked<UPCGAddTagSettings>(TestData.Settings);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	Settings->TagsToAdd = "Feature, Cool, Purple";

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	// At least one input data is required to add tags and get an output
	FPCGTaggedData& NewTaggedData = Context->InputData.TaggedData.Emplace_GetRef();
	NewTaggedData.Data = PCGTestsCommon::CreateBasePointData();
	NewTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	while (!TestElement->Execute(Context.Get())) {}

	//testing inputs and outputs
	const TArray<FPCGTaggedData>& Inputs = Context->InputData.TaggedData;

	UTEST_EQUAL("Number of Inputs", Inputs.Num(), 2);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Number of Outputs", Outputs.Num(), 1);

	//testing tag info
	const TSet<FString> OutputTagSet = Outputs[0].Tags;

	UTEST_EQUAL("Tag count", OutputTagSet.Num(), 3);

	TArray<FString, TInlineAllocator<3>> ExpectedTags = { "Feature", "Cool", "Purple" };

	for (const FString& ExpectTag : ExpectedTags)
	{
		UTEST_TRUE(*FString::Printf(TEXT("Tags contains %s"), *ExpectTag), OutputTagSet.Contains(ExpectTag));
	}
	
	return true;
}

