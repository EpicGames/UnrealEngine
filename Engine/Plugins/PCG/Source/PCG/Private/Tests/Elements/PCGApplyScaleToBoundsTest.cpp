// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"

#include "Elements/PCGApplyScaleToBounds.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGApplyScaleToBoundsTest_Basic, FPCGTestBaseClass, "Plugins.PCG.ApplyScaleToBounds.Basic", PCGTestsCommon::TestFlags)

bool FPCGApplyScaleToBoundsTest_Basic::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	
	FPCGTaggedData& Inputs = TestData.InputData.TaggedData.Emplace_GetRef();
	Inputs.Pin = PCGPinConstants::DefaultInputLabel;
	UPCGApplyScaleToBoundsSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGApplyScaleToBoundsSettings>(TestData);
	UPCGBasePointData* InData = PCGTestsCommon::CreateRandomBasePointData(10, 42, false);
	Inputs.Data = InData;

	// Setting the scale explicitly
	TPCGValueRange<FTransform> TransformRange = InData->GetTransformValueRange();
	for (FTransform& PointTransform : TransformRange)
	{
		PointTransform.SetScale3D(FVector(3.0));
	}

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);
		
	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 10);

	const TConstPCGValueRange<FVector> BoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

	for (int i = 0; i < OutPointData->GetNumPoints(); ++i)
	{
		UTEST_EQUAL("Output points should have a Bounds Min of -3", BoundsMinRange[i], FVector(-3));
		UTEST_EQUAL("Output points should have a Bounds Max of 3", BoundsMaxRange[i], FVector(3));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGApplyScaleToBoundsTest_NegativeScale, FPCGTestBaseClass, "Plugins.PCG.ApplyScaleToBounds.NegativeScale", PCGTestsCommon::TestFlags)

bool FPCGApplyScaleToBoundsTest_NegativeScale::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;

	FPCGTaggedData& Inputs = TestData.InputData.TaggedData.Emplace_GetRef();
	Inputs.Pin = PCGPinConstants::DefaultInputLabel;
	UPCGApplyScaleToBoundsSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGApplyScaleToBoundsSettings>(TestData);
	UPCGBasePointData* InData = PCGTestsCommon::CreateRandomBasePointData(10, 42, false);
	Inputs.Data = InData;

	// Setting the scale explicitly
	TPCGValueRange<FTransform> TransformRange = InData->GetTransformValueRange();
	for (FTransform& PointTransform : TransformRange)
	{
		PointTransform.SetScale3D(FVector(-2.0));
	}

	FPCGElementPtr TestElement = TestData.Settings->GetElement(); // returns nullptr

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 10);

	const TConstPCGValueRange<FTransform> OutTransformRange = OutPointData->GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> OutBoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> OutBoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

	for (int i = 0; i < OutPointData->GetNumPoints(); ++i)
	{
		UTEST_EQUAL("Output points should have a Bounds Min of -2", OutBoundsMinRange[i], FVector(-2.0));
		UTEST_EQUAL("Output points should have a Bounds Max of 2", OutBoundsMaxRange[i], FVector(2.0));
		UTEST_EQUAL("Output points Bounds Max should be negative -1", OutTransformRange[i].GetScale3D(), FVector(-1.0));
	}

	return true;
}