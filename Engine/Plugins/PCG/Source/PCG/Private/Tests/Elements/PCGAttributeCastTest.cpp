// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGPointData.h"

#include "Elements/Metadata/PCGAttributeCast.h"

class FPCGAttributeCastTestBase : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	template <typename InType, typename OutType, typename PCGDataType>
	bool RunTestInternal(TArrayView<const InType> InData, TArrayView<const OutType> ExpectedOutData)
	{
		PCGTestsCommon::FTestData TestData;
		UPCGAttributeCastSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGAttributeCastSettings>(TestData);
		check(Settings);

		check(InData.Num() == ExpectedOutData.Num() && InData.Num() > 1);

		const FName InputAttributeName = TEXT("InAttr");
		const FName OutputAttributeName = TEXT("OutAttr");
		Settings->InputSource.SetAttributeName(InputAttributeName);
		Settings->OutputTarget.SetAttributeName(OutputAttributeName);
		Settings->OutputType = static_cast<EPCGMetadataTypes>(PCG::Private::MetadataTypes<OutType>::Id);

		PCGDataType* InputData = NewObject<PCGDataType>();
		TPCGValueRange<int64> MetadataEntryRange;
		if constexpr (std::is_same_v<PCGDataType, UPCGPointData> || std::is_same_v<PCGDataType, UPCGPointArrayData>)
		{
			InputData->SetNumPoints(InData.Num()-1);
			MetadataEntryRange = InputData->GetMetadataEntryValueRange();
		}

		FPCGMetadataAttribute<InType>* Attribute = InputData->Metadata->template CreateAttribute<InType>(InputAttributeName, InData[0], true, false);
		int32 WriteIndex = 0;
		for (int32 i = 1; i < InData.Num(); ++i)
		{
			PCGMetadataEntryKey EntryKey = InputData->Metadata->AddEntry();
			Attribute->SetValue(EntryKey, InData[i]);

			if constexpr (std::is_same_v<PCGDataType, UPCGPointData> || std::is_same_v<PCGDataType, UPCGPointArrayData>)
			{
				MetadataEntryRange[WriteIndex] = EntryKey;
			}
			++WriteIndex;
		}

		FPCGTaggedData& InputTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		InputTaggedData.Data = InputData;
		InputTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		FPCGElementPtr TestElement = TestData.Settings->GetElement();
		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		UTEST_EQUAL("There is one output", Context->OutputData.TaggedData.Num(), 1);

		const PCGDataType* OutputData = Cast<const PCGDataType>(Context->OutputData.TaggedData[0].Data);

		UTEST_NOT_NULL("Output is an of the same type as input", OutputData);

		if constexpr (std::is_same_v<PCGDataType, UPCGPointData> || std::is_same_v<PCGDataType, UPCGPointArrayData>)
		{
			UTEST_EQUAL("Output has the same number of points", InputData->GetNumPoints(), OutputData->GetNumPoints());
		}

		const FPCGMetadataAttribute<OutType>* OutputAttribute = OutputData->Metadata->template GetConstTypedAttribute<OutType>(OutputAttributeName);

		UTEST_NOT_NULL("Output metadata has the expected attribute", OutputAttribute);

		UTEST_EQUAL("Default value was casted correctly", OutputAttribute->GetValue(PCGDefaultValueKey), ExpectedOutData[0]);

		TConstPCGValueRange<int64> OutMetadataEntryRange;
		if constexpr (std::is_same_v<PCGDataType, UPCGPointData> || std::is_same_v<PCGDataType, UPCGPointArrayData>)
		{
			OutMetadataEntryRange = OutputData->GetConstMetadataEntryValueRange();
		}

		for (int32 i = 1; i < ExpectedOutData.Num(); ++i)
		{
			PCGMetadataEntryKey EntryKey(i - 1);
			if constexpr (std::is_same_v<PCGDataType, UPCGPointData> || std::is_same_v<PCGDataType, UPCGPointArrayData>)
			{
				EntryKey = OutMetadataEntryRange[i-1];
			}

			UTEST_EQUAL(*FString::Printf(TEXT("Entry %d was casted correctly"), i - 1), OutputAttribute->GetValueFromItemKey(EntryKey), ExpectedOutData[i]);
		}

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeCast_AttributeSetIntToDouble, FPCGAttributeCastTestBase, "Plugins.PCG.AttributeCast.AttributeSetIntToDouble", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeCast_AttributeSetDoubleToInt, FPCGAttributeCastTestBase, "Plugins.PCG.AttributeCast.AttributeSetDoubleToInt", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeCast_AttributeSetDoubleToBool, FPCGAttributeCastTestBase, "Plugins.PCG.AttributeCast.AttributeSetDoubleToBool", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeCast_AttributeSetBoolToDouble, FPCGAttributeCastTestBase, "Plugins.PCG.AttributeCast.AttributeSetBoolToDouble", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeCast_AttributeSetIntToString, FPCGAttributeCastTestBase, "Plugins.PCG.AttributeCast.AttributeSetIntToString", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeCast_AttributeSetQuatToRotator, FPCGAttributeCastTestBase, "Plugins.PCG.AttributeCast.AttributeSetQuatToRotator", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeCast_PointsIntToDouble, FPCGAttributeCastTestBase, "Plugins.PCG.AttributeCast.PointsIntToDouble", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeCast_PointsIntToDoubleToDensity, FPCGTestBaseClass, "Plugins.PCG.AttributeCast.PointsIntToDoubleToDensity", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeCast_SameType, FPCGTestBaseClass, "Plugins.PCG.AttributeCast.SameType", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeCast_InvalidCast, FPCGTestBaseClass, "Plugins.PCG.AttributeCast.InvalidCast", PCGTestsCommon::TestFlags)

bool FPCGAttributeCast_AttributeSetIntToDouble::RunTest(const FString& Parameters)
{
	return RunTestInternal<int, double, UPCGParamData>({ 5, 1, 2 }, { 5.0, 1.0, 2.0 });
}

bool FPCGAttributeCast_AttributeSetDoubleToInt::RunTest(const FString& Parameters)
{
	return RunTestInternal<double, int64, UPCGParamData>({ 5.1, 1.6, 2.3 }, { 5, 1, 2 });
}

bool FPCGAttributeCast_AttributeSetDoubleToBool::RunTest(const FString& Parameters)
{
	return RunTestInternal<double, bool, UPCGParamData>({ 5.1, 1.6, 2.3, 0.0, 0.0, 2.7 }, { true, true, true, false, false, true });
}

bool FPCGAttributeCast_AttributeSetBoolToDouble::RunTest(const FString& Parameters)
{
	return RunTestInternal<bool, double, UPCGParamData>({ true, true, true, false, false, true }, { 1.0, 1.0, 1.0, 0.0, 0.0, 1.0 });
}

bool FPCGAttributeCast_AttributeSetIntToString::RunTest(const FString& Parameters)
{
	return RunTestInternal<int, FString, UPCGParamData>({ 0, 8, 7, 4, -5 }, { TEXT("0"), TEXT("8"), TEXT("7"), TEXT("4"), TEXT("-5") });
}

bool FPCGAttributeCast_AttributeSetQuatToRotator::RunTest(const FString& Parameters)
{
	TArray<FQuat, TInlineAllocator<4>> QuatValues = {
		FQuat::Identity,
		FQuat(-0.70710678118654757, 0, 0, 0.70710678118654768), // (90, 0, 0)
		FQuat(0, -0.38268343236508978, 0, 0.92387953251128663), // (0, 45, 0)
		FQuat(0, 0, 0.25881904510252079, 0.96592582628906842)  // (0, 0, 30)
	};

	TArray<FRotator, TInlineAllocator<4>> RotatorValues = {
		FRotator::ZeroRotator,
		FRotator::MakeFromEuler({90, 0, 0}),
		FRotator::MakeFromEuler({0, 45, 0}),
		FRotator::MakeFromEuler({0, 0, 30})
	};

	return RunTestInternal<FQuat, FRotator, UPCGParamData>(QuatValues, RotatorValues);
}

bool FPCGAttributeCast_PointsIntToDouble::RunTest(const FString& Parameters)
{
	if (CVarPCGEnablePointArrayData.GetValueOnAnyThread())
	{
		return RunTestInternal<int, double, UPCGPointArrayData>({ 5, 1, 2 }, { 5.0, 1.0, 2.0 });
	}
	else
	{
		return RunTestInternal<int, double, UPCGPointData>({ 5, 1, 2 }, { 5.0, 1.0, 2.0 });
	}
}

bool FPCGAttributeCast_PointsIntToDoubleToDensity::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGAttributeCastSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGAttributeCastSettings>(TestData);
	check(Settings);

	const FName InputAttributeName = TEXT("Int");
	Settings->InputSource.SetAttributeName(InputAttributeName);
	Settings->OutputTarget.SetPointProperty(EPCGPointProperties::Density);
	Settings->OutputType = EPCGMetadataTypes::Double;

	UPCGBasePointData* InputPoints = PCGTestsCommon::CreateEmptyBasePointData();
	FPCGMetadataAttribute<int32>* Attribute = InputPoints->Metadata->CreateAttribute<int32>(InputAttributeName, 5, true, false);

	InputPoints->SetNumPoints(2);
	TPCGValueRange<int64> MetadataEntryRange = InputPoints->GetMetadataEntryValueRange();

	InputPoints->Metadata->InitializeOnSet(MetadataEntryRange[0]);
	InputPoints->Metadata->InitializeOnSet(MetadataEntryRange[1]);
	Attribute->SetValue(MetadataEntryRange[0], 1);
	Attribute->SetValue(MetadataEntryRange[1], 2);

	FPCGTaggedData& InputTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
	InputTaggedData.Data = InputPoints;
	InputTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	UTEST_EQUAL("There is one output", Context->OutputData.TaggedData.Num(), 1);

	const UPCGBasePointData* OutputPoints = Cast<const UPCGBasePointData>(Context->OutputData.TaggedData[0].Data);

	UTEST_NOT_NULL("Output is a point data", OutputPoints);
	UTEST_EQUAL("Output has 2 points", OutputPoints->GetNumPoints(), 2);

	const TConstPCGValueRange<float> DensityRange = OutputPoints->GetConstDensityValueRange();
	UTEST_EQUAL("First point has the right value", DensityRange[0], 1.0f);
	UTEST_EQUAL("Second point has the right value", DensityRange[1], 2.0f);

	return true;
}

bool FPCGAttributeCast_SameType::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGAttributeCastSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGAttributeCastSettings>(TestData);
	check(Settings);

	const FName InputAttributeName = TEXT("Int");
	Settings->InputSource.SetAttributeName(InputAttributeName);
	Settings->OutputTarget.SetAttributeName(InputAttributeName);
	Settings->OutputType = EPCGMetadataTypes::Integer32;

	UPCGParamData* InputParam = NewObject<UPCGParamData>();
	FPCGMetadataAttribute<int32>* Attribute = InputParam->Metadata->CreateAttribute<int32>(InputAttributeName, 5, true, false);
	Attribute->SetValue(InputParam->Metadata->AddEntry(), 1);
	Attribute->SetValue(InputParam->Metadata->AddEntry(), 2);

	FPCGTaggedData& InputTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
	InputTaggedData.Data = InputParam;
	InputTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	UTEST_EQUAL("There is one output", Context->OutputData.TaggedData.Num(), 1);
	UTEST_EQUAL("Output is same as input", Context->OutputData.TaggedData[0].Data.Get(), static_cast<const UPCGData*>(InputParam));

	return true;
}

bool FPCGAttributeCast_InvalidCast::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGAttributeCastSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGAttributeCastSettings>(TestData);
	check(Settings);

	const FName InputAttributeName = TEXT("String");
	Settings->InputSource.SetAttributeName(InputAttributeName);
	Settings->OutputTarget.SetAttributeName(InputAttributeName);
	Settings->OutputType = EPCGMetadataTypes::Integer32;

	UPCGParamData* InputParam = NewObject<UPCGParamData>();
	FPCGMetadataAttribute<FString>* Attribute = InputParam->Metadata->CreateAttribute<FString>(InputAttributeName, TEXT("Hello"), true, false);
	Attribute->SetValue(InputParam->Metadata->AddEntry(), TEXT("Hey"));
	Attribute->SetValue(InputParam->Metadata->AddEntry(), TEXT("Hi"));

	FPCGTaggedData& InputTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
	InputTaggedData.Data = InputParam;
	InputTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	AddExpectedError(TEXT("Cannot convert InputAttribute 'String' of type String into Integer32"));

	while (!TestElement->Execute(Context.Get())) {}

	UTEST_EQUAL("There is one output", Context->OutputData.TaggedData.Num(), 1);

	// Failed cast will just forward.
	UTEST_EQUAL("Output is same as input", Context->OutputData.TaggedData[0].Data.Get(), static_cast<const UPCGData*>(InputParam));

	return true;
}