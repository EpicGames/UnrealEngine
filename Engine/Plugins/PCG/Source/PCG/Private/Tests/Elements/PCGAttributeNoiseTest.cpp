// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "Elements/PCGAttributeNoise.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeNoiseDensityTest, FPCGTestBaseClass, "Plugins.PCG.AttributeNoise.Density", PCGTestsCommon::TestFlags)

bool FPCGAttributeNoiseDensityTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGAttributeNoiseSettings>(TestData);
	UPCGAttributeNoiseSettings* Settings = CastChecked<UPCGAttributeNoiseSettings>(TestData.Settings);
	Settings->InputSource.SetPointProperty(EPCGPointProperties::Density);
	FPCGElementPtr NoiseElement = TestData.Settings->GetElement();

	TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateEmptyBasePointData();

	FRandomStream RandomSource(TestData.Seed);
	const int PointCount = 5;

	PointData->SetNumPoints(PointCount);
	
	TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange();
	TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();

	for (int I = 0; I < PointCount; ++I)
	{
		DensityRange[I] = RandomSource.GetFraction();
		SeedRange[I] = I;
	}

	FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedData.Data = PointData;
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	auto ValidateDensityNoise = [this, &TestData, NoiseElement, Settings](TArray<float> ExpectedOutput) -> bool
	{
		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!NoiseElement->Execute(Context.Get()))
		{}

		const TArray<FPCGTaggedData>& Inputs = Context->InputData.GetAllSpatialInputs();
		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllSpatialInputs();

		if (!TestEqual("Valid number of outputs", Outputs.Num(), Inputs.Num()))
		{
			return false;
		}

		bool bTestPassed = true;

		for (int DataIndex = 0; DataIndex < Inputs.Num(); ++DataIndex)
		{
			const FPCGTaggedData& Input = Inputs[DataIndex];
			const FPCGTaggedData& Output = Outputs[DataIndex];

			const UPCGSpatialData* InSpatialData = Cast<UPCGSpatialData>(Input.Data);
			check(InSpatialData);

			const UPCGBasePointData* InPointData = InSpatialData->ToBasePointData(Context.Get());
			check(InPointData);

			const UPCGSpatialData* OutSpatialData = Cast<UPCGSpatialData>(Output.Data);

			if (!TestNotNull("Valid output SpatialData", OutSpatialData))
			{
				bTestPassed = false;
				continue;
			}

			const UPCGBasePointData* OutPointData = OutSpatialData->ToBasePointData(Context.Get());

			if (!TestNotNull("Valid output PointData", OutPointData))
			{
				bTestPassed = false;
				continue;
			}

			if (!TestEqual("Input and output point counts match", InPointData->GetNumPoints(), OutPointData->GetNumPoints()))
			{ 
				bTestPassed = false;
				continue;
			}

			const TConstPCGValueRange<float> DensityRange = OutPointData->GetConstDensityValueRange();
			for (int PointIndex = 0; PointIndex < ExpectedOutput.Num(); ++PointIndex)
			{
				bTestPassed &= TestEqual("Correct density", DensityRange[PointIndex], ExpectedOutput[PointIndex]);
			}
		}

		return bTestPassed;
	};

	bool bTestPassed = true;

	auto ValidateDensityNoiseForAllDensityModes = [this, &bTestPassed, ValidateDensityNoise, Settings]()
	{
		Settings->Mode = EPCGAttributeNoiseMode::Set;
		bTestPassed &= ValidateDensityNoise({});

		Settings->Mode = EPCGAttributeNoiseMode::Minimum;
		bTestPassed &= ValidateDensityNoise({});

		Settings->Mode = EPCGAttributeNoiseMode::Maximum;
		bTestPassed &= ValidateDensityNoise({});

		Settings->Mode = EPCGAttributeNoiseMode::Add;
		bTestPassed &= ValidateDensityNoise({});

		Settings->Mode = EPCGAttributeNoiseMode::Multiply;
		bTestPassed &= ValidateDensityNoise({});
	};

	// Test [0-1]
	{
		Settings->NoiseMin = 0.f;
		Settings->NoiseMax = 1.f;
		Settings->bInvertSource = false;
		ValidateDensityNoiseForAllDensityModes();

		Settings->bInvertSource = true;
		ValidateDensityNoiseForAllDensityModes();
	}

	// Test [0-0.5]
	{
		Settings->NoiseMin = 0.f;
		Settings->NoiseMax = 0.5f;
		Settings->bInvertSource = false;
		ValidateDensityNoiseForAllDensityModes();
		
		Settings->bInvertSource = true;
		ValidateDensityNoiseForAllDensityModes();
	}

	// Test [0.5-1]
	{
		Settings->NoiseMin = 0.5f;
		Settings->NoiseMax = 1.f;
		Settings->bInvertSource = false;
		ValidateDensityNoiseForAllDensityModes();

		Settings->bInvertSource = true;
		ValidateDensityNoiseForAllDensityModes();
	}

	// Test [1-0]
	{
		Settings->NoiseMin = 1.f;
		Settings->NoiseMax = 0.f;
		Settings->bInvertSource = false;
		ValidateDensityNoiseForAllDensityModes();

		Settings->bInvertSource = true;
		ValidateDensityNoiseForAllDensityModes();
	}

	// Test [0.5-0]
	{
		Settings->NoiseMin = 0.5f;
		Settings->NoiseMax = 0.f;
		Settings->bInvertSource = false;
		ValidateDensityNoiseForAllDensityModes();
		
		Settings->bInvertSource = true;
		ValidateDensityNoiseForAllDensityModes();
	}

	// Test [1-0.5]
	{
		Settings->NoiseMin = 1.f;
		Settings->NoiseMax = 0.5f;
		Settings->bInvertSource = false;
		ValidateDensityNoiseForAllDensityModes();

		Settings->bInvertSource = true;
		ValidateDensityNoiseForAllDensityModes();
	}

	Settings->bInvertSource = false;

	// Test expected values for Set
	{
		Settings->NoiseMin = 0.5f;
		Settings->NoiseMax = 0.5f;
		Settings->Mode = EPCGAttributeNoiseMode::Set;
		bTestPassed &= ValidateDensityNoise({ 0.5f, 0.5f, 0.5f, 0.5f, 0.5f });
	}

	// Test expected values for Minimum
	{
		Settings->NoiseMin = 0.f;
		Settings->NoiseMax = 0.f;
		Settings->Mode = EPCGAttributeNoiseMode::Minimum;
		bTestPassed &= ValidateDensityNoise({ 0.0f, 0.0f, 0.0f, 0.0f, 0.0f });
	}

	// Test expected values for Maximum
	{
		Settings->NoiseMin = 1.f;
		Settings->NoiseMax = 1.f;
		Settings->Mode = EPCGAttributeNoiseMode::Maximum;
		bTestPassed &= ValidateDensityNoise({ 1.f, 1.f, 1.f, 1.f, 1.f });
	}

	// Test expected values for Add
	{
		Settings->NoiseMin = 0.f;
		Settings->NoiseMax = 0.f;
		Settings->Mode = EPCGAttributeNoiseMode::Add;
		bTestPassed &= ValidateDensityNoise({ 
			DensityRange[0],
			DensityRange[1],
			DensityRange[2],
			DensityRange[3],
			DensityRange[4],
		});
	}

	// Test expected values for Multiply
	{
		Settings->NoiseMin = 0.f;
		Settings->NoiseMax = 0.f;
		Settings->Mode = EPCGAttributeNoiseMode::Multiply;
		bTestPassed &= ValidateDensityNoise({ 0.0f, 0.0f, 0.0f, 0.0f, 0.0f });
	}

	return bTestPassed;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeNoiseNotExistingAttributeTest, FPCGTestBaseClass, "Plugins.PCG.AttributeNoise.NotExistingAttribute", PCGTestsCommon::TestFlags)

bool FPCGAttributeNoiseNotExistingAttributeTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGAttributeNoiseSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGAttributeNoiseSettings>(TestData);
	check(Settings);

	Settings->InputSource.SetAttributeName(TEXT("Hi"));

	UPCGBasePointData* PointData = PCGTestsCommon::CreateRandomBasePointData(/*PointCount*/5, /*Seed*/ 42, /*bRandomDensity=*/true);

	FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
	TaggedData.Data = PointData;
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	FPCGElementPtr NoiseElement = TestData.Settings->GetElement();
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	AddExpectedError(TEXT("Could not create accessor. Attribute 'Hi' was not found."));

	while (!NoiseElement->Execute(Context.Get())) {}

	UTEST_EQUAL("No output", Context->OutputData.TaggedData.Num(), 0);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeNoiseInvalidTypeTest, FPCGTestBaseClass, "Plugins.PCG.AttributeNoise.InvalidType", PCGTestsCommon::TestFlags)

bool FPCGAttributeNoiseInvalidTypeTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGAttributeNoiseSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGAttributeNoiseSettings>(TestData);
	check(Settings);

	Settings->InputSource.SetAttributeName(TEXT("MyStr"));

	UPCGBasePointData* PointData = PCGTestsCommon::CreateRandomBasePointData(/*PointCount*/5, /*Seed*/ 42, /*bRandomDensity=*/true);
	FPCGMetadataAttribute<FString>* StrAttribute = PointData->Metadata->CreateAttribute<FString>(TEXT("MyStr"), FString{}, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);

	TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
	for (int64& MetadataEntry : MetadataEntryRange)
	{
		PointData->Metadata->InitializeOnSet(MetadataEntry);
		StrAttribute->SetValue(MetadataEntry, TEXT("Hey"));
	}

	FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
	TaggedData.Data = PointData;
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	FPCGElementPtr NoiseElement = TestData.Settings->GetElement();
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	AddExpectedError(TEXT("Attribute/Property 'MyStr' is not a numerical type, we can't apply noise to it."));

	while (!NoiseElement->Execute(Context.Get())) {}

	UTEST_EQUAL("No output", Context->OutputData.TaggedData.Num(), 0);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeNoiseAttributeSetTest, FPCGTestBaseClass, "Plugins.PCG.AttributeNoise.AttributeSet", PCGTestsCommon::TestFlags)

bool FPCGAttributeNoiseAttributeSetTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData(42);
	UPCGAttributeNoiseSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGAttributeNoiseSettings>(TestData);
	check(Settings);

	const FName InputAttrName = TEXT("Attr");
	const FName OutputAttrName = TEXT("OutAttr");
	constexpr int32 NbElements = 5;

	Settings->InputSource.SetAttributeName(InputAttrName);
	Settings->OutputTarget.SetAttributeName(OutputAttrName);

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	FPCGMetadataAttribute<float>* FloatAttribute = ParamData->Metadata->CreateAttribute<float>(InputAttrName, 0.f, /*bAllowInterpolation=*/true, /*bOverrideParent=*/false);
	for (int32 i = 0; i < NbElements; ++i)
	{
		FloatAttribute->SetValue(ParamData->Metadata->AddEntry(), (float)i / NbElements);
	}

	FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
	TaggedData.Data = ParamData;
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	FPCGElementPtr NoiseElement = TestData.Settings->GetElement();
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!NoiseElement->Execute(Context.Get())) {}

	UTEST_EQUAL("1 output", Context->OutputData.TaggedData.Num(), 1);

	const UPCGParamData* OutParamData = Cast<const UPCGParamData>(Context->OutputData.TaggedData[0].Data);
	UTEST_NOT_NULL("Output is param", OutParamData);

	const FPCGMetadataAttribute<float>* OutAttribute = OutParamData->Metadata->GetConstTypedAttribute<float>(OutputAttrName);
	UTEST_NOT_NULL("Output attrbiute exists", OutAttribute);

	// Taken from execution
	const float ExpectedValues[NbElements] = { 0.193192f, 0.134616f, 0.782528f, 0.084569f, 0.868019700f };
	for (int32 i = 0; i < NbElements; ++i)
	{
		float Value = OutAttribute->GetValueFromItemKey(PCGMetadataEntryKey(i));
		UTEST_EQUAL(FString::Printf(TEXT("Value %d is noised as expected"), i), Value, ExpectedValues[i]);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeNoiseOutputAttributeExistingTest, FPCGTestBaseClass, "Plugins.PCG.AttributeNoise.OutputAttributeExisting", PCGTestsCommon::TestFlags)

bool FPCGAttributeNoiseOutputAttributeExistingTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData(42);
	UPCGAttributeNoiseSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGAttributeNoiseSettings>(TestData);
	check(Settings);

	const FName InputAttrName = TEXT("Attr");
	constexpr int32 NbElements = 5;

	Settings->InputSource.Update(TEXT("Attr.X"));
	Settings->OutputTarget.Update(TEXT("Attr.Z"));

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	FPCGMetadataAttribute<FVector>* VectorAttribute = ParamData->Metadata->CreateAttribute<FVector>(InputAttrName, FVector::ZeroVector, /*bAllowInterpolation=*/true, /*bOverrideParent=*/false);
	for (int32 i = 0; i < NbElements; ++i)
	{
		VectorAttribute->SetValue(ParamData->Metadata->AddEntry(), FVector((double)i / NbElements));
	}

	FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
	TaggedData.Data = ParamData;
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	FPCGElementPtr NoiseElement = TestData.Settings->GetElement();
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!NoiseElement->Execute(Context.Get())) {}

	UTEST_EQUAL("1 output", Context->OutputData.TaggedData.Num(), 1);

	const UPCGParamData* OutParamData = Cast<const UPCGParamData>(Context->OutputData.TaggedData[0].Data);
	UTEST_NOT_NULL("Output is param", OutParamData);

	const FPCGMetadataAttribute<FVector>* OutAttribute = OutParamData->Metadata->GetConstTypedAttribute<FVector>(InputAttrName);
	UTEST_NOT_NULL("Output attrbiute exists", OutAttribute);

	// Taken from execution
	const double ExpectedValues[NbElements] = { 0.193192, 0.134616, 0.782528, 0.084569, 0.868019700 };
	for (int32 i = 0; i < NbElements; ++i)
	{
		FVector Value = OutAttribute->GetValueFromItemKey(PCGMetadataEntryKey(i));
		UTEST_EQUAL(FString::Printf(TEXT("Value %d for X component is the same"), i), Value.X, (double)i / NbElements);
		UTEST_EQUAL(FString::Printf(TEXT("Value %d for Z component is noised as expected"), i), Value.Z, ExpectedValues[i]);
	}

	return true;
}