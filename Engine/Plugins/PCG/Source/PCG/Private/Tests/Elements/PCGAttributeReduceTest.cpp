// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "PCGComponent.h"
#include "PCGParamData.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSplineData.h"
#include "Elements/PCGAttributeReduceElement.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

class FPCGAttributeReduceTests : public FPCGTestBaseClass
{
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	template <typename... T>
	bool RunAndValidate(TArray<FPCGTaggedData> InData, EPCGAttributeReduceOperation Operation, FPCGAttributePropertyInputSelector Selector, bool bMerged, bool bInDataDomain, T&& ...ExpectedValues)
	{
		AddInfo(*FString::Printf(TEXT("Test with '%s' as input attribute"), *Selector.GetDisplayText().ToString()));

		PCGTestsCommon::FTestData TestData;
		UPCGAttributeReduceSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGAttributeReduceSettings>(TestData);
		FPCGElementPtr Element = TestData.Settings->GetElement();

		constexpr int32 NumExpectedValues = sizeof...(ExpectedValues);

		for (FPCGTaggedData& InTaggedData : InData)
		{
			InTaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		}

		TestData.InputData.TaggedData.Append(std::move(InData));
		Settings->InputSource = std::move(Selector);
		Settings->bMergeOutputAttributes = bMerged;
		Settings->bWriteToDataDomain = bInDataDomain;
		Settings->Operation = Operation;
		Settings->OutputAttributeName = NAME_None;

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!Element->Execute(Context.Get())) {}

		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

		// If it is merged, we should have just 1 output with N values
		// If not merge, N outputs with 1 value
		if (bMerged)
		{
			UTEST_EQUAL("Merged output has a single value.", Outputs.Num(), 1);
		}
		else
		{
			UTEST_EQUAL("Not Merged output is the same number of inputs.", Outputs.Num(), NumExpectedValues);
		}

		// All outputs should be param data or same data as input
		TArray<const UPCGData*> AllData;
		AllData.Reserve(Outputs.Num());
		for (int32 i = 0; i < Outputs.Num(); ++i)
		{
			if (bInDataDomain)
			{
				// i+1 for input data as the first one is the settings.
				UTEST_TRUE(*FString::Printf(TEXT("Output %d is a the same type as input"), i), Outputs[i].Data && Outputs[i].Data->IsA(TestData.InputData.TaggedData[i+1].Data->StaticClass()));
			}
			else
			{
				UTEST_NOT_NULL(*FString::Printf(TEXT("Output %d is a param data"), i), Cast<UPCGParamData>(Outputs[i].Data));
			}
			
			AllData.Add(Outputs[i].Data);
		}

		// Finally validate all values matches, use a fold expression with Index noting the current param to check or entry to check
		int32 Index = 0;
		bool bSuccess = true;

		([&Index, this, &bSuccess, bMerged, &AllData, &ExpectedValues, bInDataDomain]()
		{
			if (!bSuccess)
			{
				return;
			}

			const UPCGData* Data = bMerged ? AllData[0] : AllData[Index];
			const FPCGMetadataDomain* MetadataDomain = bInDataDomain ? Data->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data) : Data->ConstMetadata()->GetConstDefaultMetadataDomain();

			const FPCGMetadataAttribute<T>* Attribute = MetadataDomain->GetConstTypedAttribute<T>(NAME_None);
			bSuccess = TestNotNull(TEXT("Attribute 'None' exists and is the right type."), Attribute);

			if (bSuccess)
			{
				check(Attribute);
				T ComputedValue = Attribute->GetValueFromItemKey(bMerged ? Index : 0);
				if constexpr (std::is_same_v<T, FVector2D>)
				{
					bSuccess = TestTrue(*FString::Printf(TEXT("Value for index %d matches. {%f, %f} vs {%f, %f}"), Index, ComputedValue.X, ComputedValue.Y, ExpectedValues.X, ExpectedValues.Y), ComputedValue.Equals(ExpectedValues));
				}
				else
				{
					bSuccess = TestEqual(*FString::Printf(TEXT("Value for index %d matches."), Index), ComputedValue, ExpectedValues);
				}
			}

			Index++;
		}(), ...);

		return bSuccess;
	}

	TArray<FPCGTaggedData> PreparePointsAndSelectors(FPCGAttributePropertyInputSelector& DensitySelector, FPCGAttributePropertyInputSelector& PositionSelector)
	{
		UPCGBasePointData* PointData = PCGTestsCommon::CreateEmptyBasePointData();

		constexpr int32 PointCount = 5;
		PointData->SetNumPoints(PointCount);
		
		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();
		TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange();
		
		for (int32 i = 0; i < PointCount; ++i)
		{
			const FVector Position{ 3.0 * i, 3.0 * i + 1.0, 3.0 * i + 2.0 };
			TransformRange[i] = FTransform(Position);
			DensityRange[i] = 0.1 * (i + 1);
			SeedRange[i] = i;
		}

		TArray<FPCGTaggedData> Inputs;
		FPCGTaggedData& Data = Inputs.Emplace_GetRef();
		Data.Data = PointData;

		DensitySelector.SetPointProperty(EPCGPointProperties::Density);
		PositionSelector.SetPointProperty(EPCGPointProperties::Position);

		return Inputs;
	}

	TArray<FPCGTaggedData> PrepareParamAndSelectors(FPCGAttributePropertyInputSelector& IntSelector, FPCGAttributePropertyInputSelector& Vec2Selector)
	{
		const FName IntName = TEXT("int");
		const FName Vec2Name = TEXT("vec2");

		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		FPCGMetadataAttribute<int32>* IntAttribute = ParamData->Metadata->CreateAttribute<int32>(IntName, 0, /*bAllowsInterpolation=*/ true, /*bOverrideParent=*/false);
		FPCGMetadataAttribute<FVector2D>* Vec2Attribute = ParamData->Metadata->CreateAttribute<FVector2D>(Vec2Name, FVector2D::ZeroVector, /*bAllowsInterpolation=*/ true, /*bOverrideParent=*/false);

		constexpr int32 EntryCount = 5;
		for (int32 i = 0; i < EntryCount; ++i)
		{
			const FVector2D Position{ 2.0 * i, 2.0 * i + 1.0 };
			PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();

			IntAttribute->SetValue(EntryKey, i);
			Vec2Attribute->SetValue(EntryKey, Position);
		}

		TArray<FPCGTaggedData> Inputs;
		FPCGTaggedData& Data = Inputs.Emplace_GetRef();
		Data.Data = ParamData;

		IntSelector.SetAttributeName(IntName);
		Vec2Selector.SetAttributeName(Vec2Name);

		return Inputs;
	}

	TArray<FPCGTaggedData> PrepareSplineAndSelectors(FPCGAttributePropertyInputSelector& PositionSelector, FPCGAttributePropertyInputSelector& IntSelector)
	{
		const FName IntName = TEXT("int");

		UPCGSplineData* SplineData = NewObject<UPCGSplineData>();
		FPCGMetadataAttribute<int32>* IntAttribute = SplineData->Metadata->CreateAttribute<int32>(IntName, 0, /*bAllowsInterpolation=*/ true, /*bOverrideParent=*/false);

		constexpr int32 PointCount = 5;
		TArray<FSplinePoint> SplinePoints;
		TArray<PCGMetadataEntryKey> SplineEntryKeys;
		
		SplinePoints.Reserve(PointCount);
		SplineEntryKeys.Reserve(PointCount);
		
		for (int32 i = 0; i < PointCount; ++i)
		{
			const FVector Position{ 3.0 * i, 3.0 * i + 1.0, 3.0 * i + 2 };
			PCGMetadataEntryKey& EntryKey = SplineEntryKeys.Emplace_GetRef(SplineData->Metadata->AddEntry());

			IntAttribute->SetValue(EntryKey, i);

			SplinePoints.Emplace(i, Position);
		}

		SplineData->Initialize(SplinePoints, /*bInClosedLoop=*/false, FTransform::Identity, MoveTemp(SplineEntryKeys));

		TArray<FPCGTaggedData> Inputs;
		FPCGTaggedData& Data = Inputs.Emplace_GetRef();
		Data.Data = SplineData;

		IntSelector.SetAttributeName(IntName);
		PositionSelector.SetPropertyName(TEXT("Position"));

		return Inputs;
	}

	TArray<FPCGTaggedData> PrepareMixedAndSelector(FPCGAttributePropertyInputSelector& Selector)
	{
		const FName DoubleName = TEXT("double");

		UPCGBasePointData* PointData = PCGTestsCommon::CreateEmptyBasePointData();
		FPCGMetadataAttribute<double>* PointsAttribute = PointData->Metadata->CreateAttribute<double>(DoubleName, 0.0, /*bAllowsInterpolation=*/ true, /*bOverrideParent=*/false);

		UPCGParamData* ParamData = NewObject<UPCGParamData>();
		FPCGMetadataAttribute<double>* ParamAttribute = ParamData->Metadata->CreateAttribute<double>(DoubleName, 0.0, /*bAllowsInterpolation=*/ true, /*bOverrideParent=*/false);

		constexpr int32 EntryCount = 5;

		PointData->SetNumPoints(5);
		TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();

		for (int32 i = 0; i < EntryCount; ++i)
		{
			PointData->Metadata->InitializeOnSet(MetadataEntryRange[i]);
			PointsAttribute->SetValue(MetadataEntryRange[i], 0.1 * (i + 1));

			ParamAttribute->SetValue(ParamData->Metadata->AddEntry(), 0.2 * (i + 1));
		}

		TArray<FPCGTaggedData> Inputs;
		FPCGTaggedData& Data1 = Inputs.Emplace_GetRef();
		FPCGTaggedData& Data2 = Inputs.Emplace_GetRef();
		Data1.Data = PointData;
		Data2.Data = ParamData;

		Selector.SetAttributeName(DoubleName);

		return Inputs;
	}
};


IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReducePoints_Average, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Points.Average", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReducePoints_Min, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Points.Min", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReducePoints_Max, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Points.Max", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReducePoints_Sum, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Points.Sum", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReducePoints_Join, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Points.Join", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReducePoints_DataDomain, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Points.DataDomain", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceParams_Average, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Params.Average", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceParams_Min, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Params.Min", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceParams_Max, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Params.Max", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceParams_Sum, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Params.Sum", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceParams_Join, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Params.Join", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceParams_DataDomain, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Params.DataDomain", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceSpline_DataDomain, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Spline.DataDomain", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceMixed_NoMerge, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Mixed.NoMerge", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceMixed_Merge, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.Mixed.Merge", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceFirstInputInvalid_NoMerge, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.FirstInputInvalid.NoMerge", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceFirstInputInvalid_Merge, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.FirstInputInvalid.Merge", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAttributeReduceIncompatibleType, FPCGAttributeReduceTests, "Plugins.PCG.AttributeReduce.IncompatibleType", PCGTestsCommon::TestFlags)


bool FPCGAttributeReducePoints_Average::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceDensity, InputSourcePosition;
	TArray<FPCGTaggedData> Inputs = PreparePointsAndSelectors(InputSourceDensity, InputSourcePosition);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourceDensity), /*bMerged=*/ false, /*bInDataDomain=*/ false, 0.3);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourcePosition), /*bMerged=*/ false, /*bInDataDomain=*/ false, FVector(6.0, 7.0, 8.0));

	return bSuccess;
}

bool FPCGAttributeReducePoints_Min::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceDensity, InputSourcePosition;
	TArray<FPCGTaggedData> Inputs = PreparePointsAndSelectors(InputSourceDensity, InputSourcePosition);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Min, std::move(InputSourceDensity), /*bMerged=*/ false, /*bInDataDomain=*/ false, 0.1);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Min, std::move(InputSourcePosition), /*bMerged=*/ false, /*bInDataDomain=*/ false, FVector(0.0, 1.0, 2.0));

	return bSuccess;
}

bool FPCGAttributeReducePoints_Max::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceDensity, InputSourcePosition;
	TArray<FPCGTaggedData> Inputs = PreparePointsAndSelectors(InputSourceDensity, InputSourcePosition);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Max, std::move(InputSourceDensity), /*bMerged=*/ false, /*bInDataDomain=*/ false, 0.5);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Max, std::move(InputSourcePosition), /*bMerged=*/ false, /*bInDataDomain=*/ false, FVector(12.0, 13.0, 14.0));

	return bSuccess;
}

bool FPCGAttributeReducePoints_Sum::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceDensity, InputSourcePosition;
	TArray<FPCGTaggedData> Inputs = PreparePointsAndSelectors(InputSourceDensity, InputSourcePosition);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Sum, std::move(InputSourceDensity), /*bMerged=*/ false, /*bInDataDomain=*/ false, 1.5);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Sum, std::move(InputSourcePosition), /*bMerged=*/ false, /*bInDataDomain=*/ false, FVector(30.0, 35.0, 40.0));

	return bSuccess;
}

bool FPCGAttributeReducePoints_Join::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceDensity, InputSourcePosition;
	TArray<FPCGTaggedData> Inputs = PreparePointsAndSelectors(InputSourceDensity, InputSourcePosition);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Join, std::move(InputSourceDensity), /*bMerged=*/ false, /*bInDataDomain=*/ false, FString(TEXT("0.100000, 0.200000, 0.300000, 0.400000, 0.500000")));

	return bSuccess;
}

bool FPCGAttributeReducePoints_DataDomain::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceDensity, InputSourcePosition;
	TArray<FPCGTaggedData> Inputs = PreparePointsAndSelectors(InputSourceDensity, InputSourcePosition);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourceDensity), /*bMerged=*/ false, /*bInDataDomain=*/ true, 0.3);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourcePosition), /*bMerged=*/ false, /*bInDataDomain=*/ true, FVector(6.0, 7.0, 8.0));

	return bSuccess;
}

bool FPCGAttributeReduceParams_Average::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceInt, InputSourceVec2;
	TArray<FPCGTaggedData> Inputs = PrepareParamAndSelectors(InputSourceInt, InputSourceVec2);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourceInt), /*bMerged=*/ false, /*bInDataDomain=*/ false, 2);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourceVec2), /*bMerged=*/ false, /*bInDataDomain=*/ false, FVector2D(4.0, 5.0));

	return bSuccess;
}

bool FPCGAttributeReduceParams_Min::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceInt, InputSourceVec2;
	TArray<FPCGTaggedData> Inputs = PrepareParamAndSelectors(InputSourceInt, InputSourceVec2);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Min, std::move(InputSourceInt), /*bMerged=*/ false, /*bInDataDomain=*/ false, 0);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Min, std::move(InputSourceVec2), /*bMerged=*/ false, /*bInDataDomain=*/ false, FVector2D(0.0, 1.0));

	return bSuccess;
}

bool FPCGAttributeReduceParams_Max::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceInt, InputSourceVec2;
	TArray<FPCGTaggedData> Inputs = PrepareParamAndSelectors(InputSourceInt, InputSourceVec2);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Max, std::move(InputSourceInt), /*bMerged=*/ false, /*bInDataDomain=*/ false, 4);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Max, std::move(InputSourceVec2), /*bMerged=*/ false, /*bInDataDomain=*/ false, FVector2D(8.0, 9.0));

	return bSuccess;
}

bool FPCGAttributeReduceParams_Sum::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceInt, InputSourceVec2;
	TArray<FPCGTaggedData> Inputs = PrepareParamAndSelectors(InputSourceInt, InputSourceVec2);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Sum, std::move(InputSourceInt), /*bMerged=*/ false, /*bInDataDomain=*/ false, 10);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Sum, std::move(InputSourceVec2), /*bMerged=*/ false, /*bInDataDomain=*/ false, FVector2D(20.0, 25.0));

	return bSuccess;
}

bool FPCGAttributeReduceParams_Join::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceInt, InputSourceVec2;
	TArray<FPCGTaggedData> Inputs = PrepareParamAndSelectors(InputSourceInt, InputSourceVec2);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Join, std::move(InputSourceInt), /*bMerged=*/ false, /*bInDataDomain=*/ false, FString(TEXT("0, 1, 2, 3, 4")));

	return bSuccess;
}

bool FPCGAttributeReduceParams_DataDomain::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourceInt, InputSourceVec2;
	TArray<FPCGTaggedData> Inputs = PrepareParamAndSelectors(InputSourceInt, InputSourceVec2);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourceInt), /*bMerged=*/ false, /*bInDataDomain=*/ true, 2);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourceVec2), /*bMerged=*/ false, /*bInDataDomain=*/ true, FVector2D(4.0, 5.0));

	return bSuccess;
}

bool FPCGAttributeReduceSpline_DataDomain::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSourcePosition, InputSourceInt;
	TArray<FPCGTaggedData> Inputs = PrepareSplineAndSelectors(InputSourcePosition, InputSourceInt);

	bool bSuccess = true;

	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourceInt), /*bMerged=*/ false, /*bInDataDomain=*/ true, 2);
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourcePosition), /*bMerged=*/ false, /*bInDataDomain=*/ true, FVector(6.0, 7.0, 8.0));

	return bSuccess;
}

bool FPCGAttributeReduceMixed_NoMerge::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSource;
	TArray<FPCGTaggedData> Inputs = PrepareMixedAndSelector(InputSource);

	return RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSource), /*bMerged=*/ false, /*bInDataDomain=*/ false, 0.3, 0.6);
}

bool FPCGAttributeReduceMixed_Merge::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSource;
	TArray<FPCGTaggedData> Inputs = PrepareMixedAndSelector(InputSource);

	return RunAndValidate(Inputs, EPCGAttributeReduceOperation::Max, std::move(InputSource), /*bMerged=*/ true, /*bInDataDomain=*/ false, 0.5, 1.0);
}

bool FPCGAttributeReduceFirstInputInvalid_NoMerge::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSource;
	TArray<FPCGTaggedData> Inputs = PrepareMixedAndSelector(InputSource);

	// Add Random Settings at the beginning
	FPCGTaggedData& TaggedData = Inputs.EmplaceAt_GetRef(0);
	TaggedData.Data = NewObject<UPCGAttributeReduceSettings>();

	AddExpectedError(TEXT("Invalid input data."));

	return RunAndValidate(Inputs, EPCGAttributeReduceOperation::Min, std::move(InputSource), /*bMerged=*/ false, /*bInDataDomain=*/ false, 0.1, 0.2);
}

bool FPCGAttributeReduceFirstInputInvalid_Merge::RunTest(const FString& Parameters)
{
	FPCGAttributePropertyInputSelector InputSource;
	TArray<FPCGTaggedData> Inputs = PrepareMixedAndSelector(InputSource);

	// Add Random Settings at the beginning
	FPCGTaggedData& TaggedData = Inputs.EmplaceAt_GetRef(0);
	TaggedData.Data = NewObject<UPCGAttributeReduceSettings>();

	AddExpectedError(TEXT("Invalid input data."));

	return RunAndValidate(Inputs, EPCGAttributeReduceOperation::Min, std::move(InputSource), /*bMerged=*/ true, /*bInDataDomain=*/ false, 0.1, 0.2);
}

bool FPCGAttributeReduceIncompatibleType::RunTest(const FString& Parameters)
{
	const FName StringName = TEXT("string");
	const FName QuatName = TEXT("quat");
	FPCGAttributePropertyInputSelector InputSourceString;
	FPCGAttributePropertyInputSelector InputSourceQuat;

	InputSourceString.SetAttributeName(StringName);
	InputSourceQuat.SetAttributeName(QuatName);

	UPCGParamData* ParamData = NewObject<UPCGParamData>();
	// String are always invalid
	FPCGMetadataAttribute<FString>* StringAttribute = ParamData->Metadata->CreateAttribute<FString>(StringName, FString{}, /*bAllowsInterpolation=*/ false, /*bOverrideParent=*/false);
	// Quat are invalid for min/max
	FPCGMetadataAttribute<FQuat>* QuatAttribute = ParamData->Metadata->CreateAttribute<FQuat>(QuatName, FQuat{ForceInitToZero}, /*bAllowsInterpolation=*/ true, /*bOverrideParent=*/false);

	constexpr int32 EntryCount = 5;
	for (int32 i = 0; i < EntryCount; ++i)
	{
		PCGMetadataEntryKey EntryKey = ParamData->Metadata->AddEntry();

		QuatAttribute->SetValue(EntryKey, FQuat::Identity);
		StringAttribute->SetValue(EntryKey, FString::FormatAsNumber(i));
	}

	TArray<FPCGTaggedData> Inputs;
	FPCGTaggedData& TaggedData = Inputs.Emplace_GetRef();
	TaggedData.Data = ParamData;

	bool bSuccess = true;

	AddExpectedError(TEXT("Operation was not compatible with the attribute type String or could not create attribute 'None' for input 0"));
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Average, std::move(InputSourceString), /*bMerged=*/ false, /*bInDataDomain=*/ false);

	AddExpectedError(TEXT("Operation was not compatible with the attribute type Quaternion or could not create attribute 'None' for input 0"));
	bSuccess &= RunAndValidate(Inputs, EPCGAttributeReduceOperation::Min, std::move(InputSourceQuat), /*bMerged=*/ false,  /*bInDataDomain=*/ false);

	return bSuccess;
}