// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGBlurElement.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGBlurElementTest_Density, FPCGTestBaseClass, "Plugins.PCG.BlurElement.Density", PCGTestsCommon::TestFlags)

/**
* Point data with 4 points, placed on a square.
* Starting with 1 in the top left corner, 0 elsewhere. Distance is the length of the square. Indexes are shown below.
* 0    1
* 
* 
* 2    3
* 
* Start values {1, 0, 0, 0}
* First Iteration: {0.333333, 0.333333, 0.333333, 0}
* Second Iteration: {0.333333, 0.222222, 0.222222, 0.222222}
*/
bool FPCGBlurElementTest_Density::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGBlurSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGBlurSettings>(TestData);
	check(Settings);

	constexpr double SquareSize = 100.0;

	Settings->InputSource.SetPointProperty(EPCGPointProperties::Density);
	Settings->NumIterations = 2;
	Settings->SearchDistance = SquareSize + 1.0; //+ 1 for approximation errors


	UPCGBasePointData* InputPointData = PCGTestsCommon::CreateEmptyBasePointData();
	
	InputPointData->SetNumPoints(4);
	InputPointData->SetSeed(42);

	TPCGValueRange<FTransform> TransformRange = InputPointData->GetTransformValueRange();
	TPCGValueRange<float> DensityRange = InputPointData->GetDensityValueRange();

	for (int i = 0; i < InputPointData->GetNumPoints(); ++i)
	{
		const int X = i / 2;
		const int Y = i % 2;
		const FVector Location = FVector(SquareSize * X, SquareSize * Y, 0.0);
		const float Density = i == 0 ? 1.0f : 0.0f;
		
		TransformRange[i] = FTransform(Location);
		DensityRange[i] = Density;
	}

	FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedData.Data = InputPointData;
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	FPCGElementPtr BlurElement = TestData.Settings->GetElement();
	check(BlurElement);

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	while (!BlurElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> Outputs = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	UTEST_EQUAL("There is one output", Outputs.Num(), 1);

	const UPCGBasePointData* OutputPointData = Cast<UPCGBasePointData>(Outputs[0].Data);
	UTEST_NOT_NULL("Output is a point data", OutputPointData);

	check(OutputPointData);

	UTEST_EQUAL("Output has as many points as input", OutputPointData->GetNumPoints(), InputPointData->GetNumPoints());

	constexpr float DensityValues[] = { 0.333333f, 0.222222f, 0.222222f, 0.222222f };

	const TConstPCGValueRange<float> OutDensityRange = OutputPointData->GetConstDensityValueRange();
	for (int32 i = 0; i < 4; ++i)
	{
		UTEST_EQUAL(*FString::Printf(TEXT("Point %d has the expected density."), i), OutDensityRange[i], DensityValues[i]);
	}

	return true;
}

#endif // WITH_EDITOR