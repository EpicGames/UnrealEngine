// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/ControlFlow/PCGBooleanSelect.h"
#include "Tests/PCGTestsCommon.h"

#if WITH_EDITOR

class FPCGBooleanSelectTest : public FPCGTestBaseClass
{
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	inline static const FString FirstDataTag = TEXT("FirstDataTag");
	inline static const FString SecondDataTag = TEXT("SecondDataTag");
	inline static const FName FirstInputPinLabel = TEXT("Input A");
	inline static const FName SecondInputPinLabel = TEXT("Input B");

	bool ExecuteTest(const bool bUseInputB, TArray<FPCGTaggedData>& OutputData)
	{
		PCGTestsCommon::FTestData TestData;
		UPCGBooleanSelectSettings* TestSettings = PCGTestsCommon::GenerateSettings<UPCGBooleanSelectSettings>(TestData);
		TestSettings->bUseInputB = bUseInputB;

		const FPCGElementPtr TestElement = TestSettings->GetElement();

		UTEST_TRUE("Test element created and valid", TestElement.IsValid());

		FPCGTaggedData& FirstTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		FirstTaggedData.Data = PCGTestsCommon::CreateEmptyPointData();
		FirstTaggedData.Tags.Emplace(FirstDataTag);
		FirstTaggedData.Pin = FirstInputPinLabel;

		FPCGTaggedData& SecondTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		SecondTaggedData.Data = PCGTestsCommon::CreateEmptyPointData();
		SecondTaggedData.Tags.Emplace(SecondDataTag);
		SecondTaggedData.Pin = SecondInputPinLabel;

		const TUniquePtr<FPCGContext> TestContext = TestData.InitializeTestContext();

		UTEST_TRUE("Test context created and valid", TestContext.IsValid());

		while (!TestElement->Execute(TestContext.Get()))
		{
		}

		OutputData = TestContext->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);

		UTEST_TRUE("Output was generated", !OutputData.IsEmpty());

		UTEST_EQUAL("Single output was generated", OutputData.Num(), 1);

		UTEST_EQUAL("Only one tag output", OutputData.Last().Tags.Num(), 1);

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGBooleanSelectTest_Basic, FPCGBooleanSelectTest, "Plugins.PCG.BooleanSelect.Basic", PCGTestsCommon::TestFlags)

bool FPCGBooleanSelectTest_Basic::RunTest(const FString& Parameters)
{
	TArray<FPCGTaggedData> OutputData;

	UTEST_TRUE("'Input A' test succeeded", ExecuteTest(false, OutputData));

	UTEST_TRUE("Output contains correct data", OutputData[0].Tags.Contains(FirstDataTag));
	UTEST_TRUE("Output does not contain incorrect data", !OutputData[0].Tags.Contains(SecondDataTag));

	UTEST_TRUE("'Input B' test succeeded", ExecuteTest(true, OutputData));

	UTEST_TRUE("Output contains correct data", OutputData[0].Tags.Contains(SecondDataTag));
	UTEST_TRUE("Output does not contain incorrect data", !OutputData[0].Tags.Contains(FirstDataTag));

	return true;
}

#endif // WITH_EDITOR