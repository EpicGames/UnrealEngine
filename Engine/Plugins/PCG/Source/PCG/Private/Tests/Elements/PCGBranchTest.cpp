// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGGraph.h"
#include "Data/PCGPointData.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGCreateAttribute.h"
#include "Elements/PCGCreatePoints.h"
#include "Elements/PCGGather.h"
#include "Elements/PCGTransformPoints.h"
#include "Elements/ControlFlow/PCGBranch.h"
#include "Tests/PCGTestsCommon.h"
#include "Tests/Compute/PCGGPUTestCommon.h"

class FPCGBranchTest : public FPCGTestBaseClass
{
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	bool ExecuteTest(const bool bOutputToB)
	{
		PCGTestsCommon::FTestData TestData;
		UPCGBranchSettings* TestSettings = PCGTestsCommon::GenerateSettings<UPCGBranchSettings>(TestData);
		TestSettings->bOutputToB = bOutputToB;

		const FPCGElementPtr TestElement = TestSettings->GetElement();

		UTEST_TRUE("Test element created and valid", TestElement.IsValid());

		FPCGTaggedData& InputTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		InputTaggedData.Data = PCGTestsCommon::CreateEmptyPointData();
		InputTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		const TUniquePtr<FPCGContext> TestContext = TestData.InitializeTestContext();

		UTEST_TRUE("Test context created and valid", TestContext.IsValid());

		while (!TestElement->Execute(TestContext.Get()))
		{
		}

		const TArray<FPCGTaggedData> FirstOutputData = TestContext->OutputData.GetInputsByPin(FName("Output A"));
		const TArray<FPCGTaggedData> SecondOutputData = TestContext->OutputData.GetInputsByPin(FName("Output B"));

		UTEST_TRUE("Output passed through to the correct pin", bOutputToB ? !SecondOutputData.IsEmpty() : !FirstOutputData.IsEmpty());

		UTEST_TRUE("Output not passed through to the incorrect pin", bOutputToB ? FirstOutputData.IsEmpty() : SecondOutputData.IsEmpty());

		UTEST_EQUAL("Only one output", FirstOutputData.Num() + SecondOutputData.Num(), 1);

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGBranchTest_Basic, FPCGBranchTest, "Plugins.PCG.Branch.Basic", PCGTestsCommon::TestFlags)

bool FPCGBranchTest_Basic::RunTest(const FString& Parameters)
{
	TArray<FPCGTaggedData> OutputData;

	UTEST_TRUE("'Output to A' test succeeded", ExecuteTest(false));

	UTEST_TRUE("'Output to B' test succeeded", ExecuteTest(true));

	return true;
}

/**
 * Shared GPU graph builder for the branch tests.
 *
 * Builds:
 *   CreatePoints (1 point) -> Branch -> "Output A" -> TransformPointsA (+100 on X) -\
 *                                    -> "Output B" -> TransformPointsB (+200 on Y) --> Gather
 *
 * The caller is responsible for wiring the branch condition (static or dynamic).
 */
namespace PCGBranchTestGraphGPU
{
	// Expected point location when routed through Output A (TransformPoints +100 on X).
	static const FVector OutputALocation = FVector(100.0, 0.0, 0.0);
	// Expected point location when routed through Output B (TransformPoints +200 on Y).
	static const FVector OutputBLocation = FVector(0.0, 200.0, 0.0);

	struct FNodes
	{
		UPCGNode* BranchNode = nullptr;
		UPCGNode* GatherNode = nullptr;
	};

	static FNodes Build(UPCGGraph* Graph, bool bOutputToB = false)
	{
		// CreatePoints: one default point at the origin.
		UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
		UPCGNode* CreatePointsNode = Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
		CreatePointsSettings->PointsToCreate.Empty();
		CreatePointsSettings->PointsToCreate.AddDefaulted();
		CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;

		// Branch: condition wired by the caller.
		UPCGBranchSettings* BranchSettings = nullptr;
		UPCGNode* BranchNode = Graph->AddNodeOfType<UPCGBranchSettings>(BranchSettings);
		BranchSettings->bOutputToB = bOutputToB;

		// TransformPoints A: translate +100 on X.
		UPCGTransformPointsSettings* TransformASettings = nullptr;
		UPCGNode* TransformANode = Graph->AddNodeOfType<UPCGTransformPointsSettings>(TransformASettings);
		TransformASettings->SetExecuteOnGPU(true);
		TransformASettings->OffsetMin = OutputALocation;
		TransformASettings->OffsetMax = OutputALocation;

		// TransformPoints B: translate +200 on Y.
		UPCGTransformPointsSettings* TransformBSettings = nullptr;
		UPCGNode* TransformBNode = Graph->AddNodeOfType<UPCGTransformPointsSettings>(TransformBSettings);
		TransformBSettings->SetExecuteOnGPU(true);
		TransformBSettings->OffsetMin = OutputBLocation;
		TransformBSettings->OffsetMax = OutputBLocation;

		// Gather: collects from both transform nodes.
		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

		Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, BranchNode,     PCGPinConstants::DefaultInputLabel);
		Graph->AddEdge(BranchNode,       PCGBranchConstants::OutputLabelA,    TransformANode, PCGPinConstants::DefaultInputLabel);
		Graph->AddEdge(BranchNode,       PCGBranchConstants::OutputLabelB,    TransformBNode, PCGPinConstants::DefaultInputLabel);
		Graph->AddEdge(TransformANode,   PCGPinConstants::DefaultOutputLabel, GatherNode,     PCGPinConstants::DefaultInputLabel);
		Graph->AddEdge(TransformBNode,   PCGPinConstants::DefaultOutputLabel, GatherNode,     PCGPinConstants::DefaultInputLabel);

		return { BranchNode, GatherNode };
	}

	/** Validates a single tagged data item: expects exactly 1 point at the given location. */
	static bool ValidateOutput(FAutomationTestBase* Test, const FPCGTaggedData& TaggedData, const FVector& ExpectedLocation)
	{
		const UPCGBasePointData* OutPoints = Cast<UPCGBasePointData>(TaggedData.Data);
		if (!Test->TestNotNull(TEXT("Readback data is point data"), OutPoints))
		{
			return false;
		}

		if (!Test->TestEqual(TEXT("Exactly one point"), OutPoints->GetNumPoints(), 1))
		{
			return false;
		}

		const FVector ActualLocation = OutPoints->GetTransform(0).GetLocation();

		return Test->TestNearlyEqual(*FString::Printf(TEXT("Point location (expected [%.0f, %.0f, %.0f], got [%.1f, %.1f, %.1f])"),
				ExpectedLocation.X, ExpectedLocation.Y, ExpectedLocation.Z, ActualLocation.X, ActualLocation.Y, ActualLocation.Z), ActualLocation, ExpectedLocation, 1.0f);
	}
}

// -----------------------------------------------------------------------------------------
// Static branch tests
//
// bOutputToB is baked directly onto the branch settings at graph construction time.
// IsPinStaticallyActive() returns false for the inactive pin, so the inactive GPU node is
// culled at graph-compile time before any dispatch occurs.
// -----------------------------------------------------------------------------------------

/**
 * Static branch (GPU): bOutputToB=false routes through Output A.
 * TransformPointsB is statically culled. Gather produces 1 point at (100, 0, 0).
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGBranchStaticTest_RouteToA_GPU,
	FPCGTestBaseClass,
	"Plugins.PCG.Branch.Static.RouteToA.GPU",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGBranchStaticTest_RouteToA_GPU::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;
	UPCGNode* GatherNode = nullptr;
	GatherNode = PCGBranchTestGraphGPU::Build(Runner.Graph, /*bOutputToB=*/false).GatherNode;

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0],
		[&](const FPCGTaggedData& TaggedData) { return PCGBranchTestGraphGPU::ValidateOutput(this, TaggedData, PCGBranchTestGraphGPU::OutputALocation); });
}

/**
 * Static branch (GPU): bOutputToB=true routes through Output B.
 * TransformPointsA is statically culled. Gather produces 1 point at (0, 200, 0).
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGBranchStaticTest_RouteToB_GPU,
	FPCGTestBaseClass,
	"Plugins.PCG.Branch.Static.RouteToB.GPU",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGBranchStaticTest_RouteToB_GPU::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;
	UPCGNode* GatherNode = nullptr;
	GatherNode = PCGBranchTestGraphGPU::Build(Runner.Graph, /*bOutputToB=*/true).GatherNode;

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0],
		[&](const FPCGTaggedData& TaggedData) { return PCGBranchTestGraphGPU::ValidateOutput(this, TaggedData, PCGBranchTestGraphGPU::OutputBLocation); });
}

// -----------------------------------------------------------------------------------------
// Dynamic branch tests
//
// bOutputToB is driven at runtime by a bool attribute fed to the branch's dedicated
// bOutputToB property override pin. Using the dedicated pin (rather than DefaultParamsLabel)
// ensures IsPropertyOverriddenByPin() returns true at compile time, keeping both output
// pins statically active so both GPU nodes compile into the task graph.
// IsPinStaticallyActive() returns true for both output pins (the inactive one is unknown
// until execution), so culling happens dynamically after the branch executes.
// -----------------------------------------------------------------------------------------

/**
 * Dynamic branch (GPU): bOutputToB attribute=false routes through Output A.
 * TransformPointsB is dynamically culled at runtime. Gather produces 1 point at (100, 0, 0).
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGBranchDynamicTest_RouteToA_GPU,
	FPCGTestBaseClass,
	"Plugins.PCG.Branch.Dynamic.RouteToA.GPU",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGBranchDynamicTest_RouteToA_GPU::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;
	UPCGNode* BranchNode = nullptr;
	UPCGNode* GatherNode = nullptr;
	const PCGBranchTestGraphGPU::FNodes Nodes = PCGBranchTestGraphGPU::Build(Runner.Graph);
	BranchNode = Nodes.BranchNode;
	GatherNode = Nodes.GatherNode;

	// CreateAttributeSet: bool attribute "bOutputToB" = false, fed to the branch's dedicated bOutputToB override pin.
	UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
	UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
	CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::Boolean;
	CreateAttrSettings->AttributeTypes.BoolValue = false;
	CreateAttrSettings->OutputTarget.SetAttributeName(GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB));

	Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, BranchNode, GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB));

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0],
		[&](const FPCGTaggedData& TaggedData) { return PCGBranchTestGraphGPU::ValidateOutput(this, TaggedData, PCGBranchTestGraphGPU::OutputALocation); });
}

/**
 * Dynamic branch (GPU): bOutputToB attribute=true routes through Output B.
 * TransformPointsA is dynamically culled at runtime. Gather produces 1 point at (0, 200, 0).
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGBranchDynamicTest_RouteToB_GPU,
	FPCGTestBaseClass,
	"Plugins.PCG.Branch.Dynamic.RouteToB.GPU",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGBranchDynamicTest_RouteToB_GPU::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;
	UPCGNode* GatherNode = nullptr;
	const PCGBranchTestGraphGPU::FNodes Nodes = PCGBranchTestGraphGPU::Build(Runner.Graph);
	GatherNode = Nodes.GatherNode;

	// CreateAttributeSet: bool attribute "bOutputToB" = true, fed to the branch's dedicated bOutputToB override pin.
	UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
	UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
	CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::Boolean;
	CreateAttrSettings->AttributeTypes.BoolValue = true;
	CreateAttrSettings->OutputTarget.SetAttributeName(GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB));

	Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, Nodes.BranchNode, GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB));

	TArray<FPCGDataCollection> OutNodeData;
	if (!Runner.Execute({ .Test = this, .CollectOutputDataForNodes = { GatherNode } }, /*OutNodeData=*/OutNodeData))
	{
		return false;
	}

	return PCGTestsCommon::ValidateSingleTaggedData(this, OutNodeData[0],
		[&](const FPCGTaggedData& TaggedData) { return PCGBranchTestGraphGPU::ValidateOutput(this, TaggedData, PCGBranchTestGraphGPU::OutputBLocation); });
}

// -----------------------------------------------------------------------------------------
// Execution dependency culling tests
//
// A Branch node's Output A is connected to the execution dependency pin of a CustomHLSL
// PointProcessor. With bExecutionDependencyRequired=true on the PointProcessor and
// bOutputToB=true on the Branch, Output A is culled, which transitively culls the
// PointProcessor. The Gather node (fed only by the PointProcessor) produces 0 data items.
//
// Graph:
//   CreatePoints (1 point) ------------------------------------> CustomHLSL (X += 50) -> Gather
//                                                              ^ ExecDep (Required)
//   CreateAttributeSet -> Branch (bOutputToB=true) -> OutputA -+
//
// Static test: bOutputToB=true is baked at construction time.
//   IsPinStaticallyActive(OutputA)=false -> PointProcessor's required execution dependency
//   pin has no active upstream -> PointProcessor statically culled before any GPU dispatch.
//
// Dynamic test: bOutputToB=true is fed at runtime via the dedicated bOutputToB override pin.
//   IsPinStaticallyActive(OutputA)=true at compile time (branch direction unknown), so the
//   PointProcessor is compiled into the task graph. At runtime Branch deactivates OutputA,
//   triggering dynamic culling of the PointProcessor via the inactive execution dependency.
// -----------------------------------------------------------------------------------------

namespace PCGBranchExecDepTestGraphGPU
{
	struct FNodes
	{
		UPCGNode* BranchNode = nullptr;
		UPCGNode* HLSLNode   = nullptr;
		UPCGNode* GatherNode = nullptr;
	};

	static FNodes Build(UPCGGraph* Graph, bool bOutputToB = false)
	{
		// CreatePoints: one default point at the origin.
		UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
		UPCGNode* CreatePointsNode = Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
		CreatePointsSettings->PointsToCreate.Empty();
		CreatePointsSettings->PointsToCreate.AddDefaulted();
		CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;

		// CustomHLSL PointProcessor: translate X += 50.
		// bExecutionDependencyRequired=true makes the execution dependency pin Required (not
		// Advanced), so the static culling analysis considers it and culls this node when
		// Branch.OutputA is statically inactive.
		UPCGCustomHLSLSettings* HLSLSettings = nullptr;
		UPCGNode* HLSLNode = Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
		HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
		HLSLSettings->SetSourceText(
			TEXT("float3 Pos = In_GetPosition(In_DataIndex, ElementIndex);\n")
			TEXT("Pos.x += 50.0f;\n")
			TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, Pos);\n")
		);
		HLSLSettings->bExecutionDependencyRequired = true;
		// Pin status (Advanced vs Required) is baked during UpdatePins(), which ran when AddNodeOfType
		// was called above (before bExecutionDependencyRequired was set). Refresh now so the ExecDep
		// input pin carries EPCGPinStatus::Required when the graph is compiled.
		HLSLNode->UpdateAfterSettingsChangeDuringCreation();

		// Branch: condition wired by the caller.
		UPCGBranchSettings* BranchSettings = nullptr;
		UPCGNode* BranchNode = Graph->AddNodeOfType<UPCGBranchSettings>(BranchSettings);
		BranchSettings->bOutputToB = bOutputToB;

		// Gather: reads the PointProcessor output.
		UPCGGatherSettings* GatherSettings = nullptr;
		UPCGNode* GatherNode = Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

		// CreateAttributeSet: provides data to Branch so it is not culled by the unwired-input check.
		UPCGCreateAttributeSetSettings* BranchFeedSettings = nullptr;
		UPCGNode* BranchFeedNode = Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(BranchFeedSettings);
		BranchFeedSettings->AttributeTypes.Type = EPCGMetadataTypes::Boolean;

		// Data path: CreatePoints -> CustomHLSL -> Gather.
		Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, HLSLNode,   PCGPinConstants::DefaultInputLabel);
		Graph->AddEdge(HLSLNode,         PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

		// Control path: Branch.OutputA -> CustomHLSL execution dependency pin.
		Graph->AddEdge(BranchFeedNode, PCGPinConstants::DefaultOutputLabel, BranchNode, PCGPinConstants::DefaultInputLabel);
		Graph->AddEdge(BranchNode,     PCGBranchConstants::OutputLabelA,    HLSLNode,   PCGPinConstants::DefaultExecutionDependencyLabel);

		return { BranchNode, HLSLNode, GatherNode };
	}
}

/**
 * Static branch (GPU): bOutputToB=true culls OutputA at compile time, statically culling the
 * PointProcessor via its required execution dependency pin. The PointProcessor never executes.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGBranchExecDepStaticTest_Culled_GPU,
	FPCGTestBaseClass,
	"Plugins.PCG.Branch.ExecDep.Static.Culled.GPU",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGBranchExecDepStaticTest_Culled_GPU::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;
	UPCGNode* HLSLNode = nullptr;
	HLSLNode = PCGBranchExecDepTestGraphGPU::Build(Runner.Graph, /*bOutputToB=*/true).HLSLNode;

	// Branch.OutputA is statically inactive (bOutputToB=true), so the PointProcessor's required
	// execution dependency pin has no active upstream -- it must be statically culled.
	TArray<FPCGDataCollection> OutNodeData;
	return Runner.Execute({ .Test = this, .bReadbackGPUData = false, .ExpectCulledNodes = { HLSLNode } }, /*OutNodeData=*/OutNodeData);
}

/**
 * Dynamic branch (GPU): bOutputToB=true (via dedicated override pin) culls OutputA at runtime,
 * dynamically culling the PointProcessor via its execution dependency pin.
 * Gather produces 0 data items.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FPCGBranchExecDepDynamicTest_Culled_GPU,
	FPCGTestBaseClass,
	"Plugins.PCG.Branch.ExecDep.Dynamic.Culled.GPU",
	PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGBranchExecDepDynamicTest_Culled_GPU::RunTest(const FString& Parameters)
{
	FPCGGPUGraphTestRunner Runner;
	UPCGNode* GatherNode = nullptr;
	const PCGBranchExecDepTestGraphGPU::FNodes Nodes = PCGBranchExecDepTestGraphGPU::Build(Runner.Graph);
	GatherNode = Nodes.GatherNode;

	// CreateAttributeSet: bool attribute "bOutputToB" = true, fed to Branch's dedicated bOutputToB override pin.
	UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
	UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
	CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::Boolean;
	CreateAttrSettings->AttributeTypes.BoolValue = true;
	CreateAttrSettings->OutputTarget.SetAttributeName(GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB));

	Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, Nodes.BranchNode, GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB));

	// Branch.OutputA is dynamically deactivated (bOutputToB=true at runtime), which culls the
	// PointProcessor via its required execution dependency, which should transitively cull Gather.
	// ExpectCulledNodes asserts the debug callback was NOT fired for these nodes.
	TArray<FPCGDataCollection> OutNodeData;
	return Runner.Execute({ .Test = this, .bReadbackGPUData = false, .ExpectCulledNodes = { Nodes.HLSLNode, GatherNode } }, /*OutNodeData=*/OutNodeData);
}

// -----------------------------------------------------------------------------------------
// Data pin culling tests
//
// A Branch node sits in the data path. Its Output A feeds the default In pin of a
// CustomHLSL PointProcessor. With bOutputToB=true, Output A is deactivated, leaving the
// PointProcessor's required In pin without an active upstream -- which culls it.
// Gather (fed only by the PointProcessor) is transitively culled.
//
// Graph:
//   CreatePoints (1 point) -> Branch (bOutputToB=true) -> OutputA (inactive) -> CustomHLSL.In
//                                                                                CustomHLSL.Out -> Gather
//
// Static test: bOutputToB=true baked at construction time.
//   IsPinStaticallyActive(OutputA)=false -> PointProcessor's required In pin has no active
//   upstream -> PointProcessor statically culled before any GPU dispatch.
//
// Dynamic test: bOutputToB=true fed at runtime via the dedicated bOutputToB override pin.
//   IsPinStaticallyActive(OutputA)=true at compile time (branch direction unknown), so the
//   PointProcessor is compiled in. At runtime Branch deactivates OutputA, triggering dynamic
//   culling of the PointProcessor via its inactive In pin.
// -----------------------------------------------------------------------------------------

namespace PCGBranchDataPinTestGraphGPU
{
    struct FNodes
    {
        UPCGNode* BranchNode    = nullptr;
        UPCGNode* HLSLNode      = nullptr;
        UPCGNode* GatherNode    = nullptr;
    };

    static FNodes Build(UPCGGraph* Graph, bool bOutputToB = false)
    {
        // CreatePoints: one default point.
        UPCGCreatePointsSettings* CreatePointsSettings = nullptr;
        UPCGNode* CreatePointsNode = Graph->AddNodeOfType<UPCGCreatePointsSettings>(CreatePointsSettings);
        CreatePointsSettings->PointsToCreate.Empty();
        CreatePointsSettings->PointsToCreate.AddDefaulted();
        CreatePointsSettings->CoordinateSpace = EPCGCoordinateSpace::World;

        // Branch: sits in the data path. bOutputToB routes away from OutputA.
        UPCGBranchSettings* BranchSettings = nullptr;
        UPCGNode* BranchNode = Graph->AddNodeOfType<UPCGBranchSettings>(BranchSettings);
        BranchSettings->bOutputToB = bOutputToB;

        // CustomHLSL PointProcessor: translate X += 50.
        // No exec dep pin -- culling is driven purely by the inactive In data pin.
        UPCGCustomHLSLSettings* HLSLSettings = nullptr;
        UPCGNode* HLSLNode = Graph->AddNodeOfType<UPCGCustomHLSLSettings>(HLSLSettings);
        HLSLSettings->SetKernelType(EPCGKernelType::PointProcessor);
        HLSLSettings->SetSourceText(
            TEXT("float3 Pos = In_GetPosition(In_DataIndex, ElementIndex);\n")
            TEXT("Pos.x += 50.0f;\n")
            TEXT("Out_SetPosition(Out_DataIndex, ElementIndex, Pos);\n")
        );

        // Gather: reads the PointProcessor output.
        UPCGGatherSettings* GatherSettings = nullptr;
        UPCGNode* GatherNode = Graph->AddNodeOfType<UPCGGatherSettings>(GatherSettings);

        // Data path: CreatePoints -> Branch -> OutputA -> CustomHLSL.In -> Gather.
        Graph->AddEdge(CreatePointsNode, PCGPinConstants::DefaultOutputLabel, BranchNode, PCGPinConstants::DefaultInputLabel);
        Graph->AddEdge(BranchNode,       PCGBranchConstants::OutputLabelA,    HLSLNode,   PCGPinConstants::DefaultInputLabel);
        Graph->AddEdge(HLSLNode,         PCGPinConstants::DefaultOutputLabel, GatherNode, PCGPinConstants::DefaultInputLabel);

        return { BranchNode, HLSLNode, GatherNode };
    }
}

/**
 * Static branch (GPU): bOutputToB=true culls OutputA at compile time, statically culling the
 * PointProcessor via its required In data pin (no exec dep pin involved).
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGBranchDataPinStaticTest_Culled_GPU,
    FPCGTestBaseClass,
    "Plugins.PCG.Branch.DataPin.Static.Culled.GPU",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGBranchDataPinStaticTest_Culled_GPU::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;
    const PCGBranchDataPinTestGraphGPU::FNodes Nodes = PCGBranchDataPinTestGraphGPU::Build(Runner.Graph, /*bOutputToB=*/true);

    // Branch.OutputA is statically inactive, so the PointProcessor's required In data pin has
    // no active upstream -- it must be statically culled.
    TArray<FPCGDataCollection> OutNodeData;
    return Runner.Execute({ .Test = this, .bReadbackGPUData = false, .ExpectCulledNodes = { Nodes.HLSLNode, Nodes.GatherNode } }, /*OutNodeData=*/OutNodeData);
}

/**
 * Dynamic branch (GPU): bOutputToB=true (via dedicated override pin) culls OutputA at runtime,
 * dynamically culling the PointProcessor via its required In data pin (no exec dep pin involved).
 * Gather is transitively culled.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
    FPCGBranchDataPinDynamicTest_Culled_GPU,
    FPCGTestBaseClass,
    "Plugins.PCG.Branch.DataPin.Dynamic.Culled.GPU",
    PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGBranchDataPinDynamicTest_Culled_GPU::RunTest(const FString& Parameters)
{
    FPCGGPUGraphTestRunner Runner;
    const PCGBranchDataPinTestGraphGPU::FNodes Nodes = PCGBranchDataPinTestGraphGPU::Build(Runner.Graph);

    // CreateAttributeSet: bool attribute "bOutputToB" = true, fed to Branch's dedicated override pin.
    UPCGCreateAttributeSetSettings* CreateAttrSettings = nullptr;
    UPCGNode* CreateAttrNode = Runner.Graph->AddNodeOfType<UPCGCreateAttributeSetSettings>(CreateAttrSettings);
    CreateAttrSettings->AttributeTypes.Type = EPCGMetadataTypes::Boolean;
    CreateAttrSettings->AttributeTypes.BoolValue = true;
    CreateAttrSettings->OutputTarget.SetAttributeName(GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB));

    Runner.Graph->AddEdge(CreateAttrNode, PCGPinConstants::DefaultOutputLabel, Nodes.BranchNode, GET_MEMBER_NAME_CHECKED(UPCGBranchSettings, bOutputToB));

    // Branch.OutputA is dynamically deactivated, leaving the PointProcessor's required In data
    // pin without an active upstream -- triggering dynamic culling of both PointProcessor and Gather.
    TArray<FPCGDataCollection> OutNodeData;
    return Runner.Execute({ .Test = this, .bReadbackGPUData = false, .ExpectCulledNodes = { Nodes.HLSLNode, Nodes.GatherNode } }, /*OutNodeData=*/OutNodeData);
}

#endif // WITH_EDITOR
