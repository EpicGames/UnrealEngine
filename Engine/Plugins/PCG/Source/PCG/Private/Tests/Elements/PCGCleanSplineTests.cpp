// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCGContext.h"
#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"

#include "Data/PCGSplineData.h"
#include "Elements/PCGCleanSpline.h"

#include "UObject/Package.h"

class FPCGCleanSplineTest : public FPCGTestBaseClass
{
	using FPCGTestBaseClass::FPCGTestBaseClass;
	
protected:
	bool RunTestInternal(UPCGCleanSplineSettings* Settings, const TArray<FVector>& Points, bool bLinear, bool bIsClosed, TArray<int32> ExpectedOutputPoints, const TArray<FVector>* ExpectedPositions = nullptr)
	{
		PCGTestsCommon::FTestData TestData;
		TestData.Reset(Settings);

		// Add dummy metadata entries to keep track of which control point was removed.
		TArray<PCGMetadataEntryKey> EntryKeys;
		EntryKeys.Reserve(Points.Num());
		for (int64 i = 0; i < Points.Num(); ++i)
		{
			EntryKeys.Emplace(i);
		}
		
		UPCGSplineData* SplineData = NewObject<UPCGSplineData>(GetTransientPackage(), NAME_None, RF_Transient);

		TArray<FSplinePoint> SplinePoints;
		SplinePoints.Reserve(Points.Num());
		for (int32 i = 0; i < Points.Num(); ++i)
		{
			SplinePoints.Emplace_GetRef(i, Points[i]).Type = bLinear ? ESplinePointType::Linear : ESplinePointType::Curve;
		}

		SplineData->Initialize(SplinePoints, bIsClosed, FTransform::Identity, MoveTemp(EntryKeys));

		FPCGTaggedData& Inputs = TestData.InputData.TaggedData.Emplace_GetRef();
		Inputs.Pin = PCGPinConstants::DefaultInputLabel;
		Inputs.Data = SplineData;
	
		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		UTEST_EQUAL("Output has 1 data", Context->OutputData.TaggedData.Num(), 1);

		const UPCGSplineData* OutputSplineData = Cast<const UPCGSplineData>(Context->OutputData.TaggedData[0].Data);
		UTEST_NOT_NULL("Output is a spline", OutputSplineData);

		UTEST_EQUAL(*FString::Printf(TEXT("Output spline has %d points"), ExpectedOutputPoints.Num()), OutputSplineData->SplineStruct.GetNumberOfPoints(), ExpectedOutputPoints.Num());

		// With Auto, we keep the first point by default, except for the last point. So in this case, the metadata entries should be 0 and 3.
		UTEST_EQUAL(*FString::Printf(TEXT("Output spline has %d metadata entries"), ExpectedOutputPoints.Num()), OutputSplineData->GetConstVerticesEntryKeys().Num(), ExpectedOutputPoints.Num())

		for (int32 i = 0; i < ExpectedOutputPoints.Num(); ++i)
		{
			const int64 ExpectedIndex = ExpectedOutputPoints[i];
			UTEST_EQUAL(*FString::Printf(TEXT("Point nº%d has was the original point nº%lld"), i, ExpectedIndex), OutputSplineData->GetConstVerticesEntryKeys()[i], ExpectedIndex);
		}

		if (ExpectedPositions)
		{
			for (int32 i = 0; i < ExpectedPositions->Num(); ++i)
			{
				UTEST_EQUAL(*FString::Printf(TEXT("Point nº%d is at the right position"), i), OutputSplineData->SplineStruct.GetLocationAtSplineInputKey(i, ESplineCoordinateSpace::World), (*ExpectedPositions)[i]);
			}
		}
		
		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollocatedControlPoints_Auto, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollocatedControlPoints.Auto", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollocatedControlPoints_KeepFirst, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollocatedControlPoints.KeepFirst", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollocatedControlPoints_KeepSecond, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollocatedControlPoints.KeepSecond", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollocatedControlPoints_Merge, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollocatedControlPoints.Merge", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollocatedControlPoints_Closed_Auto, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollocatedControlPoints.Closed_Auto", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollocatedControlPoints_Closed_KeepFirst, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollocatedControlPoints.Closed_KeepFirst", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollocatedControlPoints_Closed_KeepSecond, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollocatedControlPoints.Closed_KeepSecond", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollocatedControlPoints_Closed_Merge, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollocatedControlPoints.Closed_Merge", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollinearControlPoints_Linear, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollinearControlPoints.Linear", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollinearControlPoints_Linear_Closed, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollinearControlPoints.Linear_Closed", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollinearControlPoints_Curve, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollinearControlPoints.Curve", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCleanSplineTest_CollinearControlPoints_Curve_Closed, FPCGCleanSplineTest, "Plugins.PCG.CleanSpline.CollinearControlPoints.Curve_Closed", PCGTestsCommon::TestFlags)


bool FPCGCleanSplineTest_CollocatedControlPoints_Auto::RunTest(const FString& Parameters)
{
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{0, 0, 0},
		{100, 100, 100},
		{100, 100, 100},
	};

	bool bIsClosed = false;
	bool bLinear = true;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = true;
	Settings->bRemoveCollinearControlPoints = false;
	Settings->FuseMode = EPCGControlPointFuseMode::Auto;
	Settings->ColocationDistanceThreshold = 1.0;

	TArray<int32> ExpectedOutputPoints = {0, 3};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints));
}

bool FPCGCleanSplineTest_CollocatedControlPoints_KeepFirst::RunTest(const FString& Parameters)
{
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{0, 0, 0},
		{100, 100, 100},
		{100, 100, 100},
	};

	bool bIsClosed = false;
	bool bLinear = true;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = true;
	Settings->bRemoveCollinearControlPoints = false;
	Settings->FuseMode = EPCGControlPointFuseMode::KeepFirst;
	Settings->ColocationDistanceThreshold = 1.0;

	TArray<int32> ExpectedOutputPoints = {0, 2};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints));
}

bool FPCGCleanSplineTest_CollocatedControlPoints_KeepSecond::RunTest(const FString& Parameters)
{
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{0, 0, 0},
		{100, 100, 100},
		{100, 100, 100},
	};

	bool bIsClosed = false;
	bool bLinear = true;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = true;
	Settings->bRemoveCollinearControlPoints = false;
	Settings->FuseMode = EPCGControlPointFuseMode::KeepSecond;
	Settings->ColocationDistanceThreshold = 1.0;

	TArray<int32> ExpectedOutputPoints = {1, 3};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints));
}

bool FPCGCleanSplineTest_CollocatedControlPoints_Merge::RunTest(const FString& Parameters)
{
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{20, 0, 0},
		{100, 0, 0},
		{120, 0, 0},
	};

	bool bIsClosed = false;
	bool bLinear = true;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = true;
	Settings->bRemoveCollinearControlPoints = false;
	Settings->FuseMode = EPCGControlPointFuseMode::Merge;
	Settings->ColocationDistanceThreshold = 21.0;

	TArray<int32> ExpectedOutputPoints = {0, 2};
	TArray<FVector> ExpectedPositions =
	{
		{10, 0, 0},
		{110, 0, 0}
	};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints), &ExpectedPositions);
}

bool FPCGCleanSplineTest_CollocatedControlPoints_Closed_Auto::RunTest(const FString& Parameters)
{
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{0, 0, 0},
		{100, 100, 100},
		{100, 100, 100},
		{0, 0, 0},
	};

	bool bIsClosed = true;
	bool bLinear = true;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = true;
	Settings->bRemoveCollinearControlPoints = false;
	Settings->FuseMode = EPCGControlPointFuseMode::Auto;
	Settings->ColocationDistanceThreshold = 1.0;

	TArray<int32> ExpectedOutputPoints = {2, 4};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints));
}

bool FPCGCleanSplineTest_CollocatedControlPoints_Closed_KeepFirst::RunTest(const FString& Parameters)
{
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{0, 0, 0},
		{100, 100, 100},
		{100, 100, 100},
		{0, 0, 0},
	};

	bool bIsClosed = true;
	bool bLinear = true;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = true;
	Settings->bRemoveCollinearControlPoints = false;
	Settings->FuseMode = EPCGControlPointFuseMode::KeepFirst;
	Settings->ColocationDistanceThreshold = 1.0;

	TArray<int32> ExpectedOutputPoints = {2, 4};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints));
}

bool FPCGCleanSplineTest_CollocatedControlPoints_Closed_KeepSecond::RunTest(const FString& Parameters)
{
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{0, 0, 0},
		{100, 100, 100},
		{100, 100, 100},
		{0, 0, 0},
	};

	bool bIsClosed = true;
	bool bLinear = true;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = true;
	Settings->bRemoveCollinearControlPoints = false;
	Settings->FuseMode = EPCGControlPointFuseMode::KeepSecond;
	Settings->ColocationDistanceThreshold = 1.0;

	TArray<int32> ExpectedOutputPoints = {1, 3};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints));
}

bool FPCGCleanSplineTest_CollocatedControlPoints_Closed_Merge::RunTest(const FString& Parameters)
{
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{10, 0, 0},
		{100, 0, 0},
		{120, 0, 0},
		{-10, 0, 0},
	};

	bool bIsClosed = true;
	bool bLinear = true;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = true;
	Settings->bRemoveCollinearControlPoints = false;
	Settings->FuseMode = EPCGControlPointFuseMode::Merge;
	Settings->ColocationDistanceThreshold = 21.0;

	TArray<int32> ExpectedOutputPoints = {2, 4};
	TArray<FVector> ExpectedPositions =
	{
		{110, 0, 0},
		{-2.5, 0, 0}
	};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints), &ExpectedPositions);
}

bool FPCGCleanSplineTest_CollinearControlPoints_Linear::RunTest(const FString& Parameters)
{
	/**
	 *          4
	 *          |
	 *          3
	 *        2
	 *      1
	 *    0
	 *
	 * becomes
	 *          4
	 *          |
	 *          3
	 *        / 
	 *      / 
	 *    0
	 */
	
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{100, 100, 100},
		{200, 200, 200},
		{300, 300, 300},
		{300, 500, 500}
	};

	bool bIsClosed = false;
	bool bLinear = true;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = false;
	Settings->bRemoveCollinearControlPoints = true;

	TArray<int32> ExpectedOutputPoints = {0, 3, 4};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints));
}

bool FPCGCleanSplineTest_CollinearControlPoints_Linear_Closed::RunTest(const FString& Parameters)
{
	/**
	 *   5----------- 4
	 *   |            |
	 *   |            3
	 *   |          2
	 *   |        1
	 *   |      0
	 *   |    /
	 *   |  7
	 *   6
	 *
	 * becomes
	 *   5----------- 4
	 *   |            |
	 *   |            3
	 *   |          /
	 *   |        /
	 *   |      /
	 *   |    /
	 *   |  /
	 *   6
	 */
	
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{100, 100, 0},
		{200, 200, 0},
		{300, 300, 0},
		{300, 500, 0},
		{-300, 500, 0},
		{-300, -300, 0},
		{-200, -200, 0},
	};

	bool bIsClosed = true;
	bool bLinear = true;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = false;
	Settings->bRemoveCollinearControlPoints = true;

	TArray<int32> ExpectedOutputPoints = {3, 4, 5, 6};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints));
}

bool FPCGCleanSplineTest_CollinearControlPoints_Curve::RunTest(const FString& Parameters)
{
	/**
	 *	 With curves, tangent in 3 is different, so 1,2,3 aren't collinear anymore.
	 *	 
	 *          4
	 *          |
	 *          3
	 *        2
	 *      1
	 *    0
	 *
	 * becomes
	 *          4
	 *          |
	 *          3
	 *        2 
	 *      / 
	 *    0
	 */
	
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{100, 100, 0},
		{200, 200, 0},
		{300, 300, 0},
		{300, 500, 0}
	};

	bool bIsClosed = false;
	bool bLinear = false;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = false;
	Settings->bRemoveCollinearControlPoints = true;

	TArray<int32> ExpectedOutputPoints = {0, 2, 3, 4};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints));
}

bool FPCGCleanSplineTest_CollinearControlPoints_Curve_Closed::RunTest(const FString& Parameters)
{
	/**
	 *   With curves, tangents in 3 and 6 are different, so 1,2,3 and 6,7,0 aren't collinear anymore.
	 *   
	 *   5----------- 4
	 *   |            |
	 *   |            3
	 *   |          2
	 *   |        1
	 *   |      0
	 *   |    /
	 *   |  7
	 *   6
	 *
	 * becomes
	 *   5----------- 4
	 *   |            |
	 *   |            3
	 *   |          2
	 *   |        /
	 *   |      /
	 *   |    /
	 *   |  7
	 *   6
	 */
	
	TArray<FVector> Points =
	{
		{0, 0, 0},
		{100, 100, 0},
		{200, 200, 0},
		{300, 300, 0},
		{300, 500, 0},
		{-300, 500, 0},
		{-300, -300, 0},
		{-200, -200, 0},
	};

	bool bIsClosed = true;
	bool bLinear = false;

	UPCGCleanSplineSettings* Settings = NewObject<UPCGCleanSplineSettings>(GetTransientPackage(), NAME_None, RF_Transient);
	Settings->bFuseColocatedControlPoints = false;
	Settings->bRemoveCollinearControlPoints = true;

	TArray<int32> ExpectedOutputPoints = {2, 3, 4, 5, 6, 7};

	return RunTestInternal(Settings, Points, bLinear, bIsClosed, MoveTemp(ExpectedOutputPoints));
}

#endif // WITH_EDITOR