// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"
#include "Elements/PCGCollapseElement.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCollapseElementSmokeTest, FPCGTestBaseClass, "Plugins.PCG.Collapse.Smoke", PCGTestsCommon::TestFlags)

bool FPCGCollapseElementSmokeTest::RunTest(const FString& Parameters)
{
	UPCGCollapseSettings* Settings = NewObject<UPCGCollapseSettings>();
	return SmokeTestAnyValidInput(Settings);
}