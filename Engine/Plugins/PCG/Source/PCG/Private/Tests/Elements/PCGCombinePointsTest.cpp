// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"

#include "Elements/PCGCombinePoints.h"

namespace CombinePointsCommonTests
{
	TUniquePtr<FPCGContext> GenerateTestDataAndRun(bool CenterPivot, bool UseSourceTransform, FTransform PtTransform)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPCGCombinePointsSettings>(TestData);
		UPCGCombinePointsSettings* Settings = CastChecked<UPCGCombinePointsSettings>(TestData.Settings);
		Settings->bCenterPivot = CenterPivot;
		Settings->bUseFirstPointTransform = UseSourceTransform;
		Settings->PointTransform = PtTransform;

		FPCGTaggedData& Inputs = TestData.InputData.TaggedData.Emplace_GetRef();
		Inputs.Pin = PCGPinConstants::DefaultInputLabel;
		UPCGBasePointData* InData = PCGTestsCommon::CreateRandomBasePointData(5, 42, false);
		Inputs.Data = InData;

		// Setting the transform explicitly
		TPCGValueRange<FTransform> TransformRange = InData->GetTransformValueRange();
		for (int i = 0; i < InData->GetNumPoints(); ++i)
		{
			TransformRange[i].SetLocation(FVector(10.0, 10.0, 10.0) * i);
			TransformRange[i].SetRotation(FQuat::Identity);
			TransformRange[i].SetScale3D(FVector::One() * i + 1);
		}

		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		return Context;
	}
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCombinePointsTest_Default, FPCGTestBaseClass, "Plugins.PCG.CombinePoints.Default", PCGTestsCommon::TestFlags)

bool FPCGCombinePointsTest_Default::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CombinePointsCommonTests::GenerateTestDataAndRun(true, true, FTransform());

	// Test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("OutputA point data", OutPointData);

	UTEST_EQUAL("OutputA point count", OutPointData->GetNumPoints(), 1);

	/** Point[0] at(0)->bounds(-1, 1)
	 * Point [1] at (10) -> bounds (8, 12)
	 * Point [2] at (20) -> bounds (17, 23)
	 * Point [3] at (30) -> bounds (26, 34)
	 * Point [4] at (40) -> bounds (35, 45)
	 * 45 + (-1) / 2 == 22 <- check center pivot
	 * 45 + |-1| / 2 == 23 <- check extents */

	const FConstPCGPointValueRanges OutRanges(OutPointData);
	UTEST_EQUAL("Output Location", OutRanges.TransformRange[0].GetLocation(), FVector(22.0));
	UTEST_EQUAL("Output Bounds Min", OutRanges.BoundsMinRange[0], FVector(-23.0));
	UTEST_EQUAL("Output Bounds Max", OutRanges.BoundsMaxRange[0], FVector(23.0));
	
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCombinePointsTest_SourceTransform, FPCGTestBaseClass, "Plugins.PCG.CombinePoints.SourceTransform", PCGTestsCommon::TestFlags)

bool FPCGCombinePointsTest_SourceTransform::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CombinePointsCommonTests::GenerateTestDataAndRun(false, true, FTransform());

	// Test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("OutputA point data", OutPointData);

	UTEST_EQUAL("OutputA point count", OutPointData->GetNumPoints(), 1);

	const FConstPCGPointValueRanges OutRanges(OutPointData);
	UTEST_EQUAL("Output Location", OutRanges.TransformRange[0].GetLocation(), FVector(0.0));
	UTEST_EQUAL("Output Bounds Min", OutRanges.BoundsMinRange[0], FVector(-1.0));
	UTEST_EQUAL("Output Bounds Max", OutRanges.BoundsMaxRange[0], FVector(45.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCombinePointsTest_TransformLocation, FPCGTestBaseClass, "Plugins.PCG.CombinePoints.TransformLocation", PCGTestsCommon::TestFlags)

bool FPCGCombinePointsTest_TransformLocation::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CombinePointsCommonTests::GenerateTestDataAndRun(false, false, FTransform(FRotator::ZeroRotator, FVector(10.0), FVector::One()));

	// Test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("OutputA point data", OutPointData);

	UTEST_EQUAL("OutputA point count", OutPointData->GetNumPoints(), 1);

	// Has min at -1, max at 45 at origin
	// Bound extents shift by (10) 
	const FConstPCGPointValueRanges OutRanges(OutPointData);
	UTEST_EQUAL("Output Location", OutRanges.TransformRange[0].GetLocation(), FVector(10.0));
	UTEST_EQUAL("Output Bounds Min", OutRanges.BoundsMinRange[0], FVector(-11.0));
	UTEST_EQUAL("Output Bounds Max", OutRanges.BoundsMaxRange[0], FVector(35.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCombinePointsTest_TransformRotation, FPCGTestBaseClass, "Plugins.PCG.CombinePoints.TransformRotation", PCGTestsCommon::TestFlags)

bool FPCGCombinePointsTest_TransformRotation::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CombinePointsCommonTests::GenerateTestDataAndRun(false, false, FTransform(FRotator(90.0, 0.0, 0.0), FVector::ZeroVector, FVector::One()));

	// Test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("OutputA point data", OutPointData);

	UTEST_EQUAL("OutputA point count", OutPointData->GetNumPoints(), 1);

	// Rotate a cube around X axis
	const FConstPCGPointValueRanges OutRanges(OutPointData);
	UTEST_EQUAL("Output Location", OutRanges.TransformRange[0].GetLocation(), FVector::ZeroVector);
	UTEST_EQUAL("Output Bounds Min", OutRanges.BoundsMinRange[0], FVector(-1.0, -1.0, -45.0));
	UTEST_EQUAL("Output Bounds Max", OutRanges.BoundsMaxRange[0], FVector(45.0, 45.0, 1.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCombinePointsTest_TransformScale, FPCGTestBaseClass, "Plugins.PCG.CombinePoints.TransformScale", PCGTestsCommon::TestFlags)

bool FPCGCombinePointsTest_TransformScale::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CombinePointsCommonTests::GenerateTestDataAndRun(false, false, FTransform(FRotator::ZeroRotator, FVector::ZeroVector, FVector(2.0)));

	// Test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("OutputA point data", OutPointData);

	UTEST_EQUAL("OutputA point count", OutPointData->GetNumPoints(), 1);

	const FConstPCGPointValueRanges OutRanges(OutPointData);
	UTEST_EQUAL("Output Location", OutRanges.TransformRange[0].GetLocation(), FVector::ZeroVector);
	UTEST_EQUAL("Output Bounds Min", OutRanges.BoundsMinRange[0], FVector(-0.5));
	UTEST_EQUAL("Output Bounds Max", OutRanges.BoundsMaxRange[0], FVector(22.5));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCombinePointsTest_TransformNegScale, FPCGTestBaseClass, "Plugins.PCG.CombinePoints.TransformNegScale", PCGTestsCommon::TestFlags)

bool FPCGCombinePointsTest_TransformNegScale::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CombinePointsCommonTests::GenerateTestDataAndRun(false, false, FTransform(FRotator::ZeroRotator, FVector::ZeroVector, FVector(-2.0)));

	// Test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("OutputA point data", OutPointData);

	UTEST_EQUAL("OutputA point count", OutPointData->GetNumPoints(), 1);

	const FConstPCGPointValueRanges OutRanges(OutPointData);
	UTEST_EQUAL("Output Location", OutRanges.TransformRange[0].GetLocation(), FVector::ZeroVector);
	UTEST_EQUAL("Output Bounds Min", OutRanges.BoundsMinRange[0], FVector(-22.5));
	UTEST_EQUAL("Output Bounds Max", OutRanges.BoundsMaxRange[0], FVector(0.5));

	return true;
}