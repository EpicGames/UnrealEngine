// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include "Elements/PCGCopyAttributes.h"
#include "Elements/PCGCopyPoints.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_Points_PropertyToProperty, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.Points.PropertyToProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_Points_PropertyToAttribute, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.Points.PropertyToAttribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_Points_AttributeToProperty, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.Points.AttributeToProperty", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_Points_AttributeToAttribute, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.Points.AttributeToAttribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_Points_CopyingToItself, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.Points.CopyingToItself", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_Points_CopyingAllToItself, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.Points.CopyingAllToItself", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_Params_SingleValue, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.Params.SingleValue", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_Params_MultiValue, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.Params.MultiValue", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_Params_CopyingToItself, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.Params.CopyingToItself", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_Params_CopyingAllToItself, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.Params.CopyingAllToItself", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_DataToData, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.DataToData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_DataToElements, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.DataToElements", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_ElementsToData, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.ElementsToData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_ElementsToDataTooMany, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.ElementsToDataTooMany", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_InvalidDomain, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.InvalidDomain", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_CopyAll_DataToData, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.CopyAll.DataToData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_CopyAll_DataToElements, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.CopyAll.DataToElements", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_CopyAll_ElementsToData, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.CopyAll.ElementsToData", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_CopyAll_ElementsToDataTooMany, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.CopyAll.ElementsToDataTooMany", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_CopyAll_ElementsToElementsExplicit, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.CopyAll.ElementsToElementsExplicit", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_CopyAll_ElementsToElementsDefault, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.CopyAll.ElementsToElementsDefault", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_CopyAll_AllToAll, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.CopyAll.AllToAll", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyAttributeTests_MultiDomain_CopyAll_InvalidDomain, FPCGTestBaseClass, "Plugins.PCG.CopyAttribute.MultiDomain.CopyAll.InvalidDomain", PCGTestsCommon::TestFlags)

namespace PCGCopyAttributeTests
{
	static const FName AttributeName = TEXT("InputAttr");

	UPCGBasePointData* CreateInputPointData(FPCGContext* Context, const int NumPoints)
	{
		check(Context);

		UPCGBasePointData* NewPointData = FPCGContext::NewPointData_AnyThread(Context);
		NewPointData->SetFlags(RF_Transient);

		check(NewPointData);
		
		NewPointData->SetNumPoints(NumPoints);
		NewPointData->AllocateProperties(EPCGPointNativeProperties::Density);

		TPCGValueRange<float> DensityValues = NewPointData->GetDensityValueRange();
		for (int i = 0; i < NumPoints; ++i)
		{
			// Store the index in the density
			DensityValues[i] = i;
		}

		return NewPointData;
	}

	UPCGParamData* CreateInputParamData(FPCGContext* Context)
	{
		check(Context);

		UPCGParamData* NewParamData = NewObject<UPCGParamData>();
		NewParamData->SetFlags(RF_Transient);

		check(NewParamData);
		return NewParamData;
	}

	void ConnectToSource(FPCGContext* Context, const UPCGData* InData)
	{
		FPCGTaggedData& InputData = Context->InputData.TaggedData.Emplace_GetRef();
		InputData.Data = InData;
		InputData.Pin = PCGCopyPointsConstants::SourcePointsLabel;
	}

	void ConnectToTarget(FPCGContext* Context, const UPCGData* InData)
	{
		FPCGTaggedData& InputData = Context->InputData.TaggedData.Emplace_GetRef();
		InputData.Data = InData;
		InputData.Pin = PCGCopyPointsConstants::TargetPointsLabel;
	}

	void ConnectToSourceAndTarget(FPCGContext* Context, const UPCGData* InData)
	{
		ConnectToSource(Context, InData);
		ConnectToTarget(Context, InData);
	}
}

bool FPCGCopyAttributeTests_Points_PropertyToProperty::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	// Write Density in Position.X
	Settings->InputSource.SetPointProperty(EPCGPointProperties::Density);
	Settings->OutputTarget.Update(TEXT("$Position.X"));

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData);

	// Density was copied in the position correctly
	TConstPCGValueRange<float> DensityRange = OutputData->GetConstDensityValueRange();
	TConstPCGValueRange<FTransform> TransformRange = OutputData->GetConstTransformValueRange();
	for (int i = 0; i < NumOfPoints; ++i)
	{
		UTEST_EQUAL(*FString::Printf(TEXT("Position.X has the same value as density for point %d"), i), TransformRange[i].GetLocation().X, static_cast<double>(DensityRange[i]));
	}

	return true;
}

bool FPCGCopyAttributeTests_Points_PropertyToAttribute::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;
	const FName OutputAttributeName = TEXT("OutputAttr");

	// Write Density in Attribute
	Settings->InputSource.SetPointProperty(EPCGPointProperties::Density);
	Settings->OutputTarget.SetAttributeName(OutputAttributeName);

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->Metadata);
	const FPCGMetadataAttribute<double>* OutputAttribute = OutputData->Metadata->GetConstTypedAttribute<double>(OutputAttributeName);
	UTEST_NOT_NULL("Output Attribute exists in the output data", OutputAttribute);

	check(OutputAttribute);

	// Attribute was copied in the position correctly
	TConstPCGValueRange<float> DensityRange = OutputData->GetConstDensityValueRange();
	TConstPCGValueRange<PCGMetadataEntryKey> MetadataEntryRange = OutputData->GetConstMetadataEntryValueRange();
	for (int i = 0; i < NumOfPoints; ++i)
	{
		UTEST_EQUAL(*FString::Printf(TEXT("Output attribute has the same value than density for point %d"), i), OutputAttribute->GetValueFromItemKey(MetadataEntryRange[i]), static_cast<double>(DensityRange[i]));
	}

	return true;
}

bool FPCGCopyAttributeTests_Points_AttributeToProperty::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	// Write Attribute in Position
	Settings->InputSource.SetAttributeName(PCGCopyAttributeTests::AttributeName);
	Settings->OutputTarget.Update(TEXT("$Position"));

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGTestsCommon::CreateAndFillRandomAttribute<FVector>(InputPointData, PCGCopyAttributeTests::AttributeName, FVector::ZeroVector, NumOfPoints);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->Metadata);
	const FPCGMetadataAttribute<FVector>* InputAttribute = OutputData->Metadata->GetConstTypedAttribute<FVector>(PCGCopyAttributeTests::AttributeName);
	UTEST_NOT_NULL("Input Attribute exists in the output data", InputAttribute);

	// Attribute value was copied in Position correctly
	TConstPCGValueRange<FTransform> TransformRange = OutputData->GetConstTransformValueRange();
	TConstPCGValueRange<PCGMetadataEntryKey> MetadataEntryRange = OutputData->GetConstMetadataEntryValueRange();
	for (int i = 0; i < NumOfPoints; ++i)
	{
		UTEST_EQUAL(*FString::Printf(TEXT("Position has the same value than the input attribute for point %d"), i), TransformRange[i].GetLocation(), InputAttribute->GetValueFromItemKey(MetadataEntryRange[i]));
	}

	return true;
}

bool FPCGCopyAttributeTests_Points_AttributeToAttribute::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;
	const FName OutputAttributeName = TEXT("OutputAttr");

	// Write Input attribute to Output attribute, testing with strings
	Settings->InputSource.SetAttributeName(PCGCopyAttributeTests::AttributeName);
	Settings->OutputTarget.SetAttributeName(OutputAttributeName);

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGTestsCommon::CreateAndFillRandomAttribute<FString>(InputPointData, PCGCopyAttributeTests::AttributeName, FString(), NumOfPoints);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->Metadata);

	const FPCGMetadataAttribute<FString>* InputAttribute = OutputData->Metadata->GetConstTypedAttribute<FString>(PCGCopyAttributeTests::AttributeName);
	UTEST_NOT_NULL("Input Attribute exists in the output data", InputAttribute);
	const FPCGMetadataAttribute<FString>* OutputAttribute = OutputData->Metadata->GetConstTypedAttribute<FString>(OutputAttributeName);
	UTEST_NOT_NULL("Output Attribute exists in the output data", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default value of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));

	// Input attribute was copied in the output attribute correctly
	TConstPCGValueRange<PCGMetadataEntryKey> MetadataEntryRange = OutputData->GetConstMetadataEntryValueRange();
	for (int i = 0; i < NumOfPoints; ++i)
	{
		UTEST_EQUAL(*FString::Printf(TEXT("Output Attribute has the same value as Input Attribute for point %d"), i), OutputAttribute->GetValueFromItemKey(MetadataEntryRange[i]), InputAttribute->GetValueFromItemKey(MetadataEntryRange[i]));
	}

	return true;
}

bool FPCGCopyAttributeTests_Points_CopyingToItself::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	// Write Attribute in itself
	Settings->InputSource.SetAttributeName(PCGCopyAttributeTests::AttributeName);
	Settings->OutputTarget.SetAttributeName(PCGCopyAttributeTests::AttributeName);

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	// Need constness for UTEST_EQUAL....
	const UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGTestsCommon::CreateAndFillRandomAttribute<double>(const_cast<UPCGBasePointData*>(InputPointData), PCGCopyAttributeTests::AttributeName, 0.0, NumOfPoints);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("It's the same data as input", OutputData, InputPointData);

	return true;
}

bool FPCGCopyAttributeTests_Points_CopyingAllToItself::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->bCopyAllAttributes = true;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGTestsCommon::CreateAndFillRandomAttribute<double>(const_cast<UPCGBasePointData*>(InputPointData), PCGCopyAttributeTests::AttributeName, 0.0, NumOfPoints);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("It's the same data as input", OutputData, InputPointData);

	return true;
}

bool FPCGCopyAttributeTests_Params_SingleValue::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	const FName OutputAttributeName = TEXT("OutputAttr");

	// Write Input attribute to Output attribute
	Settings->InputSource.SetAttributeName(PCGCopyAttributeTests::AttributeName);
	Settings->OutputTarget.SetAttributeName(OutputAttributeName);

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGParamData* InputParamData = PCGCopyAttributeTests::CreateInputParamData(Context.Get());
	PCGTestsCommon::CreateAndFillRandomAttribute<FRotator>(const_cast<UPCGParamData*>(InputParamData), PCGCopyAttributeTests::AttributeName, FRotator::ZeroRotator, 1);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputParamData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGParamData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGParamData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a param data in output points", OutputData);
	UTEST_EQUAL("There is the right number of entries in output metadata", OutputData->Metadata->GetItemCountForChild(), 1);

	check(OutputData && OutputData->Metadata);

	const FPCGMetadataAttribute<FRotator>* InputAttribute = OutputData->Metadata->GetConstTypedAttribute<FRotator>(PCGCopyAttributeTests::AttributeName);
	UTEST_NOT_NULL("Input Attribute exists in the output data", InputAttribute);
	const FPCGMetadataAttribute<FRotator>* OutputAttribute = OutputData->Metadata->GetConstTypedAttribute<FRotator>(OutputAttributeName);
	UTEST_NOT_NULL("Output Attribute exists in the output data", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));
	UTEST_EQUAL("Output Attribute has the same value as Input Attribute for entry 0", OutputAttribute->GetValueFromItemKey(PCGFirstEntryKey), InputAttribute->GetValueFromItemKey(PCGFirstEntryKey));

	return true;
}

bool FPCGCopyAttributeTests_Params_MultiValue::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumEntries = 20;
	const FName OutputAttributeName = TEXT("OutputAttr");

	// Write Input attribute to Output attribute
	Settings->InputSource.SetAttributeName(PCGCopyAttributeTests::AttributeName);
	Settings->OutputTarget.SetAttributeName(OutputAttributeName);

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGParamData* InputParamData = PCGCopyAttributeTests::CreateInputParamData(Context.Get());
	PCGTestsCommon::CreateAndFillRandomAttribute<FSoftObjectPath>(const_cast<UPCGParamData*>(InputParamData), PCGCopyAttributeTests::AttributeName, FSoftObjectPath(), NumEntries);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputParamData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGParamData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGParamData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a param data in output points", OutputData);
	UTEST_EQUAL("There is the right number of entries in output metadata", OutputData->Metadata->GetItemCountForChild(), NumEntries);

	check(OutputData && OutputData->Metadata);

	const FPCGMetadataAttribute<FSoftObjectPath>* InputAttribute = OutputData->Metadata->GetConstTypedAttribute<FSoftObjectPath>(PCGCopyAttributeTests::AttributeName);
	UTEST_NOT_NULL("Input Attribute exists in the output data", InputAttribute);
	const FPCGMetadataAttribute<FSoftObjectPath>* OutputAttribute = OutputData->Metadata->GetConstTypedAttribute<FSoftObjectPath>(OutputAttributeName);
	UTEST_NOT_NULL("Output Attribute exists in the output data", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default value of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));

	// Input attribute was copied in the output attribute correctly
	for (int i = 0; i < NumEntries; ++i)
	{
		UTEST_EQUAL(*FString::Printf(TEXT("Output Attribute has the same value as Input Attribute for entry %d"), i), OutputAttribute->GetValueFromItemKey(PCGMetadataEntryKey(i)), InputAttribute->GetValueFromItemKey(PCGMetadataEntryKey(i)));
	}

	return true;
}

bool FPCGCopyAttributeTests_Params_CopyingToItself::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	// Write Input attribute to Output attribute
	Settings->InputSource.SetAttributeName(PCGCopyAttributeTests::AttributeName);
	Settings->OutputTarget.SetAttributeName(PCGCopyAttributeTests::AttributeName);

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGParamData* InputParamData = PCGCopyAttributeTests::CreateInputParamData(Context.Get());
	PCGTestsCommon::CreateAndFillRandomAttribute<double>(const_cast<UPCGParamData*>(InputParamData), PCGCopyAttributeTests::AttributeName, 0.0, 1);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputParamData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGParamData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGParamData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a param data in output points", OutputData);
	UTEST_EQUAL("It's the same data as input", OutputData, InputParamData);

	return true;
}

bool FPCGCopyAttributeTests_Params_CopyingAllToItself::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	Settings->bCopyAllAttributes = true;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGParamData* InputParamData = PCGCopyAttributeTests::CreateInputParamData(Context.Get());
	PCGTestsCommon::CreateAndFillRandomAttribute<double>(const_cast<UPCGParamData*>(InputParamData), PCGCopyAttributeTests::AttributeName, 0.0, 1);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputParamData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGParamData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGParamData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a param data in output points", OutputData);
	UTEST_EQUAL("It's the same data as input", OutputData, InputParamData);

	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_DataToData::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->InputSource.Update(TEXT("@Data.MyAttr"));
	Settings->OutputTarget.Update(TEXT("@Data.MyAttr2"));

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	InputPointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->CreateAttribute<int32>(TEXT("MyAttr"), /*DefaultValue=*/5, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);

	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->ConstMetadata());

	const FPCGMetadataDomain* DataDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data);
	check(DataDomain);

	const FPCGMetadataAttribute<int32>* InputAttribute = DataDomain->GetConstTypedAttribute<int32>(TEXT("MyAttr"));
	UTEST_NOT_NULL("Input Attribute exists in the output data", InputAttribute);
	const FPCGMetadataAttribute<int32>* OutputAttribute = DataDomain->GetConstTypedAttribute<int32>(TEXT("MyAttr2"));
	UTEST_NOT_NULL("Output Attribute exists in the output data", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default value of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));

	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_DataToElements::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->InputSource.Update(TEXT("@Data.MyAttr"));
	Settings->OutputTarget.Update(TEXT("MyAttr2"));

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	InputPointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->CreateAttribute<int32>(TEXT("MyAttr"), /*DefaultValue=*/5, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	InputPointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->AddEntry();

	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->ConstMetadata());

	const FPCGMetadataDomain* ElementsDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements);
	const FPCGMetadataDomain* DataDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data);
	check(DataDomain && ElementsDomain);

	const FPCGMetadataAttribute<int32>* InputAttribute = DataDomain->GetConstTypedAttribute<int32>(TEXT("MyAttr"));
	UTEST_NOT_NULL("Input Attribute exists in the output data metadata domain", InputAttribute);
	const FPCGMetadataAttribute<int32>* OutputAttribute = ElementsDomain->GetConstTypedAttribute<int32>(TEXT("MyAttr2"));
	UTEST_NOT_NULL("Output Attribute exists in the output elements metadata domain", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default value of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));

	// Input attribute was copied in the output attribute correctly
	TConstPCGValueRange<PCGMetadataEntryKey> MetadataEntryRange = OutputData->GetConstMetadataEntryValueRange();
	for (int i = 0; i < NumOfPoints; ++i)
	{
		UTEST_EQUAL(*FString::Printf(TEXT("Output Attribute has the same value as Input Attribute for point %d"), i), OutputAttribute->GetValueFromItemKey(MetadataEntryRange[i]), InputAttribute->GetValueFromItemKey(0));
	}
	
	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_ElementsToData::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 1;

	Settings->InputSource.Update(TEXT("MyAttr"));
	Settings->OutputTarget.Update(TEXT("@Data.MyAttr2"));

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	FPCGMetadataDomain* InputElementsDomain = InputPointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements);
	check(InputElementsDomain);
	FPCGMetadataAttribute<int32>* Attribute = InputElementsDomain->CreateAttribute<int32>(TEXT("MyAttr"), /*DefaultValue=*/5, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	PCGMetadataEntryKey EntryKey = PCGInvalidEntryKey;
	InputElementsDomain->InitializeOnSet(EntryKey);
	Attribute->SetValue(EntryKey, 5);
	InputPointData->SetMetadataEntry(EntryKey);

	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->ConstMetadata());

	const FPCGMetadataDomain* ElementsDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements);
	const FPCGMetadataDomain* DataDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data);
	check(DataDomain && ElementsDomain);

	const FPCGMetadataAttribute<int32>* InputAttribute = ElementsDomain->GetConstTypedAttribute<int32>(TEXT("MyAttr"));
	UTEST_NOT_NULL("Input Attribute exists in the output elements metadata domain", InputAttribute);
	const FPCGMetadataAttribute<int32>* OutputAttribute = DataDomain->GetConstTypedAttribute<int32>(TEXT("MyAttr2"));
	UTEST_NOT_NULL("Output Attribute exists in the output data metadata domain", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default value of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));
	UTEST_EQUAL("Output Attribute value is the same as value of Input Attribute", OutputAttribute->GetValueFromItemKey(0), InputAttribute->GetValueFromItemKey(0));
	
	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_ElementsToDataTooMany::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->InputSource.Update(TEXT("MyAttr"));
	Settings->OutputTarget.Update(TEXT("@Data.MyAttr2"));

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGTestsCommon::CreateAndFillRandomAttribute<double>(InputPointData, TEXT("MyAttr"), 0.0, NumOfPoints);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);
	
	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->ConstMetadata());

	const FPCGMetadataDomain* ElementsDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements);
	const FPCGMetadataDomain* DataDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data);
	check(DataDomain && ElementsDomain);

	const FPCGMetadataAttribute<double>* InputAttribute = ElementsDomain->GetConstTypedAttribute<double>(TEXT("MyAttr"));
	UTEST_NOT_NULL("Input Attribute exists in the output elements metadata domain", InputAttribute);
	const FPCGMetadataAttribute<double>* OutputAttribute = DataDomain->GetConstTypedAttribute<double>(TEXT("MyAttr2"));
	UTEST_NOT_NULL("Output Attribute exists in the output data metadata domain", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default value of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));

	TConstPCGValueRange<PCGMetadataEntryKey> EntryKeyRange = OutputData->GetConstMetadataEntryValueRange();
	UTEST_EQUAL("Output Attribute value is the same as the first value of Input Attribute", OutputAttribute->GetValueFromItemKey(0), InputAttribute->GetValueFromItemKey(EntryKeyRange[0]));
	
	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_InvalidDomain::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->InputSource.Update(TEXT("MyAttr"));
	Settings->OutputTarget.Update(TEXT("@Blablabla.MyAttr2"));

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* InputPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGTestsCommon::CreateAndFillRandomAttribute<double>(const_cast<UPCGBasePointData*>(InputPointData), TEXT("MyAttr"), 0.0, NumOfPoints);
	PCGCopyAttributeTests::ConnectToSourceAndTarget(Context.Get(), InputPointData);
	
	AddExpectedError(TEXT("Metadata domain Blablabla is invalid for this data."), EAutomationExpectedErrorFlags::MatchType::Contains, 1, /*bIsRegex=*/false);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("It's the same data as input", OutputData, InputPointData);
	
	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_CopyAll_DataToData::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);
	
	Settings->MetadataDomainsMapping.Emplace(PCGDataConstants::DataDomainName, PCGDataConstants::DataDomainName);
	Settings->bCopyAllAttributes = true;

	constexpr int NumOfPoints = 20;
	
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* TargetPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	UPCGParamData* SourceParamData = PCGCopyAttributeTests::CreateInputParamData(Context.Get());
	
	TargetPointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->CreateAttribute<int32>(TEXT("MyAttr"), /*DefaultValue=*/5, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	TargetPointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->AddEntry();

	const FPCGMetadataAttribute<int32>* SourceAttribute = SourceParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->CreateAttribute<int32>(TEXT("MyAttr2"), /*DefaultValue=*/6, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	SourceParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->AddEntry();

	PCGCopyAttributeTests::ConnectToSource(Context.Get(), SourceParamData);
	PCGCopyAttributeTests::ConnectToTarget(Context.Get(), TargetPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->ConstMetadata());

	const FPCGMetadataDomain* ElementsDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements);
	const FPCGMetadataDomain* DataDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data);
	check(DataDomain && ElementsDomain);

	UTEST_EQUAL("No attributes on the output elements metadata domain", ElementsDomain->GetAttributeCount(), 0);

	const FPCGMetadataAttribute<int32>* ExistingAttribute = DataDomain->GetConstTypedAttribute<int32>(TEXT("MyAttr"));
	UTEST_NOT_NULL("Existing attribute is still there in the output data metadata domain", ExistingAttribute);
	
	const FPCGMetadataAttribute<int32>* Attribute = DataDomain->GetConstTypedAttribute<int32>(TEXT("MyAttr2"));
	UTEST_NOT_NULL("Source Attribute exists in the output data metadata domain", Attribute);

	check(Attribute);

	UTEST_EQUAL("Target Attribute default value is the same as default value of Source Attribute", Attribute->GetValueFromItemKey(PCGInvalidEntryKey), SourceAttribute->GetValueFromItemKey(PCGInvalidEntryKey));
	UTEST_EQUAL("Target Attribute value is the same as value of Source Attribute", Attribute->GetValueFromItemKey(PCGFirstEntryKey), SourceAttribute->GetValueFromItemKey(PCGFirstEntryKey));
	
	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_CopyAll_AllToAll::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	Settings->bCopyAllDomains = true;
	Settings->bCopyAllAttributes = true;

	constexpr int NumOfPoints = 20;
	
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGParamData* TargetParamData = PCGCopyAttributeTests::CreateInputParamData(Context.Get());
	UPCGBasePointData* SourcePointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	
	TargetParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->CreateAttribute<int32>(TEXT("MyDataAttr"), 5, false, false);
	TargetParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->AddEntry();

	PCGTestsCommon::CreateAndFillRandomAttribute<double>(TargetParamData, TEXT("MyAttr"), 0.0, NumOfPoints);

	const FPCGMetadataAttribute<int32>* SourceAttribute = SourcePointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->CreateAttribute<int32>(TEXT("MyDataAttr2"), /*DefaultValue=*/6, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	SourcePointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->AddEntry();

	PCGTestsCommon::CreateAndFillRandomAttribute<double>(SourcePointData, TEXT("MyAttr2"), 0.0, NumOfPoints);
	const FPCGMetadataAttribute<double>* SourceElementsAttribute = SourcePointData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements)->GetConstTypedAttribute<double>(TEXT("MyAttr2"));
	check(SourceElementsAttribute);

	PCGCopyAttributeTests::ConnectToSource(Context.Get(), SourcePointData);
	PCGCopyAttributeTests::ConnectToTarget(Context.Get(), TargetParamData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGParamData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGParamData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a param data in output points", OutputData);

	check(OutputData && OutputData->ConstMetadata());

	const FPCGMetadataDomain* ElementsDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements);
	const FPCGMetadataDomain* DataDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data);
	check(DataDomain && ElementsDomain);

	const FPCGMetadataAttribute<int32>* ExistingAttribute = DataDomain->GetConstTypedAttribute<int32>(TEXT("MyDataAttr"));
	UTEST_NOT_NULL("Existing attribute is still there in the output data metadata domain", ExistingAttribute);
	
	const FPCGMetadataAttribute<int32>* Attribute = DataDomain->GetConstTypedAttribute<int32>(TEXT("MyDataAttr2"));
	UTEST_NOT_NULL("Source Attribute exists in the output data metadata domain", Attribute);

	check(Attribute);

	UTEST_EQUAL("Target Attribute default value is the same as default value of Source Attribute", Attribute->GetValueFromItemKey(PCGInvalidEntryKey), SourceAttribute->GetValueFromItemKey(PCGInvalidEntryKey));
	UTEST_EQUAL("Target Attribute value is the same as value of Source Attribute", Attribute->GetValueFromItemKey(PCGFirstEntryKey), SourceAttribute->GetValueFromItemKey(PCGFirstEntryKey));

	const FPCGMetadataAttribute<double>* ExistingElementsAttribute = ElementsDomain->GetConstTypedAttribute<double>(TEXT("MyAttr"));
	UTEST_NOT_NULL("Existing attribute is still there in the output elements metadata domain", ExistingElementsAttribute);
	
	const FPCGMetadataAttribute<double>* TargetElementsAttribute = ElementsDomain->GetConstTypedAttribute<double>(TEXT("MyAttr2"));
	UTEST_NOT_NULL("Source Attribute exists in the output elements metadata domain", TargetElementsAttribute);

	check(TargetElementsAttribute);

	UTEST_EQUAL("Target Attribute default value is the same as default value of Source Attribute", TargetElementsAttribute->GetValueFromItemKey(PCGInvalidEntryKey), SourceElementsAttribute->GetValueFromItemKey(PCGInvalidEntryKey));
	TConstPCGValueRange<PCGMetadataEntryKey> MetadataEntryRange = SourcePointData->GetConstMetadataEntryValueRange();
	for (int32 i = 0; i < NumOfPoints; ++i)
	{
		UTEST_EQUAL(*FString::Printf(TEXT("Target Attribute has the same value as Source Attribute for index %d"), i), TargetElementsAttribute->GetValueFromItemKey(i), SourceElementsAttribute->GetValueFromItemKey(MetadataEntryRange[i]));
	}
	
	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_CopyAll_DataToElements::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->MetadataDomainsMapping.Emplace(PCGDataConstants::DataDomainName, PCGPointDataConstants::ElementsDomainName);
	Settings->bCopyAllAttributes = true;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* TargetPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	UPCGParamData* SourceParamData = PCGCopyAttributeTests::CreateInputParamData(Context.Get());

	const FPCGMetadataAttribute<int32>* InputAttribute = SourceParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->CreateAttribute<int32>(TEXT("MyDataAttr"), /*DefaultValue=*/5, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	SourceParamData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Data)->AddEntry();
	
	PCGCopyAttributeTests::ConnectToSource(Context.Get(), SourceParamData);
	PCGCopyAttributeTests::ConnectToTarget(Context.Get(), TargetPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->ConstMetadata());
	const FPCGMetadataDomain* ElementsDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Elements);
	check(ElementsDomain);
	
	const FPCGMetadataAttribute<int32>* OutputAttribute = ElementsDomain->GetConstTypedAttribute<int32>(TEXT("MyDataAttr"));
	UTEST_NOT_NULL("Output Attribute exists in the output elements metadata domain", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default value of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));
	
	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_CopyAll_ElementsToData::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->MetadataDomainsMapping.Emplace(PCGPointDataConstants::ElementsDomainName, PCGDataConstants::DataDomainName);
	Settings->bCopyAllAttributes = true;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* TargetPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	UPCGBasePointData* SourcePointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), 1);

	FPCGMetadataAttribute<int32>* InputAttribute = SourcePointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements)->CreateAttribute<int32>(TEXT("MyAttr"), /*DefaultValue=*/6, /*bAllowInterpolation=*/false, /*bOverrideParent=*/false);
	PCGMetadataEntryKey EntryKey = PCGInvalidEntryKey;
	SourcePointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements)->InitializeOnSet(EntryKey);
	InputAttribute->SetValue(EntryKey, 5);
	SourcePointData->SetMetadataEntry(EntryKey);
	
	PCGCopyAttributeTests::ConnectToSource(Context.Get(), SourcePointData);
	PCGCopyAttributeTests::ConnectToTarget(Context.Get(), TargetPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->ConstMetadata());
	const FPCGMetadataDomain* DataDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data);
	check(DataDomain);
	
	const FPCGMetadataAttribute<int32>* OutputAttribute = DataDomain->GetConstTypedAttribute<int32>(TEXT("MyAttr"));
	UTEST_NOT_NULL("Output Attribute exists in the output data metadata domain", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));
	UTEST_EQUAL("Output Attribute value is the same as value of Input Attribute", OutputAttribute->GetValueFromItemKey(0), InputAttribute->GetValueFromItemKey(0));
	
	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_CopyAll_ElementsToDataTooMany::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->MetadataDomainsMapping.Emplace(PCGPointDataConstants::ElementsDomainName, PCGDataConstants::DataDomainName);
	Settings->bCopyAllAttributes = true;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* TargetPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	UPCGBasePointData* SourcePointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);

	PCGTestsCommon::CreateAndFillRandomAttribute<double>(SourcePointData, TEXT("MyAttr"), 0.0, NumOfPoints);
	const FPCGMetadataAttribute<double>* InputAttribute = SourcePointData->MutableMetadata()->GetMetadataDomain(PCGMetadataDomainID::Elements)->GetConstTypedAttribute<double>(TEXT("MyAttr"));
	
	PCGCopyAttributeTests::ConnectToSource(Context.Get(), SourcePointData);
	PCGCopyAttributeTests::ConnectToTarget(Context.Get(), TargetPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->ConstMetadata());
	const FPCGMetadataDomain* DataDomain = OutputData->ConstMetadata()->GetConstMetadataDomain(PCGMetadataDomainID::Data);
	check(DataDomain);
	
	const FPCGMetadataAttribute<double>* OutputAttribute = DataDomain->GetConstTypedAttribute<double>(TEXT("MyAttr"));
	UTEST_NOT_NULL("Output Attribute exists in the output data metadata domain", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default value of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));

	TConstPCGValueRange<PCGMetadataEntryKey> EntryKeyRange = SourcePointData->GetConstMetadataEntryValueRange();
	UTEST_EQUAL("Output Attribute value is the same as the first value of Input Attribute", OutputAttribute->GetValueFromItemKey(0), InputAttribute->GetValueFromItemKey(EntryKeyRange[0]));
	
	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_CopyAll_ElementsToElementsExplicit::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->MetadataDomainsMapping.Emplace(PCGPointDataConstants::ElementsDomainName, PCGPointDataConstants::ElementsDomainName);
	Settings->bCopyAllAttributes = true;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* SourcePointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGTestsCommon::CreateAndFillRandomAttribute<FString>(SourcePointData, PCGCopyAttributeTests::AttributeName, FString(), NumOfPoints);
	
	UPCGBasePointData* TargetPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	
	PCGCopyAttributeTests::ConnectToSource(Context.Get(), SourcePointData);
	PCGCopyAttributeTests::ConnectToTarget(Context.Get(), TargetPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->Metadata);

	const FPCGMetadataAttribute<FString>* InputAttribute = SourcePointData->ConstMetadata()->GetConstTypedAttribute<FString>(PCGCopyAttributeTests::AttributeName);
	UTEST_NOT_NULL("Input Attribute exists in the source data", InputAttribute);
	const FPCGMetadataAttribute<FString>* OutputAttribute = OutputData->ConstMetadata()->GetConstTypedAttribute<FString>(PCGCopyAttributeTests::AttributeName);
	UTEST_NOT_NULL("Output Attribute exists in the output data", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default value of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));

	// Input attribute was copied in the output attribute correctly
	TConstPCGValueRange<PCGMetadataEntryKey> OutputMetadataEntryRange = OutputData->GetConstMetadataEntryValueRange();
	TConstPCGValueRange<PCGMetadataEntryKey> SourceMetadataEntryRange = SourcePointData->GetConstMetadataEntryValueRange();
	for (int i = 0; i < NumOfPoints; ++i)
	{
		UTEST_EQUAL(*FString::Printf(TEXT("Output Attribute has the same value as Input Attribute for point %d"), i), OutputAttribute->GetValueFromItemKey(OutputMetadataEntryRange[i]), InputAttribute->GetValueFromItemKey(SourceMetadataEntryRange[i]));
	}

	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_CopyAll_ElementsToElementsDefault::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;
	
	Settings->bCopyAllAttributes = true;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	UPCGBasePointData* SourcePointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGTestsCommon::CreateAndFillRandomAttribute<FString>(SourcePointData, PCGCopyAttributeTests::AttributeName, FString(), NumOfPoints);
	
	UPCGBasePointData* TargetPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	
	PCGCopyAttributeTests::ConnectToSource(Context.Get(), SourcePointData);
	PCGCopyAttributeTests::ConnectToTarget(Context.Get(), TargetPointData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("There is the right number of points in output", OutputData->GetNumPoints(), NumOfPoints);

	check(OutputData && OutputData->Metadata);

	const FPCGMetadataAttribute<FString>* InputAttribute = SourcePointData->ConstMetadata()->GetConstTypedAttribute<FString>(PCGCopyAttributeTests::AttributeName);
	UTEST_NOT_NULL("Input Attribute exists in the source data", InputAttribute);
	const FPCGMetadataAttribute<FString>* OutputAttribute = OutputData->ConstMetadata()->GetConstTypedAttribute<FString>(PCGCopyAttributeTests::AttributeName);
	UTEST_NOT_NULL("Output Attribute exists in the output data", OutputAttribute);

	check(InputAttribute && OutputAttribute);

	UTEST_EQUAL("Output Attribute default value is the same as default value of Input Attribute", OutputAttribute->GetValueFromItemKey(PCGInvalidEntryKey), InputAttribute->GetValueFromItemKey(PCGInvalidEntryKey));

	// Input attribute was copied in the output attribute correctly
	TConstPCGValueRange<PCGMetadataEntryKey> OutputMetadataEntryRange = OutputData->GetConstMetadataEntryValueRange();
	TConstPCGValueRange<PCGMetadataEntryKey> SourceMetadataEntryRange = SourcePointData->GetConstMetadataEntryValueRange();
	for (int i = 0; i < NumOfPoints; ++i)
	{
		UTEST_EQUAL(*FString::Printf(TEXT("Output Attribute has the same value as Input Attribute for point %d"), i), OutputAttribute->GetValueFromItemKey(OutputMetadataEntryRange[i]), InputAttribute->GetValueFromItemKey(SourceMetadataEntryRange[i]));
	}

	return true;
}

bool FPCGCopyAttributeTests_MultiDomain_CopyAll_InvalidDomain::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGCopyAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGCopyAttributesSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->MetadataDomainsMapping.Reset();
	Settings->MetadataDomainsMapping.Emplace(TEXT("Bliblibli"), TEXT("Bloubloublou"));
	Settings->bCopyAllAttributes = true;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* SourcePointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	PCGTestsCommon::CreateAndFillRandomAttribute<double>(const_cast<UPCGBasePointData*>(SourcePointData), TEXT("MyAttr"), 0.0, NumOfPoints);
	
	const UPCGBasePointData* TargetPointData = PCGCopyAttributeTests::CreateInputPointData(Context.Get(), NumOfPoints);
	
	PCGCopyAttributeTests::ConnectToSource(Context.Get(), SourcePointData);
	PCGCopyAttributeTests::ConnectToTarget(Context.Get(), TargetPointData);

	AddExpectedError(TEXT("Metadata domain Bliblibli is invalid for this data."), EAutomationExpectedErrorFlags::MatchType::Contains, 1, /*bIsRegex=*/false);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputTagged = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGBasePointData* OutputData = OutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(OutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in output points", OutputData);
	UTEST_EQUAL("It's the same data as input", OutputData, TargetPointData);
	
	return true;
}
#endif // WITH_EDITOR
