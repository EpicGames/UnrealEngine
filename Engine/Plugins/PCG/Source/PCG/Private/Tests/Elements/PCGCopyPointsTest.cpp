// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Data/PCGSpatialData.h"
#include "Elements/PCGCopyPoints.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadataAccessor.h"

#if WITH_EDITOR

class FPCGCopyPointsTestBase : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;
	virtual ~FPCGCopyPointsTestBase() = default;

protected:
	struct FParameters
	{
		EPCGCopyPointsInheritanceMode RotationInheritance = EPCGCopyPointsInheritanceMode::Relative;
		EPCGCopyPointsInheritanceMode ScaleInheritance = EPCGCopyPointsInheritanceMode::Relative;
		EPCGCopyPointsInheritanceMode ColorInheritance = EPCGCopyPointsInheritanceMode::Relative;
		EPCGCopyPointsInheritanceMode SeedInheritance = EPCGCopyPointsInheritanceMode::Relative;
		EPCGCopyPointsMetadataInheritanceMode AttributeInheritance = EPCGCopyPointsMetadataInheritanceMode::SourceFirst;
		TObjectPtr<UPCGBasePointData> SourceData = nullptr;
		TObjectPtr<UPCGBasePointData> TargetData = nullptr;
		TArray<FPCGPoint> ExpectedPoints;
	};

	bool RunTestInternal(FParameters& Parameters);
};

bool FPCGCopyPointsTestBase::RunTestInternal(FParameters& Parameters)
{
	UPCGBasePointData* SourceData = Parameters.SourceData;
	UPCGBasePointData* TargetData = Parameters.TargetData;

	check(SourceData && TargetData);

	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGCopyPointsSettings>(TestData);
	UPCGCopyPointsSettings* Settings = CastChecked<UPCGCopyPointsSettings>(TestData.Settings);
	FPCGElementPtr CopyPointsElement = TestData.Settings->GetElement();

	Settings->RotationInheritance = Parameters.RotationInheritance;
	Settings->ScaleInheritance = Parameters.ScaleInheritance;
	Settings->ColorInheritance = Parameters.ColorInheritance;
	Settings->SeedInheritance = Parameters.SeedInheritance;
	Settings->AttributeInheritance = Parameters.AttributeInheritance;

	const FName LengthName = TEXT("Length");
	const FName WidthName = TEXT("Width");
	const FName HeightName = TEXT("Height");

	const bool bAllowsInterpolation = false;
	const bool bOverrideParent = false;

	SourceData->Metadata->CreateFloatAttribute(LengthName, 0, bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateFloatAttribute(WidthName, 0, bAllowsInterpolation, bOverrideParent);
	TargetData->Metadata->CreateFloatAttribute(LengthName, 1, bAllowsInterpolation, bOverrideParent);
	TargetData->Metadata->CreateFloatAttribute(HeightName, 1, bAllowsInterpolation, bOverrideParent);

	TPCGValueRange<int64> SourceMetadataEntryRange = SourceData->GetMetadataEntryValueRange();
	FRandomStream RandomSource(TestData.Seed);
	for (int I = 0; I < SourceData->GetNumPoints(); ++I)
	{
		// first half of the points have a non-default Length attribute value, the rest have a non-default width attribute value
	 	UPCGMetadataAccessorHelpers::SetFloatAttributeByMetadataKey(SourceMetadataEntryRange[I], SourceData->Metadata, (I < SourceData->GetNumPoints() / 2) ? LengthName : WidthName, RandomSource.FRand());
	}

	TPCGValueRange<int64> TargetMetadataEntryRange = TargetData->GetMetadataEntryValueRange();
	for (int I = 0; I < TargetData->GetNumPoints(); ++I)
	{
		// first quarter of the points have a non-default Length attribute value, the rest have a non-default Height attribute value
		UPCGMetadataAccessorHelpers::SetFloatAttributeByMetadataKey(TargetMetadataEntryRange[I], TargetData->Metadata, (I < SourceData->GetNumPoints() / 4) ? LengthName : HeightName, RandomSource.FRand());
	}

	FPCGTaggedData& SourceTaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	SourceTaggedData.Data = Parameters.SourceData;
	SourceTaggedData.Pin = PCGCopyPointsConstants::SourcePointsLabel;

	FPCGTaggedData& TargetTaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TargetTaggedData.Data = Parameters.TargetData;
	TargetTaggedData.Pin = PCGCopyPointsConstants::TargetPointsLabel;

	// Test only supports float attributes for simplicity

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!CopyPointsElement->Execute(Context.Get()))
	{}

	const TArray<FPCGTaggedData> Sources = Context->InputData.GetInputsByPin(PCGCopyPointsConstants::SourcePointsLabel);
	const TArray<FPCGTaggedData> Targets = Context->InputData.GetInputsByPin(PCGCopyPointsConstants::TargetPointsLabel);
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllSpatialInputs();

	UTEST_TRUE("Valid number of inputs", Sources.Num() == 1 && Targets.Num() == 1);
	UTEST_EQUAL("Valid number of outputs", Sources.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<const UPCGBasePointData>(Outputs[0].Data);
	UTEST_NOT_NULL("Valid output point data", OutPointData);
	check(OutPointData);

	UTEST_EQUAL("Valid number of output points", SourceData->GetNumPoints() * TargetData->GetNumPoints(), OutPointData->GetNumPoints());
	check(Parameters.ExpectedPoints.IsEmpty() || Parameters.ExpectedPoints.Num() == OutPointData->GetNumPoints());

	const UPCGMetadata* RootMetadata = nullptr;
	const UPCGMetadata* NonRootMetadata = nullptr;
	EPCGCopyPointsMetadataInheritanceMode RootMode;
	EPCGCopyPointsMetadataInheritanceMode NonRootMode;

	if (Settings->AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::SourceFirst)
	{
		RootMetadata = SourceData->Metadata;
		NonRootMetadata = TargetData->Metadata;
		RootMode = EPCGCopyPointsMetadataInheritanceMode::SourceFirst;
		NonRootMode = EPCGCopyPointsMetadataInheritanceMode::TargetFirst;
	}
	else // if (Settings->AttributeInheritance == EPCGCopyPointsMetadataInheritanceMode::TargetFirst)
	{
		RootMetadata = TargetData->Metadata;
		NonRootMetadata = SourceData->Metadata;
		RootMode = EPCGCopyPointsMetadataInheritanceMode::TargetFirst;
		NonRootMode = EPCGCopyPointsMetadataInheritanceMode::SourceFirst;
	}

	UTEST_TRUE("Valid input metadata", RootMetadata && NonRootMetadata);

	check(RootMetadata && NonRootMetadata); // Static analyzer doesn't understand semantics of TestTrue.
		
	bool bTestPassed = true;

	TArray<const FPCGMetadataAttribute<float>*> InheritedAttributes;
	TArray<EPCGCopyPointsMetadataInheritanceMode> InheritedAttributeModes;
	TArray<const FPCGMetadataAttribute<float>*> OutAttributes;

	TArray<FName> AttributeNames;
	TArray<EPCGMetadataTypes> AttributeTypes;
	RootMetadata->GetAttributes(AttributeNames, AttributeTypes);

	for (const FName& AttributeName : AttributeNames)
	{
		if (TestTrue("Contains attribute found in source", OutPointData->Metadata->HasAttribute(AttributeName)))
		{
			const FPCGMetadataAttribute<float>* InheritedAttribute = static_cast<const FPCGMetadataAttribute<float>*>(RootMetadata->GetConstAttribute(AttributeName));
			const FPCGMetadataAttribute<float>* OutAttribute = static_cast<const FPCGMetadataAttribute<float>*>(OutPointData->Metadata->GetConstAttribute(AttributeName));

			InheritedAttributes.Add(InheritedAttribute);
			InheritedAttributeModes.Add(RootMode);
			OutAttributes.Add(OutAttribute);
		}
		else
		{
			return false;
		}
	}

	NonRootMetadata->GetAttributes(AttributeNames, AttributeTypes);

	for (const FName& AttributeName : AttributeNames)
	{
		if (!RootMetadata->HasAttribute(AttributeName))
		{
			if (TestTrue("Contains attribute found in target", OutPointData->Metadata->HasAttribute(AttributeName)))
			{
				const FPCGMetadataAttribute<float>* InheritedAttribute = static_cast<const FPCGMetadataAttribute<float>*>(NonRootMetadata->GetConstAttribute(AttributeName));
				const FPCGMetadataAttribute<float>* OutAttribute = static_cast<const FPCGMetadataAttribute<float>*>(OutPointData->Metadata->GetConstAttribute(AttributeName));

				InheritedAttributes.Add(InheritedAttribute);
				InheritedAttributeModes.Add(NonRootMode);
				OutAttributes.Add(OutAttribute);
			}
			else
			{
				return false;
			}
		}
	}

	check(InheritedAttributeModes.Num() == InheritedAttributeModes.Num() && InheritedAttributeModes.Num() == OutAttributes.Num())

	const FConstPCGPointValueRanges OutRanges(OutPointData);
	const FConstPCGPointValueRanges SourceRanges(SourceData);
	const FConstPCGPointValueRanges TargetRanges(TargetData);

	for (int PointIndex = 0; PointIndex < OutPointData->GetNumPoints(); ++PointIndex)
	{
		const FPCGPoint SourcePoint = SourceRanges.GetPoint(PointIndex / TargetData->GetNumPoints());
		const FPCGPoint TargetPoint = TargetRanges.GetPoint(PointIndex % TargetData->GetNumPoints());

		auto FormatMessage = [PointIndex](const FString& Message)
		{
			return FString::Printf(TEXT("Index %d: %s"), PointIndex, *Message);
		};

		UTEST_EQUAL(*FormatMessage(TEXT("SourcePoint and OutPoint index %d have same Density")), SourcePoint.Density, OutRanges.DensityRange[PointIndex]);
		UTEST_EQUAL(*FormatMessage(TEXT("SourcePoint and OutPoint index %d have same BoundsMin")), SourcePoint.BoundsMin, OutRanges.BoundsMinRange[PointIndex]);
		UTEST_EQUAL(*FormatMessage(TEXT("SourcePoint and OutPoint index %d have same BoundsMax")), SourcePoint.BoundsMax, OutRanges.BoundsMaxRange[PointIndex]);
		UTEST_EQUAL(*FormatMessage(TEXT("SourcePoint and OutPoint index %d have same Steepness")), SourcePoint.Steepness, OutRanges.SteepnessRange[PointIndex]);

		// Validate transform inheritance
		const FTransform& SourceTransform = SourcePoint.Transform;
		const FTransform& TargetTransform = TargetPoint.Transform;
		const FTransform& OutTransform = OutRanges.TransformRange[PointIndex];

		const FVector Location = TargetPoint.Transform.TransformPosition(SourcePoint.Transform.GetLocation());
		FQuat Rotation;
		FVector Scale;
		FVector4 Color;
		int32 Seed;

		if (Settings->RotationInheritance == EPCGCopyPointsInheritanceMode::Relative)
		{
			Rotation = Parameters.ExpectedPoints[PointIndex].Transform.GetRotation();
		}
		else if (Settings->RotationInheritance == EPCGCopyPointsInheritanceMode::Source)
		{
			Rotation = SourceTransform.GetRotation();
		}
		else // if (Settings->RotationInheritance == EPCGCopyPointsInheritanceMode::Target)
		{
			Rotation = TargetTransform.GetRotation();
		}

		if (Settings->ScaleInheritance == EPCGCopyPointsInheritanceMode::Relative)
		{
			Scale = Parameters.ExpectedPoints[PointIndex].Transform.GetScale3D();
		}
		else if (Settings->ScaleInheritance == EPCGCopyPointsInheritanceMode::Source)
		{
			Scale = SourceTransform.GetScale3D();
		}
		else // if (Settings->ScaleInheritance == EPCGCopyPointsInheritanceMode::Target)
		{
			Scale = TargetTransform.GetScale3D();
		}

		if (Settings->ColorInheritance == EPCGCopyPointsInheritanceMode::Relative)
		{
			Color = Parameters.ExpectedPoints[PointIndex].Color;
		}
		else if (Settings->ColorInheritance == EPCGCopyPointsInheritanceMode::Source)
		{
			Color = SourcePoint.Color;
		}
		else // if (Settings->ColorInheritance == EPCGCopyPointsInheritanceMode::Target)
		{
			Color = TargetPoint.Color;
		}

		if (Settings->SeedInheritance == EPCGCopyPointsInheritanceMode::Relative)
		{
			Seed = Parameters.ExpectedPoints[PointIndex].Seed;
		}
		else if (Settings->SeedInheritance == EPCGCopyPointsInheritanceMode::Source)
		{
			Seed = SourcePoint.Seed;
		}
		else // if (Settings->SeedInheritance == EPCGCopyPointsInheritanceMode::Target)
		{
			Seed = TargetPoint.Seed;
		}

		UTEST_TRUE(*FormatMessage(TEXT("Valid rotation")), Rotation.Equals(OutTransform.GetRotation()));
		UTEST_EQUAL(*FormatMessage(TEXT("Valid scale")), Scale, OutTransform.GetScale3D());
		UTEST_TRUE(*FormatMessage(TEXT("Valid color")), Color.Equals(OutRanges.ColorRange[PointIndex]));
		UTEST_EQUAL(*FormatMessage(TEXT("Valid location")), Location, OutTransform.GetLocation());
		UTEST_EQUAL(*FormatMessage(TEXT("Valid seed")), Seed, OutRanges.SeedRange[PointIndex]);

		// Validate point value keys
		for (int AttributeIndex = 0; AttributeIndex < InheritedAttributes.Num(); ++AttributeIndex)
		{
			const FPCGMetadataAttribute<float>* InheritedAttribute = InheritedAttributes[AttributeIndex];
			const FPCGMetadataAttribute<float>* OutAttribute = OutAttributes[AttributeIndex];
			check(InheritedAttribute && OutAttribute);

			const PCGMetadataEntryKey EntryKey = (InheritedAttributeModes[AttributeIndex] == EPCGCopyPointsMetadataInheritanceMode::SourceFirst) ? SourcePoint.MetadataEntry : TargetPoint.MetadataEntry;
			UTEST_EQUAL(*FormatMessage(TEXT("Valid metadata value")), InheritedAttribute->GetValueKey(EntryKey), OutAttribute->GetValueKey(OutRanges.MetadataEntryRange[PointIndex]));
		}
	}

	return true;
}


IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyPointsTest_Relative, FPCGCopyPointsTestBase, "Plugins.PCG.CopyPoints.Relative", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyPointsTest_Source, FPCGCopyPointsTestBase, "Plugins.PCG.CopyPoints.Source", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyPointsTest_Target, FPCGCopyPointsTestBase, "Plugins.PCG.CopyPoints.Target", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyPointsTest_EmptySource, FPCGCopyPointsTestBase, "Plugins.PCG.CopyPoints.EmptySource", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCopyPointsTest_EmptyTarget, FPCGCopyPointsTestBase, "Plugins.PCG.CopyPoints.EmptyTarget", PCGTestsCommon::TestFlags)

bool FPCGCopyPointsTest_Relative::RunTest(const FString&)
{
	FPCGCopyPointsTestBase::FParameters Parameters;
	Parameters.RotationInheritance = EPCGCopyPointsInheritanceMode::Relative;
	Parameters.ScaleInheritance = EPCGCopyPointsInheritanceMode::Relative;
	Parameters.ColorInheritance = EPCGCopyPointsInheritanceMode::Relative;
	Parameters.SeedInheritance = EPCGCopyPointsInheritanceMode::Relative;

	// For relative, we hard code the results
	UPCGBasePointData* SourceData = PCGTestsCommon::CreateEmptyBasePointData();
	UPCGBasePointData* TargetData = PCGTestsCommon::CreateEmptyBasePointData();
	
	SourceData->SetNumPoints(3);
	SourceData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Color);
	SourceData->SetDensity(1.0f);

	FPCGPointValueRanges SourceRanges(SourceData, /*bAllocate=*/false);
	SourceRanges.SetFromPoint(0, FPCGPoint(FTransform(FQuat::Identity, FVector(3.0, 4.0, 5.0), FVector(1.0, 1.0, 1.0)), 1.0f, 42));
	SourceRanges.SetFromPoint(1, FPCGPoint(FTransform(FQuat::MakeFromEuler(FVector(45.0, 45.0, 45.0)), FVector(2.0, 1.0, -1.0), FVector(1.0, 1.0, 1.0)), 1.0f, 43));
	SourceRanges.SetFromPoint(2, FPCGPoint(FTransform(FQuat::MakeFromEuler(FVector(46.0, 47.0, 48.0)), FVector(6.0, 2.0, 3.0), FVector(-1.0, 1.0, -1.0)), 1.0f, 44));
		
	SourceRanges.ColorRange[0] = FVector4(0.1, 0.5, 0.2, 0.3);
	SourceRanges.ColorRange[1] = FVector4(0.4, 0.1, 0.1, 0.6);
	SourceRanges.ColorRange[2] = FVector4(0.7, 0.4, 0.9, 0.9);

	TargetData->SetNumPoints(2);
	TargetData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::Color);
	TargetData->SetDensity(1.0f);

	FPCGPointValueRanges TargetRanges(TargetData, /*bAllocate=*/false);
	TargetRanges.SetFromPoint(0, FPCGPoint(FTransform(FQuat::MakeFromEuler(FVector(20.0, 21.0, 22.0)), FVector(-6.0, -4.0, -1.0), FVector(1.0, 1.0, 1.0)), 1.0f, 45));
	TargetRanges.SetFromPoint(1, FPCGPoint(FTransform(FQuat::MakeFromEuler(FVector(22.0, 23.0, 24.0)), FVector(-7.0, -5.0, -10.0), FVector(-1.0, 1.0, -1.0)), 1.0f, 46));

	TargetRanges.ColorRange[0] = FVector4(0.4, 0.3, 1.0, 1.0);
	TargetRanges.ColorRange[1] = FVector4(0.1, 0.2, 0.7, 0.4);

	// Hard coded values for the values above.
	Parameters.ExpectedPoints.Emplace(FTransform(FQuat(-0.133360, -0.208748, 0.153700, 0.956564), FVector(-6.558469, 1.672731, 3.184284), FVector(1.000000, 1.000000, 1.000000)), 1.0f, -21834409);
	Parameters.ExpectedPoints.Last().Color = FVector4(0.040000, 0.150000, 0.200000, 0.300000);
	Parameters.ExpectedPoints.Emplace(FTransform(FQuat(-0.142203, -0.230303, 0.162785, 0.948810), FVector(-8.079758, -3.471313, -16.818888), FVector(-1.000000, 1.000000, -1.000000)), 1.0f, -1038680728);
	Parameters.ExpectedPoints.Last().Color = FVector4(0.010000, 0.100000, 0.140000, 0.120000);
	Parameters.ExpectedPoints.Emplace(FTransform(FQuat(-0.264611, -0.622081, 0.334511, 0.656581), FVector(-4.066812, -2.574331, -1.479846), FVector(1.000000, 1.000000, 1.000000)), 1.0f, -226535144);
	Parameters.ExpectedPoints.Last().Color = FVector4(0.160000, 0.030000, 0.100000, 0.600000);
	Parameters.ExpectedPoints.Emplace(FTransform(FQuat(0.180702, -0.628874, 0.065701, 0.753357), FVector(-9.408575, -4.647380, -10.272812), FVector(-1.000000, 1.000000, -1.000000)), 1.0f, -824479449);
	Parameters.ExpectedPoints.Last().Color = FVector4(0.040000, 0.020000, 0.070000, 0.240000);
	Parameters.ExpectedPoints.Emplace(FTransform(FQuat(-0.249636, -0.634935, 0.347541, 0.643237), FVector(-2.604210, 0.505612, 3.143437), FVector(-1.000000, 1.000000, -1.000000)), 1.0f, -406069811);
	Parameters.ExpectedPoints.Last().Color = FVector4(0.280000, 0.120000, 0.900000, 0.900000);
	Parameters.ExpectedPoints.Emplace(FTransform(FQuat(0.174866, -0.648235, 0.054540, 0.739079), FVector(-11.082370, -6.017904, -15.594473), FVector(1.000000, 1.000000, 1.000000)), 1.0f, -613419534);
	Parameters.ExpectedPoints.Last().Color = FVector4(0.070000, 0.080000, 0.630000, 0.360000);

	Parameters.SourceData = SourceData;
	Parameters.TargetData = TargetData;

	return RunTestInternal(Parameters);
}

bool FPCGCopyPointsTest_Source::RunTest(const FString&)
{
	FPCGCopyPointsTestBase::FParameters Parameters;
	Parameters.RotationInheritance = EPCGCopyPointsInheritanceMode::Source;
	Parameters.ScaleInheritance = EPCGCopyPointsInheritanceMode::Source;
	Parameters.ColorInheritance = EPCGCopyPointsInheritanceMode::Source;
	Parameters.SeedInheritance = EPCGCopyPointsInheritanceMode::Source;
	Parameters.AttributeInheritance = EPCGCopyPointsMetadataInheritanceMode::SourceFirst;
	Parameters.SourceData = PCGTestsCommon::CreateRandomPointData(10, 42);
	Parameters.TargetData = PCGTestsCommon::CreateRandomPointData(4, 43);

	return RunTestInternal(Parameters);
}

bool FPCGCopyPointsTest_Target::RunTest(const FString&)
{
	FPCGCopyPointsTestBase::FParameters Parameters;
	Parameters.RotationInheritance = EPCGCopyPointsInheritanceMode::Target;
	Parameters.ScaleInheritance = EPCGCopyPointsInheritanceMode::Target;
	Parameters.ColorInheritance = EPCGCopyPointsInheritanceMode::Target;
	Parameters.SeedInheritance = EPCGCopyPointsInheritanceMode::Target;
	Parameters.AttributeInheritance = EPCGCopyPointsMetadataInheritanceMode::TargetFirst;
	Parameters.SourceData = PCGTestsCommon::CreateRandomPointData(10, 42);
	Parameters.TargetData = PCGTestsCommon::CreateRandomPointData(4, 43);

	return RunTestInternal(Parameters);
}

bool FPCGCopyPointsTest_EmptySource::RunTest(const FString&)
{
	FPCGCopyPointsTestBase::FParameters Parameters;
	Parameters.SourceData = PCGTestsCommon::CreateEmptyPointData();
	Parameters.TargetData = PCGTestsCommon::CreateRandomPointData(4, 43);

	return RunTestInternal(Parameters);
}

bool FPCGCopyPointsTest_EmptyTarget::RunTest(const FString&)
{
	FPCGCopyPointsTestBase::FParameters Parameters;
	Parameters.SourceData = PCGTestsCommon::CreateRandomPointData(10, 42);
	Parameters.TargetData = PCGTestsCommon::CreateEmptyPointData();

	return RunTestInternal(Parameters);
}

#endif // WITH_EDITOR
