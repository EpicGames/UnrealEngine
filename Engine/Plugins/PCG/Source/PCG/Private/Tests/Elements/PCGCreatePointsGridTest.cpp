// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGVolume.h"
#include "Data/PCGBasePointData.h"

#include "Elements/PCGCreatePointsGrid.h"

namespace CreatePointsGridCommonTests
{
	TUniquePtr<FPCGContext> GenerateTestDataAndRunGrid(EPCGPointPosition Position, const FVector GridSize, const FVector Cell)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPCGCreatePointsGridSettings>(TestData);
		UPCGCreatePointsGridSettings* Settings = CastChecked<UPCGCreatePointsGridSettings>(TestData.Settings);
		Settings->PointPosition = Position;
		Settings->CoordinateSpace = EPCGCoordinateSpace::World;
		Settings->GridExtents = GridSize;
		Settings->CellSize = Cell;

		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		return Context;
	}

	PCGTestsCommon::FTestData GenerateTestData(EPCGPointPosition Position, EPCGCoordinateSpace CoordinateSpace)
	{
		PCGTestsCommon::FTestData TestData(42, nullptr, APCGVolume::StaticClass());
		PCGTestsCommon::GenerateSettings<UPCGCreatePointsGridSettings>(TestData);
		UPCGCreatePointsGridSettings* Settings = CastChecked<UPCGCreatePointsGridSettings>(TestData.Settings);
		Settings->PointPosition = Position;
		Settings->CoordinateSpace = CoordinateSpace;
		Settings->GridExtents = FVector(100.0, 100.0, 50.0);
		Settings->CellSize = FVector(100.0);

		return TestData;
	}
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_CellCenter, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.CellCenter", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_CellCenter::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCenter, FVector(500.0, 500.0, 50.0), FVector(100.0));

	// Testing using the default settings
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	UTEST_EQUAL("Output point count", Transforms.Num(), 100);

	UTEST_EQUAL("Point 0 has the correct transform,", Transforms[0], FTransform(FRotator::ZeroRotator, FVector(-450.0, -450.0, 0), FVector::One()));
	UTEST_EQUAL("Point 23 has the correct transform,", Transforms[23], FTransform(FRotator::ZeroRotator, FVector(-150.0, -250.0, 0), FVector::One()));
	UTEST_EQUAL("Point 65 has the correct transform,", Transforms[65], FTransform(FRotator::ZeroRotator, FVector(50.0, 150.0, 0), FVector::One()));
	UTEST_EQUAL("Point 92 has the correct transform,", Transforms[92], FTransform(FRotator::ZeroRotator, FVector(-250.0, 450.0, 0), FVector::One()));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_CellCorner, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.CellCorner", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_CellCorner::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCorners, FVector(500.0, 500.0, 50.0), FVector(100.0));

	// Testing using the default sizes on the corners
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	// Cell Corners generate (10 + 1) * (10 + 1) * 2
	UTEST_EQUAL("Output point count", Transforms.Num(), 242);

	UTEST_EQUAL("Point 0 has the correct transform,", Transforms[0], FTransform(FRotator::ZeroRotator, FVector(-500.0, -500.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 23 has the correct transform,", Transforms[23], FTransform(FRotator::ZeroRotator, FVector(-400.0, -300.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 65 has the correct transform,", Transforms[65], FTransform(FRotator::ZeroRotator, FVector(500.0, 0.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 92 has the correct transform,", Transforms[92], FTransform(FRotator::ZeroRotator, FVector(-100.0, 300.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 145 has the correct transform,", Transforms[145], FTransform(FRotator::ZeroRotator, FVector(-300.0, -300.0, 50.0), FVector::One()));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_XYCenter, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.XYCenter", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_XYCenter::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCenter, FVector(100.0, 100.0, 50.0), FVector(100.0));

	// Testing using a small grid in "2D"
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	UTEST_EQUAL("Output point count", Transforms.Num(), 4);

	UTEST_EQUAL("Point 0 has the correct transform,", Transforms[0], FTransform(FRotator::ZeroRotator, FVector(-50.0, -50.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 1 has the correct transform,", Transforms[1], FTransform(FRotator::ZeroRotator, FVector(50.0, -50.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 2 has the correct transform,", Transforms[2], FTransform(FRotator::ZeroRotator, FVector(-50.0, 50.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 3 has the correct transform,", Transforms[3], FTransform(FRotator::ZeroRotator, FVector(50.0, 50.0, 0.0), FVector::One()));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_XYCorner, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.XYCorner", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_XYCorner::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCorners, FVector(100.0, 100.0, 0.0), FVector(100.0));

	// Testing using a small grid in "2D"
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	UTEST_EQUAL("Output point count", Transforms.Num(), 9);

	UTEST_EQUAL("Point 1 has the correct transform,", Transforms[1], FTransform(FRotator::ZeroRotator, FVector(0.0, -100.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 3 has the correct transform,", Transforms[3], FTransform(FRotator::ZeroRotator, FVector(-100.0, 0.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 5 has the correct transform,", Transforms[5], FTransform(FRotator::ZeroRotator, FVector(100.0, 0.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 7 has the correct transform,", Transforms[7], FTransform(FRotator::ZeroRotator, FVector(0.0, 100.0, 0.0), FVector::One()));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_XZCenter, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.XZCenter", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_XZCenter::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCenter, FVector(100.0, 50.0, 100.0), FVector(100.0));

	// Testing using a small grid in "2D"
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	UTEST_EQUAL("Output point count", Transforms.Num(), 4);

	UTEST_EQUAL("Point 0 has the correct transform,", Transforms[0], FTransform(FRotator::ZeroRotator, FVector(-50.0, 0.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 1 has the correct transform,", Transforms[1], FTransform(FRotator::ZeroRotator, FVector(50.0, 0.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 2 has the correct transform,", Transforms[2], FTransform(FRotator::ZeroRotator, FVector(-50.0, 0.0, 50.0), FVector::One()));
	UTEST_EQUAL("Point 3 has the correct transform,", Transforms[3], FTransform(FRotator::ZeroRotator, FVector(50.0, 0.0, 50.0), FVector::One()));
	
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_XZCorner, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.XZCorner", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_XZCorner::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCorners, FVector(100.0, 0.0, 100.0), FVector(100.0));

	// Testing using a small grid in "2D"
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	UTEST_EQUAL("Output point count", Transforms.Num(), 9);

	UTEST_EQUAL("Point 1 has the correct transform,", Transforms[1], FTransform(FRotator::ZeroRotator, FVector(0.0, 0.0, -100.0), FVector::One()));
	UTEST_EQUAL("Point 3 has the correct transform,", Transforms[3], FTransform(FRotator::ZeroRotator, FVector(-100.0, 0.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 5 has the correct transform,", Transforms[5], FTransform(FRotator::ZeroRotator, FVector(100.0, 0.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 7 has the correct transform,", Transforms[7], FTransform(FRotator::ZeroRotator, FVector(0.0, 0.0, 100.0), FVector::One()));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_YZCenter, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.YZCenter", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_YZCenter::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCenter, FVector(50.0, 100.0, 100.0), FVector(100.0));

	// Testing using a small grid in "2D"
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	UTEST_EQUAL("Output point count", Transforms.Num(), 4);

	UTEST_EQUAL("Point 0 has the correct transform,", Transforms[0], FTransform(FRotator::ZeroRotator, FVector(0.0, -50.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 1 has the correct transform,", Transforms[1], FTransform(FRotator::ZeroRotator, FVector(0.0, 50.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 2 has the correct transform,", Transforms[2], FTransform(FRotator::ZeroRotator, FVector(0.0, -50.0, 50.0), FVector::One()));
	UTEST_EQUAL("Point 3 has the correct transform,", Transforms[3], FTransform(FRotator::ZeroRotator, FVector(0.0, 50.0, 50.0), FVector::One()));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_YZCorner, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.YZCorner", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_YZCorner::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCorners, FVector(0.0, 100.0, 100.0), FVector(100.0));

	// Testing using a small grid in "2D"
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	UTEST_EQUAL("Output point count", Transforms.Num(), 9);

	UTEST_EQUAL("Point 1 has the correct transform,", Transforms[1], FTransform(FRotator::ZeroRotator, FVector(0.0, 0.0, -100.0), FVector::One()));
	UTEST_EQUAL("Point 3 has the correct transform,", Transforms[3], FTransform(FRotator::ZeroRotator, FVector(0.0, -100.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 5 has the correct transform,", Transforms[5], FTransform(FRotator::ZeroRotator, FVector(0.0, 100.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 7 has the correct transform,", Transforms[7], FTransform(FRotator::ZeroRotator, FVector(0.0, 0.0, 100.0), FVector::One()));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_NegativeValuesGrid, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.NegativeValuesGrid", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_NegativeValuesGrid::RunTest(const FString& Parameters)
{
	AddExpectedError(TEXT("must not be less than 0"));
	
	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCenter, FVector(-1.0), FVector(100.0));

	// Testing for negative values, should produce no outputs
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 0);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_NegativeValuesCell, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.NegativeValuesCell", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_NegativeValuesCell::RunTest(const FString& Parameters)
{
	AddExpectedError(TEXT("must not be less than 0"));

	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCenter, FVector(100.0), FVector(-1.0));

	// Testing for negative values, should produce no outputs
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 0);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_OversizedCell, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.OversizedCell", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_OversizedCell::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = CreatePointsGridCommonTests::GenerateTestDataAndRunGrid(EPCGPointPosition::CellCenter, FVector(100.0), FVector(400.0));

	// Testing for negative values, should produce no outputs
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	UTEST_EQUAL("Output point count", Transforms.Num(), 1);

	UTEST_EQUAL("Point 0 has the correct transform,", Transforms[0], FTransform(FRotator::ZeroRotator, FVector(0.0, 0.0, 0.0), FVector::One()));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_LocalCenter, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.LocalCenter", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_LocalCenter::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData = CreatePointsGridCommonTests::GenerateTestData(EPCGPointPosition::CellCenter, EPCGCoordinateSpace::LocalComponent);

	// Creating the volume and proper bounds
	AActor* TestActor = TestData.TestActor;
	TestActor->SetActorScale3D(FVector(50.0));
	TestActor->SetActorRelativeLocation(FVector(100.0, 100.0, 0.0));
	FBox TestBounds = TestActor->GetComponentsBoundingBox().ExpandBy(FVector(200.0));

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	// Testing using a small grid in local mode
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	UTEST_EQUAL("Output point count", Transforms.Num(), 4);

	// Check to see if the position of the points are within the bounds of the volume
	for (int i = 0; i < Transforms.Num(); ++i)
	{
		UTEST_TRUE(FString::Format(TEXT("Point {0}, {1} is within the volume"), { i }), TestBounds.IsInsideOrOn(Transforms[i].GetLocation()));
	}

	UTEST_EQUAL("Point 0 has the correct transform,", Transforms[0], FTransform(FRotator::ZeroRotator, FVector(50.0, 50.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 1 has the correct transform,", Transforms[1], FTransform(FRotator::ZeroRotator, FVector(150.0, 50.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 2 has the correct transform,", Transforms[2], FTransform(FRotator::ZeroRotator, FVector(50.0, 150.0, 0.0), FVector::One()));
	UTEST_EQUAL("Point 3 has the correct transform,", Transforms[3], FTransform(FRotator::ZeroRotator, FVector(150.0, 150.0, 0.0), FVector::One()));
	
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsGridTest_LocalCorner, FPCGTestBaseClass, "Plugins.PCG.CreatePointsGrid.LocalCorner", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsGridTest_LocalCorner::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData = CreatePointsGridCommonTests::GenerateTestData(EPCGPointPosition::CellCorners, EPCGCoordinateSpace::LocalComponent);

	// Creating the volume and proper bounds
	AActor* TestActor = TestData.TestActor;
	TestActor->SetActorScale3D(FVector(50.0));
	TestActor->SetActorRelativeLocation(FVector(100.0, 100.0, 0.0));
	FBox TestBounds = TestActor->GetComponentsBoundingBox().ExpandBy(FVector(200.0));

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	// Testing using a small grid in local mode
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	const TConstPCGValueRange<FTransform> Transforms = OutPointData->GetConstTransformValueRange();

	//Cell Corners generate (2 + 1) * (2 + 1) * 2
	UTEST_EQUAL("Output point count", Transforms.Num(), 18);

	// Check to see if the position of the points are within the bounds of the volume
	for (int i = 0; i < Transforms.Num(); ++i)
	{
		UTEST_TRUE(FString::Format(TEXT("Point {0} is within the volume"), { i }), TestBounds.IsInsideOrOn(Transforms[i].GetLocation()));
	}

	UTEST_EQUAL("Point 1 has the correct transform,", Transforms[1], FTransform(FRotator::ZeroRotator, FVector(100.0, 0.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 3 has the correct transform,", Transforms[3], FTransform(FRotator::ZeroRotator, FVector(0.0, 100.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 5 has the correct transform,", Transforms[5], FTransform(FRotator::ZeroRotator, FVector(200.0, 100.0, -50.0), FVector::One()));
	UTEST_EQUAL("Point 11 has the correct transform,", Transforms[11], FTransform(FRotator::ZeroRotator, FVector(200.0, 0.0, 50.0), FVector::One()));

	return true;
}