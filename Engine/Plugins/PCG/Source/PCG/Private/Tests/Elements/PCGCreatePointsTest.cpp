// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"

#include "Elements/PCGCreatePoints.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCreatePointsTest_Basic, FPCGTestBaseClass, "Plugins.PCG.CreatePoints.Basic", PCGTestsCommon::TestFlags)

bool FPCGCreatePointsTest_Basic::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGCreatePointsSettings>(TestData);
	UPCGCreatePointsSettings* Settings = CastChecked<UPCGCreatePointsSettings>(TestData.Settings);
	
	TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateRandomBasePointData(100, 42, false);

	// Setting the seed explicitly, first point will have a seed of 0, which mean it going to be changed and computed depending on the point position
	// All the others should stay the same.
	TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange();
	for (int32 i = 0; i < PointData->GetNumPoints(); ++i)
	{
		SeedRange[i] = i;
	}

	const FConstPCGPointValueRanges ValueRange(PointData);
	TArray<FPCGPoint> Points;
	Points.Reserve(PointData->GetNumPoints());

	for (int32 i = 0; i < PointData->GetNumPoints(); ++i)
	{
		Points.Add(ValueRange.GetPoint(i));
	}

	Settings->PointsToCreate = Points;
	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	FConstPCGPointValueRanges OutRanges(OutPointData);

	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 100);

	for (int i = 0; i < OutPointData->GetNumPoints(); ++i)
	{
		// Seed at 0 will compute a seed depending on the point position
		static constexpr int32 SeedAtZero = 907633527;
		UTEST_EQUAL(FString::Format(TEXT("InArray[{0}].Seed is equal to OutArray[{0}].Seed"), { i }), OutRanges.SeedRange[i], i == 0 ? SeedAtZero : Settings->PointsToCreate[i].Seed);
		UTEST_EQUAL(FString::Format(TEXT("InArray[{0}].Density is equal to OutArray[{0}].Density"), { i }), OutRanges.DensityRange[i], Settings->PointsToCreate[i].Density);
		UTEST_EQUAL(FString::Format(TEXT("InArray[{0}].Transform is equal to OutArray[{0}].Transform"), { i }), OutRanges.TransformRange[i], Settings->PointsToCreate[i].Transform);
	}

	return true;
}

