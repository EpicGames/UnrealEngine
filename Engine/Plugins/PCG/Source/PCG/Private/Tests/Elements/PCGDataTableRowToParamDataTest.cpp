// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"

#include "Elements/PCGDataTableRowToParamData.h"
#include "PCGContext.h"
#include "Tests/Elements/PCGDataTableRowToParamDataTestHelpers.h"
#include "Engine/DataTable.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDataTableRowToParamDataTest, FPCGTestBaseClass, "Plugins.PCG.DataTableRowToParamData.Basic", PCGTestsCommon::TestFlags)

bool FPCGDataTableRowToParamDataTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGDataTableRowToParamDataSettings>(TestData);
	UPCGDataTableRowToParamDataSettings* Settings = CastChecked<UPCGDataTableRowToParamDataSettings>(TestData.Settings);

	UDataTable *TestDataTable = NewObject<UDataTable>();
	TestDataTable->RowStruct = FPCGDataTableRowToParamDataTestStruct::StaticStruct();
	TArray<FString> Errors = TestDataTable->CreateTableFromCSVString(R"CSV(---,Name,String,I32,I64,F32,F64,V2,V3,V4,SoftPath
AAA,"A Name","A String","111","1111","1.1","1.11","(X=1.0,Y=1.0)","(X=1.0,Y=1.0,Z=1.0)","(X=1.0,Y=1.0,Z=1.0,W=1.0)","/Script/PCG"
BBB,"B Name","B String","222","2222","2.2","2.22","(X=2.0,Y=2.0)","(X=2.0,Y=2.0,Z=2.0)","(X=2.0,Y=2.0,Z=2.0,W=2.0)","/Script/PCG"
CCC,"C Name","C String","333","3333","3.3","3.33","(X=3.0,Y=3.0)","(X=3.0,Y=3.0,Z=3.0)","(X=3.0,Y=3.0,Z=3.0,W=3.0)","/Script/PCG")CSV");

	UTEST_EQUAL(*FString::Printf(TEXT("CSV Errors:\n%s"), *FString::Join(Errors, TEXT("\n"))), Errors.Num(), 0);

	Settings->RowName = TEXT("BBB");
	Settings->DataTable = TestDataTable;

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> OutputParamData = Context->OutputData.GetParamsByPin(PCGPinConstants::DefaultOutputLabel);
	const UPCGParamData* Params = OutputParamData.IsEmpty() ? nullptr : Cast<UPCGParamData>(OutputParamData[0].Data);
	UTEST_NOT_NULL("Output params", Params);

	// For static analysis
	if (!Params)
	{
		return false;
	}

	UTEST_NOT_NULL("Params has metadata", Params->Metadata.Get());

	void* RowData = TestDataTable->FindRow<FPCGDataTableRowToParamDataTestStruct>(Settings->RowName, TEXT("FPCGDataTableRowToParamDataTest"));
	UTEST_NOT_NULL(*FString::Printf(TEXT("'%s' Row not found in data table"), *Settings->RowName.ToString()), RowData);

	bool bSuccess = true;

	// first template type is the value of the row struct member, second is the pcg attribute type
	// this is because sometimes conversion occurs as PCG can't represent all types
	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<FName,FName>(*this, RowData, *Params, TEXT("Name"), TEXT("B Name"));
	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<FString,FString>(*this, RowData, *Params, TEXT("String"), TEXT("B String"));
	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<int32,int32>(*this, RowData, *Params, TEXT("I32"), 222);
	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<int64,int64>(*this, RowData, *Params, TEXT("I64"), 2222);
	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<float,double>(*this, RowData, *Params, TEXT("F32"), 2.2);
	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<double,double>(*this, RowData, *Params, TEXT("F64"), 2.22);
	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<FVector2D,FVector2D>(*this, RowData, *Params, TEXT("V2"), FVector2D(2.0, 2.0));
	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<FVector,FVector>(*this, RowData, *Params, TEXT("V3"), FVector(2.0, 2.0, 2.0));
	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<FVector4,FVector4>(*this, RowData, *Params, TEXT("V4"), FVector4(2.0, 2.0, 2.0, 2.0));
	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<FSoftObjectPath, FSoftObjectPath>(*this, RowData, *Params, TEXT("SoftPath"), FSoftObjectPath(TEXT("/Script/PCG")));

	return bSuccess;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDataTableRowToParamDataTest_ParamsOverride, FPCGTestBaseClass, "Plugins.PCG.DataTableRowToParamData.ParamsOverride", PCGTestsCommon::TestFlags)

bool FPCGDataTableRowToParamDataTest_ParamsOverride::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGDataTableRowToParamDataSettings>(TestData);

	UDataTable *TestDataTable = NewObject<UDataTable>();
	TestDataTable->RowStruct = FPCGDataTableRowToParamDataTestStruct::StaticStruct();
	TArray<FString> Errors = TestDataTable->CreateTableFromCSVString(R"CSV(---,Name,String,I32,I64,F32,F64,V2,V3,V4,SoftPath
AAA,"A Name","A String","111","1111","1.1","1.11","(X=1.0,Y=1.0)","(X=1.0,Y=1.0,Z=1.0)","(X=1.0,Y=1.0,Z=1.0,W=1.0)","/Script/PCG"
BBB,"B Name","B String","222","2222","2.2","2.22","(X=2.0,Y=2.0)","(X=2.0,Y=2.0,Z=2.0)","(X=2.0,Y=2.0,Z=2.0,W=2.0)","/Script/PCG"
CCC,"C Name","C String","333","3333","3.3","3.33","(X=3.0,Y=3.0)","(X=3.0,Y=3.0,Z=3.0)","(X=3.0,Y=3.0,Z=3.0,W=3.0)","/Script/PCG")CSV");

	UTEST_EQUAL(*FString::Printf(TEXT("CSV Errors:\n%s"), *FString::Join(Errors, TEXT("\n"))), Errors.Num(), 0);

	const FName RowNameOverride = TEXT("BBB");

	{
		UPCGParamData* InputParamData = NewObject<UPCGParamData>();
		InputParamData->MutableMetadata()->CreateSoftObjectPathAttribute(TEXT("DataTable"), FSoftObjectPath(TestDataTable), false);
		InputParamData->MutableMetadata()->CreateStringAttribute(TEXT("RowName"), RowNameOverride.ToString(), false);
		FPCGTaggedData& ParamInput = TestData.InputData.TaggedData.Emplace_GetRef();
		ParamInput.Data = InputParamData;
		ParamInput.Pin = PCGPinConstants::DefaultParamsLabel;
	}

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> AllParams = Context->OutputData.GetParamsByPin(PCGPinConstants::DefaultOutputLabel);
	UTEST_EQUAL("Output params", AllParams.Num(), 1);

	const UPCGParamData* Params = CastChecked<const UPCGParamData>(AllParams[0].Data);

	// For static analysis
	if (!Params)
	{
		return false;
	}

	UTEST_NOT_NULL("Params has metadata", Params->Metadata.Get());

	void* RowData = TestDataTable->FindRow<FPCGDataTableRowToParamDataTestStruct>(RowNameOverride, TEXT("FPCGDataTableRowToParamDataTest"));
	UTEST_NOT_NULL(*FString::Printf(TEXT("'%s' Row not found in data table"), *RowNameOverride.ToString()), RowData);

	bool bSuccess = true;

	bSuccess |= PCGDataTableRowToParamDataTestHelpers::TestAttribute<FName,FName>(*this, RowData, *Params, TEXT("Name"), TEXT("B Name"));

	return bSuccess;
}
