// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGParamData.h"
#include "Tests/Elements/PCGDataTableRowToParamDataTestStruct.h"
#include "Tests/PCGTestsCommon.h"

class FPCGMetadataAttributeBase;
template <typename T> class FPCGMetadataAttribute;

namespace PCGDataTableRowToParamDataTestHelpers
{
	template<typename StructType, typename PCGType>
	bool TestAttribute(FPCGTestBaseClass& Self, void *Data, const UPCGParamData& Params, FName FieldName, PCGType ExpectedValue)
	{
		const FPCGMetadataAttribute<PCGType>* Attribute = static_cast<const FPCGMetadataAttribute<PCGType>*>(Params.Metadata->GetConstAttribute(FieldName));

		if (!Self.TestNotNull(*FString::Printf(TEXT("'%s' Metadata attribute"), *FieldName.ToString()), Attribute))
		{
			return false;
		}

		if (!Self.TestEqual(*FString::Printf(TEXT("'%s' Metadata type"), *FieldName.ToString()), Attribute->GetTypeId(), PCG::Private::MetadataTypes<PCGType>::Id))
		{
			return false;
		}

		const FProperty* Property = FindFProperty<FProperty>(FPCGDataTableRowToParamDataTestStruct::StaticStruct(), FieldName);
		if (!Self.TestNotNull(*FString::Printf(TEXT("'%s' Property not found"), *FieldName.ToString()), Property))
		{
			return false;
		}

		const StructType* RowValuePtr = Property->ContainerPtrToValuePtr<StructType>(Data);
		if (!Self.TestNotNull(*FString::Printf(TEXT("'%s' Row Value not found"), *FieldName.ToString()), RowValuePtr))
		{
			return false;
		}

		if (!Self.TestEqual(*FString::Printf(TEXT("'%s' Row Value"), *FieldName.ToString()), *RowValuePtr, StructType(ExpectedValue)))
		{
			return false;
		}
		
		PCGType Value = Attribute->GetValueFromItemKey(0);
		if (!Self.TestEqual(*FString::Printf(TEXT("'%s' Attribute Value"), *FieldName.ToString()), Value, ExpectedValue))
		{
			return false;
		}

		return true;
	}
}
