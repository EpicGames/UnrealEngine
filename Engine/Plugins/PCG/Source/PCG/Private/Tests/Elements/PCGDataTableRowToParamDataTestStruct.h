// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataTableRowToParamDataTestStruct.generated.h"

USTRUCT()
struct FPCGDataTableRowToParamDataTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FName Name = NAME_None;

	UPROPERTY()
	FString String = "";

	UPROPERTY()
	int32 I32 = 0;

	UPROPERTY()
	int64 I64 = 0;

	UPROPERTY()
	float F32 = 0.0f;

	UPROPERTY()
	double F64 = 0.0;

	UPROPERTY()
	FVector2D V2 = FVector2D::Zero();

	UPROPERTY()
	FVector V3 = FVector::Zero();

	UPROPERTY()
	FVector4 V4 = FVector4::Zero();

	UPROPERTY()
	FSoftObjectPath SoftPath;
};
