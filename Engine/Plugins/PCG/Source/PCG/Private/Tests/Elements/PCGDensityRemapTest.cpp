// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "Data/PCGSpatialData.h"
#include "PCGComponent.h"

#include "Data/PCGPointData.h"
#include "Elements/Metadata/PCGAttributeRemap.h"
#include "PCGContext.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDensityRemapTest, FPCGTestBaseClass, "Plugins.PCG.DensityRemap.Basic", PCGTestsCommon::TestFlags)

bool FPCGDensityRemapTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGAttributeRemapSettings>(TestData);
	UPCGAttributeRemapSettings* Settings = CastChecked<UPCGAttributeRemapSettings>(TestData.Settings);
	FPCGElementPtr DensityRemapElement = TestData.Settings->GetElement();

	Settings->InputSource.SetPointProperty(EPCGPointProperties::Density);
	Settings->bClampToUnitRange = true;

	TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateEmptyBasePointData();

	FRandomStream RandomSource(TestData.Seed);
	const int PointCount = 6;

	PointData->SetNumPoints(PointCount);
	
	TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange();
	TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();

	for (int I = 0; I < PointCount; ++I)
	{
		SeedRange[I] = I;
		DensityRange[I] = static_cast<float>(I) / (PointCount - 1);
	}

	FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedData.Data = PointData;
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	auto ValidateDensityRemap = [this, &TestData, DensityRemapElement, Settings](TArray<float> CorrectDensities) -> bool
	{
		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!DensityRemapElement->Execute(Context.Get()))
		{}

		const TArray<FPCGTaggedData>& Inputs = Context->InputData.GetAllSpatialInputs();
		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllSpatialInputs();
		
		if (!TestEqual("Valid number of outputs", Outputs.Num(), Inputs.Num()))
		{
			return false;
		}
	
		bool bTestPassed = true;

		for (int DataIndex = 0; DataIndex < Inputs.Num(); ++DataIndex)
		{
			const FPCGTaggedData& Input = Inputs[DataIndex];
			const FPCGTaggedData& Output = Outputs[DataIndex];

			const UPCGSpatialData* InSpatialData = Cast<UPCGSpatialData>(Input.Data);
			check(InSpatialData);

			const UPCGBasePointData* InPointData = InSpatialData->ToBasePointData(Context.Get());
			check(InPointData);

			const UPCGSpatialData* OutSpatialData = Cast<UPCGSpatialData>(Output.Data);

			if (!TestNotNull("Valid output SpatialData", OutSpatialData))
			{
				bTestPassed = false;
				continue;
			}

			const UPCGBasePointData* OutPointData = OutSpatialData->ToBasePointData(Context.Get());

			if (!TestNotNull("Valid output PointData", OutPointData))
			{
				bTestPassed = false;
				continue;
			}

			if (!TestEqual("Input and output point counts match", OutPointData->GetNumPoints(), InPointData->GetNumPoints()))
			{ 
				bTestPassed = false;
				continue;
			}

			const TConstPCGValueRange<float> DensityRange = OutPointData->GetConstDensityValueRange();
			for (int PointIndex = 0; PointIndex < InPointData->GetNumPoints(); ++PointIndex)
			{
				bTestPassed &= TestEqual("Correct density", DensityRange[PointIndex], CorrectDensities[PointIndex]);
			}
		}

		return bTestPassed;
	};

	bool bTestPassed = true;

	// Test [0-1] -> [0-1]
	{
		Settings->InRangeMin = 0.f;
		Settings->InRangeMax = 1.f;
		Settings->OutRangeMin = 0.f;
		Settings->OutRangeMax = 1.f;
		Settings->bIgnoreValuesOutsideInputRange = true;
		bTestPassed &= TestTrue("Input and Output are identical when InRange and OutRange are identical", ValidateDensityRemap({ 0.f, 0.2f, 0.4f, 0.6f, 0.8f, 1.f }));
	}

	// Test [0-0.4] -> [0-1]
	{
		Settings->InRangeMin = 0.f;
		Settings->InRangeMax = 0.4f;
		Settings->OutRangeMin = 0.f;
		Settings->OutRangeMax = 1.f;
		Settings->bIgnoreValuesOutsideInputRange = true;
		bTestPassed &= TestTrue("Valid densities for partial InRange", ValidateDensityRemap({ 0.f, 0.5f, 1.f, 0.6f, 0.8f, 1.f }));

		Settings->InRangeMin = 0.4f;
		Settings->InRangeMax = 0.f;
		Settings->OutRangeMin = 1.f;
		Settings->OutRangeMax = 0.f;
		Settings->bIgnoreValuesOutsideInputRange = true;
		bTestPassed &= TestTrue("Inverting ranges does not effect output", ValidateDensityRemap({ 0.f, 0.5f, 1.f, 0.6f, 0.8f, 1.f }));
	}

	// Test [0-0.4] -> [0.5-1]
	{
		Settings->InRangeMin = 0.4f;
		Settings->InRangeMax = 1.f;
		Settings->OutRangeMin = 0.5f;
		Settings->OutRangeMax = 1.f;
		Settings->bIgnoreValuesOutsideInputRange = true;
		bTestPassed &= TestTrue("Valid densities for partial OutRange", ValidateDensityRemap({ 0.f, 0.2f, 3.f / 6.f, 4.f / 6.f, 5.f / 6.f, 1.f }));

		Settings->InRangeMin = 1.f;
		Settings->InRangeMax = 0.4f;
		Settings->OutRangeMin = 1.f;
		Settings->OutRangeMax = 0.5f;
		Settings->bIgnoreValuesOutsideInputRange = true;
		bTestPassed &= TestTrue("Inverting ranges does not effect output", ValidateDensityRemap({ 0.f, 0.2f, 3.f / 6.f, 4.f / 6.f, 5.f / 6.f, 1.f }));
	}

	// Test disabling Range Exclusion
	{
		Settings->InRangeMin = 0.4f;
		Settings->InRangeMax = 1.f;
		Settings->OutRangeMin = 0.5f;
		Settings->OutRangeMax = 1.f;
		Settings->bIgnoreValuesOutsideInputRange = false;
		bTestPassed &= TestTrue("All values are remapped when Range Exclusion is disabled", ValidateDensityRemap({ 1.f / 6.f, 2.f / 6.f, 3.f / 6.f, 4.f / 6.f, 5.f / 6.f, 1.f }));
	}

	// Test Point to Point, Point to Range, and Range to Point
	{
		Settings->InRangeMin = 0.2f;
		Settings->InRangeMax = 0.2f;
		Settings->OutRangeMin = 0.f;
		Settings->OutRangeMax = 1.f;
		Settings->bIgnoreValuesOutsideInputRange = true;
		bTestPassed &= TestTrue("Point input to Range output", ValidateDensityRemap({ 0.f, 0.5f, 0.4f, 0.6f, 0.8f, 1.f }));

		Settings->InRangeMin = 0.2f;
		Settings->InRangeMax = 0.2f;
		Settings->OutRangeMin = 0.5f;
		Settings->OutRangeMax = 0.5f;
		Settings->bIgnoreValuesOutsideInputRange = true;
		bTestPassed &= TestTrue("Point input to Point output", ValidateDensityRemap({ 0.f, 0.5f, 0.4f, 0.6f, 0.8f, 1.f }));

		Settings->InRangeMin = 0.2f;
		Settings->InRangeMax = 1.f;
		Settings->OutRangeMin = 0.5f;
		Settings->OutRangeMax = 0.5f;
		Settings->bIgnoreValuesOutsideInputRange = true;
		bTestPassed &= TestTrue("Range input to Point output", ValidateDensityRemap({ 0.f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f }));
	}

	return bTestPassed;
}
