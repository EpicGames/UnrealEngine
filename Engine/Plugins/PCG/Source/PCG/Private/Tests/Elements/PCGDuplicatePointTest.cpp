// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"

#include "Elements/PCGDuplicatePoint.h"

class DuplicatePointTestBase : public FPCGTestBaseClass
{

public:
	using FPCGTestBaseClass::FPCGTestBaseClass;
	struct FTestParameters
	{
		int Iterations = 0;
		FVector Direction = FVector();
		bool IncludeSource = true;
		FTransform PtTransform = FTransform();

		int ExpectedPointCount = 0;
		bool bCheckLocation = false;
		bool bCheckRotation = false;
		bool bCheckScale = false;
	};

protected:
	bool GenerateTestDataRunAndValidate(FTestParameters Parameters)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPCGDuplicatePointSettings>(TestData);
		UPCGDuplicatePointSettings* Settings = CastChecked<UPCGDuplicatePointSettings>(TestData.Settings);
		Settings->Iterations = Parameters.Iterations;
		Settings->Direction = Parameters.Direction;
		Settings->bOutputSourcePoint = Parameters.IncludeSource;
		Settings->PointTransform = Parameters.PtTransform;

		FPCGTaggedData& Inputs = TestData.InputData.TaggedData.Emplace_GetRef();
		Inputs.Pin = PCGPinConstants::DefaultInputLabel;
		Inputs.Data = PCGTestsCommon::CreateBasePointData();

		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

		UTEST_EQUAL("Output count", Outputs.Num(), 1);

		const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

		UTEST_NOT_NULL("OutputA point data", OutPointData);
				
		UTEST_EQUAL("OutputA point count", OutPointData->GetNumPoints(), Parameters.ExpectedPointCount);

		const FConstPCGPointValueRanges OutRanges(OutPointData);

		const FVector LocationGetter = (OutRanges.BoundsMaxRange[0] - OutRanges.BoundsMinRange[0]) * Parameters.Direction + Parameters.PtTransform.GetLocation();

		if (Parameters.bCheckLocation)
		{
			for (int i = 0; i < OutPointData->GetNumPoints(); ++i)
			{
				UTEST_EQUAL("Output Location", OutRanges.TransformRange[i].GetLocation(), LocationGetter * i);
			}
		}
		if (Parameters.bCheckRotation)
		{
			for (int i = 0; i < OutPointData->GetNumPoints(); ++i)
			{
				UTEST_EQUAL("Output Rotation", OutRanges.TransformRange[i].GetRotation().Rotator(), Parameters.PtTransform.GetRotation().Rotator() * i);
			}
		}
		if (Parameters.bCheckScale)
		{
			const FVector ParameterTestScale = Parameters.PtTransform.GetScale3D();
			for (int i = 0; i < OutPointData->GetNumPoints(); ++i)
			{
				UTEST_EQUAL("Output Scale", OutRanges.TransformRange[i].GetScale3D(), FVector(FMath::Pow(ParameterTestScale.X, i), FMath::Pow(ParameterTestScale.Y, i), FMath::Pow(ParameterTestScale.Z, i)));
			}
		}

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_WithSource, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.WithSource", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_WithSource::RunTest(const FString& Parameters)
{
	FTestParameters SourceParameters{};

	SourceParameters.Iterations = 1;
	SourceParameters.Direction = FVector(0.0, 0.0, 1.0);
	SourceParameters.IncludeSource = true;
	SourceParameters.PtTransform = FTransform();
	SourceParameters.ExpectedPointCount = 2;
	SourceParameters.bCheckLocation = false;
	SourceParameters.bCheckRotation = false;
	SourceParameters.bCheckScale = false;
	
	return GenerateTestDataRunAndValidate(SourceParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_WithoutSource, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.WithoutSource", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_WithoutSource::RunTest(const FString& Parameters)
{
	FTestParameters NoSourceParameters{};

	NoSourceParameters.Iterations = 1;
	NoSourceParameters.Direction = FVector(0.0, 0.0, 1.0);
	NoSourceParameters.IncludeSource = false;
	NoSourceParameters.PtTransform = FTransform();
	NoSourceParameters.ExpectedPointCount = 1;
	NoSourceParameters.bCheckLocation = false;
	NoSourceParameters.bCheckRotation = false;
	NoSourceParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(NoSourceParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_DirectionX, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.DirectionX", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_DirectionX::RunTest(const FString& Parameters)
{
	FTestParameters XParameters{};

	XParameters.Iterations = 2;
	XParameters.Direction = FVector(1.0, 0.0, 0.0);
	XParameters.IncludeSource = true;
	XParameters.PtTransform = FTransform();
	XParameters.ExpectedPointCount = 3;
	XParameters.bCheckLocation = true;
	XParameters.bCheckRotation = false;
	XParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(XParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_DirectionNegX, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.DirectionNegX", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_DirectionNegX::RunTest(const FString& Parameters)
{
	FTestParameters NegXParameters{};

	NegXParameters.Iterations = 2;
	NegXParameters.Direction = FVector(-1.0, 0.0, 0.0);
	NegXParameters.IncludeSource = true;
	NegXParameters.PtTransform = FTransform();
	NegXParameters.ExpectedPointCount = 3;
	NegXParameters.bCheckLocation = true;
	NegXParameters.bCheckRotation = false;
	NegXParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(NegXParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_DirectionY, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.DirectionY", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_DirectionY::RunTest(const FString& Parameters)
{
	FTestParameters YParameters{};

	YParameters.Iterations = 2;
	YParameters.Direction = FVector(0.0, 1.0, 0.0);
	YParameters.IncludeSource = true;
	YParameters.PtTransform = FTransform();
	YParameters.ExpectedPointCount = 3;
	YParameters.bCheckLocation = true;
	YParameters.bCheckRotation = false;
	YParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(YParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_DirectionNegY, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.DirectionNegY", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_DirectionNegY::RunTest(const FString& Parameters)
{
	FTestParameters NegYParameters{};

	NegYParameters.Iterations = 2;
	NegYParameters.Direction = FVector(0.0, -1.0, 0.0);
	NegYParameters.IncludeSource = true;
	NegYParameters.PtTransform = FTransform();
	NegYParameters.ExpectedPointCount = 3;
	NegYParameters.bCheckLocation = true;
	NegYParameters.bCheckRotation = false;
	NegYParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(NegYParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_DirectionZ, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.DirectionZ", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_DirectionZ::RunTest(const FString& Parameters)
{
	FTestParameters ZParameters{};

	ZParameters.Iterations = 2;
	ZParameters.Direction = FVector(0.0, 0.0, 1.0);
	ZParameters.IncludeSource = true;
	ZParameters.PtTransform = FTransform();
	ZParameters.ExpectedPointCount = 3;
	ZParameters.bCheckLocation = true;
	ZParameters.bCheckRotation = false;
	ZParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(ZParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_DirectionNegZ, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.DirectionNegZ", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_DirectionNegZ::RunTest(const FString& Parameters)
{
	FTestParameters NegZParameters{};

	NegZParameters.Iterations = 2;
	NegZParameters.Direction = FVector(0.0, 0.0, -1.0);
	NegZParameters.IncludeSource = true;
	NegZParameters.PtTransform = FTransform();
	NegZParameters.ExpectedPointCount = 3;
	NegZParameters.bCheckLocation = true;
	NegZParameters.bCheckRotation = false;
	NegZParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(NegZParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_ScaleOne, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.ScaleOne", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_ScaleOne::RunTest(const FString& Parameters)
{
	FTestParameters ScaleParameters{};

	ScaleParameters.Iterations = 2;
	ScaleParameters.Direction = FVector(0.0, 0.0, 1.0);
	ScaleParameters.IncludeSource = true;
	ScaleParameters.PtTransform = FTransform();
	ScaleParameters.ExpectedPointCount = 3;
	ScaleParameters.bCheckLocation = false;
	ScaleParameters.bCheckRotation = false;
	ScaleParameters.bCheckScale = true;

	return GenerateTestDataRunAndValidate(ScaleParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_ScaleNegative, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.ScaleNegative", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_ScaleNegative::RunTest(const FString& Parameters)
{
	FTestParameters NegScaleParameters{};

	NegScaleParameters.Iterations = 2;
	NegScaleParameters.Direction = FVector(0.0, 0.0, 1.0);
	NegScaleParameters.IncludeSource = true;
	NegScaleParameters.PtTransform = FTransform(FRotator::ZeroRotator, FVector::ZeroVector, FVector(-1.0));
	NegScaleParameters.ExpectedPointCount = 3;
	NegScaleParameters.bCheckLocation = false;
	NegScaleParameters.bCheckRotation = false;
	NegScaleParameters.bCheckScale = true;

	return GenerateTestDataRunAndValidate(NegScaleParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_Rotation, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.Rotation", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_Rotation::RunTest(const FString& Parameters)
{
	FTestParameters RotationParameters{};

	RotationParameters.Iterations = 2;
	RotationParameters.Direction = FVector(0.0, 0.0, 1.0);
	RotationParameters.IncludeSource = true;
	RotationParameters.PtTransform = FTransform(FRotator(20.0, 0.0, 0.0), FVector::ZeroVector, FVector::One());
	RotationParameters.ExpectedPointCount = 3;
	RotationParameters.bCheckLocation = false;
	RotationParameters.bCheckRotation = true;
	RotationParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(RotationParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_LocationX, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.LocationX", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_LocationX::RunTest(const FString& Parameters)
{
	FTestParameters LocationXParameters{};

	LocationXParameters.Iterations = 2;
	LocationXParameters.Direction = FVector(0.0, 0.0, 1.0);
	LocationXParameters.IncludeSource = true;
	LocationXParameters.PtTransform = FTransform(FRotator::ZeroRotator, FVector(5.0, 0.0, 0.0), FVector::One());
	LocationXParameters.ExpectedPointCount = 3;
	LocationXParameters.bCheckLocation = true;
	LocationXParameters.bCheckRotation = false;
	LocationXParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(LocationXParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_LocationY, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.LocationY", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_LocationY::RunTest(const FString& Parameters)
{
	FTestParameters LocationYParameters{};

	LocationYParameters.Iterations = 2;
	LocationYParameters.Direction = FVector(0.0, 0.0, 1.0);
	LocationYParameters.IncludeSource = true;
	LocationYParameters.PtTransform = FTransform(FRotator::ZeroRotator, FVector(0.0, 5.0, 0.0), FVector::One());
	LocationYParameters.ExpectedPointCount = 3;
	LocationYParameters.bCheckLocation = true;
	LocationYParameters.bCheckRotation = false;
	LocationYParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(LocationYParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGDuplicatePointTest_LocationZ, DuplicatePointTestBase, "Plugins.PCG.DuplicatePoint.LocationZ", PCGTestsCommon::TestFlags)

bool FPCGDuplicatePointTest_LocationZ::RunTest(const FString& Parameters)
{
	FTestParameters LocationZParameters{};

	LocationZParameters.Iterations = 2;
	LocationZParameters.Direction = FVector(0.0, 0.0, 1.0);
	LocationZParameters.IncludeSource = true;
	LocationZParameters.PtTransform = FTransform(FRotator::ZeroRotator, FVector(0.0, 0.0, 5.0), FVector::One());
	LocationZParameters.ExpectedPointCount = 3;
	LocationZParameters.bCheckLocation = true;
	LocationZParameters.bCheckRotation = false;
	LocationZParameters.bCheckScale = false;

	return GenerateTestDataRunAndValidate(LocationZParameters);
}