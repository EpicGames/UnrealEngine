// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Elements/PCGElementTest.h"

#include "Tests/PCGTestsCommon.h"
#include "PCGData.h"
#include "PCGContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGElementTest)

UPCGBadOutputsNodeSettings::UPCGBadOutputsNodeSettings()
{
#if WITH_EDITORONLY_DATA
	bExposeToLibrary = false;
#endif
}

FPCGElementPtr UPCGBadOutputsNodeSettings::CreateElement() const
{	
	return MakeShared<FPCGBadOutputNodeElement>();
}

bool FPCGBadOutputNodeElement::ExecuteInternal(FPCGContext* Context) const
{
	// add some null outputs
	FPCGTaggedData& BadOutput = Context->OutputData.TaggedData.Add_GetRef({});
	BadOutput.Pin = PCGPinConstants::DefaultOutputLabel;

	return true;
}

#if WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGElementTest_CleanupAndValidateOutput, FPCGTestBaseClass, "Plugins.PCG.PCGElement.CleanupAndValidateOutput", PCGTestsCommon::TestFlags)

bool FPCGElementTest_CleanupAndValidateOutput::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGBadOutputsNodeSettings>(TestData);

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	AddExpectedError(TEXT("Invalid output\\(s\\) generated for pin"));

	while (!TestElement->Execute(Context.Get())) {}

	// check we have no null outputs

	TArray<FPCGTaggedData> Outputs = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);

	UTEST_EQUAL("Null entries should have been removed", Outputs.Num(), 0);

	return true;
}

#endif // WITH_EDITOR
