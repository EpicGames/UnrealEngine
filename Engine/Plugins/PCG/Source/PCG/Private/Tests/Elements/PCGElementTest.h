// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "PCGElement.h"
#include "PCGElementTest.generated.h"


/**
 * Test node to write bad outputs
 */
UCLASS(HideDropdown, NotPlaceable, MinimalAPI)
class UPCGBadOutputsNodeSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGBadOutputsNodeSettings();

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("BadOutputs")); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

	virtual FPCGElementPtr CreateElement() const override;

};

class FPCGBadOutputNodeElement : public IPCGElement
{
public:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
