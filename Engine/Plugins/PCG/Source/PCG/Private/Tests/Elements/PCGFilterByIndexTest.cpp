// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "PCGData.h"
#include "Data/PCGBasePointData.h"

#include "Elements/PCGFilterByIndex.h"
#include "Elements/PCGFilterElementsByIndex.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

namespace PCGFilterByIndexTest
{
	struct FInsideOutsideFilterData
	{
		FPCGTaggedData InsideData;
		FPCGTaggedData OutsideData;
	};

	namespace Constants
	{
		static constexpr int32 InputNum = 5;
		static const FLazyName OriginalIndexName = "OriginalIndex";
		static const FLazyName FilterIndexName = "FilterIndex";
	}

	namespace Helpers
	{
		TArray<FPCGTaggedData> ResetAndExecuteData(PCGTestsCommon::FTestData& TestData, UPCGFilterByIndexSettings* Settings)
		{
			TestData.Reset(Settings);
			const FPCGElementPtr TestElement = Settings->GetElement();

			for (int Index = 0; Index < Constants::InputNum; ++Index)
			{
				FPCGTaggedData& InputData = TestData.InputData.TaggedData.Emplace_GetRef();
				InputData.Pin = PCGPinConstants::DefaultInputLabel;
				InputData.Data = PCGTestsCommon::CreateRandomBasePointData(Index + 1, 42);
			}

			TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

			while (!TestElement->Execute(Context.Get())) {}

			return Context->OutputData.GetAllInputs();
		}

		FInsideOutsideFilterData ResetAndExecuteElement(PCGTestsCommon::FTestData& TestData, UPCGFilterElementsByIndexSettings* Settings, const TFunctionRef<UPCGData*()> CreateInputPCGData, const TFunctionRef<UPCGData*()> CreateFilterPCGData)
		{
			TestData.Reset(Settings);
			const FPCGElementPtr TestElement = Settings->GetElement();
			Settings->bOutputDiscardedElements = true;

			UPCGData* PCGData = CreateInputPCGData();
			check(PCGData);
			UPCGData* FilterPCGData = CreateFilterPCGData();

			FPCGTaggedData& InputData = TestData.InputData.TaggedData.Emplace_GetRef();
			InputData.Pin = PCGPinConstants::DefaultInputLabel;
			InputData.Data = PCGData;

			if (FilterPCGData)
			{
				FPCGTaggedData& InputFilterData = TestData.InputData.TaggedData.Emplace_GetRef();
				InputFilterData.Pin = FName("Filter");
				InputFilterData.Data = FilterPCGData;
			}

			const TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

			FPCGAttributePropertyInputSelector IndexSelector;
			IndexSelector.SetExtraProperty(EPCGExtraProperties::Index);
			FPCGAttributePropertyOutputSelector OutputSelector;
			OutputSelector.SetAttributeName(Constants::OriginalIndexName);
			PCGMetadataHelpers::FPCGCopyAttributeParams CopyParams
			{
				.SourceData = PCGData,
				.TargetData = PCGData,
				.InputSource = IndexSelector,
				.OutputTarget = OutputSelector,
				.OptionalContext = Context.Get(),
				.OutputType = EPCGMetadataTypes::Integer32,
				.bSameOrigin = true
			};

			PCGMetadataHelpers::CopyAttribute(CopyParams);

			while (!TestElement->Execute(Context.Get())) {}

			TArray<FPCGTaggedData> InsideData = Context->OutputData.GetInputsByPin(FName("In Filter"));
			TArray<FPCGTaggedData> OutsideData = Context->OutputData.GetInputsByPin(FName("Out Filter"));

			check(InsideData.IsEmpty() || InsideData.Num() == 1);
			check(OutsideData.IsEmpty() || OutsideData.Num() == 1);

			FInsideOutsideFilterData FilterData
			{
				.InsideData = !InsideData.IsEmpty() ? InsideData[0] : FPCGTaggedData{},
				.OutsideData = !OutsideData.IsEmpty() ? OutsideData[0] : FPCGTaggedData{}
			};

			return FilterData;
		}

		bool ValidateOutputData(const TArray<FPCGTaggedData>& Outputs)
		{
			check(Outputs.Num() == PCGFilterByIndexTest::Constants::InputNum);

			for (int I = 0; I < Constants::InputNum; ++I)
			{
				const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[I].Data);
				if (!OutPointData || (OutPointData->GetNumPoints() != I + 1))
				{
					return false;
				}
			}

			return true;
		}

		bool ValidateOutputElement(const FPCGTaggedData& Output, const int32 ExpectedElementNum, const TSet<int32>& MatchedIndices)
		{
			if (ExpectedElementNum == 0 && !Output.Data)
			{
				return true;
			}

			if (!Output.Data || PCGHelpers::GetNumberOfElements(Output.Data) != ExpectedElementNum)
			{
				return false;
			}

			// Data exists, no elements expected, early out.
			if (ExpectedElementNum == 0)
			{
				return true;
			}

			FPCGAttributePropertyInputSelector Selector;
			Selector.SetAttributeName(Constants::OriginalIndexName);
			TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Output.Data, Selector);
			TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Output.Data, Selector);
			if (!Accessor || !Keys || Keys->GetNum() != ExpectedElementNum)
			{
				return false;
			}

			TArray<int32> OriginalIndices;
			OriginalIndices.SetNumUninitialized(Keys->GetNum());
			if (!Accessor->GetRange(MakeArrayView(OriginalIndices), 0, *Keys))
			{
				return false;
			}

			for (const int32 Index : MatchedIndices)
			{
				if (!OriginalIndices.Contains(Index))
				{
					return false;
				}
			}

			return true;
		}
	} // namespace Helpers
} // namespace PCGFilterByIndexTest

using namespace PCGFilterByIndexTest;

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGFilterByIndexTest_Basic, FPCGTestBaseClass, "Plugins.PCG.FilterByIndex.Basic", PCGTestsCommon::TestFlags)

bool FPCGFilterByIndexTest_Basic::RunTest(const FString& Parameters)
{
	using Constants::InputNum;

	PCGTestsCommon::FTestData TestData;
	UPCGFilterByIndexSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGFilterByIndexSettings>(TestData);

	// Empty index input string
	{
		// Empty will throw a warning
		AddExpectedError(TEXT("Empty expression in parsed string"));

		Settings->SelectedIndices.Empty();
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_EQUAL("Output count", Outputs.Num(), InputNum);
		UTEST_TRUE("Output is valid", Helpers::ValidateOutputData(Outputs));

		UTEST_EQUAL("Output 1 filtered exclusively", Outputs[0].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 2 filtered exclusively", Outputs[1].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 3 filtered exclusively", Outputs[2].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 4 filtered exclusively", Outputs[3].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 5 filtered exclusively", Outputs[4].Pin, PCGPinConstants::DefaultOutFilterLabel);
	}

	// Explicit index selection
	{
		Settings->SelectedIndices = FString("0,2,4");
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_EQUAL("Output count", Outputs.Num(), InputNum);
		UTEST_TRUE("Output is valid", Helpers::ValidateOutputData(Outputs));

		UTEST_EQUAL("Output 1 filtered inclusively", Outputs[0].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 2 filtered exclusively", Outputs[1].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 3 filtered inclusively", Outputs[2].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 4 filtered exclusively", Outputs[3].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 5 filtered inclusively", Outputs[4].Pin, PCGPinConstants::DefaultInFilterLabel);
	}

	// Negative index selection. 5 - 2 = 3
	{
		Settings->SelectedIndices = FString("-2");
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_EQUAL("Output count", Outputs.Num(), InputNum);
		UTEST_TRUE("Output is valid", Helpers::ValidateOutputData(Outputs));

		UTEST_EQUAL("Output 1 filtered exclusively", Outputs[0].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 2 filtered exclusively", Outputs[1].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 3 filtered exclusively", Outputs[2].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 4 filtered inclusively", Outputs[3].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 5 filtered exclusively", Outputs[4].Pin, PCGPinConstants::DefaultOutFilterLabel);
	}

	// Range index selection [1:3) = [1:2] since range slicing doesn't include last index
	{
		Settings->SelectedIndices = FString("1:3");
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_EQUAL("Output count", Outputs.Num(), InputNum);
		UTEST_TRUE("Output is valid", Helpers::ValidateOutputData(Outputs));

		UTEST_EQUAL("Output 1 filtered exclusively", Outputs[0].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 2 filtered inclusively", Outputs[1].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 3 filtered inclusively", Outputs[2].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 4 filtered exclusively", Outputs[3].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 5 filtered exclusively", Outputs[4].Pin, PCGPinConstants::DefaultOutFilterLabel);
	}

	// Range index selection [2:2] = [2]
	{
		Settings->SelectedIndices = FString("2:2");
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_EQUAL("Output count", Outputs.Num(), InputNum);
		UTEST_TRUE("Output is valid", Helpers::ValidateOutputData(Outputs));

		UTEST_EQUAL("Output 1 filtered exclusively", Outputs[0].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 2 filtered exclusively", Outputs[1].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 3 filtered inclusively", Outputs[2].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 4 filtered exclusively", Outputs[3].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 5 filtered exclusively", Outputs[4].Pin, PCGPinConstants::DefaultOutFilterLabel);
	}

	// Range index selection (should be [5-3,5-1) = [2,4) = [2:3]
	{
		Settings->SelectedIndices = FString("-3:-1");
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_EQUAL("Output count", Outputs.Num(), InputNum);
		UTEST_TRUE("Output is valid", Helpers::ValidateOutputData(Outputs));

		UTEST_EQUAL("Output 1 filtered exclusively", Outputs[0].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 2 filtered exclusively", Outputs[1].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 3 filtered inclusively", Outputs[2].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 4 filtered inclusively", Outputs[3].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 5 filtered exclusively", Outputs[4].Pin, PCGPinConstants::DefaultOutFilterLabel);
	}

	// Combination index selection
	{
		Settings->SelectedIndices = FString("0,2:3,4");
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_EQUAL("Output count", Outputs.Num(), InputNum);
		UTEST_TRUE("Output is valid", Helpers::ValidateOutputData(Outputs));

		UTEST_EQUAL("Output 1 filtered inclusively", Outputs[0].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 2 filtered exclusively", Outputs[1].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 3 filtered inclusively", Outputs[2].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 4 filtered exclusively", Outputs[3].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 5 filtered inclusively", Outputs[4].Pin, PCGPinConstants::DefaultInFilterLabel);
	}

	// Combination index selection, with inverted filter
	{
		Settings->bInvertFilter = true;
		Settings->SelectedIndices = FString("0,2:3,4");
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_EQUAL("Output count", Outputs.Num(), InputNum);
		UTEST_TRUE("Output is valid", Helpers::ValidateOutputData(Outputs));

		UTEST_EQUAL("Output 1 filtered exclusively", Outputs[0].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 2 filtered inclusively", Outputs[1].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 3 filtered exclusively", Outputs[2].Pin, PCGPinConstants::DefaultOutFilterLabel);
		UTEST_EQUAL("Output 4 filtered inclusively", Outputs[3].Pin, PCGPinConstants::DefaultInFilterLabel);
		UTEST_EQUAL("Output 5 filtered exclusively", Outputs[4].Pin, PCGPinConstants::DefaultOutFilterLabel);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGFilterByIndexTest_InvalidSelection, FPCGTestBaseClass, "Plugins.PCG.FilterByIndex.InvalidSelection", PCGTestsCommon::TestFlags)

bool FPCGFilterByIndexTest_InvalidSelection::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGFilterByIndexSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGFilterByIndexSettings>(TestData);

	AddExpectedError("Invalid expression in parsed string:", EAutomationExpectedMessageFlags::Contains, 2);

	// Test inverted range
	{
		Settings->SelectedIndices = FString("4:0");
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_TRUE("Output count", Outputs.IsEmpty());
	}

	// Test inverted range with negative indices
	{
		Settings->SelectedIndices = FString("-1:-3");
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_TRUE("Output count", Outputs.IsEmpty());
	}

	AddExpectedError("Invalid character in parsed string:");

	// Test invalid character
	{
		Settings->SelectedIndices = FString("0,2:3,4*");
		const TArray<FPCGTaggedData> Outputs = Helpers::ResetAndExecuteData(TestData, Settings);

		UTEST_TRUE("Output count", Outputs.IsEmpty());
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGFilterElementsByIndexTest_Basic, FPCGTestBaseClass, "Plugins.PCG.FilterElementsByIndex.Basic", PCGTestsCommon::TestFlags)

bool FPCGFilterElementsByIndexTest_Basic::RunTest(const FString& Parameters)
{
	using Constants::InputNum;

	PCGTestsCommon::FTestData TestData;
	UPCGFilterElementsByIndexSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGFilterElementsByIndexSettings>(TestData);
	Settings->bSelectIndicesByInput = false;

	int32 NumPoints = 10;
	auto CreatePointData = [NumPoints] { return PCGTestsCommon::CreateRandomBasePointData(NumPoints, 42); };

	// Empty index input string
	{
		// Empty will throw a warning
		AddExpectedError(TEXT("Empty expression in parsed string"));

		Settings->SelectedIndices.Empty();
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });

		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, 0, {}));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, NumPoints, {0, 1, 2, 3, 4, 5, 6, 7, 8, 9}));
	}

	// All index input string
	{
		Settings->SelectedIndices = FString(":");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });

		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, NumPoints, {0, 1, 2, 3, 4, 5, 6, 7, 8, 9}));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, 0, {}));
	}

	// Explicit index selection
	{
		Settings->SelectedIndices = FString("0,2,4");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });

		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, 3, {0, 2, 4}));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, 7, {1, 3, 5, 6, 7, 8, 9}));
	}

	// Negative index selection. 10 - 2 = 8
	{
		Settings->SelectedIndices = FString("-2");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });

		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, 1, {8}));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, 9, {1, 2, 3, 4, 5, 6, 7, 9}));
	}

	// Range index selection [1:3) = [1:2] since range slicing doesn't include last index
	{
		Settings->SelectedIndices = FString("1:3");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });

		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, 2, {1, 2}));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, 8, {0, 3, 4, 5, 6, 7, 8, 9}));
	}

	// Range index selection [2:2] = [2]
	{
		Settings->SelectedIndices = FString("2:2");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });

		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, 1, {2}));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, 9, {1, 3, 4, 5, 6, 7, 8, 9}));
	}

	// Range index selection (should be [10-3,10-1) = [7,9) = [7:8]
	{
		Settings->SelectedIndices = FString("-3:-1");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });

		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, 2, {7, 8}));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, 8, {1, 2, 3, 4, 5, 6, 9}));
	}

	// Combination index selection
	{
		Settings->SelectedIndices = FString("0,2:3,4");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });

		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, 3, {0, 2, 4}));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, 7, {1, 3, 5, 6, 7, 8, 9}));
	}

	// Combination index selection, with inverted filter
	{
		Settings->bInvertFilter = true;
		Settings->SelectedIndices = FString("0,2:3,4");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });

		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, 7, {1, 3, 5, 6, 7, 8, 9}));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, 3, {0, 2, 4}));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGFilterElementsByIndexTest_InvalidSelection, FPCGTestBaseClass, "Plugins.PCG.FilterElementsByIndex.InvalidSelection", PCGTestsCommon::TestFlags)

bool FPCGFilterElementsByIndexTest_InvalidSelection::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGFilterElementsByIndexSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGFilterElementsByIndexSettings>(TestData);
	Settings->bSelectIndicesByInput = false;

	int32 NumPoints = 10;
	auto CreatePointData = [NumPoints] { return PCGTestsCommon::CreateRandomBasePointData(NumPoints, 42); };

	AddExpectedError("Invalid expression in parsed string:", EAutomationExpectedMessageFlags::Contains, 2);

	// Test inverted range
	{
		Settings->SelectedIndices = FString("4:0");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });
	}

	// Test inverted range with negative indices
	{
		Settings->SelectedIndices = FString("-1:-3");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });
	}

	AddExpectedError("Invalid character in parsed string:");

	// Test invalid character
	{
		Settings->SelectedIndices = FString("0,2:3,4*");
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [] { return nullptr; });
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGFilterElementsByIndexTest_InputFilter, FPCGTestBaseClass, "Plugins.PCG.FilterElementsByIndex.InputFilter", PCGTestsCommon::TestFlags)

bool FPCGFilterElementsByIndexTest_InputFilter::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGFilterElementsByIndexSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGFilterElementsByIndexSettings>(TestData);
	Settings->bSelectIndicesByInput = true;

	static const TArray<int32> FilteredIndices = {0, 3, 5};
	static const TArray<int32> UnfilteredIndices = {1, 2, 4, 6, 7, 8, 9};

	int32 NumPoints = 10;
	int32 NumFilterPoints = 3;
	auto CreatePointData = [NumPoints] { return PCGTestsCommon::CreateRandomBasePointData(NumPoints, 42); };
	auto CreateFilterPointData = [&FilteredIndices = FilteredIndices, NumFilterPoints](const FName AttributeName)
	{
		UPCGBasePointData* FilterPointData = PCGTestsCommon::CreateRandomBasePointData(NumFilterPoints, 42);
		UPCGMetadata* Metadata = FilterPointData ? FilterPointData->MutableMetadata() : nullptr;
		FPCGMetadataDomain* DomainMetadata = Metadata ? Metadata->GetDefaultMetadataDomain() : nullptr;
		check(DomainMetadata);
		check(FilterPointData->GetNumPoints() == FilteredIndices.Num());

		const TPCGValueRange<int64> Range = FilterPointData->GetMetadataEntryValueRange(true);
		check(Range.Num() == FilteredIndices.Num());

		FPCGMetadataAttribute<int>* Attribute = DomainMetadata->CreateAttribute<int32>(AttributeName, INDEX_NONE, true, true);

		int Index = 0;
		for (int64& EntryKey : Range)
		{
			DomainMetadata->InitializeOnSet(EntryKey);
			Attribute->SetValue(EntryKey, FilteredIndices[Index++]);
		}

		return FilterPointData;
	};

	// Filter indices from $Index
	{
		Settings->IndexSelectionAttribute.SetExtraProperty(EPCGExtraProperties::Index);
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [&CreateFilterPointData] { return CreateFilterPointData(Constants::OriginalIndexName); });

		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, 3, {0, 1, 2}));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, 7, {3, 4, 5, 6, 7, 8, 9}));
	}

	// Filter indices from attribute 'FilterIndex'
	{
		Settings->IndexSelectionAttribute.SetAttributeName(Constants::FilterIndexName);
		FInsideOutsideFilterData OutputData = Helpers::ResetAndExecuteElement(TestData, Settings, CreatePointData, [&CreateFilterPointData] { return CreateFilterPointData(Constants::FilterIndexName); });
		UTEST_TRUE("Output inside filter is valid", Helpers::ValidateOutputElement(OutputData.InsideData, 3, TSet(FilteredIndices)));
		UTEST_TRUE("Output outside filter is valid", Helpers::ValidateOutputElement(OutputData.OutsideData, 7, TSet(UnfilteredIndices)));
	}

	return true;
}
