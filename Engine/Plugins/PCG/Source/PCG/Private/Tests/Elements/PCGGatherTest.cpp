// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"

#include "Elements/PCGGather.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGatherTest_Basic, FPCGTestBaseClass, "Plugins.PCG.Gather.Basic", PCGTestsCommon::TestFlags)

bool FPCGGatherTest_Basic::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGGatherSettings>(TestData);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	{
		FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
		SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
		SourcePin.Data = PCGTestsCommon::CreateRandomBasePointData(100, 42);
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData *OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 100);

	return true;
}

