// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Elements/PCGGetActorPropertyTest.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGParamData.h"
#include "PCGVolume.h"
#include "Elements/PCGGetActorProperty.h"
#include "Tests/PCGTestsCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetActorPropertyTest)

#if WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPropertyToParamDataPropertyTypeTest, FPCGTestBaseClass, "Plugins.PCG.PropertyToParamData.PropertyType", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPropertyToParamDataActorFindTest, FPCGTestBaseClass, "Plugins.PCG.PropertyToParamData.ActorFind", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPropertyToParamDataFullExtractionTest, FPCGTestBaseClass, "Plugins.PCG.PropertyToParamData.FullExtraction", PCGTestsCommon::TestFlags)


/**
* Generic method to test the extraction and then verify that the values matches the one passed as input.
* Use variadic template for expected values, allowing to test the extraction of structs/objects that doesn't have the same type for all their members.
* bShouldFail is used as a constexpr bool to cut the branch where we try to compare the values, if it is expected to fail.
* Be careful, if you extract structures, all int properties are int64 (even if they were like u8) and all float will be double. Take that into account in the ExpectedValues types.
* 
* @param TestInstance - Instance of the test, used for our test conditions
* @param TestData - Test data used to run the element
* @param PropertyName - Property name to extract
* @param AttributeNames - List of all the attributes to test. AttributeNames count MUST divide the count of expected values. If there is more than one attribute name, it will be marked as extracting structures
* @param ExtraTestWhat - Simple string to append to all test strings, to know which test we are currently running.
* @param ExpectedValues - List of all values to test. If there is more than one, it can be because it is an extraction, or testing an array. If it is an array of struct, and attribute names is {'X', 'Y'}, it must follow this order: A[0].X, A[0].Y, A[1].X, A[1].Y, ...
* @return if the test succeeded or not.
*/
template <bool bShouldFail, typename ...AttributeTypes>
bool VerifyAttributeValue(FPCGTestBaseClass* TestInstance, PCGTestsCommon::FTestData& TestData, FName PropertyName, TArray<FName> AttributeNames, FString ExtraTestWhat, AttributeTypes&& ...ExpectedValues)
{
	// Use TestData settings to set the property name
	UPCGGetActorPropertySettings* Settings = Cast<UPCGGetActorPropertySettings>(TestData.Settings);
	if (!TestInstance->TestNotNull(TEXT("CastToUPCGPropertyToParamDataSettings"), Settings))
	{
		return false;
	}

	// Make sure this call is not ill-formed
	check(!AttributeNames.IsEmpty());
	if (!ensureMsgf(sizeof...(ExpectedValues) % AttributeNames.Num() == 0, TEXT("%s"), TEXT("Call is ill-formed, there is a mismatch between the number of attribute names and the number of expected values.")))
	{
		return false;
	}

	Settings->PropertyName = PropertyName;
	Settings->OutputAttributeName.SetAttributeName(AttributeNames[0]);
	// Deprecated field bForceObjectAndStructExtraction was testing object extraction only.
	if (AttributeNames.Num() > 1)
	{
		Settings->ObjectExtractorBehavior = EPCGObjectExtractorBehavior::Extract;
	}
	else
	{
		Settings->ObjectExtractorBehavior = EPCGObjectExtractorBehavior::NoExtract;
	}

	// Add 2 nodes, PropertyToParamDataNode and a Trivial node (just there for the connection)
	UPCGNode* TestNode = TestData.TestPCGComponent->GetGraph()->AddNode(Settings);

	UPCGTrivialSettings* TrivialSettings = NewObject<UPCGTrivialSettings>();
	UPCGNode* TrivialNode = TestData.TestPCGComponent->GetGraph()->AddNode(TrivialSettings);

	TestNode->GetOutputPins()[0]->AddEdgeTo(TrivialNode->GetInputPins()[0]);

	FPCGElementPtr Element = Settings->GetElement();
	TUniquePtr<FPCGContext> Context = PCGTestsCommon::InitializeTestContext(Element.Get(), FPCGDataCollection(), TestData.TestPCGComponent, TestNode);

	// Run the element, can throw an error.
	while (!Element->Execute(Context.Get()))
	{
	};

	auto FormatWithPropertyName = [&ExtraTestWhat, PropertyName](const FString& InText) -> FString
	{
		return FString::Format(TEXT("{0}_{1}_{2}"), { ExtraTestWhat, PropertyName.ToString(), InText });
	};

	auto FormatWithPropertyNameAndSubNames = [&ExtraTestWhat, PropertyName](const FString& InText, const FName InSubName, const int InItemKey) -> FString
	{
		return FString::Format(TEXT("{0}_{1}_SubName:{2}_ItemKey:{3} - {4}"), { ExtraTestWhat, PropertyName.ToString(), InSubName.ToString(), FString::FromInt(InItemKey), InText });
	};

	bool bSuccess = true;

	if constexpr (bShouldFail)
	{
		// If it should fail, it will have no output.
		bSuccess = TestInstance->TestEqual(FormatWithPropertyName("NumOutput"), Context->OutputData.GetAllParams().Num(), 0);
	}
	else
	{
		// If it should succeed, we check the number of output.
		bSuccess = TestInstance->TestEqual(FormatWithPropertyName("NumOutput"), Context->OutputData.GetAllParams().Num(), 1);

		if (bSuccess)
		{
			const UPCGParamData* ParamData = CastChecked<UPCGParamData>(Context->OutputData.GetAllParams()[0].Data);
			const UPCGMetadata* Metadata = ParamData->ConstMetadata();
			check(Metadata);

			int32 Index = 0;
			int32 ItemKey = 0;

			// Make sure that we have exactly the same number of attributes as expected
			if (!TestInstance->TestEqual(FormatWithPropertyName("Attribute number is matching"), Metadata->GetAttributeCount(), AttributeNames.Num()))
			{
				// Cleanup behind ourselves.
				TestData.TestPCGComponent->GetGraph()->RemoveNode(TrivialNode);
				TestData.TestPCGComponent->GetGraph()->RemoveNode(TestNode);

				return false;
			}

			// Fold expression that will iterate on all Expected Values, to check their types and their values.
			// We keep track of the current Index, to know which attribute to check, and an ItemKey to know where to look at in the value array of the attribute.
			// cf. comment at the top to know in which order ExpectedValues should be.
			([&Index, &AttributeNames, Metadata, &FormatWithPropertyNameAndSubNames, TestInstance, &bSuccess, &ItemKey, &ExpectedValues]()
			{
				if (bSuccess)
				{
					// In a fold expression, ExpectedValues will be a single value. We know its underlying type by decaying the type of the value (using decltype)
					using AttributeType = typename std::decay_t<decltype(ExpectedValues)>;

					// We use Index to track the current attribute to check
					const FName AttributeName = AttributeNames[Index];
					const FPCGMetadataAttributeBase* Attribute = Metadata->GetConstAttribute(AttributeName);

					bSuccess = TestInstance->TestNotNull(FormatWithPropertyNameAndSubNames("Attribute should not be null", AttributeName, ItemKey), Attribute);

					if (bSuccess)
					{
						bSuccess = TestInstance->TestEqual(FormatWithPropertyNameAndSubNames("Attribute type is matching", AttributeName, ItemKey), Attribute->GetTypeId(), PCG::Private::MetadataTypes<AttributeType>::Id);
					}

					if (bSuccess)
					{
						AttributeType Value = static_cast<const FPCGMetadataAttribute<AttributeType>*>(Attribute)->GetValueFromItemKey(ItemKey);
						bSuccess = TestInstance->TestEqual(FormatWithPropertyNameAndSubNames("Attribute value is the same as expected", AttributeName, ItemKey), Value, ExpectedValues);
					}

					// At each iteration on the ExpectedValues, we increment the index and if we reached the end of the Attribute Names we reset it to 0
					// and we increment ItemKey, to check the next value in the attribute value array.
					if (++Index >= AttributeNames.Num())
					{
						Index = 0;
						ItemKey++;
					}
				}
			}(), ...);
		}
	}

	// Cleanup behind ourselves.
	TestData.TestPCGComponent->GetGraph()->RemoveNode(TrivialNode);
	TestData.TestPCGComponent->GetGraph()->RemoveNode(TestNode);

	return bSuccess;
}

// Use aliases for convenience

// Single valid attribute, Property Name will be attribute name
template <typename AttributeType>
bool VerifyAttributeValueValid(FPCGTestBaseClass* TestInstance, PCGTestsCommon::FTestData& TestData, FName PropertyName, AttributeType&& ExpectedValue, FString ExtraTestWhat)
{
	return VerifyAttributeValue</*bShouldFail=*/false>(TestInstance, TestData, PropertyName, { PropertyName }, ExtraTestWhat, std::forward<AttributeType>(ExpectedValue));
}

// Single invalid attribute, Property Name will be attribute name
template <typename AttributeType>
bool VerifyAttributeValueInvalid(FPCGTestBaseClass* TestInstance, PCGTestsCommon::FTestData& TestData, FName PropertyName, AttributeType&& ExpectedValue, FString ExtraTestWhat)
{
	return VerifyAttributeValue</*bShouldFail=*/true>(TestInstance, TestData, PropertyName, { PropertyName }, ExtraTestWhat, std::forward<AttributeType>(ExpectedValue));
}

// Valid Single or multi attributes with single or multiple values.
template <typename ...AttributeTypes>
bool VerifyAttributeValuesValid(FPCGTestBaseClass* TestInstance, PCGTestsCommon::FTestData& TestData, FName PropertyName, const TArray<FName>& AttributeNames, FString ExtraTestWhat, AttributeTypes&& ...ExpectedValues)
{
	return VerifyAttributeValue</*bShouldFail=*/false>(TestInstance, TestData, PropertyName, AttributeNames, ExtraTestWhat, std::forward<AttributeTypes>(ExpectedValues)...);
}

bool FPCGPropertyToParamDataPropertyTypeTest::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	UPCGGetActorPropertySettings* Settings = NewObject<UPCGGetActorPropertySettings>();
	Settings->ActorSelector.ActorSelection = EPCGActorSelection::ByClass;
	Settings->ActorSelector.ActorSelectionClass = APCGUnitTestDummyActor::StaticClass();
	Settings->ActorSelector.ActorFilter = EPCGActorFilter::Self;

	static constexpr int32 Seed = 42;
	const FString ExtraTestWhat = "PropertyToParamDataPropertyTypeTest";

	PCGTestsCommon::FTestData TestData(Seed, Settings, APCGUnitTestDummyActor::StaticClass());

	// Set all properties
	const FName NameValue = TEXT("HelloWorld");
	const FString StringValue = TEXT("HelloWorld");

	const FVector VectorValue{ 1.0, 2.0, 3.0 };
	const FVector SecondVectorValue{ 4.0, 5.0, 6.0 };
	const FVector4 Vector4Value{ 1.0, 2.0, 3.0, 4.0 };
	const FRotator RotatorValue{ 45.0, 45.0, 45.0 };
	const FQuat QuatValue = RotatorValue.Quaternion();
	const FTransform TransformValue{ QuatValue, VectorValue, VectorValue };

	UPCGDummyGetPropertyTest* ObjectValue = NewObject<UPCGDummyGetPropertyTest>();
	ObjectValue->SetFlags(RF_Transient);
	ObjectValue->Int64Property = 42ll;
	ObjectValue->DoubleProperty = 1.0;

	UPCGDummyGetPropertyTest* SecondObjectValue = NewObject<UPCGDummyGetPropertyTest>();
	SecondObjectValue->SetFlags(RF_Transient);
	SecondObjectValue->Int64Property = 43ll;
	SecondObjectValue->DoubleProperty = 2.0;

	const FSoftObjectPath SoftObjectPathValue{ ObjectValue };
	const FSoftClassPath SoftClassPathValue{ UPCGDummyGetPropertyTest::StaticClass() };

	const FVector2D Vector2Value = { 1.0, 2.0 };
	const FPCGTestMyColorStruct PCGColorValue{ 1.0, 1.0, 0.0, 1.0 };
	const FPCGTestMyColorStruct SecondPCGColorValue{ 1.0, 0.0, 1.0, 1.0 };
	const FColor ColorValue = FColor::White;
	const FLinearColor LinearColorValue = FLinearColor::Blue;

	APCGUnitTestDummyActor* Actor = Cast<APCGUnitTestDummyActor>(TestData.TestActor);
	Actor->IntProperty = 42;
	Actor->Int64Property = 42ll;
	Actor->FloatProperty = 1.0f;
	Actor->DoubleProperty = 1.0;
	Actor->BoolProperty = true;
	Actor->NameProperty = NameValue;
	Actor->StringProperty = StringValue;
	Actor->EnumProperty = EPCGUnitTestDummyEnum::Three;
	Actor->VectorProperty = VectorValue;
	Actor->Vector4Property = Vector4Value;
	Actor->RotatorProperty = RotatorValue;
	Actor->QuatProperty = QuatValue;
	Actor->TransformProperty = TransformValue;
	Actor->SoftObjectPathProperty = SoftObjectPathValue;
	Actor->SoftClassPathProperty = SoftClassPathValue;
	Actor->ClassProperty = UPCGDummyGetPropertyTest::StaticClass();
	Actor->ObjectProperty = ObjectValue;
	Actor->Vector2Property = Vector2Value;
	Actor->ColorProperty = ColorValue;
	Actor->LinearColorProperty = LinearColorValue;
	Actor->PCGColorProperty = PCGColorValue;
	Actor->ArrayOfIntsProperty = { 42, 43, 44 };
	Actor->ArrayOfVectorsProperty = { VectorValue, SecondVectorValue };
	Actor->ArrayOfStructsProperty = { PCGColorValue, SecondPCGColorValue };
	Actor->ArrayOfObjectsProperty = { ObjectValue, SecondObjectValue };
	Actor->SetOfIntsProperty = {1, 2, 3};
	Actor->DummyStruct.FloatProperty = 1.2f;
	Actor->DummyStruct.IntArrayProperty = { 5, 6, 7 };
	Actor->DummyStruct.Level2Struct.DoubleArrayProperty = { 0.1, 0.2, 0.3 };

	// Basic properties
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, IntProperty), 42, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, Int64Property), 42ll, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, FloatProperty), 1.0, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, DoubleProperty), 1.0, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, BoolProperty), true, ExtraTestWhat);

	// String/Name Properties
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, NameProperty), NameValue, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, StringProperty), StringValue, ExtraTestWhat);

	// Enum Property
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, EnumProperty), (int64)EPCGUnitTestDummyEnum::Three, ExtraTestWhat);

	// Supported struct Properties
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, Vector2Property), Vector2Value, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, VectorProperty), VectorValue, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, Vector4Property), Vector4Value, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, RotatorProperty), RotatorValue, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, QuatProperty), QuatValue, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, TransformProperty), TransformValue, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, SoftObjectPathProperty), SoftObjectPathValue, ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, SoftClassPathProperty), SoftClassPathValue, ExtraTestWhat);

	// Objects properties as String
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ClassProperty), FSoftClassPath(UPCGDummyGetPropertyTest::StaticClass()), ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ObjectProperty), FSoftObjectPath(ObjectValue->GetPathName()), ExtraTestWhat);

	// Colors as Vector 4
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ColorProperty), FVector4(1.0, 1.0, 1.0, 1.0), ExtraTestWhat);
	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, LinearColorProperty), FVector4(0.0, 0.0, 1.0, 1.0), ExtraTestWhat);

	// Struct Property Extracted - Dummy Color struct
	// Extracting int will always yield a int64 and extracting floats with yield doubles. Here color is u8, so cast all of them to int64
	const TArray<FName> ColorPropertyNames = { TEXT("R"), TEXT("G"), TEXT("B"), TEXT("A") };
	bSuccess &= VerifyAttributeValuesValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, PCGColorProperty), ColorPropertyNames, ExtraTestWhat, PCGColorValue.R, PCGColorValue.G, PCGColorValue.B, PCGColorValue.A);

	// Object Property Extracted - UPCGDummyGetPropertyTest
	const TArray<FName> ObjectPropertyNames = { GET_MEMBER_NAME_CHECKED(UPCGDummyGetPropertyTest, Int64Property), GET_MEMBER_NAME_CHECKED(UPCGDummyGetPropertyTest, DoubleProperty) };
	bSuccess &= VerifyAttributeValuesValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ObjectProperty), ObjectPropertyNames, ExtraTestWhat, 42ll, 1.0);

	// Arrays of supported properties
	bSuccess &= VerifyAttributeValuesValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ArrayOfIntsProperty), { GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ArrayOfIntsProperty) }, ExtraTestWhat, 42, 43, 44);
	bSuccess &= VerifyAttributeValuesValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ArrayOfVectorsProperty), { GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ArrayOfVectorsProperty) }, ExtraTestWhat, VectorValue, SecondVectorValue);
	bSuccess &= VerifyAttributeValuesValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ArrayOfObjectsProperty), { GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ArrayOfObjectsProperty) }, ExtraTestWhat, FSoftObjectPath(ObjectValue), FSoftObjectPath(SecondObjectValue));
	bSuccess &= VerifyAttributeValuesValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, SetOfIntsProperty), { GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, SetOfIntsProperty) }, ExtraTestWhat, 1, 2, 3);

	// Arrays of extracted properties
	bSuccess &= VerifyAttributeValuesValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ArrayOfStructsProperty), ColorPropertyNames, ExtraTestWhat, 
		PCGColorValue.R, PCGColorValue.G, PCGColorValue.B, PCGColorValue.A, SecondPCGColorValue.R, SecondPCGColorValue.G, SecondPCGColorValue.B, SecondPCGColorValue.A);
	bSuccess &= VerifyAttributeValuesValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ArrayOfObjectsProperty), ObjectPropertyNames, ExtraTestWhat, ObjectValue->Int64Property, ObjectValue->DoubleProperty, SecondObjectValue->Int64Property, SecondObjectValue->DoubleProperty);

	// Extractors
	bSuccess &= VerifyAttributeValuesValid(this, TestData, TEXT("DummyStruct.FloatProperty"), { GET_MEMBER_NAME_CHECKED(FPCGDummyGetPropertyStruct, FloatProperty) }, ExtraTestWhat, 1.2);
	bSuccess &= VerifyAttributeValuesValid(this, TestData, TEXT("DummyStruct.IntArrayProperty"), { GET_MEMBER_NAME_CHECKED(FPCGDummyGetPropertyStruct, IntArrayProperty) }, ExtraTestWhat, 5, 6, 7);
	bSuccess &= VerifyAttributeValuesValid(this, TestData, TEXT("DummyStruct.Level2Struct.DoubleArrayProperty"), { GET_MEMBER_NAME_CHECKED(FPCGDummyGetPropertyLevel2Struct, DoubleArrayProperty) }, ExtraTestWhat, 0.1, 0.2, 0.3);

	// Extracting the DummyStruct should only extract the float, as arrays and deeper structs are discarded
	bSuccess &= VerifyAttributeValuesValid(this, TestData, TEXT("DummyStruct"), { GET_MEMBER_NAME_CHECKED(FPCGDummyGetPropertyStruct, FloatProperty) }, ExtraTestWhat, 1.2);

	// Unknown property
	AddExpectedError(TEXT("Property 'DummyMissingProperty' does not exist"), EAutomationExpectedErrorFlags::Contains, 1);
	AddExpectedError(TEXT("Fail to extract the property 'DummyMissingProperty' on actor"), EAutomationExpectedErrorFlags::Contains, 1);
	bSuccess &= VerifyAttributeValueInvalid(this, TestData, TEXT("DummyMissingProperty"), 42, ExtraTestWhat);

	ObjectValue->MarkAsGarbage();
	SecondObjectValue->MarkAsGarbage();

	return bSuccess;
}

bool FPCGPropertyToParamDataActorFindTest::RunTest(const FString& Parameters)
{
	bool bSuccess = true;

	static constexpr int32 Seed = 42;
	static const FName Tag = TEXT("PCGUnitTestDummyMyTag");
	static const FString Name = TEXT("MyPCGUnitTestDummyActor");

	UPCGGetActorPropertySettings* Settings = NewObject<UPCGGetActorPropertySettings>();
	Settings->ActorSelector.ActorSelectionClass = APCGUnitTestDummyActor::StaticClass();
	Settings->ActorSelector.ActorSelectionTag = Tag;
	Settings->ComponentClass = UPCGUnitTestDummyComponent::StaticClass();

	// Self by class
	{
		Settings->ActorSelector.ActorSelection = EPCGActorSelection::ByClass;
		Settings->ActorSelector.ActorFilter = EPCGActorFilter::Self;

		PCGTestsCommon::FTestData TestData(Seed, Settings, APCGUnitTestDummyActor::StaticClass());
		Cast<APCGUnitTestDummyActor>(TestData.TestActor)->IntProperty = 42;

		bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, IntProperty), 42, "PropertyToParamDataActorFindTest_Self_Class");
	}

	// Self by tag
	{
		Settings->ActorSelector.ActorSelection = EPCGActorSelection::ByTag;
		Settings->ActorSelector.ActorFilter = EPCGActorFilter::Self;

		PCGTestsCommon::FTestData TestData(Seed, Settings, APCGUnitTestDummyActor::StaticClass());
		TestData.TestActor->Tags.Add(Tag);
		Cast<APCGUnitTestDummyActor>(TestData.TestActor)->IntProperty = 42;

		bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, IntProperty), 42, "PropertyToParamDataActorFindTest_Self_Tag");
	}

	// TODO: Need a good way to spawn actors with parenting relation between them
	//// Parent by class
	//{
	//	Settings->ActorSelection = EPCGActorSelection::ByClass;
	//	Settings->ActorFilter = EPCGActorFilter::Parent;

	//	PCGTestsCommon::FTestData TestData(Seed, Settings, AActor::StaticClass());
	//	TestData.AddActor(APCGUnitTestDummyActor::StaticClass(), true);
	//	Cast<APCGUnitTestDummyActor>(TestData.TestActor)->IntProperty = 42;

	//	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, IntProperty), 42, "PropertyToParamDataActorFindTest_Parent_Class");
	//}

	//// Root by tag
	//{
	//	Settings->ActorSelection = EPCGActorSelection::ByTag;
	//	Settings->ActorFilter = EPCGActorFilter::Root;

	//	PCGTestsCommon::FTestData TestData(Seed, Settings, AActor::StaticClass());
	//	TestData.AddActor(AActor::StaticClass(), true);
	//	TestData.AddActor(APCGUnitTestDummyActor::StaticClass(), true);
	//	TestData.TestActor->Tags.Add(Tag);
	//	Cast<APCGUnitTestDummyActor>(TestData.TestActor)->IntProperty = 42;

	//	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, IntProperty), 42, "PropertyToParamDataActorFindTest_Root_Tag");
	//}

	//// Root by tag including children
	//{
	//	Settings->ActorSelection = EPCGActorSelection::ByTag;
	//	Settings->ActorFilter = EPCGActorFilter::Root;
	//	Settings->bIncludeChildren = true;

	//	PCGTestsCommon::FTestData TestData(Seed, Settings, AActor::StaticClass());
	//	TestData.AddActor(APCGUnitTestDummyActor::StaticClass(), true);
	//	TestData.TestActor->Tags.Add(Tag);
	//	Cast<APCGUnitTestDummyActor>(TestData.TestActor)->IntProperty = 42;

	//	TestData.AddActor(AActor::StaticClass(), true);

	//	bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, IntProperty), 42, "PropertyToParamDataActorFindTest_Root_Tag_Children");
	//	Settings->bIncludeChildren = false;
	//}

	// Self by class including component
	{
		Settings->ActorSelector.ActorSelection = EPCGActorSelection::ByTag;
		Settings->ActorSelector.ActorFilter = EPCGActorFilter::Self;
		Settings->bSelectComponent = true;
		
		PCGTestsCommon::FTestData TestData(Seed, Settings, AActor::StaticClass());
		TestData.TestActor->Tags.Add(Tag);
		UPCGUnitTestDummyComponent* Component = Cast<UPCGUnitTestDummyComponent>(TestData.TestActor->AddComponentByClass(Settings->ComponentClass, false, FTransform::Identity, false));
		Component->IntProperty = 42;

		bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(UPCGUnitTestDummyComponent, IntProperty), 42, "PropertyToParamDataActorFindTest_Self_Tag_Component");
		Settings->bSelectComponent = false;
	}

	// World by class
	{
		Settings->ActorSelector.ActorSelection = EPCGActorSelection::ByClass;
		Settings->ActorSelector.ActorFilter = EPCGActorFilter::AllWorldActors;

		PCGTestsCommon::FTestData TestData(Seed, Settings, APCGUnitTestDummyActor::StaticClass());
		Cast<APCGUnitTestDummyActor>(TestData.TestActor)->IntProperty = 42;

		bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, IntProperty), 42, "PropertyToParamDataActorFindTest_World_Class");
	}

	// World by tag
	{
		Settings->ActorSelector.ActorSelection = EPCGActorSelection::ByTag;
		Settings->ActorSelector.ActorFilter = EPCGActorFilter::AllWorldActors;

		PCGTestsCommon::FTestData TestData(Seed, Settings, APCGUnitTestDummyActor::StaticClass());
		TestData.TestActor->Tags.Add(Tag);
		Cast<APCGUnitTestDummyActor>(TestData.TestActor)->IntProperty = 42;

		bSuccess &= VerifyAttributeValueValid(this, TestData, GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, IntProperty), 42, "PropertyToParamDataActorFindTest_World_Tag");
	}

	return bSuccess;
}

bool FPCGPropertyToParamDataFullExtractionTest::RunTest(const FString& Parameters)
{
	UPCGGetActorPropertySettings* Settings = NewObject<UPCGGetActorPropertySettings>();
	Settings->ActorSelector.ActorSelection = EPCGActorSelection::ByClass;
	Settings->ActorSelector.ActorSelectionClass = APCGUnitTestDummyActor::StaticClass();
	Settings->ActorSelector.ActorFilter = EPCGActorFilter::Self;

	static constexpr int32 Seed = 42;

	PCGTestsCommon::FTestData TestData(Seed, Settings, APCGUnitTestDummyActor::StaticClass());

	// Set all properties
	const FName NameValue = TEXT("HelloWorld");
	const FString StringValue = TEXT("HelloWorld");

	const FVector VectorValue{ 1.0, 2.0, 3.0 };
	const FVector SecondVectorValue{ 4.0, 5.0, 6.0 };
	const FVector4 Vector4Value{ 1.0, 2.0, 3.0, 4.0 };
	const FRotator RotatorValue{ 45.0, 45.0, 45.0 };
	const FQuat QuatValue = RotatorValue.Quaternion();
	const FTransform TransformValue{ QuatValue, VectorValue, VectorValue };

	UPCGDummyGetPropertyTest* ObjectValue = NewObject<UPCGDummyGetPropertyTest>();
	ObjectValue->SetFlags(RF_Transient);
	ObjectValue->Int64Property = 42ll;
	ObjectValue->DoubleProperty = 1.0;

	UPCGDummyGetPropertyTest* SecondObjectValue = NewObject<UPCGDummyGetPropertyTest>();
	SecondObjectValue->SetFlags(RF_Transient);
	SecondObjectValue->Int64Property = 43ll;
	SecondObjectValue->DoubleProperty = 2.0;

	const FSoftObjectPath SoftObjectPathValue{ ObjectValue };
	const FSoftClassPath SoftClassPathValue{ UPCGDummyGetPropertyTest::StaticClass() };

	const FVector2D Vector2Value = { 1.0, 2.0 };
	const FPCGTestMyColorStruct PCGColorValue{ 1.0, 1.0, 0.0, 1.0 };
	const FPCGTestMyColorStruct SecondPCGColorValue{ 1.0, 0.0, 1.0, 1.0 };
	const FColor ColorValue = FColor::White;
	const FLinearColor LinearColorValue = FLinearColor::Blue;

	APCGUnitTestDummyActor* Actor = Cast<APCGUnitTestDummyActor>(TestData.TestActor);
	Actor->IntProperty = 42;
	Actor->Int64Property = 42ll;
	Actor->FloatProperty = 1.0f;
	Actor->DoubleProperty = 1.0;
	Actor->BoolProperty = true;
	Actor->NameProperty = NameValue;
	Actor->StringProperty = StringValue;
	Actor->EnumProperty = EPCGUnitTestDummyEnum::Three;
	Actor->VectorProperty = VectorValue;
	Actor->Vector4Property = Vector4Value;
	Actor->RotatorProperty = RotatorValue;
	Actor->QuatProperty = QuatValue;
	Actor->TransformProperty = TransformValue;
	Actor->SoftObjectPathProperty = SoftObjectPathValue;
	Actor->SoftClassPathProperty = SoftClassPathValue;
	Actor->ClassProperty = UPCGDummyGetPropertyTest::StaticClass();
	Actor->ObjectProperty = ObjectValue;
	Actor->Vector2Property = Vector2Value;
	Actor->ColorProperty = ColorValue;
	Actor->LinearColorProperty = LinearColorValue;

	const TArray<FName> ListOfAllExtractedValues =
	{
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, IntProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, Int64Property),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, FloatProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, DoubleProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, BoolProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, NameProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, StringProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, EnumProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, VectorProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, Vector4Property),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, RotatorProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, QuatProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, TransformProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, SoftObjectPathProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, SoftClassPathProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ClassProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ObjectProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, Vector2Property),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, ColorProperty),
		GET_MEMBER_NAME_CHECKED(APCGUnitTestDummyActor, LinearColorProperty),
		// Arrays are not extracted, neither are deeper structs.
	};

	return VerifyAttributeValuesValid(this, TestData, NAME_None, ListOfAllExtractedValues, TEXT("FullExtraction"),
		42,
		42ll,
		1.0,
		1.0,
		true,
		NameValue,
		StringValue,
		(int64)EPCGUnitTestDummyEnum::Three,
		VectorValue,
		Vector4Value,
		RotatorValue,
		QuatValue,
		TransformValue,
		SoftObjectPathValue,
		SoftClassPathValue,
		FSoftClassPath(UPCGDummyGetPropertyTest::StaticClass()),
		FSoftObjectPath(ObjectValue),
		Vector2Value,
		FVector4(ColorValue),
		FVector4(LinearColorValue));
}
#endif // WITH_EDITOR
