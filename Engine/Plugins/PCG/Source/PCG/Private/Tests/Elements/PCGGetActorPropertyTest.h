// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GameFramework/Actor.h"


#include "PCGGetActorPropertyTest.generated.h"

UENUM(meta = (Hidden))
enum class EPCGUnitTestDummyEnum : int64
{
	One,
	Two,
	Three
};

USTRUCT(BlueprintType, meta = (Hidden))
struct FPCGDummyGetPropertyLevel2Struct
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "")
	TArray<double> DoubleArrayProperty;
};

USTRUCT(BlueprintType, meta = (Hidden))
struct FPCGDummyGetPropertyStruct
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "")
	TArray<int> IntArrayProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	float FloatProperty = 0.0f;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FPCGDummyGetPropertyLevel2Struct Level2Struct;
};

USTRUCT(BlueprintType, meta = (Hidden))
struct FPCGTestMyColorStruct
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "")
	double B = 0;

	UPROPERTY(BlueprintReadOnly, Category = "")
	double G = 0;

	UPROPERTY(BlueprintReadOnly, Category = "")
	double R = 0;

	UPROPERTY(BlueprintReadOnly, Category = "")
	double A = 0;
};

UCLASS(MinimalAPI, NotBlueprintable, NotPlaceable, NotBlueprintType, Transient, HideDropdown, meta = (Hidden))
class UPCGDummyGetPropertyTest : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadOnly, Category = "")
	int64 Int64Property = 0;

	UPROPERTY(BlueprintReadOnly, Category = "")
	double DoubleProperty = 0.0;
};

UCLASS(MinimalAPI, NotBlueprintable, NotPlaceable, NotBlueprintType, Transient, HideDropdown, meta = (Hidden))
class APCGUnitTestDummyActor : public AActor
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadOnly, Category="")
	int IntProperty = 0;

	UPROPERTY(BlueprintReadOnly, Category = "")
	float FloatProperty = 0.0f;

	UPROPERTY(BlueprintReadOnly, Category = "")
	int64 Int64Property = 0;

	UPROPERTY(BlueprintReadOnly, Category = "")
	double DoubleProperty = 0.0;

	UPROPERTY(BlueprintReadOnly, Category = "")
	bool BoolProperty = false;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FName NameProperty = NAME_None;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FString StringProperty = "";

	UPROPERTY(BlueprintReadOnly, Category = "")
	EPCGUnitTestDummyEnum EnumProperty = EPCGUnitTestDummyEnum::One;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FVector VectorProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FVector4 Vector4Property;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FTransform TransformProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FRotator RotatorProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FQuat QuatProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FSoftObjectPath SoftObjectPathProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FSoftClassPath SoftClassPathProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	TSubclassOf<AActor> ClassProperty = AActor::StaticClass();

	UPROPERTY(BlueprintReadOnly, Category = "")
	TObjectPtr<UPCGDummyGetPropertyTest> ObjectProperty = nullptr;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FVector2D Vector2Property;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FColor ColorProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FLinearColor LinearColorProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FPCGTestMyColorStruct PCGColorProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	TArray<int32> ArrayOfIntsProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	TArray<FVector> ArrayOfVectorsProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	TArray<FPCGTestMyColorStruct> ArrayOfStructsProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	TArray<TObjectPtr<UPCGDummyGetPropertyTest>> ArrayOfObjectsProperty;

	UPROPERTY(BlueprintReadOnly, Category = "")
	FPCGDummyGetPropertyStruct DummyStruct;
	
	UPROPERTY(BlueprintReadOnly, Category = "")
    TArray<int32> SetOfIntsProperty;
};

UCLASS(MinimalAPI, NotBlueprintable, NotPlaceable, NotBlueprintType, Transient, HideDropdown, meta = (Hidden))
class UPCGUnitTestDummyComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadOnly, Category = "")
	int IntProperty = 0;
};
