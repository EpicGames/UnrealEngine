// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Elements/PCGAttributeGetFromIndexElement.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGetAttributeSetByIndexTest_Basic, FPCGTestBaseClass, "Plugins.PCG.GetAttributeSetByIndex.Basic", PCGTestsCommon::TestFlags)

bool FPCGGetAttributeSetByIndexTest_Basic::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGAttributeGetFromIndexSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGAttributeGetFromIndexSettings>(TestData);
	Settings->Index = 1;
	
	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	static const FName DoubleAttrName = TEXT("Double");
	static const FName IntAttrName = TEXT("Int");

	// Data preparation
	{
		UPCGParamData* ParamData = NewObject<UPCGParamData>();

		// Static analyser paranoid
		check(ParamData && ParamData->Metadata);
		PCGMetadataEntryKey FirstEntry = ParamData->Metadata->AddEntry();
		PCGMetadataEntryKey SecondEntry = ParamData->Metadata->AddEntry();

		FPCGMetadataAttribute<double>* DoubleAttr = ParamData->Metadata->CreateAttribute(DoubleAttrName, 0.0, true, false);
		check(DoubleAttr);
		DoubleAttr->SetValue(FirstEntry, 1.0);
		DoubleAttr->SetValue(SecondEntry, 2.0);

		FPCGMetadataAttribute<int>* IntAttr = ParamData->Metadata->CreateAttribute(IntAttrName, 0, true, false);
		check(IntAttr);
		IntAttr->SetValue(FirstEntry, 4);
		IntAttr->SetValue(SecondEntry, 3);

		FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
		SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
		SourcePin.Data = ParamData;
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);
	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGParamData* OutParamData = Cast<const UPCGParamData>(Outputs[0].Data);
	UTEST_NOT_NULL("Output Param data", OutParamData);
	UTEST_NOT_NULL("Output Metadata", OutParamData->Metadata.Get());

	check(OutParamData && OutParamData->Metadata);

	UTEST_EQUAL("Metadata has 1 entry", OutParamData->Metadata->GetLocalItemCount(), 1);

	FPCGMetadataAttribute<double>* OutDoubleAttr = OutParamData->Metadata->GetMutableTypedAttribute<double>(DoubleAttrName);
	UTEST_NOT_NULL("Double attribute exists", OutDoubleAttr);

	FPCGMetadataAttribute<int>* OutIntAttr = OutParamData->Metadata->GetMutableTypedAttribute<int>(IntAttrName);
	UTEST_NOT_NULL("Int attribute exists", OutIntAttr);

	check(OutDoubleAttr && OutIntAttr);

	UTEST_EQUAL("Double attribute first entry", OutDoubleAttr->GetValueFromItemKey(PCGFirstEntryKey), 2.0);
	UTEST_EQUAL("Double attribute default value was not changed", OutDoubleAttr->GetValueFromItemKey(PCGInvalidEntryKey), 0.0);

	UTEST_EQUAL("Int attribute first entry", OutIntAttr->GetValueFromItemKey(PCGFirstEntryKey), 3);
	UTEST_EQUAL("Int attribute default value was not changed", OutIntAttr->GetValueFromItemKey(PCGInvalidEntryKey), 0);

	return true;
}