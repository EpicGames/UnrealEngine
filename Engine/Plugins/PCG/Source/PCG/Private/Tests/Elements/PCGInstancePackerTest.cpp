// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSelectors/PCGMeshSelectorBase.h"
#include "Tests/PCGTestsCommon.h"

#include "Metadata/PCGMetadataAccessor.h"
#include "PCGComponent.h"
#include "PCGManagedResource.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGStaticMeshSpawner.h"
#include "Elements/PCGStaticMeshSpawnerContext.h"
#include "InstanceDataPackers/PCGInstanceDataPackerBase.h"
#include "InstanceDataPackers/PCGInstanceDataPackerByAttribute.h"
#include "InstanceDataPackers/PCGInstanceDataPackerByRegex.h"
#include "MeshSelectors/PCGMeshSelectorWeighted.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"

#if WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGStaticMeshSpawnerInstancePackerByAttributeTest, FPCGTestBaseClass, "Plugins.PCG.StaticMeshSpawner.InstancePacker.ByAttribute", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGStaticMeshSpawnerInstancePackerByRegexTest, FPCGTestBaseClass, "Plugins.PCG.StaticMeshSpawner.InstancePacker.ByRegex", PCGTestsCommon::TestFlags)

namespace
{
	struct FPCGTestCustomData
	{
		TArray<float> CustomData;
		int NumFloats;
	};

	TArray<float> LocalPackFloats(TArray<FPCGTestCustomData> InFloatArrays, int NumPoints)
	{
		TArray<float> OutPackedFloats;

		for (int InstanceIndex = 0; InstanceIndex < NumPoints; ++InstanceIndex)
		{
			for (const FPCGTestCustomData& FloatArray : InFloatArrays) {
				const int StartingIndex = InstanceIndex * FloatArray.NumFloats;
				const int EndingIndex = (InstanceIndex + 1) * FloatArray.NumFloats;

				OutPackedFloats.Append(&(FloatArray.CustomData[StartingIndex]), EndingIndex - StartingIndex);
			}
		}

		return OutPackedFloats;
	}

	bool ValidateInstancePacker(
		FPCGTestBaseClass* Test,
		const PCGTestsCommon::FTestData& TestData,
		const UPCGStaticMeshSpawnerSettings* Settings,
		TArray<float> ExpectedCustomData, 
		int NumCustomDataFloats)
	{
		FPCGElementPtr StaticMeshSpawnerElement = Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!StaticMeshSpawnerElement->Execute(Context.Get()))
		{}

		bool bTestPassed = true;

		TArray<UInstancedStaticMeshComponent*> ISMCs;
		TestData.TestPCGComponent->ForEachManagedResource([&ISMCs](UPCGManagedResource* InResource)
		{
			if (UPCGManagedISMComponent* Resource = Cast<UPCGManagedISMComponent>(InResource))
			{
				ISMCs.Add(Resource->GetComponent());
			}
		});

		for (UInstancedStaticMeshComponent* ISMC : ISMCs)
		{
			if (!Test->TestEqual("Valid NumCustomDataFloats in ISMC", ISMC->NumCustomDataFloats, NumCustomDataFloats))
			{
				bTestPassed = false;
				continue;
			}

			const int32 NumInstances = ISMC->GetInstanceCount();
			const int32 NumCustomData = ISMC->PerInstanceSMCustomData.Num();

			if (!Test->TestEqual("ISMC CustomData count matches expected CustomData count", NumCustomData, ExpectedCustomData.Num()))
			{
				bTestPassed = false;
				continue;
			}

			bTestPassed &= Test->TestEqual("Packed data matches expected data", ISMC->PerInstanceSMCustomData, ExpectedCustomData);
		}

		TestData.TestPCGComponent->bGenerated = true;
		TestData.TestPCGComponent->CleanupLocalImmediate(true);

		return bTestPassed;
	};
}

bool FPCGStaticMeshSpawnerInstancePackerByAttributeTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData{};
	UPCGStaticMeshSpawnerSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGStaticMeshSpawnerSettings>(TestData);
	check(Settings);

	FPCGTaggedData& SourceTaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TObjectPtr<UPCGBasePointData> SourceData = PCGTestsCommon::CreateRandomBasePointData(5, TestData.Seed);

	// TODO - remove this code when the tests support flipped instances (generates multiple ISMC components)
	TPCGValueRange<FTransform> TransformRange = SourceData->GetTransformValueRange();
	for (FTransform& PointTransform : TransformRange)
	{
		PointTransform.SetScale3D(PointTransform.GetScale3D().GetAbs());
	}

	SourceTaggedData.Data = SourceData;
	SourceData->TargetActor = TestData.TestActor;

	const FName BoolName = TEXT("Bool");
	const FName FloatName = TEXT("Float");
	const FName DoubleName = TEXT("Double");
	const FName IntName = TEXT("Int");
	const FName VecName = TEXT("Vec");
	const FName Vec4Name = TEXT("Vec4");
	const FName RotatorName = TEXT("Rotator");
	const FName QuatName = TEXT("Quat");

	const FPCGAttributePropertyInputSelector BoolSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(BoolName);
	const FPCGAttributePropertyInputSelector FloatSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(FloatName);
	const FPCGAttributePropertyInputSelector DoubleSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(DoubleName);
	const FPCGAttributePropertyInputSelector IntSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(IntName);
	const FPCGAttributePropertyInputSelector VecSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(VecName);
	const FPCGAttributePropertyInputSelector Vec4Selector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Vec4Name);
	const FPCGAttributePropertyInputSelector RotatorSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(RotatorName);
	const FPCGAttributePropertyInputSelector QuatSelector = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(QuatName);

	const bool bAllowsInterpolation = false;
	const bool bOverrideParent = false;

	SourceData->Metadata->CreateBoolAttribute(BoolName, 0, bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateFloatAttribute(FloatName, 0, bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateDoubleAttribute(DoubleName, 0, bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateInteger64Attribute(IntName, 0, bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateVectorAttribute(VecName, FVector::Zero(), bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateVector4Attribute(Vec4Name, FVector4::Zero(), bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateRotatorAttribute(RotatorName, FRotator::ZeroRotator, bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateQuatAttribute(QuatName, FQuat::Identity, bAllowsInterpolation, bOverrideParent);

	TArray<float> BoolValues;
	TArray<float> FloatValues;
	TArray<float> DoubleValues;
	TArray<float> IntValues;
	TArray<float> VecValues;
	TArray<float> Vec4Values;
	TArray<float> RotatorValues;
	TArray<float> QuatValues;

	TPCGValueRange<int64> MetadataEntryRange = SourceData->GetMetadataEntryValueRange();

	FRandomStream RandomSource(TestData.Seed);
	int NumPoints = 5;
	for (int I = 0; I < NumPoints; ++I)
	{
		const bool BoolValue = I % 2 == 0;
		const float FloatValue = (I + 0.f) / NumPoints;
		const double DoubleValue = FloatValue + 1;
		const int64 IntValue = I;
		const FVector VecValue = FVector(1, 2, 3) * I;
		const FVector4 Vec4Value = -FVector(1, 2, 3) * I;
		const FRotator RotatorValue = FRotator(15, 30, 45) * I;
		const FQuat QuatValue(RotatorValue);

		BoolValues.Add(BoolValue);
		FloatValues.Add(FloatValue);
		DoubleValues.Add(DoubleValue);
		IntValues.Add(IntValue);
		VecValues.Add(VecValue.X);
		VecValues.Add(VecValue.Y);
		VecValues.Add(VecValue.Z);
		Vec4Values.Add(Vec4Value.X);
		Vec4Values.Add(Vec4Value.Y);
		Vec4Values.Add(Vec4Value.Z);
		Vec4Values.Add(Vec4Value.W);
		RotatorValues.Add(RotatorValue.Roll);
		RotatorValues.Add(RotatorValue.Pitch);
		RotatorValues.Add(RotatorValue.Yaw);
		QuatValues.Add(QuatValue.X);
		QuatValues.Add(QuatValue.Y);
		QuatValues.Add(QuatValue.Z);
		QuatValues.Add(QuatValue.W);

		UPCGMetadataAccessorHelpers::SetBoolAttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, BoolName, BoolValue);
		UPCGMetadataAccessorHelpers::SetFloatAttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, FloatName, FloatValue);
		UPCGMetadataAccessorHelpers::SetDoubleAttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, DoubleName, DoubleValue);
		UPCGMetadataAccessorHelpers::SetInteger64AttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, IntName, IntValue);
		UPCGMetadataAccessorHelpers::SetVectorAttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, VecName, VecValue);
		UPCGMetadataAccessorHelpers::SetVector4AttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, Vec4Name, Vec4Value);
		UPCGMetadataAccessorHelpers::SetRotatorAttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, RotatorName, RotatorValue);
		UPCGMetadataAccessorHelpers::SetQuatAttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, QuatName, QuatValue);
	}

	Settings->SetInstancePackerType(UPCGInstanceDataPackerByAttribute::StaticClass());
	UPCGInstanceDataPackerByAttribute* InstancePacker = CastChecked<UPCGInstanceDataPackerByAttribute>(Settings->InstanceDataPackerParameters);
	UPCGMeshSelectorWeighted* MeshSelector = CastChecked<UPCGMeshSelectorWeighted>(Settings->MeshSelectorParameters);

	const FString CubePath = TEXT("StaticMesh'/Engine/BasicShapes/Cube.Cube'");
	UStaticMesh* CubeMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, *CubePath));
	MeshSelector->MeshEntries.Add(FPCGMeshSelectorWeightedEntry(CubeMesh, 1));

	bool bTestPassed = true;

	// No attributes
	{
		InstancePacker->AttributeSelectors = {};
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, {}, 0);
	}

	// Bool Attribute
	{
		InstancePacker->AttributeSelectors = { BoolSelector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, BoolValues, 1);
	}

	// Float Attribute
	{
		InstancePacker->AttributeSelectors = { FloatSelector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, FloatValues, 1);
	}

	// Double Attribute
	{
		InstancePacker->AttributeSelectors = { DoubleSelector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, DoubleValues, 1);
	}

	// Int Attribute
	{
		InstancePacker->AttributeSelectors = { IntSelector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, IntValues, 1);
	}

	// Vector Attribute
	{
		InstancePacker->AttributeSelectors = { VecSelector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, VecValues, 3);
	}

	// Vector4 Attribute
	{
		InstancePacker->AttributeSelectors = { Vec4Selector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, Vec4Values, 4);
	}

	// Rotator Attribute
	{
		InstancePacker->AttributeSelectors = { RotatorSelector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, RotatorValues, 3);
	}

	// Quat Attribute
	{
		InstancePacker->AttributeSelectors = { QuatSelector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, QuatValues, 4);
	}

	// Float + Vector
	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { FloatValues, 1 }, { VecValues, 3 } }, NumPoints);

		InstancePacker->AttributeSelectors = { FloatSelector, VecSelector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 4);
	}

	// Vector + Float
	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { VecValues, 3 }, { FloatValues, 1 } }, NumPoints);

		InstancePacker->AttributeSelectors = { VecSelector, FloatSelector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 4);
	}

	// Vector + Float + Vector
	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { VecValues, 3 }, { FloatValues, 1 }, { VecValues, 3 } }, NumPoints);

		InstancePacker->AttributeSelectors = { VecSelector, FloatSelector, VecSelector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 7);
	}

	// Vector + Float + Vector + Vector4
	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { VecValues, 3 }, { FloatValues, 1 }, { VecValues, 3 }, { Vec4Values, 4 } }, NumPoints);

		InstancePacker->AttributeSelectors = { VecSelector, FloatSelector, VecSelector, Vec4Selector };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 11);
	}

	return bTestPassed;
}

bool FPCGStaticMeshSpawnerInstancePackerByRegexTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData{};
	UPCGStaticMeshSpawnerSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGStaticMeshSpawnerSettings>(TestData);
	check(Settings);

	FPCGTaggedData& SourceTaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TObjectPtr<UPCGBasePointData> SourceData = PCGTestsCommon::CreateRandomBasePointData(5, TestData.Seed);

	// TODO - remove this code when the tests support flipped instances (generates multiple ISMC components)
	TPCGValueRange<FTransform> TransformRange = SourceData->GetTransformValueRange();
	for (FTransform& PointTransform : TransformRange)
	{
		PointTransform.SetScale3D(PointTransform.GetScale3D().GetAbs());
	}

	SourceTaggedData.Data = SourceData;
	SourceData->TargetActor = TestData.TestActor;

	const FName NameA1 = TEXT("A1"); // Float
	const FName NameA2 = TEXT("A2"); // Double
	const FName NameA3 = TEXT("A3"); // Int
	const FName NameB1 = TEXT("B1"); // FVector
	const FName NameB2 = TEXT("B2"); // FVector4

	const bool bAllowsInterpolation = false;
	const bool bOverrideParent = false;

	SourceData->Metadata->CreateFloatAttribute(NameA1, 0, bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateDoubleAttribute(NameA2, 0, bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateInteger64Attribute(NameA3, 0, bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateVectorAttribute(NameB1, FVector::Zero(), bAllowsInterpolation, bOverrideParent);
	SourceData->Metadata->CreateVector4Attribute(NameB2, FVector4::Zero(), bAllowsInterpolation, bOverrideParent);

	TArray<float> FloatValues;
	TArray<float> DoubleValues;
	TArray<float> IntValues;
	TArray<float> VecValues;
	TArray<float> Vec4Values;

	TPCGValueRange<int64> MetadataEntryRange = SourceData->GetMetadataEntryValueRange();

	FRandomStream RandomSource(TestData.Seed);
	int NumPoints = 5;
	for (int I = 0; I < NumPoints; ++I)
	{
		const float FloatValue = (I + 0.f) / NumPoints;
		const double DoubleValue = FloatValue + 1;
		const int64 IntValue = I;
		const FVector VecValue = FVector(1, 2, 3) * I;
		const FVector4 Vec4Value = -FVector(1, 2, 3) * I;

		FloatValues.Add(FloatValue);
		DoubleValues.Add(DoubleValue);
		IntValues.Add(IntValue);
		VecValues.Add(VecValue.X);
		VecValues.Add(VecValue.Y);
		VecValues.Add(VecValue.Z);
		Vec4Values.Add(Vec4Value.X);
		Vec4Values.Add(Vec4Value.Y);
		Vec4Values.Add(Vec4Value.Z);
		Vec4Values.Add(Vec4Value.W);

		UPCGMetadataAccessorHelpers::SetFloatAttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, NameA1, FloatValue);
		UPCGMetadataAccessorHelpers::SetDoubleAttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, NameA2, DoubleValue);
		UPCGMetadataAccessorHelpers::SetInteger64AttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, NameA3, IntValue);
		UPCGMetadataAccessorHelpers::SetVectorAttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, NameB1, VecValue);
		UPCGMetadataAccessorHelpers::SetVector4AttributeByMetadataKey(MetadataEntryRange[I], SourceData->Metadata, NameB2, Vec4Value);
	}

	Settings->SetInstancePackerType(UPCGInstanceDataPackerByRegex::StaticClass());
	UPCGInstanceDataPackerByRegex* InstancePacker = CastChecked<UPCGInstanceDataPackerByRegex>(Settings->InstanceDataPackerParameters);
	UPCGMeshSelectorWeighted* MeshSelector = CastChecked<UPCGMeshSelectorWeighted>(Settings->MeshSelectorParameters);

	const FString CubePath = TEXT("StaticMesh'/Engine/BasicShapes/Cube.Cube'");
	UStaticMesh* CubeMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, *CubePath));
	MeshSelector->MeshEntries.Add(FPCGMeshSelectorWeightedEntry(CubeMesh, 1));

	bool bTestPassed = true;

	// No attributes
	{
		InstancePacker->RegexPatterns = {};
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, {}, 0);
	}

	{
		InstancePacker->RegexPatterns = { TEXT("A1") };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, FloatValues, 1);
	}

	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { FloatValues, 1 }, { Vec4Values, 4 } }, NumPoints);

		InstancePacker->RegexPatterns = { TEXT("A1"), TEXT("B2") };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 5);
	}

	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { FloatValues, 1 }, { DoubleValues, 1 }, { IntValues, 1 } }, NumPoints);

		InstancePacker->RegexPatterns = { TEXT("A.") };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 3);
	}

	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { VecValues, 3 }, { Vec4Values, 4 } }, NumPoints);

		InstancePacker->RegexPatterns = { TEXT("B.") };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 7);
	}

	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { FloatValues, 1 }, { DoubleValues, 1 }, { IntValues, 1 }, { VecValues, 3 }, { Vec4Values, 4 } }, NumPoints);

		InstancePacker->RegexPatterns = { TEXT("A."), TEXT("B.") };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 10);
	}

	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { VecValues, 3 }, { Vec4Values, 4 }, { FloatValues, 1 }, { DoubleValues, 1 }, { IntValues, 1 } }, NumPoints);

		InstancePacker->RegexPatterns = { TEXT("B."), TEXT("A.") };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 10);
	}

	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { FloatValues, 1 }, { VecValues, 3 } }, NumPoints);

		InstancePacker->RegexPatterns = { TEXT(".1") };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 4);
	}

	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { FloatValues, 1 }, { VecValues, 3 } }, NumPoints);

		InstancePacker->RegexPatterns = { TEXT(".1"), TEXT(".1") };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 4);
	}

	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { FloatValues, 1 }, { VecValues, 3 }, { DoubleValues, 1 }, { Vec4Values, 4 } }, NumPoints);

		InstancePacker->RegexPatterns = { TEXT(".1"), TEXT(".2") };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 9);
	}

	// Capture all attributes
	{
		TArray<float> ExpectedCustomData = LocalPackFloats({ { FloatValues, 1 }, { DoubleValues, 1 }, { IntValues, 1 }, { VecValues, 3 }, { Vec4Values, 4 } }, NumPoints);

		InstancePacker->RegexPatterns = { TEXT("[A-Z][0-9]") };
		bTestPassed &= ValidateInstancePacker(this, TestData, Settings, ExpectedCustomData, 10);
	}

	return bTestPassed;
}

#endif // WITH_EDITOR
