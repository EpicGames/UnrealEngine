// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/ControlFlow/PCGMultiSelect.h"
#include "Tests/PCGTestsCommon.h"

#if WITH_EDITOR

class FPCGMultiSelectTest : public FPCGTestBaseClass
{
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	inline static const FString DefaultPathPinDataTag = TEXT("DataTagDefault");
	inline static const FName DefaultPathPinLabel = TEXT("Default");

	static FString CreateDataTag(const int32 Index)
	{
		return FString("DataTag") + FString::FromInt(Index);
	}

	static void UpdateSettings(const UPCGMultiSelectSettings* InSettings, UPCGMultiSelectSettings* SelectSettings)
	{
		SelectSettings->SelectionMode = InSettings->SelectionMode;
		SelectSettings->IntOptions = InSettings->IntOptions;
		SelectSettings->IntegerSelection = InSettings->IntegerSelection;
		SelectSettings->StringOptions = InSettings->StringOptions;
		SelectSettings->StringSelection = InSettings->StringSelection;
		SelectSettings->EnumSelection = InSettings->EnumSelection;
		SelectSettings->UpdateSettings();
	}

	bool ExecuteTest(const UPCGMultiSelectSettings* InSettings, const int32 InputNum, TArray<FPCGTaggedData>& OutputData)
	{
		PCGTestsCommon::FTestData TestData;
		UPCGMultiSelectSettings* TestSettings = PCGTestsCommon::GenerateSettings<UPCGMultiSelectSettings>(TestData);
		UpdateSettings(InSettings, TestSettings);

		const FPCGElementPtr TestElement = TestSettings->GetElement();

		UTEST_TRUE("Test element created and valid", TestElement.IsValid());

		// Add a tagged input to each of the valid pin paths
		TArray<FPCGPinProperties> SelectElementPinProperties = TestSettings->AllInputPinProperties();
		for (int i = 0; i < SelectElementPinProperties.Num(); ++i)
		{
			if (!SelectElementPinProperties[i].IsAdvancedPin() && SelectElementPinProperties[i].Label != DefaultPathPinLabel)
			{
				FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
				TaggedData.Data = PCGTestsCommon::CreateEmptyPointData();
				TaggedData.Tags.Emplace(CreateDataTag(i));
				TaggedData.Pin = SelectElementPinProperties[i].Label;
			}
		}

		// Add an input to the default path
		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = PCGTestsCommon::CreateEmptyPointData();
		TaggedData.Tags.Emplace(DefaultPathPinDataTag);
		TaggedData.Pin = DefaultPathPinLabel;

		const TUniquePtr<FPCGContext> TestContext = TestData.InitializeTestContext();

		UTEST_TRUE("Test context created and valid", TestContext.IsValid());

		while (!TestElement->Execute(TestContext.Get()))
		{
		}

		OutputData = TestContext->OutputData.GetInputsByPin(PCGPinConstants::DefaultOutputLabel);

		return true;
	}

	bool ValidateOutput(const TArray<FPCGTaggedData>& Outputs)
	{
		UTEST_TRUE("Output was generated", !Outputs.IsEmpty());

		UTEST_EQUAL("Single output was generated", Outputs.Num(), 1);

		UTEST_EQUAL("Only one tag output", Outputs.Last().Tags.Num(), 1);

		return true;
	}

	bool ValidateOutputInt(const TArray<FPCGTaggedData>& OutputData, const int32 ValidKey)
	{
		UTEST_TRUE("Validate output and output count", ValidateOutput(OutputData));

		UTEST_TRUE("Output contains correct data", OutputData.Last().Tags.Contains(CreateDataTag(ValidKey)));

		return true;
	}

	bool ValidateOutputString(const TArray<FString>& Options, const TArray<FPCGTaggedData>& OutputData, const FString& ValidKey)
	{
		UTEST_TRUE("Validate output and output count", ValidateOutput(OutputData));

		for (int i = 0; i < Options.Num(); ++i)
		{
			if (Options[i] == ValidKey)
			{
				UTEST_TRUE("Output contains correct data", OutputData.Last().Tags.Contains(CreateDataTag(i)));
			}
		}

		return true;
	}

	bool ValidateOutputEnum(const UEnum* Enum, const TArray<FPCGTaggedData>& OutputData, const int64 ValidKey)
	{
		UTEST_NOT_NULL("Enum is valid", Enum);

		UTEST_TRUE("Validate output and output count", ValidateOutput(OutputData));

		UTEST_TRUE("Output contains correct data", OutputData.Last().Tags.Contains(CreateDataTag(ValidKey)));

		return true;
	}

	bool ValidateDefaultPathOutput(const TArray<FPCGTaggedData>& OutputData)
	{
		UTEST_TRUE("Validate output and output count", ValidateOutput(OutputData));

		UTEST_TRUE("Output contains correct data", OutputData.Last().Tags.Contains(DefaultPathPinDataTag));

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMultiSelectTest_Integer, FPCGMultiSelectTest, "Plugins.PCG.MultiSelect.Integer", PCGTestsCommon::TestFlags)

bool FPCGMultiSelectTest_Integer::RunTest(const FString& Parameters)
{
	TArray<FPCGTaggedData> OutputData;
	UPCGMultiSelectSettings* TestSettings = NewObject<UPCGMultiSelectSettings>();
	TestSettings->SelectionMode = EPCGControlFlowSelectionMode::Integer;

	TArray<int32>& Options = TestSettings->IntOptions;
	Options = {-1, 0, 2, 3, 5};

	// Positive Validation
	for (int i = 0; i < Options.Num(); ++i)
	{
		TestSettings->IntegerSelection = Options[i];

		UTEST_TRUE("Integer test succeeded", ExecuteTest(TestSettings, Options.Num(), OutputData));

		UTEST_TRUE("Integer selection is valid", ValidateOutputInt(OutputData, i));
	}

	const int32 Min = FMath::Min(Options);
	const int32 Max = FMath::Max(Options);
	for (int i = Min - 1; i <= Max + 1; ++i)
	{
		if (Options.Contains(i))
		{
			continue;
		}

		TestSettings->IntegerSelection = i;

		UTEST_TRUE("Integer test succeeded", ExecuteTest(TestSettings, Options.Num(), OutputData));

		UTEST_TRUE("Invalid int selection defaults output", ValidateDefaultPathOutput(OutputData));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMultiSelectTest_String, FPCGMultiSelectTest, "Plugins.PCG.MultiSelect.String", PCGTestsCommon::TestFlags)

bool FPCGMultiSelectTest_String::RunTest(const FString& Parameters)
{
	TArray<FPCGTaggedData> OutputData;
	UPCGMultiSelectSettings* TestSettings = NewObject<UPCGMultiSelectSettings>();
	TestSettings->SelectionMode = EPCGControlFlowSelectionMode::String;

	TArray<FString>& Options = TestSettings->StringOptions;
	Options = {"Red", "Green", "Blue", "Fuschia", "Orange"};

	FString& Selection = TestSettings->StringSelection;

	// Positive Validation
	for (int i = 0; i < Options.Num(); ++i)
	{
		Selection = Options[i];

		UTEST_TRUE("String test succeeded", ExecuteTest(TestSettings, Options.Num(), OutputData));

		UTEST_TRUE("String selection is valid", ValidateOutputString(Options, OutputData, Selection));
	}

	// Negative Validation
	Selection = "Square";

	UTEST_TRUE("String test succeeded", ExecuteTest(TestSettings, Options.Num(), OutputData));

	UTEST_TRUE("Invalid string selection defaults output", ValidateDefaultPathOutput(OutputData));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGMultiSelectTest_Enum, FPCGMultiSelectTest, "Plugins.PCG.MultiSelect.Enum", PCGTestsCommon::TestFlags)

bool FPCGMultiSelectTest_Enum::RunTest(const FString& Parameters)
{
	TArray<FPCGTaggedData> OutputData;
	UPCGMultiSelectSettings* TestSettings = NewObject<UPCGMultiSelectSettings>();
	TestSettings->SelectionMode = EPCGControlFlowSelectionMode::Enum;

	TObjectPtr<UEnum>& EnumClass = TestSettings->EnumSelection.Class;
	EnumClass = StaticEnum<EPCGMetadataTypes>();

	const int32 NumEnums = EnumClass->NumEnums();

	// Positive Validation (also accounting for hidden metadata)
	for (uint8 i = 0; i < NumEnums - 1; ++i)
	{
		TestSettings->EnumSelection.Value = EnumClass->GetValueByIndex(i);

		UTEST_TRUE("Selection value is valid", TestSettings->EnumSelection.Value != INDEX_NONE);

		UTEST_TRUE("Enum test succeeded", ExecuteTest(TestSettings, NumEnums - 1, OutputData));

		if (TestSettings->EnumSelection.Class->HasMetaData(TEXT("Hidden"), i) || TestSettings->EnumSelection.Class->HasMetaData(TEXT("Spacer"), i))
		{
			UTEST_TRUE("Hidden or spacer enum selection defaults output", ValidateDefaultPathOutput(OutputData));
		}
		else
		{
			UTEST_TRUE("Enum selection is valid", ValidateOutputEnum(EnumClass, OutputData, i));
		}
	}

	// Negative Validation - Invalid selection value
	TestSettings->EnumSelection.Value = NumEnums;

	UTEST_TRUE("Enum test succeeded", ExecuteTest(TestSettings, NumEnums, OutputData));

	UTEST_TRUE("Invalid enum value defaults output", ValidateDefaultPathOutput(OutputData));

	// Negative Validation - Invalid enum
	EnumClass = nullptr;

	UTEST_TRUE("Enum test succeeded", ExecuteTest(TestSettings, 1, OutputData));

	UTEST_TRUE("Invalid enum class defaults output", ValidateDefaultPathOutput(OutputData));

	return true;
}

#endif // WITH_EDITOR