// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "Data/PCGSpatialData.h"
#include "PCGComponent.h"

#include "Data/PCGPointData.h"
#include "Elements/PCGNormalToDensity.h"
#include "PCGContext.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGNormalToDensityTest_Set, FPCGTestBaseClass, "Plugins.PCG.NormalToDensity.Set", PCGTestsCommon::TestFlags)

bool FPCGNormalToDensityTest_Set::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGNormalToDensitySettings>(TestData);
	UPCGNormalToDensitySettings* Settings = CastChecked<UPCGNormalToDensitySettings>(TestData.Settings);
	FPCGElementPtr Element = TestData.Settings->GetElement();

	Settings->DensityMode = PCGNormalToDensityMode::Set;

	{
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateBasePointData();
		
		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();

		TransformRange[0].SetRotation(FQuat::MakeFromEuler(FVector(45.0, 0, 0)));
		DensityRange[0] = 0.5f;

		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Data = PointData;
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!Element->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGSpatialData* OutputData = Cast<UPCGSpatialData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);

	const UPCGBasePointData* OutputPointData = OutputData->ToBasePointData(Context.Get());
	UTEST_NOT_NULL("Output point data", OutputPointData);

	UTEST_EQUAL("Output point count", OutputPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<float> DensityRange = OutputPointData->GetConstDensityValueRange();
	UTEST_EQUAL_TOLERANCE("Output point density", DensityRange[0], 0.707f, 0.001f);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGNormalToDensityTest_CustomNormal, FPCGTestBaseClass, "Plugins.PCG.NormalToDensity.CustomNormal", PCGTestsCommon::TestFlags)

bool FPCGNormalToDensityTest_CustomNormal::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGNormalToDensitySettings>(TestData);
	UPCGNormalToDensitySettings* Settings = CastChecked<UPCGNormalToDensitySettings>(TestData.Settings);
	FPCGElementPtr Element = TestData.Settings->GetElement();

	Settings->DensityMode = PCGNormalToDensityMode::Set;
	Settings->Normal = FVector(0.707, 0, 0.707);

	{
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateBasePointData();

		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();
		DensityRange[0] = 0.5f;

		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Data = PointData;
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!Element->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGSpatialData* OutputData = Cast<UPCGSpatialData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);

	const UPCGBasePointData* OutputPointData = OutputData->ToBasePointData(Context.Get());
	UTEST_NOT_NULL("Output point data", OutputPointData);

	UTEST_EQUAL("Output point count", OutputPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<float> DensityRange = OutputPointData->GetConstDensityValueRange();
	UTEST_EQUAL_TOLERANCE("Output point density", DensityRange[0], 0.707f, 0.001f);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGNormalToDensityTest_Strength, FPCGTestBaseClass, "Plugins.PCG.NormalToDensity.Strength", PCGTestsCommon::TestFlags)

bool FPCGNormalToDensityTest_Strength::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGNormalToDensitySettings>(TestData);
	UPCGNormalToDensitySettings* Settings = CastChecked<UPCGNormalToDensitySettings>(TestData.Settings);
	FPCGElementPtr Element = TestData.Settings->GetElement();

	Settings->DensityMode = PCGNormalToDensityMode::Set;
	Settings->Strength = 2;
	Settings->Normal = FVector(0.866, 0, 0.5); // so the dot with up is 0.5

	{
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateBasePointData();

		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();
		DensityRange[0] = 0.5f;

		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Data = PointData;
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!Element->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGSpatialData* OutputData = Cast<UPCGSpatialData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);

	const UPCGBasePointData* OutputPointData = OutputData->ToBasePointData(Context.Get());
	UTEST_NOT_NULL("Output point data", OutputPointData);

	UTEST_EQUAL("Output point count", OutputPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<float> DensityRange = OutputPointData->GetConstDensityValueRange();
	UTEST_EQUAL_TOLERANCE("Output point density", DensityRange[0], 0.707f, 0.001f);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGNormalToDensityTest_Minimum, FPCGTestBaseClass, "Plugins.PCG.NormalToDensity.Minimum", PCGTestsCommon::TestFlags)

bool FPCGNormalToDensityTest_Minimum::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGNormalToDensitySettings>(TestData);
	UPCGNormalToDensitySettings* Settings = CastChecked<UPCGNormalToDensitySettings>(TestData.Settings);
	FPCGElementPtr Element = TestData.Settings->GetElement();

	Settings->DensityMode = PCGNormalToDensityMode::Minimum;

	{
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateBasePointData();

		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();

		TransformRange[0].SetRotation(FQuat::MakeFromEuler(FVector(45.0, 0, 0)));
		DensityRange[0] = 0.5f;

		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Data = PointData;
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!Element->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGSpatialData* OutputData = Cast<UPCGSpatialData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);

	const UPCGBasePointData* OutputPointData = OutputData->ToBasePointData(Context.Get());
	UTEST_NOT_NULL("Output point data", OutputPointData);

	UTEST_EQUAL("Output point count", OutputPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<float> DensityRange = OutputPointData->GetConstDensityValueRange();
	UTEST_EQUAL_TOLERANCE("Output point density", DensityRange[0], 0.5f, 0.001f);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGNormalToDensityTest_Maximum, FPCGTestBaseClass, "Plugins.PCG.NormalToDensity.Maximum", PCGTestsCommon::TestFlags)

bool FPCGNormalToDensityTest_Maximum::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGNormalToDensitySettings>(TestData);
	UPCGNormalToDensitySettings* Settings = CastChecked<UPCGNormalToDensitySettings>(TestData.Settings);
	FPCGElementPtr Element = TestData.Settings->GetElement();

	Settings->DensityMode = PCGNormalToDensityMode::Maximum;

	{
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateBasePointData();

		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();

		TransformRange[0].SetRotation(FQuat::MakeFromEuler(FVector(45.0, 0, 0)));
		DensityRange[0] = 0.5f;

		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Data = PointData;
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!Element->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGSpatialData* OutputData = Cast<UPCGSpatialData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);

	const UPCGBasePointData* OutputPointData = OutputData->ToBasePointData(Context.Get());
	UTEST_NOT_NULL("Output point data", OutputPointData);

	UTEST_EQUAL("Output point count", OutputPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<float> DensityRange = OutputPointData->GetConstDensityValueRange();
	UTEST_EQUAL_TOLERANCE("Output point density", DensityRange[0], 0.707f, 0.001f);

	return true;
}


IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGNormalToDensityTest_Add, FPCGTestBaseClass, "Plugins.PCG.NormalToDensity.Add", PCGTestsCommon::TestFlags)

bool FPCGNormalToDensityTest_Add::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGNormalToDensitySettings>(TestData);
	UPCGNormalToDensitySettings* Settings = CastChecked<UPCGNormalToDensitySettings>(TestData.Settings);
	FPCGElementPtr Element = TestData.Settings->GetElement();

	Settings->DensityMode = PCGNormalToDensityMode::Add;

	{
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateBasePointData();

		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();

		TransformRange[0].SetRotation(FQuat::MakeFromEuler(FVector(45.0, 0, 0)));
		DensityRange[0] = 0.1f;

		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Data = PointData;
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!Element->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGSpatialData* OutputData = Cast<UPCGSpatialData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);

	const UPCGBasePointData* OutputPointData = OutputData->ToBasePointData(Context.Get());
	UTEST_NOT_NULL("Output point data", OutputPointData);

	UTEST_EQUAL("Output point count", OutputPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<float> DensityRange = OutputPointData->GetConstDensityValueRange();
	UTEST_EQUAL_TOLERANCE("Output point density", DensityRange[0], 0.807f, 0.001f);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGNormalToDensityTest_Subtract, FPCGTestBaseClass, "Plugins.PCG.NormalToDensity.Subtract", PCGTestsCommon::TestFlags)

bool FPCGNormalToDensityTest_Subtract::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGNormalToDensitySettings>(TestData);
	UPCGNormalToDensitySettings* Settings = CastChecked<UPCGNormalToDensitySettings>(TestData.Settings);
	FPCGElementPtr Element = TestData.Settings->GetElement();

	Settings->DensityMode = PCGNormalToDensityMode::Subtract;

	{
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateBasePointData();

		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();

		TransformRange[0].SetRotation(FQuat::MakeFromEuler(FVector(45.0, 0, 0)));
		DensityRange[0] = 0.9f;

		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Data = PointData;
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!Element->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGSpatialData* OutputData = Cast<UPCGSpatialData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);

	const UPCGBasePointData* OutputPointData = OutputData->ToBasePointData(Context.Get());
	UTEST_NOT_NULL("Output point data", OutputPointData);

	UTEST_EQUAL("Output point count", OutputPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<float> DensityRange = OutputPointData->GetConstDensityValueRange();
	UTEST_EQUAL_TOLERANCE("Output point density", DensityRange[0], 0.9f - 0.707f, 0.001f);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGNormalToDensityTest_Multiply, FPCGTestBaseClass, "Plugins.PCG.NormalToDensity.Multiply", PCGTestsCommon::TestFlags)

bool FPCGNormalToDensityTest_Multiply::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGNormalToDensitySettings>(TestData);
	UPCGNormalToDensitySettings* Settings = CastChecked<UPCGNormalToDensitySettings>(TestData.Settings);
	FPCGElementPtr Element = TestData.Settings->GetElement();

	Settings->DensityMode = PCGNormalToDensityMode::Multiply;

	{
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateBasePointData();

		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();

		TransformRange[0].SetRotation(FQuat::MakeFromEuler(FVector(45.0, 0, 0)));
		DensityRange[0] = 0.5f;

		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Data = PointData;
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!Element->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGSpatialData* OutputData = Cast<UPCGSpatialData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);

	const UPCGBasePointData* OutputPointData = OutputData->ToBasePointData(Context.Get());
	UTEST_NOT_NULL("Output point data", OutputPointData);

	UTEST_EQUAL("Output point count", OutputPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<float> DensityRange = OutputPointData->GetConstDensityValueRange();
	UTEST_EQUAL_TOLERANCE("Output point density", DensityRange[0], 0.5f*0.707f, 0.001f);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGNormalToDensityTest_Divide, FPCGTestBaseClass, "Plugins.PCG.NormalToDensity.Divide", PCGTestsCommon::TestFlags)

bool FPCGNormalToDensityTest_Divide::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGNormalToDensitySettings>(TestData);
	UPCGNormalToDensitySettings* Settings = CastChecked<UPCGNormalToDensitySettings>(TestData.Settings);
	FPCGElementPtr Element = TestData.Settings->GetElement();

	Settings->DensityMode = PCGNormalToDensityMode::Divide;

	{
		TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateBasePointData();

		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();

		TransformRange[0].SetRotation(FQuat::MakeFromEuler(FVector(45.0, 0, 0)));
		DensityRange[0] = 0.5f;

		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
		TaggedData.Data = PointData;
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!Element->Execute(Context.Get())) {}

	// test our point data
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGSpatialData* OutputData = Cast<UPCGSpatialData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);

	const UPCGBasePointData* OutputPointData = OutputData->ToBasePointData(Context.Get());
	UTEST_NOT_NULL("Output point data", OutputPointData);

	UTEST_EQUAL("Output point count", OutputPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<float> DensityRange = OutputPointData->GetConstDensityValueRange();
	UTEST_EQUAL_TOLERANCE("Output point density", DensityRange[0], 0.5f/0.707f, 0.001f);

	return true;
}

