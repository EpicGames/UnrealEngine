// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Elements/Metadata/PCGParseString.h"

class FPCGParseStringTestsBase : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	template <typename T>
	bool RunTestInternal(TArrayView<const FString> InData, TArrayView<const T> ExpectedOutData)
	{
		PCGTestsCommon::FTestData TestData;
		UPCGParseStringSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGParseStringSettings>(TestData);
		check(Settings);

		check(InData.Num() == ExpectedOutData.Num() && InData.Num() > 1);

		const FName InputAttributeName = TEXT("InAttr");
		const FName OutputAttributeName = TEXT("OutAttr");
		Settings->InputSource.SetAttributeName(InputAttributeName);
		Settings->OutputTarget.SetAttributeName(OutputAttributeName);
		Settings->TargetType = static_cast<EPCGMetadataTypes>(PCG::Private::MetadataTypes<T>::Id);

		UPCGParamData* InputData = NewObject<UPCGParamData>();
		FPCGMetadataAttribute<FString>* Attribute = InputData->Metadata->CreateAttribute<FString>(InputAttributeName, InData[0], true, false);
		for (int32 i = 1; i < InData.Num(); ++i)
		{
			Attribute->SetValue(InputData->Metadata->AddEntry(), InData[i]);
		}

		FPCGTaggedData& InputTaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		InputTaggedData.Data = InputData;
		InputTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		FPCGElementPtr TestElement = TestData.Settings->GetElement();
		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		UTEST_EQUAL("There is one output", Context->OutputData.TaggedData.Num(), 1);

		const UPCGParamData* OutputData = Cast<const UPCGParamData>(Context->OutputData.TaggedData[0].Data);

		UTEST_NOT_NULL("Output is an of the same type as input", OutputData);

		const FPCGMetadataAttribute<T>* OutputAttribute = OutputData->Metadata->template GetConstTypedAttribute<T>(OutputAttributeName);

		UTEST_NOT_NULL("Output metadata has the expected attribute", OutputAttribute);

		UTEST_EQUAL(*FString::Printf(TEXT("Default value %s, was parsed correctly"), *InData[0]), OutputAttribute->GetValue(PCGDefaultValueKey), ExpectedOutData[0]);
		for (int32 i = 1; i < ExpectedOutData.Num(); ++i)
		{
			PCGMetadataEntryKey EntryKey(i - 1);
			UTEST_EQUAL(*FString::Printf(TEXT("Entry %d (%s) was parsed expectedly"), i - 1, *InData[i]), OutputAttribute->GetValueFromItemKey(EntryKey), ExpectedOutData[i]);
		}

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Bool, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Bool", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Int32, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Int32", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Int64, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Int64", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Float, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Float", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Double, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Double", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Vector2D, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Vector2D", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Vector, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Vector", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Vector4, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Vector4", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Rotator, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Rotator", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Quat, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Quat", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParseStringTests_Transform, FPCGParseStringTestsBase, "Plugins.PCG.ParseString.Transform", PCGTestsCommon::TestFlags)

bool FPCGParseStringTests_Bool::RunTest(const FString& Parameters)
{
	return RunTestInternal<bool>({ "On", "Off", "True", "False", "Yes", "No", "0", "1", "54", "Blabla"}, {true, false, true, false, true, false, false, true, true, false });
}

bool FPCGParseStringTests_Int32::RunTest(const FString& Parameters)
{
	return RunTestInternal<int32>({
#if PLATFORM_MAC || PLATFORM_LINUX  // @todo_pcg: Mac/Linux Atoi handles this kind of overflow slightly differently, returning -1, which is a valid index. Disabling temporarily until further investigation with the core team.
	"0", "1", "-1", "1254", "Bla", "0xff", "0547"},
	{0, 1, -1, 1254, 0, 0xff, 547}
#else
	"0", "1", "-1", "1254", "9223372036854775807", "-9223372036854775808", "Bla", "0xff", "0547"},
	{0, 1, -1, 1254, -1, -1, 0, 0xff, 547}
#endif
		);
}

bool FPCGParseStringTests_Int64::RunTest(const FString& Parameters)
{
	return RunTestInternal<int64>({ "0", "1", "-1", "1254", "9223372036854775807", "-9223372036854775808", "99999999999999999999999999999999999999999", "Bla", "0xff", "0547" }, {0, 1, -1, 1254, std::numeric_limits<int64>::max(), std::numeric_limits<int64>::min(), std::numeric_limits<int64>::max(), 0, 0xff, 547 });
}

bool FPCGParseStringTests_Float::RunTest(const FString& Parameters)
{
	return RunTestInternal<float>({ "0.0", "1.0", "-1.0", "12.54", "92233720368.54775807", "-92233720368.54775808", "Bla", "0xff", "1e-3"}, {0.0f, 1.0f, -1.0f, 12.54f, 92233720368.54775807f, -92233720368.54775808f, 0.0f, 0.0f, 1e-3f});
}

bool FPCGParseStringTests_Double::RunTest(const FString& Parameters)
{
	return RunTestInternal<double>({ "0.0", "1.0", "-1.0", "12.54", "92233720368.54775807", "-92233720368.54775808", "Bla", "0xff", "1e-3" }, { 0.0, 1.0, -1.0, 12.54, 92233720368.54775807, -92233720368.54775808, 0.0, 0.0, 1e-3 });
}

bool FPCGParseStringTests_Vector2D::RunTest(const FString& Parameters)
{
	return RunTestInternal<FVector2D>({ "0.1,0.2", "1.0,    2.0", "X=0.5 Y=0.7", "X=8.0,            Y=4.1", "Bla", "0.1;0.2" }, { FVector2D{0.1, 0.2}, FVector2D{1.0, 2.0}, FVector2D{0.5, 0.7}, FVector2D{8.0, 4.1}, FVector2D::ZeroVector, FVector2D::ZeroVector });
}

bool FPCGParseStringTests_Vector::RunTest(const FString& Parameters)
{
	return RunTestInternal<FVector>({ "0.1,0.2,0.3", "1.0,   2.0,    3.0", "X=0.5 Y=0.7 Z=4.5", "X=8.0,            Y=4.1,         Z=9.4", "Bla", "0.1;0.2;0.3" }, { FVector{0.1, 0.2, 0.3}, FVector{1.0, 2.0, 3.0}, FVector{0.5, 0.7, 4.5}, FVector{8.0, 4.1, 9.4}, FVector::ZeroVector, FVector::ZeroVector });
}

bool FPCGParseStringTests_Vector4::RunTest(const FString& Parameters)
{
	return RunTestInternal<FVector4>({ "0.1,0.2,0.3,0.4", "1.0,   2.0,    3.0,    4.0", "X=0.5 Y=0.7 Z=4.5 W=6.1", "X=8.0,            Y=4.1,         Z=9.4,     W=7.3", "Bla", "0.1;0.2;0.3;0.4" }, { FVector4{0.1, 0.2, 0.3, 0.4}, FVector4{1.0, 2.0, 3.0, 4.0}, FVector4{0.5, 0.7, 4.5, 6.1}, FVector4{8.0, 4.1, 9.4, 7.3}, FVector4::Zero(), FVector4::Zero()});
}

bool FPCGParseStringTests_Rotator::RunTest(const FString& Parameters)
{
	return RunTestInternal<FRotator>({ "20.5,12.4,3.14", "10.2,     51.3,    65.4", "P=54.7 Y=87 R=89",  "Pitch=54.7 Yaw=87 Roll=89", "P=54.7     Y=87      R=89", "Bla", "25.1;21.4;98.3"}, {FRotator(20.5, 12.4, 3.14),FRotator(10.2, 51.3, 65.4), FRotator(54.7, 87, 89), FRotator(54.7, 87, 89), FRotator(54.7, 87, 89), FRotator::ZeroRotator, FRotator::ZeroRotator });
}

bool FPCGParseStringTests_Quat::RunTest(const FString& Parameters)
{
	return RunTestInternal<FQuat>({ "0.1,0.2,0.3,0.4", "1.0,   2.0,    3.0,    4.0", "X=0.5 Y=0.7 Z=4.5 W=6.1", "X=8.0,            Y=4.1,         Z=9.4,     W=7.3", "Bla", "0.1;0.2;0.3;0.4" }, { FQuat{0.1, 0.2, 0.3, 0.4}, FQuat{1.0, 2.0, 3.0, 4.0}, FQuat{0.5, 0.7, 4.5, 6.1}, FQuat{8.0, 4.1, 9.4, 7.3}, FQuat::Identity, FQuat::Identity});
}

bool FPCGParseStringTests_Transform::RunTest(const FString& Parameters)
{
	return RunTestInternal<FTransform>({ "1.0,2.0,3.0|45.0,45.0,46.0|1.5,2.5,3.5", "Bla", "1.0;2.0;3.0|45.0;45.0;46.0|1.5;2.5;3.5"}, {FTransform(FRotator(45, 45, 46).Quaternion(), FVector(1.0,2.0,3.0), FVector(1.5, 2.5, 3.5)), FTransform::Identity, FTransform::Identity});
}
