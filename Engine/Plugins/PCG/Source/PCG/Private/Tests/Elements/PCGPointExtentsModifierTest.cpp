// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "Data/PCGSpatialData.h"
#include "PCGComponent.h"

#include "Data/PCGPointData.h"
#include "Elements/PCGPointExtentsModifier.h"
#include "PCGContext.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointsBoundsModiferTest, FPCGTestBaseClass, "Plugins.PCG.PointExtentsModifier.Basic", PCGTestsCommon::TestFlags)

bool FPCGPointsBoundsModiferTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGPointExtentsModifierSettings>(TestData);
	UPCGPointExtentsModifierSettings* Settings = CastChecked<UPCGPointExtentsModifierSettings>(TestData.Settings);
	FPCGElementPtr BoundsModifierElement = TestData.Settings->GetElement();

	TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateEmptyBasePointData();
	
	static const int PointCount = 4;
	
	PointData->SetNumPoints(PointCount);
	PointData->SetDensity(1);
	PointData->AllocateProperties(EPCGPointNativeProperties::Seed | EPCGPointNativeProperties::BoundsMin | EPCGPointNativeProperties::BoundsMax);
	
	FPCGPointValueRanges ValueRanges(PointData, /*bAllocate=*/false);

	for (int I = 0; I < PointCount; ++I)
	{
		ValueRanges.SeedRange[I] = I;

		const float Value = float(I+1);

		PCGPointHelpers::SetExtents(FVector(Value, Value, Value), ValueRanges.BoundsMinRange[I], ValueRanges.BoundsMaxRange[I]);

		if (I == PointCount-1)
		{
			// make the last one off-center, so (-3,-3,-3),(5,5,5)
			PCGPointHelpers::SetLocalCenter(FVector(1.0f, 1.0f, 1.0f), ValueRanges.BoundsMinRange[I], ValueRanges.BoundsMaxRange[I]);
		}
	}

	FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedData.Pin = PCGPinConstants::DefaultInputLabel;
	TaggedData.Data = PointData;

	auto ValidateBounds = [this, &TestData, BoundsModifierElement, Settings](TArray<FVector> ExpectedOutput) -> bool
	{
		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!BoundsModifierElement->Execute(Context.Get()))
		{}

		const TArray<FPCGTaggedData>& Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

		if (!TestEqual("Valid number of Outputs", Inputs.Num(), Outputs.Num()))
		{
			return false;
		}

		if (!TestEqual("Has Outputs", Outputs.Num() >= 1, true))
		{
			return false;
		}

		bool bTestPassed = true;

		for (int DataIndex = 0; DataIndex < Inputs.Num(); ++DataIndex)
		{
			const FPCGTaggedData& Input = Inputs[DataIndex];
			const FPCGTaggedData& Output = Outputs[DataIndex];

			const UPCGSpatialData* InSpatialData = Cast<UPCGSpatialData>(Input.Data);
			check(InSpatialData);

			const UPCGBasePointData* InPointData = InSpatialData->ToBasePointData(Context.Get());
			check(InPointData);

			const UPCGSpatialData* OutSpatialData = Cast<UPCGSpatialData>(Output.Data);

			if (!TestNotNull("Valid output SpatialData", OutSpatialData))
			{
				bTestPassed = false;
				continue;
			}

			const UPCGBasePointData* OutPointData = OutSpatialData->ToBasePointData(Context.Get());

			if (!TestNotNull("Valid output PointData", OutPointData))
			{
				bTestPassed = false;
				continue;
			}

			if (!TestEqual("Input and output point counts match", InPointData->GetNumPoints(), OutPointData->GetNumPoints()))
			{ 
				bTestPassed = false;
				continue;
			}

			if (!TestEqual("Test data has enough points", PointCount, OutPointData->GetNumPoints()))
			{
				bTestPassed = false;
				continue;
			}

			const TConstPCGValueRange<FVector> OutBoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
			const TConstPCGValueRange<FVector> OutBoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

			for (int PointIndex = 0; PointIndex < ExpectedOutput.Num()/2; ++PointIndex)
			{
				bTestPassed &= TestEqual("Correct BoundsMin", OutBoundsMinRange[PointIndex], ExpectedOutput[PointIndex*2+0]);
				bTestPassed &= TestEqual("Correct BoundsMax", OutBoundsMaxRange[PointIndex], ExpectedOutput[PointIndex*2+1]);
			}
		}

		return bTestPassed;
	};

	bool bTestPassed = true;

	// Test expected values for Set
	{
		Settings->Extents = FVector(0.5f, 0.5f, 0.5f);
		Settings->Mode = EPCGPointExtentsModifierMode::Set;
		bTestPassed &= ValidateBounds({
			{-0.5f, -0.5f, -0.5f}, {0.5f, 0.5f, 0.5f},
			{-0.5f, -0.5f, -0.5f}, {0.5f, 0.5f, 0.5f},
			{-0.5f, -0.5f, -0.5f}, {0.5f, 0.5f, 0.5f},
			{0.5f, 0.5f, 0.5f}, {1.5f, 1.5f, 1.5f},
		});
	}


	// Test expected values for Minimum
	{
		Settings->Extents = FVector(2.0f, 2.0f, 2.0f);
		Settings->Mode = EPCGPointExtentsModifierMode::Minimum;

		bTestPassed &= ValidateBounds({
			{-1.0f, -1.0f, -1.0f}, {1.0f, 1.0f, 1.0f},
			{-2.0f, -2.0f, -2.0f}, {2.0f, 2.0f, 2.0f},
			{-2.0f, -2.0f, -2.0f}, {2.0f, 2.0f, 2.0f},
			{-1.0f, -1.0f, -1.0f}, {3.0f, 3.0f, 3.0f},
		});
	}

	// Test expected values for Maximum
	{
		Settings->Extents = FVector(2.0f, 2.0f, 2.0f);
		Settings->Mode = EPCGPointExtentsModifierMode::Maximum;

		bTestPassed &= ValidateBounds({
			{-2.0f, -2.0f, -2.0f}, {2.0f, 2.0f, 2.0f},
			{-2.0f, -2.0f, -2.0f}, {2.0f, 2.0f, 2.0f},
			{-3.0f, -3.0f, -3.0f}, {3.0f, 3.0f, 3.0f},
			{-3.0f, -3.0f, -3.0f}, {5.0f, 5.0f, 5.0f},
		});
	}

	// Test expected values for Add
	{
		Settings->Extents = FVector(2.0f, 2.0f, 2.0f);
		Settings->Mode = EPCGPointExtentsModifierMode::Add;

		bTestPassed &= ValidateBounds({
			{-3.0f, -3.0f, -3.0f}, {3.0f, 3.0f, 3.0f},
			{-4.0f, -4.0f, -4.0f}, {4.0f, 4.0f, 4.0f},
			{-5.0f, -5.0f, -5.0f}, {5.0f, 5.0f, 5.0f},
			{-5.0f, -5.0f, -5.0f}, {7.0f, 7.0f, 7.0f},
		});
	}


	// Test expected values for Multiply
	{
		Settings->Extents = FVector(2.0f, 2.0f, 2.0f);
		Settings->Mode = EPCGPointExtentsModifierMode::Multiply;

		bTestPassed &= ValidateBounds({
			{-2.0f, -2.0f, -2.0f}, {2.0f, 2.0f, 2.0f},
			{-4.0f, -4.0f, -4.0f}, {4.0f, 4.0f, 4.0f},
			{-6.0f, -6.0f, -6.0f}, {6.0f, 6.0f, 6.0f},
			{-7.0f, -7.0f, -7.0f}, {9.0f, 9.0f, 9.0f},
		});
	}

	return bTestPassed;
}
