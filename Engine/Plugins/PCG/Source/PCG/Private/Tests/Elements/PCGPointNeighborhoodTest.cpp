// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"

#include "Elements/PCGPointNeighborhood.h"

class PCGPointNeighborhoodTestBase : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;

	struct FTestParameters
	{
		double SearchDistance = 0.0;
		bool bDistanceToAttr = false;
		FName DistanceName;
		bool bPositionToAttr = false;
		FName PositionName;
		EPCGPointNeighborhoodDensityMode SetDensity = EPCGPointNeighborhoodDensityMode::None;
		bool bAvgPosition = false;
		bool bAvgColor = false;
		bool bUseBounds = true;
	};

protected:
	TUniquePtr<FPCGContext> GenerateTestDataAndRun(const FTestParameters& Parameters)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPCGPointNeighborhoodSettings>(TestData);
		UPCGPointNeighborhoodSettings* Settings = CastChecked<UPCGPointNeighborhoodSettings>(TestData.Settings);
		Settings->SearchDistance = Parameters.SearchDistance;
		Settings->bSetDistanceToAttribute = Parameters.bDistanceToAttr;
		Settings->DistanceAttribute = Parameters.DistanceName;
		Settings->bSetAveragePositionToAttribute = Parameters.bPositionToAttr;
		Settings->AveragePositionAttribute = Parameters.PositionName;
		Settings->SetDensity = Parameters.SetDensity;
		Settings->bSetAveragePosition = Parameters.bAvgPosition;
		Settings->bSetAverageColor = Parameters.bAvgColor;
		Settings->bWeightedAverage = Parameters.bUseBounds;

		FPCGTaggedData& Inputs = TestData.InputData.TaggedData.Emplace_GetRef();
		Inputs.Pin = PCGPinConstants::DefaultInputLabel;
		UPCGBasePointData* InData = PCGTestsCommon::CreateBasePointData();
		Inputs.Data = InData;

		InData->SetNumPoints(9);
		InData->SetBoundsMin(FVector(-10));
		InData->SetBoundsMax(FVector(10));

		TPCGValueRange<FTransform> TransformRange = InData->GetTransformValueRange();

		for (int i = 0; i < 3; ++i)
		{
			for (int j = 0; j < 3; ++j)
			{
				TransformRange[(i * 3) + j].SetLocation(FVector(j * 100.0, i * 100.0, 0.0));
			}
		}

		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		return Context;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointNeighborhoodTest_AttrDistance, PCGPointNeighborhoodTestBase, "Plugins.PCG.PointNeighborhood.AttrDistance", PCGTestsCommon::TestFlags)

bool FPCGPointNeighborhoodTest_AttrDistance::RunTest(const FString& Parameters)
{
	FTestParameters DistanceAttrParameters{};
	DistanceAttrParameters.SearchDistance = 225.0;
	DistanceAttrParameters.bDistanceToAttr = true;
	DistanceAttrParameters.DistanceName = TEXT("Distance");
	DistanceAttrParameters.bPositionToAttr = false;
	DistanceAttrParameters.PositionName = NAME_None;
	DistanceAttrParameters.SetDensity = EPCGPointNeighborhoodDensityMode::None;
	DistanceAttrParameters.bAvgPosition = false;
	DistanceAttrParameters.bAvgColor = false;
	DistanceAttrParameters.bUseBounds = true;

	TUniquePtr<FPCGContext> Context = GenerateTestDataAndRun(DistanceAttrParameters);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 9);

	const FPCGMetadataAttribute<double>* DistanceAttr = OutPointData->Metadata->GetConstTypedAttribute<double>(DistanceAttrParameters.DistanceName);
	UTEST_NOT_NULL("Distance Attribute exists", DistanceAttr);

	const TConstPCGValueRange<int64> MetadataEntryRange = OutPointData->GetConstMetadataEntryValueRange();
	UTEST_EQUAL("Point 4 of Distance Attribute", DistanceAttr->GetValueFromItemKey(MetadataEntryRange[4]), 0.0)

	return true; 
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointNeighborhoodTest_AttrPosition, PCGPointNeighborhoodTestBase, "Plugins.PCG.PointNeighborhood.AttrPosition", PCGTestsCommon::TestFlags)

bool FPCGPointNeighborhoodTest_AttrPosition::RunTest(const FString& Parameters)
{
	FTestParameters PositionAttrParameters{};
	PositionAttrParameters.SearchDistance = 225.0;
	PositionAttrParameters.bDistanceToAttr = false;
	PositionAttrParameters.DistanceName = NAME_None;
	PositionAttrParameters.bPositionToAttr = true;
	PositionAttrParameters.PositionName = TEXT("AvgPosition");
	PositionAttrParameters.SetDensity = EPCGPointNeighborhoodDensityMode::None;
	PositionAttrParameters.bAvgPosition = false;
	PositionAttrParameters.bAvgColor = false;
	PositionAttrParameters.bUseBounds = true;

	TUniquePtr<FPCGContext> Context = GenerateTestDataAndRun(PositionAttrParameters);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 9);

	const FPCGMetadataAttribute<FVector>* PositionAttr = OutPointData->Metadata->GetConstTypedAttribute<FVector>(PositionAttrParameters.PositionName);
	UTEST_NOT_NULL("Average Position Attribute exists", PositionAttr);

	const TConstPCGValueRange<int64> MetadataEntryRange = OutPointData->GetConstMetadataEntryValueRange();
	UTEST_EQUAL("Point 4 of Position Attribute", PositionAttr->GetValueFromItemKey(MetadataEntryRange[4]), FVector(100.0, 100.0, 0.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointNeighborhoodTest_DensityAvg, PCGPointNeighborhoodTestBase, "Plugins.PCG.PointNeighborhood.DensityAvg", PCGTestsCommon::TestFlags)

bool FPCGPointNeighborhoodTest_DensityAvg::RunTest(const FString& Parameters)
{
	FTestParameters AvgDensityParameters{};
	AvgDensityParameters.SearchDistance = 225.0;
	AvgDensityParameters.bDistanceToAttr = false;
	AvgDensityParameters.DistanceName = NAME_None;
	AvgDensityParameters.bPositionToAttr = false;
	AvgDensityParameters.PositionName = NAME_None;
	AvgDensityParameters.SetDensity = EPCGPointNeighborhoodDensityMode::SetAverageDensity;
	AvgDensityParameters.bAvgPosition = false;
	AvgDensityParameters.bAvgColor = false;
	AvgDensityParameters.bUseBounds = true;

	TUniquePtr<FPCGContext> Context = GenerateTestDataAndRun(AvgDensityParameters);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 9);

	/** Steepness taken into account:
	 * Point Bounds = 30 -> (20cm * (1 + 0.5))
	 * Search Bounds = 450 (|-225| + 225)
	 * Given 9 contributions:
	 * 9 * (30 ^ 3) / (450 ^ 3) == 0.00266...*/
	const TConstPCGValueRange<float> DensityRange = OutPointData->GetConstDensityValueRange();
	UTEST_EQUAL("Average Density in Point 0 Density Attribute", DensityRange[0], 9.0f * FMath::Pow(30.0f, 3.0f) / FMath::Pow(450.0f, 3.0f));
	UTEST_EQUAL("Average Density in Point 4 Density Attribute", DensityRange[4], 9.0f * FMath::Pow(30.0f, 3.0f) / FMath::Pow(450.0f, 3.0f));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointNeighborhoodTest_DensityAvgNoBounds, PCGPointNeighborhoodTestBase, "Plugins.PCG.PointNeighborhood.DensityAvgNoBounds", PCGTestsCommon::TestFlags)

bool FPCGPointNeighborhoodTest_DensityAvgNoBounds::RunTest(const FString& Parameters)
{
	FTestParameters NoBoundsAvgParameters{};
	NoBoundsAvgParameters.SearchDistance = 225.0;
	NoBoundsAvgParameters.bDistanceToAttr = false;
	NoBoundsAvgParameters.DistanceName = NAME_None;
	NoBoundsAvgParameters.bPositionToAttr = false;
	NoBoundsAvgParameters.PositionName = NAME_None;
	NoBoundsAvgParameters.SetDensity = EPCGPointNeighborhoodDensityMode::SetAverageDensity;
	NoBoundsAvgParameters.bAvgPosition = false;
	NoBoundsAvgParameters.bAvgColor = false;
	NoBoundsAvgParameters.bUseBounds = false;

	TUniquePtr<FPCGContext> Context = GenerateTestDataAndRun(NoBoundsAvgParameters);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 9);

	/** Density / num points total = 1 / 9 contribution per point
	 * 9 * 1/9 == 1 average density */ 
	const TConstPCGValueRange<float> DensityRange = OutPointData->GetConstDensityValueRange();
	UTEST_EQUAL("Average Density in Point 0 Density Attribute", DensityRange[0], 1.0f);
	UTEST_EQUAL("Average Density in Point 4 Density Attribute", DensityRange[4], 1.0f);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointNeighborhoodTest_DensityNormal, PCGPointNeighborhoodTestBase, "Plugins.PCG.PointNeighborhood.DensityNormal", PCGTestsCommon::TestFlags)

bool FPCGPointNeighborhoodTest_DensityNormal::RunTest(const FString& Parameters)
{
	FTestParameters NormalDensityParameters{};
	NormalDensityParameters.SearchDistance = 225.0;
	NormalDensityParameters.bDistanceToAttr = false;
	NormalDensityParameters.DistanceName = NAME_None;
	NormalDensityParameters.bPositionToAttr = false;
	NormalDensityParameters.PositionName = NAME_None;
	NormalDensityParameters.SetDensity = EPCGPointNeighborhoodDensityMode::SetNormalizedDistanceToDensity;
	NormalDensityParameters.bAvgPosition = false;
	NormalDensityParameters.bAvgColor = false;
	NormalDensityParameters.bUseBounds = true;

	TUniquePtr<FPCGContext> Context = GenerateTestDataAndRun(NormalDensityParameters);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 9);

	/** Distance between each point / search distance (radius)
	 * sqrt(100^2 + 100^2) == 100*sqrt(2) / 225
	 * == 0.629...
	*/
	const TConstPCGValueRange<float> DensityRange = OutPointData->GetConstDensityValueRange();
	UTEST_EQUAL("Normalized Distance in Point 0 Density Attribute", DensityRange[0], (100.0f * FMath::Sqrt(2.0f)) / 225.0f);
	UTEST_EQUAL("Normalized Distance in Point 4 Density Attribute", DensityRange[4], 0.0f);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointNeighborhoodTest_AvgPosition, PCGPointNeighborhoodTestBase, "Plugins.PCG.PointNeighborhood.AvgPosition", PCGTestsCommon::TestFlags)

bool FPCGPointNeighborhoodTest_AvgPosition::RunTest(const FString& Parameters)
{
	FTestParameters AveragePositionParameters{};
	AveragePositionParameters.SearchDistance = 225.0;
	AveragePositionParameters.bDistanceToAttr = false;
	AveragePositionParameters.DistanceName = NAME_None;
	AveragePositionParameters.bPositionToAttr = false;
	AveragePositionParameters.PositionName = NAME_None;
	AveragePositionParameters.SetDensity = EPCGPointNeighborhoodDensityMode::None;
	AveragePositionParameters.bAvgPosition = true;
	AveragePositionParameters.bAvgColor = false;
	AveragePositionParameters.bUseBounds = true;

	TUniquePtr<FPCGContext> Context = GenerateTestDataAndRun(AveragePositionParameters);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 9);

	const TConstPCGValueRange<FTransform> TransformRange = OutPointData->GetConstTransformValueRange();
	UTEST_EQUAL("Average Position in Point 0 Location Attribute", TransformRange[0].GetLocation(), FVector(100.0, 100.0, 0.0));
	UTEST_EQUAL("Average Position in Point 4 Location Attribute", TransformRange[4].GetLocation(), FVector(100.0, 100.0, 0.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGPointNeighborhoodTest_AvgColor, PCGPointNeighborhoodTestBase, "Plugins.PCG.PointNeighborhood.AvgColor", PCGTestsCommon::TestFlags)

bool FPCGPointNeighborhoodTest_AvgColor::RunTest(const FString& Parameters)
{
	FTestParameters AverageColorParameters{};
	AverageColorParameters.SearchDistance = 225.0;
	AverageColorParameters.bDistanceToAttr = false;
	AverageColorParameters.DistanceName = NAME_None;
	AverageColorParameters.bPositionToAttr = false;
	AverageColorParameters.PositionName = NAME_None;
	AverageColorParameters.SetDensity = EPCGPointNeighborhoodDensityMode::None;
	AverageColorParameters.bAvgPosition = true;
	AverageColorParameters.bAvgColor = false;
	AverageColorParameters.bUseBounds = true;

	TUniquePtr<FPCGContext> Context = GenerateTestDataAndRun(AverageColorParameters);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);

	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 9);

	const TConstPCGValueRange<FVector4> ColorRange = OutPointData->GetConstColorValueRange();
	for (int i = 0; i < OutPointData->GetNumPoints(); ++i)
	{
		UTEST_EQUAL("Average Color in Location Attribute", ColorRange[i], FVector4(1.0, 1.0, 1.0, 1.0));
	}
	
	return true;
}