// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include "Elements/PCGRandomChoice.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGRandomChoiceTest_Fixed, FPCGTestBaseClass, "Plugins.PCG.RandomChoice.Fixed", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGRandomChoiceTest_Ratio, FPCGTestBaseClass, "Plugins.PCG.RandomChoice.Ratio", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGRandomChoiceTest_SelectNone, FPCGTestBaseClass, "Plugins.PCG.RandomChoice.SelectNone", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGRandomChoiceTest_SelectAll, FPCGTestBaseClass, "Plugins.PCG.RandomChoice.SelectAll", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGRandomChoiceTest_NoDiscard, FPCGTestBaseClass, "Plugins.PCG.RandomChoice.NoDiscard", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGRandomChoiceTest_MultiData_SameSeed, FPCGTestBaseClass, "Plugins.PCG.RandomChoice.MultiData.SameSeed", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGRandomChoiceTest_MultiData_DifferentSeed, FPCGTestBaseClass, "Plugins.PCG.RandomChoice.MultiData.DifferentSeed", PCGTestsCommon::TestFlags)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGRandomChoiceTest_Fixed_ParamData, FPCGTestBaseClass, "Plugins.PCG.RandomChoice.FixedParamData", PCGTestsCommon::TestFlags)

namespace PCGRandomChoiceTest
{
	UPCGBasePointData* CreateInputPointData(FPCGContext* Context, const int NumPoints, const int Seed = 42)
	{
		check(Context);

		UPCGBasePointData* NewPointData = PCGTestsCommon::CreateEmptyBasePointData();
		NewPointData->SetFlags(RF_Transient);
		NewPointData->SetNumPoints(NumPoints);
		NewPointData->SetSeed(Seed);
		NewPointData->AllocateProperties(EPCGPointNativeProperties::Density);

		TPCGValueRange<float> DensityRange = NewPointData->GetDensityValueRange(/*bAllocate=*/false);
		for (int i = 0; i < NumPoints; ++i)
		{
			// Store the index in the density
			DensityRange[i] = i;
		}

		FPCGTaggedData& InputData = Context->InputData.TaggedData.Emplace_GetRef();
		InputData.Data = NewPointData;
		InputData.Pin = PCGPinConstants::DefaultInputLabel;

		return NewPointData;
	}

	const FName DensityAttributeName = TEXT("MyDensity");

	UPCGParamData* CreateInputParamData(FPCGContext* Context, const int NumElements)
	{
		check(Context);

		UPCGParamData* NewParamData = NewObject<UPCGParamData>();
		NewParamData->SetFlags(RF_Transient);

		FPCGMetadataAttribute<double>* DensityAttribute = NewParamData->Metadata->CreateAttribute<double>(DensityAttributeName, 0.0, true, false);
		check(DensityAttribute);

		for (int i = 0; i < NumElements; ++i)
		{
			DensityAttribute->SetValue(NewParamData->Metadata->AddEntry(), static_cast<double>(i));
		}

		FPCGTaggedData& InputData = Context->InputData.TaggedData.Emplace_GetRef();
		InputData.Data = NewParamData;
		InputData.Pin = PCGPinConstants::DefaultInputLabel;

		return NewParamData;
	}

	bool VerifyAllPointsThere(const int NumPoints, const UPCGBasePointData* ChosenPointData, const UPCGBasePointData* DiscardedPointData)
	{
		check(ChosenPointData && DiscardedPointData);
		TSet<int> IndexesSeen;

		auto Check = [&IndexesSeen](const UPCGBasePointData* PointData)
		{
			const TConstPCGValueRange<float> DensityRange = PointData->GetConstDensityValueRange();
			for (int i = 0; i < PointData->GetNumPoints(); ++i)
			{
				const int Index = static_cast<int>(DensityRange[i]);
				if (IndexesSeen.Contains(Index))
				{
					return false;
				}

				IndexesSeen.Add(Index);

				// It needs to be stable so density should be ascending.
				if (i > 0 && DensityRange[i] <= DensityRange[i - 1])
				{
					return false;
				}
			}

			return true;
		};

		if (!Check(ChosenPointData))
		{
			return false;
		}

		if (!Check(DiscardedPointData))
		{
			return false;
		}

		return IndexesSeen.Num() == NumPoints;
	}

	bool VerifyAllEntriesAreThere(const int NumElements, const UPCGParamData* ChosenParamData, const UPCGParamData* DiscardedParamData)
	{
		check(ChosenParamData && DiscardedParamData);
		TSet<int> IndexesSeen;

		auto Check = [&IndexesSeen](const UPCGParamData* ParamData)
		{
			const FPCGMetadataAttribute<double>* DensityAttribute = ParamData->Metadata->GetConstTypedAttribute<double>(DensityAttributeName);
			if (!DensityAttribute)
			{
				return false;
			}

			int PreviousIndex = -1;
			for (int i = 0; i < ParamData->Metadata->GetLocalItemCount(); ++i)
			{
				const int Index = static_cast<int>(DensityAttribute->GetValueFromItemKey(PCGMetadataEntryKey(i)));
				if (IndexesSeen.Contains(Index))
				{
					return false;
				}

				IndexesSeen.Add(Index);

				// It needs to be stable so density should be ascending.
				if (i > 0 && Index <= PreviousIndex)
				{
					return false;
				}

				PreviousIndex = Index;
			}

			return true;
		};

		if (!Check(ChosenParamData))
		{
			return false;
		}

		if (!Check(DiscardedParamData))
		{
			return false;
		}

		return IndexesSeen.Num() == NumElements;
	}
}

bool FPCGRandomChoiceTest_Fixed::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGRandomChoiceSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGRandomChoiceSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;
	constexpr int ExpectedNumElementsChosen = 7;
	constexpr int ExpectedNumElementsDiscarded = 13;

	Settings->bFixedMode = true;
	Settings->FixedNumber = ExpectedNumElementsChosen;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* InputPointData = PCGRandomChoiceTest::CreateInputPointData(Context.Get(), NumOfPoints);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> ChosenOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::ChosenEntriesLabel);
	TArray<FPCGTaggedData> DiscardedOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::DiscardedEntriesLabel);

	const UPCGBasePointData* ChosenOutputData = ChosenOutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(ChosenOutputTagged[0].Data) : nullptr;
	const UPCGBasePointData* DiscardedOutputData = DiscardedOutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(DiscardedOutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in chosen points", ChosenOutputData);
	UTEST_NOT_NULL("There is a point data in discarded points", DiscardedOutputData);

	UTEST_EQUAL("There is the right number of points in chosen", ChosenOutputData->GetNumPoints(), ExpectedNumElementsChosen);
	UTEST_EQUAL("There is the right number of points in discarded", DiscardedOutputData->GetNumPoints(), ExpectedNumElementsDiscarded);

	UTEST_TRUE("All points are there and in the right order", PCGRandomChoiceTest::VerifyAllPointsThere(NumOfPoints, ChosenOutputData, DiscardedOutputData));

	return true;
}

bool FPCGRandomChoiceTest_Ratio::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGRandomChoiceSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGRandomChoiceSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;
	constexpr int ExpectedNumElementsChosen = 5;
	constexpr int ExpectedNumElementsDiscarded = 15;

	Settings->bFixedMode = false;
	Settings->Ratio = 0.25f;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* InputPointData = PCGRandomChoiceTest::CreateInputPointData(Context.Get(), NumOfPoints);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> ChosenOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::ChosenEntriesLabel);
	TArray<FPCGTaggedData> DiscardedOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::DiscardedEntriesLabel);

	const UPCGBasePointData* ChosenOutputData = ChosenOutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(ChosenOutputTagged[0].Data) : nullptr;
	const UPCGBasePointData* DiscardedOutputData = DiscardedOutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(DiscardedOutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in chosen points", ChosenOutputData);
	UTEST_NOT_NULL("There is a point data in discarded points", DiscardedOutputData);

	UTEST_EQUAL("There is the right number of points in chosen", ChosenOutputData->GetNumPoints(), ExpectedNumElementsChosen);
	UTEST_EQUAL("There is the right number of points in discarded", DiscardedOutputData->GetNumPoints(), ExpectedNumElementsDiscarded);

	UTEST_TRUE("All points are there and in the right order", PCGRandomChoiceTest::VerifyAllPointsThere(NumOfPoints, ChosenOutputData, DiscardedOutputData));

	return true;
}

bool FPCGRandomChoiceTest_SelectNone::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGRandomChoiceSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGRandomChoiceSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->bFixedMode = true;
	Settings->FixedNumber = 0;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* InputPointData = PCGRandomChoiceTest::CreateInputPointData(Context.Get(), NumOfPoints);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> ChosenOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::ChosenEntriesLabel);
	TArray<FPCGTaggedData> DiscardedOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::DiscardedEntriesLabel);

	const UPCGBasePointData* ChosenOutputData = ChosenOutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(ChosenOutputTagged[0].Data) : nullptr;
	const UPCGBasePointData* DiscardedOutputData = DiscardedOutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(DiscardedOutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in chosen points", ChosenOutputData);
	UTEST_NOT_NULL("There is a point data in discarded points", DiscardedOutputData);

	UTEST_EQUAL("Chosen points is empty", ChosenOutputData->GetNumPoints(), 0);
	UTEST_EQUAL("Discarded points is the input data", DiscardedOutputData, InputPointData);

	return true;
}

bool FPCGRandomChoiceTest_SelectAll::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGRandomChoiceSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGRandomChoiceSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;

	Settings->bFixedMode = false;
	Settings->Ratio = 1.0f;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* InputPointData = PCGRandomChoiceTest::CreateInputPointData(Context.Get(), NumOfPoints);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> ChosenOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::ChosenEntriesLabel);
	TArray<FPCGTaggedData> DiscardedOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::DiscardedEntriesLabel);

	const UPCGBasePointData* ChosenOutputData = ChosenOutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(ChosenOutputTagged[0].Data) : nullptr;
	const UPCGBasePointData* DiscardedOutputData = DiscardedOutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(DiscardedOutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in chosen points", ChosenOutputData);
	UTEST_NOT_NULL("There is a point data in discarded points", DiscardedOutputData);

	UTEST_EQUAL("Chosen points is the input data", ChosenOutputData, InputPointData);
	UTEST_EQUAL("Discarded points is empty", DiscardedOutputData->GetNumPoints(), 0);

	return true;
}

bool FPCGRandomChoiceTest_NoDiscard::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGRandomChoiceSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGRandomChoiceSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 20;
	constexpr int ExpectedNumElementsChosen = 2;

	Settings->bFixedMode = false;
	Settings->Ratio = 0.1f;
	Settings->bOutputDiscardedEntries = false;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* InputPointData = PCGRandomChoiceTest::CreateInputPointData(Context.Get(), NumOfPoints);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> ChosenOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::ChosenEntriesLabel);
	TArray<FPCGTaggedData> DiscardedOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::DiscardedEntriesLabel);

	const UPCGBasePointData* ChosenOutputData = ChosenOutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(ChosenOutputTagged[0].Data) : nullptr;
	const UPCGBasePointData* DiscardedOutputData = DiscardedOutputTagged.Num() == 1 ? Cast<const UPCGBasePointData>(DiscardedOutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a point data in chosen points", ChosenOutputData);
	UTEST_NULL("There is no point data in discarded points", DiscardedOutputData);

	UTEST_EQUAL("There is the right number of points in chosen", ChosenOutputData->GetNumPoints(), ExpectedNumElementsChosen);

	return true;
}

bool FPCGRandomChoiceTest_MultiData_SameSeed::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGRandomChoiceSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGRandomChoiceSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 100;
	constexpr int ExpectedNumElementsChosen = 7;
	constexpr int ExpectedNumElementsDiscarded = 93;

	Settings->bFixedMode = true;
	Settings->FixedNumber = ExpectedNumElementsChosen;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* FirstInputPointData = PCGRandomChoiceTest::CreateInputPointData(Context.Get(), NumOfPoints);
	const UPCGBasePointData* SecondInputPointData = PCGRandomChoiceTest::CreateInputPointData(Context.Get(), NumOfPoints);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> ChosenOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::ChosenEntriesLabel);
	TArray<FPCGTaggedData> DiscardedOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::DiscardedEntriesLabel);

	const UPCGBasePointData* FirstChosenOutputData = ChosenOutputTagged.Num() == 2 ? Cast<const UPCGBasePointData>(ChosenOutputTagged[0].Data) : nullptr;
	const UPCGBasePointData* FirstDiscardedOutputData = DiscardedOutputTagged.Num() == 2 ? Cast<const UPCGBasePointData>(DiscardedOutputTagged[0].Data) : nullptr;

	const UPCGBasePointData* SecondChosenOutputData = ChosenOutputTagged.Num() == 2 ? Cast<const UPCGBasePointData>(ChosenOutputTagged[1].Data) : nullptr;
	const UPCGBasePointData* SecondDiscardedOutputData = DiscardedOutputTagged.Num() == 2 ? Cast<const UPCGBasePointData>(DiscardedOutputTagged[1].Data) : nullptr;

	UTEST_TRUE("There is 2 point data in chosen points", FirstChosenOutputData && SecondChosenOutputData);
	UTEST_TRUE("There is 2 point data in discarded points", FirstDiscardedOutputData && SecondDiscardedOutputData);

	check(FirstChosenOutputData && SecondChosenOutputData && FirstDiscardedOutputData && SecondDiscardedOutputData);

	UTEST_EQUAL("There is the right number of points in first chosen", FirstChosenOutputData->GetNumPoints(), ExpectedNumElementsChosen);
	UTEST_EQUAL("There is the right number of points in second chosen", SecondChosenOutputData->GetNumPoints(), ExpectedNumElementsChosen);

	UTEST_EQUAL("There is the right number of points in first discarded", FirstDiscardedOutputData->GetNumPoints(), ExpectedNumElementsDiscarded);
	UTEST_EQUAL("There is the right number of points in second discarded", SecondDiscardedOutputData->GetNumPoints(), ExpectedNumElementsDiscarded);

	UTEST_TRUE("All points are there and in the right order for first data", PCGRandomChoiceTest::VerifyAllPointsThere(NumOfPoints, FirstChosenOutputData, FirstDiscardedOutputData));
	UTEST_TRUE("All points are there and in the right order for second data", PCGRandomChoiceTest::VerifyAllPointsThere(NumOfPoints, SecondChosenOutputData, SecondDiscardedOutputData));

	// If the point seeds are the same, chosen points should be exactly the same
	const TConstPCGValueRange<float> FirstDensityRange = FirstChosenOutputData->GetConstDensityValueRange();
	const TConstPCGValueRange<float> SecondDensityRange = SecondChosenOutputData->GetConstDensityValueRange();
	for (int32 i = 0; i < ExpectedNumElementsChosen; ++i)
	{
		UTEST_TRUE(*FString::Printf(TEXT("Point %d is the same for both chosen inputs"), i), FirstDensityRange[i] == SecondDensityRange[i]);
	}

	return true;
}

bool FPCGRandomChoiceTest_MultiData_DifferentSeed::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGRandomChoiceSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGRandomChoiceSettings>(TestData);
	check(Settings);

	constexpr int NumOfPoints = 100;
	constexpr int ExpectedNumElementsChosen = 5;
	constexpr int ExpectedNumElementsDiscarded = 95;

	Settings->bFixedMode = true;
	Settings->FixedNumber = ExpectedNumElementsChosen;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGBasePointData* FirstInputPointData = PCGRandomChoiceTest::CreateInputPointData(Context.Get(), NumOfPoints);
	const UPCGBasePointData* SecondInputPointData = PCGRandomChoiceTest::CreateInputPointData(Context.Get(), NumOfPoints, 4653465);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> ChosenOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::ChosenEntriesLabel);
	TArray<FPCGTaggedData> DiscardedOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::DiscardedEntriesLabel);

	const UPCGBasePointData* FirstChosenOutputData = ChosenOutputTagged.Num() == 2 ? Cast<const UPCGBasePointData>(ChosenOutputTagged[0].Data) : nullptr;
	const UPCGBasePointData* FirstDiscardedOutputData = DiscardedOutputTagged.Num() == 2 ? Cast<const UPCGBasePointData>(DiscardedOutputTagged[0].Data) : nullptr;

	const UPCGBasePointData* SecondChosenOutputData = ChosenOutputTagged.Num() == 2 ? Cast<const UPCGBasePointData>(ChosenOutputTagged[1].Data) : nullptr;
	const UPCGBasePointData* SecondDiscardedOutputData = DiscardedOutputTagged.Num() == 2 ? Cast<const UPCGBasePointData>(DiscardedOutputTagged[1].Data) : nullptr;

	UTEST_TRUE("There is 2 point data in chosen points", FirstChosenOutputData && SecondChosenOutputData);
	UTEST_TRUE("There is 2 point data in discarded points", FirstDiscardedOutputData && SecondDiscardedOutputData);

	check(FirstChosenOutputData && SecondChosenOutputData && FirstDiscardedOutputData && SecondDiscardedOutputData);

	UTEST_EQUAL("There is the right number of points in first chosen", FirstChosenOutputData->GetNumPoints(), ExpectedNumElementsChosen);
	UTEST_EQUAL("There is the right number of points in second chosen", SecondChosenOutputData->GetNumPoints(), ExpectedNumElementsChosen);

	UTEST_EQUAL("There is the right number of points in first discarded", FirstDiscardedOutputData->GetNumPoints(), ExpectedNumElementsDiscarded);
	UTEST_EQUAL("There is the right number of points in second discarded", SecondDiscardedOutputData->GetNumPoints(), ExpectedNumElementsDiscarded);

	UTEST_TRUE("All points are there and in the right order for first data", PCGRandomChoiceTest::VerifyAllPointsThere(NumOfPoints, FirstChosenOutputData, FirstDiscardedOutputData));
	UTEST_TRUE("All points are there and in the right order for second data", PCGRandomChoiceTest::VerifyAllPointsThere(NumOfPoints, SecondChosenOutputData, SecondDiscardedOutputData));

	// If point seeds are different, chosen points should be different. Verify that at least one point is different.
	bool bIsDifferent = false;
	const TConstPCGValueRange<float> FirstDensityRange = FirstChosenOutputData->GetConstDensityValueRange();
	const TConstPCGValueRange<float> SecondDensityRange = SecondChosenOutputData->GetConstDensityValueRange();
	for (int32 i = 0; i < ExpectedNumElementsChosen && !bIsDifferent; ++i)
	{
		bIsDifferent = FirstDensityRange[i] != SecondDensityRange[i];
	}

	UTEST_TRUE("Chosen points are different", bIsDifferent);

	return true;
}

bool FPCGRandomChoiceTest_Fixed_ParamData::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGRandomChoiceSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGRandomChoiceSettings>(TestData);
	check(Settings);

	constexpr int NumOfElements = 20;
	constexpr int ExpectedNumElementsChosen = 7;
	constexpr int ExpectedNumElementsDiscarded = 13;

	Settings->bFixedMode = true;
	Settings->FixedNumber = ExpectedNumElementsChosen;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	const UPCGParamData* InputParamData = PCGRandomChoiceTest::CreateInputParamData(Context.Get(), NumOfElements);

	FPCGElementPtr TestElement = TestData.Settings->GetElement();
	while (!TestElement->Execute(Context.Get())) {}

	TArray<FPCGTaggedData> ChosenOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::ChosenEntriesLabel);
	TArray<FPCGTaggedData> DiscardedOutputTagged = Context->OutputData.GetInputsByPin(PCGRandomChoiceConstants::DiscardedEntriesLabel);

	const UPCGParamData* ChosenOutputData = ChosenOutputTagged.Num() == 1 ? Cast<const UPCGParamData>(ChosenOutputTagged[0].Data) : nullptr;
	const UPCGParamData* DiscardedOutputData = DiscardedOutputTagged.Num() == 1 ? Cast<const UPCGParamData>(DiscardedOutputTagged[0].Data) : nullptr;

	UTEST_NOT_NULL("There is a param data in chosen entries", ChosenOutputData);
	UTEST_NOT_NULL("There is a param data in discarded entries", DiscardedOutputData);

	UTEST_EQUAL("There is the right number of entries in chosen", ChosenOutputData->Metadata->GetLocalItemCount(), ExpectedNumElementsChosen);
	UTEST_EQUAL("There is the right number of entries in discarded", DiscardedOutputData->Metadata->GetLocalItemCount(), ExpectedNumElementsDiscarded);

	UTEST_TRUE("All entries are there and in the right order", PCGRandomChoiceTest::VerifyAllEntriesAreThere(NumOfElements, ChosenOutputData, DiscardedOutputData));

	return true;
}

#endif // WITH_EDITOR
