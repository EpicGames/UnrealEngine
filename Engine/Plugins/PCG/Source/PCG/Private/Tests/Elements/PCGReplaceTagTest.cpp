// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"

#include "Elements/PCGReplaceTags.h"

class PCGReplaceTagsTestBase : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;

	struct FTestParameters
	{
		bool ExpectedToHaveOneOutput = true;
		FString SelectedTags;
		FString ReplacedTags;
		TSet<FString> ExpectedOutputTags;
	};

	bool GenerateTestDataRunAndValidate(const FTestParameters& Parameters)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPCGReplaceTagsSettings>(TestData);
		UPCGReplaceTagsSettings* Settings = CastChecked<UPCGReplaceTagsSettings>(TestData.Settings);
		Settings->SelectedTags = Parameters.SelectedTags;
		Settings->ReplacedTags = Parameters.ReplacedTags;

		FPCGElementPtr TestElement = TestData.Settings->GetElement();
		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		FPCGTaggedData& InputTaggedData = Context->InputData.TaggedData.Emplace_GetRef();
		InputTaggedData.Data = PCGTestsCommon::CreateBasePointData();
		InputTaggedData.Pin = PCGPinConstants::DefaultInputLabel;
		InputTaggedData.Tags = TSet<FString>{ TEXT("Awesome"), TEXT("Fun"), TEXT("Cool") };

		while (!TestElement->Execute(Context.Get())) {}

		// Testing output
		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

		UTEST_EQUAL("Number of Outputs", Outputs.Num(), Parameters.ExpectedToHaveOneOutput? 1 : 0);

		if (Parameters.ExpectedToHaveOneOutput)
		{
			// Testing tag info
			const TSet<FString> OutputTagSet = Context->OutputData.TaggedData[0].Tags;

			UTEST_EQUAL("Tag count", OutputTagSet.Num(), Parameters.ExpectedOutputTags.Num());
			UTEST_TRUE("Output Tags", OutputTagSet.Includes(Parameters.ExpectedOutputTags) && Parameters.ExpectedOutputTags.Includes(OutputTagSet));
		}

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGReplaceTagTest_OneToOne, PCGReplaceTagsTestBase, "Plugins.PCG.ReplaceTag.OneToOne", PCGTestsCommon::TestFlags)

bool FPCGReplaceTagTest_OneToOne::RunTest(const FString& Parameters)
{
	FTestParameters OneToOneParameters{};
	OneToOneParameters.SelectedTags = TEXT("Awesome");
	OneToOneParameters.ReplacedTags = TEXT("Neat");
	OneToOneParameters.ExpectedToHaveOneOutput = true;
	OneToOneParameters.ExpectedOutputTags = TSet<FString>{ TEXT("Neat"), TEXT("Fun"), TEXT("Cool") };
	
	return GenerateTestDataRunAndValidate(OneToOneParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGReplaceTagTest_NToOne, PCGReplaceTagsTestBase, "Plugins.PCG.ReplaceTag.NToOne", PCGTestsCommon::TestFlags)

bool FPCGReplaceTagTest_NToOne::RunTest(const FString& Parameters)
{
	FTestParameters NtoOneParameters{};
	NtoOneParameters.SelectedTags = TEXT("Awesome, Fun");
	NtoOneParameters.ReplacedTags = TEXT("Neat");
	NtoOneParameters.ExpectedToHaveOneOutput = true;
	NtoOneParameters.ExpectedOutputTags = TSet<FString>{ TEXT("Neat"), TEXT("Cool") };

	return GenerateTestDataRunAndValidate(NtoOneParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGReplaceTagTest_NToN, PCGReplaceTagsTestBase, "Plugins.PCG.ReplaceTag.NToN", PCGTestsCommon::TestFlags)

bool FPCGReplaceTagTest_NToN::RunTest(const FString& Parameters)
{
	FTestParameters NtoNParameters{};
	NtoNParameters.SelectedTags = TEXT("Awesome, Fun, Cool");
	NtoNParameters.ReplacedTags = TEXT("Neat, Great, Amazing");
	NtoNParameters.ExpectedToHaveOneOutput = true;
	NtoNParameters.ExpectedOutputTags = TSet<FString>{ TEXT("Neat"), TEXT("Great"), TEXT("Amazing")};

	return GenerateTestDataRunAndValidate(NtoNParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGReplaceTagTest_OneToNError, PCGReplaceTagsTestBase, "Plugins.PCG.ReplaceTag.OneToNError", PCGTestsCommon::TestFlags)

bool FPCGReplaceTagTest_OneToNError::RunTest(const FString& Parameters)
{
	AddExpectedError(TEXT("Input only supports 1:1, N:1, and N:N mappings from source to replaced tags, Input data discarded."));

	FTestParameters OneToNErrorParameters{};
	OneToNErrorParameters.SelectedTags = TEXT("Awesome");
	OneToNErrorParameters.ReplacedTags = TEXT("Neat, Great");

	// Expects one output because input is forwarded
	OneToNErrorParameters.ExpectedToHaveOneOutput = true;
	OneToNErrorParameters.ExpectedOutputTags = TSet<FString>{ TEXT("Awesome"), TEXT("Fun"), TEXT("Cool") };

	return GenerateTestDataRunAndValidate(OneToNErrorParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGReplaceTagTest_RemoveTags, PCGReplaceTagsTestBase, "Plugins.PCG.ReplaceTag.RemoveTags", PCGTestsCommon::TestFlags)

bool FPCGReplaceTagTest_RemoveTags::RunTest(const FString& Parameters)
{
	FTestParameters RemoveTagsParameters{};
	RemoveTagsParameters.SelectedTags = TEXT("Awesome");
	RemoveTagsParameters.ReplacedTags = FString();
	RemoveTagsParameters.ExpectedToHaveOneOutput = true;
	RemoveTagsParameters.ExpectedOutputTags = TSet<FString>{ TEXT("Fun"), TEXT("Cool") };

	return GenerateTestDataRunAndValidate(RemoveTagsParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGReplaceTagTest_ForwardInput, PCGReplaceTagsTestBase, "Plugins.PCG.ReplaceTag.ForwardInput", PCGTestsCommon::TestFlags)

bool FPCGReplaceTagTest_ForwardInput::RunTest(const FString& Parameters)
{
	FTestParameters ForwardInputParameters{};
	ForwardInputParameters.SelectedTags = FString();
	ForwardInputParameters.ReplacedTags = FString();
	ForwardInputParameters.ExpectedToHaveOneOutput = true;
	ForwardInputParameters.ExpectedOutputTags = TSet<FString>{ TEXT("Awesome"), TEXT("Fun"), TEXT("Cool") };

	return GenerateTestDataRunAndValidate(ForwardInputParameters);
}