// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"

#include "Elements/PCGResetPointCenter.h"

namespace ResetPointCenterCommonTests
{
	TUniquePtr<FPCGContext> GenerateTestDataAndRun(FVector PointPosition, FRotator PointRotation, FVector PointScale)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPCGResetPointCenterSettings>(TestData);
		UPCGResetPointCenterSettings* Settings = CastChecked<UPCGResetPointCenterSettings>(TestData.Settings);
		Settings->PointCenterLocation = PointPosition;

		FPCGTaggedData& Inputs = TestData.InputData.TaggedData.Emplace_GetRef();
		Inputs.Pin = PCGPinConstants::DefaultInputLabel;
		UPCGBasePointData* InData = PCGTestsCommon::CreateBasePointData();
		Inputs.Data = InData;

		// Setting the scale and rotation explicitly
		TPCGValueRange<FTransform> TransformRange = InData->GetTransformValueRange();
		for (FTransform& PointTransform : TransformRange)
		{
			PointTransform.SetRotation(PointRotation.Quaternion());
			PointTransform.SetScale3D(PointScale);
		}
		
		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		return Context;
	}
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGResetPointCenterTest_Center, FPCGTestBaseClass, "Plugins.PCG.ResetPointCenter.Center", PCGTestsCommon::TestFlags)

bool FPCGResetPointCenterTest_Center::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = ResetPointCenterCommonTests::GenerateTestDataAndRun(FVector(0.5), FRotator::ZeroRotator, FVector::One());

	// Point center should be in the center of the bounding box
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);
	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<FTransform> TransformRange = OutPointData->GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

	UTEST_EQUAL("Output has the correct transform,", TransformRange[0], FTransform(FRotator::ZeroRotator, FVector(0.0), FVector::One()));
	UTEST_EQUAL("Output has the correct minimum bounds,", BoundsMinRange[0], FVector(-1.0));
	UTEST_EQUAL("Output has the correct maximum bounds,", BoundsMaxRange[0], FVector(1.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGResetPointCenterTest_Zero, FPCGTestBaseClass, "Plugins.PCG.ResetPointCenter.Zero", PCGTestsCommon::TestFlags)

bool FPCGResetPointCenterTest_Zero::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = ResetPointCenterCommonTests::GenerateTestDataAndRun(FVector::ZeroVector, FRotator::ZeroRotator, FVector::One());

	// Point center should be in the bottom left corner
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);
	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<FTransform> TransformRange = OutPointData->GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

	UTEST_EQUAL("Output has the correct transform,", TransformRange[0], FTransform(FRotator::ZeroRotator, FVector(-1.0), FVector::One()));
	UTEST_EQUAL("Output has the correct minimum bounds,", BoundsMinRange[0], FVector(0.0));
	UTEST_EQUAL("Output has the correct maximum bounds,", BoundsMaxRange[0], FVector(2.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGResetPointCenterTest_One, FPCGTestBaseClass, "Plugins.PCG.ResetPointCenter.One", PCGTestsCommon::TestFlags)

bool FPCGResetPointCenterTest_One::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = ResetPointCenterCommonTests::GenerateTestDataAndRun(FVector::One(), FRotator::ZeroRotator, FVector::One());

	// Point center should be in the top right corner
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);
	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<FTransform> TransformRange = OutPointData->GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

	UTEST_EQUAL("Output has the correct transform,", TransformRange[0], FTransform(FRotator::ZeroRotator, FVector(1.0), FVector::One()));
	UTEST_EQUAL("Output has the correct minimum bounds,", BoundsMinRange[0], FVector(-2.0));
	UTEST_EQUAL("Output has the correct maximum bounds,", BoundsMaxRange[0], FVector(0.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGResetPointCenterTest_NegativeOutside, FPCGTestBaseClass, "Plugins.PCG.ResetPointCenter.NegativeOutside", PCGTestsCommon::TestFlags)

bool FPCGResetPointCenterTest_NegativeOutside::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = ResetPointCenterCommonTests::GenerateTestDataAndRun(FVector(-3.0), FRotator::ZeroRotator, FVector::One());

	// Point center should be outside its bounding range, further bottom left
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);
	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<FTransform> TransformRange = OutPointData->GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

	UTEST_EQUAL("Output has the correct transform,", TransformRange[0], FTransform(FRotator::ZeroRotator, FVector(-7.0), FVector::One()));
	UTEST_EQUAL("Output has the correct minimum bounds,", BoundsMinRange[0], FVector(6.0));
	UTEST_EQUAL("Output has the correct maximum bounds,", BoundsMaxRange[0], FVector(8.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGResetPointCenterTest_PositiveOutside, FPCGTestBaseClass, "Plugins.PCG.ResetPointCenter.PositiveOutside", PCGTestsCommon::TestFlags)

bool FPCGResetPointCenterTest_PositiveOutside::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = ResetPointCenterCommonTests::GenerateTestDataAndRun(FVector(4.0), FRotator::ZeroRotator, FVector::One());

	// Point center should be outside its bounding range, further top right
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);
	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<FTransform> TransformRange = OutPointData->GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

	UTEST_EQUAL("Output has the correct transform,", TransformRange[0], FTransform(FRotator::ZeroRotator, FVector(7.0), FVector::One()));
	UTEST_EQUAL("Output has the correct minimum bounds,", BoundsMinRange[0], FVector(-8.0));
	UTEST_EQUAL("Output has the correct maximum bounds,", BoundsMaxRange[0], FVector(-6.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGResetPointCenterTest_NonUniform, FPCGTestBaseClass, "Plugins.PCG.ResetPointCenter.NonUniform", PCGTestsCommon::TestFlags)

bool FPCGResetPointCenterTest_NonUniform::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = ResetPointCenterCommonTests::GenerateTestDataAndRun(FVector(-1.0, 0.5, 2.0), FRotator::ZeroRotator, FVector::One());

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);
	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<FTransform> TransformRange = OutPointData->GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

	UTEST_EQUAL("Output has the correct transform,", TransformRange[0], FTransform(FRotator::ZeroRotator, FVector(-3.0, 0.0, 3.0), FVector::One()));
	UTEST_EQUAL("Output has the correct minimum bounds,", BoundsMinRange[0], FVector(2.0, -1.0, -4.0));
	UTEST_EQUAL("Output has the correct maximum bounds,", BoundsMaxRange[0], FVector(4.0, 1.0, -2.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGResetPointCenterTest_CenterRotation, FPCGTestBaseClass, "Plugins.PCG.ResetPointCenter.CenterRotation", PCGTestsCommon::TestFlags)

bool FPCGResetPointCenterTest_CenterRotation::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = ResetPointCenterCommonTests::GenerateTestDataAndRun(FVector(0.5), FRotator(20.0), FVector::One());

	// Point center should be in the center of the bounding box
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);
	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<FTransform> TransformRange = OutPointData->GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

	UTEST_EQUAL("Output has the correct transform,", TransformRange[0], FTransform(FRotator(20.0), FVector(0.0), FVector::One()));
	UTEST_EQUAL("Output has the correct minimum bounds,", BoundsMinRange[0], FVector(-1.0));
	UTEST_EQUAL("Output has the correct maximum bounds,", BoundsMaxRange[0], FVector(1.0));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGResetPointCenterTest_CenterScale, FPCGTestBaseClass, "Plugins.PCG.ResetPointCenter.CenterScale", PCGTestsCommon::TestFlags)

bool FPCGResetPointCenterTest_CenterScale::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = ResetPointCenterCommonTests::GenerateTestDataAndRun(FVector(0.5), FRotator::ZeroRotator, FVector(3.0));

	// Point center should be in the center of the bounding box
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output point data", OutPointData);
	UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 1);

	const TConstPCGValueRange<FTransform> TransformRange = OutPointData->GetConstTransformValueRange();
	const TConstPCGValueRange<FVector> BoundsMinRange = OutPointData->GetConstBoundsMinValueRange();
	const TConstPCGValueRange<FVector> BoundsMaxRange = OutPointData->GetConstBoundsMaxValueRange();

	UTEST_EQUAL("Output has the correct transform,", TransformRange[0], FTransform(FRotator::ZeroRotator, FVector(0.0), FVector(3.0)));
	UTEST_EQUAL("Output has the correct minimum bounds,", BoundsMinRange[0], FVector(-1.0));
	UTEST_EQUAL("Output has the correct maximum bounds,", BoundsMaxRange[0], FVector(1.0));

	return true;
}