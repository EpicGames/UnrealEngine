// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Data/PCGTextureData.h"
#include "Engine/Texture2D.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "UObject/Package.h"

#include "Elements/PCGSampleTexture.h"

#if WITH_EDITOR
class PCGSampleTextureTestBase : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;

	struct FTestParameters
	{
		FName UVAttribute = TEXT("UV");
		EPCGTextureMappingMethod TextureMappingMethod = EPCGTextureMappingMethod::Planar;
		FVector2D UVPosition;
		FVector4 ExpectedColor = FVector4();
		float ExpectedDensity = 0.0f;
	};
protected:
	UPCGTextureData* GenerateTextureData()
	{
		UTexture2D* Texture = NewObject<UTexture2D>(GetTransientPackage(), NAME_None, RF_Transient);
		
		constexpr int32 SizeX = 4;
		constexpr int32 SizeY = 4;
		Texture->Source.Init(
			SizeX,
			SizeY,
			1,
			1,
			TSF_BGRA8
		);

		Texture->SRGB = false;
		Texture->CompressionNone = true;
		Texture->MipGenSettings = TMGS_NoMipmaps;
		Texture->AddressX = TA_Clamp;
		Texture->AddressY = TA_Clamp;

		const uint32 BPP = Texture->Source.GetBytesPerPixel();

		uint8* TexData = Texture->Source.LockMip(0);
		FMemory::Memzero(TexData, SizeX * SizeY * sizeof(uint8) * BPP);

		for (int32 Y = 0; Y < SizeY; ++Y)
		{
			for (int32 X = 0; X < SizeX; ++X)
			{
				// 4 channels
				TexData[(X + Y * SizeX) * BPP] = static_cast<uint8>((X / 8.0f) * 255); // Blue
				TexData[(X + Y * SizeX) * BPP + 1] = static_cast<uint8>((Y / 8.0f) * 255); // Green
				TexData[(X + Y * SizeX) * BPP + 2] = static_cast<uint8>(1.0f * 255); // Red
				TexData[(X + Y * SizeX) * BPP + 3] = static_cast<uint8>(0.5f * 255); // Alpha
			}
		}
		
		Texture->Source.UnlockMip(0);
		Texture->UpdateResource();

		UPCGTextureData* TextureData = NewObject<UPCGTextureData>();
		while (!TextureData->Initialize(Texture, /*InTextureIndex*/0, FTransform())) {}

		// CPU readback is deferred; poll until ColorData is populated before sampling.
		while (!TextureData->RequestCPUReadback()) {}

		if (ensure(TextureData->IsSuccessfullyInitialized()))
		{
			return TextureData;
		}
		else
		{
			return nullptr;
		}
	}
	
	bool GenerateTestDataRunAndValidate(const FTestParameters& Parameters)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPCGSampleTextureSettings>(TestData);
		UPCGSampleTextureSettings* Settings = CastChecked<UPCGSampleTextureSettings>(TestData.Settings);
		Settings->UVCoordinatesAttribute.SetAttributeName(Parameters.UVAttribute);
		Settings->TextureMappingMethod = Parameters.TextureMappingMethod;
	
		FPCGTaggedData& PointInputs = TestData.InputData.TaggedData.Emplace_GetRef();
		UPCGBasePointData* InData = PCGTestsCommon::CreateBasePointData();
		PointInputs.Data = InData;
		PointInputs.Pin = PCGSampleTextureConstants::InputPointLabel;

		FPCGMetadataAttribute<FVector2D>* InputUV = InData->Metadata->CreateAttribute<FVector2D>(Parameters.UVAttribute, FVector2D::ZeroVector, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/true);
		InData->SetNumPoints(16);

		TPCGValueRange<FTransform> TransformRange = InData->GetTransformValueRange();
		TPCGValueRange<int64> MetadataEntryRange = InData->GetMetadataEntryValueRange();

		for (int i = 0; i < 4; ++i)
		{
			for (int j = 0; j < 4; ++j)
			{
				int CurrentPoint = i * 4 + j;

				// Place each point at the texel center for easy UV access
				TransformRange[CurrentPoint].SetLocation(FVector((j + 0.5) * 0.25, (i + 0.5) * 0.25, 0.0));

				// Set the UV metadata for each point
				InData->Metadata->InitializeOnSet(MetadataEntryRange[CurrentPoint]);
				InputUV->SetValue(MetadataEntryRange[CurrentPoint], FVector2D(TransformRange[CurrentPoint].GetLocation()));
			}
		}

		FPCGTaggedData& TextureInputs = TestData.InputData.TaggedData.Emplace_GetRef();
		TextureInputs.Data = GenerateTextureData();
		TextureInputs.Pin = PCGSampleTextureConstants::InputTextureLabel;

		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

		UTEST_EQUAL("Output count", Outputs.Num(), 1);

		const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Outputs[0].Data);

		UTEST_NOT_NULL("Output point data", OutPointData);

		UTEST_EQUAL("Output point count", OutPointData->GetNumPoints(), 16);

		const FPCGMetadataAttribute<FVector2D>* OutputUV = OutPointData->Metadata->GetConstTypedAttribute<FVector2D>(Parameters.UVAttribute);
		UTEST_NOT_NULL("UV Attribute exists", OutputUV);

		PCGMetadataValueKey ValueKey = OutputUV->FindValue(Parameters.UVPosition);

		UTEST_TRUE("Output Value Key is not default value key.", ValueKey != PCGDefaultValueKey)

		const FConstPCGPointValueRanges OutRanges(OutPointData);
		for (int32 PointIndex = 0; PointIndex < OutPointData->GetNumPoints(); ++PointIndex)
		{
			if (OutputUV->GetValueKey(OutRanges.MetadataEntryRange[PointIndex]) == ValueKey)
			{
				UTEST_EQUAL("Point Color", OutRanges.ColorRange[PointIndex], Parameters.ExpectedColor);
				UTEST_EQUAL("Point Density", OutRanges.DensityRange[PointIndex], Parameters.ExpectedDensity);
			}
		}

		return true;
	}
};

// EAutomationTestFlags::NonNullRHI is important here since texture need some kind of GPU support, and without any rendering enabled, it will crash otherwise.
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSampleTextureTest_LocalCorner, PCGSampleTextureTestBase, "Plugins.PCG.SampleTexture.LocalCorner", PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGSampleTextureTest_LocalCorner::RunTest(const FString& Parameters)
{
	FTestParameters LocalCornerParameters{};
	LocalCornerParameters.TextureMappingMethod = EPCGTextureMappingMethod::UVCoordinates;
	LocalCornerParameters.UVPosition = FVector2D(0.0, 0.0);
	LocalCornerParameters.ExpectedColor = FVector4(0.0, 0.0, 255.0, 128.0);
	LocalCornerParameters.ExpectedDensity = 0.0f;

	return GenerateTestDataRunAndValidate(LocalCornerParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSampleTextureTest_LocalTexelCenter, PCGSampleTextureTestBase, "Plugins.PCG.SampleTexture.LocalTexelCenter", PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGSampleTextureTest_LocalTexelCenter::RunTest(const FString& Parameters)
{
	FTestParameters LocalTexelCenterParameters{};
	LocalTexelCenterParameters.TextureMappingMethod = EPCGTextureMappingMethod::UVCoordinates;
	LocalTexelCenterParameters.UVPosition = FVector2D(0.125, 0.125);
	LocalTexelCenterParameters.ExpectedColor = FVector4(0.0, 0.0, 255.0, 128.0);
	LocalTexelCenterParameters.ExpectedDensity = 0.0f;
	
	return GenerateTestDataRunAndValidate(LocalTexelCenterParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSampleTextureTest_LocalCenter, PCGSampleTextureTestBase, "Plugins.PCG.SampleTexture.LocalCenter", PCGTestsCommon::TestFlags | EAutomationTestFlags::NonNullRHI)

bool FPCGSampleTextureTest_LocalCenter::RunTest(const FString& Parameters)
{
	FTestParameters LocalCenterParameters{};
	LocalCenterParameters.TextureMappingMethod = EPCGTextureMappingMethod::UVCoordinates;
	LocalCenterParameters.UVPosition = FVector2D(0.5, 0.5);
	LocalCenterParameters.ExpectedColor = FVector4((3.0/16.0 * 255.0), (3.0/16.0 * 255.0), 255.0, 128.0); // Average color across points 5, 6, 9, and 10
	LocalCenterParameters.ExpectedDensity = 0.0f;

	return GenerateTestDataRunAndValidate(LocalCenterParameters);
}
#endif