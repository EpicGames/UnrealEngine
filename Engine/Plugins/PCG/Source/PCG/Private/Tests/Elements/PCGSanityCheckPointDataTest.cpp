// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "PCGSubsystem.h"
#include "PCGGraph.h"
#include "Utils/PCGNodeVisualLogs.h"

#include "Elements/PCGSanityCheckPointData.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSanityCheckPointData_MinPointCount, FPCGTestBaseClass, "Plugins.PCG.SanityCheckPointData.MinPointCount", PCGTestsCommon::TestFlags)

bool FPCGSanityCheckPointData_MinPointCount::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGSanityCheckPointDataSettings>(TestData);
	UPCGSanityCheckPointDataSettings* Settings = CastChecked<UPCGSanityCheckPointDataSettings>(TestData.Settings);

	Settings->MinPointCount = 10;
	Settings->MaxPointCount = 100;

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	UPCGBasePointData *SourceData = PCGTestsCommon::CreateRandomBasePointData(2, 42);

	FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
	SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
	SourcePin.Data = SourceData;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	AddExpectedError(TEXT("Expected at least 10 Points, found 2"));

	while (!TestElement->Execute(Context.Get())) {}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSanityCheckPointData_MaxPointCount, FPCGTestBaseClass, "Plugins.PCG.SanityCheckPointData.MaxPointCount", PCGTestsCommon::TestFlags)

bool FPCGSanityCheckPointData_MaxPointCount::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGSanityCheckPointDataSettings>(TestData);
	UPCGSanityCheckPointDataSettings* Settings = CastChecked<UPCGSanityCheckPointDataSettings>(TestData.Settings);

	Settings->MinPointCount = 10;
	Settings->MaxPointCount = 100;

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	UPCGBasePointData *SourceData = PCGTestsCommon::CreateRandomBasePointData(200, 42);

	FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
	SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
	SourcePin.Data = SourceData;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext(TestData.TestPCGComponent->GetGraph()->AddNode(Settings));

	AddExpectedError(TEXT("Expected less than 100 Points, found 200"));

	while (!TestElement->Execute(Context.Get())) {}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSanityCheckPointData_Success, FPCGTestBaseClass, "Plugins.PCG.SanityCheckPointData.Success", PCGTestsCommon::TestFlags)

bool FPCGSanityCheckPointData_Success::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGSanityCheckPointDataSettings>(TestData);
	UPCGSanityCheckPointDataSettings* Settings = CastChecked<UPCGSanityCheckPointDataSettings>(TestData.Settings);

	Settings->MinPointCount = 10;
	Settings->MaxPointCount = 100;

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	UPCGBasePointData *SourceData = PCGTestsCommon::CreateRandomBasePointData(50, 42);

	FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
	SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
	SourcePin.Data = SourceData;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext(TestData.TestPCGComponent->GetGraph()->AddNode(Settings));

	while (!TestElement->Execute(Context.Get())) {}

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllSpatialInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	UTEST_TRUE("Data was forwarded", Outputs[0].Data == SourceData);

	return true;
}
