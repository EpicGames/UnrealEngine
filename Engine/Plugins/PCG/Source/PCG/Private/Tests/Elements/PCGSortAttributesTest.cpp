// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGBasePointData.h"
#include "Elements/PCGSortAttributes.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

namespace SortCommonTestData
{
	static const FName AttributeName = TEXT("Attr");

	UPCGParamData* GenerateParamData(int32 EntriesCount, int32 Seed, bool bRandom)
	{
		UPCGParamData* OutData = NewObject<UPCGParamData>();
		check(OutData && OutData->Metadata);
		FPCGMetadataAttribute<int32>* Attribute = OutData->Metadata->CreateAttribute<int32>(AttributeName, 0, false, false);
		check(Attribute);
		FRandomStream RandomStream(Seed);

		for (int32 i = 0; i < EntriesCount; ++i)
		{
			Attribute->SetValueFromValueKey(OutData->Metadata->AddEntry(), bRandom ? RandomStream.RandRange(1, 9999) : 1);
		}

		return OutData;
	}

	TUniquePtr<FPCGContext> RunSortElementOnData(EPCGSortMethod Method, const UPCGData* InData, const FPCGAttributePropertyInputSelector& InputSource)
	{
		PCGTestsCommon::FTestData TestData;
		UPCGSortAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGSortAttributesSettings>(TestData);
		check(Settings);
		// Default is one attribute entry.
		check(Settings->SortAttributes.Num() == 1);
		FPCGSortAttributeEntry& Entry = Settings->SortAttributes[0];
		Entry.InputSource = InputSource;
		Entry.SortMethod = Method;
		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
		SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
		SourcePin.Data = InData;

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}
	
		return Context;
	}
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSortPointsTest_Ascending, FPCGTestBaseClass, "Plugins.PCG.SortAttributes.Points.Ascending", PCGTestsCommon::TestFlags)

bool FPCGSortPointsTest_Ascending::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = SortCommonTestData::RunSortElementOnData(EPCGSortMethod::Ascending,
		PCGTestsCommon::CreateRandomPointData(100, 42, /*bRandomDensity=*/ true), 
		FPCGAttributePropertySelector::CreatePointPropertySelector<FPCGAttributePropertyInputSelector>(EPCGPointProperties::Density));

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	// compare point data
	const FPCGTaggedData& Output = Outputs[0];
	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Output.Data);
	UTEST_NOT_NULL("Output is a point data", OutPointData);
	check(OutPointData);
	
	const TConstPCGValueRange<float> DensityRange = OutPointData->GetConstDensityValueRange();

	for (int32 i = 0; i < DensityRange.Num() - 1; ++i)
	{
		UTEST_TRUE(FString::Format(TEXT("{0} is less than/equal to {1}"), { DensityRange[i], DensityRange[i + 1] }), DensityRange[i] <= DensityRange[i + 1]);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSortAttributesTest_Ascending, FPCGTestBaseClass, "Plugins.PCG.SortAttributes.AttributeSet.Ascending", PCGTestsCommon::TestFlags)

bool FPCGSortAttributesTest_Ascending::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = SortCommonTestData::RunSortElementOnData(EPCGSortMethod::Ascending,
		SortCommonTestData::GenerateParamData(100, 42, /*bRandom=*/true),
		FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(SortCommonTestData::AttributeName));

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const FPCGTaggedData& Output = Outputs[0];
	const UPCGParamData* OutParamData = Cast<UPCGParamData>(Output.Data);
	UTEST_NOT_NULL("Output is an attribute set data", OutParamData);
	check(OutParamData && OutParamData->Metadata);

	const FPCGMetadataAttribute<int32>* Attribute = OutParamData->Metadata->GetConstTypedAttribute<int32>(SortCommonTestData::AttributeName);
	UTEST_NOT_NULL("Output has the sorted attribute", Attribute);
	check(Attribute);

	for (int i = 0; i < OutParamData->Metadata->GetLocalItemCount() - 1; ++i)
	{
		const int32 Value = Attribute->GetValue(i);
		const int32 NextValue = Attribute->GetValue(i + 1);
		UTEST_TRUE(FString::Format(TEXT("{0} is less than/equal to {1}"), { Value, NextValue }), Value <= NextValue);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSortPointsTest_Descending, FPCGTestBaseClass, "Plugins.PCG.SortAttributes.Points.Descending", PCGTestsCommon::TestFlags)

bool FPCGSortPointsTest_Descending::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = SortCommonTestData::RunSortElementOnData(EPCGSortMethod::Descending,
		PCGTestsCommon::CreateRandomPointData(100, 42, /*bRandomDensity=*/ true),
		FPCGAttributePropertySelector::CreatePointPropertySelector<FPCGAttributePropertyInputSelector>(EPCGPointProperties::Density));

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	// compare point data
	const FPCGTaggedData& Output = Outputs[0];
	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Output.Data);

	UTEST_NOT_NULL("Output is a point data", OutPointData);
	check(OutPointData);
	
	const TConstPCGValueRange<float> DensityRange = OutPointData->GetConstDensityValueRange();

	for (int i = 0; i < DensityRange.Num() - 1; ++i)
	{
		UTEST_TRUE(FString::Format(TEXT("{0} is greater than/equal to {1}"), { DensityRange[i], DensityRange[i + 1] }), DensityRange[i] >= DensityRange[i + 1]);
	}
	
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSortAttributesTest_Descending, FPCGTestBaseClass, "Plugins.PCG.SortAttributes.AttributeSet.Descending", PCGTestsCommon::TestFlags)

bool FPCGSortAttributesTest_Descending::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = SortCommonTestData::RunSortElementOnData(EPCGSortMethod::Descending,
		SortCommonTestData::GenerateParamData(100, 42, /*bRandom=*/true),
		FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(SortCommonTestData::AttributeName));

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const FPCGTaggedData& Output = Outputs[0];
	const UPCGParamData* OutParamData = Cast<UPCGParamData>(Output.Data);
	UTEST_NOT_NULL("Output is an attribute set data", OutParamData);
	check(OutParamData && OutParamData->Metadata);

	const FPCGMetadataAttribute<int32>* Attribute = OutParamData->Metadata->GetConstTypedAttribute<int32>(SortCommonTestData::AttributeName);
	UTEST_NOT_NULL("Output has the sorted attribute", Attribute);
	check(Attribute);

	for (int i = 0; i < OutParamData->Metadata->GetLocalItemCount() - 1; ++i)
	{
		const int32 Value = Attribute->GetValue(i);
		const int32 NextValue = Attribute->GetValue(i + 1);
		UTEST_TRUE(FString::Format(TEXT("{0} is greater than/equal to {1}"), { Value, NextValue }), Value >= NextValue);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSortPointsTest_SameValues, FPCGTestBaseClass, "Plugins.PCG.SortAttributes.SameValues", PCGTestsCommon::TestFlags)

bool FPCGSortPointsTest_SameValues::RunTest(const FString& Parameters)
{
	TUniquePtr<FPCGContext> Context = SortCommonTestData::RunSortElementOnData(EPCGSortMethod::Ascending,
		PCGTestsCommon::CreateRandomPointData(100, 42, /*bRandomDensity=*/ false),
		FPCGAttributePropertySelector::CreatePointPropertySelector<FPCGAttributePropertyInputSelector>(EPCGPointProperties::Density));

	const TArray<FPCGTaggedData>& Inputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	// compare point data
	const FPCGTaggedData& Input = Inputs[0];
	const FPCGTaggedData& Output = Outputs[0];

	const UPCGBasePointData* InPointData = Cast<UPCGBasePointData>(Input.Data);
	const UPCGBasePointData* OutPointData = Cast<UPCGBasePointData>(Output.Data);

	UTEST_NOT_NULL("Output is a point data", OutPointData);
	check(InPointData && OutPointData);
		
	UTEST_EQUAL("Arrays have the same number of points:", InPointData->GetNumPoints(), OutPointData->GetNumPoints());

	const TConstPCGValueRange<int32> InSeedRange = InPointData->GetConstSeedValueRange();
	const TConstPCGValueRange<float> InDensityRange = InPointData->GetConstDensityValueRange();
	const TConstPCGValueRange<FTransform> InTransformRange = InPointData->GetConstTransformValueRange();

	const TConstPCGValueRange<int32> OutSeedRange = OutPointData->GetConstSeedValueRange();
	const TConstPCGValueRange<float> OutDensityRange = OutPointData->GetConstDensityValueRange();
	const TConstPCGValueRange<FTransform> OutTransformRange = OutPointData->GetConstTransformValueRange();

	for (int i = 0; i < InPointData->GetNumPoints(); ++i)
	{
		//if they're in the same spots after sorting, they should have exactly the same properties across the board
		UTEST_EQUAL(FString::Format(TEXT("UnsortedArray[{0}].Seed is equal to SortedArray[{0}].Seed"), {i}), OutSeedRange[i], InSeedRange[i]);
		UTEST_EQUAL(FString::Format(TEXT("UnsortedArray[{0}].Density is equal to SortedArray[{0}].Density"), {i}), OutDensityRange[i], InDensityRange[i]);
		UTEST_EQUAL(FString::Format(TEXT("UnsortedArray[{0}].Transform is equal to SortedArray[{0}].Transform"), {i}), OutTransformRange[i], InTransformRange[i]);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSortAttributesTest_EmptyAttributes, FPCGTestBaseClass, "Plugins.PCG.SortAttributes.EmptyAttributes", PCGTestsCommon::TestFlags)

bool FPCGSortAttributesTest_EmptyAttributes::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	UPCGSortAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGSortAttributesSettings>(TestData);
	check(Settings);
	Settings->SortAttributes.Empty();
	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	const UPCGBasePointData* InputPointData = PCGTestsCommon::CreateRandomPointData(100, 42, /*bRandomDensity=*/true);

	FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
	SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
	SourcePin.Data = InputPointData;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	const FPCGTaggedData* SourceEntry = Outputs.FindByPredicate([InputPointData](const FPCGTaggedData& Output)
	{
		return Output.Data == static_cast<const UPCGData*>(InputPointData);
	});

	UTEST_NOT_NULL("Empty attributes forwards the input data pointer unchanged", SourceEntry);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSortAttributesTest_SameValues, FPCGTestBaseClass, "Plugins.PCG.SortAttributes.AttributeSet.SameValues", PCGTestsCommon::TestFlags)

bool FPCGSortAttributesTest_SameValues::RunTest(const FString& Parameters)
{
	const UPCGParamData* InputParamData = SortCommonTestData::GenerateParamData(100, 42, /*bRandom=*/false);
	check(InputParamData);

	TUniquePtr<FPCGContext> Context = SortCommonTestData::RunSortElementOnData(EPCGSortMethod::Ascending,
		InputParamData,
		FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(SortCommonTestData::AttributeName));

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const FPCGTaggedData& Output = Outputs[0];
	const UPCGParamData* OutParamData = Cast<UPCGParamData>(Output.Data);
	UTEST_NOT_NULL("Output is an attribute set data", OutParamData);
	check(OutParamData && OutParamData->Metadata);


	const FPCGMetadataAttribute<int32>* InAttribute = InputParamData->Metadata->GetConstTypedAttribute<int32>(SortCommonTestData::AttributeName);
	check(InAttribute);

	const FPCGMetadataAttribute<int32>* OutAttribute = OutParamData->Metadata->GetConstTypedAttribute<int32>(SortCommonTestData::AttributeName);
	UTEST_NOT_NULL("Output has the sorted attribute", OutAttribute);
	check(OutAttribute);

	for (int i = 0; i < OutParamData->Metadata->GetLocalItemCount() - 1; ++i)
	{
		const PCGMetadataValueKey InValueKey = InAttribute->GetValueKey(i);
		const PCGMetadataValueKey OutValueKey = OutAttribute->GetValueKey(i);
		UTEST_TRUE(FString::Format(TEXT("InAttribute and OutAttribute have the same value key for the same entry key ({0} vs {1})"), { InValueKey, OutValueKey }), InValueKey == OutValueKey);
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSortAttributesTest_MultiAttribute, FPCGTestBaseClass, "Plugins.PCG.SortAttributes.AttributeSet.MultiAttribute", PCGTestsCommon::TestFlags)

bool FPCGSortAttributesTest_MultiAttribute::RunTest(const FString& Parameters)
{
	static const FName GroupName = TEXT("Group");
	static const FName ValueName = TEXT("Value");

	UPCGParamData* InputData = NewObject<UPCGParamData>();
	check(InputData && InputData->Metadata);

	FPCGMetadataAttribute<int32>* GroupAttr = InputData->Metadata->CreateAttribute<int32>(GroupName, 0, false, false);
	FPCGMetadataAttribute<int32>* ValueAttr = InputData->Metadata->CreateAttribute<int32>(ValueName, 0, false, false);
	check(GroupAttr && ValueAttr);

	// Unordered (Group, Value) pairs; only lex sort with Group asc + Value desc produces the expected order.
	const TArray<TPair<int32, int32>> InputPairs = {
		{2, 20}, {1, 10}, {3, 5}, {1, 30}, {2, 50}, {3, 40}
	};

	for (const TPair<int32, int32>& Pair : InputPairs)
	{
		const PCGMetadataEntryKey Entry = InputData->Metadata->AddEntry();
		GroupAttr->SetValue(Entry, Pair.Key);
		ValueAttr->SetValue(Entry, Pair.Value);
	}

	PCGTestsCommon::FTestData TestData;
	UPCGSortAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGSortAttributesSettings>(TestData);
	check(Settings);
	Settings->SortAttributes.Empty();

	FPCGSortAttributeEntry& GroupEntry = Settings->SortAttributes.AddDefaulted_GetRef();
	GroupEntry.InputSource = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(GroupName);
	GroupEntry.SortMethod = EPCGSortMethod::Ascending;

	FPCGSortAttributeEntry& ValueEntry = Settings->SortAttributes.AddDefaulted_GetRef();
	ValueEntry.InputSource = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(ValueName);
	ValueEntry.SortMethod = EPCGSortMethod::Descending;

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
	SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
	SourcePin.Data = InputData;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	while (!TestElement->Execute(Context.Get())) {}

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGParamData* OutParamData = Cast<UPCGParamData>(Outputs[0].Data);
	UTEST_NOT_NULL("Output is an attribute set data", OutParamData);
	check(OutParamData && OutParamData->Metadata);

	const FPCGMetadataAttribute<int32>* OutGroup = OutParamData->Metadata->GetConstTypedAttribute<int32>(GroupName);
	const FPCGMetadataAttribute<int32>* OutValue = OutParamData->Metadata->GetConstTypedAttribute<int32>(ValueName);
	UTEST_NOT_NULL("Output has Group attribute", OutGroup);
	UTEST_NOT_NULL("Output has Value attribute", OutValue);
	check(OutGroup && OutValue);

	const int64 Count = OutParamData->Metadata->GetLocalItemCount();
	UTEST_EQUAL("Output entry count matches input", static_cast<int32>(Count), InputPairs.Num());

	for (PCGMetadataEntryKey EntryIndex = 0; EntryIndex + 1 < Count; ++EntryIndex)
	{
		const int32 GroupA = OutGroup->GetValueFromItemKey(EntryIndex);
		const int32 GroupB = OutGroup->GetValueFromItemKey(EntryIndex + 1);
		UTEST_TRUE(FString::Format(TEXT("Group[{0}]={1} is less-than-or-equal to Group[{2}]={3}"), { EntryIndex, GroupA, EntryIndex + 1, GroupB }), GroupA <= GroupB);

		if (GroupA == GroupB)
		{
			const int32 ValueA = OutValue->GetValueFromItemKey(EntryIndex);
			const int32 ValueB = OutValue->GetValueFromItemKey(EntryIndex + 1);
			UTEST_TRUE(FString::Format(TEXT("Within Group {0}, Value[{1}]={2} is greater-than-or-equal to Value[{3}]={4}"), { GroupA, EntryIndex, ValueA, EntryIndex + 1, ValueB }), ValueA >= ValueB);
		}
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSortAttributesTest_StableSort, FPCGTestBaseClass, "Plugins.PCG.SortAttributes.AttributeSet.StableSort", PCGTestsCommon::TestFlags)

bool FPCGSortAttributesTest_StableSort::RunTest(const FString& Parameters)
{
	static const FName SortKeyName = TEXT("SortKey");
	static const FName OrderName = TEXT("Order");

	UPCGParamData* InputData = NewObject<UPCGParamData>();
	check(InputData && InputData->Metadata);

	FPCGMetadataAttribute<int32>* SortKeyAttr = InputData->Metadata->CreateAttribute<int32>(SortKeyName, 0, false, false);
	FPCGMetadataAttribute<int32>* OrderAttr = InputData->Metadata->CreateAttribute<int32>(OrderName, 0, false, false);
	check(SortKeyAttr && OrderAttr);

	// All rows share the same SortKey value so the comparator returns 0 for every pair; a stable sort must preserve the original Order sequence.
	constexpr int32 EntryCount = 16;
	for (int32 i = 0; i < EntryCount; ++i)
	{
		const PCGMetadataEntryKey Entry = InputData->Metadata->AddEntry();
		SortKeyAttr->SetValue(Entry, 1);
		OrderAttr->SetValue(Entry, i);
	}

	PCGTestsCommon::FTestData TestData;
	UPCGSortAttributesSettings* Settings = PCGTestsCommon::GenerateSettings<UPCGSortAttributesSettings>(TestData);
	check(Settings);
	Settings->SortAttributes.Empty();
	Settings->bUseStableSort = true;

	FPCGSortAttributeEntry& Entry = Settings->SortAttributes.AddDefaulted_GetRef();
	Entry.InputSource = FPCGAttributePropertySelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(SortKeyName);
	Entry.SortMethod = EPCGSortMethod::Ascending;

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
	SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
	SourcePin.Data = InputData;

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();
	while (!TestElement->Execute(Context.Get())) {}

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGParamData* OutParamData = Cast<UPCGParamData>(Outputs[0].Data);
	UTEST_NOT_NULL("Output is an attribute set data", OutParamData);
	check(OutParamData && OutParamData->Metadata);

	const FPCGMetadataAttribute<int32>* OutOrder = OutParamData->Metadata->GetConstTypedAttribute<int32>(OrderName);
	UTEST_NOT_NULL("Output has Order attribute", OutOrder);
	check(OutOrder);

	for (int32 i = 0; i < EntryCount; ++i)
	{
		UTEST_EQUAL(FString::Format(TEXT("Stable sort preserves Order[{0}]"), { i }), OutOrder->GetValueFromItemKey(i), i);
	}

	return true;
}