// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGSpatialNoise.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPCGSpatialNoise_CalcLocalCoordinates2D, "Plugins.PCG.Noise.CalcLocalCoordinates2D", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter)

bool FPCGSpatialNoise_CalcLocalCoordinates2D::RunTest(const FString& Parameters)
{
	const FBox TestBox(-FVector::One(), FVector::One());
	const FVector2D Scale = FVector2D::One();

	UTEST_EQUAL("Disabled Edge value", CalcEdgeBlendAmount2D(PCGSpatialNoise::CalcLocalCoordinates2D(TestBox, FTransform::Identity, Scale, FVector::One()), 0.0), 1.0);
	UTEST_EQUAL("Disabled Edge value", CalcEdgeBlendAmount2D(PCGSpatialNoise::CalcLocalCoordinates2D(TestBox, FTransform::Identity, Scale, FVector::Zero()), 0.0), 1.0);

	PCGSpatialNoise::FLocalCoordinates2D Test0 = PCGSpatialNoise::CalcLocalCoordinates2D(TestBox, FTransform::Identity, Scale, FVector::One());
	const double EdgeBlendAmount0 = CalcEdgeBlendAmount2D(Test0, 1.0);
	UTEST_EQUAL_TOLERANCE("Enabled Edge Value", EdgeBlendAmount0, 1.0, 0.001);
	UTEST_EQUAL_TOLERANCE("Edge Sample X0", Test0.X0, 2.0, 0.001);
	UTEST_EQUAL_TOLERANCE("Edge Sample Y0", Test0.Y0, 2.0, 0.001);
	UTEST_EQUAL_TOLERANCE("Edge Sample X1", Test0.X1, 0.0, 0.001);
	UTEST_EQUAL_TOLERANCE("Edge Sample Y1", Test0.Y1, 0.0, 0.001);

	UTEST_EQUAL_TOLERANCE("Enabled Edge Value Center", CalcEdgeBlendAmount2D(PCGSpatialNoise::CalcLocalCoordinates2D(TestBox, FTransform::Identity, Scale, FVector::Zero()), 1.0), 0.0, 0.001);

	PCGSpatialNoise::FLocalCoordinates2D Test1 = PCGSpatialNoise::CalcLocalCoordinates2D(TestBox, FTransform::Identity, Scale, FVector(0.9));
	const double EdgeBlendAmount1 = CalcEdgeBlendAmount2D(Test1, 0.5);
	UTEST_EQUAL_TOLERANCE("Blended Edge Value", EdgeBlendAmount1, 0.8, 0.001);
	UTEST_EQUAL_TOLERANCE("Blended Edge Sample X", Test1.X0, 1.9, 0.001);
	UTEST_EQUAL_TOLERANCE("Blended Edge Sample Y", Test1.Y0, 1.9, 0.001);
	UTEST_EQUAL_TOLERANCE("Blended Edge Sample X", Test1.X1, -0.1, 0.001);
	UTEST_EQUAL_TOLERANCE("Blended Edge Sample Y", Test1.Y1, -0.1, 0.001);

	return true;
}

// just make sure stuff doesn't crash

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSpatialNoise_Perlin2D, FPCGTestBaseClass, "Plugins.PCG.Noise.Perlin2D", PCGTestsCommon::TestFlags)

bool FPCGSpatialNoise_Perlin2D::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGSpatialNoiseSettings>(TestData);
	UPCGSpatialNoiseSettings* Settings = CastChecked<UPCGSpatialNoiseSettings>(TestData.Settings);

	Settings->Mode = PCGSpatialNoiseMode::Perlin2D;
	Settings->ValueTarget.SetAttributeName(TEXT("Noise"));

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	{
		FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
		SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
		SourcePin.Data = PCGTestsCommon::CreateRandomPointData(100, 42);
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}	

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutputData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);
	UTEST_EQUAL("Output point data count", OutputData->GetNumPoints(), 100);

	const FPCGMetadataAttribute<double>* NoiseAttribute = OutputData->Metadata->GetConstTypedAttribute<double>(Settings->ValueTarget.GetAttributeName());
	UTEST_NOT_NULL("Noise Attribute Created", NoiseAttribute);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSpatialNoise_Caustic2D, FPCGTestBaseClass, "Plugins.PCG.Noise.Caustic2D", PCGTestsCommon::TestFlags)

bool FPCGSpatialNoise_Caustic2D::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGSpatialNoiseSettings>(TestData);
	UPCGSpatialNoiseSettings* Settings = CastChecked<UPCGSpatialNoiseSettings>(TestData.Settings);

	Settings->Mode = PCGSpatialNoiseMode::Caustic2D;
	Settings->ValueTarget.SetAttributeName(TEXT("Noise"));

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	{
		FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
		SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
		SourcePin.Data = PCGTestsCommon::CreateRandomPointData(100, 42);
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}	

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutputData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);
	UTEST_EQUAL("Output point data count", OutputData->GetNumPoints(), 100);

	const FPCGMetadataAttribute<double>* NoiseAttribute = OutputData->Metadata->GetConstTypedAttribute<double>(Settings->ValueTarget.GetAttributeName());
	UTEST_NOT_NULL("Noise Attribute Created", NoiseAttribute);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSpatialNoise_Voronoi2D, FPCGTestBaseClass, "Plugins.PCG.Noise.Voronoi2D", PCGTestsCommon::TestFlags)

bool FPCGSpatialNoise_Voronoi2D::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGSpatialNoiseSettings>(TestData);
	UPCGSpatialNoiseSettings* Settings = CastChecked<UPCGSpatialNoiseSettings>(TestData.Settings);

	Settings->Mode = PCGSpatialNoiseMode::Voronoi2D;
	Settings->ValueTarget.SetAttributeName(TEXT("Distance"));
	Settings->VoronoiCellIDTarget.SetAttributeName(TEXT("CellID"));

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	{
		FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
		SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
		SourcePin.Data = PCGTestsCommon::CreateRandomPointData(100, 42);
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}	

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutputData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);
	UTEST_EQUAL("Output point data count", OutputData->GetNumPoints(), 100);

	const FPCGMetadataAttribute<double>* NoiseAttribute = OutputData->Metadata->GetConstTypedAttribute<double>(Settings->ValueTarget.GetAttributeName());
	UTEST_NOT_NULL("Distance Attribute Created", NoiseAttribute);

	const FPCGMetadataAttribute<double>* CellIDAttribute = OutputData->Metadata->GetConstTypedAttribute<double>(Settings->VoronoiCellIDTarget.GetAttributeName());
	UTEST_NOT_NULL("Cell ID Attribute Created", CellIDAttribute);


	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSpatialNoise_FractionalBrownian2D, FPCGTestBaseClass, "Plugins.PCG.Noise.FractionalBrownian2D", PCGTestsCommon::TestFlags)

bool FPCGSpatialNoise_FractionalBrownian2D::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGSpatialNoiseSettings>(TestData);
	UPCGSpatialNoiseSettings* Settings = CastChecked<UPCGSpatialNoiseSettings>(TestData.Settings);

	Settings->Mode = PCGSpatialNoiseMode::FractionalBrownian2D;
	Settings->ValueTarget.SetAttributeName(TEXT("Noise"));

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	{
		FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
		SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
		SourcePin.Data = PCGTestsCommon::CreateRandomPointData(100, 42);
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}	

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutputData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);
	UTEST_EQUAL("Output point data count", OutputData->GetNumPoints(), 100);

	const FPCGMetadataAttribute<double>* NoiseAttribute = OutputData->Metadata->GetConstTypedAttribute<double>(Settings->ValueTarget.GetAttributeName());
	UTEST_NOT_NULL("Noise Attribute Created", NoiseAttribute);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSpatialNoise_EdgeMask2D, FPCGTestBaseClass, "Plugins.PCG.Noise.EdgeMask2D", PCGTestsCommon::TestFlags)

bool FPCGSpatialNoise_EdgeMask2D::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGSpatialNoiseSettings>(TestData);
	UPCGSpatialNoiseSettings* Settings = CastChecked<UPCGSpatialNoiseSettings>(TestData.Settings);

	Settings->Mode = PCGSpatialNoiseMode::EdgeMask2D;
	Settings->ValueTarget.SetAttributeName(TEXT("Noise"));

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	{
		FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
		SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
		SourcePin.Data = PCGTestsCommon::CreateRandomPointData(100, 42);
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}	

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutputData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);
	UTEST_EQUAL("Output point data count", OutputData->GetNumPoints(), 100);

	const FPCGMetadataAttribute<double>* NoiseAttribute = OutputData->Metadata->GetConstTypedAttribute<double>(Settings->ValueTarget.GetAttributeName());
	UTEST_NOT_NULL("Noise Attribute Created", NoiseAttribute);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSpatialNoise_TilingPerlin2D, FPCGTestBaseClass, "Plugins.PCG.Noise.TilingPerlin2D", PCGTestsCommon::TestFlags)
bool FPCGSpatialNoise_TilingPerlin2D::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGSpatialNoiseSettings>(TestData);
	UPCGSpatialNoiseSettings* Settings = CastChecked<UPCGSpatialNoiseSettings>(TestData.Settings);

	Settings->Mode = PCGSpatialNoiseMode::Perlin2D;
	Settings->bTiling = true;
	Settings->ValueTarget.SetAttributeName(TEXT("Noise"));

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	{
		FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
		SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
		SourcePin.Data = PCGTestsCommon::CreateRandomPointData(100, 42);
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}	

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutputData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);
	UTEST_EQUAL("Output point data count", OutputData->GetNumPoints(), 100);

	const FPCGMetadataAttribute<double>* NoiseAttribute = OutputData->Metadata->GetConstTypedAttribute<double>(Settings->ValueTarget.GetAttributeName());
	UTEST_NOT_NULL("Noise Attribute Created", NoiseAttribute);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSpatialNoise_TilingVoronoi2D, FPCGTestBaseClass, "Plugins.PCG.Noise.TilingVoronoi2D", PCGTestsCommon::TestFlags)
bool FPCGSpatialNoise_TilingVoronoi2D::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGSpatialNoiseSettings>(TestData);
	UPCGSpatialNoiseSettings* Settings = CastChecked<UPCGSpatialNoiseSettings>(TestData.Settings);

	Settings->Mode = PCGSpatialNoiseMode::Voronoi2D;
	Settings->bTiling = true;
	Settings->ValueTarget.SetAttributeName(TEXT("Distance"));
	Settings->VoronoiCellIDTarget.SetAttributeName(TEXT("CellID"));

	FPCGElementPtr TestElement = TestData.Settings->GetElement();

	{
		FPCGTaggedData& SourcePin = TestData.InputData.TaggedData.Emplace_GetRef();
		SourcePin.Pin = PCGPinConstants::DefaultInputLabel;
		SourcePin.Data = PCGTestsCommon::CreateRandomPointData(100, 42);
	}

	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!TestElement->Execute(Context.Get())) {}	

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllInputs();

	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPCGBasePointData* OutputData = Cast<UPCGBasePointData>(Outputs[0].Data);

	UTEST_NOT_NULL("Output data", OutputData);
	UTEST_EQUAL("Output point data count", OutputData->GetNumPoints(), 100);

	const FPCGMetadataAttribute<double>* NoiseAttribute = OutputData->Metadata->GetConstTypedAttribute<double>(Settings->ValueTarget.GetAttributeName());
	UTEST_NOT_NULL("Distance Attribute Created", NoiseAttribute);

	const FPCGMetadataAttribute<double>* CellIDAttribute = OutputData->Metadata->GetConstTypedAttribute<double>(Settings->VoronoiCellIDTarget.GetAttributeName());
	UTEST_NOT_NULL("Cell ID Attribute Created", CellIDAttribute);

	return true;
}
