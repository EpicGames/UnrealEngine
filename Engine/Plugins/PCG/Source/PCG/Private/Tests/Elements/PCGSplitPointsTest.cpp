// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"

#include "Elements/PCGSplitPoints.h"

class FPCGSplitPointTestBase : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	struct FTestParameters
	{
		// All your test parameters, like the expected value of BoundsMin
		float SplitPosition = 0.0f;
		EPCGSplitAxis SplitAxis = EPCGSplitAxis::Z;

		FVector ExpectedABoundsMin = FVector();
		FVector ExpectedABoundsMax = FVector();
		FVector ExpectedBBoundsMin = FVector();
		FVector ExpectedBBoundsMax = FVector();
	};

	bool GenerateTestDataRunAndValidate(const FTestParameters& Parameters)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPCGSplitPointsSettings>(TestData);
		UPCGSplitPointsSettings* Settings = CastChecked<UPCGSplitPointsSettings>(TestData.Settings);
		Settings->SplitPosition = Parameters.SplitPosition;
		Settings->SplitAxis = Parameters.SplitAxis;

		FPCGTaggedData& Inputs = TestData.InputData.TaggedData.Emplace_GetRef();
		Inputs.Pin = PCGPinConstants::DefaultInputLabel;
		Inputs.Data = PCGTestsCommon::CreateBasePointData();

		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		// test our point data
		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;

		UTEST_EQUAL("Output count", Outputs.Num(), 2);

		// checking the PointA portion
		const UPCGBasePointData* OutPointDataA = Cast<UPCGBasePointData>(Outputs[0].Data);

		UTEST_NOT_NULL("OutputA point data", OutPointDataA);

		UTEST_EQUAL("OutputA point count", OutPointDataA->GetNumPoints(), 1);

		const FConstPCGPointValueRanges RangesA(OutPointDataA);
		UTEST_EQUAL("OutputA Bounds Min", RangesA.BoundsMinRange[0], Parameters.ExpectedABoundsMin);
		UTEST_EQUAL("OutputA Bounds Max", RangesA.BoundsMaxRange[0], Parameters.ExpectedABoundsMax);

		// checking the PointB portion
		const UPCGBasePointData* OutPointDataB = Cast<UPCGBasePointData>(Outputs[1].Data);

		UTEST_NOT_NULL("OutputB point data", OutPointDataB);

		UTEST_EQUAL("OutputB point count", OutPointDataB->GetNumPoints(), 1);

		const FConstPCGPointValueRanges RangesB(OutPointDataB);
		UTEST_EQUAL("OutputB Bounds Min", RangesB.BoundsMinRange[0], Parameters.ExpectedBBoundsMin);
		UTEST_EQUAL("OutputB Bounds Max", RangesB.BoundsMaxRange[0], Parameters.ExpectedBBoundsMax);

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplitPointsTest_DefaultX, FPCGSplitPointTestBase, "Plugins.PCG.SplitPoints.DefaultX", PCGTestsCommon::TestFlags)

bool FPCGSplitPointsTest_DefaultX::RunTest(const FString& Parameters)
{
	FTestParameters DefaultXParameters{};

	DefaultXParameters.SplitPosition = 0.5f;
	DefaultXParameters.SplitAxis = EPCGSplitAxis::X;
	DefaultXParameters.ExpectedABoundsMin = FVector(-1.0, -1.0, -1.0);
	DefaultXParameters.ExpectedABoundsMax = FVector(0.0, 1.0, 1.0);
	DefaultXParameters.ExpectedBBoundsMin = FVector(0.0, -1.0, -1.0);
	DefaultXParameters.ExpectedBBoundsMax = FVector(1.0, 1.0, 1.0);

	return GenerateTestDataRunAndValidate(DefaultXParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplitPointsTest_DefaultY, FPCGSplitPointTestBase, "Plugins.PCG.SplitPoints.DefaultY", PCGTestsCommon::TestFlags)

bool FPCGSplitPointsTest_DefaultY::RunTest(const FString& Parameters)
{
	FTestParameters DefaultYParameters{};

	DefaultYParameters.SplitPosition = 0.5f;
	DefaultYParameters.SplitAxis = EPCGSplitAxis::Y;
	DefaultYParameters.ExpectedABoundsMin = FVector(-1.0, -1.0, -1.0);
	DefaultYParameters.ExpectedABoundsMax = FVector(1.0, 0.0, 1.0);
	DefaultYParameters.ExpectedBBoundsMin = FVector(-1.0, 0.0, -1.0);
	DefaultYParameters.ExpectedBBoundsMax = FVector(1.0, 1.0, 1.0);

	return GenerateTestDataRunAndValidate(DefaultYParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplitPointsTest_DefaultZ, FPCGSplitPointTestBase, "Plugins.PCG.SplitPoints.DefaultZ", PCGTestsCommon::TestFlags)

bool FPCGSplitPointsTest_DefaultZ::RunTest(const FString& Parameters)
{
	FTestParameters DefaultZParameters{};

	DefaultZParameters.SplitPosition = 0.5f;
	DefaultZParameters.SplitAxis = EPCGSplitAxis::Z;
	DefaultZParameters.ExpectedABoundsMin = FVector(-1.0, -1.0, -1.0);
	DefaultZParameters.ExpectedABoundsMax = FVector(1.0, 1.0, 0.0);
	DefaultZParameters.ExpectedBBoundsMin = FVector(-1.0, -1.0, 0.0);
	DefaultZParameters.ExpectedBBoundsMax = FVector(1.0, 1.0, 1.0);

	return GenerateTestDataRunAndValidate(DefaultZParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplitPointsTest_ZeroX, FPCGSplitPointTestBase, "Plugins.PCG.SplitPoints.ZeroX", PCGTestsCommon::TestFlags)

bool FPCGSplitPointsTest_ZeroX::RunTest(const FString& Parameters)
{
	FTestParameters ZeroXParameters{};

	ZeroXParameters.SplitPosition = 0.0f;
	ZeroXParameters.SplitAxis = EPCGSplitAxis::X;
	ZeroXParameters.ExpectedABoundsMin = FVector(-1.0, -1.0, -1.0);
	ZeroXParameters.ExpectedABoundsMax = FVector(-1.0, 1.0, 1.0);
	ZeroXParameters.ExpectedBBoundsMin = FVector(-1.0, -1.0, -1.0);
	ZeroXParameters.ExpectedBBoundsMax = FVector(1.0, 1.0, 1.0);

	return GenerateTestDataRunAndValidate(ZeroXParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplitPointsTest_ZeroY, FPCGSplitPointTestBase, "Plugins.PCG.SplitPoints.ZeroY", PCGTestsCommon::TestFlags)

bool FPCGSplitPointsTest_ZeroY::RunTest(const FString& Parameters)
{
	FTestParameters ZeroYParameters{};

	ZeroYParameters.SplitPosition = 0.0f;
	ZeroYParameters.SplitAxis = EPCGSplitAxis::Y;
	ZeroYParameters.ExpectedABoundsMin = FVector(-1.0, -1.0, -1.0);
	ZeroYParameters.ExpectedABoundsMax = FVector(1.0, -1.0, 1.0);
	ZeroYParameters.ExpectedBBoundsMin = FVector(-1.0, -1.0, -1.0);
	ZeroYParameters.ExpectedBBoundsMax = FVector(1.0, 1.0, 1.0);

	return GenerateTestDataRunAndValidate(ZeroYParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplitPointsTest_ZeroZ, FPCGSplitPointTestBase, "Plugins.PCG.SplitPoints.ZeroZ", PCGTestsCommon::TestFlags)

bool FPCGSplitPointsTest_ZeroZ::RunTest(const FString& Parameters)
{
	FTestParameters ZeroZParameters{};

	ZeroZParameters.SplitPosition = 0.0f;
	ZeroZParameters.SplitAxis = EPCGSplitAxis::Z;
	ZeroZParameters.ExpectedABoundsMin = FVector(-1.0, -1.0, -1.0);
	ZeroZParameters.ExpectedABoundsMax = FVector(1.0, 1.0, -1.0);
	ZeroZParameters.ExpectedBBoundsMin = FVector(-1.0, -1.0, -1.0);
	ZeroZParameters.ExpectedBBoundsMax = FVector(1.0, 1.0, 1.0);

	return GenerateTestDataRunAndValidate(ZeroZParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplitPointsTest_OneX, FPCGSplitPointTestBase, "Plugins.PCG.SplitPoints.OneX", PCGTestsCommon::TestFlags)

bool FPCGSplitPointsTest_OneX::RunTest(const FString& Parameters)
{
	FTestParameters OneXParameters{};

	OneXParameters.SplitPosition = 1.0f;
	OneXParameters.SplitAxis = EPCGSplitAxis::X;
	OneXParameters.ExpectedABoundsMin = FVector(-1.0, -1.0, -1.0);
	OneXParameters.ExpectedABoundsMax = FVector(1.0, 1.0, 1.0);
	OneXParameters.ExpectedBBoundsMin = FVector(1.0, -1.0, -1.0);
	OneXParameters.ExpectedBBoundsMax = FVector(1.0, 1.0, 1.0);

	return GenerateTestDataRunAndValidate(OneXParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplitPointsTest_OneY, FPCGSplitPointTestBase, "Plugins.PCG.SplitPoints.OneY", PCGTestsCommon::TestFlags)

bool FPCGSplitPointsTest_OneY::RunTest(const FString& Parameters)
{
	FTestParameters OneYParameters{};

	OneYParameters.SplitPosition = 1.0f;
	OneYParameters.SplitAxis = EPCGSplitAxis::Y;
	OneYParameters.ExpectedABoundsMin = FVector(-1.0, -1.0, -1.0);
	OneYParameters.ExpectedABoundsMax = FVector(1.0, 1.0, 1.0);
	OneYParameters.ExpectedBBoundsMin = FVector(-1.0, 1.0, -1.0);
	OneYParameters.ExpectedBBoundsMax = FVector(1.0, 1.0, 1.0);

	return GenerateTestDataRunAndValidate(OneYParameters);
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSplitPointsTest_OneZ, FPCGSplitPointTestBase, "Plugins.PCG.SplitPoints.OneZ", PCGTestsCommon::TestFlags)

bool FPCGSplitPointsTest_OneZ::RunTest(const FString& Parameters)
{
	FTestParameters OneZParameters{};

	OneZParameters.SplitPosition = 1.0f;
	OneZParameters.SplitAxis = EPCGSplitAxis::Z;
	OneZParameters.ExpectedABoundsMin = FVector(-1.0, -1.0, -1.0);
	OneZParameters.ExpectedABoundsMax = FVector(1.0, 1.0, 1.0);
	OneZParameters.ExpectedBBoundsMin = FVector(-1.0, -1.0, 1.0);
	OneZParameters.ExpectedBBoundsMax = FVector(1.0, 1.0, 1.0);

	return GenerateTestDataRunAndValidate(OneZParameters);
}