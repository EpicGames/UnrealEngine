// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/PCGMetadataAccessor.h"
#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGManagedResource.h"

#include "Data/PCGPointData.h"
#include "Elements/PCGStaticMeshSpawner.h"

#include "MeshSelectors/PCGMeshSelectorWeighted.h"
#include "MeshSelectors/PCGMeshSelectorByAttribute.h"
#include "MeshSelectors/PCGMeshSelectorWeightedByCategory.h"


#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"

// FIXME: Unit tests are crashing, disabled for now.
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGStaticMeshSpawnerByAttributeTest, FPCGTestBaseClass, "Plugins.PCG.StaticMeshSpawner.ByAttribute", PCGTestsCommon::TestFlags | EAutomationTestFlags::Disabled)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGStaticMeshSpawnerWeightedTest, FPCGTestBaseClass, "Plugins.PCG.StaticMeshSpawner.Weighted", PCGTestsCommon::TestFlags | EAutomationTestFlags::Disabled)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGStaticMeshSpawnerWeightedByCategoryTest, FPCGTestBaseClass, "Plugins.PCG.StaticMeshSpawner.WeightedByCategory", PCGTestsCommon::TestFlags | EAutomationTestFlags::Disabled)

namespace
{
	struct FPCGByAttributeValidationData
	{
		FPCGByAttributeValidationData(TSoftObjectPtr<UStaticMesh> InMesh, int InStartIndex, int InEndIndex)
			: Mesh(InMesh), StartIndex(InStartIndex), EndIndex(InEndIndex)
		{}
	
		TSoftObjectPtr<UStaticMesh> Mesh;
		int StartIndex;
		int EndIndex;
	};

	struct FPCGWeightedByCategoryValidationData
	{
		FPCGWeightedByCategoryValidationData(UStaticMesh* InMesh, int InExpectedCount)
			: Mesh(InMesh), ExpectedCount(InExpectedCount)
		{}

		UStaticMesh* Mesh;
		int ExpectedCount;
	};
}

/** Finds the output UPCGBasePointData and builds a mapping from world-space position to each Point Index */
void GetOutputDataAndBuildPositionMap(FPCGTestBaseClass* Test, const TArray<FPCGTaggedData>& OutputData, TMap<FVector, int32>& OutLocationToPoint, const UPCGBasePointData*& OutPointData)
{
	check(Test);

	if (Test->TestEqual("Valid number of data elements in output", OutputData.Num(), 2))
	{
		TObjectPtr<const UPCGSettings> OutSettings = Cast<UPCGSettings>(OutputData[0].Data);
		Test->TestTrue("Valid output settings", OutSettings != nullptr);

		OutPointData = Cast<UPCGBasePointData>(OutputData[1].Data);
	}

	if (Test->TestNotNull("Valid output data", OutPointData))
	{
		const TConstPCGValueRange<FTransform> TransformRange = OutPointData->GetConstTransformValueRange();
		
		for (int32 Index = 0; Index < OutPointData->GetNumPoints(); ++Index)
		{
			OutLocationToPoint.Emplace(TransformRange[Index].GetLocation(), Index);
		}
	}
}

/** Iterates over instances of the ISMC to ensure each instance matches the softobjectpath metadata of the corresponding FPCGPoint */
void ValidateOutPointData(
	FPCGTestBaseClass* Test, 
	const UInstancedStaticMeshComponent* ISMC, 
	const TMap<FVector, int32>& LocationToPointIndex, 
	const FName& OutputAttributeName, 
	const UPCGBasePointData* OutPointData) 
{
	check(Test);
	check(ISMC);
	check(OutPointData && OutPointData->Metadata);

	const FPCGMetadataAttributeBase* AttributeBase = OutPointData->Metadata->GetConstAttribute(OutputAttributeName);
	if (!Test->TestNotNull("Metadata contains out attribute", AttributeBase)) 
	{ 
		return; 
	}

	const bool bIsMetadataTypeValid = AttributeBase->GetTypeId() == PCG::Private::MetadataTypes<FString>::Id;
	if (!Test->TestTrue("Output attribute is a valid type", bIsMetadataTypeValid)) 
	{ 
		return; 
	}

	const FPCGMetadataAttribute<FString>* OutAttribute = static_cast<const FPCGMetadataAttribute<FString>*>(AttributeBase);
	const FString MeshPath = ISMC->GetStaticMesh().GetPath();
	const PCGMetadataValueKey MeshValueKey = OutAttribute->FindValue(MeshPath);

	if (MeshValueKey == PCGDefaultValueKey)
	{
		if (!Test->TestEqual("Mesh value exists in output attribute", MeshPath, OutAttribute->GetValue(MeshValueKey)))
		{
			return;
		}
	}

	const TConstPCGValueRange<int64> MetadataEntryRange = OutPointData->GetConstMetadataEntryValueRange();
	for (int InstanceIndex = 0; InstanceIndex < ISMC->GetInstanceCount(); ++InstanceIndex)
	{
		FTransform Transform;
		ISMC->GetInstanceTransform(InstanceIndex, Transform, /*bWorldSpace=*/true);

		const int32* OutPointIndex = LocationToPointIndex.Find(Transform.GetLocation());

		if (Test->TestNotNull("Output point index exists", OutPointIndex) && Test->TestNotEqual("Ouput point index is valid", *OutPointIndex, (int32)INDEX_NONE))
		{
			const PCGMetadataValueKey ValueKey = OutAttribute->GetValueKey(MetadataEntryRange[*OutPointIndex]);
			Test->TestEqual("Valid mesh stored in output point metadata", ValueKey, MeshValueKey);
		}
	}
}

void TestMeshSelectorByAttribute(
	FPCGStaticMeshSpawnerByAttributeTest* Test, 
	const PCGTestsCommon::FTestData& TestData, 
	const FName& AttributeName, 
	const TArray<FPCGByAttributeValidationData>& ValidationDataEntries,
	bool bValidateOutput=false)
{
	check(Test);

	// set the MeshSelector
	UPCGStaticMeshSpawnerSettings* Settings = CastChecked<UPCGStaticMeshSpawnerSettings>(TestData.Settings);
	Settings->SetMeshSelectorType(UPCGMeshSelectorByAttribute::StaticClass());
	UPCGMeshSelectorByAttribute* MeshSelector = CastChecked<UPCGMeshSelectorByAttribute>(Settings->MeshSelectorParameters);
	MeshSelector->AttributeName = AttributeName;

	// initialize and execute the StaticMeshSpawner
	FPCGElementPtr StaticMeshSpawner = Settings->GetElement();
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!StaticMeshSpawner->Execute(Context.Get()))
	{}

	TMap<FVector, int32> LocationToPointIndex;
	const UPCGBasePointData* OutPointData = nullptr;

	if (bValidateOutput)
	{
		GetOutputDataAndBuildPositionMap(Test, Context->OutputData.TaggedData, LocationToPointIndex, OutPointData);
	}

	struct FPCGByAttributeValidIndexList
	{
		TArray<TPair<int, int>> ValidIndices;
		int Count = 0;
	};

	TMap<TSoftObjectPtr<UStaticMesh>, FPCGByAttributeValidIndexList> MapToIndexList;

	int ValidInstanceCount = 0;
	for (const FPCGByAttributeValidationData& Entry : ValidationDataEntries)
	{
		FPCGByAttributeValidIndexList* IndexList = MapToIndexList.Find(Entry.Mesh);

		if (!IndexList)
		{
			IndexList = &MapToIndexList.Emplace(Entry.Mesh, FPCGByAttributeValidIndexList());
		}

		IndexList->ValidIndices.Add(TPair<int, int>(Entry.StartIndex, Entry.EndIndex));
		IndexList->Count += Entry.EndIndex - Entry.StartIndex;

		ValidInstanceCount += Entry.EndIndex - Entry.StartIndex;
	}

	check(TestData.TestPCGComponent);

	TArray<UInstancedStaticMeshComponent*> ISMCs;
	TestData.TestPCGComponent->ForEachManagedResource([&ISMCs](UPCGManagedResource* InResource)
	{
		if (UPCGManagedISMComponent* Resource = Cast<UPCGManagedISMComponent>(InResource))
		{
			if (Resource->GetComponent())
			{
				ISMCs.Add(Resource->GetComponent());
			}
		}
	});

	int TotalInstanceCount = 0;
	for (UInstancedStaticMeshComponent* ISMC : ISMCs)
	{
		const int InstanceCount = ISMC->GetInstanceCount();
		TotalInstanceCount += InstanceCount;

		TSoftObjectPtr<UStaticMesh> StaticMesh = ISMC->GetStaticMesh();

		const FPCGByAttributeValidationData* Entry = ValidationDataEntries.FindByPredicate([StaticMesh](const FPCGByAttributeValidationData& Entry) {
			return Entry.Mesh == StaticMesh;
			});

		if (Test->TestNotNull("Validate instanced mesh exists in MeshEntries", Entry))
		{
			const FPCGByAttributeValidIndexList* IndexList = MapToIndexList.Find(Entry->Mesh);
			check(IndexList);

			Test->TestEqual("Valid instance count per mesh", InstanceCount, IndexList->Count);

			for (int InstanceIndex = 0; InstanceIndex < InstanceCount; InstanceIndex++)
			{
				FTransform Transform;
				ISMC->GetInstanceTransform(InstanceIndex, Transform);
				FVector Location = Transform.GetLocation();

				bool bFound = false;

				for (int i = 0; i < IndexList->ValidIndices.Num(); i++)
				{
					const int StartIndex = IndexList->ValidIndices[i].Key;
					const int EndIndex = IndexList->ValidIndices[i].Value;

					if (Location.X >= StartIndex && Location.X < EndIndex)
					{
						bFound = true;
						break;
					}
				}

				Test->TestTrue("Valid transform locations", bFound);
			}

			if (bValidateOutput)
			{
				ValidateOutPointData(Test, ISMC, LocationToPointIndex, Settings->OutAttributeName, OutPointData);
			}
		}
	}

	TestData.TestPCGComponent->bGenerated = true;
	TestData.TestPCGComponent->CleanupLocalImmediate(true);
	
	Test->TestEqual("Valid total instance count", TotalInstanceCount, ValidInstanceCount);
}

void TestMeshSelectorWeighted(
	FPCGStaticMeshSpawnerWeightedTest* Test, 
	const PCGTestsCommon::FTestData& TestData, 
	const TArray<FPCGMeshSelectorWeightedEntry>& Entries, 
	int PointCount,
	bool bValidateOutput=false)
{
	check(Test);

	// set the MeshSelector
	UPCGStaticMeshSpawnerSettings* Settings = CastChecked<UPCGStaticMeshSpawnerSettings>(TestData.Settings);
	Settings->SetMeshSelectorType(UPCGMeshSelectorWeighted::StaticClass());
	UPCGMeshSelectorWeighted* MeshSelector = CastChecked<UPCGMeshSelectorWeighted>(Settings->MeshSelectorParameters);
	MeshSelector->MeshEntries = Entries;

	// initialize and execute the StaticMeshSpawner
	FPCGElementPtr StaticMeshSpawner = Settings->GetElement();
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!StaticMeshSpawner->Execute(Context.Get()))
	{}

	TMap<FVector, int32> LocationToPointIndex;
	const UPCGBasePointData* OutPointData = nullptr;

	if (bValidateOutput)
	{
		GetOutputDataAndBuildPositionMap(Test, Context->OutputData.TaggedData, LocationToPointIndex, OutPointData);
	}

	int TotalWeight = 0;
	for (const FPCGMeshSelectorWeightedEntry& Entry : Entries)
	{
		TotalWeight += Entry.Weight;
	}

	check(TestData.TestPCGComponent);

	TArray<UInstancedStaticMeshComponent*> ISMCs;
	TestData.TestPCGComponent->ForEachManagedResource([&ISMCs](UPCGManagedResource* InResource)
	{
		if (UPCGManagedISMComponent* Resource = Cast<UPCGManagedISMComponent>(InResource))
		{
			if (Resource->GetComponent())
			{
				ISMCs.Add(Resource->GetComponent());
			}
		}
	});

	int TotalInstanceCount = 0;
	for (UInstancedStaticMeshComponent* ISMC : ISMCs)
	{
		const int InstanceCount = ISMC->GetInstanceCount();
		TotalInstanceCount += InstanceCount;

		TObjectPtr<UStaticMesh> StaticMesh = ISMC->GetStaticMesh();

		FPCGMeshSelectorWeightedEntry* Entry = MeshSelector->MeshEntries.FindByPredicate([StaticMesh](const FPCGMeshSelectorWeightedEntry& Entry) {
			return Entry.Descriptor.StaticMesh == StaticMesh.Get();
			});

		if (Test->TestNotNull("Validate instanced mesh exists in MeshEntries", Entry))
		{
			check(TotalWeight > 0);

			// validate correct InstanceCount (within a range) for the weight of this mesh
			const int TargetCount = PointCount * Entry->Weight / TotalWeight;
			const float ErrorBound = FMath::Max(TargetCount * 0.1f, 10); // 10% error boundary 
			Test->TestTrue("Valid instance count per mesh", InstanceCount >= TargetCount - ErrorBound && InstanceCount <= TargetCount + ErrorBound);

			if (bValidateOutput)
			{
				ValidateOutPointData(Test, ISMC, LocationToPointIndex, Settings->OutAttributeName, OutPointData);
			}
		}
	}

	TestData.TestPCGComponent->bGenerated = true;
	TestData.TestPCGComponent->CleanupLocalImmediate(true);
	
	Test->TestEqual("Valid total instance count", TotalInstanceCount, PointCount);
}

void TestMeshSelectorWeightedByCategory(
	FPCGStaticMeshSpawnerWeightedByCategoryTest* Test, 
	const PCGTestsCommon::FTestData& TestData, 
	const FName& AttributeName, 
	const TArray<FPCGWeightedByCategoryEntryList>& Entries, 
	const TArray<FPCGWeightedByCategoryValidationData>& ValidationDataEntries,
	int PointCount,
	bool bValidateOutput=false)
{
	check(Test);

	// set the MeshSelector
	UPCGStaticMeshSpawnerSettings* Settings = CastChecked<UPCGStaticMeshSpawnerSettings>(TestData.Settings);
	Settings->SetMeshSelectorType(UPCGMeshSelectorWeightedByCategory::StaticClass());
	UPCGMeshSelectorWeightedByCategory* MeshSelector = CastChecked<UPCGMeshSelectorWeightedByCategory>(Settings->MeshSelectorParameters);
	MeshSelector->CategoryAttribute = AttributeName;
	MeshSelector->Entries = Entries;

	// initialize and execute the StaticMeshSpawner
	FPCGElementPtr StaticMeshSpawner = Settings->GetElement();
	TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

	while (!StaticMeshSpawner->Execute(Context.Get()))
	{
	}

	TMap<FVector, int32> LocationToPointIndex;
	const UPCGBasePointData* OutPointData = nullptr;

	if (bValidateOutput)
	{
		GetOutputDataAndBuildPositionMap(Test, Context->OutputData.TaggedData, LocationToPointIndex, OutPointData);
	}

	check(TestData.TestPCGComponent);

	TArray<UInstancedStaticMeshComponent*> ISMCs;
	TestData.TestPCGComponent->ForEachManagedResource([&ISMCs](UPCGManagedResource* InResource)
		{
			if (UPCGManagedISMComponent* Resource = Cast<UPCGManagedISMComponent>(InResource))
			{
				if (Resource->GetComponent())
				{
					ISMCs.Add(Resource->GetComponent());
				}
			}
		});

	int TotalInstanceCount = 0;
	for (UInstancedStaticMeshComponent* ISMC : ISMCs)
	{
		const int InstanceCount = ISMC->GetInstanceCount();
		TotalInstanceCount += InstanceCount;

		TObjectPtr<UStaticMesh> StaticMesh = ISMC->GetStaticMesh();

		const FPCGWeightedByCategoryValidationData* Entry = ValidationDataEntries.FindByPredicate([StaticMesh](const FPCGWeightedByCategoryValidationData& Entry) {
			return Entry.Mesh == StaticMesh;
			});

		if (Test->TestNotNull("Validate instanced mesh exists in entries", Entry))
		{
			const float ErrorBound = FMath::Max(Entry->ExpectedCount * 0.1f, 10); // 10% error boundary 
			Test->TestTrue("Valid instance count per mesh", InstanceCount >= Entry->ExpectedCount - ErrorBound && InstanceCount <= Entry->ExpectedCount + ErrorBound);

			if (bValidateOutput)
			{
				ValidateOutPointData(Test, ISMC, LocationToPointIndex, Settings->OutAttributeName, OutPointData);
			}
		}
	}

	TestData.TestPCGComponent->bGenerated = true;
	TestData.TestPCGComponent->CleanupLocalImmediate(true);

	Test->TestEqual("Valid total instance count", TotalInstanceCount, PointCount);
}

bool FPCGStaticMeshSpawnerByAttributeTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGStaticMeshSpawnerSettings>(TestData);

	// create test attribute data
	const FName RockAttribute = "Rock";
	const FName TreeAttribute = "Tree";
	const FName BushAttribute = "Bush";

	FSoftObjectPath CubePath(TEXT("StaticMesh'/Engine/BasicShapes/Cube.Cube'"));
	FSoftObjectPath SpherePath(TEXT("StaticMesh'/Engine/BasicShapes/Sphere.Sphere'"));
	FSoftObjectPath CylinderPath(TEXT("StaticMesh'/Engine/BasicShapes/Cylinder.Cylinder'"));
	FSoftObjectPath ConePath(TEXT("StaticMesh'/Engine/BasicShapes/Cone.Cone'"));

	TSoftObjectPtr<UStaticMesh> CubeMesh(CubePath);
	TSoftObjectPtr<UStaticMesh> SphereMesh(SpherePath);
	TSoftObjectPtr<UStaticMesh> CylinderMesh(CylinderPath);
	TSoftObjectPtr<UStaticMesh> ConeMesh(ConePath);

	// compose the test data
	TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateEmptyBasePointData();
	PointData->TargetActor = TestData.TestActor;
	PointData->Metadata->CreateStringAttribute(RockAttribute, ConePath.ToString(), false);
	PointData->Metadata->CreateStringAttribute(TreeAttribute, ConePath.ToString(), false);
	PointData->Metadata->CreateStringAttribute(BushAttribute, ConePath.ToString(), false);

	// first 20 points are Rock, next 50 are Tree, next 30 are Bush, for a total of 100 points
	const int PointCount = 100;
	PointData->SetNumPoints(PointCount);
	PointData->SetDensity(1);

	TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
	TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange();
	TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();
	
	for (int i = 0; i < PointCount; i++)
	{
		const FVector Position(i, 0, 0);
		TransformRange[i] = FTransform(Position);
		SeedRange[i] = i;

		FName AttributeName;
		FSoftObjectPath ObjectPath;

		if (i < 20)
		{
			AttributeName = RockAttribute;
			ObjectPath = (i < 10) ? CubePath : SpherePath; // half are CubePath, half are SpherePath
		}
		else if (i < 70)
		{
			AttributeName = TreeAttribute;
			ObjectPath = (i < 35) ? CubePath : (i < 50) ? SpherePath : CylinderPath; // some are CubePath, then SpherePath, then CylinderPath
		}
		else 
		{
			AttributeName = BushAttribute;
			ObjectPath = CylinderPath;
			ObjectPath = (i < 85) ? CubePath : CylinderPath; // half are CubePath, half are CylinderPath
		}
		
		UPCGMetadataAccessorHelpers::SetStringAttributeByMetadataKey(MetadataEntryRange[i], PointData->Metadata, AttributeName, ObjectPath.ToString());
	}
	
	FPCGTaggedData& TaggedDataPoints = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedDataPoints.Data = PointData;

	// Test 1
	// should have 10 CubeMesh at Loc.x = [0, 10), then 10 SphereMesh at Loc.x = [10, 20), and then 80 ConeMesh at Loc.x = [20, 100)
	TArray<FPCGByAttributeValidationData> ValidationDataEntries;
	ValidationDataEntries.Emplace(CubeMesh, 0, 10);
	ValidationDataEntries.Emplace(SphereMesh, 10, 20);
	ValidationDataEntries.Emplace(ConeMesh, 20, 100);
	TestMeshSelectorByAttribute(this, TestData, RockAttribute, ValidationDataEntries);

	// Test 2
	// should have ConeMesh at Loc.x = [0, 20), [70, 100), CubeMesh at Loc.x = [20, 35), SphereMesh at Loc.x = [35, 50), CylinderMesh at Loc.x = [50, 70)
	ValidationDataEntries.Empty();
	ValidationDataEntries.Emplace(ConeMesh, 0, 20);
	ValidationDataEntries.Emplace(CubeMesh, 20, 35);
	ValidationDataEntries.Emplace(SphereMesh, 35, 50);
	ValidationDataEntries.Emplace(CylinderMesh, 50, 70);
	ValidationDataEntries.Emplace(ConeMesh, 70, 100);
	TestMeshSelectorByAttribute(this, TestData, TreeAttribute, ValidationDataEntries);

	// Test 3 - also tests optional output data
	// should have ConeMesh at Loc.x = [0, 70), CubeMesh at Loc.x = [70, 85), CylinderMesh at Loc.x = [85, 100)
	ValidationDataEntries.Empty();
	ValidationDataEntries.Emplace(ConeMesh, 0, 70);
	ValidationDataEntries.Emplace(CubeMesh, 70, 85);
	ValidationDataEntries.Emplace(CylinderMesh, 85, 100);
	TestMeshSelectorByAttribute(this, TestData, BushAttribute, ValidationDataEntries, /*bValidateOutput=*/true);

	return true;
}

bool FPCGStaticMeshSpawnerWeightedTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGStaticMeshSpawnerSettings>(TestData);

	// create test meshes
	const FString CubePath = TEXT("StaticMesh'/Engine/BasicShapes/Cube.Cube'");
	const FString SpherePath = TEXT("StaticMesh'/Engine/BasicShapes/Sphere.Sphere'");
	const FString CylinderPath = TEXT("StaticMesh'/Engine/BasicShapes/Cylinder.Cylinder'");

	UStaticMesh* CubeMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, *CubePath));
	UStaticMesh* SphereMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, *SpherePath));
	UStaticMesh* CylinderMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, *CylinderPath));

	// compose the test data
	const int PointCount = 1000;
	TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateEmptyBasePointData();
	PointData->TargetActor = TestData.TestActor;

	PointData->SetNumPoints(PointCount);
	PointData->SetDensity(1);

	TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
	TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange();
	
	for (int i = 0; i < PointCount; i++)
	{
		const FVector Position(i, 0, 0);
		TransformRange[i] = FTransform(Position);
		SeedRange[i] = i;
	}
	
	FPCGTaggedData& TaggedDataPoints = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedDataPoints.Data = PointData;

	// Test 1: even distribution
	TArray<FPCGMeshSelectorWeightedEntry> Entries;
	Entries.Emplace(CubeMesh, 1);
	Entries.Emplace(SphereMesh, 1);
	Entries.Emplace(CylinderMesh, 1);
	TestMeshSelectorWeighted(this, TestData, Entries, PointCount);

	// Test 2: skewed distribution
	Entries[0].Weight = 1;
	Entries[1].Weight = 10;
	Entries[2].Weight = 1;
	TestMeshSelectorWeighted(this, TestData, Entries, PointCount);

	// Test 3: null weight
	Entries[0].Weight = 1;
	Entries[1].Weight = 0;
	Entries[2].Weight = 1;
	TestMeshSelectorWeighted(this, TestData, Entries, PointCount);

	// Test 4: heavily skewed distribution
	Entries[0].Weight = 1000;
	Entries[1].Weight = 1;
	Entries[2].Weight = 1;
	TestMeshSelectorWeighted(this, TestData, Entries, PointCount);

	// Test 5: zero total weight
	Entries[0].Weight = 0;
	Entries[1].Weight = 0;
	Entries[2].Weight = 0;
	TestMeshSelectorWeighted(this, TestData, Entries, 0);

	// Test 6: validate optional output data
	Entries[0].Weight = 1;
	Entries[1].Weight = 10;
	Entries[2].Weight = 100;
	TestMeshSelectorWeighted(this, TestData, Entries, PointCount, /*bValidateOutput=*/true);

	return true;
}

bool FPCGStaticMeshSpawnerWeightedByCategoryTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGStaticMeshSpawnerSettings>(TestData);

	// create test meshes
	const FString CubePath = TEXT("StaticMesh'/Engine/BasicShapes/Cube.Cube'");
	const FString SpherePath = TEXT("StaticMesh'/Engine/BasicShapes/Sphere.Sphere'");
	const FString CylinderPath = TEXT("StaticMesh'/Engine/BasicShapes/Cylinder.Cylinder'");
	const FString ConePath = TEXT("StaticMesh'/Engine/BasicShapes/Cone.Cone'");

	UStaticMesh* CubeMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, *CubePath));
	UStaticMesh* SphereMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, *SpherePath));
	UStaticMesh* CylinderMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, *CylinderPath));
	UStaticMesh* ConeMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, *ConePath));

	// Attribute categories
	const FName AttributeName = TEXT("RockType");
	const FString IgneousRockType = TEXT("Igneous");
	const FString SedimentaryRockType = TEXT("Sedimentary");
	const FString MetamorphicRockType = TEXT("Metamorphic");
	const FString DefaultRockType = TEXT("DefaultRockType");
	const FString InvalidRockType = TEXT("InvalidRockType");

	// compose the test data
	TObjectPtr<UPCGBasePointData> PointData = PCGTestsCommon::CreateEmptyBasePointData();
	PointData->TargetActor = TestData.TestActor;
	PointData->Metadata->CreateStringAttribute(AttributeName, DefaultRockType, false);

	// first 1k are igneous, next 1.5k are sedimenatary, next 500 are metamorphic, then last 500 are Default
	const int PointCount = 3500;
	const int IgneousCategorySize = 1000;
	const int SedimentaryCategorySize = 1500;
	const int MetamorphicCategorySize = 500;
	const int DefaultCategorySize = 500;

	PointData->SetNumPoints(PointCount);
	PointData->SetDensity(1);

	TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
	TPCGValueRange<int32> SeedRange = PointData->GetSeedValueRange();
	TPCGValueRange<int64> MetadataEntryRange = PointData->GetMetadataEntryValueRange();

	for (int i = 0; i < PointCount; i++)
	{
		const FVector Position(i, 0, 0);
		TransformRange[i] = FTransform(Position);
		SeedRange[i] = i;

		FString AttributeValue;
		
		if (i < IgneousCategorySize)
		{
			AttributeValue = IgneousRockType;
		} 
		else if (i < IgneousCategorySize + SedimentaryCategorySize)
		{
			AttributeValue = SedimentaryRockType;
		} 
		else if (i < IgneousCategorySize + SedimentaryCategorySize + MetamorphicCategorySize)
		{
			AttributeValue = MetamorphicRockType;
		}
		else
		{
			AttributeValue = DefaultRockType;
		} 
		
		UPCGMetadataAccessorHelpers::SetStringAttributeByMetadataKey(MetadataEntryRange[i], PointData->Metadata, AttributeName, AttributeValue);
	}
	
	FPCGTaggedData& TaggedDataPoints = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	TaggedDataPoints.Data = PointData;

	// Test 1 - basic weighted functionality
	{
		const int CubeWeight = 1;
		const int SphereWeight = 1;
		const int CylinderWeight = 1;

		const int IgneousCategoryTotalWeight = 3;

		const int TotalSize = IgneousCategorySize;

		TArray<FPCGMeshSelectorWeightedEntry> IgneousEntries;
		IgneousEntries.Emplace(CubeMesh, CubeWeight);
		IgneousEntries.Emplace(SphereMesh, SphereWeight);
		IgneousEntries.Emplace(CylinderMesh, CylinderWeight);

		TArray<FPCGWeightedByCategoryEntryList> Entries;
		Entries.Emplace(IgneousRockType, IgneousEntries);

		const int CubeCount = (IgneousCategorySize * CubeWeight / IgneousCategoryTotalWeight);
		const int SphereCount = (IgneousCategorySize * SphereWeight / IgneousCategoryTotalWeight);
		const int CylinderCount = (IgneousCategorySize * CylinderWeight / IgneousCategoryTotalWeight);

		TArray<FPCGWeightedByCategoryValidationData> ValidationData;
		ValidationData.Emplace(CubeMesh, CubeCount);
		ValidationData.Emplace(SphereMesh, SphereCount);
		ValidationData.Emplace(CylinderMesh, CylinderCount);

		TestMeshSelectorWeightedByCategory(this, TestData, AttributeName, Entries, ValidationData, TotalSize);
	}

	// Test 2 - selection based on rocktype
	{
		const int IgneousCubeWeight = 1;
		const int IgneousSphereWeight = 1;

		const int SedimentaryCubeWeight = 10;
		const int SedimentaryCylinderWeight = 1;

		const int MetamorphicSphereWeight = 1;
		const int MetamorphicCylinderWeight = 10;

		const int IgneousCategoryTotalWeight = 2;
		const int SedimentaryCategoryTotalWeight = 11;
		const int MetamorphicCategoryTotalWeight = 11;

		const int TotalSize = IgneousCategorySize + SedimentaryCategorySize + MetamorphicCategorySize;

		TArray<FPCGMeshSelectorWeightedEntry> IgneousEntries;
		IgneousEntries.Emplace(CubeMesh, IgneousCubeWeight);
		IgneousEntries.Emplace(SphereMesh, IgneousSphereWeight);

		TArray<FPCGMeshSelectorWeightedEntry> SedimentaryEntries;
		SedimentaryEntries.Emplace(CubeMesh, SedimentaryCubeWeight);
		SedimentaryEntries.Emplace(CylinderMesh, SedimentaryCylinderWeight);

		TArray<FPCGMeshSelectorWeightedEntry> MetamorphicEntries;
		MetamorphicEntries.Emplace(SphereMesh, MetamorphicSphereWeight);
		MetamorphicEntries.Emplace(CylinderMesh, MetamorphicCylinderWeight);

		TArray<FPCGWeightedByCategoryEntryList> Entries;
		Entries.Emplace(IgneousRockType, IgneousEntries);
		Entries.Emplace(SedimentaryRockType, SedimentaryEntries);
		Entries.Emplace(MetamorphicRockType, MetamorphicEntries);

		const int CubeCount = (IgneousCategorySize * IgneousCubeWeight / IgneousCategoryTotalWeight) + (SedimentaryCategorySize * SedimentaryCubeWeight / SedimentaryCategoryTotalWeight);
		const int SphereCount = (IgneousCategorySize * IgneousSphereWeight / IgneousCategoryTotalWeight) + (MetamorphicCategorySize * MetamorphicSphereWeight / MetamorphicCategoryTotalWeight);
		const int CylinderCount = (SedimentaryCategorySize * SedimentaryCylinderWeight / SedimentaryCategoryTotalWeight) + (MetamorphicCategorySize * MetamorphicCylinderWeight / MetamorphicCategoryTotalWeight);

		TArray<FPCGWeightedByCategoryValidationData> ValidationData;
		ValidationData.Emplace(CubeMesh, CubeCount);
		ValidationData.Emplace(SphereMesh, SphereCount);
		ValidationData.Emplace(CylinderMesh, CylinderCount);

		TestMeshSelectorWeightedByCategory(this, TestData, AttributeName, Entries, ValidationData, TotalSize);
	}

	// Test 3 - empty weighted entries
	{
		const int CubeWeight = 1;
		const int IgneousSphereWeight = 1;
		const int MetamorphicSphereWeight = 1;
		const int CylinderWeight = 10;

		const int IgneousCategoryTotalWeight = 2;
		const int MetamorphicCategoryTotalWeight = 11;

		const int TotalSize = IgneousCategorySize + MetamorphicCategorySize;

		TArray<FPCGMeshSelectorWeightedEntry> IgneousEntries;
		IgneousEntries.Emplace(CubeMesh, CubeWeight);
		IgneousEntries.Emplace(SphereMesh, IgneousSphereWeight);

		TArray<FPCGMeshSelectorWeightedEntry> SedimentaryEntries;

		TArray<FPCGMeshSelectorWeightedEntry> MetamorphicEntries;
		MetamorphicEntries.Emplace(SphereMesh, MetamorphicSphereWeight);
		MetamorphicEntries.Emplace(CylinderMesh, CylinderWeight);

		TArray<FPCGWeightedByCategoryEntryList> Entries;
		Entries.Emplace(IgneousRockType, IgneousEntries);
		Entries.Emplace(SedimentaryRockType, SedimentaryEntries);
		Entries.Emplace(MetamorphicRockType, MetamorphicEntries);

		const int CubeCount = (IgneousCategorySize * CubeWeight / IgneousCategoryTotalWeight);
		const int SphereCount = (IgneousCategorySize * IgneousSphereWeight / IgneousCategoryTotalWeight) + (MetamorphicCategorySize * MetamorphicSphereWeight / MetamorphicCategoryTotalWeight);
		const int CylinderCount = (MetamorphicCategorySize * CylinderWeight / MetamorphicCategoryTotalWeight);

		TArray<FPCGWeightedByCategoryValidationData> ValidationData;
		ValidationData.Emplace(CubeMesh, CubeCount);
		ValidationData.Emplace(SphereMesh, SphereCount);
		ValidationData.Emplace(CylinderMesh, CylinderCount);

		TestMeshSelectorWeightedByCategory(this, TestData, AttributeName, Entries, ValidationData, TotalSize);
	}

	// Test 4 - duplicate category
	{
		const int CubeWeight = 1;
		const int SphereWeight = 1;

		const int IgneousCategoryTotalWeight = 1;
		const int SedimentaryCategoryTotalWeight = 1;

		const int TotalSize = IgneousCategorySize + SedimentaryCategorySize;

		TArray<FPCGMeshSelectorWeightedEntry> IgneousEntries;
		IgneousEntries.Emplace(CubeMesh, CubeWeight);

		TArray<FPCGMeshSelectorWeightedEntry> SedimentaryEntries;
		SedimentaryEntries.Emplace(SphereMesh, SphereWeight);

		TArray<FPCGMeshSelectorWeightedEntry> SedimentaryEntriesDuplicate;
		SedimentaryEntriesDuplicate.Emplace(SphereMesh, 10000); // heavily skewed to ensure its effect on the data is noticeable

		TArray<FPCGWeightedByCategoryEntryList> Entries;
		Entries.Emplace(IgneousRockType, IgneousEntries);
		Entries.Emplace(SedimentaryRockType, SedimentaryEntries);
		Entries.Emplace(SedimentaryRockType, SedimentaryEntriesDuplicate);

		const int CubeCount = (IgneousCategorySize * CubeWeight / IgneousCategoryTotalWeight);
		const int SphereCount = (SedimentaryCategorySize * SphereWeight / SedimentaryCategoryTotalWeight);

		TArray<FPCGWeightedByCategoryValidationData> ValidationData;
		ValidationData.Emplace(CubeMesh, CubeCount);
		ValidationData.Emplace(SphereMesh, SphereCount);

		const FString& ErrorMessage = TEXT("Duplicate entry found in category ") + SedimentaryRockType + TEXT(". Subsequent entries are ignored.");
		AddExpectedError(ErrorMessage, EAutomationExpectedErrorFlags::Contains, 1);

		TestMeshSelectorWeightedByCategory(this, TestData, AttributeName, Entries, ValidationData, TotalSize);
	}

	// Test 5 - invalid rock type
	{
		const int CubeWeight = 1;
		const int SphereWeight = 1;
		const int CategoryTotalWeight = 2;
		const int TotalSize = IgneousCategorySize;

		TArray<FPCGMeshSelectorWeightedEntry> IgneousEntries;
		IgneousEntries.Emplace(CubeMesh, 1);
		IgneousEntries.Emplace(SphereMesh, 1);

		TArray<FPCGMeshSelectorWeightedEntry> InvalidEntries;
		InvalidEntries.Emplace(SphereMesh, 10000); // heavily skewed to ensure its effect on the data is noticeable

		TArray<FPCGWeightedByCategoryEntryList> Entries;
		Entries.Emplace(IgneousRockType, IgneousEntries);
		Entries.Emplace(InvalidRockType, InvalidEntries);

		const int CubeCount = (IgneousCategorySize * CubeWeight / CategoryTotalWeight);
		const int SphereCount = (IgneousCategorySize * SphereWeight / CategoryTotalWeight);

		TArray<FPCGWeightedByCategoryValidationData> ValidationData;
		ValidationData.Emplace(CubeMesh, CubeCount);
		ValidationData.Emplace(SphereMesh, SphereCount);

		TestMeshSelectorWeightedByCategory(this, TestData, AttributeName, Entries, ValidationData, TotalSize);
	}

	// Test 6 - default entry list + duplicate default ignored
	{
		const int IgneousCubeWeight = 1;
		const int IgneousSphereWeight = 1;

		const int SedimentaryCubeWeight = 10;
		const int SedimentaryCylinderWeight = 1;

		const int MetamorphicSphereWeight = 1;
		const int MetamorphicCylinderWeight = 10;

		const int IgneousCategoryTotalWeight = 2;
		const int SedimentaryCategoryTotalWeight = 11;
		const int MetamorphicCategoryTotalWeight = 11;

		const int TotalSize = IgneousCategorySize + SedimentaryCategorySize + MetamorphicCategorySize + DefaultCategorySize;

		TArray<FPCGMeshSelectorWeightedEntry> IgneousEntries;
		IgneousEntries.Emplace(CubeMesh, IgneousCubeWeight);
		IgneousEntries.Emplace(SphereMesh, IgneousSphereWeight);

		TArray<FPCGMeshSelectorWeightedEntry> SedimentaryEntries;
		SedimentaryEntries.Emplace(CubeMesh, SedimentaryCubeWeight);
		SedimentaryEntries.Emplace(CylinderMesh, SedimentaryCylinderWeight);

		TArray<FPCGMeshSelectorWeightedEntry> MetamorphicEntries;
		MetamorphicEntries.Emplace(SphereMesh, MetamorphicSphereWeight);
		MetamorphicEntries.Emplace(CylinderMesh, MetamorphicCylinderWeight);

		TArray<FPCGMeshSelectorWeightedEntry> DuplicateDefault;
		DuplicateDefault.Emplace(CubeMesh, 10000); // heavily skewed to ensure its effect on the data is noticeable

		TArray<FPCGWeightedByCategoryEntryList> Entries;
		Entries.Emplace(IgneousRockType, IgneousEntries);
		Entries.Emplace(MetamorphicRockType, MetamorphicEntries);

		FPCGWeightedByCategoryEntryList& DefaultEntryList = Entries.Emplace_GetRef(SedimentaryRockType, SedimentaryEntries);
		FPCGWeightedByCategoryEntryList& DefaultEntryListDuplicate = Entries.Emplace_GetRef(MetamorphicRockType, DuplicateDefault);

		DefaultEntryList.IsDefault = true;
		DefaultEntryListDuplicate.IsDefault = true;

		const int CubeCount = (IgneousCategorySize * IgneousCubeWeight / IgneousCategoryTotalWeight)
			+ (SedimentaryCategorySize * SedimentaryCubeWeight / SedimentaryCategoryTotalWeight)
			+ (DefaultCategorySize * SedimentaryCubeWeight / SedimentaryCategoryTotalWeight);

		const int SphereCount = (IgneousCategorySize * IgneousSphereWeight / IgneousCategoryTotalWeight)
			+ (MetamorphicCategorySize * MetamorphicSphereWeight / MetamorphicCategoryTotalWeight);

		const int CylinderCount = (SedimentaryCategorySize * SedimentaryCylinderWeight / SedimentaryCategoryTotalWeight)
			+ (MetamorphicCategorySize * MetamorphicCylinderWeight / MetamorphicCategoryTotalWeight)
			+ (DefaultCategorySize * SedimentaryCylinderWeight / SedimentaryCategoryTotalWeight);

		TArray<FPCGWeightedByCategoryValidationData> ValidationData;
		ValidationData.Emplace(CubeMesh, CubeCount);
		ValidationData.Emplace(SphereMesh, SphereCount);
		ValidationData.Emplace(CylinderMesh, CylinderCount);

		const FString& ErrorMessage = TEXT("Duplicate entry found in category ") + MetamorphicRockType + TEXT(". Subsequent entries are ignored.");
		AddExpectedError(ErrorMessage, EAutomationExpectedErrorFlags::Contains, 1);

		TestMeshSelectorWeightedByCategory(this, TestData, AttributeName, Entries, ValidationData, TotalSize);
	}

	// Test 7 - zero total weight for a category
	{
		const int CubeWeight = 0;
		const int SphereWeight = 0;

		const int CylinderWeight = 1;
		const int ConeWeight = 1;

		const int SedimentaryCategoryTotalWeight = 2;

		const int TotalSize = SedimentaryCategorySize;

		TArray<FPCGMeshSelectorWeightedEntry> IgneousEntries;
		IgneousEntries.Emplace(CubeMesh, CubeWeight);
		IgneousEntries.Emplace(SphereMesh, SphereWeight);

		TArray<FPCGMeshSelectorWeightedEntry> SedimentaryEntries;
		SedimentaryEntries.Emplace(CylinderMesh, CylinderWeight);
		SedimentaryEntries.Emplace(ConeMesh, ConeWeight);

		TArray<FPCGWeightedByCategoryEntryList> Entries;
		Entries.Emplace(IgneousRockType, IgneousEntries);
		Entries.Emplace(SedimentaryRockType, SedimentaryEntries);

		const int CubeCount = 0;
		const int SphereCount = 0;
		const int CylinderCount = (SedimentaryCategorySize * CylinderWeight / SedimentaryCategoryTotalWeight);
		const int ConeCount = (SedimentaryCategorySize * ConeWeight / SedimentaryCategoryTotalWeight);

		TArray<FPCGWeightedByCategoryValidationData> ValidationData;
		ValidationData.Emplace(CubeMesh, CubeCount);
		ValidationData.Emplace(SphereMesh, SphereCount);
		ValidationData.Emplace(CylinderMesh, CylinderCount);
		ValidationData.Emplace(ConeMesh, ConeCount);

		TestMeshSelectorWeightedByCategory(this, TestData, AttributeName, Entries, ValidationData, TotalSize);
	}

	// Test 8 - validate optional output data
	{
		const int IgneousCubeWeight = 1;
		const int IgneousSphereWeight = 1;

		const int SedimentaryCubeWeight = 10;
		const int SedimentaryCylinderWeight = 1;

		const int MetamorphicSphereWeight = 1;
		const int MetamorphicCylinderWeight = 10;

		const int IgneousCategoryTotalWeight = 2;
		const int SedimentaryCategoryTotalWeight = 11;
		const int MetamorphicCategoryTotalWeight = 11;

		const int TotalSize = IgneousCategorySize + SedimentaryCategorySize + MetamorphicCategorySize;

		TArray<FPCGMeshSelectorWeightedEntry> IgneousEntries;
		IgneousEntries.Emplace(CubeMesh, IgneousCubeWeight);
		IgneousEntries.Emplace(SphereMesh, IgneousSphereWeight);

		TArray<FPCGMeshSelectorWeightedEntry> SedimentaryEntries;
		SedimentaryEntries.Emplace(CubeMesh, SedimentaryCubeWeight);
		SedimentaryEntries.Emplace(CylinderMesh, SedimentaryCylinderWeight);

		TArray<FPCGMeshSelectorWeightedEntry> MetamorphicEntries;
		MetamorphicEntries.Emplace(SphereMesh, MetamorphicSphereWeight);
		MetamorphicEntries.Emplace(CylinderMesh, MetamorphicCylinderWeight);

		TArray<FPCGWeightedByCategoryEntryList> Entries;
		Entries.Emplace(IgneousRockType, IgneousEntries);
		Entries.Emplace(SedimentaryRockType, SedimentaryEntries);
		Entries.Emplace(MetamorphicRockType, MetamorphicEntries);

		const int CubeCount = (IgneousCategorySize * IgneousCubeWeight / IgneousCategoryTotalWeight) + (SedimentaryCategorySize * SedimentaryCubeWeight / SedimentaryCategoryTotalWeight);
		const int SphereCount = (IgneousCategorySize * IgneousSphereWeight / IgneousCategoryTotalWeight) + (MetamorphicCategorySize * MetamorphicSphereWeight / MetamorphicCategoryTotalWeight);
		const int CylinderCount = (SedimentaryCategorySize * SedimentaryCylinderWeight / SedimentaryCategoryTotalWeight) + (MetamorphicCategorySize * MetamorphicCylinderWeight / MetamorphicCategoryTotalWeight);

		TArray<FPCGWeightedByCategoryValidationData> ValidationData;
		ValidationData.Emplace(CubeMesh, CubeCount);
		ValidationData.Emplace(SphereMesh, SphereCount);
		ValidationData.Emplace(CylinderMesh, CylinderCount);

		TestMeshSelectorWeightedByCategory(this, TestData, AttributeName, Entries, ValidationData, TotalSize, /*bValidateOutput=*/true);
	}

	return true;
}
