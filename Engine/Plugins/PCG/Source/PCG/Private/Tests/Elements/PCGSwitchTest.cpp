// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/ControlFlow/PCGSwitch.h"
#include "Tests/PCGTestsCommon.h"

#if WITH_EDITOR

class FPCGSwitchTest : public FPCGTestBaseClass
{
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	inline static const FName DefaultPathPinLabel = TEXT("Default");

	static void UpdateSettings(const UPCGSwitchSettings* InSettings, UPCGSwitchSettings* SelectSettings)
	{
		SelectSettings->SelectionMode = InSettings->SelectionMode;
		SelectSettings->IntOptions = InSettings->IntOptions;
		SelectSettings->IntegerSelection = InSettings->IntegerSelection;
		SelectSettings->StringOptions = InSettings->StringOptions;
		SelectSettings->StringSelection = InSettings->StringSelection;
		SelectSettings->EnumSelection = InSettings->EnumSelection;
		SelectSettings->UpdateSettings();
	}

	bool ExecuteTest(const UPCGSwitchSettings* InSettings, FPCGDataCollection& OutputData)
	{
		PCGTestsCommon::FTestData TestData;
		UPCGSwitchSettings* TestSettings = PCGTestsCommon::GenerateSettings<UPCGSwitchSettings>(TestData);
		UpdateSettings(InSettings, TestSettings);

		const FPCGElementPtr TestElement = TestSettings->GetElement();

		UTEST_TRUE("Test element created and valid", TestElement.IsValid());

		FPCGTaggedData& TaggedData = TestData.InputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = PCGTestsCommon::CreateEmptyPointData();
		TaggedData.Pin = PCGPinConstants::DefaultInputLabel;

		const TUniquePtr<FPCGContext> TestContext = TestData.InitializeTestContext();

		UTEST_TRUE("Test context created and valid", TestContext.IsValid());

		while (!TestElement->Execute(TestContext.Get()))
		{
		}

		OutputData = TestContext->OutputData;

		return true;
	}

	bool ValidateOutput(const FPCGDataCollection& OutputData, const FName CorrectOutputLabel)
	{
		UTEST_EQUAL("Only one output exists", OutputData.GetAllSpatialInputs().Num(), 1);

		const TArray<FPCGTaggedData> OnPinData = OutputData.GetInputsByPin(CorrectOutputLabel);

		UTEST_EQUAL("Output is on the correct pin", OnPinData.Num(), 1);

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSwitchTest_Integer, FPCGSwitchTest, "Plugins.PCG.Switch.Integer", PCGTestsCommon::TestFlags)

bool FPCGSwitchTest_Integer::RunTest(const FString& Parameters)
{
	FPCGDataCollection OutputData;
	UPCGSwitchSettings* TestSettings = NewObject<UPCGSwitchSettings>();
	TestSettings->SelectionMode = EPCGControlFlowSelectionMode::Integer;

	TArray<int32>& Options = TestSettings->IntOptions;
	Options = {-1, 0, 2, 3, 5};

	// Positive Validation
	for (int i = 0; i < Options.Num(); ++i)
	{
		TestSettings->IntegerSelection = Options[i];

		UTEST_TRUE("Integer test succeeded", ExecuteTest(TestSettings, OutputData));

		UTEST_TRUE("Integer selection is valid", ValidateOutput(OutputData, FName(FString::FromInt(Options[i]))));
	}

	// Negative validation
	const int32 Min = FMath::Min(Options);
	const int32 Max = FMath::Max(Options);
	for (int i = Min - 1; i <= Max + 1; ++i)
	{
		if (Options.Contains(i))
		{
			continue;
		}

		TestSettings->IntegerSelection = i;

		UTEST_TRUE("Integer test succeeded", ExecuteTest(TestSettings, OutputData));

		UTEST_TRUE("Integer selection is invalid. Default path was selected", ValidateOutput(OutputData, DefaultPathPinLabel));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSwitchTest_String, FPCGSwitchTest, "Plugins.PCG.Switch.String", PCGTestsCommon::TestFlags)

bool FPCGSwitchTest_String::RunTest(const FString& Parameters)
{
	FPCGDataCollection OutputData;
	UPCGSwitchSettings* TestSettings = NewObject<UPCGSwitchSettings>();
	TestSettings->SelectionMode = EPCGControlFlowSelectionMode::String;

	TArray<FString>& Options = TestSettings->StringOptions;
	Options = {"Red", "Green", "Blue", "Fuschia", "Orange"};

	FString& Selection = TestSettings->StringSelection;

	// Positive Validation
	for (int i = 0; i < Options.Num(); ++i)
	{
		Selection = Options[i];

		UTEST_TRUE("Integer test succeeded", ExecuteTest(TestSettings, OutputData));

		UTEST_TRUE("Integer selection is valid", ValidateOutput(OutputData, FName(Options[i])));
	}

	// Negative Validation
	Selection = "Square";

	UTEST_TRUE("String test succeeded", ExecuteTest(TestSettings, OutputData));

	UTEST_TRUE("String selection is invalid. Default path was selected", ValidateOutput(OutputData, DefaultPathPinLabel));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSwitchTest_Enum, FPCGSwitchTest, "Plugins.PCG.Switch.Enum", PCGTestsCommon::TestFlags)

bool FPCGSwitchTest_Enum::RunTest(const FString& Parameters)
{
	FPCGDataCollection OutputData;
	UPCGSwitchSettings* TestSettings = NewObject<UPCGSwitchSettings>();
	TestSettings->SelectionMode = EPCGControlFlowSelectionMode::Enum;

	TObjectPtr<UEnum>& EnumClass = TestSettings->EnumSelection.Class;
	EnumClass = StaticEnum<EPCGMetadataTypes>();

	const int32 NumEnums = EnumClass->NumEnums();

	// Positive Validation (also accounting for hidden metadata)
	for (uint8 i = 0; i < NumEnums - 1; ++i)
	{
		TestSettings->EnumSelection.Value = EnumClass->GetValueByIndex(i);

		UTEST_TRUE("Selection value is valid", TestSettings->EnumSelection.Value != INDEX_NONE);

		UTEST_TRUE("Enum test succeeded", ExecuteTest(TestSettings, OutputData));

		if (TestSettings->EnumSelection.Class->HasMetaData(TEXT("Hidden"), i) || TestSettings->EnumSelection.Class->HasMetaData(TEXT("Spacer"), i))
		{
			UTEST_TRUE("Hidden or spacer enum selection defaults output", ValidateOutput(OutputData, DefaultPathPinLabel));
		}
		else
		{
			UTEST_TRUE("Enum selection is valid", ValidateOutput(OutputData, FName(TestSettings->EnumSelection.GetCultureInvariantDisplayName())));
		}
	}

	// Negative Validation - Invalid selection value
	TestSettings->EnumSelection.Value = NumEnums;

	UTEST_TRUE("Enum test succeeded", ExecuteTest(TestSettings, OutputData));

	UTEST_TRUE("Invalid enum value defaults output", ValidateOutput(OutputData, DefaultPathPinLabel));

	// Negative Validation - Invalid enum
	EnumClass = nullptr;

	UTEST_TRUE("Enum test succeeded", ExecuteTest(TestSettings, OutputData));

	UTEST_TRUE("Invalid enum class defaults output", ValidateOutput(OutputData, DefaultPathPinLabel));

	return true;
}

#endif // WITH_EDITOR