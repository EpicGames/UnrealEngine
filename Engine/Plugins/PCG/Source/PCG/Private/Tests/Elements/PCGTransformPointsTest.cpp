// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGSpatialData.h"
#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGTransformPoints.h"
#include "Helpers/PCGHelpers.h"

#if WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGTransformPointsTest, FPCGTestBaseClass, "Plugins.PCG.TransformPoints.Basic", PCGTestsCommon::TestFlags)

bool FPCGTransformPointsTest::RunTest(const FString& Parameters)
{
	PCGTestsCommon::FTestData TestData;
	PCGTestsCommon::GenerateSettings<UPCGTransformPointsSettings>(TestData);
	UPCGTransformPointsSettings* Settings = CastChecked<UPCGTransformPointsSettings>(TestData.Settings);
	FPCGElementPtr TransformPointsElement = TestData.Settings->GetElement();

	TObjectPtr<UPCGBasePointData> EmptyData = PCGTestsCommon::CreateEmptyBasePointData();
	TObjectPtr<UPCGBasePointData> SimpleData = PCGTestsCommon::CreateEmptyBasePointData();
	TObjectPtr<UPCGBasePointData> ComplexData = PCGTestsCommon::CreateEmptyBasePointData();

	FRandomStream RandomSource(PCGHelpers::ComputeSeed(TestData.Seed));
	const int PointCount = 100;

	// SimplePoints are empty transforms
	SimpleData->SetNumPoints(PointCount);
	SimpleData->SetDensity(1);
	TPCGValueRange<int32> SimpleSeedRange = SimpleData->GetSeedValueRange();

	// ComplexPoints have unique location, rotation, and scale
	ComplexData->SetNumPoints(PointCount);
	ComplexData->SetDensity(1);
	TPCGValueRange<FTransform> ComplexTransformRange = ComplexData->GetTransformValueRange();
	TPCGValueRange<int32> ComplexSeedRange = ComplexData->GetSeedValueRange();

	for (int I = 0; I < PointCount; ++I)
	{
		SimpleSeedRange[I] = I;

		const FRotator RandomRotation = FRotator(RandomSource.FRandRange(0.f, 360.f));
		const FVector RandomLocation = RandomSource.VRand();
		const FVector RandomScale = RandomSource.VRand().GetAbs();

		ComplexSeedRange[I] = I;
		ComplexTransformRange[I] = FTransform(RandomRotation, RandomLocation, RandomScale);
	}

	FPCGTaggedData& EmptyTaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	EmptyTaggedData.Data = EmptyData;
	EmptyTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	FPCGTaggedData& SimpleTaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	SimpleTaggedData.Data = SimpleData;
	SimpleTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	FPCGTaggedData& ComplexTaggedData = TestData.InputData.TaggedData.Emplace_GetRef(FPCGTaggedData());
	ComplexTaggedData.Data = ComplexData;
	ComplexTaggedData.Pin = PCGPinConstants::DefaultInputLabel;

	// Basic parameters we will use for all tests
	Settings->OffsetMin = FVector(-1.f);
	Settings->OffsetMax = FVector(1.f);
	Settings->RotationMin = FRotator(-1.f);
	Settings->RotationMax = FRotator(1.f);
	Settings->ScaleMin = FVector(1.f);
	Settings->ScaleMax = FVector(2.f);

	auto ValidateTransformPoints = [this, &TestData, TransformPointsElement, Settings]() -> bool
	{
		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TransformPointsElement->Execute(Context.Get()))
		{}

		const TArray<FPCGTaggedData>& Inputs = Context->InputData.GetAllSpatialInputs();
		const TArray<FPCGTaggedData>& Outputs = Context->OutputData.GetAllSpatialInputs();

		if (!TestEqual("Valid number of outputs", Inputs.Num(), Outputs.Num()))
		{
			return false;
		}

		bool bTestPassed = true;

		for (int DataIndex = 0; DataIndex < Inputs.Num(); ++DataIndex)
		{
			const FPCGTaggedData& Input = Inputs[DataIndex];
			const FPCGTaggedData& Output = Outputs[DataIndex];

			check(Input.Data);
			
			const UPCGSpatialData* InSpatialData = Cast<UPCGSpatialData>(Input.Data);
			check(InSpatialData);

			const UPCGBasePointData* InPointData = InSpatialData->ToBasePointData(Context.Get());
			check(InPointData);

			if (!TestTrue("Valid output data", Output.Data != nullptr))
			{
				bTestPassed = false;
				continue;
			}

			const UPCGSpatialData* OutSpatialData = Cast<UPCGSpatialData>(Output.Data);

			if (!TestNotNull("Valid ouptut SpatialData", OutSpatialData))
			{
				bTestPassed = false;
				continue;
			}

			const UPCGBasePointData* OutPointData = OutSpatialData->ToBasePointData(Context.Get());

			if (!TestNotNull("Valid output PointData", OutPointData))
			{
				bTestPassed = false;
				continue;
			}

			if (!TestEqual("Input and output point counts match", InPointData->GetNumPoints(), OutPointData->GetNumPoints()))
			{ 
				bTestPassed = false;
				continue;
			}

			const FConstPCGPointValueRanges InRanges(InPointData);
			const FConstPCGPointValueRanges OutRanges(OutPointData);

			for (int PointIndex = 0; PointIndex < InPointData->GetNumPoints(); ++PointIndex)
			{
				const FPCGPoint InPoint = InRanges.GetPoint(PointIndex);
				const FPCGPoint OutPoint = OutRanges.GetPoint(PointIndex);

				FPCGPoint RepositionedPoint = InPoint;
				RepositionedPoint.Transform = OutPoint.Transform;
				bTestPassed &= TestTrue("RepositionedPoint and OutPoint are identical", PCGTestsCommon::PointsAreIdentical(RepositionedPoint, OutPoint));

				// Validate transform is within range of original transform
				const FTransform& InTransform = InPoint.Transform;
				const FTransform& OutTransform = OutPoint.Transform;

				FTransform AbsoluteTransform = OutTransform;
				AbsoluteTransform.SetLocation(OutTransform.GetLocation() - InTransform.GetLocation());

				FTransform RelativeTransform = OutTransform.GetRelativeTransform(InTransform);
				RelativeTransform.SetLocation(RelativeTransform.GetLocation() * InTransform.GetScale3D());

				const FTransform& TransformForLocation = (Settings->bAbsoluteOffset ? AbsoluteTransform : RelativeTransform);
				const FTransform& TransformForRotation = (Settings->bAbsoluteRotation ? AbsoluteTransform : RelativeTransform);
				const FTransform& TransformForScale = (Settings->bAbsoluteScale ? AbsoluteTransform : RelativeTransform);

				FVector OriginalOffset = TransformForLocation.GetLocation();
				FRotator OriginalRotation = TransformForRotation.GetRotation().Rotator();
				FVector OriginalScale = TransformForScale.GetScale3D();

				bTestPassed &= TestTrue("Valid location", OriginalOffset.X >= Settings->OffsetMin.X && OriginalOffset.X <= Settings->OffsetMax.X
					&& OriginalOffset.Y >= Settings->OffsetMin.Y && OriginalOffset.Y <= Settings->OffsetMax.Y
					&& OriginalOffset.Z >= Settings->OffsetMin.Z && OriginalOffset.Z <= Settings->OffsetMax.Z);

				bTestPassed &= TestTrue("Valid rotation", OriginalRotation.Pitch >= Settings->RotationMin.Pitch && OriginalRotation.Pitch <= Settings->RotationMax.Pitch
					&& OriginalRotation.Yaw >= Settings->RotationMin.Yaw && OriginalRotation.Yaw <= Settings->RotationMax.Yaw
					&& OriginalRotation.Roll >= Settings->RotationMin.Roll && OriginalRotation.Roll <= Settings->RotationMax.Roll);

				if (Settings->bUniformScale)
				{
					bTestPassed &= TestTrue("Valid scale", OriginalScale.X >= Settings->ScaleMin.X && OriginalScale.X <= Settings->ScaleMax.X
						&& OriginalScale.Y >= Settings->ScaleMin.X && OriginalScale.Y <= Settings->ScaleMax.X
						&& OriginalScale.Z >= Settings->ScaleMin.X && OriginalScale.Z <= Settings->ScaleMax.X);
				}
				else
				{
					bTestPassed &= TestTrue("Valid scale", OriginalScale.X >= Settings->ScaleMin.X && OriginalScale.X <= Settings->ScaleMax.X
						&& OriginalScale.Y >= Settings->ScaleMin.Y && OriginalScale.Y <= Settings->ScaleMax.Y
						&& OriginalScale.Z >= Settings->ScaleMin.Z && OriginalScale.Z <= Settings->ScaleMax.Z);
				}

				if (Settings->bRecomputeSeed)
				{
					const FVector& Position = OutTransform.GetLocation();
					bTestPassed &= TestEqual("Valid seed", OutPoint.Seed, PCGHelpers::ComputeSeed((int)Position.X, (int)Position.Y, (int)Position.Z));
				}
				else
				{
					bTestPassed &= TestEqual("Valid seed", OutPoint.Seed, InPoint.Seed);
				}
			}
		}

		return bTestPassed;
	};

	bool bTestPassed = true;

	// Test 1 - absolute transformations
	Settings->bAbsoluteOffset = true;
	Settings->bAbsoluteRotation = true;
	Settings->bAbsoluteScale = true;
	bTestPassed &= ValidateTransformPoints();

	// Test 2 - relative transformations
	Settings->bAbsoluteOffset = false;
	Settings->bAbsoluteRotation = false;
	Settings->bAbsoluteScale = false;
	bTestPassed &= ValidateTransformPoints();

	// Test 3 - uniform scale and recompute seed
	Settings->bUniformScale = true;
	Settings->bRecomputeSeed = true;
	bTestPassed &= ValidateTransformPoints();

	return bTestPassed;
}

#endif // WITH_EDITOR
