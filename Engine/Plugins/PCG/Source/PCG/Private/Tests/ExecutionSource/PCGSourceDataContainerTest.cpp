// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/PCGSourceDataContainer.h"
#include "Tests/PCGTestsCommon.h"

#if WITH_EDITOR

// -------------------------------------------------------------------
// Store, Get, GetMutable, Remove — basic editor API
// -------------------------------------------------------------------

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSourceDataContainerStoreAndGetTest, FPCGTestBaseClass, "Plugins.PCG.SourceDataContainer.StoreAndGet", PCGTestsCommon::TestFlags)

bool FPCGSourceDataContainerStoreAndGetTest::RunTest(const FString& Parameters)
{
	const FPCGSourceDataStorageKey KeyA(TEXT("LabelA"), 42);
	const FPCGSourceDataStorageKey KeyB(TEXT("LabelB"), 99);
	const FPCGSourceDataStorageKey PayloadA(TEXT("PayloadA"), 100);
	const FPCGSourceDataStorageKey PayloadB(TEXT("PayloadB"), 200);

	FPCGSourceDataContainer Container;

	// Store two entries
	Container.Store<FPCGSourceDataStorageKey>(KeyA, PayloadA);
	Container.Store<FPCGSourceDataStorageKey>(KeyB, PayloadB);

	UTEST_EQUAL("Num after two stores", Container.Num(), 2);
	UTEST_FALSE("Not empty", Container.IsEmpty());

	// Retrieve first entry
	{
		const FConstSharedStruct ResultA = Container.Get<FPCGSourceDataStorageKey>(KeyA);
		UTEST_TRUE("KeyA result valid", ResultA.IsValid());

		const FPCGSourceDataStorageKey* PtrA = ResultA.GetPtr<FPCGSourceDataStorageKey>();
		UTEST_NOT_NULL("KeyA ptr not null", PtrA);
		UTEST_TRUE("KeyA payload matches", *PtrA == PayloadA);
	}

	// Retrieve second entry
	{
		const FConstSharedStruct ResultB = Container.Get<FPCGSourceDataStorageKey>(KeyB);
		UTEST_TRUE("KeyB result valid", ResultB.IsValid());

		const FPCGSourceDataStorageKey* PtrB = ResultB.GetPtr<FPCGSourceDataStorageKey>();
		UTEST_NOT_NULL("KeyB ptr not null", PtrB);
		UTEST_TRUE("KeyB payload matches", *PtrB == PayloadB);
	}

	return true;
}

// -------------------------------------------------------------------
// Store overwrites existing key
// -------------------------------------------------------------------

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSourceDataContainerStoreOverwriteTest, FPCGTestBaseClass, "Plugins.PCG.SourceDataContainer.StoreOverwrite", PCGTestsCommon::TestFlags)

bool FPCGSourceDataContainerStoreOverwriteTest::RunTest(const FString& Parameters)
{
	const FPCGSourceDataStorageKey Key(TEXT("Label"), 42);
	const FPCGSourceDataStorageKey OriginalPayload(TEXT("Original"), 1);
	const FPCGSourceDataStorageKey ReplacementPayload(TEXT("Replacement"), 2);

	FPCGSourceDataContainer Container;
	Container.Store<FPCGSourceDataStorageKey>(Key, OriginalPayload);
	Container.Store<FPCGSourceDataStorageKey>(Key, ReplacementPayload);

	UTEST_EQUAL("Num stays 1 after overwrite", Container.Num(), 1);

	const FConstSharedStruct Result = Container.Get<FPCGSourceDataStorageKey>(Key);
	UTEST_TRUE("Result valid after overwrite", Result.IsValid());

	const FPCGSourceDataStorageKey* Ptr = Result.GetPtr<FPCGSourceDataStorageKey>();
	UTEST_NOT_NULL("Ptr not null", Ptr);
	UTEST_TRUE("Payload is replacement, not original", *Ptr == ReplacementPayload);

	return true;
}

// -------------------------------------------------------------------
// GetMutable — mutation persists
// -------------------------------------------------------------------

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSourceDataContainerGetMutableTest, FPCGTestBaseClass, "Plugins.PCG.SourceDataContainer.GetMutable", PCGTestsCommon::TestFlags)

bool FPCGSourceDataContainerGetMutableTest::RunTest(const FString& Parameters)
{
	const FPCGSourceDataStorageKey Key(TEXT("MutableTest"), 42);
	const FPCGSourceDataStorageKey InitialPayload(TEXT("Initial"), 1);
	const FPCGSourceDataStorageKey MutatedPayload(TEXT("Mutated"), 2);

	FPCGSourceDataContainer Container;
	Container.Store<FPCGSourceDataStorageKey>(Key, InitialPayload);

	// Mutate via GetMutable
	{
		FSharedStruct MutableResult = Container.GetMutable<FPCGSourceDataStorageKey>(Key);
		UTEST_TRUE("Mutable result valid", MutableResult.IsValid());

		FPCGSourceDataStorageKey* MutablePtr = MutableResult.GetPtr<FPCGSourceDataStorageKey>();
		UTEST_NOT_NULL("Mutable ptr not null", MutablePtr);
		*MutablePtr = MutatedPayload;
	}

	// Verify mutation persisted
	{
		const FConstSharedStruct ConstResult = Container.Get<FPCGSourceDataStorageKey>(Key);
		UTEST_TRUE("Const result valid after mutation", ConstResult.IsValid());

		const FPCGSourceDataStorageKey* ConstPtr = ConstResult.GetPtr<FPCGSourceDataStorageKey>();
		UTEST_NOT_NULL("Const ptr not null", ConstPtr);
		UTEST_TRUE("Payload reflects mutation", *ConstPtr == MutatedPayload);
	}

	// GetMutable on non-existent key returns invalid
	{
		const FPCGSourceDataStorageKey MissingKey(TEXT("Missing"), 0);
		const FSharedStruct MissingResult = Container.GetMutable<FPCGSourceDataStorageKey>(MissingKey);
		UTEST_FALSE("GetMutable on missing key returns invalid", MissingResult.IsValid());
	}

	return true;
}

// -------------------------------------------------------------------
// Remove
// -------------------------------------------------------------------

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSourceDataContainerRemoveTest, FPCGTestBaseClass, "Plugins.PCG.SourceDataContainer.Remove", PCGTestsCommon::TestFlags)

bool FPCGSourceDataContainerRemoveTest::RunTest(const FString& Parameters)
{
	const FPCGSourceDataStorageKey Key(TEXT("RemoveMe"), 42);
	const FPCGSourceDataStorageKey Payload(TEXT("Payload"), 1);

	FPCGSourceDataContainer Container;
	Container.Store<FPCGSourceDataStorageKey>(Key, Payload);

	UTEST_EQUAL("Num before remove", Container.Num(), 1);

	const bool bRemoved = Container.Remove(Key);
	UTEST_TRUE("Remove returns true for existing key", bRemoved);
	UTEST_EQUAL("Num after remove", Container.Num(), 0);
	UTEST_TRUE("Empty after remove", Container.IsEmpty());

	// Get after removal returns invalid
	const FConstSharedStruct RemovedResult = Container.Get<FPCGSourceDataStorageKey>(Key);
	UTEST_FALSE("Get after remove returns invalid", RemovedResult.IsValid());

	// Removing again returns false
	const bool bRemovedAgain = Container.Remove(Key);
	UTEST_FALSE("Remove non-existent returns false", bRemovedAgain);

	return true;
}

// -------------------------------------------------------------------
// MarkDirty
// -------------------------------------------------------------------

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSourceDataContainerMarkDirtyTest, FPCGTestBaseClass, "Plugins.PCG.SourceDataContainer.MarkDirty", PCGTestsCommon::TestFlags)

bool FPCGSourceDataContainerMarkDirtyTest::RunTest(const FString& Parameters)
{
	FPCGSourceDataContainer Container;

	UTEST_EQUAL("Initial generation is 0", Container.GetDirtyGeneration(), 0u);

	Container.MarkDirty();
	UTEST_EQUAL("Generation after first MarkDirty", Container.GetDirtyGeneration(), 1u);

	Container.MarkDirty();
	UTEST_EQUAL("Generation after second MarkDirty", Container.GetDirtyGeneration(), 2u);

	return true;
}

// -------------------------------------------------------------------
// AutoDirty — Store and Remove increment generation
// -------------------------------------------------------------------

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSourceDataContainerAutoDirtyTest, FPCGTestBaseClass, "Plugins.PCG.SourceDataContainer.AutoDirty", PCGTestsCommon::TestFlags)

bool FPCGSourceDataContainerAutoDirtyTest::RunTest(const FString& Parameters)
{
	const FPCGSourceDataStorageKey Key(TEXT("AutoDirty"), 42);
	const FPCGSourceDataStorageKey Payload(TEXT("Payload"), 1);

	FPCGSourceDataContainer Container;

	// Without auto-dirty: Store does NOT increment generation
	Container.Store<FPCGSourceDataStorageKey>(Key, Payload);
	UTEST_EQUAL("Generation unchanged without auto-dirty", Container.GetDirtyGeneration(), 0u);

	Container.Remove(Key);
	UTEST_EQUAL("Generation unchanged on remove without auto-dirty", Container.GetDirtyGeneration(), 0u);

	// Enable auto-dirty
	Container.SetShouldAutoDirty(true);

	Container.Store<FPCGSourceDataStorageKey>(Key, Payload);
	UTEST_EQUAL("Generation incremented on Store with auto-dirty", Container.GetDirtyGeneration(), 1u);

	// Store overwrites — still increments
	Container.Store<FPCGSourceDataStorageKey>(Key, Payload);
	UTEST_EQUAL("Generation incremented on overwrite with auto-dirty", Container.GetDirtyGeneration(), 2u);

	// Remove with auto-dirty
	Container.Remove(Key);
	UTEST_EQUAL("Generation incremented on Remove with auto-dirty", Container.GetDirtyGeneration(), 3u);

	// Remove non-existent does NOT increment
	Container.Remove(Key);
	UTEST_EQUAL("Generation unchanged on remove of non-existent with auto-dirty", Container.GetDirtyGeneration(), 3u);

	return true;
}

// -------------------------------------------------------------------
// Empty
// -------------------------------------------------------------------

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSourceDataContainerEmptyTest, FPCGTestBaseClass, "Plugins.PCG.SourceDataContainer.Empty", PCGTestsCommon::TestFlags)

bool FPCGSourceDataContainerEmptyTest::RunTest(const FString& Parameters)
{
	const FPCGSourceDataStorageKey KeyA(TEXT("A"), 1);
	const FPCGSourceDataStorageKey KeyB(TEXT("B"), 2);
	const FPCGSourceDataStorageKey Payload(TEXT("P"), 0);

	FPCGSourceDataContainer Container;
	Container.SetShouldAutoDirty(true);
	Container.Store<FPCGSourceDataStorageKey>(KeyA, Payload);
	Container.Store<FPCGSourceDataStorageKey>(KeyB, Payload);

	UTEST_EQUAL("Num before Empty", Container.Num(), 2);
	UTEST_EQUAL("Generation before Empty", Container.GetDirtyGeneration(), 2u);

	Container.Empty();

	UTEST_TRUE("IsEmpty after Empty()", Container.IsEmpty());
	UTEST_EQUAL("Num after Empty()", Container.Num(), 0);
	UTEST_EQUAL("Generation reset to 0 after Empty()", Container.GetDirtyGeneration(), 0u);

	return true;
}

// -------------------------------------------------------------------
// Type mismatch — GetAs and GetMutable with wrong type
// -------------------------------------------------------------------

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSourceDataContainerTypeMismatchTest, FPCGTestBaseClass, "Plugins.PCG.SourceDataContainer.TypeMismatch", PCGTestsCommon::TestFlags)

bool FPCGSourceDataContainerTypeMismatchTest::RunTest(const FString& Parameters)
{
	const FPCGSourceDataStorageKey Key(TEXT("Typed"), 42);
	const FPCGSourceDataStorageKey Payload(TEXT("Payload"), 1);

	FPCGSourceDataContainer Container;
	Container.Store<FPCGSourceDataStorageKey>(Key, Payload);

	AddExpectedError(TEXT("type mismatch"), EAutomationExpectedErrorFlags::Contains, 1);
	const FConstSharedStruct WrongTypeResult = Container.Get<FPCGSourceDataStorageValue>(Key);
	UTEST_FALSE("Get with wrong type returns invalid", WrongTypeResult.IsValid());

	AddExpectedError(TEXT("type mismatch"), EAutomationExpectedErrorFlags::Contains, 1);
	const FSharedStruct WrongTypeMutableResult = Container.GetMutable<FPCGSourceDataStorageValue>(Key);
	UTEST_FALSE("GetMutable with wrong type returns invalid", WrongTypeMutableResult.IsValid());

	return true;
}

// -------------------------------------------------------------------
// Copy semantics
// -------------------------------------------------------------------

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSourceDataContainerCopyTest, FPCGTestBaseClass, "Plugins.PCG.SourceDataContainer.CopySemantics", PCGTestsCommon::TestFlags)

bool FPCGSourceDataContainerCopyTest::RunTest(const FString& Parameters)
{
	const FPCGSourceDataStorageKey Key(TEXT("CopyTest"), 42);
	const FPCGSourceDataStorageKey Payload(TEXT("Payload"), 1);

	FPCGSourceDataContainer Original;
	Original.SetShouldAutoDirty(true);
	Original.Store<FPCGSourceDataStorageKey>(Key, Payload);

	// Copy construction
	{
		FPCGSourceDataContainer Copy(Original);

		UTEST_EQUAL("Copy num matches original", Copy.Num(), Original.Num());
		UTEST_EQUAL("Copy dirty generation matches original", Copy.GetDirtyGeneration(), Original.GetDirtyGeneration());

		const FConstSharedStruct CopyResult = Copy.Get<FPCGSourceDataStorageKey>(Key);
		UTEST_TRUE("Copy Get result valid", CopyResult.IsValid());

		const FPCGSourceDataStorageKey* CopyPtr = CopyResult.GetPtr<FPCGSourceDataStorageKey>();
		UTEST_NOT_NULL("Copy ptr not null", CopyPtr);
		UTEST_TRUE("Copy payload matches", *CopyPtr == Payload);
	}

	// Copy assignment
	{
		FPCGSourceDataContainer Assigned;
		Assigned = Original;

		UTEST_EQUAL("Assigned num matches original", Assigned.Num(), Original.Num());
		UTEST_EQUAL("Assigned dirty generation matches", Assigned.GetDirtyGeneration(), Original.GetDirtyGeneration());

		const FConstSharedStruct AssignedResult = Assigned.Get<FPCGSourceDataStorageKey>(Key);
		UTEST_TRUE("Assigned Get result valid", AssignedResult.IsValid());
	}

	return true;
}

// -------------------------------------------------------------------
// Move semantics
// -------------------------------------------------------------------

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGSourceDataContainerMoveTest, FPCGTestBaseClass, "Plugins.PCG.SourceDataContainer.MoveSemantics", PCGTestsCommon::TestFlags)

bool FPCGSourceDataContainerMoveTest::RunTest(const FString& Parameters)
{
	const FPCGSourceDataStorageKey Key(TEXT("MoveTest"), 42);
	const FPCGSourceDataStorageKey Payload(TEXT("Payload"), 1);

	// Move construction
	{
		FPCGSourceDataContainer Source;
		Source.SetShouldAutoDirty(true);
		Source.Store<FPCGSourceDataStorageKey>(Key, Payload);
		const uint32 OriginalGeneration = Source.GetDirtyGeneration();

		FPCGSourceDataContainer Moved(MoveTemp(Source));

		UTEST_EQUAL("Moved-to has correct num", Moved.Num(), 1);
		UTEST_EQUAL("Moved-to has correct generation", Moved.GetDirtyGeneration(), OriginalGeneration);

		const FConstSharedStruct MovedResult = Moved.Get<FPCGSourceDataStorageKey>(Key);
		UTEST_TRUE("Moved-to Get result valid", MovedResult.IsValid());

		// Source is reset after move
		UTEST_TRUE("Source empty after move", Source.IsEmpty());
		UTEST_EQUAL("Source generation reset after move", Source.GetDirtyGeneration(), 0u);
	}

	// Move assignment
	{
		FPCGSourceDataContainer Source;
		Source.SetShouldAutoDirty(true);
		Source.Store<FPCGSourceDataStorageKey>(Key, Payload);

		FPCGSourceDataContainer Assigned;
		Assigned = MoveTemp(Source);

		UTEST_EQUAL("Move-assigned has correct num", Assigned.Num(), 1);
		UTEST_TRUE("Source empty after move assignment", Source.IsEmpty());
		UTEST_EQUAL("Source generation reset after move assignment", Source.GetDirtyGeneration(), 0u);
	}

	// Self move-assignment is safe
	{
		FPCGSourceDataContainer SelfMove;
		SelfMove.Store<FPCGSourceDataStorageKey>(Key, Payload);
		SelfMove = MoveTemp(SelfMove);

		UTEST_EQUAL("Self move preserves num", SelfMove.Num(), 1);
	}

	return true;
}

#endif // WITH_EDITOR
