// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/GraphAuthoring/PCGGraphAuthoringTestHelperSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGraphAuthoringTestHelperSettings)

UPCGGraphAuthoringTestHelperSettings::UPCGGraphAuthoringTestHelperSettings()
{
#if WITH_EDITORONLY_DATA
	bExposeToLibrary = false;
#endif
}

TArray<FPCGPinProperties> UPCGGraphAuthoringTestHelperSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, PinType);

	return PinProperties;
}

TArray<FPCGPinProperties> UPCGGraphAuthoringTestHelperSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, PinType);

	return PinProperties;
}
