// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGSettings.h"

#include "PCGGraphAuthoringTestHelperSettings.generated.h"

/** Testing helper - generates a node with a single input and output pin of the stipulated type. */
UCLASS(NotBlueprintable, NotBlueprintType, ClassGroup = (Procedural))
class UPCGGraphAuthoringTestHelperSettings : public UPCGSettings
{
	GENERATED_BODY()

	UPCGGraphAuthoringTestHelperSettings();
	
protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	// Does not support execution currently, testing purposes only.
	virtual FPCGElementPtr CreateElement() const override { return nullptr; }

public:
	EPCGDataType PinType = EPCGDataType::None;
};
