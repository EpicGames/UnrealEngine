// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PCGComponent.h"
#include "Graph/PCGGraphExecutor.h"
#include "Tests/PCGTestsCommon.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphExecutorTests, FPCGTestBaseClass, "Plugins.PCG.GraphExecutor.Tests", PCGTestsCommon::TestFlags | EAutomationTestFlags::Disabled)

bool FPCGGraphExecutorTests::RunTest(const FString& Parameters)
{
	bool bTestPassed = true;

	constexpr int ComponentsNum = 3;

	TArray<PCGTestsCommon::FTestData> TestData;
	TestData.SetNum(ComponentsNum);

	TSharedPtr<FPCGGraphExecutor> Executor = MakeShared<FPCGGraphExecutor>();
	TArray<TArray<FPCGTaskId>> Tasks;
	TArray<TArray<bool>> bExecuted;
	TArray<TArray<bool>> bAborted;

	auto AddTasks = [&](int TasksPerComponent)
	{
		Tasks.SetNum(ComponentsNum);
		bExecuted.SetNum(ComponentsNum);
		bAborted.SetNum(ComponentsNum);

		for (int ComponentIndex = 0; ComponentIndex < ComponentsNum; ++ComponentIndex)
		{
			TArray<FPCGTaskId>& ComponentTasks = Tasks[ComponentIndex];
			ComponentTasks.SetNum(TasksPerComponent);
			bExecuted[ComponentIndex].Init(false, TasksPerComponent);
			bAborted[ComponentIndex].Init(false, TasksPerComponent);

			for (int TaskIndex = 0; TaskIndex < TasksPerComponent; ++TaskIndex)
			{
				TArray<FPCGTaskId> PreviousTasks;

				if (ComponentIndex != 0)
				{
					PreviousTasks.Add(Tasks[ComponentIndex - 1][TaskIndex]);
				}

				if (TaskIndex != 0)
				{
					PreviousTasks.Add(Tasks[ComponentIndex][TaskIndex - 1]);
				}

				ComponentTasks[TaskIndex] = Executor->ScheduleGeneric(
					[&bExecuted, ComponentIndex, TaskIndex]()
					{
						bExecuted[ComponentIndex][TaskIndex] = true;
						return true;
					},
					[&bAborted, ComponentIndex, TaskIndex]()
					{
						bAborted[ComponentIndex][TaskIndex] = true;
					}, TestData[ComponentIndex].TestPCGComponent, PreviousTasks);
			}
		}
	};

	auto ExecuteNTimes = [&Executor](int NTimes)
	{
		for (int i = 0; i < NTimes; ++i)
		{
			Executor->Execute();
		}
	};

	auto ExecuteAll = [&ExecuteNTimes]()
	{
		ExecuteNTimes(1024);
	};

	// First test: schedule all three tasks, and cancel them, checking that the appropriate number is cancelled.
	{
		AddTasks(1);
		Executor->Cancel(TestData[0].TestPCGComponent);
		ExecuteAll();

		bTestPassed &= TestTrue(TEXT("Expect that no tasks are executed"), !bExecuted[0][0] && !bExecuted[1][0] && !bExecuted[2][0]);
		bTestPassed &= TestTrue(TEXT("Expect that all tasks are aborted"), bAborted[0][0] && bAborted[1][0] && bAborted[2][0]);

		AddTasks(1);
		Executor->Cancel(TestData[1].TestPCGComponent);
		ExecuteAll();

		bTestPassed &= TestTrue(TEXT("Expect that only task from A is executed"), bExecuted[0][0] && !bExecuted[1][0] && !bExecuted[2][0]);
		bTestPassed &= TestTrue(TEXT("Expect that only task from A is not aborted"), !bAborted[0][0] && bAborted[1][0] && bAborted[2][0]);

		AddTasks(1);
		Executor->Cancel(TestData[2].TestPCGComponent);
		ExecuteAll();

		bTestPassed &= TestTrue(TEXT("Expect that only task from C is not executed"), bExecuted[0][0] && bExecuted[1][0] && !bExecuted[2][0]);
		bTestPassed &= TestTrue(TEXT("Expect that onlt task from C is aborted"), !bAborted[0][0] && !bAborted[1][0] && bAborted[2][0]);

		AddTasks(1);
		ExecuteAll();

		bTestPassed &= TestTrue(TEXT("Expect that all tasks are executed"), bExecuted[0][0] && bExecuted[1][0] && bExecuted[2][0]);
		bTestPassed &= TestTrue(TEXT("Expect that no tasks are aborted"), !bAborted[0][0] && !bAborted[1][0] && !bAborted[2][0]);
	}

	// Set editor frame time to zero so we can accurately control execution
	bool PreviousWasMultithreaded = PCGGraphExecutor::CVarGraphMultithreading.GetValueOnAnyThread();
	PCGGraphExecutor::CVarGraphMultithreading->Set(false);
	float PreviousEditorFrameTime = PCGGraphExecutor::CVarEditorTimePerFrame.GetValueOnAnyThread();
	PCGGraphExecutor::CVarEditorTimePerFrame->Set(0.0f);

	// Second test: execute partially, then cancel
	{
		const int TasksPerComponent = 3;
		const int ExecutionPhasesCount = 4;

		for (int TaskIndex = 0; TaskIndex <= ComponentsNum * TasksPerComponent * ExecutionPhasesCount; ++TaskIndex)
		{
			int ComponentIndex = TaskIndex / (TasksPerComponent * ExecutionPhasesCount);
			int ComponentTaskIndex = (TaskIndex / ExecutionPhasesCount) % TasksPerComponent;
			int ExecutionPhaseIndex = TaskIndex % ExecutionPhasesCount;

			AddTasks(TasksPerComponent);
			ExecuteNTimes(TaskIndex);

			FPCGTaskId FirstTaskCancelled = InvalidPCGTaskId;

			if (ComponentIndex < ComponentsNum)
			{
				FirstTaskCancelled = Tasks[ComponentIndex][ComponentTaskIndex];
				Executor->Cancel(TestData[ComponentIndex].TestPCGComponent);
			}

			// Finish execution
			ExecuteAll();

			// Perform validation
			bool bTestIterationPassed = true;
			for (int ValidationComponentIndex = 0; ValidationComponentIndex < ComponentsNum; ++ValidationComponentIndex)
			{
				for (int ValidationTaskIndex = 0; ValidationTaskIndex < TasksPerComponent; ++ValidationTaskIndex)
				{
					bool bShouldBeExecuted;
					bool bShouldBeAborted;

					if (Tasks[ValidationComponentIndex][ValidationTaskIndex] == FirstTaskCancelled)
 					{
						bShouldBeExecuted = (ExecutionPhaseIndex == 3);
						bShouldBeAborted = true;
					}
					else
					{
						bShouldBeExecuted = Tasks[ValidationComponentIndex][ValidationTaskIndex] < FirstTaskCancelled;
						bShouldBeAborted = Tasks[ValidationComponentIndex][ValidationTaskIndex] >= FirstTaskCancelled;
					}

					bTestIterationPassed &= (bExecuted[ValidationComponentIndex][ValidationTaskIndex] == bShouldBeExecuted);
					bTestIterationPassed &= (bAborted[ValidationComponentIndex][ValidationTaskIndex] == bShouldBeAborted);
				}
			}

			bTestPassed &= TestTrue(FString::Printf(TEXT("Iteration %d failed"), TaskIndex), bTestIterationPassed);
		}
	}

	PCGGraphExecutor::CVarGraphMultithreading->Set(PreviousWasMultithreaded);
	PCGGraphExecutor::CVarEditorTimePerFrame->Set(PreviousEditorFrameTime);

	return bTestPassed;
}

#endif // WITH_EDITOR