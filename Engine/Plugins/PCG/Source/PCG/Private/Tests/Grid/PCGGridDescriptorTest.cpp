// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Grid/PCGGridDescriptor.h"
#include "Tests/PCGTestsCommon.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGridDescriptorTests, FPCGTestBaseClass, "Plugins.PCG.GridDescriptor.Tests", PCGTestsCommon::TestFlags)

bool FPCGGridDescriptorTests::RunTest(const FString& Parameters)
{
	TSet<FPCGGridDescriptor> GridDescriptors;

	FPCGGridDescriptor DefaultGridDescriptor;
	UTEST_TRUE_EXPR(!GridDescriptors.Contains(DefaultGridDescriptor));

	GridDescriptors.Add(DefaultGridDescriptor);
	UTEST_TRUE_EXPR(GridDescriptors.Contains(DefaultGridDescriptor));
	
	FPCGGridDescriptor OtherDefaultDescriptor;
	
	UTEST_TRUE_EXPR(DefaultGridDescriptor == OtherDefaultDescriptor);
	UTEST_TRUE_EXPR(GridDescriptors.Contains(OtherDefaultDescriptor));
		
	GridDescriptors.Remove(OtherDefaultDescriptor);
	UTEST_TRUE_EXPR(GridDescriptors.IsEmpty());

	FPCGGridDescriptor Grid_25600_2D_NotRuntime = FPCGGridDescriptor().SetGridSize(25600).SetIs2DGrid(true).SetIsRuntime(false);
	UTEST_TRUE_EXPR(Grid_25600_2D_NotRuntime.GetGridSize() == 25600);
	UTEST_TRUE_EXPR(Grid_25600_2D_NotRuntime.Is2DGrid());
	UTEST_TRUE_EXPR(!Grid_25600_2D_NotRuntime.IsRuntime());

	UTEST_TRUE_EXPR(!GridDescriptors.Contains(Grid_25600_2D_NotRuntime));
	GridDescriptors.Add(Grid_25600_2D_NotRuntime);
	UTEST_TRUE_EXPR(GridDescriptors.Contains(Grid_25600_2D_NotRuntime));
	
	FPCGGridDescriptor Grid_3200_3D_Runtime = FPCGGridDescriptor().SetGridSize(3200).SetIs2DGrid(false).SetIsRuntime(true);
	UTEST_TRUE_EXPR(Grid_3200_3D_Runtime.GetGridSize() == 3200);
	UTEST_TRUE_EXPR(!Grid_3200_3D_Runtime.Is2DGrid());
	UTEST_TRUE_EXPR(Grid_3200_3D_Runtime.IsRuntime());

	UTEST_TRUE_EXPR(!GridDescriptors.Contains(Grid_3200_3D_Runtime));
	GridDescriptors.Add(Grid_3200_3D_Runtime);
	UTEST_TRUE_EXPR(GridDescriptors.Contains(Grid_3200_3D_Runtime));

	UTEST_TRUE_EXPR(GridDescriptors.Num() == 2);

	GridDescriptors.Remove(Grid_3200_3D_Runtime);
	UTEST_TRUE_EXPR(GridDescriptors.Num() == 1);

	UTEST_TRUE_EXPR(GridDescriptors.Contains(Grid_25600_2D_NotRuntime));
	GridDescriptors.Remove(Grid_25600_2D_NotRuntime);
	UTEST_TRUE_EXPR(GridDescriptors.IsEmpty());


	TSet<FPCGGridCellDescriptor> GridCellDescriptors;

	auto AddToGrid = [&GridCellDescriptors, this](FPCGGridCellDescriptor GridCellDescriptor) -> bool
	{
		UTEST_TRUE_EXPR(!GridCellDescriptors.Contains(GridCellDescriptor));
		GridCellDescriptors.Add(GridCellDescriptor);
		UTEST_TRUE_EXPR(GridCellDescriptors.Contains(GridCellDescriptor));
		return true;
	};

	auto RemoveFromGrid = [&GridCellDescriptors, this](FPCGGridCellDescriptor GridCellDescriptor) -> bool
	{
		UTEST_TRUE_EXPR(GridCellDescriptors.Contains(GridCellDescriptor));
		GridCellDescriptors.Remove(GridCellDescriptor);
		UTEST_TRUE_EXPR(!GridCellDescriptors.Contains(GridCellDescriptor));
		return true;
	};

	auto IterateGrid = [](int32 MinX, int32 SizeX, int32 MinY, int32 SizeY, int32 MinZ, int32 SizeZ, FPCGGridDescriptor GridDescriptor, TFunctionRef<bool(FPCGGridCellDescriptor)> Func) -> bool
	{
		MinZ = GridDescriptor.Is2DGrid() ? 0 : MinZ;
		SizeZ = GridDescriptor.Is2DGrid() ? 1 : SizeZ;
		for(int32 x = MinX; x < MinX+SizeX; ++x)
		{
			for (int32 y = MinY; y < MinY+SizeY; ++y)
			{
				for (int32 z = MinZ; z < MinZ+SizeZ; ++z)
				{
					FPCGGridCellDescriptor CellDescriptor(GridDescriptor, FIntVector(x, y, z));
					if (!Func(CellDescriptor))
					{
						return false;
					}
				}
			}
		}

		return true;
	};
	
	constexpr int32 MinX = -5;
	constexpr int32 SizeX = 12;
	constexpr int32 MinY = -10;
	constexpr int32 SizeY = 9;
	constexpr int32 MinZ = 2;
	constexpr int32 SizeZ = 17;

	if (!IterateGrid(MinX, SizeX, MinY, SizeY, MinZ, SizeZ, Grid_3200_3D_Runtime, AddToGrid))
	{
		return false;
	}

	int32 TotalCells = SizeX * SizeY * SizeZ;
	
	UTEST_TRUE_EXPR(GridCellDescriptors.Num() == TotalCells);
	
	if (!IterateGrid(MinX, SizeX, MinY, SizeY, MinZ, SizeZ, Grid_25600_2D_NotRuntime, AddToGrid))
	{
		return false;
	}

	TotalCells += (SizeX * SizeY);
	
	UTEST_TRUE_EXPR(GridCellDescriptors.Num() == TotalCells);

	if (!IterateGrid(MinX, SizeX, MinY, SizeY, MinZ, SizeZ, Grid_3200_3D_Runtime, RemoveFromGrid))
	{
		return false;
	}

	TotalCells -= (SizeX * SizeY * SizeZ);
	
	UTEST_TRUE_EXPR(GridCellDescriptors.Num() == TotalCells);
	
	if (!IterateGrid(MinX, SizeX, MinY, SizeY, MinZ, SizeZ, Grid_25600_2D_NotRuntime, RemoveFromGrid))
	{
		return false;
	}

	UTEST_TRUE_EXPR(GridCellDescriptors.IsEmpty());

	return true;
}

#endif // WITH_EDITOR