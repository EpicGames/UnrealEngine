// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"

#include "Helpers/PCGAsync.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAsyncTest_AsyncProcessing, FPCGTestBaseClass, "Plugins.PCG.Async.AsyncProcessing", PCGTestsCommon::TestFlags)

bool FPCGAsyncTest_AsyncProcessing::RunTest(const FString& Parameters)
{
	FPCGAsyncState AsyncState;
	AsyncState.NumAvailableTasks = 10;

	TArray<double> TestOutput;
	const bool bIsDone = FPCGAsync::AsyncProcessing<double>(
		&AsyncState,
		6,
		TestOutput,
		[Value = MakeUnique<int32>(42)](int32 Index, double& Output)
		{
			if ((Index % 2) == 0)
			{
				return false; // remove all even
			}

			Output = double(Index * (*Value));

			return true;
		},
		/*bEnableTimeSlicing*/ false,
		/*ChunkSize*/ 1 // force this to thread 
	);

	UTEST_TRUE("IsDone", bIsDone);

	UTEST_EQUAL("Wrote Correct Number", TestOutput.Num(), 3);

	UTEST_EQUAL("TestOutput[0]", TestOutput[0], 42.0 * 1);
	UTEST_EQUAL("TestOutput[1]", TestOutput[1], 42.0 * 3);
	UTEST_EQUAL("TestOutput[2]", TestOutput[2], 42.0 * 5);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAsyncTest_AsyncProcessingEx, FPCGTestBaseClass, "Plugins.PCG.Async.AsyncProcessingEx", PCGTestsCommon::TestFlags)

bool FPCGAsyncTest_AsyncProcessingEx::RunTest(const FString& Parameters)
{
	FPCGAsyncState AsyncState;
	AsyncState.NumAvailableTasks = 10;

	TArray<double> TestOutput;
	const bool bIsDone = FPCGAsync::AsyncProcessingEx(
		&AsyncState,
		6,
 		[&TestOutput]()
		{
			TestOutput.SetNumUninitialized(6); 
		},
		[Value = MakeUnique<int32>(42), &TestOutput](int32 ReadIndex, int32 WriteIndex)
		{
			if ((ReadIndex % 2) == 0)
			{
				return false; // remove all even
			}

			TestOutput[WriteIndex] = double(ReadIndex * (*Value));

			return true;
		},
		[&TestOutput](int32 ReadIndex, int32 WriteIndex)
		{
			TestOutput[WriteIndex] = TestOutput[ReadIndex];
		},
		[&TestOutput](int32 Count)
		{
			TestOutput.SetNum(Count);
		},
		/*bEnableTimeSlicing*/ false,
		/*ChunkSize*/ 1 // force this to thrad 
	);

	UTEST_TRUE("IsDone", bIsDone);

	UTEST_EQUAL("Wrote Correct Number", TestOutput.Num(), 3);

	UTEST_EQUAL("TestOutput[0]", TestOutput[0], 42.0 * 1);
	UTEST_EQUAL("TestOutput[1]", TestOutput[1], 42.0 * 3);
	UTEST_EQUAL("TestOutput[2]", TestOutput[2], 42.0 * 5);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGAsyncTest_AsyncProcessingOneToOneEx, FPCGTestBaseClass, "Plugins.PCG.Async.AsyncProcessingOneToOneEx", PCGTestsCommon::TestFlags)

bool FPCGAsyncTest_AsyncProcessingOneToOneEx::RunTest(const FString& Parameters)
{
	FPCGAsyncState AsyncState;
	AsyncState.NumAvailableTasks = 10;

	TArray<double> TestOutput;
	const bool bIsDone = FPCGAsync::AsyncProcessingOneToOneEx(
		&AsyncState,
		3,
		[&TestOutput]()
		{
			TestOutput.SetNumUninitialized(3);
		},
		[Value = MakeUnique<int32>(42), &TestOutput](int32 ReadIndex, int32 WriteIndex)
		{
			TestOutput[WriteIndex] = double(ReadIndex * (*Value));
		},
		/*bEnableTimeSlicing*/ false,
		/*ChunkSize*/ 1 // force this to thread
	);

	UTEST_TRUE("IsDone", bIsDone);

	UTEST_EQUAL("Wrote Correct Number", TestOutput.Num(), 3);

	UTEST_EQUAL("TestOutput[0]", TestOutput[0], 42.0 * 0);
	UTEST_EQUAL("TestOutput[1]", TestOutput[1], 42.0 * 1);
	UTEST_EQUAL("TestOutput[2]", TestOutput[2], 42.0 * 2);

	return true;
}

#endif // WITH_EDITOR
