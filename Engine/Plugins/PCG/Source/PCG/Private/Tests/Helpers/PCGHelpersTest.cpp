// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"
#include "PCG/Public/Helpers/PCGHelpers.h"

#include "Algo/MaxElement.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGHelpersTest_GetRandomIndices, FPCGTestBaseClass, "Plugins.PCG.Helpers.GetRandomIndices", PCGTestsCommon::TestFlags)

bool FPCGHelpersTest_GetRandomIndices::RunTest(const FString& Parameters)
{
	FRandomStream RandomStream(42);

	static int32 constexpr Num = 100;

	TArray<int32> RandomIndices = PCGHelpers::GetRandomIndices(RandomStream, Num, 0);
	UTEST_TRUE("Correct array size", RandomIndices.IsEmpty());

	RandomIndices = PCGHelpers::GetRandomIndices(RandomStream, Num, -1);
	UTEST_TRUE("Correct array size", RandomIndices.IsEmpty());

	RandomIndices = PCGHelpers::GetRandomIndices(RandomStream, Num, 1);
	UTEST_EQUAL("Correct array size", RandomIndices.Num(), 1);

	RandomIndices = PCGHelpers::GetRandomIndices(RandomStream, Num, Num + 1);
	UTEST_EQUAL("Correct array size", RandomIndices.Num(), Num);

	RandomIndices = PCGHelpers::GetRandomIndices(RandomStream, Num, Num / 2);
	UTEST_EQUAL("Correct array size", RandomIndices.Num(), Num / 2);

	// Uniqueness
	static int32 constexpr BigNum = 1000000;

	RandomIndices = PCGHelpers::GetRandomIndices(RandomStream, BigNum, BigNum);
	UTEST_EQUAL("Correct array size", RandomIndices.Num(), BigNum);

	TArray<int32> Counts;
	Counts.SetNumZeroed(RandomIndices.Num());
	for (int i = 0; i < RandomIndices.Num(); ++i)
	{
		UTEST_TRUE("All values are unique", ++Counts[RandomIndices[i]] < 2);
	}

	// Distribution
	static int32 constexpr BinCount = 25;
	static int32 constexpr IndicesPerBin = BigNum / BinCount;
	static double constexpr SelectionRatio = 0.2; // Select 1/5 of the array.
	RandomIndices = PCGHelpers::GetRandomIndices(RandomStream, BigNum, BigNum * SelectionRatio);

	TArray<int32> BinCounts;
	BinCounts.SetNumZeroed(BinCount);
	for (int i = 0; i < RandomIndices.Num(); ++i)
	{
		const int32 Index = RandomIndices[i];
		const int32 Bin = Index / IndicesPerBin;
		++BinCounts[Bin];
	}

	static constexpr int32 ExpectedCount = BigNum * SelectionRatio / BinCount;
	static constexpr int32 Tolerance = ExpectedCount * 0.05;
	for (int i = 0; i < BinCounts.Num(); ++i)
	{
		UTEST_LESS("Bins are within tolerance", FMath::Abs(BinCounts[i] - ExpectedCount), Tolerance);
	}

	return true;
}

#endif // WITH_EDITOR
