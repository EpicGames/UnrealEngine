// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "PCGParamData.h"
#include "Algo/Compare.h"

#include "Tests/PCGTestsCommon.h"

#include "Data/PCGPointData.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include "Engine/Engine.h"
#include "InstanceDataPackers/PCGInstanceDataPackerBase.h"

struct FPCGInstancePackerTestState
{
	FPCGPackedCustomData ExpectedCustomData;
	FPCGPackedCustomData ActualCustomData;
};

class FPCGInstancePackerTestClass : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;

protected:
	static constexpr int32 NumPoints = 10;
	static constexpr FLazyName Int32AttributeName = "Int32";
	static constexpr FLazyName Int64AttributeName = "Int64";
	static constexpr FLazyName FloatAttributeName = "Float";
	static constexpr FLazyName DoubleAttributeName = "Double";
	static constexpr FLazyName BoolAttributeName = "Bool";
	static constexpr FLazyName Vec2AttributeName = "Vec2";
	static constexpr FLazyName Vec3AttributeName = "Vec3";
	static constexpr FLazyName Vec4AttributeName = "Vec4";
	static constexpr FLazyName RotatorAttributeName = "Rotator";
	static constexpr FLazyName QuatAttributeName = "Quat";
	static constexpr FLazyName StringAttributeName = "String";

	static UPCGPointArrayData* CreatePointData()
	{
		UPCGPointArrayData* PointData = NewObject<UPCGPointArrayData>();
		UPCGMetadata* Metadata = PointData->MutableMetadata();
		
		PointData->SetNumPoints(NumPoints);
		PointData->AllocateProperties(EPCGPointNativeProperties::Transform | EPCGPointNativeProperties::Density | EPCGPointNativeProperties::MetadataEntry);

		TPCGValueRange<float> DensityRange = PointData->GetDensityValueRange();
		TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange();
		TPCGValueRange<PCGMetadataEntryKey> MetadataEntryRange = PointData->GetMetadataEntryValueRange();

		auto InitMetadata = [Metadata, &MetadataEntryRange](const FPCGMetadataDomainID DomainID)
		{
			FPCGMetadataDomain* Domain = Metadata->GetMetadataDomain(DomainID);

			FPCGMetadataAttribute<int32>* Int32Attribute = Domain->CreateAttribute<int32>(Int32AttributeName, 1, false, false);
			FPCGMetadataAttribute<int64>* Int64Attribute = Domain->CreateAttribute<int64>(Int64AttributeName, 1ll, false, false);
			FPCGMetadataAttribute<float>* FloatAttribute = Domain->CreateAttribute<float>(FloatAttributeName, 1.0f, false, false);
			FPCGMetadataAttribute<double>* DoubleAttribute = Domain->CreateAttribute<double>(DoubleAttributeName, 1.0, false, false);
			FPCGMetadataAttribute<bool>* BoolAttribute = Domain->CreateAttribute<bool>(BoolAttributeName, true, false, false);
			FPCGMetadataAttribute<FVector2D>* Vec2Attribute = Domain->CreateAttribute<FVector2D>(Vec2AttributeName, FVector2D::One(), false, false);
			FPCGMetadataAttribute<FVector>* Vec3Attribute = Domain->CreateAttribute<FVector>(Vec3AttributeName, FVector::One(), false, false);
			FPCGMetadataAttribute<FVector4>* Vec4Attribute = Domain->CreateAttribute<FVector4>(Vec4AttributeName, FVector4::One(), false, false);
			FPCGMetadataAttribute<FRotator>* RotatorAttribute = Domain->CreateAttribute<FRotator>(RotatorAttributeName, FRotator(1.0, 1.0, 1.0), false, false);
			FPCGMetadataAttribute<FQuat>* QuatAttribute = Domain->CreateAttribute<FQuat>(QuatAttributeName, FQuat(1.0, 1.0, 1.0, 1.0), false, false);
			FPCGMetadataAttribute<FString>* StringAttribute = Domain->CreateAttribute<FString>(StringAttributeName, TEXT("1"), false, false);
				
			check(Int32Attribute);
			check(Int64Attribute);
			check(FloatAttribute);
			check(DoubleAttribute);
			check(BoolAttribute);
			check(Vec2Attribute);
			check(Vec3Attribute);
			check(Vec4Attribute);
			check(RotatorAttribute);
			check(QuatAttribute);
			check(StringAttribute);

			if (DomainID == PCGMetadataDomainID::Elements)
			{
				for (int32 i = 0; i < NumPoints; ++i)
				{
					MetadataEntryRange[i] = Domain->AddEntry();
					
					Int32Attribute->SetValue(MetadataEntryRange[i], i);
					Int64Attribute->SetValue(MetadataEntryRange[i], i);
					FloatAttribute->SetValue(MetadataEntryRange[i], i);
					DoubleAttribute->SetValue(MetadataEntryRange[i], i);
					BoolAttribute->SetValue(MetadataEntryRange[i], i != 0);
					Vec2Attribute->SetValue(MetadataEntryRange[i], FVector2D(i, i));
					Vec3Attribute->SetValue(MetadataEntryRange[i], FVector(i, i, i));
					Vec4Attribute->SetValue(MetadataEntryRange[i], FVector4(i, i, i, i));
					RotatorAttribute->SetValue(MetadataEntryRange[i], FRotator(i, i, i));
					QuatAttribute->SetValue(MetadataEntryRange[i], FQuat(i, i, i, i));
					StringAttribute->SetValue(MetadataEntryRange[i], LexToString(i));
				}
			}
		};

		InitMetadata(PCGMetadataDomainID::Elements);
		InitMetadata(PCGMetadataDomainID::Data);

		for (int32 i = 0; i < NumPoints; ++i)
		{
			DensityRange[i] = i;
			TransformRange[i].SetLocation(FVector(i, i, i));
		}

		return PointData;
	}

	bool Validate(const FPCGInstancePackerTestState& State)
	{
		UTEST_EQUAL("Custom Data has the right number of expected floats", State.ActualCustomData.NumCustomDataFloats, State.ExpectedCustomData.NumCustomDataFloats);

		if (!State.ExpectedCustomData.Mask.IsEmpty())
		{
			UTEST_EQUAL("Custom Data has the expected mask", State.ActualCustomData.Mask, State.ExpectedCustomData.Mask);
		}

		UTEST_EQUAL("Custom Data has the right number of values", State.ActualCustomData.CustomData.Num(), State.ExpectedCustomData.CustomData.Num());
		UTEST_TRUE("Custom data has expected values", Algo::Compare(State.ActualCustomData.CustomData, State.ExpectedCustomData.CustomData, [](const float LHS, const float RHS) { return FMath::IsNearlyEqual(LHS, RHS); }))

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGInstancePackerTest_AllPoints, FPCGInstancePackerTestClass, "Plugins.PCG.InstancePacker.AllPoints", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGInstancePackerTest_InvalidType, FPCGInstancePackerTestClass, "Plugins.PCG.InstancePacker.InvalidType", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGInstancePackerTest_PointsOffset, FPCGInstancePackerTestClass, "Plugins.PCG.InstancePacker.PointsOffset", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGInstancePackerTest_InvalidOffset_NotEnoughFloats, FPCGInstancePackerTestClass, "Plugins.PCG.InstancePacker.InvalidOffset.NotEnoughFloats", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGInstancePackerTest_InvalidOffset_Overlapping, FPCGInstancePackerTestClass, "Plugins.PCG.InstancePacker.InvalidOffset.Overlapping", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGInstancePackerTest_TooManyValuesToBeExtracted, FPCGInstancePackerTestClass, "Plugins.PCG.InstancePacker.TooManyValuesToBeExtracted", PCGTestsCommon::TestFlags)

bool FPCGInstancePackerTest_AllPoints::RunTest(const FString& Parameters)
{
	FPCGInstancePackerTestState State;
	const UPCGPointArrayData* PointData = CreatePointData();

	// All of the attributes (21 floats) + $Density and $Position (4 floats)  -> 25 floats
	constexpr int32 NumCustomDataFloats = 25;

	State.ExpectedCustomData.NumCustomDataFloats = NumCustomDataFloats;

	// Values should be 25 zeroes, then 25 ones, then 25 twos, etc...
	State.ExpectedCustomData.CustomData.SetNumZeroed(NumCustomDataFloats * NumPoints);
	for (int32 i = NumCustomDataFloats; i < NumCustomDataFloats * NumPoints; ++i)
	{
		if (i % NumCustomDataFloats == 0)
		{
			// bool
			State.ExpectedCustomData.CustomData[i] = 1.0f;
		}
		else
		{
			State.ExpectedCustomData.CustomData[i] = i / NumCustomDataFloats;
		}
	}

	TArray<FPCGAttributePropertyInputSelector> Selectors =
	{
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(BoolAttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Int32AttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Int64AttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(FloatAttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(DoubleAttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Vec2AttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Vec3AttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Vec4AttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(RotatorAttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(QuatAttributeName),
		FPCGAttributePropertyInputSelector::CreatePointPropertySelector<FPCGAttributePropertyInputSelector>(EPCGPointProperties::Density),
		FPCGAttributePropertyInputSelector::CreatePointPropertySelector<FPCGAttributePropertyInputSelector>(EPCGPointProperties::Position),
	};

	PCGInstanceDataPackerBase::FPackedDataParams Params =
	{
		.CommonParams =
		{
			.NumInstances = NumPoints,
			.OutPackedCustomData = &State.ActualCustomData,
		},
		.InData = PointData,
		.Selectors = MakeConstArrayView(Selectors)
	};

	PCGInstanceDataPackerBase::PackCustomData(Params);

	return Validate(State);
}

/** String attribute is not supported for packing. We set the expected number of float to 1, so we have NumPoints zeroes in the custom data. */
bool FPCGInstancePackerTest_InvalidType::RunTest(const FString& Parameters)
{
	FPCGInstancePackerTestState State;
	const UPCGPointArrayData* PointData = CreatePointData();

	State.ExpectedCustomData.NumCustomDataFloats = 1;
	State.ExpectedCustomData.CustomData.SetNumZeroed(NumPoints);

	State.ActualCustomData.NumCustomDataFloats = 1;

	TArray<FPCGAttributePropertyInputSelector> Selectors =
	{
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(StringAttributeName),
	};

	PCGInstanceDataPackerBase::FPackedDataParams Params =
	{
		.CommonParams =
		{
			.NumInstances = NumPoints,
			.OutPackedCustomData = &State.ActualCustomData,
		},
		.InData = PointData,
		.Selectors = MakeConstArrayView(Selectors)
	};

	PCGInstanceDataPackerBase::PackCustomData(Params);

	return Validate(State);
}

bool FPCGInstancePackerTest_PointsOffset::RunTest(const FString& Parameters)
{
	FPCGInstancePackerTestState State;
	const UPCGPointArrayData* PointData = CreatePointData();

	constexpr int32 NumCustomDataFloats = 5;

	State.ExpectedCustomData.NumCustomDataFloats = NumCustomDataFloats;
	State.ActualCustomData.NumCustomDataFloats = NumCustomDataFloats;

	State.ExpectedCustomData.Mask.Init(true, NumCustomDataFloats);
	State.ExpectedCustomData.Mask[2] = false;
	State.ExpectedCustomData.Mask[4] = false;

	// Values [i, i, 50, i, -47] for each point, where i is point index.
	// Values are initialize for both buffers, to check if the values not on the offset are kept.
	State.ExpectedCustomData.CustomData.Reserve(NumCustomDataFloats * NumPoints);
	State.ActualCustomData.CustomData.Reserve(NumCustomDataFloats * NumPoints);
	for (int32 i = 0; i < NumCustomDataFloats * NumPoints; ++i)
	{
		if (i % 5 == 2)
		{
			State.ExpectedCustomData.CustomData.Add(50.f);
			State.ActualCustomData.CustomData.Add(50.f);
		}
		else if (i % 5 == 4)
		{
			State.ExpectedCustomData.CustomData.Add(-47.f);
			State.ActualCustomData.CustomData.Add(-47.f);
		}
		else
		{
			State.ExpectedCustomData.CustomData.Add(i / 5);
			State.ActualCustomData.CustomData.Add(0.f);
		}
	}

	TArray<FPCGAttributePropertyInputSelector> Selectors =
	{
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Vec2AttributeName),
		FPCGAttributePropertyInputSelector::CreatePointPropertySelector<FPCGAttributePropertyInputSelector>(EPCGPointProperties::Position, NAME_None, {TEXT("X")}), // $Position.X
	};

	// Should write at offset 0 and at offset 3
	constexpr int32 Offsets[2] = {0, 3};

	PCGInstanceDataPackerBase::FPackedDataParams Params =
	{
		.CommonParams =
		{
			.NumInstances = NumPoints,
			.OutPackedCustomData = &State.ActualCustomData,
			.OptionalOffsets = MakeConstArrayView(Offsets)
		},
		.InData = PointData,
		.Selectors = MakeConstArrayView(Selectors)
	};

	PCGInstanceDataPackerBase::PackCustomData(Params);

	return Validate(State);
}

bool FPCGInstancePackerTest_InvalidOffset_NotEnoughFloats::RunTest(const FString& Parameters)
{
	FPCGInstancePackerTestState State;
	const UPCGPointArrayData* PointData = CreatePointData();

	constexpr int32 NumCustomDataFloats = 1;

	State.ExpectedCustomData.NumCustomDataFloats = NumCustomDataFloats;
	State.ActualCustomData.NumCustomDataFloats = NumCustomDataFloats;

	State.ExpectedCustomData.CustomData.SetNumZeroed(NumCustomDataFloats * NumPoints);

	TArray<FPCGAttributePropertyInputSelector> Selectors =
	{
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Int32AttributeName)
	};

	// Offset too big
	constexpr int32 Offsets[1] = {5};
	
	PCGInstanceDataPackerBase::FPackedDataParams Params =
	{
		.CommonParams =
		{
			.NumInstances = NumPoints,
			.OutPackedCustomData = &State.ActualCustomData,
			.OptionalOffsets = MakeConstArrayView(Offsets)
		},
		.InData = PointData,
		.Selectors = MakeConstArrayView(Selectors)
	};
	
	AddExpectedError(TEXT("[PCGInstanceDataPackerBase::ValidateNumOfFloats] Expected to have at least 6 custom floats, but was explicitly set to 1, which is not enough."), EAutomationExpectedErrorFlags::Contains, 1, /*bIsRegex=*/false);
	
	PCGInstanceDataPackerBase::PackCustomData(Params);

	return Validate(State);
}

bool FPCGInstancePackerTest_InvalidOffset_Overlapping::RunTest(const FString& Parameters)
{
	FPCGInstancePackerTestState State;
	const UPCGPointArrayData* PointData = CreatePointData();

	constexpr int32 NumCustomDataFloats = 4;

	State.ExpectedCustomData.NumCustomDataFloats = NumCustomDataFloats;
	State.ActualCustomData.NumCustomDataFloats = NumCustomDataFloats;

	State.ExpectedCustomData.CustomData.SetNumZeroed(NumCustomDataFloats * NumPoints);

	TArray<FPCGAttributePropertyInputSelector> Selectors =
	{
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Vec3AttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(FloatAttributeName)
	};

	// Overlapping offset, the second one should be higher or equal to 3
	constexpr int32 Offsets[2] = {0, 1};

	PCGInstanceDataPackerBase::FPackedDataParams Params =
	{
		.CommonParams =
		{
			.NumInstances = NumPoints,
			.OutPackedCustomData = &State.ActualCustomData,
			.OptionalOffsets = MakeConstArrayView(Offsets)
		},
		.InData = PointData,
		.Selectors = MakeConstArrayView(Selectors)
	};
	
	AddExpectedError(TEXT("[PCGInstanceDataPackerBase::ValidateOffsets] Offset 1 at index 1 is too small for the number of expected floats (3)"), EAutomationExpectedErrorFlags::Contains, 1, /*bIsRegex=*/false);
	
	PCGInstanceDataPackerBase::PackCustomData(Params);

	return Validate(State);
}

bool FPCGInstancePackerTest_TooManyValuesToBeExtracted::RunTest(const FString& Parameters)
{
	FPCGInstancePackerTestState State;
	const UPCGPointArrayData* PointData = CreatePointData();

	constexpr int32 NumCustomDataFloats = 1;

	State.ExpectedCustomData.NumCustomDataFloats = NumCustomDataFloats;
	State.ActualCustomData.NumCustomDataFloats = NumCustomDataFloats;

	State.ExpectedCustomData.CustomData.SetNumZeroed(NumCustomDataFloats * NumPoints);

	TArray<FPCGAttributePropertyInputSelector> Selectors =
	{
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(Vec3AttributeName),
		FPCGAttributePropertyInputSelector::CreateAttributeSelector<FPCGAttributePropertyInputSelector>(FloatAttributeName)
	};

	PCGInstanceDataPackerBase::FPackedDataParams Params =
	{
		.CommonParams =
		{
			.NumInstances = NumPoints,
			.OutPackedCustomData = &State.ActualCustomData,
		},
		.InData = PointData,
		.Selectors = MakeConstArrayView(Selectors)
	};
	
	AddExpectedError(TEXT("[PCGInstanceDataPackerBase::ValidateNumOfFloats] Expected to have at least 4 custom floats, but was explicitly set to 1, which is not enough."), EAutomationExpectedErrorFlags::Contains, 1, /*bIsRegex=*/false);
	
	PCGInstanceDataPackerBase::PackCustomData(Params);

	return Validate(State);
}

#endif // WITH_EDITOR