// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"

#include "PCGGraph.h"
#include "StructUtils/PropertyBag.h"
#include "StructUtils/StructView.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphNoUserParameters, FPCGTestBaseClass, "Plugins.PCG.Graph.NoUserParameters", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphAddSingleUserParameters, FPCGTestBaseClass, "Plugins.PCG.Graph.AddSingleUserParameter", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphAddSingleUserParametersPropagates, FPCGTestBaseClass, "Plugins.PCG.Graph.AddSingleUserParameterPropagates", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphRemoveSingleUserParameters, FPCGTestBaseClass, "Plugins.PCG.Graph.RemoveSingleUserParameter", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphRemoveSingleUserParametersPropagates, FPCGTestBaseClass, "Plugins.PCG.Graph.RemoveSingleUserParameterPropagates", PCGTestsCommon::TestFlags)

// We test the graph add/remove parameters just as a redundancy, but it is already tested by StructUtils. So we won't test more.
// What is important is the propagation between graphs and graphs instances
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphRenameUserParametersPropagates, FPCGTestBaseClass, "Plugins.PCG.Graph.RenameUserParameterPropagates", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphChangeTypeUserParametersPropagates, FPCGTestBaseClass, "Plugins.PCG.Graph.ChangeTypeUserParameterPropagates", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphRemovingGraphCleansUserParameters, FPCGTestBaseClass, "Plugins.PCG.Graph.RemovingGraphCleansUserParameters", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphSetGraphUpdatesUserParameters, FPCGTestBaseClass, "Plugins.PCG.Graph.SetGraphCleansUserParameters", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGGraphSetValueUserParametersPropagates, FPCGTestBaseClass, "Plugins.PCG.Graph.SetValueUserParametersPropagates", PCGTestsCommon::TestFlags)

#define PCG_FMT(Text, ...) FString::Printf(TEXT(Text), __VA_ARGS__)

namespace PCGTests
{
	constexpr const TCHAR* UserParametersName = TEXT("UserParameters");

	void CallPostEditChange(UPCGGraph* Graph, FProperty* Property)
	{
		FEditPropertyChain PropertyChain;
		PropertyChain.AddHead(Property);
		FPropertyChangedEvent PropertyChangedEvent{ Property };

		FPropertyChangedChainEvent PropertyChangedChainEvent{ PropertyChain, PropertyChangedEvent };

		Graph->PostEditChangeChainProperty(PropertyChangedChainEvent);
	}

	template <typename Func>
	void EmulateModifyingUserParameters(UPCGGraph* Graph, Func Callback)
	{
		FProperty* Property = Graph->GetClass()->FindPropertyByName(UserParametersName);
		check(Property);
		Graph->PreEditChange(Property);

		Callback();

		CallPostEditChange(Graph, Property);
	}

	template <typename Func>
	void EmulateModifyingUserParametersValue(UPCGGraph* Graph, FName PropertyName, Func Callback)
	{
		const FInstancedPropertyBag* UserParameters = Graph->GetUserParametersStruct();
		check(UserParameters && UserParameters->GetPropertyBagStruct());

		FProperty* ValueProperty = UserParameters->GetPropertyBagStruct()->FindPropertyByName(PropertyName);
		check(ValueProperty);
		Graph->PreEditChange(ValueProperty);

		Callback();

		CallPostEditChange(Graph, ValueProperty);
	}
}

bool FPCGGraphNoUserParameters::RunTest(const FString& Parameters)
{
	UPCGGraph* Graph = NewObject<UPCGGraph>();
	UTEST_NOT_NULL("Graph UserParameters is not null", Graph->GetUserParametersStruct());

	// Bags are now valid (but empty) for new graphs.
	UTEST_TRUE("Graph UserParameters property bag is valid", Graph->GetUserParametersStruct()->IsValid());
	UTEST_EQUAL("Graph UserParameters has no property", Graph->GetUserParametersStruct()->GetNumPropertiesInBag(), 0);

	return true;
}

bool FPCGGraphAddSingleUserParameters::RunTest(const FString& Parameters)
{
	UPCGGraph* Graph = NewObject<UPCGGraph>();

	const FName MyPropertyName = TEXT("MyProperty");

	// Creating a new property MyProperty, which is a double
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->AddProperty(MyPropertyName, EPropertyBagPropertyType::Double, nullptr);
	});

	UTEST_NOT_NULL("Graph UserParameters is not null", Graph->GetUserParametersStruct());

	UTEST_TRUE("Graph UserParameters property bag is valid", Graph->GetUserParametersStruct()->IsValid());
	UTEST_EQUAL("Graph UserParameters has 1 property", Graph->GetUserParametersStruct()->GetNumPropertiesInBag(), 1);

	const FPropertyBagPropertyDesc* PropertyDesc = Graph->GetUserParametersStruct()->FindPropertyDescByName(MyPropertyName);
	UTEST_NOT_NULL("Property exists", PropertyDesc);

	UTEST_TRUE("Property is a double", PropertyDesc->IsNumericFloatType());

	return true;
}

bool FPCGGraphAddSingleUserParametersPropagates::RunTest(const FString& Parameters)
{
	UPCGGraph* Graph = NewObject<UPCGGraph>();
	UPCGGraphInstance* GraphInstance = NewObject<UPCGGraphInstance>();
	UPCGGraphInstance* GraphInstanceChild = NewObject<UPCGGraphInstance>();

	GraphInstance->SetGraph(Graph);
	GraphInstanceChild->SetGraph(GraphInstance);

	UTEST_EQUAL("Graph and GraphInstance have the same graph", GraphInstance->GetGraph(), (UPCGGraph*)Graph);
	UTEST_EQUAL("GraphInstance and GraphInstanceChild have the same graph", GraphInstanceChild->GetGraph(), GraphInstance->GetGraph());

	const FName MyPropertyName = TEXT("MyProperty");

	// Creating a new property MyProperty, which is a double
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->AddProperty(MyPropertyName, EPropertyBagPropertyType::Double, nullptr);
	});

	auto Verification = [this, MyPropertyName](UPCGGraphInstance* GraphInstance, const TCHAR* Name) -> bool
	{
		UTEST_NOT_NULL(PCG_FMT("%s UserParameters is not null", Name), GraphInstance->GetUserParametersStruct());

		UTEST_TRUE(PCG_FMT("%s UserParameters property bag is valid", Name), GraphInstance->GetUserParametersStruct()->IsValid());
		UTEST_EQUAL(PCG_FMT("%s UserParameters has 1 property", Name), GraphInstance->GetUserParametersStruct()->GetNumPropertiesInBag(), 1);

		const FPropertyBagPropertyDesc* PropertyDesc = GraphInstance->GetUserParametersStruct()->FindPropertyDescByName(MyPropertyName);
		UTEST_NOT_NULL(PCG_FMT("%s: Property exists", Name), PropertyDesc);

		UTEST_TRUE(PCG_FMT("%s: Property is a double", Name), PropertyDesc->IsNumericFloatType());

		return true;
	};

	if (!Verification(GraphInstance, TEXT("GraphInstance"))) { return false; }
	if (!Verification(GraphInstanceChild, TEXT("GraphInstanceChild"))) { return false; }

	return true;
}

bool FPCGGraphRemoveSingleUserParameters::RunTest(const FString& Parameters)
{
	UPCGGraph* Graph = NewObject<UPCGGraph>();

	const FName MyPropertyName = TEXT("MyProperty");

	// Creating a new property MyProperty, which is a double
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->AddProperty(MyPropertyName, EPropertyBagPropertyType::Double, nullptr);
	});

	// Remove that same property
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->RemovePropertyByName(MyPropertyName);
	});

	UTEST_NOT_NULL("Graph UserParameters is not null", Graph->GetUserParametersStruct());

	UTEST_TRUE("Graph UserParameters property bag is valid", Graph->GetUserParametersStruct()->IsValid());
	UTEST_EQUAL("Graph UserParameters has no property", Graph->GetUserParametersStruct()->GetNumPropertiesInBag(), 0);

	return true;
}

bool FPCGGraphRemoveSingleUserParametersPropagates::RunTest(const FString& Parameters)
{
	UPCGGraph* Graph = NewObject<UPCGGraph>();
	UPCGGraphInstance* GraphInstance = NewObject<UPCGGraphInstance>();
	UPCGGraphInstance* GraphInstanceChild = NewObject<UPCGGraphInstance>();

	GraphInstance->SetGraph(Graph);
	GraphInstanceChild->SetGraph(GraphInstance);

	UTEST_EQUAL("Graph and GraphInstance have the same graph", GraphInstance->GetGraph(), (UPCGGraph*)Graph);
	UTEST_EQUAL("GraphInstance and GraphInstanceChild have the same graph", GraphInstanceChild->GetGraph(), GraphInstance->GetGraph());

	const FName MyPropertyName = TEXT("MyProperty");

	// Creating a new property MyProperty, which is a double
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->AddProperty(MyPropertyName, EPropertyBagPropertyType::Double, nullptr);
	});

	// Remove that same property
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->RemovePropertyByName(MyPropertyName);
	});

	auto Verification = [this, MyPropertyName](UPCGGraphInstance* GraphInstance, const TCHAR* Name) -> bool
	{
		UTEST_NOT_NULL(PCG_FMT("%s UserParameters is not null", Name), GraphInstance->GetUserParametersStruct());

		UTEST_TRUE(PCG_FMT("%s UserParameters property bag is valid", Name), GraphInstance->GetUserParametersStruct()->IsValid());
		UTEST_EQUAL(PCG_FMT("%s UserParameters has 0 property", Name), GraphInstance->GetUserParametersStruct()->GetNumPropertiesInBag(), 0);

		return true;
	};

	if (!Verification(GraphInstance, TEXT("GraphInstance"))) { return false; }
	if (!Verification(GraphInstanceChild, TEXT("GraphInstanceChild"))) { return false; }

	return true;
}

bool FPCGGraphRenameUserParametersPropagates::RunTest(const FString& Parameters)
{
	UPCGGraph* Graph = NewObject<UPCGGraph>();
	UPCGGraphInstance* GraphInstance = NewObject<UPCGGraphInstance>();
	UPCGGraphInstance* GraphInstanceChild = NewObject<UPCGGraphInstance>();

	GraphInstance->SetGraph(Graph);
	GraphInstanceChild->SetGraph(GraphInstance);

	UTEST_EQUAL("Graph and GraphInstance have the same graph", GraphInstance->GetGraph(), (UPCGGraph*)Graph);
	UTEST_EQUAL("GraphInstance and GraphInstanceChild have the same graph", GraphInstanceChild->GetGraph(), GraphInstance->GetGraph());

	const FName MyPropertyName = TEXT("MyProperty");
	const FName MyNewPropertyName = TEXT("MyNewProperty");

	// Creating a new property MyProperty, which is a double
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->AddProperty(MyPropertyName, EPropertyBagPropertyType::Double, nullptr);
	});

	// Rename it to MyNewProperty, while keeping the type Double
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyNewPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();

		const UPropertyBag* NewPropertyBag = UPropertyBag::GetOrCreateFromDescs({ FPropertyBagPropertyDesc{MyNewPropertyName, EPropertyBagPropertyType::Double, nullptr} });
		UserParameters->MigrateToNewBagStruct(NewPropertyBag);
	});

	auto Verification = [this, MyPropertyName, MyNewPropertyName](UPCGGraphInstance* GraphInstance, const TCHAR* Name) -> bool
	{
		UTEST_NOT_NULL(PCG_FMT("%s UserParameters is not null", Name), GraphInstance->GetUserParametersStruct());

		UTEST_TRUE(PCG_FMT("%s UserParameters property bag is valid", Name), GraphInstance->GetUserParametersStruct()->IsValid());
		UTEST_EQUAL(PCG_FMT("%s UserParameters has 1 property", Name), GraphInstance->GetUserParametersStruct()->GetNumPropertiesInBag(), 1);

		const FPropertyBagPropertyDesc* PropertyDesc = GraphInstance->GetUserParametersStruct()->FindPropertyDescByName(MyPropertyName);
		UTEST_NULL(PCG_FMT("%s: Old property exists", Name), PropertyDesc);

		PropertyDesc = GraphInstance->GetUserParametersStruct()->FindPropertyDescByName(MyNewPropertyName);
		UTEST_NOT_NULL(PCG_FMT("%s: New property exists", Name), PropertyDesc);

		UTEST_TRUE(PCG_FMT("%s: Property is a double", Name), PropertyDesc->IsNumericFloatType());

		return true;
	};

	if (!Verification(GraphInstance, TEXT("GraphInstance"))) { return false; }
	if (!Verification(GraphInstanceChild, TEXT("GraphInstanceChild"))) { return false; }

	return true;
}

bool FPCGGraphChangeTypeUserParametersPropagates::RunTest(const FString& Parameters)
{
	UPCGGraph* Graph = NewObject<UPCGGraph>();
	UPCGGraphInstance* GraphInstance = NewObject<UPCGGraphInstance>();
	UPCGGraphInstance* GraphInstanceChild = NewObject<UPCGGraphInstance>();

	GraphInstance->SetGraph(Graph);
	GraphInstanceChild->SetGraph(GraphInstance);

	UTEST_EQUAL("Graph and GraphInstance have the same graph", GraphInstance->GetGraph(), (UPCGGraph*)Graph);
	UTEST_EQUAL("GraphInstance and GraphInstanceChild have the same graph", GraphInstanceChild->GetGraph(), GraphInstance->GetGraph());

	const FName MyPropertyName = TEXT("MyProperty");

	// Creating a new property MyProperty, which is a double
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->AddProperty(MyPropertyName, EPropertyBagPropertyType::Double, nullptr);
	});

	// Keeping the name MyProperty but changing the type to Int64
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();

		const UPropertyBag* NewPropertyBag = UPropertyBag::GetOrCreateFromDescs({ FPropertyBagPropertyDesc{MyPropertyName, EPropertyBagPropertyType::Int64, nullptr} });
		UserParameters->MigrateToNewBagStruct(NewPropertyBag);
	});

	auto Verification = [this, MyPropertyName](UPCGGraphInstance* GraphInstance, const TCHAR* Name) -> bool
	{
		UTEST_NOT_NULL(PCG_FMT("%s UserParameters is not null", Name), GraphInstance->GetUserParametersStruct());

		UTEST_TRUE(PCG_FMT("%s UserParameters property bag is valid", Name), GraphInstance->GetUserParametersStruct()->IsValid());
		UTEST_EQUAL(PCG_FMT("%s UserParameters has 1 property", Name), GraphInstance->GetUserParametersStruct()->GetNumPropertiesInBag(), 1);

		const FPropertyBagPropertyDesc* PropertyDesc = GraphInstance->GetUserParametersStruct()->FindPropertyDescByName(MyPropertyName);
		UTEST_NOT_NULL(PCG_FMT("%s: Old property still exists", Name), PropertyDesc);

		UTEST_TRUE(PCG_FMT("%s: Property is a int", Name), PropertyDesc->IsNumericType() && !PropertyDesc->IsNumericFloatType());

		return true;
	};

	if (!Verification(GraphInstance, TEXT("GraphInstance"))) { return false; }
	if (!Verification(GraphInstanceChild, TEXT("GraphInstanceChild"))) { return false; }

	return true;
}

bool FPCGGraphRemovingGraphCleansUserParameters::RunTest(const FString& Parameters)
{
	UPCGGraph* Graph = NewObject<UPCGGraph>();
	UPCGGraphInstance* GraphInstance = NewObject<UPCGGraphInstance>();

	GraphInstance->SetGraph(Graph);

	UTEST_EQUAL("Graph and GraphInstance have the same graph", GraphInstance->GetGraph(), (UPCGGraph*)Graph);

	const FName MyPropertyName = TEXT("MyProperty");
	const FName MyPropertyName2 = TEXT("MyProperty2");

	// Creating a new property MyProperty, which is a double
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->AddProperty(MyPropertyName, EPropertyBagPropertyType::Double, nullptr);
	});

	// Creating a new property MyProperty2, which is a int64
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName2]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->AddProperty(MyPropertyName2, EPropertyBagPropertyType::Int64, nullptr);
	});

	GraphInstance->SetGraph(nullptr);

	UTEST_NOT_NULL("UserParameters is not null", GraphInstance->GetUserParametersStruct());

	UTEST_FALSE("UserParameters property bag is valid", GraphInstance->GetUserParametersStruct()->IsValid());
	UTEST_EQUAL("UserParameters has no property", GraphInstance->GetUserParametersStruct()->GetNumPropertiesInBag(), 0);

	return true;
}

bool FPCGGraphSetGraphUpdatesUserParameters::RunTest(const FString& Parameters)
{
	UPCGGraph* Graph = NewObject<UPCGGraph>();
	UPCGGraphInstance* GraphInstance = NewObject<UPCGGraphInstance>();

	const FName MyPropertyName = TEXT("MyProperty");
	const FName MyPropertyName2 = TEXT("MyProperty2");

	// Creating a new property MyProperty, which is a double
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->AddProperty(MyPropertyName, EPropertyBagPropertyType::Double, nullptr);
	});

	// Creating a new property MyProperty2, which is a int64
	PCGTests::EmulateModifyingUserParameters(Graph, [&Graph, MyPropertyName2]()
	{
		FInstancedPropertyBag* UserParameters = Graph->GetMutableUserParametersStruct_Unsafe();
		UserParameters->AddProperty(MyPropertyName2, EPropertyBagPropertyType::Int64, nullptr);
	});

	GraphInstance->SetGraph(Graph);

	UTEST_EQUAL("Graph and GraphInstance have the same graph", GraphInstance->GetGraph(), (UPCGGraph*)Graph);

	UTEST_NOT_NULL("UserParameters is not null", GraphInstance->GetUserParametersStruct());

	UTEST_TRUE("UserParameters property bag is valid", GraphInstance->GetUserParametersStruct()->IsValid());
	UTEST_EQUAL("UserParameters has 2 properties", GraphInstance->GetUserParametersStruct()->GetNumPropertiesInBag(), 2);

	const FPropertyBagPropertyDesc* PropertyDesc = Graph->GetUserParametersStruct()->FindPropertyDescByName(MyPropertyName);
	UTEST_NOT_NULL("First property exists", PropertyDesc);

	UTEST_TRUE("First property is a double", PropertyDesc->IsNumericFloatType());

	PropertyDesc = Graph->GetUserParametersStruct()->FindPropertyDescByName(MyPropertyName2);
	UTEST_NOT_NULL("Second property exists", PropertyDesc);

	UTEST_TRUE("First property is an int", PropertyDesc->IsNumericType() && !PropertyDesc->IsNumericFloatType());

	return true;
}

bool FPCGGraphSetValueUserParametersPropagates::RunTest(const FString& Parameters)
{
	UPCGGraph* Graph = NewObject<UPCGGraph>();
	UPCGGraphInstance* GraphInstance = NewObject<UPCGGraphInstance>();
	UPCGGraphInstance* GraphInstanceChild = NewObject<UPCGGraphInstance>();

	GraphInstance->SetGraph(Graph);
	GraphInstanceChild->SetGraph(GraphInstance);

	UTEST_EQUAL("Graph and GraphInstance have the same graph", GraphInstance->GetGraph(), (UPCGGraph*)Graph);
	UTEST_EQUAL("GraphInstance and GraphInstanceChild have the same graph", GraphInstanceChild->GetGraph(), GraphInstance->GetGraph());

	const FName MyPropertyName = TEXT("MyProperty");
	const FName MyPropertyName2 = TEXT("MyProperty2");

	// Creating new properties MyProperty, which is a double and MyProperty2, which is a int64
	Graph->UpdateUserParametersStruct([MyPropertyName, MyPropertyName2](FInstancedPropertyBag& UserParameters)
		{
			UserParameters.AddProperty(MyPropertyName, EPropertyBagPropertyType::Double, nullptr);
			UserParameters.AddProperty(MyPropertyName2, EPropertyBagPropertyType::Int64, nullptr);
		});

	// Setting MyProperty to 3.0
	const double NewValue = 3.0;
	Graph->SetGraphParameter<float>(MyPropertyName, NewValue);

	auto Verification = [this, MyPropertyName, NewValue](UPCGGraphInstance* GraphInstance, const TCHAR* Name) -> bool
	{
		UTEST_NOT_NULL(PCG_FMT("%s UserParameters is not null", Name), GraphInstance->GetUserParametersStruct());

		UTEST_TRUE(PCG_FMT("%s UserParameters property bag is valid", Name), GraphInstance->GetUserParametersStruct()->IsValid());
		UTEST_EQUAL(PCG_FMT("%s UserParameters has 2 properties", Name), GraphInstance->GetUserParametersStruct()->GetNumPropertiesInBag(), 2);

		TValueOrError<double, EPropertyBagResult> FirstPropertyValue = GraphInstance->GetGraphParameter<double>(MyPropertyName);
		
		UTEST_FALSE(PCG_FMT("%s: First property exists and is the expected type", Name), FirstPropertyValue.HasError());
		UTEST_EQUAL(PCG_FMT("%s: Property has the right value", Name), FirstPropertyValue.GetValue(), NewValue);

		return true;
	};

	if (!Verification(GraphInstance, TEXT("GraphInstance"))) { return false; }
	if (!Verification(GraphInstanceChild, TEXT("GraphInstanceChild"))) { return false; }

	return true;
}

#undef PCG_FMT

#endif // WITH_EDITOR