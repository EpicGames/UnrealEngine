// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGTestsCommon.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGCrc.h"
#include "PCGElement.h"
#include "PCGGraph.h"
#include "PCGParamData.h"
#include "PCGPin.h"
#include "Elements/PCGCreatePoints.h"
#include "Data/PCGPointData.h"
#include "Data/PCGPolyLineData.h" // IWYU pragma: keep
#include "Data/PCGPrimitiveData.h" // IWYU pragma: keep
#include "Data/PCGSurfaceData.h" // IWYU pragma: keep
#include "Data/PCGVolumeData.h"

#if WITH_EDITOR
#include "Engine/World.h"
#include "Editor.h"
#include "Components/SceneComponent.h"
#include "Landscape.h"
#include "LandscapeComponent.h"
#include "LandscapeDataAccess.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#endif // WITH_EDITOR

#if WITH_EDITOR
namespace
{
	constexpr int32 LandscapeNumSubsections = 1;
	constexpr int32 LandscapeSubsectionSizeQuads = 63;
	constexpr int32 LandscapeSizeVerts = LandscapeSubsectionSizeQuads * LandscapeNumSubsections + 1;
	constexpr int32 LandscapeTotalVerts = LandscapeSizeVerts * LandscapeSizeVerts;

	ALandscape* CreateLandscapeFromHeightData(UWorld* World, const FTransform& Transform, TArray<uint16>&& HeightData)
	{
		check(World);
		check(HeightData.Num() == LandscapeTotalVerts);

		FActorSpawnParameters SpawnParams;
		SpawnParams.ObjectFlags = RF_Transient;
		ALandscape* Landscape = World->SpawnActor<ALandscape>(Transform.GetLocation(), Transform.GetRotation().Rotator(), SpawnParams);
		check(Landscape);

		TMap<FGuid, TArray<uint16>> HeightDataPerLayer;
		HeightDataPerLayer.Add(FGuid(), MoveTemp(HeightData));

		TMap<FGuid, TArray<FLandscapeImportLayerInfo>> MaterialLayerDataPerLayer;
		MaterialLayerDataPerLayer.Add(FGuid(), TArray<FLandscapeImportLayerInfo>());

		Landscape->Import(
			FGuid::NewGuid(),
			0, 0, LandscapeSizeVerts - 1, LandscapeSizeVerts - 1,
			LandscapeNumSubsections, LandscapeSubsectionSizeQuads,
			HeightDataPerLayer,
			nullptr,
			MaterialLayerDataPerLayer,
			ELandscapeImportAlphamapType::Additive,
			MakeArrayView<const FLandscapeLayer>({})
		);

		Landscape->CreateLandscapeInfo();

		return Landscape;
	}
}
#endif // WITH_EDITOR

namespace PCGTestsCommon
{
#if WITH_EDITOR
	ALandscape* CreateFlatLandscape(UWorld* World, const FTransform& Transform)
	{
		TArray<uint16> HeightData;
		HeightData.SetNum(LandscapeTotalVerts);
		for (uint16& H : HeightData)
		{
			H = LandscapeDataAccess::GetTexHeight(0.0f);
		}

		return CreateLandscapeFromHeightData(World, Transform, MoveTemp(HeightData));
	}

	ALandscape* CreatePerlinLandscape(UWorld* World, const FTransform& Transform, int32 Seed)
	{
		TArray<uint16> HeightData;
		HeightData.SetNum(LandscapeTotalVerts);

		// Offset the noise domain by seed so different seeds produce different terrain.
		const float SeedOffset = Seed * 17.31f;

		for (int32 Y = 0; Y < LandscapeSizeVerts; ++Y)
		{
			for (int32 X = 0; X < LandscapeSizeVerts; ++X)
			{
				const float NX = X / static_cast<float>(LandscapeSizeVerts) + SeedOffset;
				const float NY = Y / static_cast<float>(LandscapeSizeVerts) + SeedOffset;

				// Three octaves: broad hills, medium ridges, fine bumps.
				float H = 0.0f;
				H += 200.0f * FMath::PerlinNoise2D(FVector2D(NX * 2.0f, NY * 2.0f));
				H +=  50.0f * FMath::PerlinNoise2D(FVector2D(NX * 8.0f, NY * 8.0f));
				H +=  10.0f * FMath::PerlinNoise2D(FVector2D(NX * 20.0f, NY * 20.0f));

				HeightData[Y * LandscapeSizeVerts + X] = LandscapeDataAccess::GetTexHeight(H);
			}
		}

		return CreateLandscapeFromHeightData(World, Transform, MoveTemp(HeightData));
	}
#endif // WITH_EDITOR

#if WITH_EDITOR
	UPCGNode* AddCreatePointsNode(UPCGGraph* Graph, TArrayView<const FTransform> Transforms)
	{
		UPCGCreatePointsSettings* Settings = nullptr;
		UPCGNode* Node = Graph->AddNodeOfType<UPCGCreatePointsSettings>(Settings);
		Settings->PointsToCreate.Empty();
		Settings->CoordinateSpace = EPCGCoordinateSpace::World;
		for (const FTransform& Transform : Transforms)
		{
			Settings->PointsToCreate.AddDefaulted_GetRef().Transform = Transform;
		}
		return Node;
	}
#endif // WITH_EDITOR

	FTestData::FTestData(int32 RandomSeed, UPCGSettings* DefaultSettings, TSubclassOf<AActor> ActorClass)
		: Settings(DefaultSettings)
		, Seed(RandomSeed)
		, RandomStream(Seed)
	{
#if WITH_EDITOR
		check(GEditor);
		UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
		check(EditorWorld);

		// No getting the level dirty
		FActorSpawnParameters TransientActorParameters;
		TransientActorParameters.bHideFromSceneOutliner = true;
		TransientActorParameters.bTemporaryEditorActor = true;
		TransientActorParameters.ObjectFlags = RF_Transient;
		TestActor = EditorWorld->SpawnActor<AActor>(ActorClass, TransientActorParameters);
		check(TestActor);

		if (UPCGComponent* PCGComponent = TestActor->GetComponentByClass<UPCGComponent>())
		{
			TestPCGComponent = PCGComponent;
		}
		else
		{
			TestPCGComponent = NewObject<UPCGComponent>(TestActor, FName(TEXT("Test PCG Component")), RF_Transient);
			check(TestPCGComponent);
			TestActor->AddInstanceComponent(TestPCGComponent);
			TestPCGComponent->RegisterComponent();
		}

		// By default PCG components for tests will be non-partitioned
		TestPCGComponent->SetIsPartitioned(false);

		UPCGGraph* TestGraph = NewObject<UPCGGraph>(TestPCGComponent, FName(TEXT("Test PCG Graph")), RF_Transient);
		check(TestGraph);
		TestPCGComponent->SetGraphLocal(TestGraph);

		// Add Root Component to actor if none exists
		if (!TestActor->GetRootComponent())
		{
			USceneComponent* NewRootComponent = NewObject<USceneComponent>(TestActor, FName(TEXT("DefaultSceneRoot")), RF_Transient);
			TestActor->SetRootComponent(NewRootComponent);
			TestActor->AddInstanceComponent(NewRootComponent);
			NewRootComponent->RegisterComponent();
		}

		// Initialize CRC to avoid asserts.
		InputData.ComputeCrcs(/*bFullDataCrc=*/false);
#else
		TestActor = nullptr;
		TestPCGComponent = nullptr;
		Settings = nullptr;
#endif
	}

	FTestData::~FTestData()
	{
#if WITH_EDITOR
		if (GEditor)
		{
			if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
			{
				if (TestActor)
				{
					EditorWorld->DestroyActor(TestActor);
				}
			}
		}
#endif // WITH_EDITOR
	}

	void FTestData::Reset(UPCGSettings* InSettings)
	{
		// Clear all the data
		RandomStream.Reset();
		InputData.TaggedData.Empty();
		OutputData.TaggedData.Empty();
		Settings = InSettings;

		if (Settings)
		{
			InputData.TaggedData.Emplace_GetRef().Data = Settings;
			InputData.TaggedData.Last().Pin = FName(TEXT("Settings"));
		}
	}

	TUniquePtr<FPCGContext> InitializeTestContext(IPCGElement* InElement, const FPCGDataCollection& InputData, UPCGComponent* InSourceComponent, const UPCGNode* InNode)
	{
		check(InElement);
		TUniquePtr<FPCGContext> Context{ InElement->Initialize(FPCGInitializeElementParams(&InputData, InSourceComponent, InNode)) };
		Context->InitializeSettings();
		Context->AsyncState.NumAvailableTasks = 1;
		return Context;
	}

	TUniquePtr<FPCGContext> FTestData::InitializeTestContext(const UPCGNode* InNode) const
	{
		check(Settings)
		return PCGTestsCommon::InitializeTestContext(Settings->GetElement().Get(), InputData, TestPCGComponent, InNode);
	}

	void FTestData::SetCurrentGenerationTask(FPCGTaskId InTaskId)
	{
		if (TestPCGComponent)
		{
			TestPCGComponent->CurrentGenerationTask = InTaskId;
		}
	}

	AActor* CreateTemporaryActor()
	{
		return NewObject<AActor>();
	}

	UPCGPolyLineData* CreatePolyLineData()
	{
		// TODO: spline, landscape spline
		return nullptr;
	}

	UPCGSurfaceData* CreateSurfaceData()
	{
		// TODO: either landscape, texture, render target
		return nullptr;
	}

	UPCGVolumeData* CreateVolumeData(const FBox& InBounds)
	{
		UPCGVolumeData* VolumeData = NewObject<UPCGVolumeData>();
		VolumeData->Initialize(InBounds);
		return VolumeData;
	}

	UPCGPrimitiveData* CreatePrimitiveData()
	{
		// TODO: need UPrimitiveComponent on an actor
		return nullptr;
	}

	UPCGParamData* CreateEmptyParamData()
	{
		return NewObject<UPCGParamData>();
	}

	UPCGBasePointData* CreateEmptyBasePointData()
	{
		if (CVarPCGEnablePointArrayData.GetValueOnAnyThread())
		{
			return CreateEmptyPointData<UPCGPointArrayData>();
		}
		else
		{
			return CreateEmptyPointData<UPCGPointData>();
		}
	}

	UPCGBasePointData* CreateBasePointData()
	{
		if (CVarPCGEnablePointArrayData.GetValueOnAnyThread())
		{
			return CreatePointData<UPCGPointArrayData>();
		}
		else
		{
			return CreatePointData<UPCGPointData>();
		}
	}

	UPCGBasePointData* CreateBasePointData(const FVector& InLocation)
	{
		if (CVarPCGEnablePointArrayData.GetValueOnAnyThread())
		{
			return CreatePointData<UPCGPointArrayData>(InLocation);
		}
		else
		{
			return CreatePointData<UPCGPointData>(InLocation);
		}
	}

	UPCGBasePointData* CreateRandomBasePointData(int32 PointCount, int32 Seed, bool bRandomDensity)
	{
		if (CVarPCGEnablePointArrayData.GetValueOnAnyThread())
		{
			return CreateRandomPointData<UPCGPointArrayData>(PointCount, Seed, bRandomDensity);
		}
		else
		{
			return CreateRandomPointData<UPCGPointData>(PointCount, Seed, bRandomDensity);
		}
	}

	TArray<FPCGDataCollection> GenerateAllowedData(const FPCGPinProperties& PinProperties)
	{
		TArray<FPCGDataCollection> Data;

		static const TMap<FPCGDataTypeIdentifier, TFunction<UPCGData* (void)>> TypeToDataFn
		{
			{ FPCGDataTypeIdentifier{EPCGDataType::Point}, []() { return PCGTestsCommon::CreatePointData(); }},
			{ FPCGDataTypeIdentifier{EPCGDataType::PolyLine}, []() { return PCGTestsCommon::CreatePolyLineData(); }},
			{ FPCGDataTypeIdentifier{EPCGDataType::Surface}, []() { return PCGTestsCommon::CreateSurfaceData(); }},
			{ FPCGDataTypeIdentifier{EPCGDataType::Volume}, []() { return PCGTestsCommon::CreateVolumeData(); }},
			{ FPCGDataTypeIdentifier{EPCGDataType::Primitive}, []() { return PCGTestsCommon::CreatePrimitiveData(); }},
			{ FPCGDataTypeIdentifier{EPCGDataType::Param}, []() { return PCGTestsCommon::CreateEmptyParamData();}}
		};

		// Create empty data
		Data.Emplace();

		// Create single data & data pairs
		for (const auto& TypeToData : TypeToDataFn)
		{
			if (!(TypeToData.Key & PinProperties.AllowedTypes))
			{
				continue;
			}

			const UPCGData* SingleData = TypeToData.Value();
			if (!SingleData)
			{
				continue;
			}

			FPCGDataCollection& SingleCollection = Data.Emplace_GetRef();
			FPCGTaggedData& SingleTaggedData = SingleCollection.TaggedData.Emplace_GetRef();
			SingleTaggedData.Data = SingleData;
			SingleTaggedData.Pin = PinProperties.Label;

			if (!PinProperties.AllowsMultipleConnections())
			{
				continue;
			}

			for (const auto& SecondaryTypeToData : TypeToDataFn)
			{
				if (!(SecondaryTypeToData.Key & PinProperties.AllowedTypes))
				{
					continue;
				}

				const UPCGData* SecondaryData = SecondaryTypeToData.Value();
				if (!SecondaryData)
				{
					continue;
				}

				FPCGDataCollection& MultiCollection = Data.Emplace_GetRef();
				FPCGTaggedData& FirstTaggedData = MultiCollection.TaggedData.Emplace_GetRef();
				FirstTaggedData.Data = SingleData;
				FirstTaggedData.Pin = PinProperties.Label;

				FPCGTaggedData& SecondTaggedData = MultiCollection.TaggedData.Emplace_GetRef();
				SecondTaggedData.Data = SecondaryData;
				SecondTaggedData.Pin = PinProperties.Label;
			}
		}

		return Data;
	}

	bool PointsAreIdentical(const FPCGPoint& FirstPoint, const FPCGPoint& SecondPoint)
	{
		// Trivial checks first for pruning
		if (FirstPoint.Density != SecondPoint.Density || FirstPoint.Steepness != SecondPoint.Steepness ||
			FirstPoint.BoundsMin != SecondPoint.BoundsMin || FirstPoint.BoundsMax != SecondPoint.BoundsMax ||
			FirstPoint.Color != SecondPoint.Color)
		{
			return false;
		}

		// Transform checks with epsilon
		return FirstPoint.Transform.Equals(SecondPoint.Transform);
	}

#if WITH_EDITOR
	bool ValidateSingleTaggedData(FAutomationTestBase* Test, const FPCGDataCollection& DataCollection, TFunctionRef<bool(const FPCGTaggedData&)> Validator)
	{
		if (!Test->TestEqual(TEXT("Data collection contains a single data item"), DataCollection.TaggedData.Num(), 1))
		{
			return false;
		}
		return Validator(DataCollection.TaggedData[0]);
	}

	bool CompareMetadata(FAutomationTestBase* Test, const UPCGMetadata* ExpMeta, const UPCGMetadata* ActMeta, int32 NumEntries)
	{
		// Both null is fine — no metadata to compare.
		if (!ExpMeta && !ActMeta)
		{
			return true;
		}

		if (!Test->TestNotNull(TEXT("Expected metadata"), ExpMeta) || !Test->TestNotNull(TEXT("Actual metadata"), ActMeta))
		{
			return false;
		}

		// Explicit null guard to satisfy static analysis (C6011). The TestNotNull calls above already return false on null.
		if (!ExpMeta || !ActMeta)
		{
			return false;
		}

		TArray<FName> ExpNames, ActNames;
		TArray<EPCGMetadataTypes> ExpTypes, ActTypes;
		ExpMeta->GetAttributes(ExpNames, ExpTypes);
		ActMeta->GetAttributes(ActNames, ActTypes);

		if (!Test->TestEqual(TEXT("Metadata attribute count"), ActNames.Num(), ExpNames.Num()))
		{
			return false;
		}

		bool bPassed = true;

		for (int32 AttrIdx = 0; AttrIdx < ExpNames.Num(); ++AttrIdx)
		{
			const FName& AttrName = ExpNames[AttrIdx];
			const int32 ActIdx = ActNames.IndexOfByKey(AttrName);

			if (!Test->TestTrue(*FString::Printf(TEXT("Attribute '%s' exists in actual"), *AttrName.ToString()), ActIdx != INDEX_NONE))
			{
				bPassed = false;
				continue;
			}

			if (!Test->TestEqual(*FString::Printf(TEXT("Attribute '%s' type"), *AttrName.ToString()), static_cast<uint8>(ActTypes[ActIdx]), static_cast<uint8>(ExpTypes[AttrIdx])))
			{
				bPassed = false;
				continue;
			}

			const FPCGMetadataAttributeBase* ExpAttr = ExpMeta->GetConstAttribute(AttrName);
			const FPCGMetadataAttributeBase* ActAttr = ActMeta->GetConstAttribute(AttrName);

			if (!Test->TestNotNull(*FString::Printf(TEXT("Attribute '%s' expected base"), *AttrName.ToString()), ExpAttr)
				|| !Test->TestNotNull(*FString::Printf(TEXT("Attribute '%s' actual base"), *AttrName.ToString()), ActAttr))
			{
				bPassed = false;
				continue;
			}

			// Type-dispatch to compare values per entry.
			bPassed &= PCGMetadataAttribute::CallbackWithRightType(ExpAttr->GetTypeId(),
				[&](auto Dummy) -> bool
				{
					using T = decltype(Dummy);
					const auto* TypedExp = static_cast<const FPCGMetadataAttribute<T>*>(ExpAttr);
					const auto* TypedAct = static_cast<const FPCGMetadataAttribute<T>*>(ActAttr);
					bool bAttrPassed = true;

					for (int32 Entry = 0; Entry < NumEntries; ++Entry)
					{
						const T ExpVal = TypedExp->GetValueFromItemKey(static_cast<PCGMetadataEntryKey>(Entry));
						const T ActVal = TypedAct->GetValueFromItemKey(static_cast<PCGMetadataEntryKey>(Entry));

						if (!PCG::Private::MetadataTraits<T>::Equal(ExpVal, ActVal))
						{
							Test->AddError(FString::Printf(TEXT("[%d] Attribute '%s' value mismatch"), Entry, *AttrName.ToString()));
							bAttrPassed = false;
						}
					}

					return bAttrPassed;
				});
		}

		return bPassed;
	}

	bool CompareBasePointData(FAutomationTestBase* Test, const UPCGBasePointData* Expected, const UPCGBasePointData* Actual)
	{
		if (!Test->TestNotNull(TEXT("Expected point data"), Expected) || !Test->TestNotNull(TEXT("Actual point data"), Actual))
		{
			return false;
		}

		const int32 NumPoints = Expected->GetNumPoints();
		if (!Test->TestEqual(TEXT("Point count"), Actual->GetNumPoints(), NumPoints))
		{
			return false;
		}

		if (NumPoints == 0)
		{
			return true;
		}

		bool bPassed = true;
		const FConstPCGPointValueRanges E(Expected);
		const FConstPCGPointValueRanges A(Actual);

		for (int32 i = 0; i < NumPoints; ++i)
		{
			// Position: world units (cm), float vs double precision loss
			bPassed &= Test->TestNearlyEqual(*FString::Printf(TEXT("[%d] Position"), i), A.TransformRange[i].GetLocation(), E.TransformRange[i].GetLocation(), 1.0);

			// Rotation: quaternion dot product (1.0 = identical, 0.001 tolerance ≈ 2.6 degrees)
			const FQuat ERot = E.TransformRange[i].GetRotation();
			const FQuat ARot = A.TransformRange[i].GetRotation();
			const double RotDiff = 1.0 - FMath::Abs(ERot | ARot);
			bPassed &= Test->TestTrue(*FString::Printf(TEXT("[%d] Rotation (dot diff %.6f)"), i, RotDiff), RotDiff < 0.001);

			bPassed &= Test->TestNearlyEqual(*FString::Printf(TEXT("[%d] Scale"), i), A.TransformRange[i].GetScale3D(), E.TransformRange[i].GetScale3D(), 0.01f);
			bPassed &= Test->TestNearlyEqual(*FString::Printf(TEXT("[%d] Density"), i), static_cast<double>(A.DensityRange[i]), static_cast<double>(E.DensityRange[i]), 0.01);
			bPassed &= Test->TestEqual(*FString::Printf(TEXT("[%d] Seed"), i), A.SeedRange[i], E.SeedRange[i]);
			bPassed &= Test->TestNearlyEqual(*FString::Printf(TEXT("[%d] BoundsMin"), i), A.BoundsMinRange[i], E.BoundsMinRange[i], 0.01f);
			bPassed &= Test->TestNearlyEqual(*FString::Printf(TEXT("[%d] BoundsMax"), i), A.BoundsMaxRange[i], E.BoundsMaxRange[i], 0.01f);
			bPassed &= Test->TestNearlyEqual(*FString::Printf(TEXT("[%d] Steepness"), i), static_cast<double>(A.SteepnessRange[i]), static_cast<double>(E.SteepnessRange[i]), 0.01);

			const FVector4& EColor = E.ColorRange[i];
			const FVector4& AColor = A.ColorRange[i];
			bPassed &= Test->TestNearlyEqual(*FString::Printf(TEXT("[%d] Color R"), i), AColor.X, EColor.X, 0.01);
			bPassed &= Test->TestNearlyEqual(*FString::Printf(TEXT("[%d] Color G"), i), AColor.Y, EColor.Y, 0.01);
			bPassed &= Test->TestNearlyEqual(*FString::Printf(TEXT("[%d] Color B"), i), AColor.Z, EColor.Z, 0.01);
			bPassed &= Test->TestNearlyEqual(*FString::Printf(TEXT("[%d] Color A"), i), AColor.W, EColor.W, 0.01);
		}

		// Metadata attributes
		bPassed &= CompareMetadata(Test, Expected->Metadata.Get(), Actual->Metadata.Get(), NumPoints);

		return bPassed;
	}

	bool CompareDataCollections(FAutomationTestBase* Test, const FPCGDataCollection& Expected, const FPCGDataCollection& Actual, FTaggedDataComparator InComparator)
	{
		if (!Test->TestEqual(TEXT("Tagged data count"), Actual.TaggedData.Num(), Expected.TaggedData.Num()))
		{
			return false;
		}

		bool bPassed = true;

		for (int32 i = 0; i < Expected.TaggedData.Num(); ++i)
		{
			const FPCGTaggedData& ExpectedEntry = Expected.TaggedData[i];
			const FPCGTaggedData& ActualEntry = Actual.TaggedData[i];

			if (!Test->TestNotNull(*FString::Printf(TEXT("Data[%d] expected"), i), ExpectedEntry.Data.Get()))
			{
				bPassed = false;
				continue;
			}
			if (!Test->TestNotNull(*FString::Printf(TEXT("Data[%d] actual"), i), ActualEntry.Data.Get()))
			{
				bPassed = false;
				continue;
			}

			if (!Test->TestEqual(*FString::Printf(TEXT("Data[%d] type"), i), ActualEntry.Data->GetClass()->GetName(), ExpectedEntry.Data->GetClass()->GetName()))
			{
				bPassed = false;
				continue;
			}

			bPassed &= InComparator(Test, i, ExpectedEntry, ActualEntry);
		}

		return bPassed;
	}

	bool ComparePointDataCollections(FAutomationTestBase* Test, const FPCGDataCollection& Expected, const FPCGDataCollection& Actual)
	{
		return CompareDataCollections(Test, Expected, Actual, [](FAutomationTestBase* Test, int32 Index, const FPCGTaggedData& ExpectedEntry, const FPCGTaggedData& ActualEntry)
		{
			const UPCGBasePointData* ExpectedPoints = Cast<UPCGBasePointData>(ExpectedEntry.Data);
			const UPCGBasePointData* ActualPoints = Cast<UPCGBasePointData>(ActualEntry.Data);

			if (!Test->TestNotNull(*FString::Printf(TEXT("Data[%d] expected is point data"), Index), ExpectedPoints))
			{
				return false;
			}
			if (!Test->TestNotNull(*FString::Printf(TEXT("Data[%d] actual is point data"), Index), ActualPoints))
			{
				return false;
			}

			return CompareBasePointData(Test, ExpectedPoints, ActualPoints);
		});
	}
#endif // WITH_EDITOR
}

bool FPCGTestBaseClass::SmokeTestAnyValidInput(UPCGSettings* InSettings, TFunction<bool(const FPCGDataCollection&, const FPCGDataCollection&)> ValidationFn)
{
	TestTrue("Valid settings", InSettings != nullptr);

	if (!InSettings)
	{
		return false;
	}

	FPCGElementPtr Element = InSettings->GetElement();

	TestTrue("Valid element", Element != nullptr);

	if (!Element)
	{
		return false;
	}

	TArray<FPCGPinProperties> InputProperties = InSettings->AllInputPinProperties();
	// For each pin: take nothing, take 1 of any supported type, take 2 of any supported types (if enabled)
	TArray<TArray<FPCGDataCollection>> InputsPerProperties;
	TArray<uint32> InputIndices;

	if (!InputProperties.IsEmpty())
	{
		for (const FPCGPinProperties& InputProperty : InputProperties)
		{
			InputsPerProperties.Add(PCGTestsCommon::GenerateAllowedData(InputProperty));
			InputIndices.Add(0);
		}
	}
	else
	{
		TArray<FPCGDataCollection>& EmptyCollection = InputsPerProperties.Emplace_GetRef();
		EmptyCollection.Emplace();
		InputIndices.Add(0);
	}

	check(InputIndices.Num() == InputsPerProperties.Num());

	bool bDone = false;
	while (!bDone)
	{
		// Prepare input
		FPCGDataCollection InputData;
		InputData.TaggedData.Emplace_GetRef().Data = InSettings;

		for (int32 PinIndex = 0; PinIndex < InputIndices.Num(); ++PinIndex)
		{
			InputData.TaggedData.Append(InputsPerProperties[PinIndex][InputIndices[PinIndex]].TaggedData);
		}

		TUniquePtr<FPCGContext> Context = PCGTestsCommon::InitializeTestContext(Element.Get(), InputData, nullptr, nullptr);
		
		// Execute element until done
		while (!Element->Execute(Context.Get()))
		{
		}

		if (ValidationFn)
		{
			TestTrue("Validation", ValidationFn(Context->InputData, Context->OutputData));
		}

		// Bump indices
		int BumpIndex = 0;
		while (BumpIndex < InputIndices.Num())
		{
			if (InputIndices[BumpIndex] == InputsPerProperties[BumpIndex].Num() - 1)
			{
				InputIndices[BumpIndex] = 0;
				++BumpIndex;
			}
			else
			{
				++InputIndices[BumpIndex];
				break;
			}
		}

		if (BumpIndex == InputIndices.Num())
		{
			bDone = true;
		}
	}

	return true;
}
