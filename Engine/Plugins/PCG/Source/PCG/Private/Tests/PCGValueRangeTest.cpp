// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"
#include "Utils/PCGValueRange.h"

#include "Math/IntVector.h"

#define PCG_FMT(Text, ...) FString::Printf(TEXT(Text), __VA_ARGS__)

class FPCGValueRangeTestBaseClass : public FPCGTestBaseClass
{
public:
	using FPCGTestBaseClass::FPCGTestBaseClass;

	template <EPCGPointNativeProperties Property>
	bool TestRange()
	{
		const int32 NumPoints = 100;
		const int32 Seed = 42;

		UPCGPointData* PointDataA = PCGTestsCommon::CreateRandomPointData<UPCGPointData>(NumPoints, Seed, /*bRandomDensity=*/true);
		UPCGPointArrayData* PointDataB = PCGTestsCommon::CreateRandomPointData<UPCGPointArrayData>(NumPoints, Seed, /*bRandomDensity=*/true);

		const typename TPCGPointNativeProperty<Property>::ConstValueRange RangeA = PointDataA->GetConstValueRange<Property>();
		const typename TPCGPointNativeProperty<Property>::ConstValueRange RangeB = PointDataB->GetConstValueRange<Property>();

		UTEST_EQUAL("RangeA.Num() == RangeB.Num()", RangeA.Num(), RangeB.Num());

		for (int32 i = 0; i < RangeA.Num(); ++i)
		{
			UTEST_EQUAL(PCG_FMT("RangeA[%d] == RangeB[%d]", i, i), RangeA[i], RangeB[i]);
		}

		return true;
	};
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGValueRangeTest_Base, FPCGValueRangeTestBaseClass, "Plugins.PCG.ValueRange.Base", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGValueRangeTest_PointData_Transform, FPCGValueRangeTestBaseClass, "Plugins.PCG.ValueRange.PointData.Transform", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGValueRangeTest_PointData_Steepness, FPCGValueRangeTestBaseClass, "Plugins.PCG.ValueRange.PointData.Steepness", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGValueRangeTest_PointData_BoundsMin, FPCGValueRangeTestBaseClass, "Plugins.PCG.ValueRange.PointData.BoundsMin", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGValueRangeTest_PointData_BoundsMax, FPCGValueRangeTestBaseClass, "Plugins.PCG.ValueRange.PointData.BoundsMax", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGValueRangeTest_PointData_Color, FPCGValueRangeTestBaseClass, "Plugins.PCG.ValueRange.PointData.Color", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGValueRangeTest_PointData_Seed, FPCGValueRangeTestBaseClass, "Plugins.PCG.ValueRange.PointData.Seed", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGValueRangeTest_PointData_Density, FPCGValueRangeTestBaseClass, "Plugins.PCG.ValueRange.PointData.Density", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGValueRangeTest_PointData_MetadataEntry, FPCGValueRangeTestBaseClass, "Plugins.PCG.ValueRange.PointData.MetadataEntry", PCGTestsCommon::TestFlags)

bool FPCGValueRangeTest_Base::RunTest(const FString& Parameters)
{
	TConstPCGValueRange<float> FloatRange;

	// Default range is empty
	UTEST_EQUAL("FloatRange is empty", FloatRange.Num(), 0);

	// Default range Num elements is equal to the underlying stride view Num elements
	UTEST_EQUAL("FloatRange Num() == ViewNum()", FloatRange.Num(), FloatRange.ViewNum());

	// Default range doesn't have a single value because it has no values
	UTEST_FALSE("FloatRange is not single value", FloatRange.GetSingleValue().IsSet());

	TArray<int> IntArray{ 1, 2, 3 };
	TConstPCGValueRange<int> IntRange(MakeConstStridedView(IntArray));

	// Range constructed from strided view has same elements as strided view
	UTEST_EQUAL("IntRange num elements", IntRange.Num(), 3);
	UTEST_EQUAL("IntRange Num() == ViewNum()", IntRange.Num(), IntRange.ViewNum());
	
	// Test values
	UTEST_EQUAL("Test IntRange[0] value", IntRange[0], 1);
	UTEST_EQUAL("Test IntRange[1] value", IntRange[1], 2);
	UTEST_EQUAL("Test IntRange[2] value", IntRange[2], 3);

	// Mutliple value range doesn't have a single value because it has multiple values
	UTEST_FALSE("IntRange is not single value", FloatRange.GetSingleValue().IsSet());

	const UE::Math::TIntVector3<int> IntVectorA = { 1, 2, 3 };
	const UE::Math::TIntVector3<int> IntVectorB = { 4, 5, 6 };
	const UE::Math::TIntVector3<int> IntVectorC = { 7, 8, 9 };
	const TArray<UE::Math::TIntVector3<int>> IntVectorArray{ IntVectorA, IntVectorB, IntVectorC };
	const int NumElements = 20;
	TConstPCGValueRange<UE::Math::TIntVector3<int>> IntVectorRange(MakeConstStridedView(IntVectorArray), NumElements);

	// Range constructed with specific element count
	UTEST_EQUAL("IntVectorRange num elements", IntVectorRange.Num(), NumElements);
	UTEST_EQUAL("IntVectorRange num view elements", IntVectorRange.ViewNum(), IntVectorArray.Num())

	// Validate range values against view 
	for (int32 i = 0; i < IntVectorRange.Num(); ++i)
	{
		UTEST_EQUAL("Validate IntVectorRange Value against IntVectorArray", IntVectorRange[i], IntVectorArray[i % IntVectorArray.Num()]);
	}

	// Test values
	UTEST_EQUAL("Test IntVectorRange[0].X value", IntVectorRange[0].X, IntVectorA.X);
	UTEST_EQUAL("Test IntVectorRange[1].Y value", IntVectorRange[1].Y, IntVectorB.Y);
	UTEST_EQUAL("Test IntVectorRange[2].Z value", IntVectorRange[2].Z, IntVectorC.Z);

	const TArray<UE::Math::TIntVector3<int>> IntVectorArraySingleValue{ IntVectorA };
	TConstPCGValueRange<UE::Math::TIntVector3<int>> SingleIntVectorRange(MakeConstStridedView(IntVectorArraySingleValue), NumElements);

	// Single value range
	UTEST_EQUAL("SingleIntVectorRange num elements", SingleIntVectorRange.Num(), NumElements);
	UTEST_GREATER("SingleIntVectorRange Num() > ViewNum()", SingleIntVectorRange.Num(), SingleIntVectorRange.ViewNum());

	UTEST_TRUE("SingleIntVectorRange is single value", SingleIntVectorRange.GetSingleValue().IsSet());

	UTEST_EQUAL("Test SingleIntVectorRange SingleValue()", SingleIntVectorRange.GetSingleValue().GetValue(), IntVectorA);
	UTEST_EQUAL("Test equality index 0 and index NumElements-1", SingleIntVectorRange[0], SingleIntVectorRange[NumElements - 1]);

	return true;
}

bool FPCGValueRangeTest_PointData_Transform::RunTest(const FString& Parameters)
{
	return TestRange<EPCGPointNativeProperties::Transform>();
}

bool FPCGValueRangeTest_PointData_Steepness::RunTest(const FString& Parameters)
{
	return TestRange<EPCGPointNativeProperties::Steepness>();
}

bool FPCGValueRangeTest_PointData_BoundsMin::RunTest(const FString& Parameters)
{
	return TestRange<EPCGPointNativeProperties::BoundsMin>();
}

bool FPCGValueRangeTest_PointData_BoundsMax::RunTest(const FString& Parameters)
{
	return TestRange<EPCGPointNativeProperties::BoundsMax>();
}

bool FPCGValueRangeTest_PointData_Color::RunTest(const FString& Parameters)
{
	return TestRange<EPCGPointNativeProperties::Color>();
}

bool FPCGValueRangeTest_PointData_Seed::RunTest(const FString& Parameters)
{
	return TestRange<EPCGPointNativeProperties::Seed>();
}

bool FPCGValueRangeTest_PointData_Density::RunTest(const FString& Parameters)
{
	return TestRange<EPCGPointNativeProperties::Density>();
}

bool FPCGValueRangeTest_PointData_MetadataEntry::RunTest(const FString& Parameters)
{
	return TestRange<EPCGPointNativeProperties::MetadataEntry>();
}

#endif // WITH_EDITOR