// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Parsing/PCGParsing.h"
#include "Tests/PCGTestsCommon.h"

namespace PCGParsingTestConstants
{
	// Arbitrary constant used to determine the hypothetical array size
	static constexpr int32 ArraySize = 512;
}

namespace PCGParsingTestPrivate
{
	PCGIndexing::FPCGIndexCollection ParseAndRetrieveIndices(const FString& Expression, const int32 ArraySize)
	{
		PCGIndexing::FPCGIndexCollection IndexCollection(ArraySize);
		PCGParser::ParseIndexRanges(IndexCollection, Expression);
		return IndexCollection;
	}
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParsingTest_Basic, FPCGTestBaseClass, "Plugins.PCG.Parsing.Basic", PCGTestsCommon::TestFlags)

bool FPCGParsingTest_Basic::RunTest(const FString& Parameters)
{
	using PCGParsingTestConstants::ArraySize;

	// Simple index
	{
		const PCGIndexing::FPCGIndexCollection Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("4", ArraySize);
		UTEST_EQUAL("Singular non-range index has 1 index", Results.GetTotalIndexCount(), 1);
		UTEST_EQUAL("Singular non-range index has 1 range", Results.GetTotalRangeCount(), 1);
		UTEST_TRUE("Singular non-range index is correct", Results.ContainsIndex(4));
	}

	// Final index
	{
		const PCGIndexing::FPCGIndexCollection Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("-1", ArraySize);
		UTEST_EQUAL("Single final index has 1 index", Results.GetTotalIndexCount(), 1);
		UTEST_EQUAL("Single final index has 1 range", Results.GetTotalRangeCount(), 1);
		UTEST_TRUE("Single final index is correct", Results.ContainsIndex(ArraySize - 1));
	}

	// Simple negative index
	{
		const PCGIndexing::FPCGIndexCollection Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("-4", ArraySize);
		UTEST_EQUAL("Negative non-range index has 1 index", Results.GetTotalIndexCount(), 1);
		UTEST_EQUAL("Negative non-range index has 1 range", Results.GetTotalRangeCount(), 1);
		UTEST_TRUE("Negative non-range index is correct", Results.ContainsIndex(ArraySize - 4));
	}

	// Two simple indices
	{
		const PCGIndexing::FPCGIndexCollection Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("4,-4", ArraySize);
		UTEST_EQUAL("Multiple index has 2 index", Results.GetTotalIndexCount(), 2);
		UTEST_EQUAL("Multiple index has 2 ranges", Results.GetTotalRangeCount(), 2);
		UTEST_TRUE("Multiple index (first) is correct", Results.ContainsIndex(4) && Results.ContainsIndex(ArraySize - 4));
	}

	// Repeated index
	{
		const PCGIndexing::FPCGIndexCollection Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("2:2", ArraySize);
		UTEST_EQUAL("Repeated index has 1 index", Results.GetTotalIndexCount(), 1);
		UTEST_EQUAL("Repeated index has 1 range", Results.GetTotalRangeCount(), 1);
		UTEST_TRUE("Repeated index is correct", Results.ContainsIndex(2));
	}

	// More complex use case
	{
		TArray<int32> TestArray;
		TestArray.SetNumUninitialized(ArraySize);
		for (int I = 0; I < ArraySize; ++I)
		{
			TestArray[I] = I;
		}

		// Should be 0,2,3,4,5 and from 80 to 502
		const PCGIndexing::FPCGIndexCollection ForwardsResults = PCGParsingTestPrivate::ParseAndRetrieveIndices("0,2:5,80:-10", ArraySize);
		const PCGIndexing::FPCGIndexCollection BackwardsResults = PCGParsingTestPrivate::ParseAndRetrieveIndices("80:-10,2:5,0", ArraySize);

		UTEST_TRUE("Valid single index", ForwardsResults.ContainsIndex(0));
		UTEST_TRUE("Valid single index", BackwardsResults.ContainsIndex(0));
		UTEST_TRUE("Missing single index", !ForwardsResults.ContainsIndex(1));
		UTEST_TRUE("Missing single index", !BackwardsResults.ContainsIndex(1));

		for (int I = 2; I < 5; ++I)
		{
			UTEST_TRUE("Valid range index", ForwardsResults.ContainsIndex(I));
			UTEST_TRUE("Valid range index", BackwardsResults.ContainsIndex(I));
		}

		for (int I = 6; I < 80; ++I)
		{
			UTEST_TRUE("Missing range index", !ForwardsResults.ContainsIndex(I));
			UTEST_TRUE("Missing range index", !BackwardsResults.ContainsIndex(I));
		}

		for (int I = 80; I < ArraySize - 10; ++I)
		{
			UTEST_TRUE("Valid range negative index", ForwardsResults.ContainsIndex(I));
			UTEST_TRUE("Valid range negative index", BackwardsResults.ContainsIndex(I));
		}

		for (int I = ArraySize - 9; I < ArraySize; ++I)
		{
			UTEST_TRUE("Missing range negative index", !ForwardsResults.ContainsIndex(I));
			UTEST_TRUE("Missing range negative index", !BackwardsResults.ContainsIndex(I));
		}
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParsingTest_Ranges, FPCGTestBaseClass, "Plugins.PCG.Parsing.Ranges", PCGTestsCommon::TestFlags)

bool FPCGParsingTest_Ranges::RunTest(const FString& Parameters)
{
	using PCGParsingTestConstants::ArraySize;
	PCGIndexing::FPCGIndexCollection Results(ArraySize);

	auto CheckSplitTestArray = [this, &Results](const int32 SplitIndex)
	{
		for (int I = 0; I < ArraySize; ++I)
		{
			if (I < SplitIndex != Results.ContainsIndex(I))
			{
				return false;
			}
		}

		return true;
	};

	// Basic ranges
	// ------------

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0:1", ArraySize);
	UTEST_EQUAL("Sequential single index results in a single range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Sequential single index results in a single index", Results.GetTotalIndexCount(), 1);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(1));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0:10", ArraySize);
	UTEST_EQUAL("Single index range results in a single range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Single index range results in correct number of indices", Results.GetTotalIndexCount(), 10);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(10));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices(":10", ArraySize);
	UTEST_EQUAL("Inferred start index results in a single range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Inferred start index results in correct number of indices", Results.GetTotalIndexCount(), 10);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(10));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices(":", ArraySize);
	UTEST_EQUAL("Full range has single range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Full range has all indices", Results.GetTotalIndexCount(), ArraySize);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(ArraySize));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0:", ArraySize);
	UTEST_EQUAL("Inferred end index results in a single range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Inferred end index results in correct number of indices", Results.GetTotalIndexCount(), ArraySize);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(ArraySize));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices(":-4", ArraySize);
	UTEST_EQUAL("Inferred start index, negative end index results in a single range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Inferred start index, negative end results in correct number of indices", Results.GetTotalIndexCount(), ArraySize - 4);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(ArraySize - 4));

	// Complete overlap
	// ----------------

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0:10,2:5", ArraySize);
	UTEST_EQUAL("Complete preceeding overlap collapsed into single index range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Complete preceeding overlap collapsed into correct index count", Results.GetTotalIndexCount(), 10);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(10));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("2:5,0:10", ArraySize);
	UTEST_EQUAL("Complete subsequent overlap collapsed into single index range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Complete subsequent overlap collapsed into correct index count", Results.GetTotalIndexCount(), 10);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(10));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0,1,2,3:50,0:512,80:250", ArraySize);
	UTEST_EQUAL("Multiple complete overlaps collapsed into single index range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Multiple complete overlaps collapsed into correct index count", Results.GetTotalIndexCount(), 512);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(ArraySize));

	// Partial overlap
	// ---------------

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0:50,25:75", ArraySize);
	UTEST_EQUAL("Single partial overlap is collapsed with single range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Single partial overlap is collapsed with correct index count", Results.GetTotalIndexCount(), 75);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(75));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0:50,25:75,1:5,13:72,30:40,40:50", ArraySize);
	UTEST_EQUAL("Multiple partial overlap is collapsed with single range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Multiple partial overlap is collapsed with correct index count", Results.GetTotalIndexCount(), 75);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(75));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0:2,1:3,2:4,3:5,4:6,5:7", ArraySize);
	UTEST_EQUAL("Multiple partial overlap is collapsed with single range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Multiple partial overlap is collapsed with correct index count", Results.GetTotalIndexCount(), 7);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(7));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0:26,26:50", ArraySize);
	UTEST_EQUAL("Multiple partial overlap is collapsed with one range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Multiple partial overlap is collapsed with correct index count", Results.GetTotalIndexCount(), 50);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(50));

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0:20,10:31,31:51,41:62,62:82,72:93", ArraySize);
	UTEST_EQUAL("Multiple partial overlap is collapsed with one range", Results.GetTotalRangeCount(), 1);
	UTEST_EQUAL("Multiple partial overlap is collapsed with correct index count", Results.GetTotalIndexCount(), 93);
	UTEST_TRUE("Included indices verified", CheckSplitTestArray(93));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGParsingTest_Robustness, FPCGTestBaseClass, "Plugins.PCG.Parsing.Robustness", PCGTestsCommon::TestFlags)

bool FPCGParsingTest_Robustness::RunTest(const FString& Parameters)
{
	using PCGParsingTestConstants::ArraySize;
	PCGIndexing::FPCGIndexCollection Results(ArraySize);

	// Copy
	{
		PCGIndexing::FPCGIndexCollection Original(42);
		Original.AddRange(21, 41);
		PCGIndexing::FPCGIndexCollection Copy(Original);
		UTEST_EQUAL("Copy construction", Original, Copy);
	}

	// Assignment
	{
		PCGIndexing::FPCGIndexCollection Original(42);
		Original.AddRange(21, 41);
		PCGIndexing::FPCGIndexCollection Copy = Original;
		UTEST_EQUAL("Assignment operator", Original, Copy);
	}

	// Forgivable user mistakes
	// ------------------------

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("00000000000000000", ArraySize);
	UTEST_EQUAL("Repeated 0s results in 0 index only", Results.GetTotalIndexCount(), 1);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("1:2", ArraySize);
	UTEST_EQUAL("Single index range created", Results.GetTotalIndexCount(), 1);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("2,", ArraySize);
	UTEST_EQUAL("Empty value ignored, after", Results.GetTotalIndexCount(), 1);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices(",2", ArraySize);
	UTEST_EQUAL("Empty value ignored, before", Results.GetTotalIndexCount(), 1);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices(",,,,2,,,", ArraySize);
	UTEST_EQUAL("Empty value ignored, multiple", Results.GetTotalIndexCount(), 1);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0,*", ArraySize);
	UTEST_EQUAL("Invalid character has no result", Results.GetTotalIndexCount(), 1);

	// Unforgivable mistakes
	// ---------------------

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("", ArraySize);
	UTEST_EQUAL("Empty expression has no result", Results.GetTotalIndexCount(), 0);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices(",", ArraySize);
	UTEST_EQUAL("Element delimiter only has no result", Results.GetTotalIndexCount(), 0);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("#", ArraySize);
	UTEST_EQUAL("Invalid character has no result", Results.GetTotalIndexCount(), 0);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0(,5:6", ArraySize);
	UTEST_EQUAL("Invalid character has no result", Results.GetTotalIndexCount(), 0);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("3:", ArraySize);
	UTEST_EQUAL("Range end index missing has no result", Results.GetTotalIndexCount(), ArraySize - 3);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("0,1,2,5:2", ArraySize);
	UTEST_EQUAL("Inverted range is excluded in the results", Results.GetTotalIndexCount(), 3);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("5:2", ArraySize);
	UTEST_EQUAL("Inverted range has no result", Results.GetTotalIndexCount(), 0);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices(FString::FromInt(ArraySize), ArraySize);
	UTEST_EQUAL("Selected array max size", Results.GetTotalIndexCount(), 0);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices(FString::FromInt(ArraySize + 1), ArraySize);
	UTEST_EQUAL("Index outside the bounds of the array", Results.GetTotalIndexCount(), 0);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices(FString::FromInt(-1 * (ArraySize + 1)), ArraySize);
	UTEST_EQUAL("Index outside the bounds of the array", Results.GetTotalIndexCount(), 0);

#if !PLATFORM_MAC && !PLATFORM_LINUX  // @todo_pcg: Mac/Linux Atoi handles this kind of overflow slightly differently, returning -1, which is a valid index. Disabling temporarily until further investigation with the core team.
	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("65465465498475309485730495873409587654684986146162342348438", std::numeric_limits<int32>::max());
	UTEST_EQUAL("Large value converts to int32 max which is the max size of the array which fails with no result", Results.GetTotalIndexCount(), 0);
#endif

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices(":::::::::", ArraySize);
	UTEST_EQUAL("Range delimiter only has no result", Results.GetTotalIndexCount(), 0);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("1:::::::::", ArraySize);
	UTEST_EQUAL("Multiple range delimiter with start index has no result", Results.GetTotalIndexCount(), 0);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("::::::::::1", ArraySize);
	UTEST_EQUAL("Multiple range delimiter with end index has no result", Results.GetTotalIndexCount(), 0);

	Results = PCGParsingTestPrivate::ParseAndRetrieveIndices("1::::::::::1", ArraySize);
	UTEST_EQUAL("Redundant delimiter between indices has no result", Results.GetTotalIndexCount(), 0);

	return true;
}
