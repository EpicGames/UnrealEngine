// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/PCGTestsCommon.h"

#include "Data/PCGPointData.h"
#include "SpatialAlgo/PCGOctreeQueries.h"

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGOctreeQueriesTest_Sphere, FPCGTestBaseClass, "Plugins.PCG.OctreeQueries.Sphere", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGOctreeQueriesTest_ClosestPoint, FPCGTestBaseClass, "Plugins.PCG.OctreeQueries.ClosestPoint", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGOctreeQueriesTest_ClosestPointDiscardCenter, FPCGTestBaseClass, "Plugins.PCG.OctreeQueries.ClosestPointDiscardCenter", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGOctreeQueriesTest_ClosestPointFromOtherPoint, FPCGTestBaseClass, "Plugins.PCG.OctreeQueries.ClosestPointFromOtherPoint", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGOctreeQueriesTest_FarthestPoint, FPCGTestBaseClass, "Plugins.PCG.OctreeQueries.FarthestPoint", PCGTestsCommon::TestFlags)
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGOctreeQueriesTest_FarthestPointFromOtherPoint, FPCGTestBaseClass, "Plugins.PCG.OctreeQueries.FarthestPointFromOtherPoint", PCGTestsCommon::TestFlags)

/**
* For all the tests, the point data will be points scattered on a straight line (Direction {1, 1, 1}), and evenly spaced.
* Query will be on the first point, at the origin.
*/
namespace PCGOctreeQueriesTest
{
	static constexpr double Distance = 100.0;
	static constexpr int32 NumPoints = 10;

	UPCGBasePointData* CreatePointData()
	{
		UPCGBasePointData* InputPointData = PCGTestsCommon::CreateEmptyBasePointData();
		InputPointData->SetNumPoints(NumPoints);
		InputPointData->SetDensity(1.0f);
		InputPointData->SetSeed(42);

		TPCGValueRange<FTransform> TransformRange = InputPointData->GetTransformValueRange();

		for (int32 i = 0; i < NumPoints; ++i)
		{
			FVector Location = FVector::OneVector * Distance * i;
			TransformRange[i] = FTransform(Location);
		}

		return InputPointData;
	}
}


bool FPCGOctreeQueriesTest_Sphere::RunTest(const FString& Parameters)
{
	const UPCGBasePointData* InputPointData = PCGOctreeQueriesTest::CreatePointData();
	check(InputPointData);

	TArray<int32> ExpectedIndexes = { 0, 1, 2 };
	int32 CountFound = 0;
	UPCGOctreeQueries::ForEachPointInsideSphere(InputPointData, FVector::ZeroVector, 350.0, [&ExpectedIndexes, &CountFound](const UPCGBasePointData* PointData, int32 PointIndex, double)
	{
		if (ExpectedIndexes.Contains(PointIndex))
		{
			++CountFound;
		}
	});

	UTEST_EQUAL("We found the expected number of points", CountFound, ExpectedIndexes.Num());

	CountFound = 0;
	UPCGOctreeQueries::ForEachPointInsideSphere(InputPointData, FVector::ZeroVector, 350.0, [&ExpectedIndexes, &CountFound](const UPCGBasePointData* PointData, int32 PointIndex, double)
	{
		if (ExpectedIndexes.Contains(PointIndex))
		{
			++CountFound;
		}
	});

	UTEST_EQUAL("We found the expected number of points", CountFound, ExpectedIndexes.Num());

	return true;
}

bool FPCGOctreeQueriesTest_ClosestPoint::RunTest(const FString& Parameters)
{
	const UPCGBasePointData* InputPointData = PCGOctreeQueriesTest::CreatePointData();
	check(InputPointData);

	const int32 PointIndex = UPCGOctreeQueries::GetClosestPointIndex(InputPointData, FVector::ZeroVector, /*bDiscardCenter=*/false, 350.0);
	UTEST_NOT_EQUAL("Closest point was found", PointIndex, (int32)INDEX_NONE);

	UTEST_EQUAL("Closest point is the right index", PointIndex, 0);
	return true;
}

bool FPCGOctreeQueriesTest_ClosestPointDiscardCenter::RunTest(const FString& Parameters)
{
	const UPCGBasePointData* InputPointData = PCGOctreeQueriesTest::CreatePointData();
	check(InputPointData);

	const int32 PointIndex = UPCGOctreeQueries::GetClosestPointIndex(InputPointData, FVector::ZeroVector, /*bDiscardCenter=*/true, 350.0);
	UTEST_NOT_EQUAL("Closest point was found", PointIndex, (int32)INDEX_NONE);

	UTEST_EQUAL("Closest point is the right index", PointIndex, 1);
	return true;
}

bool FPCGOctreeQueriesTest_ClosestPointFromOtherPoint::RunTest(const FString& Parameters)
{
	const UPCGBasePointData* InputPointData = PCGOctreeQueriesTest::CreatePointData();
	check(InputPointData);

	const int32 PointIndex = UPCGOctreeQueries::GetClosestPointIndexFromOtherPointIndex(InputPointData, 0, 350.0);
	UTEST_NOT_EQUAL("Closest point was found", PointIndex, (int32)INDEX_NONE);

	UTEST_EQUAL("Closest point is the right index", PointIndex, 1);
	return true;
}

bool FPCGOctreeQueriesTest_FarthestPoint::RunTest(const FString& Parameters)
{
	const UPCGBasePointData* InputPointData = PCGOctreeQueriesTest::CreatePointData();
	check(InputPointData);

	const int32 PointIndex = UPCGOctreeQueries::GetFarthestPointIndex(InputPointData, FVector::ZeroVector, 350.0);
	UTEST_NOT_EQUAL("Farthest point was found", PointIndex, (int32)INDEX_NONE);

	UTEST_EQUAL("Farthest point is the right index", PointIndex, 2);
	return true;
}

bool FPCGOctreeQueriesTest_FarthestPointFromOtherPoint::RunTest(const FString& Parameters)
{
	const UPCGBasePointData* InputPointData = PCGOctreeQueriesTest::CreatePointData();
	check(InputPointData);

	const int32 PointIndex = UPCGOctreeQueries::GetFarthestPointIndexFromOtherPointIndex(InputPointData, 0, 10000.0);
	UTEST_NOT_EQUAL("Farthest point was found", PointIndex, (int32)INDEX_NONE);

	UTEST_EQUAL("Farthest point is the right index", PointIndex, 9);
	return true;
}

#endif // WITH_EDITOR