// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGInputOutputSettings.h"
#include "PCGPoint.h"
#include "PCGSubgraph.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGAttributeNoise.h"
#include "Elements/PCGSurfaceSampler.h"
#include "Elements/PCGUserParameterGet.h"
#include "Graph/PCGGraphExecutor.h"
#include "Helpers/PCGSubgraphHelpers.h"
#include "Tests/PCGTestsCommon.h"

#include "UObject/Package.h"


IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCollapseSubgraphSimple, FPCGTestBaseClass, "Plugins.PCG.Subgraph.Collapse.Simple", PCGTestsCommon::TestFlags);
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCollapseSubgraphWithParams, FPCGTestBaseClass, "Plugins.PCG.Subgraph.Collapse.WithParams", PCGTestsCommon::TestFlags);
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPCGCollapseSubgraphCombinationOfCollapses, FPCGTestBaseClass, "Plugins.PCG.Subgraph.Collapse.Combination", PCGTestsCommon::TestFlags);

namespace PCGCollapseSubgraphTests
{
	/**
	* Utility function to test if an edge exists between two nodes of class InputClass and OutputClass.
	* You need to set at least InLabel or OutLabel to be different to NAME_None, if you only care if a pin is connected to a node and not a specific pin.
	* Only works if there is a single node of type InputClass or OutputClass (exception made for input and output node).
	*/
	template <typename InputClass, typename OutputClass>
	bool ValidateNodeAndEdge(FPCGTestBaseClass* InTestClass, UPCGGraph* InGraph, FName InLabel, FName OutLabel)
	{
		check(InGraph);

		if (!InTestClass->TestTrue("Test is not illed formed", (InLabel != NAME_None) || (OutLabel != NAME_None))) { return false; }

		UPCGNode* InNode = nullptr;
		UPCGNode* OutNode = nullptr;

		if constexpr (std::is_same_v<InputClass, UPCGGraphInputOutputSettings>)
		{
			InNode = InGraph->GetInputNode();
		}
		else
		{
			UPCGNode* const* NodePtr = InGraph->GetNodes().FindByPredicate([](const UPCGNode* Node) { return Node && Node->GetSettings() && Node->GetSettings()->IsA<InputClass>(); });
			InNode = NodePtr ? *NodePtr : nullptr;
		}

		if constexpr (std::is_same_v<OutputClass, UPCGGraphInputOutputSettings>)
		{
			OutNode = InGraph->GetOutputNode();
		}
		else
		{
			UPCGNode* const* NodePtr = InGraph->GetNodes().FindByPredicate([](const UPCGNode* Node) { return Node && Node->GetSettings() && Node->GetSettings()->IsA<OutputClass>(); });
			OutNode = NodePtr ? *NodePtr : nullptr;
		}

		if (!InTestClass->TestNotNull(*FString::Printf(TEXT("Input Node of class %s is valid"), *InputClass::StaticClass()->GetName()), InNode)) { return false; }
		if (!InTestClass->TestNotNull(*FString::Printf(TEXT("Output Node of class %s is valid"), *OutputClass::StaticClass()->GetName()), OutNode)) { return false; }

		check(InNode && OutNode);

		UPCGPin* InPin = InNode->GetOutputPin(InLabel);
		UPCGPin* OutPin = OutNode->GetInputPin(OutLabel);

		if (!InTestClass->TestTrue(
			*FString::Printf(TEXT("InLabel (%s) exists on node %s"), *InLabel.ToString(), *InNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString()),
			(InLabel == NAME_None) || InPin))
		{
			return false;
		}
		if (!InTestClass->TestTrue(
			*FString::Printf(TEXT("OutLabel (%s) exists on node %s"), *OutLabel.ToString(), *OutNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString()),
			(OutLabel == NAME_None) || OutPin))
		{
			return false;
		}

		check(InPin || OutPin);

		const TArray<TObjectPtr<UPCGEdge>>& Edges = InPin ? InPin->Edges : OutPin->Edges;

		auto Predicate = [InPin, OutPin](const TObjectPtr<UPCGEdge>& InEdge)
		{
			if (InPin && OutPin)
			{
				return (InEdge->InputPin == InPin) && (InEdge->OutputPin == OutPin);
			}
			else if (OutPin)
			{
				return (InEdge->OutputPin == OutPin) && InEdge->InputPin && InEdge->InputPin->Node && InEdge->InputPin->Node->GetSettings() && InEdge->InputPin->Node->GetSettings()->IsA<InputClass>();
			}
			else
			{
				return (InEdge->InputPin == InPin) && InEdge->OutputPin && InEdge->OutputPin->Node && InEdge->OutputPin->Node->GetSettings() && InEdge->OutputPin->Node->GetSettings()->IsA<OutputClass>();
			}
		};

		return InTestClass->TestTrue(
			*FString::Printf(TEXT("Edge between %s(%s) and %s(%s) exists"),
				*InNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString(),
				*InLabel.ToString(),
				*OutNode->GetNodeTitle(EPCGNodeTitleType::ListView).ToString(),
				*OutLabel.ToString()),
			Edges.ContainsByPredicate(Predicate));
	}

	UPCGNode* AddGetGraphParameterNode(UPCGGraph* InGraph, const FName InParamName)
	{
		const FPropertyBagPropertyDesc* Desc = InGraph->GetUserParametersStruct()->FindPropertyDescByName(InParamName);
		if (!Desc)
		{
			return nullptr;
		}

		UPCGUserParameterGetSettings* Settings = nullptr;
		UPCGNode* GetParameterNode = InGraph->AddNodeOfType<UPCGUserParameterGetSettings>(Settings);

		Settings->PropertyName = InParamName;
		Settings->PropertyGuid = Desc->ID;

		GetParameterNode->UpdateAfterSettingsChangeDuringCreation();

		return GetParameterNode;
	}
}

#define VALIDATE_EDGE(InSettings, OutSettings, Graph, InLabel, OutLabel) if (!PCGCollapseSubgraphTests::ValidateNodeAndEdge<InSettings, OutSettings>(this, Graph, InLabel, OutLabel)) { return false; }

bool FPCGCollapseSubgraphSimple::RunTest(const FString& Parameters)
{
	UPCGGraph* MainGraph = NewObject<UPCGGraph>();
	MainGraph->SetFlags(RF_Transient);

	// Add a few nodes:
	// * Surface sampler connected to the landscape pin
	// * Attribute noise on the output of the surface sampler, connected to the output

	UPCGNode* InputNode = MainGraph->GetInputNode();
	UPCGNode* OutputNode = MainGraph->GetOutputNode();

	UTEST_NOT_NULL("InputNode is valid", InputNode);
	UTEST_NOT_NULL("OutputNode is valid", OutputNode);

	// Add a Landscape pin to the input node, and update the node
	const FPCGPinProperties& InputLandscapePin = CastChecked<UPCGGraphInputOutputSettings>(InputNode->GetSettings())->AddPin(FPCGPinProperties(PCGInputOutputConstants::DefaultLandscapeLabel, EPCGDataType::Landscape));
	InputNode->UpdateAfterSettingsChangeDuringCreation();

	UPCGSurfaceSamplerSettings* SurfaceSamplerSettings = nullptr;
	UPCGAttributeNoiseSettings* AttributeNoiseSettings = nullptr;
	UPCGNode* SurfaceSamplerNode = MainGraph->AddNodeOfType<UPCGSurfaceSamplerSettings>(SurfaceSamplerSettings);
	UPCGNode* AttributeNoiseNode = MainGraph->AddNodeOfType<UPCGAttributeNoiseSettings>(AttributeNoiseSettings);

	UTEST_NOT_NULL("SurfaceSampler node is valid", SurfaceSamplerNode);
	UTEST_NOT_NULL("AttributeNoise node is valid", AttributeNoiseNode);
	UTEST_EQUAL("There are 2 nodes", MainGraph->GetNodes().Num(), 2);

	MainGraph->AddLabeledEdge(InputNode, InputLandscapePin.Label, SurfaceSamplerNode, TEXT("Surface"));
	MainGraph->AddLabeledEdge(SurfaceSamplerNode, PCGPinConstants::DefaultOutputLabel, AttributeNoiseNode, PCGPinConstants::DefaultInputLabel);
	MainGraph->AddLabeledEdge(AttributeNoiseNode, PCGPinConstants::DefaultOutputLabel, OutputNode, PCGPinConstants::DefaultOutputLabel);

	VALIDATE_EDGE(UPCGGraphInputOutputSettings, UPCGSurfaceSamplerSettings, MainGraph, InputLandscapePin.Label, TEXT("Surface"))
	VALIDATE_EDGE(UPCGSurfaceSamplerSettings, UPCGAttributeNoiseSettings, MainGraph, PCGPinConstants::DefaultOutputLabel, PCGPinConstants::DefaultInputLabel)
	VALIDATE_EDGE(UPCGAttributeNoiseSettings, UPCGGraphInputOutputSettings, MainGraph, PCGPinConstants::DefaultOutputLabel, PCGPinConstants::DefaultOutputLabel)

	// Try to collapse all the nodes. Input/Output should stay, the other not.
	TArray<UPCGNode*> NodesToCollapse = { InputNode, OutputNode, SurfaceSamplerNode, AttributeNoiseNode };
	UPCGGraph* Subgraph = FPCGSubgraphHelpers::CollapseIntoSubgraph(MainGraph, NodesToCollapse, {});

	UTEST_NOT_NULL("Subgraph collapse succeeded", Subgraph);
	UTEST_EQUAL("There is 1 node in the main graph", MainGraph->GetNodes().Num(), 1);

	UPCGNode* SubgraphNode = MainGraph->GetNodes()[0];
	UTEST_NOT_NULL("SubgraphNode is valid", SubgraphNode);

	UPCGSubgraphSettings* SubgraphSettings = Cast<UPCGSubgraphSettings>(SubgraphNode->GetSettings());
	UTEST_NOT_NULL("Node is a subgraph node", SubgraphSettings);
	UTEST_EQUAL("Subgraph graph is the right one", SubgraphSettings->SubgraphInstance->GetGraph(), Subgraph);

	const FName InputPinSubgraphLabel = TEXT("Landscape");

	// Landscape pin from input is connected to created pin of subgraph node
	VALIDATE_EDGE(UPCGGraphInputOutputSettings, UPCGSubgraphSettings, MainGraph, InputLandscapePin.Label, InputPinSubgraphLabel)

	// Output node is connected to the subgraph node
	VALIDATE_EDGE(UPCGSubgraphSettings, UPCGGraphInputOutputSettings, MainGraph, NAME_None, PCGPinConstants::DefaultOutputLabel)

	// Same verification for subgraph than the initial main graph, expect for output pin
	UTEST_EQUAL("There is 2 nodes in the subgraph", Subgraph->GetNodes().Num(), 2);

	VALIDATE_EDGE(UPCGGraphInputOutputSettings, UPCGSurfaceSamplerSettings, Subgraph, InputPinSubgraphLabel, TEXT("Surface"))
	VALIDATE_EDGE(UPCGSurfaceSamplerSettings, UPCGAttributeNoiseSettings, Subgraph, PCGPinConstants::DefaultOutputLabel, PCGPinConstants::DefaultInputLabel)
	VALIDATE_EDGE(UPCGAttributeNoiseSettings, UPCGGraphInputOutputSettings, Subgraph, PCGPinConstants::DefaultOutputLabel, NAME_None);

	return true;
}

bool FPCGCollapseSubgraphWithParams::RunTest(const FString& Parameters)
{
	UPCGGraph* MainGraph = NewObject<UPCGGraph>();
	MainGraph->SetFlags(RF_Transient);

	const FName Param1Name = TEXT("FloatParam");
	const FName Param2Name = TEXT("VectorParam");
	const float Param1Value = 0.1f;
	const FVector Param2Value = FVector(1.0, 2.0, 3.0);

	TArray<FPropertyBagPropertyDesc> GraphParameters;
	GraphParameters.Emplace(Param1Name, EPropertyBagPropertyType::Float);
	GraphParameters.Emplace(Param2Name, EPropertyBagPropertyType::Struct, TBaseStructure<FVector>::Get());

	MainGraph->AddUserParameters(GraphParameters);
	UTEST_EQUAL("Graph has 2 parameters", MainGraph->GetUserParametersStruct()->GetNumPropertiesInBag(), 2);

	// Set the values
	MainGraph->GetMutableUserParametersStruct_Unsafe()->SetValueFloat(Param1Name, Param1Value);
	MainGraph->GetMutableUserParametersStruct_Unsafe()->SetValueStruct(Param2Name, Param2Value);

	UPCGSurfaceSamplerSettings* SurfaceSamplerSettings = nullptr;
	UPCGNode* SurfaceSamplerNode = MainGraph->AddNodeOfType<UPCGSurfaceSamplerSettings>(SurfaceSamplerSettings);

	UPCGNode* GetParameter1Node = PCGCollapseSubgraphTests::AddGetGraphParameterNode(MainGraph, Param1Name);
	UPCGNode* GetParameter2Node = PCGCollapseSubgraphTests::AddGetGraphParameterNode(MainGraph, Param2Name);

	MainGraph->AddLabeledEdge(GetParameter1Node, Param1Name, SurfaceSamplerNode, TEXT("Looseness"));
	MainGraph->AddLabeledEdge(GetParameter2Node, Param2Name, SurfaceSamplerNode, TEXT("PointExtents"));

	TArray<UPCGNode*> NodesToCollapse = { SurfaceSamplerNode, GetParameter1Node, GetParameter2Node };

	UPCGGraph* Subgraph = FPCGSubgraphHelpers::CollapseIntoSubgraph(MainGraph, NodesToCollapse, {});

	UTEST_NOT_NULL("Subgraph collapse succeeded", Subgraph);
	UTEST_EQUAL("There are 3 node in the main graph", MainGraph->GetNodes().Num(), 3);

	// Get parameters nodes are still there and plugged
	UTEST_TRUE("GetParam1 node is still in the main graph", MainGraph->GetNodes().Contains(GetParameter1Node));
	UTEST_TRUE("GetParam2 node is still in the main graph", MainGraph->GetNodes().Contains(GetParameter2Node));

	UTEST_TRUE("Edge between param 1 and subgraph exists", GetParameter1Node->GetOutputPin(Param1Name)->Edges.ContainsByPredicate([GetParameter1Node](const UPCGEdge* Edge) { return Edge && Edge->InputPin && Edge->OutputPin && (Edge->InputPin->Node == GetParameter1Node) && Edge->OutputPin->Node && Edge->OutputPin->Node->GetSettings() && Edge->OutputPin->Node->GetSettings()->IsA<UPCGSubgraphSettings>(); }));
	UTEST_TRUE("Edge between param 2 and subgraph exists", GetParameter2Node->GetOutputPin(Param2Name)->Edges.ContainsByPredicate([GetParameter2Node](const UPCGEdge* Edge) { return Edge && Edge->InputPin && Edge->OutputPin && (Edge->InputPin->Node == GetParameter2Node) && Edge->OutputPin->Node && Edge->OutputPin->Node->GetSettings() && Edge->OutputPin->Node->GetSettings()->IsA<UPCGSubgraphSettings>(); }));

	// Also verify that the subgraph has the 2 parameters, with the right value, and their getters are connected to the surface sampler.
	UTEST_NOT_NULL("Subgraph has Param1 as parameter", Subgraph->GetUserParametersStruct()->FindPropertyDescByName(Param1Name));
	UTEST_NOT_NULL("Subgraph has Param2 as parameter", Subgraph->GetUserParametersStruct()->FindPropertyDescByName(Param2Name));

	UTEST_EQUAL("Param1 has the right value", Subgraph->GetUserParametersStruct()->GetValueFloat(Param1Name).GetValue(), Param1Value);
	UTEST_EQUAL("Param2 has the right value", *Subgraph->GetUserParametersStruct()->GetValueStruct<FVector>(Param2Name).GetValue(), Param2Value);

	VALIDATE_EDGE(UPCGUserParameterGetSettings, UPCGSurfaceSamplerSettings, Subgraph, NAME_None, TEXT("Looseness"))
	VALIDATE_EDGE(UPCGUserParameterGetSettings, UPCGSurfaceSamplerSettings, Subgraph, NAME_None, TEXT("PointExtents"))

	return true;
}

bool FPCGCollapseSubgraphCombinationOfCollapses::RunTest(const FString& Parameters)
{
	const FSoftObjectPath UnitTestGraphAssetPath(TEXT("/Script/PCG.PCGGraph'/PCG/UnitTest/UnitTestSubgraphCollapse.UnitTestSubgraphCollapse'"));
	UPCGGraph* UnitTestGraph = TSoftObjectPtr<UPCGGraph>(UnitTestGraphAssetPath).LoadSynchronous();
	UTEST_NOT_NULL("UnitTest graph was successfully loaded", UnitTestGraph);
	check(UnitTestGraph);

	// We expect our graph to have 12 nodes, as it will impact the number of combinations below.
	UTEST_EQUAL("Expected number of nodes", UnitTestGraph->GetNodes().Num(), 12);

	PCGTestsCommon::FTestData TestData{};

	// Temporary solution to run a full graph and extract the resulting points.
	auto ExecuteGraph = [&TestData](UPCGGraph* GraphToExecute, double FirstConstant, double SecondConstant) -> TArray<FTransform>
	{
		TSharedPtr<FPCGGraphExecutor> GraphExecutor = MakeShared<FPCGGraphExecutor>();
		TArray<FTransform> OutputTransforms;
		bool bDone = false;

		TestData.TestPCGComponent->SetGraphLocal(GraphToExecute);
		TestData.TestPCGComponent->GetGraphInstance()->SetGraphParameter<double>(TEXT("FloatConstant"), FirstConstant);
		TestData.TestPCGComponent->GetGraphInstance()->SetGraphParameter<double>(TEXT("AnotherConstant"), SecondConstant);

		const FPCGTaskId TaskId = GraphExecutor->Schedule(TestData.TestPCGComponent);

		// TODO: Post-release GraphExecutor should be setting this on the Component
		TestData.SetCurrentGenerationTask(TaskId);

		const FPCGTaskId FinalTaskId = GraphExecutor->ScheduleGeneric([&OutputTransforms, &GraphExecutor, TaskId, &bDone]() -> bool
		{
			FPCGDataCollection OutputData{};
			GraphExecutor->GetOutputData(TaskId, OutputData);

			if (OutputData.TaggedData.Num() == 1)
			{
				if (const UPCGBasePointData* PointData = Cast<UPCGBasePointData>(OutputData.TaggedData[0].Data))
				{
					OutputTransforms = PointData->GetTransformsCopy();
				}
			}

			bDone = true;
			return true;
		}, TestData.TestPCGComponent, { TaskId });

		while(!bDone)
		{
			GraphExecutor->Execute();
		}

		return OutputTransforms;
	};

	auto CompareTransforms = [this](const TArray<FTransform>& TransformsA, const TArray<FTransform>& TransformsB, const FString& Name) -> bool
	{
		UTEST_EQUAL(*FString::Printf(TEXT("%s: Same number of points"), *Name), TransformsB.Num(), TransformsA.Num());
		for (int i = 0; i < TransformsA.Num(); ++i)
		{
			UTEST_EQUAL(*FString::Printf(TEXT("%s: Same position"), *Name), TransformsB[i].GetLocation(), TransformsA[i].GetLocation());
		}

		return true;
	};

	constexpr double FirstConstantValue = 50.0;
	constexpr double FirstConstantValue2 = 100.0;
	constexpr double SecondConstantValue = 500.0;
	constexpr double SecondConstantValue2 = 5000.0;

	// Run 3 times with different graph parameters on each
	const TArray<FTransform> FirstRunPoints = ExecuteGraph(UnitTestGraph, FirstConstantValue, SecondConstantValue);
	const TArray<FTransform> SecondRunPoints = ExecuteGraph(UnitTestGraph, FirstConstantValue2, SecondConstantValue);
	const TArray<FTransform> ThirdRunPoints = ExecuteGraph(UnitTestGraph, FirstConstantValue, SecondConstantValue2);

	// Then do all the 4 nodes collapse combinations. We expect our initial node to have 12 nodes. 12 choose 4 is 495 (which is the sum of ExpectedNumberOfCycles and ExpectedNumberOfSuccess)
	bool bSuccess = true;
	const int32 NodesNum = UnitTestGraph->GetNodes().Num();
	constexpr int32 ExpectedNumberOfCycles = 315;
	constexpr int32 ExpectedNumberOfSuccess = 180;
	int32 NumberOfCycles = 0;
	int32 NumberOfSuccess = 0;
	for (int i = 0; i < NodesNum && bSuccess; ++i)
	{
		for (int j = i + 1; j < NodesNum && bSuccess; j++)
		{
			for (int k = j + 1; k < NodesNum && bSuccess; ++k)
			{
				for (int l = k + 1; l < NodesNum && bSuccess; ++l)
				{
					UPCGGraph* OriginalGraph = DuplicateObject(UnitTestGraph, GetTransientPackage(), TEXT("UnitTest_GraphDuplicate"));
					check(OriginalGraph);
					OriginalGraph->SetFlags(RF_Transient);

					FString Name = FString::Printf(TEXT("[%d, %d, %d, %d]"), i, j, k, l);
					FString AllNodes = FString::Printf(TEXT("[%s, %s, %s, %s]"), *OriginalGraph->GetNodes()[i]->GetName(), *OriginalGraph->GetNodes()[j]->GetName(), *OriginalGraph->GetNodes()[k]->GetName(), *OriginalGraph->GetNodes()[l]->GetName());

					const TArray<UPCGNode*> NodesToCollapse = { OriginalGraph->GetNodes()[i], OriginalGraph->GetNodes()[j], OriginalGraph->GetNodes()[k], OriginalGraph->GetNodes()[l] };
					FText OutFailReason;
					if (UPCGGraph* Subgraph = FPCGSubgraphHelpers::CollapseIntoSubgraphWithReason(OriginalGraph, NodesToCollapse, {}, OutFailReason))
					{
						Subgraph->SetFlags(RF_Transient);

						const TArray<FTransform> FirstRunCollapsedPoints = ExecuteGraph(OriginalGraph, FirstConstantValue, SecondConstantValue);
						const TArray<FTransform> SecondRunCollapsedPoints = ExecuteGraph(OriginalGraph, FirstConstantValue2, SecondConstantValue);
						const TArray<FTransform> ThirdRunCollapsedPoints = ExecuteGraph(OriginalGraph, FirstConstantValue, SecondConstantValue2);

						bool bLocalSuccess = CompareTransforms(FirstRunPoints, FirstRunCollapsedPoints, Name + TEXT(" First run"));
						bLocalSuccess &= CompareTransforms(SecondRunPoints, SecondRunCollapsedPoints, Name + TEXT(" Second run"));
						bLocalSuccess &= CompareTransforms(ThirdRunPoints, ThirdRunCollapsedPoints, Name + TEXT(" Third run"));

						if (!bLocalSuccess)
						{
							UE_LOGF(LogPCG, Log, "Combination %ls failed. Nodes: %ls", *Name, *AllNodes);
						}
						else
						{
							NumberOfSuccess++;
						}

						bSuccess &= bLocalSuccess;

						Subgraph->MarkAsGarbage();
					}
					else
					{
						UE_LOGF(LogPCG, Log, "Combination %ls has probably a cycle (Msg: %ls)", *Name, *OutFailReason.ToString());
						NumberOfCycles++;
					}

					OriginalGraph->MarkAsGarbage();
				}
			}
		}
	}

	if (bSuccess)
	{
		UTEST_EQUAL("Number of success expected", NumberOfSuccess, ExpectedNumberOfSuccess);
		UTEST_EQUAL("Number of cycles expected", NumberOfCycles, ExpectedNumberOfCycles);
	}

	return bSuccess;
}

#undef VALIDATE_EDGE