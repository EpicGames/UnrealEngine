// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/PCGExtraCapture.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGInputOutputSettings.h"
#include "PCGProfilingLog.h"
#include "PCGSubgraph.h"
#include "Graph/PCGGraphCache.h"
#include "Graph/PCGGraphCompiler.h"
#include "Graph/PCGGraphExecutor.h"

#if PCG_PROFILING_ENABLED

void PCGUtils::FExtraCapture::Update(const PCGUtils::FScopedCall& InScopedCall)
{
#if !WITH_EDITOR
	// Capture only required for profiling data outside of editor.
	if (!PCGProfilingLog::IsEnabled())
	{
		return;
	}
#endif

	if (!InScopedCall.Context || !InScopedCall.Context->GetStack())
	{
		return;
	}

	const double CurrentTime = FPlatformTime::Seconds();
	const double ThisFrameTime = CurrentTime - InScopedCall.StartTime;

	FCallTime& Timer = InScopedCall.Context->Timer;

	switch (InScopedCall.Phase)
	{
	case EPCGExecutionPhase::NotExecuted:
		Timer = FCallTime(); // reset it
		break;
	case EPCGExecutionPhase::PrepareData:
		if (Timer.PrepareDataFrameCount == 0)
		{
			Timer.PrepareDataStartTime = InScopedCall.StartTime;
		}

		Timer.PrepareDataFrameCount++;
		Timer.PrepareDataTime += ThisFrameTime;
		Timer.PrepareDataEndTime = CurrentTime;
		break;
	case EPCGExecutionPhase::Execute:
		if (Timer.ExecutionFrameCount == 0)
		{
			Timer.ExecutionStartTime = InScopedCall.StartTime;
		}

		Timer.ExecutionTime += ThisFrameTime;
		Timer.ExecutionFrameCount++;
		Timer.ExecutionEndTime = CurrentTime;

		Timer.MaxExecutionFrameTime = FMath::Max(Timer.MaxExecutionFrameTime, ThisFrameTime);
		Timer.MinExecutionFrameTime = FMath::Min(Timer.MinExecutionFrameTime, ThisFrameTime);
		break;
	case EPCGExecutionPhase::PostExecute:
		Timer.PostExecuteTime = ThisFrameTime;

		if (InScopedCall.Context->GetInputSettings<UPCGGraphInputOutputSettings>())
		{
			// Ignore input/output nodes as they don't create any data and could count towards memory costs this way (not in totals, but locally).
		}
		else
		{
			ensure(!Timer.OutputCPUMemorySize.IsSet() && !Timer.OutputGPUMemorySize.IsSet());
			// Count memory here
			// FIXME / TODO: This is sensitive to representation of data. If/when we change data representation to separate data from metadata, this might not report memory accurately
			auto VisitAllData = [](const FPCGDataCollection& InCollection, TSet<const UPCGData*>& OutData)
			{
				for (const FPCGTaggedData& Data : InCollection.TaggedData)
				{
					if (Data.Data)
					{
						Data.Data->VisitDataNetwork([&OutData](const UPCGData* VisitedData) { OutData.Add(VisitedData); });
					}
				}
			};

			TSet<const UPCGData*> InputData;
			TSet<const UPCGData*> OutputData;
			VisitAllData(InScopedCall.Context->InputData, InputData);
			VisitAllData(InScopedCall.Context->OutputData, OutputData);

			TSet<const UPCGData*> NewData = OutputData.Difference(InputData);

			uint64 CPUMemorySize = 0;
			uint64 GPUMemorySize = 0;

			PCGGraphCacheHelpers::bIsGatheringCacheSize = PCGGraphCacheHelpers::CVarEnableTLSGatherCacheSize.GetValueOnAnyThread();
			ON_SCOPE_EXIT
			{
				PCGGraphCacheHelpers::bIsGatheringCacheSize = false;
			};

			for (const UPCGData* Data : NewData)
			{
				FResourceSizeEx ResSize = FResourceSizeEx(EResourceSizeMode::Exclusive);
				const_cast<UPCGData*>(Data)->GetResourceSizeEx(ResSize);
				CPUMemorySize += ResSize.GetDedicatedSystemMemoryBytes();
				GPUMemorySize += ResSize.GetDedicatedVideoMemoryBytes();
			}
			Timer.OutputCPUMemorySize = CPUMemorySize;
			Timer.OutputGPUMemorySize = GPUMemorySize;
		}

		break;
	}

#if WITH_EDITOR
	PCG::TScopeLock ScopedLock(Lock);
	if (!InScopedCall.CapturedMessages.IsEmpty())
	{
		TArray<FCapturedMessage>& InstanceMessages = CapturedMessages.FindOrAdd(InScopedCall.Context->Node);
		InstanceMessages.Append(std::move(InScopedCall.CapturedMessages));
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR

void PCGUtils::FExtraCapture::ResetCapturedMessages()
{
	CapturedMessages.Empty();
}

namespace PCGUtils
{
	void BuildTreeInfo(FCallTreeInfo& Info, const TMap<const UPCGNode*, const UPCGGraph*>& SubgraphNodeToGraphMap)
	{
		if (const UPCGGraph* const* Subgraph = SubgraphNodeToGraphMap.Find(Info.Node))
		{
			check(*Subgraph);
			// This is the same as in UPCGSubgraphSettings::GetAdditionalTitleInformation
			Info.Name = FName::NameToDisplayString((*Subgraph)->GetName(), /*bIsBool=*/false);
		}

		if (!Info.Children.IsEmpty())
		{
			Info.NumberOfTasks = 0;
		}

		for (FCallTreeInfo& Child : Info.Children)
		{
			BuildTreeInfo(Child, SubgraphNodeToGraphMap);

			Info.CallTime.PrepareDataTime += Child.CallTime.PrepareDataTime;
			Info.CallTime.ExecutionTime += Child.CallTime.ExecutionTime;
			Info.CallTime.PostExecuteTime += Child.CallTime.PostExecuteTime;

			Info.CallTime.PrepareDataStartTime = FMath::Min(Info.CallTime.PrepareDataStartTime, Child.CallTime.PrepareDataStartTime);
			Info.CallTime.PrepareDataEndTime = FMath::Max(Info.CallTime.PrepareDataEndTime, Child.CallTime.PrepareDataEndTime);
			Info.CallTime.ExecutionStartTime = FMath::Min(Info.CallTime.ExecutionStartTime, Child.CallTime.ExecutionStartTime);
			Info.CallTime.ExecutionEndTime = FMath::Max(Info.CallTime.ExecutionEndTime, Child.CallTime.ExecutionEndTime);

			Info.CallTime.MinExecutionFrameTime = FMath::Min(Info.CallTime.MinExecutionFrameTime, Child.CallTime.MinExecutionFrameTime);
			Info.CallTime.MaxExecutionFrameTime = FMath::Max(Info.CallTime.MaxExecutionFrameTime, Child.CallTime.MaxExecutionFrameTime);

			if (Child.CallTime.OutputCPUMemorySize.IsSet())
			{
				Info.CallTime.OutputCPUMemorySize = Info.CallTime.OutputCPUMemorySize.Get(0) + Child.CallTime.OutputCPUMemorySize.GetValue();
			}

			if (Child.CallTime.OutputGPUMemorySize.IsSet())
			{
				Info.CallTime.OutputGPUMemorySize = Info.CallTime.OutputGPUMemorySize.Get(0) + Child.CallTime.OutputGPUMemorySize.GetValue();
			}

			Info.NumberOfTasks += Child.NumberOfTasks;
		}
	}
}

PCGUtils::FCallTreeInfo PCGUtils::FExtraCapture::CalculateCallTreeInfo(const IPCGGraphExecutionSource* ExecutionSource, const FPCGStack& RootStack) const
{
	FCallTreeInfo RootInfo;

	// Basically, what we want is - visit all entries in the "NodeToStacksInWhichNodeExecuted" and build our information from there.
	TMap<TObjectKey<const UPCGNode>, TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData>> NodeToStacksInWhichNodeExecuted = ExecutionSource->GetExecutionState().GetInspection().GetExecutedNodeStacks();

	TArray<const UPCGNode*> NodePath;
	TArray<int32> NodePathLoop;
	TMap<const UPCGNode*, const UPCGGraph*> SubgraphToGraphMap;

	auto GetNodePath = [&NodePath, &NodePathLoop, &RootStack](const FPCGStack& Stack)
	{
		NodePath.Reset();
		NodePathLoop.Reset();

		if (!Stack.BeginsWith(RootStack))
		{
			return false;
		}

		const TArray<FPCGStackFrame>& StackFrames = Stack.GetStackFrames();
		for(int StackFrameIndex = RootStack.GetStackFrames().Num(); StackFrameIndex < StackFrames.Num(); ++StackFrameIndex)
		{
			const FPCGStackFrame& StackFrame = StackFrames[StackFrameIndex];

			if (const UPCGNode* Node = StackFrame.GetObject_GameThread<UPCGNode>())
			{
				NodePath.Add(Node);
				NodePathLoop.Add(-1);
			}
			else if (StackFrame.LoopIndex != INDEX_NONE)
			{
				NodePath.Add(nullptr);
				NodePathLoop.Add(StackFrame.LoopIndex);
			}
		}

		return true;
	};

	auto GetCallInfo = [&NodePath, &NodePathLoop, &RootInfo]()
	{
		FCallTreeInfo* Current = &RootInfo;

		int NodeDepth = 0;
		while (NodeDepth < NodePath.Num())
		{
			const UPCGNode* NodeToFind = NodePath[NodeDepth];
			const int32 LoopToFind = NodePathLoop[NodeDepth];
			++NodeDepth;

			bool bFound = false;
			for (FCallTreeInfo& Child : Current->Children)
			{
				if((NodeToFind != nullptr && Child.Node == NodeToFind) || 
					(LoopToFind != INDEX_NONE && Child.Node == nullptr && Child.LoopIndex == LoopToFind))
				{
					bFound = true;
					Current = &Child;
					break;
				}
			}

			if (!bFound)
			{
				FCallTreeInfo& Child = Current->Children.Emplace_GetRef();
				Child.Node = NodeToFind;
				Child.LoopIndex = LoopToFind;
				Current = &Child;
			}
		}

		return Current;
	};

	for (const auto& NodeToStacks : NodeToStacksInWhichNodeExecuted)
	{
		const UPCGNode* Node = NodeToStacks.Key.ResolveObjectPtr();
		const TSet<FPCGGraphExecutionInspection::FNodeExecutedNotificationData>& NodeExecutions = NodeToStacks.Value;

		for(const FPCGGraphExecutionInspection::FNodeExecutedNotificationData& NodeExecution : NodeExecutions)
		{
			const FPCGStack& Stack = NodeExecution.Stack;

			if (!GetNodePath(Stack))
			{
				continue;
			}

			// Set subgraph node name if immediate parent is a subgraph (loop or subgraph - NOT spawn actor)
			const UPCGNode* ParentNode = NodePath.IsEmpty() ? nullptr : ((NodePathLoop.Last() != INDEX_NONE && NodePath.Num() > 1) ? NodePath.Last(1) : NodePath.Last());
			if (const UPCGSubgraphNode* SubgraphNode = Cast<UPCGSubgraphNode>(ParentNode))
			{
				const UPCGGraph* CurrentGraph = Stack.GetGraphForCurrentFrame();
				if(CurrentGraph && !SubgraphToGraphMap.Contains(SubgraphNode))
				{
					SubgraphToGraphMap.Add(SubgraphNode, CurrentGraph);
				}
			}
			
			// Need to add "this" node to the stack
			NodePath.Add(Node);
			NodePathLoop.Add(INDEX_NONE);

			FCallTreeInfo* Info = GetCallInfo();

			check(Info);
			Info->CallTime = NodeExecution.Timer;
		}
	}

	PCGUtils::BuildTreeInfo(RootInfo, SubgraphToGraphMap);
	return RootInfo;
}

#endif // WITH_EDITOR

PCGUtils::FScopedCall::FScopedCall(const IPCGElement* InOwner, FPCGContext* InContext, FScopedCallOutputDevice& InOutputDevice)
	: Owner(InOwner)
	, Context(InContext)
	, Phase(InContext ? InContext->CurrentPhase : EPCGExecutionPhase::NotExecuted)
	, OutputDevice(InOutputDevice)
{
	StartTime = FPlatformTime::Seconds();
#if WITH_EDITOR
	check(!OutputDevice.ScopedCall);
	OutputDevice.ScopedCall = this;
#endif
}

PCGUtils::FScopedCall::~FScopedCall()
{
	if (Context && Context->ExecutionSource.IsValid())
	{
		Context->ExecutionSource->GetExecutionState().GetExtraCapture().Update(*this);
	}

#if WITH_EDITOR
	OutputDevice.ScopedCall = nullptr;
#endif
}

#if WITH_EDITOR

PCGUtils::FScopedCallOutputDevice::FScopedCallOutputDevice()
	: ThreadID(FPlatformTLS::GetCurrentThreadId())
{
	GLog->AddOutputDevice(this);
}

PCGUtils::FScopedCallOutputDevice::~FScopedCallOutputDevice()
{
	GLog->RemoveOutputDevice(this);
}

void PCGUtils::FScopedCallOutputDevice::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	// TODO: this thread id check will also filter out messages spawned from threads spawned inside of nodes. To improve that,
	// perhaps set at TLS bit on things from here and inside of PCGAsync spawned jobs. If this was done, CapturedMessages below also will
	// need protection
	if (Verbosity > ELogVerbosity::Warning || FPlatformTLS::GetCurrentThreadId() != ThreadID)
	{
		// ignore
		return;
	}

	// this is a dumb counter just so messages can be sorted in a similar order as when they were logged
	static volatile int32 MessageCounter = 0;

	if (ScopedCall)
	{
		ScopedCall->CapturedMessages.Add(PCGUtils::FCapturedMessage{ MessageCounter++, Category, V, Verbosity });
	}
}

#endif // WITH_EDITOR

#endif // PCG_PROFILING_ENABLED
