// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGGeneratedResourcesLogging.h"

#include "PCGGraphExecutionStateInterface.h"
#include "PCGGraph.h"
#include "PCGManagedResource.h"
#include "PCGModule.h"

#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"

namespace PCGGeneratedResourcesLogging
{
#if WITH_EDITOR
	static TAutoConsoleVariable<bool> CVarGenResourcesLoggingEnable(
		TEXT("pcg.ManagedResourcesLogging.Enable"),
		false,
		TEXT("Enables fine grained log of generated resources management"));

	static TAutoConsoleVariable<int32> CVarGenResourcesLoggingMaxPrintCount(
		TEXT("pcg.ManagedResourcesLogging.MaxElementPrintCount"),
		3,
		TEXT("Sets how many entries to display for resources that are arrays of objects"));
#endif

	bool LogEnabled()
	{
#if WITH_EDITOR
		return CVarGenResourcesLoggingEnable.GetValueOnAnyThread();
#else
		return false;
#endif
	}

#if WITH_EDITOR
	FString GetExecutionSourceString(const IPCGGraphExecutionSource* InExecutionSource)
	{
		return InExecutionSource ? InExecutionSource->GetExecutionState().GetDebugName() : TEXT("MISSINGEXECUTIONSOURCE");
	}

	FString GetExecutionSourceGraphString(const IPCGGraphExecutionSource* InExecutionSource)
	{
		return InExecutionSource && InExecutionSource->GetExecutionState().GetGraph() ? *InExecutionSource->GetExecutionState().GetGraph()->GetName() : TEXT("MISSINGGRAPH");
	}
#endif

#if WITH_EDITOR
	void LogResource(const IPCGGraphExecutionSource* InExecutionSource, UPCGManagedResource* InResource)
	{
		if (const UPCGManagedActors* ManagedActors = Cast<UPCGManagedActors>(InResource))
		{
			UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls]         Managed actors (%d):",
				*GetExecutionSourceString(InExecutionSource),
				*GetExecutionSourceGraphString(InExecutionSource),
				ManagedActors->GetConstGeneratedActors().Num());

			const int32 MaxPrint = CVarGenResourcesLoggingMaxPrintCount.GetValueOnAnyThread();
			int32 Count = 0;
			for (const TSoftObjectPtr<AActor>& Actor : ManagedActors->GetConstGeneratedActors())
			{
				if (Count++ >= MaxPrint)
				{
					break;
				}

				if (Actor.Get())
				{
					UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls]                 |- '%ls' (%d/%d)", 
						*GetExecutionSourceString(InExecutionSource), 
						*GetExecutionSourceGraphString(InExecutionSource),
						*Actor->GetFName().ToString(),
						Count, 
						ManagedActors->GetConstGeneratedActors().Num());
				}
				else
				{
					UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls]                 |- NULL (%d/%d)",
						*GetExecutionSourceString(InExecutionSource),
						*GetExecutionSourceGraphString(InExecutionSource),
						Count, ManagedActors->GetConstGeneratedActors().Num());
				}
			}
		}
		else if (const UPCGManagedComponent* ManagedComponent = Cast<UPCGManagedComponent>(InResource))
		{
			if (ManagedComponent->GeneratedComponent)
			{
				UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls]         Managed component: '%ls'",
					*GetExecutionSourceString(InExecutionSource),
					*GetExecutionSourceGraphString(InExecutionSource),
					*ManagedComponent->GeneratedComponent->GetReadableName());
			}
			else
			{
				UE_LOGF(LogPCG, Warning, "[PCGMANAGEDRESOURCES] [%ls/%ls]         NULL managed component or no owner",
					*GetExecutionSourceString(InExecutionSource),
					*GetExecutionSourceGraphString(InExecutionSource));
			}
		}
		else
		{
			if (InResource)
			{
				UE_LOGF(LogPCG, Warning, "[PCGMANAGEDRESOURCES] [%ls/%ls]         Unidentified managed resource",
					*GetExecutionSourceString(InExecutionSource),
					*GetExecutionSourceGraphString(InExecutionSource));
			}
			else
			{
				UE_LOGF(LogPCG, Warning, "[PCGMANAGEDRESOURCES] [%ls/%ls]         NULL unidentified resource",
					*GetExecutionSourceString(InExecutionSource),
					*GetExecutionSourceGraphString(InExecutionSource));
			}
		}
	}

	void LogResources(const IPCGGraphExecutionSource* InComponent, const TArray<UPCGManagedResource*>& Resources)
	{
		for (UPCGManagedResource* Resource : Resources)
		{
			LogResource(InComponent, Resource);
		}
	}
#endif // WITH_EDITOR

	void LogAddToManagedResources(const IPCGGraphExecutionSource* InExecutionSource, UPCGManagedResource* InResource)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] AddToManagedResources:",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource));

		LogResource(InExecutionSource, InResource);
#endif
	}

	void LogCleanupLocal(const IPCGGraphExecutionSource* InExecutionSource, bool bReleaseManagedResources)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES]"); // Blank line
		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] CleanupLocal, bReleaseManagedResources: %d",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource),
			bReleaseManagedResources);
#endif
	}

	void LogCleanupLocalImmediate(const IPCGGraphExecutionSource* InExecutionSource, bool bHardRelease, const TArray<UPCGManagedResource*>& GeneratedResources)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES]"); // Blank line
		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] CleanupLocalImmediate BEGIN, bHardRelease: %d, GeneratedResources.Num() = %d",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource),
			bHardRelease,
			GeneratedResources.Num());

		LogResources(InExecutionSource, GeneratedResources);
#endif
	}

	void LogCleanupLocalImmediateResource(const IPCGGraphExecutionSource* InExecutionSource, UPCGManagedResource* Resource)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] CleanupLocalImmediate:",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource));
		
		LogResource(InExecutionSource, Resource);
#endif
	}

	void LogCleanupLocalImmediateFinished(const IPCGGraphExecutionSource* InExecutionSource, const TArray<UPCGManagedResource*>& GeneratedResources)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] CleanupLocalImmediate FINISHED, Final GeneratedResources (%d):",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource),
			GeneratedResources.Num());

		LogResources(InExecutionSource, GeneratedResources);
#endif
	}

	void LogCreateCleanupTask(const IPCGGraphExecutionSource* InExecutionSource, bool bReleaseManagedResources)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] CreateCleanupTask, bReleaseManagedResources: %d",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource),
			bReleaseManagedResources);
#endif
	}

	void LogCreateCleanupTaskResource(const IPCGGraphExecutionSource* InExecutionSource, UPCGManagedResource* Resource)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] CreateCleanupTask::CleanupTask:",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource));

		LogResource(InExecutionSource, Resource);
#endif
	}

	void LogCreateCleanupTaskFinished(const IPCGGraphExecutionSource* InExecutionSource, const TArray<TObjectPtr<UPCGManagedResource>>* InGeneratedResources)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] CreateCleanupTask::CleanupTask FINISHED, GeneratedResources.Num() = %d",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource),
			InGeneratedResources ? InGeneratedResources->Num() : -1);
#endif
	}

	void LogCreateCleanupTaskFinished(const TArray<UPCGManagedResource*>* InGeneratedResources)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] CreateCleanupTask::CleanupTask FINISHED, GeneratedResources.Num() = %d",
			InGeneratedResources ? InGeneratedResources->Num() : -1);
#endif
	}

	void LogCleanupUnusedManagedResources(const IPCGGraphExecutionSource* InExecutionSource, const TArray<UPCGManagedResource*>& GeneratedResources)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] CleanupUnusedManagedResources BEGIN, GeneratedResources.Num() = %d",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource),
			GeneratedResources.Num());
#endif
	}

	void LogCleanupUnusedManagedResourcesResource(const IPCGGraphExecutionSource* InExecutionSource, UPCGManagedResource* InResource)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] CleanupUnusedManagedResources:",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource));

		LogResource(InExecutionSource, InResource);
#endif
	}

	void LogCleanupUnusedManagedResourcesFinished(const IPCGGraphExecutionSource* InExecutionSource, const TArray<UPCGManagedResource*>& GeneratedResources)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] CleanupUnusedManagedResources, FINISHED:",
			*GetExecutionSourceString(InExecutionSource),
			*GetExecutionSourceGraphString(InExecutionSource));

		LogResources(InExecutionSource, GeneratedResources);
#endif
	}

	void LogManagedResourceSoftRelease(UPCGManagedResource* InResource)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		const IPCGGraphExecutionSource* ExecutionSource = InResource ? InResource->GetImplementingOuter<IPCGGraphExecutionSource>() : nullptr;

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] UPCGManagedResource::Release, SOFT release:",
			*GetExecutionSourceString(ExecutionSource),
			*GetExecutionSourceGraphString(ExecutionSource));

		LogResource(ExecutionSource, InResource);
#endif
	}

	void LogManagedResourceHardRelease(UPCGManagedResource* InResource)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		const IPCGGraphExecutionSource* ExecutionSource = InResource ? InResource->GetImplementingOuter<IPCGGraphExecutionSource>() : nullptr;

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] UPCGManagedResource::Release, HARD release:",
			*GetExecutionSourceString(ExecutionSource),
			*GetExecutionSourceGraphString(ExecutionSource));

		LogResource(ExecutionSource, InResource);
#endif
	}

	void LogManagedActorsRelease(const UPCGManagedResource* InResource, const TArray<TSoftObjectPtr<AActor>>& ActorsToDelete, bool bHardRelease, bool bOnlyMarkedForCleanup)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		const IPCGGraphExecutionSource* ExecutionSource = InResource ? InResource->GetImplementingOuter<IPCGGraphExecutionSource>() : nullptr;
		const bool bMarkedTransientOnLoad = InResource && InResource->IsMarkedTransientOnLoad();

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] UPCGManagedActors::Release, %ls release, bMarkedTransientOnLoad: %d, bOnlyMarkedForCleanup: %d, actor count %d scheduled for delete",
			*GetExecutionSourceString(ExecutionSource),
			*GetExecutionSourceGraphString(ExecutionSource),
			bHardRelease ? TEXT("HARD") : TEXT("SOFT"),
			bMarkedTransientOnLoad,
			bOnlyMarkedForCleanup,
			ActorsToDelete.Num());

		const uint32 MaxPrint = 3;
		uint32 Count = 0;
		for (const TSoftObjectPtr<AActor>& ActorToDelete : ActorsToDelete)
		{
			if (Count++ >= MaxPrint)
			{
				break;
			}

			if (ActorToDelete)
			{
				UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls]                 |- '%ls' (%d/%d)",
					*GetExecutionSourceString(ExecutionSource),
					*GetExecutionSourceGraphString(ExecutionSource),
					*ActorToDelete->GetFName().ToString(),
					Count,
					ActorsToDelete.Num());
			}
			else
			{
				UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls]                 |- NULL (%d/%d)",
					*GetExecutionSourceString(ExecutionSource),
					*GetExecutionSourceGraphString(ExecutionSource),
					Count,
					ActorsToDelete.Num());
			}
		}
#endif
	}

	void LogManagedComponentHidden(UPCGManagedResource* InResource)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		const IPCGGraphExecutionSource* ExecutionSource = InResource ? InResource->GetImplementingOuter<IPCGGraphExecutionSource>() : nullptr;

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] UPCGManagedComponent::Release, hidden, component:",
			*GetExecutionSourceString(ExecutionSource),
			*GetExecutionSourceGraphString(ExecutionSource));

		LogResource(ExecutionSource, InResource);
#endif
	}

	void LogManagedComponentDeleteNull(UPCGManagedResource* InResource)
	{
#if WITH_EDITOR
		if (!LogEnabled())
		{
			return;
		}

		const IPCGGraphExecutionSource* ExecutionSource = InResource ? InResource->GetImplementingOuter<IPCGGraphExecutionSource>() : nullptr;

		UE_LOGF(LogPCG, Log, "[PCGMANAGEDRESOURCES] [%ls/%ls] UPCGManagedComponent::Release, delete null component",
			*GetExecutionSourceString(ExecutionSource),
			*GetExecutionSourceGraphString(ExecutionSource));
#endif
	}
}
