// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "UObject/SoftObjectPtr.h"

class IPCGGraphExecutionSource;
class UPCGManagedComponent;
class UPCGManagedResource;
class AActor;

namespace PCGGeneratedResourcesLogging
{
	void LogAddToManagedResources(const IPCGGraphExecutionSource* InExecutionSource, UPCGManagedResource* InResource);
	
	void LogCleanupLocal(const IPCGGraphExecutionSource* InExecutionSource, bool bRemoveComponents);
	
	void LogCleanupLocalImmediate(const IPCGGraphExecutionSource* InExecutionSource, bool bHardRelease, const TArray<UPCGManagedResource*>& GeneratedResources);
	void LogCleanupLocalImmediateResource(const IPCGGraphExecutionSource* InExecutionSource, UPCGManagedResource* InResource);
	void LogCleanupLocalImmediateFinished(const IPCGGraphExecutionSource* InExecutionSource, const TArray<UPCGManagedResource*>& GeneratedResources);
	
	void LogCreateCleanupTask(const IPCGGraphExecutionSource* InExecutionSource, bool bRemoveComponents);
	void LogCreateCleanupTaskResource(const IPCGGraphExecutionSource* InExecutionSource, UPCGManagedResource* InResource);
	void LogCreateCleanupTaskFinished(const IPCGGraphExecutionSource* InExecutionSource, const TArray<TObjectPtr<UPCGManagedResource>>* InGeneratedResources);

	void LogCleanupUnusedManagedResources(const IPCGGraphExecutionSource* InExecutionSource, const TArray<UPCGManagedResource*>& GeneratedResources);
	void LogCleanupUnusedManagedResourcesResource(const IPCGGraphExecutionSource* InExecutionSource, UPCGManagedResource* Resource);
	void LogCleanupUnusedManagedResourcesFinished(const IPCGGraphExecutionSource* InExecutionSource, const TArray<UPCGManagedResource*>& GeneratedResources);

	void LogManagedActorsRelease(const UPCGManagedResource* InResource, const TArray<TSoftObjectPtr<AActor>>& ActorsToDelete, bool bHardRelease, bool bOnlyMarkedForCleanup);

	void LogManagedResourceSoftRelease(UPCGManagedResource* InResource);
	void LogManagedResourceHardRelease(UPCGManagedResource* InResource);
	void LogManagedComponentHidden(UPCGManagedResource* InResource);
	void LogManagedComponentDeleteNull(UPCGManagedResource* InResource);
}
