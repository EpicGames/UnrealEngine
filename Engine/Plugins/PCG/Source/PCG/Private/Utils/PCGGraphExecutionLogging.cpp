// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/PCGGraphExecutionLogging.h"

#include "PCGCommon.h"
#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGGraph.h"
#include "PCGModule.h"
#include "Graph/PCGGraphExecutor.h"

#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"

namespace PCGGraphExecutionLogging
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	static TAutoConsoleVariable<bool> CVarGraphExecutionLoggingEnable(
		TEXT("pcg.GraphExecution.EnableLogging"),
		false,
		TEXT("Enables fine grained log of graph execution"));

	static TAutoConsoleVariable<bool> CVarGraphExecutionCullingLoggingEnable(
		TEXT("pcg.GraphExecution.EnableCullingLogging"),
		false,
		TEXT("Enables fine grained log of dynamic task culling during graph execution"));
#endif

	bool LogEnabled()
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		return CVarGraphExecutionLoggingEnable.GetValueOnAnyThread();
#else
		return false;
#endif
	}

	bool CullingLogEnabled()
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		return CVarGraphExecutionCullingLoggingEnable.GetValueOnAnyThread();
#else
		return false;
#endif
	}

	void LogGraphTask(FPCGTaskId TaskId, const FPCGGraphTask& Task, const TSet<FPCGTaskId>* SuccessorIds)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}

		auto GenerateInputsString = [](const TArray<FPCGGraphTaskInput>& Inputs)
		{
			FString InputString;
			bool bFirstInput = true;

			for (const FPCGGraphTaskInput& Input : Inputs)
			{
				if (!bFirstInput)
				{
					InputString += TEXT(",");
				}
				bFirstInput = false;

				InputString += FString::Printf(
					TEXT("%" UINT64_FMT "->'%s'"),
					Input.TaskId,
					Input.DownstreamPin.IsSet() ? *Input.DownstreamPin.GetValue().Label.ToString() : TEXT("NoPin"));
			}

			return InputString;
		};

		FString SuccessorsString;
		if (SuccessorIds)
		{
			bool bFirstSuccessor = true;
			for (const FPCGTaskId& SuccessorId : *SuccessorIds)
			{
				SuccessorsString += bFirstSuccessor ? FString::Printf(TEXT("%" UINT64_FMT), SuccessorId) : FString::Printf(TEXT(",%" UINT64_FMT), SuccessorId);
				bFirstSuccessor = false;
			}
		}

		const FString PinDependencyString =
#if WITH_EDITOR
			Task.PinDependency.ToString();
#else
			TEXT("MISSINGPINDEPS");
#endif

		UE_LOGF(LogPCG, Log, "\t\tID: %llu\tParent: %llu\tNode: %ls\tInputs: %ls\tPinDeps: %ls\tSuccessors: %ls",
			TaskId,
			Task.ParentId != InvalidPCGTaskId ? Task.ParentId : 0,
			Task.Node ? (*Task.Node->GetNodeTitle(EPCGNodeTitleType::ListView).ToString()) : TEXT("NULL"),
			*GenerateInputsString(Task.Inputs),
			*PinDependencyString,
			*SuccessorsString
		);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGraphTasks(const TMap<FPCGTaskId, FPCGGraphTask>& Tasks, const TMap<FPCGTaskId, TSet<FPCGTaskId>>* TaskSuccessors)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}

		for (const TPair<FPCGTaskId, FPCGGraphTask>& TaskIdAndTask : Tasks)
		{
			PCGGraphExecutionLogging::LogGraphTask(TaskIdAndTask.Key, TaskIdAndTask.Value, TaskSuccessors ? TaskSuccessors->Find(TaskIdAndTask.Value.NodeId) : nullptr);
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGraphTasks(const TArray<FPCGGraphTask>& Tasks)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}

		for (const FPCGGraphTask& Task : Tasks)
		{
			LogGraphTask(Task.NodeId, Task);
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGraphSchedule(const IPCGGraphExecutionSource* InExecutionSource, const UPCGGraph* InScheduledGraph)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled() && !IsRunningCommandlet())
		{
			return;
		}

		UE_LOGF(LogPCG, Display, "[%ls/%ls] --- SCHEDULE GRAPH %ls ---",
			*PCGLog::GetExecutionSourceName(InExecutionSource),
			(InExecutionSource && InExecutionSource->GetExecutionState().GetGraph()) ? *InExecutionSource->GetExecutionState().GetGraph()->GetName() : TEXT("MISSINGGRAPH"),
			InScheduledGraph ? *InScheduledGraph->GetName() : TEXT("MISSINGGRAPH)"));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGraphScheduleDependency(const IPCGGraphExecutionSource* InExecutionSource, const FPCGStack* InFromStack)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled() && !IsRunningCommandlet())
		{
			return;
		}

		FString FromStackString;
		if (InFromStack)
		{
			InFromStack->CreateStackFramePath(FromStackString);
		}

		UE_LOGF(LogPCG, Display, "[%ls/%ls] --- SCHEDULE GRAPH FOR DEPENDENCY, from stack: %ls",
			*PCGLog::GetExecutionSourceName(InExecutionSource),
			(InExecutionSource && InExecutionSource->GetExecutionState().GetGraph()) ? *InExecutionSource->GetExecutionState().GetGraph()->GetName() : TEXT("MISSINGGRAPH"),
			*FromStackString);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGraphScheduleDependencyFailed(const IPCGGraphExecutionSource* InExecutionSource, const FPCGStack* InFromStack)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled() && !IsRunningCommandlet())
		{
			return;
		}

		FString FromStackString;
		if (InFromStack)
		{
			InFromStack->CreateStackFramePath(FromStackString);
		}

		UE_LOGF(LogPCG, Warning, "[%ls/%ls] Failed to schedule dependency, from stack: %ls",
			*PCGLog::GetExecutionSourceName(InExecutionSource),
			(InExecutionSource && InExecutionSource->GetExecutionState().GetGraph()) ? *InExecutionSource->GetExecutionState().GetGraph()->GetName() : TEXT("MISSINGGRAPH"),
			*FromStackString);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}
	
	void LogGraphPostSchedule(const TMap<FPCGTaskId, FPCGGraphTask>& Tasks, const TMap<FPCGTaskId, TSet<FPCGTaskId>>& TaskSuccessors)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!CullingLogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "POST SCHEDULE:");

		LogGraphTasks(Tasks, &TaskSuccessors);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogPostProcessGraph(const IPCGGraphExecutionSource* InExecutionSource)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled() && !IsRunningCommandlet())
		{
			return;
		}

		UE_LOGF(LogPCG, Display, "[%ls/%ls] IPCGGraphExecutionSource::PostProcessGraph",
			*PCGLog::GetExecutionSourceName(InExecutionSource),
			(InExecutionSource && InExecutionSource->GetExecutionState().GetGraph()) ? *InExecutionSource->GetExecutionState().GetGraph()->GetName() : TEXT("MISSINGGRAPH"));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogExecutionSourceCancellation(const TSet<IPCGGraphExecutionSource*>& CancelledExecutionSources)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled() && !IsRunningCommandlet())
		{
			return;
		}

		for (const IPCGGraphExecutionSource* ExecutionSource : CancelledExecutionSources)
		{
			UE_LOGF(LogPCG, Display, "[%ls/%ls] ExecutionSource cancelled",
				*PCGLog::GetExecutionSourceName(ExecutionSource),
				(ExecutionSource && ExecutionSource->GetExecutionState().GetGraph()) ? *ExecutionSource->GetExecutionState().GetGraph()->GetName() : TEXT("MISSINGGRAPH"));
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogChangeOriginIgnoredForComponent(const UObject* InObject, const IPCGGraphExecutionSource* InExecutionSource, EPCGIgnoreChangeOriginReason InReason)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[%ls/%ls] Change origin ignored: '%ls' (%ls)",
			*PCGLog::GetExecutionSourceName(InExecutionSource),
			(InExecutionSource && InExecutionSource->GetExecutionState().GetGraph()) ? *InExecutionSource->GetExecutionState().GetGraph()->GetName() : TEXT("MISSINGGRAPH"),
			InObject ? *InObject->GetName() : TEXT("MISSINGOBJECT"),
			*StaticEnum<EPCGIgnoreChangeOriginReason>()->GetValueAsString(InReason));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGraphExecuteFrameFinished()
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "--- FINISH FPCGGRAPHEXECUTOR::EXECUTE ---");
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	FString GetPinsToDeactivateString(const TArray<FPCGPinId>& PinIdsToDeactivate)
	{
		FString PinIdsToDeactivateString;
		bool bFirst = true;

		for (const FPCGPinId& PinId : PinIdsToDeactivate)
		{
			const FPCGTaskId NodeId = PCGPinIdHelpers::GetNodeIdFromPinId(PinId);
			const uint64 PinIndex = PCGPinIdHelpers::GetPinIndexFromPinId(PinId);
			PinIdsToDeactivateString += bFirst ? FString::Printf(TEXT("%" UINT64_FMT "_%" UINT64_FMT), NodeId, PinIndex) : FString::Printf(TEXT(",%" UINT64_FMT "_%" UINT64_FMT), NodeId, PinIndex);
			bFirst = false;
		}

		return PinIdsToDeactivateString;
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING

	void LogTaskExecute(const FPCGGraphTask& Task)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}

		IPCGGraphExecutionSource* ExecutionSource = Task.ExecutionSource.Get();
		if (!ExecutionSource)
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "         [%ls/%ls] %ls\t\tEXECUTE",
			*PCGLog::GetExecutionSourceName(ExecutionSource),
			ExecutionSource->GetExecutionState().GetGraph() ? *ExecutionSource->GetExecutionState().GetGraph()->GetName() : TEXT("MISSINGGRAPH"),
			*FString::Printf(TEXT("%" UINT64_FMT "'%s'"), Task.NodeId, Task.Node ? *Task.Node->GetNodeTitle(EPCGNodeTitleType::ListView).ToString() : TEXT("")));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogTaskExecuteCachingDisabled(const FPCGGraphTask& Task)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}

		IPCGGraphExecutionSource* ExecutionSource = Task.ExecutionSource.Get();
		if (!ExecutionSource)
		{
			return;
		}

		UE_LOGF(LogPCG, Warning, "[%ls/%ls] %ls\t\tCACHING DISABLED",
			*PCGLog::GetExecutionSourceName(ExecutionSource),
			ExecutionSource->GetExecutionState().GetGraph() ? *ExecutionSource->GetExecutionState().GetGraph()->GetName() : TEXT("MISSINGGRAPH"),
			*FString::Printf(TEXT("%" UINT64_FMT "'%s'"), Task.NodeId, Task.Node ? *Task.Node->GetNodeTitle(EPCGNodeTitleType::ListView).ToString() : TEXT("")));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogTaskCullingBegin(FPCGTaskId CompletedTaskId, uint64 InactiveOutputPinBitmask, const TArray<FPCGPinId>& PinIdsToDeactivate)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!CullingLogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "BEGIN CullInactiveDownstreamNodes, CompletedTaskId: %llu, InactiveOutputPinBitmask: %llu, Deactivating pin IDs: %ls",
			CompletedTaskId, InactiveOutputPinBitmask, *GetPinsToDeactivateString(PinIdsToDeactivate));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogTaskCullingBeginLoop(FPCGTaskId PinTaskId, uint64 PinIndex, const TArray<FPCGPinId>& PinIdsToDeactivate)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!CullingLogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "LOOP: DEACTIVATE %llu_%llu, remaining IDs: %ls", PinTaskId, PinIndex, *GetPinsToDeactivateString(PinIdsToDeactivate));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogTaskCullingUpdatedPinDeps(FPCGTaskId TaskId, const FPCGPinDependencyExpression& PinDependency, bool bDependencyExpressionBecameFalse)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!CullingLogEnabled())
		{
			return;
		}

		const FString PinDependencyString =
#if WITH_EDITOR
			PinDependency.ToString();
#else
			TEXT("MISSINGPINDEPS");
#endif

		UE_LOGF(LogPCG, Log, "UPDATED PIN DEP EXPRESSION (task ID %llu): %ls", TaskId, *PinDependencyString);

		if (bDependencyExpressionBecameFalse)
		{
			UE_LOGF(LogPCG, Log, "CULL task ID %llu", TaskId);
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGridLinkageTaskExecuteStore(const FPCGContext* InContext, uint32 InGenerationGrid, uint32 InFromGridSize, uint32 InToGridSize, const FString& InResourcePath, int32 InDataItemCount)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}
		check(InContext);

		UE_LOGF(LogPCG, Log, "[GRIDLINKING] [%ls] STORE. GenerationGridSize=%u, FromGridSize=%u, ToGridSize=%u, Path=%ls, DataItems=%d",
			*PCGLog::GetExecutionSourceName(InContext->ExecutionSource.Get(), /*bUseLabel=*/true),
			InGenerationGrid,
			InFromGridSize,
			InToGridSize,
			*InResourcePath,
			InDataItemCount);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGridLinkageTaskExecuteRetrieve(const FPCGContext* InContext, uint32 InGenerationGrid, uint32 InFromGridSize, uint32 InToGridSize, const FString& InResourcePath)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}
		check(InContext);

		UE_LOGF(LogPCG, Log, "[GRIDLINKING] [%ls] RETRIEVE. GenerationGridSize=%u, FromGridSize=%u, ToGridSize=%u, Path=%ls",
			*PCGLog::GetExecutionSourceName(InContext->ExecutionSource.Get(), /*bUseLabel=*/true),
			InGenerationGrid,
			InFromGridSize,
			InToGridSize,
			*InResourcePath);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGridLinkageTaskExecuteRetrieveSuccess(const FPCGContext* InContext, const IPCGGraphExecutionSource* InExecutionSource, const FString& InResourcePath, int32 InDataItemCount)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}
		check(InContext);

		UE_LOGF(LogPCG, Log, "[GRIDLINKING] [%ls] RETRIEVE: SUCCESS. Path=%ls, DataItems=%d",
			*PCGLog::GetExecutionSourceName(InContext->ExecutionSource.Get(), /*bUseLabel=*/true),
			*InResourcePath,
			InDataItemCount);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGridLinkageTaskExecuteRetrieveScheduleGraph(const FPCGContext* InContext, const IPCGGraphExecutionSource* InScheduledSource, const FString& InResourcePath)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}
		check(InContext);

		UE_LOGF(LogPCG, Log, "[GRIDLINKING] [%ls] RETRIEVE: SCHEDULE GRAPH. Source=%ls, Path=%ls",
			*PCGLog::GetExecutionSourceName(InContext->ExecutionSource.Get(), /*bUseLabel=*/true),
			*PCGLog::GetExecutionSourceName(InScheduledSource, /*bUseLabel=*/true),
			*InResourcePath);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGridLinkageTaskExecuteRetrieveWaitOnScheduledGraph(const FPCGContext* InContext, const IPCGGraphExecutionSource* InWaitOnSource, const FString& InResourcePath)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}
		check(InContext);

		UE_LOGF(LogPCG, Log, "[GRIDLINKING] [%ls] RETRIEVE: WAIT FOR SCHEDULED GRAPH. Source=%ls, Path=%ls",
			*PCGLog::GetExecutionSourceName(InContext->ExecutionSource.Get(), /*bUseLabel=*/true),
			*PCGLog::GetExecutionSourceName(InWaitOnSource, /*bUseLabel=*/true),
			*InResourcePath);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}
	
	void LogGridLinkageTaskExecuteRetrieveWakeUp(const FPCGContext* InContext, const IPCGGraphExecutionSource* InWokenBySource)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}
		check(InContext);

		UE_LOGF(LogPCG, Log, "[GRIDLINKING] [%ls] RETRIEVE: WOKEN BY Source=%ls",
			*PCGLog::GetExecutionSourceName(InContext->ExecutionSource.Get(), /*bUseLabel=*/true),
			*PCGLog::GetExecutionSourceName(InWokenBySource, /*bUseLabel=*/true));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGridLinkageTaskExecuteRetrieveNoLocalSource(const FPCGContext* InContext, const FString& InResourcePath)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}
		check(InContext);

		UE_LOGF(LogPCG, Warning, "[GRIDLINKING] [%ls] RETRIEVE: FAILED: No overlapping local source found. This may be expected. Path=%ls",
			*PCGLog::GetExecutionSourceName(InContext->ExecutionSource.Get(), /*bUseLabel=*/true),
			*InResourcePath);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGridLinkageTaskExecuteRetrieveNoData(const FPCGContext* InContext, const IPCGGraphExecutionSource* InExecutionSource, const FString& InResourcePath)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}
		check(InContext);

		UE_LOGF(LogPCG, Warning, "[GRIDLINKING] [%ls] RETRIEVE: FAILED: No data found on source. Source=%ls, Path=%ls",
			*PCGLog::GetExecutionSourceName(InContext->ExecutionSource.Get(), /*bUseLabel=*/true),
			*PCGLog::GetExecutionSourceName(InExecutionSource, /*bUseLabel=*/true),
			*InResourcePath);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}

	void LogGridLinkageTaskStoreNullDataOnCulled(const IPCGGraphExecutionSource* InExecutionSource, uint32 InGenerationGrid, uint32 InFromGridSize, uint32 InToGridSize, const FString& InResourcePath)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
		if (!LogEnabled())
		{
			return;
		}

		UE_LOGF(LogPCG, Log, "[GRIDLINKING] [%ls] STORE NULL ON CULLED. GenerationGridSize=%u, FromGridSize=%u, ToGridSize=%u, Path=%ls",
			*PCGLog::GetExecutionSourceName(InExecutionSource, /*bUseLabel=*/true),
			InGenerationGrid,
			InFromGridSize,
			InToGridSize,
			*InResourcePath);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING
	}
}
