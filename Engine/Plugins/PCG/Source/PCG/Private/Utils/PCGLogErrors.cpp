// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/PCGLogErrors.h"

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGModule.h"

#define LOCTEXT_NAMESPACE "PCGLog"

void PCGLog::LogErrorOnGraph(const FText& InMsg, const FPCGContext* InContext)
{
	if (const FPCGContext* Context = InContext ? InContext : FPCGContext::GetContextForThread())
	{
		PCGE_LOG_C(Error, GraphAndLog, Context, InMsg);
	}
	else
	{
		UE_LOGF(LogPCG, Error, "[NOCONTEXT] %ls", *InMsg.ToString());
	}
}

void PCGLog::LogWarningOnGraph(const FText& InMsg, const FPCGContext* InContext)
{
	if (const FPCGContext* Context = InContext ? InContext : FPCGContext::GetContextForThread())
	{
		PCGE_LOG_C(Warning, GraphAndLog, Context, InMsg);
	}
	else
	{
		UE_LOGF(LogPCG, Warning, "[NOCONTEXT] %ls", *InMsg.ToString());
	}
}

namespace PCGLog::InputOutput
{
	namespace Format
	{
		const FText InvalidInputData = LOCTEXT("InvalidInputData", "Invalid input data.");

		const FTextFormat TypedInputNotFound = LOCTEXT("TypedInputNotFound", "Data of type {0} not found on pin '{1}'.");
		const FTextFormat FirstInputOnly = LOCTEXT("FirstInputOnly", "Multiple inputs found on single-input pin '{0}'. Only the first will be selected.");
		const FTextFormat InvalidCardinality = LOCTEXT("InvalidCardinality", "Invalid cardinality among pins '{0}' and '{1}'. They must match 1:1, N:1, or N:N.");
		const FTextFormat FileNotFound = LOCTEXT("FileNotFound", "File was not found at given path: '{0}'");
		const FTextFormat InvalidFileType = LOCTEXT("InvalidFileType", "File was of the wrong type: '{0}' Expected type: {1}");
		const FTextFormat DirectoryNotFound = LOCTEXT("DirectoryNotFound", "Directory was not found at given path: '{0}'");
	}

	// Warnings
	void LogTypedDataNotFoundWarning(EPCGDataType DataType, const FName PinLabel, const FPCGContext* InContext)
	{
		const UEnum* PCGDataTypeEnum = StaticEnum<EPCGDataType>();
		const FText TypeText = PCGDataTypeEnum ? PCGDataTypeEnum->GetDisplayNameTextByValue(static_cast<int64>(DataType)) : LOCTEXT("UnknownDataType", "Unknown");
		LogWarningOnGraph(FText::Format(Format::TypedInputNotFound, TypeText, FText::FromName(PinLabel)), InContext);
	}

	void LogFirstInputOnlyWarning(const FName PinLabel, const FPCGContext* InContext)
	{
		LogWarningOnGraph(FText::Format(Format::FirstInputOnly, FText::FromName(PinLabel)), InContext);
	}

	// Errors
	void LogInvalidInputDataError(const FPCGContext* InContext)
	{
		LogErrorOnGraph(Format::InvalidInputData, InContext);
	}

	void LogInvalidCardinalityError(const FName SourcePinLabel, const FName TargetPinLabel, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::InvalidCardinality, FText::FromName(SourcePinLabel), FText::FromName(TargetPinLabel)), InContext);
	}

	void LogFileNotFound(const FString& FilePath, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::FileNotFound, FText::FromString(FilePath)), InContext);
	}

	void LogInvalidFileType(const FString& FileType, const FString& ExpectedType, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::InvalidFileType, FText::FromString(FileType), FText::FromString(ExpectedType)), InContext);
	}

	void LogDirectoryNotFound(const FString& DirectoryPath, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::DirectoryNotFound, FText::FromString(DirectoryPath)), InContext);
	}
}

namespace PCGLog::Metadata
{
	namespace Format
	{
		const FTextFormat CreateAccessorFailure = LOCTEXT("CreateAccessorFailure", "Could not create accessor. Attribute '{0}' was not found.");
		const FTextFormat CreateAttributeFailure = LOCTEXT("CreateAttributeFailure", "Could not create attribute '{0}' of type '{1}'.");
		const FTextFormat GetAttributeFailure = LOCTEXT("GetAttributeFailure", "Could not retrieve attribute '{0}' value.");
		const FTextFormat GetTypedAttributeFailure = LOCTEXT("GetTypedAttributeFailure", "Could not retrieve attribute '{0}' value. Expected type: {1}, Actual Type: {2}.");
		const FTextFormat GetTypedAttributeFailureNoAccessor = LOCTEXT("GetTypedAttributeFailureNoAccessor", "Could not retrieve attribute '{0}' value of type: '{1}.");
		const FTextFormat SetTypedAttributeFailure = LOCTEXT("SetTypedAttributeFailure", "Could not set value in attribute '{0}'. Attribute type: {1}, Value Type: {2}.");
		const FTextFormat SetTypedAttributeFailureNoAccessor = LOCTEXT("SetTypedAttributeFailureNoAccessor", "Could not set value in attribute '{0}' value of type: '{1}.");
		const FTextFormat IncomparableTypesFailure = LOCTEXT("IncomparableTypesFailure", "Attributes '{0}' and '{1}' are incomparable. Ensure they are either of the same or compatible types.");
		const FTextFormat InvalidMetadataDomain = LOCTEXT("InvalidMetadataDomain", "Metadata domain {0} is invalid for this data.");
		const FText InvalidMetadata = LOCTEXT("InvalidMetadata", "Metadata is unsupported or invalid for this data.");
	}

	void LogFailToCreateAccessorError(const FPCGAttributePropertySelector& Selector, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::CreateAccessorFailure, Selector.GetDisplayText()), InContext);
	}

	void LogInvalidMetadata(const FPCGContext* InContext)
	{
		LogErrorOnGraph(Format::InvalidMetadata, InContext);
	}

	void LogFailToCreateAttributeError(const FPCGMetadataAttributeDesc& AttributeDesc, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::CreateAttributeFailure, FText::FromName(AttributeDesc.Name), AttributeDesc.GetTypeText()), InContext);
	}

	void LogInvalidMetadataDomain(const FPCGAttributePropertySelector& Selector, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::InvalidMetadataDomain, FText::FromName(Selector.GetDomainName())), InContext);
	}

	void LogFailToGetAttributeError(const FText& AttributeName, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::GetAttributeFailure, AttributeName), InContext);
	}

	void LogFailToGetAttributeError(FName AttributeName, const FPCGContext* InContext)
	{
		LogFailToGetAttributeError(FText::FromName(AttributeName), InContext);
	}

	void LogFailToGetAttributeError(const FPCGAttributePropertySelector& Selector, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::GetAttributeFailure, Selector.GetDisplayText()), InContext);
	}

	void LogIncomparableAttributesError(const FPCGAttributePropertySelector& FirstSelector, const FPCGAttributePropertySelector& SecondSelector, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::IncomparableTypesFailure, FirstSelector.GetDisplayText(), SecondSelector.GetDisplayText()), InContext);
	}
}

namespace PCGLog::Parsing
{
	namespace Format
	{
		const FText EmptyExpression = LOCTEXT("EmptyExpression", "Empty expression in parsed string.");

		const FTextFormat InvalidCharacter = LOCTEXT("InvalidCharacter", "Invalid character in parsed string: '{0}'.");
		const FTextFormat InvalidExpression = LOCTEXT("InvalidExpression", "Invalid expression in parsed string: '{0}'.");
	}

	// Warnings
	void LogEmptyExpressionWarning(const FPCGContext* InContext)
	{
		LogWarningOnGraph(Format::EmptyExpression, InContext);
	}

	// Errors
	void LogInvalidCharacterInParsedStringError(const FStringView& ParsedString, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::InvalidCharacter, FText::FromStringView(ParsedString)), InContext);
	}

	void LogInvalidExpressionInParsedStringError(const FStringView& ParsedString, const FPCGContext* InContext)
	{
		LogErrorOnGraph(FText::Format(Format::InvalidExpression, FText::FromStringView(ParsedString)), InContext);
	}
}

namespace PCGLog::Component
{
	namespace Format
	{
		const FText ComponentAttachmentFailed = LOCTEXT("ComponentAttachmentFailed", "Failed to attach the component, check the logs.");
	}
	
	void LogComponentAttachmentFailedWarning(const FPCGContext* InContext)
	{
		LogWarningOnGraph(Format::ComponentAttachmentFailed, InContext);
	}
}

namespace PCGLog::Settings
{
	namespace Format
	{
		const FTextFormat InvalidPreconfiguration = LOCTEXT("InvalidPreconfiguration", "Invalid preconfiguration index '{0}' for node settings '{1}'. Default settings will be used.");
		const FTextFormat InvalidConversion = LOCTEXT("InvalidConversion", "Invalid conversion for preconfiguration index '{0}' for node settings '{1}'. Reason: {2}");
	}

	// Warnings
	void LogInvalidPreconfigurationWarning(const int32 PreconfigurationIndex, const FText& NodeTitle)
	{
		LogWarningOnGraph(FText::Format(Format::InvalidPreconfiguration, FText::AsNumber(PreconfigurationIndex), NodeTitle));
	}

	// Errors
	void LogInvalidConversionError(const int32 PreconfigurationIndex, const FText& NodeTitle, const FText& Reason)
	{
		LogErrorOnGraph(FText::Format(Format::InvalidConversion, FText::AsNumber(PreconfigurationIndex), NodeTitle, Reason));
	}
}

namespace PCGLog::Landscape
{
	namespace Format
	{
		const FText LandscapeCacheNotAvailable = LOCTEXT("LandscapeCacheNotAvailableInPIEOrCookedBuilds", "PCG Landscape cache (on the PCG World Actor) is not set to be serialized and will not work in non-editor modes.");
	}

	void LogLandscapeCacheNotAvailableError(const FPCGContext* InContext)
	{
		LogErrorOnGraph(Format::LandscapeCacheNotAvailable, InContext);
	}
}

namespace PCGLog::Validation
{
	namespace Format
	{
		const FText InvalidRotationReset = LOCTEXT("InvalidRotationsReset", "Some elements have an invalid rotation and had their rotation reset. Make sure you are not setting individual components of the rotation quaternion (like `$Rotation.X`) without normalizing the quaternion afterwards.");
		const FText InvalidRotationSkip = LOCTEXT("InvalidRotationsSkip", "Some elements have an invalid rotation and were skipped. Make sure you are not setting individual components of the rotation quaternion (like `$Rotation.X`) without normalizing the quaternion afterwards.");
	}

	void LogInvalidRotationsReset(const FPCGContext* InContext)
	{
		LogErrorOnGraph(Format::InvalidRotationReset, InContext);
	}

	void LogInvalidRotationsSkipped(const FPCGContext* InContext)
	{
		LogErrorOnGraph(Format::InvalidRotationSkip, InContext);
	}
}

#undef LOCTEXT_NAMESPACE
