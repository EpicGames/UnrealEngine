// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/PCGNodeVisualLogs.h"

#include "PCGComponent.h"
#include "PCGModule.h"

#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "Containers/Ticker.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "PCGNode"

#if WITH_EDITOR

namespace PCGNodeVisualLogsConstants
{
	static constexpr int MaxLogsInSummary = 8;
	static const FText Warning = LOCTEXT("PCGLogWarning", "Warning");
	static const FText Error = LOCTEXT("PCGLogError", "Error");
}

void FPCGNodeVisualLogs::Log(const FPCGStack& InPCGStack, ELogVerbosity::Type InVerbosity, const FText& InMessage)
{
	bool bAdded = false;

	{
		PCG::TUniqueScopeLock WriteLock(LogsLock);

		FPCGPerNodeVisualLogs& NodeLogs = StackToLogs.FindOrAdd(InPCGStack);

		constexpr int32 MaxLogged = 1024;
		if (StackToLogs.Num() < MaxLogged)
		{
			NodeLogs.Emplace(InMessage, InVerbosity);

			bAdded = true;
		}
	}

	// Broadcast outside of write scope lock
	if (bAdded && !InPCGStack.GetStackFrames().IsEmpty())
	{
		const bool bIsInGameThread = IsInGameThread();

		TArray<TWeakObjectPtr<const UPCGNode>> NodeWeakPtrs;
		{
			FGCScopeGuard Guard;
			for (const FPCGStackFrame& Frame : InPCGStack.GetStackFrames())
			{
				if (const UPCGNode* Node = Frame.GetObject_NoGuard<UPCGNode>())
				{
					if (bIsInGameThread)
					{
						Node->OnNodeChangedDelegate.Broadcast(const_cast<UPCGNode*>(Node), EPCGChangeType::Cosmetic);
					}
					else
					{
						NodeWeakPtrs.Add(Node);
					}
				}
			}
		}

		if(!NodeWeakPtrs.IsEmpty())
		{
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [NodeWeakPtrs]()
			{
				for(const TWeakObjectPtr<const UPCGNode>& NodeWeakPtr : NodeWeakPtrs)
				{
					if (const UPCGNode* Node = NodeWeakPtr.Get())
					{
						Node->OnNodeChangedDelegate.Broadcast(const_cast<UPCGNode*>(Node), EPCGChangeType::Cosmetic);
					}
				}
			});
		}
	}
}

bool FPCGNodeVisualLogs::HasLogs(const FPCGStack& InPCGStack) const
{
	PCG::TSharedScopeLock ReadLock(LogsLock);

	for (const TPair<FPCGStack, FPCGPerNodeVisualLogs>& Entry : StackToLogs)
	{
		if (Entry.Key.BeginsWith(InPCGStack) && !Entry.Value.IsEmpty())
		{
			return true;
		}
	}

	return false;
}

bool FPCGNodeVisualLogs::HasLogs(const FPCGStack& InPCGStack, ELogVerbosity::Type& OutMinVerbosity) const
{
	OutMinVerbosity = ELogVerbosity::All;
	
	PCG::TSharedScopeLock ReadLock(LogsLock);

	for (const TPair<FPCGStack, FPCGPerNodeVisualLogs>& Entry : StackToLogs)
	{
		if (Entry.Key.BeginsWith(InPCGStack))
		{
			for (const FPCGNodeLogEntry& Log : Entry.Value)
			{
				OutMinVerbosity = FMath::Min(OutMinVerbosity, Log.Verbosity);
			}
		}
	}

	return OutMinVerbosity != ELogVerbosity::All;
}

bool FPCGNodeVisualLogs::HasLogsOfVerbosity(const FPCGStack& InPCGStack, ELogVerbosity::Type InVerbosity) const
{
	PCG::TSharedScopeLock ReadLock(LogsLock);

	for (const TPair<FPCGStack, FPCGPerNodeVisualLogs>& Entry : StackToLogs)
	{
		if (Entry.Key.BeginsWith(InPCGStack))
		{
			if (Algo::FindByPredicate(Entry.Value, [InVerbosity](const FPCGNodeLogEntry& Log) { return Log.Verbosity == InVerbosity; }))
			{
				return true;
			}
		}
	}

	return false;
}

FPCGPerNodeVisualLogs FPCGNodeVisualLogs::GetLogs(const FPCGStack& InPCGStack) const
{
	FPCGPerNodeVisualLogs Logs;

	ForAllMatchingLogs(InPCGStack, [&Logs](const FPCGStack&, const FPCGPerNodeVisualLogs& InLogs)
	{
		Logs.Append(InLogs);
		return true;
	});
	
	return Logs;
}

void FPCGNodeVisualLogs::ForAllMatchingLogs(const FPCGStack& InPCGStack, TFunctionRef<bool(const FPCGStack&, const FPCGPerNodeVisualLogs&)> InFunc) const
{
	PCG::TSharedScopeLock ReadLock(LogsLock);
	for (const TPair<FPCGStack, FPCGPerNodeVisualLogs>& Entry : StackToLogs)
	{
		if (Entry.Key.BeginsWith(InPCGStack))
		{
			if (!InFunc(Entry.Key, Entry.Value))
			{
				break;
			}
		}
	}
}

void FPCGNodeVisualLogs::GetLogsAndSources(const UPCGNode* InNode, FPCGPerNodeVisualLogs& OutLogs, TArray<const IPCGGraphExecutionSource*>& OutExecutionSources) const
{
	OutLogs.Reset();
	OutExecutionSources.Reset();

	PCG::TSharedScopeLock ReadLock(LogsLock);
	for (const TPair<FPCGStack, FPCGPerNodeVisualLogs>& Entry : StackToLogs)
	{
		const FPCGStack& Stack = Entry.Key;
		if (Stack.HasObject(InNode))
		{
			OutLogs.Append(Entry.Value);
			const IPCGGraphExecutionSource* RootSource = Stack.GetRootSource();
			for (int LogIndex = 0; LogIndex < Entry.Value.Num(); ++LogIndex)
			{
				OutExecutionSources.Add(RootSource);
			}
		}
	}
}

FText FPCGNodeVisualLogs::GetSummaryTextWithSources(const FPCGPerNodeVisualLogs& InLogs, const TArray<const IPCGGraphExecutionSource*>* InExecutionSources, ELogVerbosity::Type* OutMinimumVerbosity)
{
	check(!InExecutionSources || InLogs.Num() == InExecutionSources->Num());

	FText Summary = FText::GetEmpty();
	ELogVerbosity::Type MinVerbosity = ELogVerbosity::All;

	const int32 NumLogs = FMath::Min(InLogs.Num(), PCGNodeVisualLogsConstants::MaxLogsInSummary);

	for (int32 LogIndex = 0; LogIndex < NumLogs; ++LogIndex)
	{
		const FPCGNodeLogEntry& LogEntry = InLogs[LogIndex];

		const FText& MessageVerbosity = (LogEntry.Verbosity == ELogVerbosity::Warning ? PCGNodeVisualLogsConstants::Warning : PCGNodeVisualLogsConstants::Error);
		if (InExecutionSources)
		{
			const IPCGGraphExecutionSource* ExecutionSource = (*InExecutionSources)[LogIndex];
			const UPCGComponent* Component = Cast<UPCGComponent>(ExecutionSource);

			FText SourceName;
			// Backward compat
			if (Component && Component->GetOwner())
			{
				SourceName = FText::FromString(Component->GetOwner()->GetActorLabel());
			}
			else if(ExecutionSource)
			{ 
				SourceName = FText::FromString(ExecutionSource->GetExecutionState().GetDebugName());
			}
			else
			{
				SourceName = LOCTEXT("PCGLogMissingSource", "MissingSource");
			}

			if (Summary.IsEmpty())
			{
				Summary = FText::Format(LOCTEXT("NodeTooltipLogWithActorEmpty", "[{0}] {1}: {2}"), SourceName, MessageVerbosity, LogEntry.Message);
			}
			else
			{
				Summary = FText::Format(LOCTEXT("NodeTooltipLogWithActor", "{0}\n[{1}] {2}: {3}"), Summary, SourceName, MessageVerbosity, LogEntry.Message);
			}
		}
		else
		{
			if (Summary.IsEmpty())
			{
				Summary = FText::Format(LOCTEXT("NodeTooltipLogEmpty", "{0}: {1}"), MessageVerbosity, LogEntry.Message);
			}
			else
			{
				Summary = FText::Format(LOCTEXT("NodeTooltipLog", "{0}\n{1}: {2}"), Summary, MessageVerbosity, LogEntry.Message);
			}
		}

		MinVerbosity = FMath::Min(MinVerbosity, LogEntry.Verbosity);
	}

	// Check log level for all entries, not only the first entries
	for (int32 LogIndex = NumLogs; LogIndex < InLogs.Num(); ++LogIndex)
	{
		const FPCGNodeLogEntry& LogEntry = InLogs[LogIndex];
		MinVerbosity = FMath::Min(MinVerbosity, LogEntry.Verbosity);
	}

	// Finally, if we had at most the limit of entries, we'll add an ellipsis.
	if (InLogs.Num() > PCGNodeVisualLogsConstants::MaxLogsInSummary)
	{
		Summary = FText::Format(LOCTEXT("NodeTooltipEllipsis", "{0}\n..."), Summary);
	}

	if (OutMinimumVerbosity)
	{
		*OutMinimumVerbosity = MinVerbosity;
	}

	return Summary;
}


FText FPCGNodeVisualLogs::GetLogsSummaryText(const UPCGNode* InNode, ELogVerbosity::Type& OutMinimumVerbosity) const
{
	FPCGPerNodeVisualLogs Logs;
	TArray<const IPCGGraphExecutionSource*> ExecutionSources;
	GetLogsAndSources(InNode, Logs, ExecutionSources);

	return GetSummaryTextWithSources(Logs, &ExecutionSources, &OutMinimumVerbosity);
}

FText FPCGNodeVisualLogs::GetLogsSummaryText(const FPCGStack& InBaseStack, ELogVerbosity::Type* OutMinimumVerbosity) const
{
	return GetSummaryTextWithSources(GetLogs(InBaseStack), nullptr, OutMinimumVerbosity);
}

void FPCGNodeVisualLogs::ClearLogs(const FPCGStack& InPCGStack)
{
	TSet<const UPCGNode*> TouchedNodes;

	{
		FGCScopeGuard Guard;
		PCG::TUniqueScopeLock WriteLock(LogsLock);

		TArray<FPCGStack> StacksToRemove;
		for (const TPair<FPCGStack, FPCGPerNodeVisualLogs>& Entry : StackToLogs)
		{
			// Always take every opportunity to flush messages logged against invalid/dead sources.
			const bool bSourceValid = Entry.Key.GetRootSource() && IsValid(Cast<UObject>(Entry.Key.GetRootSource()));

			if (!bSourceValid || Entry.Key.BeginsWith(InPCGStack))
			{
				if (!bSourceValid)
				{
					UE_LOGF(LogPCG, Verbose, "Cleared out logs for null source.");
				}

				StacksToRemove.Add(Entry.Key);

				for (const FPCGStackFrame& Frame : Entry.Key.GetStackFrames())
				{
					if (const UPCGNode* Node = Frame.GetObject_NoGuard<UPCGNode>())
					{
						TouchedNodes.Add(Node);
					}
				}
			}
		}

		for (const FPCGStack& StackToRemove : StacksToRemove)
		{
			StackToLogs.Remove(StackToRemove);
		}
	}

	// Broadcast change notification outside of write scope lock
	if (IsInGameThread())
	{
		for (const UPCGNode* Node : TouchedNodes)
		{
			if (Node)
			{
				Node->OnNodeChangedDelegate.Broadcast(const_cast<UPCGNode*>(Node), EPCGChangeType::Cosmetic);
			}
		}
	}
	else if(!TouchedNodes.IsEmpty())
	{
		TArray<TWeakObjectPtr<const UPCGNode>> NodeWeakPtrs;
		NodeWeakPtrs.Reserve(TouchedNodes.Num());
		Algo::Transform(TouchedNodes, NodeWeakPtrs, [](const UPCGNode* Node) { return Node; });

		ExecuteOnGameThread(UE_SOURCE_LOCATION, [NodeWeakPtrs]()
		{
			for (const TWeakObjectPtr<const UPCGNode>& NodeWeakPtr : NodeWeakPtrs)
			{
				if (const UPCGNode* Node = NodeWeakPtr.Get())
				{
					Node->OnNodeChangedDelegate.Broadcast(const_cast<UPCGNode*>(Node), EPCGChangeType::Cosmetic);
				}
			}
		});
	}
}

void FPCGNodeVisualLogs::ClearLogs(const IPCGGraphExecutionSource* InSource)
{
	if (const UObject* SourceObject = Cast<UObject>(InSource))
	{
		FPCGStack Stack;
		Stack.PushFrame(SourceObject);
		ClearLogs(Stack);
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
