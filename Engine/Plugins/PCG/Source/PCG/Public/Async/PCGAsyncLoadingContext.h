// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Engine/StreamableManager.h"
#include "UObject/SoftObjectPath.h"

#define UE_API PCG_API

struct FPCGContext;

/**
* Extension interface to add Async Loading support to any Context.
* Just inherit IPCGAsyncLoadingContext on the element context.
*/
struct IPCGAsyncLoadingContext
{
public:
	UE_API virtual ~IPCGAsyncLoadingContext();

	/** Request a load. If load was already requested, do nothing. LoadHandle will be set in the context, meaning that assets will stay alive while context is loaded.
	* Request can be synchronous or asynchronous. If loading is asynchronous, the current task is paused and will be woken up when the loading is done.
	* Returns true if the execution can continue (objects are loaded or invalid), or false if we need to wait for loading
	*/
	UE_API bool RequestResourceLoad(FPCGContext* ThisContext, TArray<FSoftObjectPath>&& ObjectsToLoad, bool bAsynchronous = true);

	UE_API void CancelLoading();

	bool WasLoadRequested() const { return bLoadRequested; }

	/**
	 * If we need multiple loads (like ObjectA first load, and has a soft ObjectB that also needs to be loaded), we can use this function to reset the bLoadRequested flag.
	 * Only works if the loading was requested and is done.
	 * Returns true if a new loading can be requested.
	 */
	UE_API bool ResetLoadRequestIfDone();

private:
	/** Resolve already loaded objects and store a reference on them and also remove null paths, will modify in place the array of objects that actually require loading. */
	UE_API void ResolveAlreadyLoadedObjects(TArray<FSoftObjectPath>& ObjectsToLoad);

	/** If the load was already requested */
	bool bLoadRequested = false;

	/** Handle holder for any loaded resources */
	TSharedPtr<FStreamableHandle> LoadHandle;

	/** 
	* Keep a reference on objects already loaded, to avoid starting a loading task if everything is already loaded.
	* It's a strong object ptr to make sure it doesn't get GC before the end of the execution of the PCG Element.
	*/
	TArray<TStrongObjectPtr<UObject>> AlreadyLoadedObjects;
};

#undef UE_API
