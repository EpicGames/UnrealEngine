// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Volume.h"
#include "PCGBuilderVolume.generated.h"

UCLASS(DisplayName = "PCG Builder Volume", hidecategories = (Actor, Advanced, Display, Events, Object, Attachment, Info, Input, Blueprint, Layers, Tags, Replication), MinimalAPI)
class APCGBuilderVolume : public AVolume
{
	GENERATED_BODY()

public:
	PCG_API APCGBuilderVolume();

	virtual bool IsEditorOnly() const final { return true; }

#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const final { return false; }
	virtual bool ActorTypeSupportsExternalDataLayer() const final { return false; }
	virtual bool CanChangeIsSpatiallyLoadedFlag() const { return false; }
#endif
};
