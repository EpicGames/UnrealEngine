// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "PCGCommon.h"
#include "PCGSettings.h"
#include "Elements/PCGActorSelector.h"

class UPCGGraph;
class IPCGGraphExecutionSource;

struct FPCGChangeTracker
{
public:
	PCG_API void AddLocalSourceDynamicTrackingKeys(const FPCGSelectionKeyToSettingsMap& InLocalSourceDynamicTrackingKeys);
	PCG_API void AddLocalSourceCurrentTrackingKeys(const FPCGSelectionKeyToSettingsMap& InLocalSourceCurrentExecutionTrackingKeys, const TSet<const UPCGSettings*>& InLocalSourceCurrentExecutionTrackingSettings);
	PCG_API void RegisterDynamicTracking(const UPCGSettings* InSettings, const TArrayView<TPair<FPCGSelectionKey, bool>>& InDynamicKeysAndCulling);
	PCG_API void RegisterDynamicTracking(const FPCGSelectionKeyToSettingsMap& InKeysToSettings);
	PCG_API void UpdateDynamicTracking(IPCGGraphExecutionSource* InExecutionSource);
	PCG_API void ResetCurrentExecution();

	PCG_API TArray<FPCGSelectionKey> GatherTrackingKeys() const;
	PCG_API bool IsKeyTrackedAndCulled(const FPCGSelectionKey& Key, bool& bOutIsCulled) const;
	PCG_API bool UpdateTrackingCache(const UPCGGraph* InGraph, TArray<FPCGSelectionKey>* OptionalChangedKeys);
	PCG_API void ForEachSettingTrackingKey(const FPCGSelectionKey& InKey, const TFunctionRef<void(const FPCGSelectionKey&, const FPCGSettingsAndCulling&)> InCallback) const;
	PCG_API void Serialize(FArchive& Ar);

private: 
	// @todo_pcg: remove friending and expose what is needed
	friend class UPCGComponent;
	friend struct FPCGComponentInstanceData;

	FPCGSelectionKeyToSettingsMap StaticallyTrackedKeysToSettings;

	// Temporary storage for dynamic tracking that will be filled during execution source execution.
	FPCGSelectionKeyToSettingsMap CurrentExecutionDynamicTracking;
	// Temporary storage for dynamic tracking that will keep all settings that could have dynamic tracking, in order to detect changes.
	TSet<const UPCGSettings*> CurrentExecutionDynamicTrackingSettings;
	mutable FTransactionallySafeCriticalSection CurrentExecutionDynamicTrackingLock;

	// Need to keep a reference to all tracked settings to still react to changes after a map load (since the execution source won't have been executed).
	// Serialization will be done in the Serialize function
	FPCGSelectionKeyToSettingsMap DynamicallyTrackedKeysToSettings;
};

#endif