// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Delegates/Delegate.h"
#include "Elements/PCGActorSelector.h"

class FPCGTrackingManager;

#if WITH_EDITOR

// Unique change types
using FPCGTrackingChangeID = FGuid;

// This interface provides change tracking for different systems that may live across different plugins and have those trackers send out formatted events to their FPCGTrackingManager owner.
class IPCGChangeTracker
{
public:
	IPCGChangeTracker(FPCGTrackingManager* InOwner)
		: Owner(InOwner)
	{
		check(Owner);
	}

	virtual ~IPCGChangeTracker() = default;

	// Called every frame from FPCGTrackingManager.
	virtual void Tick() {}

	// Instead of hooking up to the core delegate, the FPCGTrackingManager will call this method and returning true means that this tracker is handling the event.
	virtual bool OnObjectPropertyChanged(const UObject* InObject, FPropertyChangedEvent& InEvent) { return false; }

	// Can be implemented to react to other tracker events (currently used to send out Landscape tracker event to Actor tracker).
	virtual bool OnNotifyObjectChanged(const UObject* InObject, const FPCGTrackingChangeID& InChangeID, const UObject* InOriginatingChangeObject) { return false; }

	// Called from FPCGTrackingManager::OnSelectionKeyChanged to allow skipping the execution source refresh
	virtual bool ShouldSkipRefresh(const UObject* InChangedObject) { return false; }

protected:
	FPCGTrackingManager* Owner = nullptr;
};

// This interface provides handling of changes once they are processed through the FPCGTrackingManager
class IPCGChangeHandler
{
public:
	IPCGChangeHandler(FPCGTrackingManager* InOwner)
		: Owner(InOwner)
	{
		check(Owner);
	}

	struct FPCGChangeHandlerChange
	{
	public:
		// Root object to attribute change to
		const UObject* ChangedObject = nullptr;
		// Object the change originates from (can be sub-object of ChangedObject)
		const UObject* OriginatingChangeObject = nullptr;
		// Tracked keys that match the ChangedObject or OriginatingChangedObject
		TArray<FPCGSelectionKey> MatchedKeys;
		// Flag passed down to FPCGTrackingManager::OnSelectionKeyChanged
		bool bNoRefreshOwner = false;
	};

	virtual ~IPCGChangeHandler() = default;

	virtual void BeginChangeHandling(const TSharedRef<FPCGChangeHandlerChange>& InChange) {}
	virtual void HandleChange(IPCGGraphExecutionSource* InExecutionSource) {}
	virtual void HandleBoundedChange(IPCGGraphExecutionSource* InExecutionSource, const FBox& InExecutionSourceBounds, const FBox& InChangeBounds) {}
	virtual void EndChangeHandling(bool bSkipRefresh) {}

protected:
	FPCGTrackingManager* Owner = nullptr;
};

DECLARE_DELEGATE_RetVal_OneParam(TUniquePtr<IPCGChangeTracker>, FPCGCreateTrackerInstance, FPCGTrackingManager*);
DECLARE_DELEGATE_RetVal_OneParam(TUniquePtr<IPCGChangeHandler>, FPCGCreateHandlerInstance, FPCGTrackingManager*);

#endif // WITH_EDITOR

// FPCGGetExecutionSourcesFromSelectionKey params
struct FPCGGetExecutionSourcesFromSelectionKeyParams
{
	FPCGGetExecutionSourcesFromSelectionKeyParams() = delete;
	FPCGGetExecutionSourcesFromSelectionKeyParams(const FPCGSelectionKey& InSelectionKey, EPCGActorFilter InActorFilter)
		: SelectionKey(InSelectionKey)
		, ActorFilter(InActorFilter)
	{

	}

	const FPCGSelectionKey& SelectionKey;
	EPCGActorFilter ActorFilter;
};

DECLARE_DELEGATE_RetVal_OneParam(TArray<IPCGGraphExecutionSource*>, FPCGGetExecutionSourcesFromSelectionKey, const FPCGGetExecutionSourcesFromSelectionKeyParams&);

class FPCGChangeTrackingRegistry
{
public:

#if WITH_EDITOR
	// Creates the tracker instances for the FPCGTrackingManager
	TArray<TUniquePtr<IPCGChangeTracker>> CreateChangeTrackers(FPCGTrackingManager* InManager) const;
	TArray<TUniquePtr<IPCGChangeHandler>> CreateChangeHandlers(FPCGTrackingManager* InManager) const;

	// Register/unregister custom tracker factories (allows other modules to register new trackers)
	PCG_API void RegisterTrackerFactory(FName InName, FPCGCreateTrackerInstance InTrackerFactory);
	PCG_API void UnregisterTrackerFactory(FName InName);

	// Register/unregister custom handler factories (allows other modules to register new handlers)
	PCG_API void RegisterHandlerFactory(FName InName, FPCGCreateHandlerInstance InHandlerFactory);
	PCG_API void UnregisterHandlerFactory(FName InName);
#endif

	TArray<IPCGGraphExecutionSource*> GetExecutionSourcesFromSelectionKey(const FPCGGetExecutionSourcesFromSelectionKeyParams& InParams) const;

	// Register/unregister execution source resolving
	PCG_API void RegisterGetExecutionSourcesFromSelectionKey(UClass* InSelectedObjectClass, FPCGGetExecutionSourcesFromSelectionKey InGetExecutionSourcesFromKey);
	PCG_API void UnregisterGetExecutionSourcesFromSelectionKey(UClass* InSelectedObjectClass);

private:

#if WITH_EDITOR
	TMap<FName, FPCGCreateTrackerInstance> TrackerFactories;
	TMap<FName, FPCGCreateHandlerInstance> HandlerFactories;
#endif

	TMap<UClass*, FPCGGetExecutionSourcesFromSelectionKey> GetExecutionSourcesFromSelectionKeyEntries;
};
