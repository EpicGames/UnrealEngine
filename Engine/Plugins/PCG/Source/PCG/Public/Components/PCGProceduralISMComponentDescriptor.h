// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISMPartition/ProceduralISMComponentDescriptor.h"

#include "PCGProceduralISMComponentDescriptor.generated.h"

struct FPCGSoftISMComponentDescriptor;

/** Struct that holds properties that can be used to initialize PCG Procedural ISM Components. */
USTRUCT()
struct FPCGProceduralISMComponentDescriptor : public FProceduralISMComponentDescriptor
{
	GENERATED_BODY()

public:
	PCG_API FPCGProceduralISMComponentDescriptor& operator=(const FPCGSoftISMComponentDescriptor& Other);

	PCG_API bool operator==(const FPCGProceduralISMComponentDescriptor& Other) const;
	bool operator!=(const FPCGProceduralISMComponentDescriptor& Other) const { return !(*this == Other); }

	friend inline uint32 GetTypeHash(const FPCGProceduralISMComponentDescriptor& Key)
	{
		return Key.GetTypeHash();
	}

public:
	UPROPERTY()
	TArray<FName> ComponentTags;
};
