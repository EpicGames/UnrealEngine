// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"

#include "Compute/PCGDataDescription.h"

#include "Misc/SpinLock.h"

#include "PCGProxyForGPUData.generated.h"

struct FPCGContext;
struct FPCGProxyForGPUDataCollection;

class FRDGPooledBuffer;
class FRHIGPUBufferReadback;

USTRUCT()
struct FPCGDataTypeInfoProxyForGPU : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::ProxyForGPU)

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; }
#endif // WITH_EDITOR
};

/** A proxy for data residing on the GPU with functionality to read the data back to the CPU. */
UCLASS(MinimalAPI, ClassGroup = (Procedural), DisplayName = "GPU Proxy")
class UPCGProxyForGPUData : public UPCGData
{
	GENERATED_BODY()

public:
	void Initialize(TSharedPtr<FPCGProxyForGPUDataCollection> InDataCollection, int InDataIndexInCollection);

	//~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoProxyForGPU)

	PCG_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	virtual bool CanBeSerialized() const override { return false; }
	virtual bool IsCacheable() const override { return false; }
	PCG_API virtual void ReleaseTransientResources(const TCHAR* InReason = nullptr) override;
	PCG_API virtual EPCGDataType GetUnderlyingDataType() const override;
	PCG_API virtual FPCGDataTypeBaseId GetUnderlyingDataTypeId() const override;
	//~End UPCGData interface

	PCG_API virtual FIntVector4 GetNumElements() const;

	/** Returns the GPU info. Returns null if the buffer has been discarded. */
	TSharedPtr<const FPCGProxyForGPUDataCollection> GetInputDataCollectionInfo() const;

	int GetDataIndexInCollection() const;

	struct FReadbackResult
	{
		/** Set false until read back has been performed. */
		bool bComplete = false;

		/** The data created from readback, null if read back failed. */
		FPCGTaggedData TaggedData;
	};

	/** Populates a CPU data object representing the GPU data, performing a readback from GPU->CPU if required. */
	PCG_API FReadbackResult GetCPUData(FPCGContext* InContext) const;

	/** Returns the element count for this data. Does not trigger a GPU->CPU readback. */
	FIntVector4 GetElementCount() const;

	/** Returns a description of this data. Does not trigger a GPU->CPU readback. */
	bool GetDescription(FPCGDataDesc& OutDescription) const;

	TSharedPtr<const FPCGProxyForGPUDataCollection> GetGPUInfo() const;

private:
	TSharedPtr<FPCGProxyForGPUDataCollection> GetGPUInfoMutable() const;

private:
	UPROPERTY()
	int32 DataIndexInCollection = INDEX_NONE;

	TSharedPtr<FPCGProxyForGPUDataCollection> DataCollectionOnGPU;
};

/** A proxy for a data collection residing in a GPU buffer along with functionality to retrieve the data on the CPU. Holds onto GPU memory. */
struct FPCGProxyForGPUDataCollection : public TSharedFromThis<FPCGProxyForGPUDataCollection>
{
public:
	FPCGProxyForGPUDataCollection(TRefCountPtr<FRDGPooledBuffer> InBuffer, uint32 InBufferSizeBytes, TSharedPtr<const FPCGDataCollectionDesc> InDescription, const TArray<FString>& InStringTable);

	/** Populates a CPU data object representing the GPU data for the given index, performing a readback from GPU->CPU if required. */
	bool GetCPUData(FPCGContext* InContext, int32 InDataIndex, FPCGTaggedData& OutData);

	const TRefCountPtr<FRDGPooledBuffer>& GetBuffer() const { return Buffer; }

	uint32 GetBufferSizeBytes() const { return BufferSizeBytes; }

	TSharedPtr<const FPCGDataCollectionDesc> GetDescription() const { return Description; }

	const TArray<FString>& GetStringTable() const { return StringTable; }

private:
	/** Persistent GPU buffer that can be read back. Buffer will be freed when this ref count is 0. */
	TRefCountPtr<FRDGPooledBuffer> Buffer;

	uint32 BufferSizeBytes = 0;

	TSharedPtr<const FPCGDataCollectionDesc> Description = nullptr;

	/** Used to comprehend string IDs in buffer. */
	TArray<FString> StringTable;

	/** Read back data. Populated once upon first readback request. */
	TArray<const FPCGTaggedData> ReadbackData;
	TArray<TStrongObjectPtr<const UPCGData>> ReadbackDataRefs;

	TSharedPtr<FRHIGPUBufferReadback> ReadbackRequest;

	TArray<uint8> RawReadbackData;

	std::atomic<bool> bReadbackDataProcessed = false;

	bool bReadbackDataArrived = false;

	PCG::FLock ReadbackLock;
};
