// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"

#include "PCGRawBufferData.generated.h"

USTRUCT()
struct FPCGDataTypeInfoRawBuffer : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PCG_API)

};

/** A proxy for data residing on the GPU with functionality to read the data back to the CPU. */
UCLASS(MinimalAPI, ClassGroup = (Procedural), DisplayName = "Raw Buffer")
class UPCGRawBufferData : public UPCGData
{
	GENERATED_BODY()

public:
	//~Begin UPCGData interface
	PCG_ASSIGN_TYPE_INFO(FPCGDataTypeInfoRawBuffer);

	PCG_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	PCG_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	PCG_API virtual void ReleaseTransientResources(const TCHAR* InRease = nullptr) override;
	//~End UPCGData interface

	void SetData(TArray<uint32>&& InData) { Data = MoveTemp(InData); }
	int32 GetNumUint32s() const { return Data.Num(); }

	TArray<uint32>& GetData() { return Data; }
	const TArray<uint32>& GetConstData() const { return Data; }

protected:
	UPROPERTY()
	TArray<uint32> Data;
};
