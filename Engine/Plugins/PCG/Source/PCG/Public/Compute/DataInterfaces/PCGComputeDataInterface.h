// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeCommon.h"

#include "ComputeFramework/ComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "PCGComputeDataInterface.generated.h"

class FShaderParametersMetadata;
class UPCGComputeKernel;
class UPCGDataBinding;
struct FPCGDataCollectionDesc;
struct FPCGKernelOverridableParam;
struct FPCGKernelParams;

UCLASS(MinimalAPI, Abstract, ClassGroup = (Procedural))
class UPCGComputeDataInterface : public UComputeDataInterface
{
	GENERATED_BODY()

public:
	/** Register a downstream pin (and optionally compute graph element pin alias). */
	PCG_API virtual void AddDownstreamInputPin(FName InInputPinLabel, const FName* InOptionalInputPinLabelAlias = nullptr);

	/** Set the output pin label and label alias that this data interface is associated with. */
	PCG_API virtual void SetOutputPin(FName InOutputPinLabel, const FName* InOptionalOutputPinLabelAlias = nullptr);

	void SetProducerSettings(const UPCGSettings* InProducerSettings);
	PCG_API const UPCGSettings* GetProducerSettings() const;

	void SetProducerKernel(TObjectPtr<const UPCGComputeKernel> InProducerKernel) { ProducerKernel = InProducerKernel; }
	TObjectPtr<const UPCGComputeKernel> GetProducerKernel() const { return ProducerKernel; }

	/** Set whether data originates from the CPU and should be uploaded to the GPU. */
	void SetProducedByCPU(bool bInProducedByCPU) { bProducedByCPU = bInProducedByCPU; }
	bool IsProducedByCPU() const { return bProducedByCPU; }

	void SetGraphBindingIndex(int32 InGraphBindingIndex) { GraphBindingIndex = InGraphBindingIndex; }
	int32 GetGraphBindingIndex() const { return GraphBindingIndex; }

	FName GetOutputPinLabel() const { return OutputPinLabel; }
	FName GetOutputPinLabelAlias() const { return OutputPinLabelAlias; }
	const TArray<FName>& GetDownstreamInputPinLabelAliases() const { return DownstreamInputPinLabelAliases; }

	PCG_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

private:
	UPROPERTY()
	int32 GraphBindingIndex = INDEX_NONE;

	/** Label of output pin that this data interface is associated with. */
	UPROPERTY()
	FName OutputPinLabel;

	UPROPERTY()
	FName OutputPinLabelAlias;

	/** Generated PCG data will be assigned these labels. */
	UPROPERTY()
	TArray<FName> DownstreamInputPinLabelAliases;

	UPROPERTY()
	bool bProducedByCPU = false;

	UPROPERTY()
	TObjectPtr<const UPCGComputeKernel> ProducerKernel;

	UPROPERTY()
	TSoftObjectPtr<const UPCGSettings> ProducerSettings;

	mutable TObjectPtr<const UPCGSettings> ResolvedProducerSettings = nullptr;
};

UCLASS(MinimalAPI, Abstract, ClassGroup (Procedural))
class UPCGComputeDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	PCG_API virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	PCG_API virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	/** Called before PrepareForExecute to allow data providers to do any readbacks and finalize data descriptions. */
	virtual bool PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding) { return true; }

	/* Called prior to GetRenderProxy and execution. Any processing of data descriptions should be done here (after PerformPreExecuteReadbacks has been called). */
	virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) { return true; }

	/** Return true when all done. */
	virtual bool PostExecute(UPCGDataBinding* InBinding) { return true; }

	/** Release any handles to transient resources like GPU buffers. */
	virtual void ReleaseTransientResources(const TCHAR* InReason = nullptr) {}

	PCG_API const UPCGSettings* GetProducerSettings() const;
	void SetProducerSettings(const UPCGSettings* InSettings);

	TObjectPtr<const UPCGComputeKernel> GetProducerKernel() const { return ProducerKernel; }

	/** Set whether data originates from the CPU and should be uploaded to the GPU. */
	bool IsProducedByCPU() const { return bProducedByCPU; }
	int32 GetGraphBindingIndex() const { return GraphBindingIndex; }
	FName GetOutputPinLabel() const { return OutputPinLabel; }
	FName GetOutputPinLabelAlias() const { return OutputPinLabelAlias; }
	const TArray<FName>& GetDownstreamInputPinLabelAliases() const { return DownstreamInputPinLabelAliases; }
	int64 GetGenerationCounter() const { return GenerationCounter; }

protected:
#if WITH_EDITOR
	PCG_API void NotifyProducerUploadedData(UPCGDataBinding* InBinding);
#endif

private:
	int32 GraphBindingIndex = INDEX_NONE;
	FName OutputPinLabel;
	FName OutputPinLabelAlias;
	TArray<FName> DownstreamInputPinLabelAliases;
	bool bProducedByCPU = false;

	UPROPERTY(Transient)
	TObjectPtr<const UPCGComputeKernel> ProducerKernel;

	/** Settings of node that produces this data, normally the upstream node. */
	UPROPERTY(Transient)
	TObjectPtr<const UPCGSettings> ProducerSettings;

	/** Bumped each time the data provider is initialized or reset, so that async callbacks can detect if they originated from a previous usage of the data provider and no-op. */
	std::atomic<uint64> GenerationCounter = 0;
};

/** Interface for data that can be exported/downloaded from the GPU to the CPU. */
UCLASS(MinimalAPI, Abstract, ClassGroup = (Procedural))
class UPCGExportableDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	void SetRequiresExport(bool bInRequiresExport) { bRequiresExport = bInRequiresExport; }
	bool GetRequiresExport() const { return bRequiresExport; }

private:
	/** Whether this data is passed to downstream tasks outside of this compute graph. */
	UPROPERTY()
	bool bRequiresExport = false;
};

UCLASS(MinimalAPI, Abstract, ClassGroup (Procedural))
class UPCGExportableDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	PCG_API virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	PCG_API virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	//~ Begin UPCGComputeDataProvider Interface
	PCG_API virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

	/** Returns true if the generated buffer should be exported to a persistent buffer to be passed to downstream nodes. */
	bool IsExportRequired() const { return ExportMode != EPCGExportMode::NoExport; }

	EPCGExportMode GetExportMode() const { return ExportMode; }
	UPCGDataBinding* GetDataBinding() const { return Binding.Get(); }
	TSharedPtr<const FPCGDataCollectionDesc> GetPinDescription() const { return PinDataDescription; }

	DECLARE_EVENT(UPCGExportableDataProvider, FOnDataExported);
	FOnDataExported& OnDataExported_GameThread() { return OnDataExported; }

private:
	EPCGExportMode ExportMode = EPCGExportMode::NoExport;
	FOnDataExported OnDataExported;

	TWeakObjectPtr<UPCGDataBinding> Binding;
	TSharedPtr<const FPCGDataCollectionDesc> PinDataDescription = nullptr;
};

/** Byte-level layout of override params in a shader parameter buffer. */
struct FPCGKernelParamLayout
{
	TArray<uint32> ParamOffsets;
	uint32 BufferSize = 0;

	bool IsValid() const { return BufferSize > 0; }

	/** Compute the layout from a kernel's overridable params. */
	PCG_API static FPCGKernelParamLayout Build(const TArray<FPCGKernelOverridableParam>& InParams);
};

/**
 * Marshals kernel overridable params to the GPU.
 *
 * Usage:
 *  - Automatic: If a kernel sets bRequiresOverridableParams during creation, an instance of this
 *    class is automatically created during graph compilation.
 *
 *  - Extra (non-overridable) params: If a kernel also needs to marshal additional data to the GPU
 *    (e.g. resolved attribute IDs, RDG buffers), create a separate UPCGComputeDataInterface
 *    subclass for those and add it in CreateAdditionalInputDataInterfaces. The auto-created
 *    kernel params DI handles override params independently.
 *
 *  - Full override: If a kernel needs complete control over how override params are marshalled,
 *    create a UPCGKernelParamsDataInterface subclass and add it in CreateAdditionalInputDataInterfaces.
 *    This suppresses auto-creation.
 */
UCLASS(MinimalAPI, ClassGroup = (Procedural))
class UPCGKernelParamsDataInterface : public UPCGComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGKernelParamsDataInterface"); }
	PCG_API UComputeDataProvider* CreateDataProvider() const override;
	PCG_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	PCG_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	PCG_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	//~ End UComputeDataInterface Interface

	/** Returns the cached param layout. Built once on first access. */
	PCG_API const FPCGKernelParamLayout& GetKernelParamLayout() const;

protected:
	/** Add override param members to a shader parameter builder. Name strings are stored in InOutAllocations to outlive the metadata. */
	PCG_API void AddKernelShaderParams(FShaderParametersMetadataBuilder& InOutBuilder, TArray<FShaderParametersMetadata*>& InOutNestedStructs, FShaderParametersMetadataAllocations& InOutAllocations) const;

private:
	/** Byte-level layout of override params, cached once and reused by the data provider. */
	mutable FPCGKernelParamLayout CachedKernelParamLayout;
};

/**
 * Provides kernel overridable param values to the GPU at dispatch time.
 *
 * This class automatically created alongside UPCGKernelParamsDataInterface.
 * It reads override param values from FPCGKernelParams and writes them into a byte
 * buffer for the shader. Subclass only if you need custom marshalling logic.
 */
UCLASS(MinimalAPI, ClassGroup (Procedural))
class UPCGKernelParamsDataProvider : public UPCGComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	PCG_API virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	PCG_API virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	PCG_API virtual void Reset() override;
	//~ End UComputeDataProvider Interface

	//~ Begin UPCGComputeDataProvider Interface
	PCG_API virtual bool PerformPreExecuteReadbacks_GameThread(UPCGDataBinding* InBinding) override;
	//~ End UPCGComputeDataProvider Interface

protected:
	/** Write override param values into a byte buffer at offsets from the given layout. Writes one entry per CachedOverridableParam. */
	PCG_API void WriteKernelParamValues(uint8* OutBuffer, const FPCGKernelParamLayout& InLayout) const;

	/** Cached kernel param values (owned by FPCGKernelParamsCache, not this provider). */
	const FPCGKernelParams* CachedKernelParams = nullptr;

	/** Byte-level layout of override params. */
	FPCGKernelParamLayout CachedKernelParamLayout;
};

/** Generic proxy that writes pre-built kernel param bytes into the shader parameter buffer. */
class FPCGKernelParamsDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FPCGKernelParamsDataProviderProxy(TArray<uint8>&& InParameterData, uint32 InParameterStructSize)
		: ParameterData(MoveTemp(InParameterData))
		, ParameterStructSize(InParameterStructSize)
	{}

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	TArray<uint8> ParameterData;
	uint32 ParameterStructSize = 0;
};
