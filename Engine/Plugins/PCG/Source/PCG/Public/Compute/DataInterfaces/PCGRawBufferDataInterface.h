// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/DataInterfaces/PCGComputeDataInterface.h"

#include "RHIResources.h"

#include "PCGRawBufferDataInterface.generated.h"

class FPCGRawBufferDataInterfaceParameters;
class UPCGRawBufferData;

/** A data interface for a simple array of uint values. No data format header or attributes, just raw array access. */
UCLASS(ClassGroup = (Procedural))
class UPCGRawBufferDataInterface : public UPCGExportableDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PCGRawBuffer"); }
	PCG_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	PCG_API void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	PCG_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	PCG_API TCHAR const* GetShaderVirtualPath() const override;
	PCG_API void GetShaderHash(FString& InOutKey) const override;
	PCG_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual bool GetRequiresReadback() const override { return true; }
	PCG_API UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

	bool GetRequiresZeroInitialization() const { return bRequiresZeroInitialization; }
	void SetRequiresZeroInitialization(bool bInZeroInit) { bRequiresZeroInitialization = bInZeroInit; }

protected:
	/** Whether to perform full 0-initialization of the buffer. */
	UPROPERTY()
	bool bRequiresZeroInitialization = false;
};

UCLASS()
class UPCGRawBufferDataProvider : public UPCGExportableDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	PCG_API virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	PCG_API virtual bool PrepareForExecute_GameThread(UPCGDataBinding* InBinding) override;
	PCG_API virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	PCG_API virtual void Reset() override;
	//~ End UComputeDataProvider Interface

protected:
	int32 SizeBytes = INDEX_NONE;

	bool bZeroInitialize = false;

	UPROPERTY()
	TObjectPtr<const UPCGRawBufferData> DataToUpload;

	struct FReadbackState
	{
		TArray<uint32> Data;
	};
	TSharedPtr<FReadbackState, ESPMode::ThreadSafe> ReadbackState;
};

class FPCGRawBufferDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FParams
	{
		int32 SizeBytes = 0;
		bool bZeroInitialize = false;
		const UPCGRawBufferData* DataToUpload = nullptr;
		EPCGExportMode ExportMode = EPCGExportMode::NoExport;
		FName OutputPinLabel;
		FName OutputPinLabelAlias;
		TWeakObjectPtr<UPCGRawBufferDataProvider> DataProviderWeakPtr;
		FReadbackCallback AsyncReadbackCallback_RenderThread;
	};

	PCG_API FPCGRawBufferDataProviderProxy(const FParams& InParams);

	//~ Begin FComputeDataProviderRenderProxy Interface
	PCG_API bool IsValid(FValidationData const& InValidationData) const override;
	PCG_API void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData);
	PCG_API void GatherDispatchData(FDispatchData const& InDispatchData) override;
	PCG_API void GetReadbackData(TArray<FReadbackData>& OutReadbackData) const override;
	//~ End FComputeDataProviderRenderProxy Interface

protected:
	using FParameters = FPCGRawBufferDataInterfaceParameters;

	int32 SizeBytes = INDEX_NONE;

	bool bZeroInitialize = false;

	EPCGExportMode ExportMode = EPCGExportMode::NoExport;

	FName OutputPinLabel;
	FName OutputPinLabelAlias;

	FRDGBufferRef Data = nullptr;
	FRDGBufferSRVRef DataSRV = nullptr;
	FRDGBufferUAVRef DataUAV = nullptr;

	TArray<uint32> DataToUpload;

	TWeakObjectPtr<UPCGRawBufferDataProvider> DataProviderWeakPtr;

	/** Called from render thread when readback from GPU is complete. */
	FReadbackCallback AsyncReadbackCallback_RenderThread;
};
