// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Data/Registry/PCGDataTypeIdentifier.h"

#include "Engine/TextureRenderTarget2D.h"

#include "PCGComputeCommon.generated.h"

class FRHIShaderResourceView;
class UObject;
class UPCGComputeDataInterface;
class UPCGComputeKernel;
class UPCGData;
class UPCGDataBinding;
class UPCGNode;
class UPCGSettings;
enum EPixelFormat : uint8;
struct FPCGContext;
struct FPCGDataCollectionDesc;
struct FPCGGPUCompilationContext;
struct FPCGPinProperties;

#define PCG_KERNEL_LOGGING_ENABLED (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING)

#if PCG_KERNEL_LOGGING_ENABLED
#define PCG_KERNEL_VALIDATION_WARN(Context, Settings, ValidationMessage) PCGComputeHelpers::LogKernelWarning(Context, Settings, ValidationMessage);
#define PCG_KERNEL_VALIDATION_ERR(Context, Settings, ValidationMessage) PCGComputeHelpers::LogKernelError(Context, Settings, ValidationMessage);
#else
#define PCG_KERNEL_VALIDATION_WARN(Context, Settings, ValidationMessage) // Log removed
#define PCG_KERNEL_VALIDATION_ERR(Context, Settings, ValidationMessage) // Log removed
#endif

/** Modes for exporting the buffer from transient to persistent for downstream consumption. */
UENUM(meta = (Bitflags))
enum class EPCGExportMode : uint8
{
	/** Buffer is transient and freed after usage. */
	NoExport = 0,
	/** Buffer will be exported and a proxy will be output from the compute graph and passed to downstream nodes. */
	ComputeGraphOutput = 1 << 0,
	/** Producer node is being inspected, read back data and store in inspection data. */
	Inspection = 1 << 1,
	/** Producer node is being debugged, read back data and execute debug visualization. */
	DebugVisualization = 1 << 2,
};
ENUM_CLASS_FLAGS(EPCGExportMode);

/** Dimensionality of a data description element count. */
UENUM()
enum class EPCGElementDimension : uint8
{
	One,
	Two,
	Three,
	Four
};

/** Mirror of ETextureRenderTargetFormat that exposes only formats supported in PCG data descriptions. */
UENUM()
enum class EPCGRenderTargetFormat : uint8
{
	/** R channel, 8 bit per channel fixed point, range [0, 1]. */
	R8 = RTF_R8,

	/** RG channels, 8 bit per channel fixed point, range [0, 1]. */
	RG8 = ETextureRenderTargetFormat::RTF_RG8,

	/** RGBA channels, 8 bit per channel fixed point, range [0, 1]. */
	RGBA8 = ETextureRenderTargetFormat::RTF_RGBA8,

	/** R channel, 16 bit per channel floating point, range [-65504, 65504]. */
	R16f = ETextureRenderTargetFormat::RTF_R16f,

	/** RG channels, 16 bit per channel floating point, range [-65504, 65504]. */
	RG16f = ETextureRenderTargetFormat::RTF_RG16f,

	/** RGBA channels, 16 bit per channel floating point, range [-65504, 65504]. */
	RGBA16f = ETextureRenderTargetFormat::RTF_RGBA16f,

	/** R channel, 32 bit per channel floating point, range [-3.402823 x 10^38, 3.402823 x 10^38]. */
	R32f = ETextureRenderTargetFormat::RTF_R32f,

	/** RG channels, 32 bit per channel floating point, range [-3.402823 x 10^38, 3.402823 x 10^38]. */
	RG32f = ETextureRenderTargetFormat::RTF_RG32f,

	/** RGBA channels, 32 bit per channel floating point, range [-3.402823 x 10^38, 3.402823 x 10^38]. */
	RGBA32f = ETextureRenderTargetFormat::RTF_RGBA32f,
};

namespace PCGComputeConstants
{
	constexpr int32 MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER = 256;
	constexpr uint32 THREAD_GROUP_SIZE = 64;
}

namespace PCGComputeHelpers
{
	/** Gets the element count for a given data. E.g. number of points in a PointData, number of metadata entries in a ParamData, etc. */
	FIntVector4 GetElementCount(const UPCGData* InData);

	/** Gets the element dimension for a given data. E.g. OneD for PointData, TwoD for TextureData, etc. */
	EPCGElementDimension GetElementDimension(const UPCGData* InData);
	EPCGElementDimension GetElementDimension(const FPCGDataTypeIdentifier& InDataType);

	/** Maximum number of data that a GPU node pin of the given type can receive/emit. */
	int32 GetMaxOutputDataCount(const FPCGDataTypeIdentifier& InDataType);

	/** PCG data types supported in GPU node inputs. */
	const TArray<FPCGDataTypeIdentifier>& GetAllowedInputTypesList();

	/** PCG data types supported in GPU node outputs. */
	const TArray<FPCGDataTypeIdentifier>& GetAllowedOutputTypesList();

	/** True if 'Type' is valid on a GPU input pin. */
	bool IsTypeAllowedAsInput(const FPCGDataTypeIdentifier& Type);

	/** True if 'Type' is valid on a GPU output pin. */
	bool IsTypeAllowedAsOutput(const FPCGDataTypeIdentifier& Type);

	/** True if 'Type' is valid in a GPU data collection. Some types are only supported as DataInterfaces, and cannot be uploaded in data collections. */
	bool IsTypeAllowedInDataCollection(const FPCGDataTypeIdentifier& Type);

	/** Whether metadata attributes should be read from the given data and registered for use in GPU graphs. */
	bool ShouldImportAttributesFromData(const UPCGData* InData);

#if PCG_KERNEL_LOGGING_ENABLED
	/** Logs a warning on a GPU node in the graph and console. */
	PCG_API void LogKernelWarning(const FPCGContext* Context, const UPCGSettings* Settings, const FText& InText);

	/** Logs an error on a GPU node in the graph and console. */
	PCG_API void LogKernelError(const FPCGContext* Context, const UPCGSettings* Settings, const FText& InText);
#endif

	/** Returns true if the given buffer size is dangerously large. Optionally emits error log. */
	bool IsBufferSizeTooLarge(uint64 InBufferSizeBytes, bool bInLogError = true);

	/** True if all data descriptions share the same type and that type is allowed in a GPU data collection. */
	PCG_API bool CanPackIntoDataCollection(const FPCGDataCollectionDesc& InDataCollectionDesc);

	/** Computes total GPU memory size in bytes for a data collection descriptor, covering all output types:
	* data-collection types (Point, Param) via packing helpers, and non-collection types (textures, raw buffers) via element count and format. */
	PCG_API uint64 ComputeSizeBytes(TSharedPtr<const FPCGDataCollectionDesc> InDataCollectionDesc);

	int32 GetAttributeIdFromMetadataAttributeIndex(int32 InAttributeIndex);
	int32 GetMetadataAttributeIndexFromAttributeId(int32 InAttributeId);

	/** Produces the data label prefixed with PCGComputeConstants::DataLabelTagPrefix. */
	PCG_API FString GetPrefixedDataLabel(const FString& InLabel);
	PCG_API void GetPrefixedDataLabel(const FString& InLabel, FString& OutPrefixedLabel);

	/** Produces the data interface name of a data label resolver. */
	FString GetDataLabelResolverName(FName InPinLabel);

	PCG_API EPixelFormat GetPixelFormatFromPCGRenderTargetFormat(EPCGRenderTargetFormat InRenderTargetFormat);
	PCG_API EPCGRenderTargetFormat GetPCGRenderTargetFormatFromPixelFormat(EPixelFormat InPixelFormat);

#if WITH_EDITOR
	/**
	 * Processes a .ush template by expanding two directive forms:
	 *  - {{BeginDuplicateForSRVUAV}} / {{EndDuplicateForSRVUAV}}: emit each region twice, {NameSuffix}=PCG_SRV {BufferSuffix}=SRV and {NameSuffix}=PCG_UAV {BufferSuffix}=UAV. Use for readers that work against either view.
	 *  - {{BeginUAVOnly}} / {{EndUAVOnly}}: emit region with {NameSuffix}=PCG_UAV / {BufferSuffix}=UAV. Use for setters, atomics, and any UAV-only helper whose body calls into a duplicated reader or references the UAV buffer.
	 *
	 * Names produced for the duplicated form should be registered C++-side via GetSRVFunction / GetUAVFunction so the suffixes stay in sync.
	 *
	 * If the markers are missing or malformed, ensures and returns the template text unchanged.
	 */
	PCG_API FString ExpandShaderTemplateForSRVUAV(const FString& InTemplate);

	/**
	 * Builds the SRV/UAV-side function name produced by ExpandShaderTemplateForSRVUAV. Use these when populating FShaderFunctionDefinition entries that target a duplicated template region so the C++ and shader-side names stay in sync.
	 *   GetSRVFunction(TEXT("Load")) -> "Load_PCG_SRV"
	 *   GetUAVFunction(TEXT("Load")) -> "Load_PCG_UAV"
	 */
	PCG_API FString GetSRVFunction(const TCHAR* InBaseName);
	PCG_API FString GetUAVFunction(const TCHAR* InBaseName);

	/** Inverse of GetSRVFunction / GetUAVFunction: strips a trailing _PCG_SRV or _PCG_UAV suffix in place. */
	PCG_API void StripSRVUAVSuffix(FString& InOutName);

	void ConvertObjectPathToShaderFilePath(FString& InOutPath);

	struct FCreateDataInterfaceParams
	{
		FPCGGPUCompilationContext* Context = nullptr;
		const FPCGPinProperties* PinProperties = nullptr;
		const UPCGComputeKernel* ProducerKernel = nullptr;
		UObject* ObjectOuter = nullptr;
		bool bProducedByCPU = false;
		bool bRequiresExport = false;
		const UPCGNode* NodeForDebug = nullptr;
	};

	UPCGComputeDataInterface* CreateOutputPinDataInterface(const FCreateDataInterfaceParams& InParams);

	PCG_API void NotifyGPUToCPUReadback(UPCGDataBinding* InBinding, const UPCGComputeKernel* InKernel, const UPCGSettings* InSettings);
	PCG_API void NotifyCPUToGPUUpload(UPCGDataBinding* InBinding, const UPCGComputeKernel* InKernel, const UPCGSettings* InSettings);

	/** Sanitizes a pin label for use as a valid HLSL identifier. Replaces invalid characters with underscores and trims leading non-alpha characters. */
	PCG_API FString SanitizePinLabelForHLSL(FName InLabel);

	/** Returns true if the pin label is already a valid HLSL identifier (i.e. sanitization would not change it). */
	PCG_API bool IsValidHLSLPinLabel(FName InLabel);
#endif
}

namespace PCGComputeDummies
{
	FRHIShaderResourceView* GetDummyFloatBuffer();
	FRHIShaderResourceView* GetDummyFloat2Buffer();
	FRHIShaderResourceView* GetDummyFloat4Buffer();
}

/** A by-label reference to a pin, used for wiring kernels within a node. */
struct FPCGPinReference
{
	/** Reference a pin by label only, used for referencing node pins. */
	explicit FPCGPinReference(FName InLabel)
		: Kernel(nullptr)
		, Label(InLabel)
	{
	}

	/** Reference a pin by kernel and label. */
	explicit FPCGPinReference(UPCGComputeKernel* InKernel, FName InLabel)
		: Kernel(InKernel)
		, Label(InLabel)
	{
	}

	bool operator==(const FPCGPinReference& Other) const
	{
		return Label == Other.Label
			&& Kernel == Other.Kernel;
	}

	/** Associated kernel. If null then compiler will look for pin on owning node. */
	UPCGComputeKernel* Kernel = nullptr;

	/** Pin label. */
	FName Label;
};

PCG_API uint32 GetTypeHash(const FPCGPinReference& In);

/** A connection for wiring kernels within a node. */
struct FPCGKernelEdge
{
	FPCGKernelEdge(const FPCGPinReference& InUpstreamPin, const FPCGPinReference& InDownstreamPin)
		: UpstreamPin(InUpstreamPin)
		, DownstreamPin(InDownstreamPin)
	{
	}

	bool IsConnectedToNodeInput() const { return UpstreamPin.Kernel == nullptr; }
	bool IsConnectedToNodeOutput() const { return DownstreamPin.Kernel == nullptr; }

	UPCGComputeKernel* GetUpstreamKernel() const { return UpstreamPin.Kernel; }
	UPCGComputeKernel* GetDownstreamKernel() const { return DownstreamPin.Kernel; }

	FPCGPinReference UpstreamPin;
	FPCGPinReference DownstreamPin;
};

/** An input or output pin of a kernel. Compute graph does not internally have 'pins' so this is useful for mapping between kernel data and PCG pins. */
USTRUCT()
struct FPCGKernelPin
{
	GENERATED_BODY()

public:
	FPCGKernelPin() = default;

	explicit FPCGKernelPin(int32 InKernelIndex, FName InPinLabel, bool bInIsInput)
		: KernelIndex(InKernelIndex)
		, PinLabel(InPinLabel)
		, bIsInput(bInIsInput)
	{
	}

	bool operator==(const FPCGKernelPin& Other) const = default;

public:
	UPROPERTY()
	int32 KernelIndex = INDEX_NONE;

	UPROPERTY()
	FName PinLabel = NAME_None;

	UPROPERTY()
	bool bIsInput = false;

	PCG_API friend uint32 GetTypeHash(const FPCGKernelPin& In);
};

/** Helper struct for serializing data labels. */
USTRUCT()
struct FPCGDataLabels
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FString> Labels;
};

/** Helper struct for serializing map of pin name to data labels. */
USTRUCT()
struct FPCGPinDataLabels
{
	GENERATED_BODY()

	UPROPERTY()
	TMap</*PinLabel*/FName, FPCGDataLabels> PinToDataLabels;
};
