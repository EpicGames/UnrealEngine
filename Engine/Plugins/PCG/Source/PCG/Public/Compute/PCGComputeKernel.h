// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Compute/PCGComputeCommon.h"

#include "ComputeFramework/ComputeKernel.h"
#include "UObject/ObjectKey.h"

#include "PCGComputeKernel.generated.h"

class UPCGDataBinding;
class UPCGNode;
class UPCGSettings;
struct FPCGContext;
struct FPCGDataCollectionDesc;
struct FPCGGPUCompilationContext;
struct FPCGKernelAttributeKey;
struct FPCGPinProperties;
struct FPCGPinPropertiesGPU;

class UComputeDataInterface;

UENUM()
enum class EPCGComputeKernelFlags : uint32
{
	None = 0,
	/** Errors were detected with node setup at compute graph compilation time. */
	HasStaticValidationErrors = 1 << 0,
	/** Dump data descriptions to log. */
	LogDataDescriptions = 1 << 1,
	/** Dispatch compute graph containing this kernel every frame to allow profiling. Has no effect unless PCG_GPU_KERNEL_PROFILING defined. */
	RepeatDispatchForProfiling = 1 << 2,
};
ENUM_CLASS_FLAGS(EPCGComputeKernelFlags);

UENUM()
enum class EPCGKernelLogVerbosity
{
	Verbose,
	Warning,
	Error
};

USTRUCT()
struct FPCGKernelLogEntry
{
	GENERATED_BODY()

	FPCGKernelLogEntry() = default;

	explicit FPCGKernelLogEntry(const FText& InMessage, EPCGKernelLogVerbosity InVerbosity)
		: Message(InMessage)
		, Verbosity(InVerbosity)
	{}

	UPROPERTY()
	FText Message;

	UPROPERTY()
	EPCGKernelLogVerbosity Verbosity = EPCGKernelLogVerbosity::Verbose;
};

#if WITH_EDITOR
struct FPCGComputeKernelParams
{
	const UPCGSettings* Settings = nullptr;
	const UPCGNode* Node = nullptr;
	bool bLogDescriptions = false;
	/** When true, discovers GPU-overridable params and exposes them as wireable pins via an auto-created UPCGKernelParamsDataInterface. */
	bool bRequiresOverridableParams = true;
};
#endif

/** Represents a parameter that can be overridden on a kernel. */
USTRUCT()
struct FPCGKernelOverridableParam
{
	GENERATED_BODY()

	UPROPERTY()
	FName Label = NAME_None;

	UPROPERTY()
	TArray<FName> PropertiesNames;

	UPROPERTY()
	bool bRequiresGPUReadback = false;

	/** True if this param is actually overidden by a pin. */
	UPROPERTY()
	bool bIsPropertyOverriddenByPin = false;

	/** The underlying metadata type, used for type-aware codegen and generic shader parameter building. */
	UPROPERTY()
	EPCGMetadataTypes UnderlyingType = EPCGMetadataTypes::Integer32;
};

UCLASS(MinimalAPI, Abstract)
class UPCGComputeKernel : public UComputeKernel
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	/** Initialize kernel. Editor only as shader compilation only available in editor. */
	PCG_API void Initialize(const FPCGComputeKernelParams& InParams);
#endif

	/** Get/set index of kernel in kernels array on owning compute graph. */
	int32 GetKernelIndex() const { return KernelIndex; }
	void SetKernelIndex(int32 InKernelIndex) { KernelIndex = InKernelIndex; }

	/** Gets settings for node associated with this kernel, if any. */
	PCG_API virtual const UPCGSettings* GetSettings() const;

	/** Gets the PCG node associated with this kernel, if any. Stored during compilation to avoid relying on Settings->GetOuter() which fails for settings instances. */
	PCG_API const UPCGNode* GetNode() const;

	/** Performs settings validation and returns true if this node is suitable for execution. */
	PCG_API virtual bool AreKernelSettingsValid(FPCGContext* InContext) const;

	/** Performs data validation and returns true if this node is suitable for deployment to the GPU. */
	PCG_API virtual bool IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const;

#if WITH_EDITOR
	/** Produces the node specific portion of kernel shader source text, including the main entry point. */
	PCG_API virtual FString GetCookedSource(FPCGGPUCompilationContext& InOutContext) const;

	/** Get the virtual file path of the shader source file for this kernel. */
	virtual const TCHAR* GetSourceFilePath() const { return nullptr; }

	/** Get the name of the main kernel function in the source. This name is also displayed in GPU profile scopes. */
	virtual FString GetEntryPoint() const PURE_VIRTUAL(UPCGComputeKernel::GetEntryPoint, return TEXT("Main"););

	/**
	 * Gathers additional compute sources referenced by this kernel.
	 * This is preferred to directly using includes, since it allows us to detect hash diffs and recompile when additional sources are changed externally.
	 * Note, you should not create new compute sources here, as their object names are not deterministic and will result in a new hash every time.
	 */
	PCG_API virtual void GatherAdditionalSources(TArray<TObjectPtr<class UComputeSource>>& OutAdditionalSources) const;

	/** Create additional input data interfaces to marshal any required input data. */
	virtual void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const {}

	/** Create a UPCGKernelParamsDataInterface if this kernel has overridable params and one was not already provided by a subclass in CreateAdditionalInputDataInterfaces. */
	void CreateKernelParamsDataInterface(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& InOutDataInterfaces) const;

	/** Create additional output data interfaces to marshal any required output data. */
	virtual void CreateAdditionalOutputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const {}

	/** Whether compute graphs should be split at this input pin. Useful for overridable parameters. */
	PCG_API virtual bool SplitGraphAtInput(FName InPinLabel) const;

	/** Whether compute graphs should be split at the output of this node. Useful for reading back runtime statistics or diagnostic info before continuing. */
	virtual bool SplitGraphAtOutput() const { return false; }

	/** Mark a pin as being internal to the kernel graph. This means it is not on the CPU/GPU boundary. */
	void AddInternalPin(FName InPinLabel) { InternalPinLabels.Add(InPinLabel); }

	/** Whether the pin matching the given label should be eligible for inspect/debug. */
	bool IsPinInternal(FName InPinLabel) const { return InternalPinLabels.Contains(InPinLabel); }

	/** Create a data interface for the given output pin. */
	PCG_API virtual UPCGComputeDataInterface* CreateOutputPinDataInterface(const PCGComputeHelpers::FCreateDataInterfaceParams& InParams) const;

	/** Whether the data on the input pin is used on the GPU (and thus requires a data interface) or not. */
	virtual bool UploadInputPinDataToGPU(FName InPinLabel) const { return true; }
#endif

	/** Compute a description of data that will be output from pin InOutputPinLabel. */
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const PURE_VIRTUAL(UPCGComputeKernel::ComputeOutputBindingDataDesc, return nullptr;);

	/** Compute how many threads should be dispatched to execute this node on the GPU. */
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const PURE_VIRTUAL(UPCGComputeKernel::ComputeThreadCount, return 0;);

	/** Get any data labels that are used statically by this node. */
	virtual void GetDataLabels(FName InPinLabel, TArray<FString>& OutDataLabels) const {}

	/** Initialize all buffer data to 0. This is not free so should only be used when kernel execution requires 0-initialized data, such as for counters. */
	virtual bool DoesOutputPinRequireZeroInitialization(FName InOutputPinLabel) const { return false; }

	/** Add any strings emitted by this node that are known statically at compile time. */
	virtual void AddStaticCreatedStrings(TArray<FString>& InOutStringTable) const {}

	/** Get all the attributes read or written by this node for which we know the name and type statically. In cases that only the
	* attribute name used by the node is known (example: user specifies SM Spawner instance data attributes by name only, not type), the attribute
	* is omitted from this list and must be resolved at execution time in the data provider.
	*/
	virtual void GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const {}

	/** Get multiplier to be applied to the dispatched thread count. */
	virtual uint32 GetThreadCountMultiplier() const { return 1u; }

	/** Multiplier to be applied to the element count of each output data. */
	virtual uint32 GetElementCountMultiplier(FName InOutputPinLabel) const { return 1u; }

	/** Helper to compute data descriptions when data is entirely determined by pin properties. */
	void ComputeDataDescFromPinProperties(const FPCGPinPropertiesGPU& OutputPinProps, const TArrayView<const FPCGPinProperties>& InInputPinProps, UPCGDataBinding* InBinding, TSharedPtr<FPCGDataCollectionDesc> OutPinDesc) const;

	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const {}
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const {}

	/** Return the concatenation of GetInputPins() and GetOverridableParamPins(). */
	void GetInputPinsAndOverridablePins(TArray<FPCGPinProperties>& OutPins) const;

	const TArray<FPCGKernelOverridableParam>& GetCachedOverridableParams() const { return CachedOverridableParams; }

	bool GetLogDataDescriptions() const { return !!(PCGKernelFlags & EPCGComputeKernelFlags::LogDataDescriptions); }

	bool HasStaticValidationErrors() const { return !!(PCGKernelFlags & EPCGComputeKernelFlags::HasStaticValidationErrors); }

	/** True if this compute kernel must run after the TLAS is built. This will force the entire compute graph to execute in this group. */
	PCG_API virtual bool RequiresPostTLASBuildExecutionGroup() const;

#ifdef PCG_GPU_KERNEL_PROFILING
	void EnableRepeatDispatch() { PCGKernelFlags |= EPCGComputeKernelFlags::RepeatDispatchForProfiling; }

	/** Whether to repeatedly dispatch every frame to help profiling. Either configured on the GPU node or overridden here to avoid data cooks. */
	virtual bool ShouldRepeatDispatch() const { return !!(PCGKernelFlags & EPCGComputeKernelFlags::RepeatDispatchForProfiling); }
#endif

	PCG_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

protected:
#if WITH_EDITOR
	/** Implement to do any additional initialization specific to your kernel. */
	virtual void InitializeInternal() {}

	/** Performs validation on compile time information (e.g. Node, Settings, etc.). Caches validation errors/warnings. */
	PCG_API virtual bool PerformStaticValidation();

	template <typename... ArgsType>
	void AddStaticLogEntry(ArgsType&&... Args)
	{
		StaticLogEntries.Emplace(Forward<ArgsType>(Args)...);
	}

	/** Do any validation of the PCG node. Returns true if node is valid and GPU execution can proceed. */
	bool ValidatePCGNode(TArray<FPCGKernelLogEntry>& OutLogEntries) const;

	/** Validate input edges, for example checking connected pin types are compatible. */
	bool AreInputEdgesValid(TArray<FPCGKernelLogEntry>& OutLogEntries) const;
#endif

	/** Collects all param pins which are overridable on the GPU. */
	void GetOverridableParamPins(TArray<FPCGPinProperties>& OutPins) const;

private:
	UPROPERTY()
	EPCGComputeKernelFlags PCGKernelFlags = EPCGComputeKernelFlags::None;

	/** Index into kernels array on owning compute graph. */
	UPROPERTY()
	int32 KernelIndex = INDEX_NONE;

	UPROPERTY()
	TSoftObjectPtr<const UPCGSettings> SettingsPath;

	UPROPERTY()
	TSoftObjectPtr<const UPCGNode> NodePath;

	UPROPERTY()
	bool bInitialized = false;

	// @todo_pcg: These are a bit of a hack to avoid supporting static/compilation logging in PCGNodeVisualLogs.h
	/** Log entries created and cached on initialization. Are logged during AreKernelSettingsValid(), unless PCG_KERNEL_LOGGING_ENABLED is false. */
	UPROPERTY()
	TArray<FPCGKernelLogEntry> StaticLogEntries;

	/** Overridable params supported by this kernel, cached during compilation. */
	UPROPERTY()
	TArray<FPCGKernelOverridableParam> CachedOverridableParams;

#if WITH_EDITOR
	TSet<FName> InternalPinLabels;
#endif

	mutable TObjectPtr<const UPCGSettings> ResolvedSettings = nullptr;
	mutable TObjectPtr<const UPCGNode> ResolvedNode = nullptr;

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "Override IsKernelDataValid(const UPCGDataBinding*, FPCGContext*) instead.")
	PCG_API virtual bool IsKernelDataValid(FPCGContext* InContext) const;
};

namespace PCGComputeKernel
{
	PCG_API TConstArrayView<FSoftObjectPath> GetDefaultAdditionalSourcePaths();
}
