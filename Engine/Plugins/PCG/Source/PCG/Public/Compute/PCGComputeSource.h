// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/IPCGCodeEditorTextProvider.h"

#include "ComputeFramework/ComputeSource.h"

#include "PCGComputeSource.generated.h"

#if WITH_EDITOR
class UPCGComputeSource;
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPCGComputeSourceModified, const UPCGComputeSource*);
#endif // WITH_EDITOR

#define UE_API PCG_API

UCLASS(MinimalAPI)
class UPCGComputeSource : public UComputeSource, public IPCGCodeEditorTextProvider
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	//~Begin UObject interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
	//~Begin UObject interface
#endif

	//~ Begin UComputeSource Interface.
	UE_API FString GetSource() const override;
	UE_API FString GetVirtualPath() const override;
	//~ End UComputeSource Interface.

#if WITH_EDITOR
	//~Begin IPCGCodeEditorTextProvider interface
	FString GetDeclarationsText() const override { return {}; }
	FString GetSourceText() const override { return GetSource(); }
	FString GetFunctionsText() const override { return {}; }
	void SetSourceText(const FString& InText) override;
	void SetFunctionsText(const FString& InText) override {}
	bool IsReadOnly() const override { return false; }
	//~End IPCGCodeEditorTextProvider interface
	
	UE_API void SetSource(const FString& InSource);

	static FOnPCGComputeSourceModified OnModifiedDelegate;
#endif

#if WITH_EDITORONLY_DATA
protected:
	UPROPERTY(EditAnywhere, Category = "Source", meta = (DisplayAfter = "AdditionalSources"))
	FString Source;
#endif

	// Deprecated section
#if WITH_EDITOR
public:
	UE_DEPRECATED(5.8, "Use GetSourceText() instead")
	FString GetShaderText() const { return GetSource(); }
	UE_DEPRECATED(5.8, "Use GetFunctionsText() instead")
	FString GetShaderFunctionsText() const { return {}; }
	UE_DEPRECATED(5.8, "Use SetFunctionsText() instead")
	void SetShaderFunctionsText(const FString& InText) {}
	UE_DEPRECATED(5.8, "Use IsReadOnly() instead")
	bool IsShaderTextReadOnly() const { return false; }
#endif
};

#undef UE_API
