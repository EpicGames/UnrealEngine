// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "PCGSettings.h"
#include "Compute/PCGDataDescription.h"
#include "Compute/PCGKernelParamsCache.h"

#include "PCGDataBinding.generated.h"

class IPCGRuntimePrimitiveFactory;
class UComputeKernel;
class UPCGComputeGraph;
class UPCGComputeKernel;
struct FPCGContextHandle;
struct FPCGKernelPin;
struct FPCGProxyForGPUDataCollection;

class FRDGPooledBuffer;

USTRUCT()
struct FPCGSpawnerPrimitives
{
	GENERATED_BODY()

	TSharedPtr<IPCGRuntimePrimitiveFactory> PrimitiveFactory;

	TArray<FBox> PrimitiveMeshBounds;

	/** Cumulative distribution function values (one per primitive) to enable choosing a primitive based on a random draw value. */
	TArray<float> SelectionCDF;

	// Same for all primitives
	uint32 NumCustomFloats = 0;

	// Same for all primitives
	TArray<FUintVector4> AttributeIdOffsetStrides;

	int32 SelectorAttributeId = INDEX_NONE;
};

USTRUCT()
struct FPCGDataToDebug
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPCGDataToDebug() = default;
	FPCGDataToDebug(const FPCGDataToDebug& Other) = default;
	FPCGDataToDebug& operator=(const FPCGDataToDebug& Other) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	TObjectPtr<UPCGData> Data;

	UPROPERTY()
	TObjectPtr<UPCGData> DataPendingInit;
	
	UPROPERTY()
	TWeakObjectPtr<const UPCGSettings> ProducerSettings;
	
	UPROPERTY()
	FName PinLabel;
	
	UPROPERTY()
	FName PinLabelAlias;

	UPROPERTY()
	TSet<FString> Tags;

	UE_DEPRECATED(5.8, "Use Tags field instead")
	UPROPERTY()
	TSet<FString> AdditionalTags;
};

UCLASS(MinimalAPI, Transient, Category = PCG)
class UPCGDataBinding : public UObject
{
	GENERATED_BODY()

public:
	/** Pre-initialization, set up pointers and store input data. */
	void Initialize(const UPCGComputeGraph* InComputeGraph, FPCGContext* InContext);

	/** Initialization look up tables from input data. */
	void InitializeTables(FPCGContext* InContext);

	/** Clear state and release any handles to resources such as GPU buffers. */
	void ReleaseTransientResources();

	int32 GetAttributeId(const FPCGKernelAttributeKey& InAttribute) const { ensure(bTablesInitialized); return AttributeTable.GetAttributeId(InAttribute); }
	int32 GetAttributeId(FPCGAttributeIdentifier InIdentifier, EPCGKernelAttributeType InType) const { ensure(bTablesInitialized); return AttributeTable.GetAttributeId(InIdentifier, InType); }

	int32 GetAttributeTableSize() const { return AttributeTable.Num(); }

	/** Get the set of all unique strings entering the compute graph. */
	const TArray<FString>& GetStringTable() const { ensure(bTablesInitialized); return StringTable; }

	/** Computes a description of data for every pin in the compute graph and caches it. */
	void PrimeDataDescriptionCache();

	bool IsDataDescriptionCachePrimed() const { return bIsDataDescriptionCachePrimed; }

	/** Flushes computed data descriptions. Subsequent attempts to retrieve from the cache will trigger errors. */
	void FlushDataDescriptionCache();

	/** Computes a description of data for a single pin in the compute graph and caches it. */
	PCG_API TSharedPtr<const FPCGDataCollectionDesc> ComputeKernelPinDataDesc(const FPCGKernelPin& InKernelPin);
	TSharedPtr<const FPCGDataCollectionDesc> ComputeKernelBindingDataDesc(int32 InGraphBindingIndex);

	/** Get description of data produced by a pin. */
	PCG_API TSharedPtr<const FPCGDataCollectionDesc> GetCachedKernelPinDataDesc(const UPCGComputeKernel* InKernel, FName InPinLabel, bool bIsInput) const;
	TSharedPtr<const FPCGDataCollectionDesc> GetCachedKernelPinDataDesc(int32 InGraphBindingIndex) const;

	PCG_API void ReceiveDataFromGPU_GameThread(UPCGData* InData, const UPCGSettings* InProducerSettings, EPCGExportMode InExportMode, FName InPinLabel, FName InPinLabelAlias, const FPCGDataDesc& InDataDesc);

	using FSourceBufferAndAttributeIndex = TPair<TSharedPtr<const FPCGProxyForGPUDataCollection>, /*Source Metadata Attribute Index*/int32>;
	using FGraphAttributeIndexAndSourceBuffer = TPair</*Graph Metadata Attribute Index*/int32, TRefCountPtr<FRDGPooledBuffer>>;

	/** Get map from source buffer and source buffer attribute index to this-graph attribute index. */
	const TMap<FSourceBufferAndAttributeIndex, /*Graph Attribute Index*/int32>& GetSourceBufferAttributeToGraphAttributeIndex() const;

	/** Get map from this-graph attribute index and upstream source buffer to upstream attribute index. */
	const TMap<FGraphAttributeIndexAndSourceBuffer, /*Source Buffer Attribute Index*/int32>& GetGraphAttributeToSourceBufferAttributeIndex() const;

	using FSourceBufferAndStringKey = TPair<TRefCountPtr<FRDGPooledBuffer>, /*String Key*/int32>;

	/** Map from string key value in graph to the key value for this string in the source buffer. */
	const TMap<FSourceBufferAndStringKey, /*Graph String Key*/int32>& GetSourceBufferToGraphStringKey() const;

	/** Map from string key value in source buffer to the key value usable in this graph. */
	const TMap<FSourceBufferAndStringKey, /*Source Buffer String Key*/int32>& GetGraphToSourceBufferStringKey() const;

	PCG_API IPCGGraphExecutionSource* GetExecutionSource() const;

	/** Helper to get index in input data collection of the first data item for the given kernel and input pin label. */
	PCG_API int32 GetFirstInputDataIndex(const UPCGComputeKernel* InKernel, FName InPinLabel) const;

	/** Returns indices of data in the input data collection arriving on the given kernel pin. */
	PCG_API TArray<int32> GetPinInputDataIndices(const UPCGComputeKernel* InKernel, FName InPinLabel) const;

	/** If there is a GPU proxy at the given index of the input data collection, triggers a readback and replaces the data item with CPU data if readback succeeds.
	* Returns false while readback is in progress.
	*/
	PCG_API bool ReadbackInputDataToCPU(int32 InInputDataIndex);

	void DebugLogDataDescriptions() const;

	/** Initialize the kernel params cache. Performs readbacks of overridable params that require it, and caches all values for quick access from kernels and data providers.
	* Returns false while readback is in progress.
	*/
	bool InitializeKernelParams(FPCGContext* InContext);

	/** Look up the cached parameter struct for a kernel. */
	PCG_API const FPCGKernelParams* GetCachedKernelParams(const UPCGComputeKernel* InKernel) const;
	const FPCGKernelParams* GetCachedKernelParams(int32 InKernelIndex) const;

	TWeakPtr<FPCGContextHandle> GetContextHandle() const { return ContextHandle; }

	TArray<FPCGDataToDebug>& GetDataToDebugMutable();
	TArray<FPCGDataToDebug>& GetDataToInspectMutable();

	void AddCompletedMeshSpawnerKernel(TObjectPtr<const UComputeKernel> InCompletedMeshSpawnerKernel);
	bool IsMeshSpawnerKernelComplete(TObjectPtr<const UComputeKernel> InMeshSpawnerKernel) const;

	void AddMeshSpawnerPrimitives(TObjectPtr<const UComputeKernel> InMeshSpawnerKernel, const FPCGSpawnerPrimitives& Primitives);
	void AddMeshSpawnerPrimitives(TObjectPtr<const UComputeKernel> InMeshSpawnerKernel, FPCGSpawnerPrimitives&& Primitives);
	FPCGSpawnerPrimitives& FindOrAddMeshSpawnerPrimitives(TObjectPtr<const UComputeKernel> InMeshSpawnerKernel);
	FPCGSpawnerPrimitives* FindMeshSpawnerPrimitives(TObjectPtr<const UComputeKernel> InMeshSpawnerKernel);

	const UPCGComputeGraph* GetComputeGraph() const;

	PCG_API const FPCGDataCollection& GetInputDataCollection() const;
	FPCGDataCollection& GetInputDataCollectionMutable();

	const FPCGDataCollection& GetOutputDataCollection() const;

	/** Compute how many culling cells are needed to cover the current generation volume. */
	FIntVector GetMaxNumCullingCells(int32 InCullingCellExtent) const;

private:
	/** Loop over all metadata attributes in all input data ensure all attributes are registered in attribute table. */
	void AddInputDataAttributesToTable();
	void AddInputDataStringsToTable();
	void AddInputDataTagsToTable();

private:
	UPROPERTY()
	TObjectPtr<const UPCGComputeGraph> Graph = nullptr;

	/** Data arriving on compute graph element. Since the compute graph is collapsed to a single element, all data crossing from CPU to GPU is in a single collection. */
	UPROPERTY()
	FPCGDataCollection InputDataCollection;

	/** Compute graph element output data. Data items are labeled with unique virtual output pin labels so that the can be routed correctly by the graph
	 * executor to downstream nodes. */
	UPROPERTY()
	FPCGDataCollection OutputDataCollection;

	UPROPERTY()
	TMap<TObjectPtr<const UComputeKernel>, FPCGSpawnerPrimitives> MeshSpawnersToPrimitives;

	UPROPERTY()
	TArray<TObjectPtr<const UComputeKernel>> CompletedMeshSpawners;

	UPROPERTY()
	TArray<FPCGDataToDebug> DataToDebug;

	UPROPERTY()
	TArray<FPCGDataToDebug> DataToInspect;

	TWeakPtr<FPCGContextHandle> ContextHandle;

	/** All attributes present in graph at execution time. Coherent across all graph branches. Seeded using statically known created attributes at compile time, then
	 * augmented with incoming attributes from input data collection at runtime.
	 */
	UPROPERTY()
	FPCGKernelAttributeTable AttributeTable;

	UPROPERTY()
	TArray<FString> StringTable;

	/** Map from source buffer and source buffer attribute index to this-graph attribute index. */
	TMap<FSourceBufferAndAttributeIndex, /*Attribute ID*/int32> SourceBufferAttributeToGraphAttributeIndex;

	/** Map from this-graph attribute index and upstream source buffer to upstream attribute index. */
	TMap<FGraphAttributeIndexAndSourceBuffer, /*Source Buffer Attribute Index*/int32> GraphAttributeToSourceBufferAttributeIndex;

	/** Map from string key value in graph to the key value for this string in the source buffer. */
	TMap<FSourceBufferAndStringKey, /*Graph String Key*/int32> SourceBufferToGraphStringKey;

	/** Map from string key value in source buffer to the key value usable in this graph. */
	TMap<FSourceBufferAndStringKey, /*Source Buffer String Key*/int32> GraphToSourceBufferStringKey;

	/** Cache of data descriptions to amortize cost of computing them at runtime. */
	TMap</*Binding index*/int32, TSharedPtr<const FPCGDataCollectionDesc>> DataDescriptionCache;
	std::atomic<bool> bIsDataDescriptionCachePrimed = false;

	std::atomic<bool> bTablesInitialized = false;

	/** Cache of overridden settings/values per-kernel, built during compute graph execution. */
	FPCGKernelParamsCache KernelParamsCache;

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "Use the version that takes a data description instead.")
	PCG_API void ReceiveDataFromGPU_GameThread(UPCGData* InData, const UPCGSettings* InProducerSettings, EPCGExportMode InExportMode, FName InPinLabel, FName InPinLabelAlias);
	UE_DEPRECATED(5.8, "Use the version that takes a data description instead.")
	PCG_API void ReceiveDataFromGPU_GameThread(UPCGData* InData, const UPCGSettings* InProducerSettings, EPCGExportMode InExportMode, FName InPinLabel, FName InPinLabelAlias, const TSet<FString>& AdditionalTags);
};
