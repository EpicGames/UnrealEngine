// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGData.h"
#include "Compute/PCGComputeCommon.h"
#include "Data/PCGTextureData.h"

#include "PCGDataDescription.generated.h"

class UPCGData;
class UPCGDataBinding;
enum class EPCGElementMultiplicity : uint8;
enum class EPCGMetadataTypes : uint8;
enum class EPCGPointNativeProperties : uint32;

UENUM()
enum class EPCGKernelAttributeType : uint8
{
	Bool = 1,
	Int,
	Float,
	Float2,
	Float3,
	Float4,
	Rotator,
	Quat,
	Transform,
	StringKey,
	Name,

	Invalid = std::numeric_limits<uint8>::max() UMETA(Hidden),
};

/** Attribute name and type which uniquely identify an attribute in a compute graph. */
USTRUCT()
struct FPCGKernelAttributeKey
{
	GENERATED_BODY()

	FPCGKernelAttributeKey() = default;

	explicit FPCGKernelAttributeKey(FPCGAttributeIdentifier InIdentifier, EPCGKernelAttributeType InType)
		: Identifier(InIdentifier)
		, Type(InType)
	{}

	PCG_API explicit FPCGKernelAttributeKey(const FPCGAttributePropertySelector& InSelector, EPCGKernelAttributeType InType);

	const FPCGAttributeIdentifier& GetIdentifier() const { return Identifier; }
	EPCGKernelAttributeType GetType() const { return Type; }

	/** To be called everytime the Selector changes, to update the Identifier. Return true if it has changed. */
	bool UpdateIdentifierFromSelector();

	PCG_API bool IsValid() const;

	PCG_API bool operator==(const FPCGKernelAttributeKey& Other) const;

	PCG_API friend uint32 GetTypeHash(const FPCGKernelAttributeKey& In);

private:
	void SetSelector(const FPCGAttributePropertySelector& InSelector);

private:
	/** Cached identifier. Need to be updated if the Selector ever change. */
	UPROPERTY()
	FPCGAttributeIdentifier Identifier;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayAfter = "Attribute"))
	EPCGKernelAttributeType Type = EPCGKernelAttributeType::Float;

	/** Selector to specify which attribute to create and on which domain. At the moment, only support `@Data` domains or no domain (default one). */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyOutputNoSourceSelector Name;
};

/** Table of attributes used in compute graph with helpers to get unique attribute ID used to read/write attribute in data collection buffers. */
USTRUCT()
struct FPCGKernelAttributeTable
{
	GENERATED_BODY()

	FPCGKernelAttributeTable() = default;

	int32 GetAttributeId(const FPCGKernelAttributeKey& InAttribute) const;

	int32 GetAttributeId(FPCGAttributeIdentifier InIdentifier, EPCGKernelAttributeType InType) const;

	/** Adds an attribute of given name and type. Returns index or INDEX_NONE if add failed (happens if max table size reached). */
	int32 AddAttribute(const FPCGKernelAttributeKey& Key);
	int32 AddAttribute(FPCGAttributeIdentifier InIdentifier, EPCGKernelAttributeType InType);

	/** Caller must not insert the same key twice. */
	int32 AddAttributeAssumeUnique(const FPCGKernelAttributeKey& Key);

	int32 Num() const { return AttributeTable.Num(); }

	void Empty() { AttributeTable.Empty(); }

	void Reserve(int32 InCapacity) { AttributeTable.Reserve(InCapacity); }
	void Shrink() { AttributeTable.Shrink(); }

	using FAttributeKeyToIndexMap = TMap<FPCGKernelAttributeKey, int32, TInlineSetAllocator<32>>;

	/** Export a map of keys to attribute indices. */
	FAttributeKeyToIndexMap GetAttributeKeyToIndex() const;

#if PCG_KERNEL_LOGGING_ENABLED
	void DebugLog() const;
#endif

private:
	UPROPERTY()
	TArray<FPCGKernelAttributeKey> AttributeTable;
};

/** Data description for a metadata attribute. Stores identifying name and type as well as the unique attribute ID. */
struct FPCGKernelAttributeDesc
{
public:
	FPCGKernelAttributeDesc() = default;

	explicit FPCGKernelAttributeDesc(int32 InAttributeId, EPCGKernelAttributeType InAttributeType, FPCGAttributeIdentifier InAttributeIdentifier)
		: AttributeId(InAttributeId)
		, AttributeKey(InAttributeIdentifier, InAttributeType)
	{
	}

	explicit FPCGKernelAttributeDesc(int32 InAttributeId, EPCGKernelAttributeType InAttributeType, FPCGAttributeIdentifier InAttributeIdentifier, const TArray<int32>* InUniqueStringKeys)
		: AttributeId(InAttributeId)
		, AttributeKey(InAttributeIdentifier, InAttributeType)
	{
		if (InUniqueStringKeys)
		{
			UniqueStringKeys = *InUniqueStringKeys;
		}
	}

	int32 GetAttributeId() const { return AttributeId; }
	void SetAttributeId(int32 InAttributeId);

	const FPCGKernelAttributeKey& GetAttributeKey() const { return AttributeKey; }

	PCG_API void AddUniqueStringKeys(const TArray<int32>& InOtherStringKeys);
	PCG_API void SetStringKeys(const TConstArrayView<int32>& InStringKeys);
	const TArray<int32>& GetUniqueStringKeys() const { return UniqueStringKeys; }

	PCG_API bool operator==(const FPCGKernelAttributeDesc& Other) const;

private:
	int32 AttributeId = INDEX_NONE;

	FPCGKernelAttributeKey AttributeKey{};

	/* All possible string keys arriving on this attribute (string keys are indices into the string table in the data binding). */
	TArray<int32> UniqueStringKeys;
};

/** Data description for a single data object (UPCGData). */
struct FPCGDataDesc
{
public:
	FPCGDataDesc() = default;
	PCG_API FPCGDataDesc(FPCGDataTypeIdentifier InType, int32 InElementCount);
	PCG_API FPCGDataDesc(FPCGDataTypeIdentifier InType, FIntPoint InElementCount);
	FPCGDataDesc(FPCGDataTypeIdentifier InType, FIntVector3 InElementCount);
	FPCGDataDesc(FPCGDataTypeIdentifier InType, FIntVector4 InElementCount);
	FPCGDataDesc(const FPCGTaggedData& InTaggedData, const UPCGDataBinding* InBinding);

	bool HasElementsMetadataDomainAttributes() const;

	PCG_API bool ContainsAttribute(FPCGAttributeIdentifier InAttributeIdentifier) const;

	PCG_API bool ContainsAttribute(FPCGAttributeIdentifier InAttributeIdentifier, EPCGKernelAttributeType InAttributeType) const;

	PCG_API void AddAttribute(FPCGKernelAttributeKey InAttribute, const UPCGDataBinding* InBinding, const TArray<int32>* InOptionalUniqueStringKeys = nullptr);

	const FPCGDataTypeIdentifier& GetType() const { return Type; }

	FIntVector4 GetElementCount() const { return ElementCount; }
	FIntVector4 GetElementCountForAttribute(const FPCGKernelAttributeDesc& AttributeDesc) const;
	EPCGElementDimension GetElementDimension() const { return ElementDimension; }
	PCG_API int32 ComputeTotalElementCount() const;

	void AddElementCount(int32 InElementCountToAdd);
	void AddElementCount(FIntPoint InElementCountToAdd);
	void AddElementCount(FIntVector3 InElementCountToAdd);
	void AddElementCount(FIntVector4 InElementCountToAdd);
	void ScaleElementCount(int32 InMultiplier);

	FPCGDataDesc& CombineElementCount(const FPCGDataDesc& Other, EPCGElementMultiplicity Multiplicity);
	static FPCGDataDesc CombineElementCount(const FPCGDataDesc& A, const FPCGDataDesc& B, EPCGElementMultiplicity Multiplicity);

	TConstArrayView<FPCGKernelAttributeDesc> GetAttributeDescriptions() const { return MakeConstArrayView(AttributeDescs); }
	TArray<FPCGKernelAttributeDesc>& GetAttributeDescriptionsMutable() { return AttributeDescs; }

	void AllocateProperties(EPCGPointNativeProperties InProperties) { AllocatedPointProperties |= InProperties; }
	PCG_API void DeallocateProperties(EPCGPointNativeProperties InProperties);
	EPCGPointNativeProperties GetAllocatedProperties() const { return AllocatedPointProperties; }

	bool IsAttributeAllocated(const FPCGKernelAttributeDesc& InAttributeDesc) const { return IsAttributeAllocated(InAttributeDesc.GetAttributeId()); }
	PCG_API bool IsAttributeAllocated(int32 InAttributeId) const;

	uint16 GetTextureArraySize() const { return TextureArraySize; }
	void SetTextureArraySize(uint16 InTextureArraySize) { TextureArraySize = InTextureArraySize; }

	const FTransform& GetTextureTransform() const { return TextureTransform; }
	void SetTextureTransform(const FTransform& InTransform) { TextureTransform = InTransform; }

	// There's no store shader function to write to particular mips so we always generate textures with 1 mip for now.
	const int32 GetTextureNumMips() const { return 1; }

	TConstArrayView<int32> GetTagStringKeys() const { return MakeConstArrayView(TagStringKeys); }
	TArrayView<int32> GetTagStringKeysMutable();
	void SetTagStringKeys(TConstArrayView<int32> InTagStringKeys) { TagStringKeys = InTagStringKeys; }
	void AddUniqueTagStringKey(int32 InTagStringKey) { TagStringKeys.AddUnique(InTagStringKey); }

	EPCGRenderTargetFormat GetRenderTargetFormat() const { return RenderTargetFormat; }
	void SetRenderTargetFormat(EPCGRenderTargetFormat InRenderTargetFormat) { RenderTargetFormat = InRenderTargetFormat; }

	EPCGTextureFilter GetTextureFilter() const { return TextureFilter; }
	void SetTextureFilter(EPCGTextureFilter InTextureFilter) { TextureFilter = InTextureFilter; }

private:
	void InitializeAttributeDescs(const UPCGData* InData, const UPCGDataBinding* InBinding);

private:
	FPCGDataTypeIdentifier Type = EPCGDataType::Point;
	FIntVector4 ElementCount = FIntVector4::ZeroValue;
	EPCGElementDimension ElementDimension = EPCGElementDimension::One;
	TArray<FPCGKernelAttributeDesc> AttributeDescs;

	/** Properties that should be fully allocated for this data. Only applies to EPCGDataType::Point currently. */
	EPCGPointNativeProperties AllocatedPointProperties = EPCGPointNativeProperties::None;

	TArray<int32, TInlineAllocator<4>> TagStringKeys;

	// Texture settings
	/** Texture format used to create a render target UAV from this data. If this description represents a CPU texture data, then this format will be a narrowed conversion from its EPixelFormat. */
	EPCGRenderTargetFormat RenderTargetFormat = EPCGRenderTargetFormat::RGBA16f;
	EPCGTextureFilter TextureFilter = EPCGTextureFilter::Bilinear;
	FTransform TextureTransform = FTransform::Identity;
	uint16 TextureArraySize = 0;
};

/** Data description for a data collection (FPCGDataCollection). */
struct FPCGDataCollectionDesc
{
	/** FPCGDataCollectionDesc should not be created directly, use MakeShared(). */
	FPCGDataCollectionDesc(const FPCGDataCollectionDesc&) = delete;
	FPCGDataCollectionDesc& operator=(const FPCGDataCollectionDesc&) = delete;
	FPCGDataCollectionDesc(FPCGDataCollectionDesc&&) = delete;
	FPCGDataCollectionDesc& operator=(FPCGDataCollectionDesc&&) = delete;

	/** Create an FPCGDataCollectionDesc. */
	static PCG_API TSharedPtr<FPCGDataCollectionDesc> MakeShared();

	/** Create an FPCGDataCollectionDesc from another FPCGDataCollectionDesc. */
	static PCG_API TSharedPtr<FPCGDataCollectionDesc> MakeSharedFrom(const TSharedPtr<const FPCGDataCollectionDesc> InOtherDataDesc);

	TConstArrayView<FPCGDataDesc> GetDataDescriptions() const { return MakeConstArrayView(DataDescs); }
	TArray<FPCGDataDesc>& GetDataDescriptionsMutable() { return DataDescs; }

	/** Compute total number of processing elements. */
	PCG_API uint32 ComputeTotalElementCount() const;

	/** Get description of first attribute with matching identifier in input data. Returns true if such an attribute found and also signals whether multiple matching attributes
	* with conflicting names are present. */
	PCG_API bool GetAttributeDesc(FPCGAttributeIdentifier InAttributeIdentifier, FPCGKernelAttributeDesc& OutAttributeDesc, bool& bOutConflictingTypesFound, bool& bOutPresentOnAllData) const;

	PCG_API bool ContainsAttributeOnAnyData(FPCGAttributeIdentifier InAttributeIdentifier) const;

	/** Makes attribute present on all data. If data has existing attribute with same name then the given type will be applied. */
	PCG_API void AddAttributeToAllData(FPCGKernelAttributeKey InAttribute, const UPCGDataBinding* InBinding, const TArray<int32>* InOptionalUniqueStringKeys = nullptr);

	PCG_API void AllocatePropertiesForAllData(EPCGPointNativeProperties InProperties);
	PCG_API void DeallocatePropertiesForAllData(EPCGPointNativeProperties InProperties);
	PCG_API EPCGPointNativeProperties GetAllocatedPropertiesUnion() const;

	PCG_API void GetUniqueStringKeyValues(int32 InAttributeId, TArray<int32>& OutUniqueStringKeys) const;

	PCG_API int GetNumStringKeyValues(int32 InAttributeId) const;

	PCG_API void SetRenderTargetFormatForAllData(EPCGRenderTargetFormat InRenderTargetFormat);

	PCG_API void SetTextureFilterForAllData(EPCGTextureFilter InTextureFilter);

	PCG_API void SetTextureArraySizeForAllData(uint16 InTextureArraySize);

	PCG_API void SetTextureTransformForAllData(const FTransform& InTransform);

private:
	FPCGDataCollectionDesc() = default;

private:
	/** Description of each data in this data collection. */
	TArray<FPCGDataDesc> DataDescs;
};

namespace PCGDataDescriptionHelpers
{
	/** Returns GPU type that will be used to represent the given metadata type. */
	EPCGKernelAttributeType GetAttributeTypeFromMetadataType(EPCGMetadataTypes MetadataType);

	int GetAttributeTypeStrideBytes(EPCGKernelAttributeType Type);
}
