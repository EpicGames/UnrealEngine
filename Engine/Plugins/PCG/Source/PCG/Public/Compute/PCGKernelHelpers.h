// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGSettings.h"
#include "Compute/PCGComputeCommon.h"
#include "Compute/PCGComputeKernel.h"
#include "Graph/PCGGPUGraphCompilationContext.h"

class UPCGNode;
class UPCGSettings;

namespace PCGKernelHelpers
{
#if WITH_EDITOR
	struct FCreateKernelParams
	{
		FCreateKernelParams(UObject* InObjectOuter, const UPCGSettings* InSettings, const UPCGNode* InNode)
			: ObjectOuter(InObjectOuter)
			, OwnerSettings(InSettings)
			, OwnerNode(InNode)
			, bDumpDataDescriptions(InSettings && InSettings->bDumpDataDescriptions)
		{
		}

		UE_DEPRECATED(5.8, "Use the overload that takes a UPCGNode.")
		FCreateKernelParams(UObject* InObjectOuter, const UPCGSettings* InSettings = nullptr)
			: FCreateKernelParams(InObjectOuter, InSettings, nullptr)
		{
		}

		UObject* ObjectOuter = nullptr;

		/** When true, discovers GPU-overridable params and exposes them as wireable pins via an auto-created UPCGKernelParamsDataInterface. */
		bool bRequiresOverridableParams = true;

		const UPCGSettings* OwnerSettings = nullptr;

		const UPCGNode* OwnerNode = nullptr;

		bool bDumpDataDescriptions = false;

		/** Create edges from node input pins to kernel inputs, assuming label is same on both. */
		TArray<FName> NodeInputPinsToWire = { PCGPinConstants::DefaultInputLabel };

		/** Create edges from node output pins to kernel outputs, assuming label is same on both. */
		TArray<FName> NodeOutputPinsToWire = { PCGPinConstants::DefaultOutputLabel };
	};

	template<class KernelType>
	KernelType* CreateKernel(FPCGGPUCompilationContext& InCompilationContext, const FCreateKernelParams& InParams, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutKernelEdges)
	{
		FPCGComputeKernelParams KernelParams;
		KernelParams.Settings = InParams.OwnerSettings;
		KernelParams.Node = InParams.OwnerNode;
		KernelParams.bLogDescriptions = InParams.bDumpDataDescriptions;
		KernelParams.bRequiresOverridableParams = InParams.bRequiresOverridableParams;

		KernelType* Kernel = InCompilationContext.NewObject_AnyThread<KernelType>(InParams.ObjectOuter);

		Kernel->Initialize(KernelParams);
		OutKernels.Add(Kernel);

		// Connect node pins to kernel pins
		for (FName PinLabel : InParams.NodeInputPinsToWire)
		{
			OutKernelEdges.Emplace(FPCGPinReference(PinLabel), FPCGPinReference(Kernel, PinLabel));
		}

		for (FName PinLabel : InParams.NodeOutputPinsToWire)
		{
			OutKernelEdges.Emplace(FPCGPinReference(Kernel, PinLabel), FPCGPinReference(PinLabel));
		}

		return CastChecked<KernelType>(Kernel);
	}
#endif // WITH_EDITOR
}
