// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeKernel.h"

#include "StructUtils/PropertyBag.h"

class UPCGDataBinding;
class UPCGSettings;
struct FPCGContext;

/** Parameters for compute kernel execution. */
struct FPCGKernelParams
{
public:
	PCG_API bool GetValueBool(FName InParamName) const;
	PCG_API int GetValueInt(FName InParamName) const;
	PCG_API float GetValueFloat(FName InParamName) const;
	PCG_API double GetValueDouble(FName InParamName) const;
	PCG_API FName GetValueName(FName InParamName) const;
	PCG_API FVector2D GetValueVector2D(FName InParamName) const;
	PCG_API FVector GetValueVector(FName InParamName) const;
	PCG_API FVector4 GetValueVector4(FName InParamName) const;
	PCG_API FQuat GetValueQuat(FName InParamName) const;
	PCG_API FRotator GetValueRotator(FName InParamName) const;
	PCG_API FTransform GetValueTransform(FName InParamName) const;

	/** Non-templated enum accessor that returns the raw unsigned integer value. */
	PCG_API uint32 GetValueEnumAsUint(FName InParamName) const;

	template<typename EnumValueType>
	EnumValueType GetValueEnum(FName InParamName) const
	{
		static_assert(TIsEnum<EnumValueType>::Value, "Should only call this with enum types.");

		EnumValueType Value{};

		const FEnumProperty* Property = nullptr;
		const void* ValuePtr = nullptr;

		if (const FPropertyAndAddress* PropertyAndAddress = OverriddenProperties.Find(InParamName))
		{
			Property = CastField<FEnumProperty>(PropertyAndAddress->Key);
			ValuePtr = PropertyAndAddress->Value;
		}
		else if (const UPCGSettings* Settings = WeakSettings.Get())
		{
			Property = CastField<FEnumProperty>(Settings->GetClass()->FindPropertyByName(InParamName));
			ValuePtr = Property ? Property->ContainerPtrToValuePtr<void>(Settings) : nullptr;
		}

		const FNumericProperty* NumericProperty = Property ? Property->GetUnderlyingProperty() : nullptr;
		return static_cast<EnumValueType>(ensure(NumericProperty && ValuePtr) ? NumericProperty->GetUnsignedIntPropertyValue(ValuePtr) : 0);
	}

private:
	template<typename T>
	T GetStructValue(FName InParamName) const
	{
		const FStructProperty* Property = nullptr;
		const void* ValuePtr = nullptr;

		if (const FPropertyAndAddress* PropertyAndAddress = OverriddenProperties.Find(InParamName))
		{
			Property = CastField<FStructProperty>(PropertyAndAddress->Key);
			ValuePtr = PropertyAndAddress->Value;
		}
		else if (const UPCGSettings* Settings = WeakSettings.Get())
		{
			Property = CastField<FStructProperty>(Settings->GetClass()->FindPropertyByName(InParamName));
			ValuePtr = Property ? Property->ContainerPtrToValuePtr<void>(Settings) : nullptr;
		}

		if (ensure(Property && ValuePtr && Property->Struct == TBaseStructure<T>::Get()))
		{
			return *static_cast<const T*>(ValuePtr);
		}

		return T{};
	}

private:
	/** Optional underlying settings object to get default values that are not overridden. */
	TWeakObjectPtr<const UPCGSettings> WeakSettings;

	/** Property bag instance that mirrors the WeakSettings. Contains the actual overridden values. */
	FInstancedPropertyBag OverriddenValues;

	using FPropertyAndAddress = TPair<const FProperty*, const void*>;

	/** Maps param name to its property and an address to read the overridden value. This will only be populated for params which are overidden and read back for CPU logic. */
	TMap<FName, FPropertyAndAddress> OverriddenProperties;

	friend struct FPCGKernelParamsCache;
};

/** Helper struct to cache parameters for compute kernels. */
struct FPCGKernelParamsCache
{
public:
	/** Performs a readback of any kernels with overridable parameters that require readback, and caches them for quick access from kernels and data providers.
	* Returns false while readback is in progress.
	*/
	bool Initialize(FPCGContext* InContext, UPCGDataBinding* InDataBinding);

	/** Look up the cached parameter struct for a kernel. */
	const FPCGKernelParams* GetCachedKernelParams(int32 InKernelIndex) const;

	/** Clear the cache. */
	void Reset();

private:
	/** Cache readback indices for any overridable param data that require readback. */
	void CacheOverridableParamDataIndices(FPCGContext* InContext, const UPCGDataBinding* InDataBinding);

	/** Poll readback for overridable param data. Returns false while readback is in progress.  */
	bool ReadbackOverridableParamData(UPCGDataBinding* InDataBinding);

	/** Create and cache a property bag instance to store kernel param values and apply overrides if they exist. */
	void CreateAndCacheKernelParams(FPCGContext* InContext, const UPCGDataBinding* InDataBinding);

private:
	/** Maps kernels to their cached kernel parameter values. */
	TMap</*KernelIndex*/int32, FPCGKernelParams> KernelParams;

	/** Maps kernels to their overridable params and an index into the DataBinding input data collection that holds the corresponding param data. */
	TMap</*KernelIndex*/int32, TArray<TPair<FPCGKernelOverridableParam, int32>>> OverridableParamDataIndices;
};
