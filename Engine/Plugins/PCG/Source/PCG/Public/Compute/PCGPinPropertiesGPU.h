// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPin.h"
#include "Compute/PCGDataDescription.h"
#include "Data/PCGTextureData.h"

#include "PCGPinPropertiesGPU.generated.h"

#define UE_API PCG_API

/** Method for initializing an output pin. */
UENUM()
enum class EPCGPinInitMode : uint8
{
	FromInputPins UMETA(DisplayName = "FromInput", Tooltip = "Initialize structure of data (data count, element count, attribute definitions, etc) from one or more input pins."),
	Custom UMETA(Tooltip = "Initialize structure of data (data count, element count, attribute definitions, etc) from fixed values."),
};

// @todo_pcg: Data/Element count mode could be consolidated with the generic EPCGPropertyInheritanceMode below.
/** Method for computing data count. */
UENUM()
enum class EPCGDataCountMode : uint8
{
	FromInputData UMETA(DisplayName = "FromInput"),
	Fixed UMETA(DisplayName = "Custom"),
};

/** Method for computing element count. */
UENUM()
enum class EPCGElementCountMode : uint8
{
	FromInputData UMETA(DisplayName = "FromInput"),
	Fixed UMETA(DisplayName = "Custom"),
};

/** Method for combining two or more data counts. */
UENUM()
enum class EPCGDataMultiplicity : uint8
{
	Pairwise UMETA(Tooltip = "A data item will be produced for each pair/tuple/etc of input data items across the input pins. Requires all pins to have 1 or N data items."),
	CartesianProduct UMETA(Tooltip = "If there are two input pins with N and M data items respectively, the output will have NxM data items."),
};

/** Method for combining two or more element counts. */
UENUM()
enum class EPCGElementMultiplicity : uint8
{
	Product UMETA(Tooltip = "Element count will be the product of the elements in each pair/tuple/etc of input data."),
	Sum UMETA(Tooltip = "Element count will be the sum of the elements in each pair/tuple/etc of input data."),
};

/** Method for inheriting attribute data from input pins. */
UENUM()
enum class EPCGAttributeInheritanceMode : uint8
{
	None UMETA(Tooltip = "No attributes inherited from initialization pins."),
	CopyAttributeSetup UMETA(Tooltip = "Take attribute names and types from initialization pins. Pins declared first will take priority during conflicts. Does not copy values."),
	// TODO: Mode to automatically copy values. Requires code-gen to loop on batches of elements for pins based on the total number of elements and total number of threads.
	//CopyAttributeSetupAndValues UMETA(Tooltip = "Take attribute names, types, and initial values from initialization pins. Pins declared first will take priority during conflicts."),
};

/** Method for inheriting properties from input pins. */
UENUM()
enum class EPCGPropertyInheritanceMode : uint8
{
	FromInput,
	Custom
};

/** Helper struct to nest GPU pin properties inside a UI category. */
USTRUCT(BlueprintType)
struct FPCGPinPropertiesGPUStruct
{
	GENERATED_BODY()

	/** How the output data for this pin will be initialized. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode", EditConditionHides, HideEditConditionToggle))
	EPCGPinInitMode InitializationMode = EPCGPinInitMode::FromInputPins;

	/** Input pins to initialize this pin's data from. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (GetOptions = "GetInputPinNames", EditCondition = "bAllowEditInitMode && InitializationMode == EPCGPinInitMode::FromInputPins", EditConditionHides))
	TArray<FName> PinsToInititalizeFrom;

	/** How the number of data is determined. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && bAllowEditDataCount && InitializationMode == EPCGPinInitMode::FromInputPins", EditConditionHides))
	EPCGDataCountMode DataCountMode = EPCGDataCountMode::FromInputData;

	/** How to combine data counts. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && bAllowEditDataCount && InitializationMode == EPCGPinInitMode::FromInputPins && DataCountMode == EPCGDataCountMode::FromInputData && bMultipleInitPins", EditConditionHides))
	EPCGDataMultiplicity DataMultiplicity = EPCGDataMultiplicity::Pairwise;

	/** Number of data to create. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && (InitializationMode == EPCGPinInitMode::Custom || DataCountMode == EPCGDataCountMode::Fixed)", EditConditionHides, ClampMin = 1))
	int DataCount = 1;

	/** How the number of elements is determined. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && InitializationMode == EPCGPinInitMode::FromInputPins", EditConditionHides))
	EPCGElementCountMode ElementCountMode = EPCGElementCountMode::FromInputData;

	/** How to combine element counts. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && InitializationMode == EPCGPinInitMode::FromInputPins && ElementCountMode == EPCGElementCountMode::FromInputData && bMultipleInitPins", EditConditionHides))
	EPCGElementMultiplicity ElementMultiplicity = EPCGElementMultiplicity::Product;

	/** Fixed number of elements to create in each output data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, DisplayName = "Num Elements", meta = (EditCondition = "bAllowEditInitMode && (InitializationMode == EPCGPinInitMode::Custom || ElementCountMode == EPCGElementCountMode::Fixed) && !bShowTexturePinSettings", EditConditionHides, ClampMin = 1))
	int ElementCount = 1;

	/** Fixed number of elements to create in each output data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, DisplayName = "Num Elements", meta = (EditCondition = "bAllowEditInitMode && (InitializationMode == EPCGPinInitMode::Custom || ElementCountMode == EPCGElementCountMode::Fixed) && bShowTexturePinSettings", EditConditionHides, ClampMin = 1))
	FIntPoint NumElements2D = FIntPoint(64, 64);

	/** Method for inheriting texture format. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && bShowTexturePinSettings && InitializationMode == EPCGPinInitMode::FromInputPins", EditConditionHides, HideEditConditionToggle))
	EPCGPropertyInheritanceMode TextureFormatInheritanceMode = EPCGPropertyInheritanceMode::FromInput;

	/** Format for all texture data produced on this pin. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && (InitializationMode == EPCGPinInitMode::Custom || TextureFormatInheritanceMode == EPCGPropertyInheritanceMode::Custom) && bShowTexturePinSettings", EditConditionHides, HideEditConditionToggle))
	EPCGRenderTargetFormat TextureFormat = EPCGRenderTargetFormat::RGBA16f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && bShowTexturePinSettings", EditConditionHides, HideEditConditionToggle))
	bool bInitializeTextureTransformFromGenerationVolume = false;

	/** Method for inheriting texture transform. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && bShowTexturePinSettings && !bInitializeTextureTransformFromGenerationVolume", EditConditionHides, HideEditConditionToggle))
	EPCGPropertyInheritanceMode TextureTransformInheritanceMode = EPCGPropertyInheritanceMode::FromInput;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && (InitializationMode == EPCGPinInitMode::Custom || TextureTransformInheritanceMode == EPCGPropertyInheritanceMode::Custom) && bShowTexturePinSettings && !bInitializeTextureTransformFromGenerationVolume", EditConditionHides, HideEditConditionToggle))
	FTransform TextureTransform = FTransform::Identity;

	/** Method for inheriting texture filter. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && bShowTexturePinSettings && InitializationMode == EPCGPinInitMode::FromInputPins", EditConditionHides, HideEditConditionToggle))
	EPCGPropertyInheritanceMode TextureFilterInheritanceMode = EPCGPropertyInheritanceMode::FromInput;

	/** Method used to determine the value for a sample based on the value of nearby texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && (InitializationMode == EPCGPinInitMode::Custom || TextureFilterInheritanceMode == EPCGPropertyInheritanceMode::Custom) && bShowTexturePinSettings", EditConditionHides, HideEditConditionToggle))
	EPCGTextureFilter TextureFilter = EPCGTextureFilter::Bilinear;

	/** Method for inheriting texture array size. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && bShowTextureArrayPinSettings && InitializationMode == EPCGPinInitMode::FromInputPins", EditConditionHides, HideEditConditionToggle))
	EPCGPropertyInheritanceMode TextureArraySizeInheritanceMode = EPCGPropertyInheritanceMode::FromInput;

	/** Number of layers in the texture array created by this pin. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && (InitializationMode == EPCGPinInitMode::Custom || TextureArraySizeInheritanceMode == EPCGPropertyInheritanceMode::Custom) && bShowTextureArrayPinSettings", EditConditionHides, ClampMin = 1))
	int TextureArraySize = 1;

	/** Scalar on the fixed number of elements to create in each output data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && (InitializationMode == EPCGPinInitMode::Custom || ElementCountMode == EPCGElementCountMode::FromInputData)", EditConditionHides, ClampMin = 1))
	int ElementCountMultiplier = 1;

	/** Statically initialize data to 0 from CPU before execution. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bShowInitializeToZeroSetting", EditConditionHides, HideEditConditionToggle))
	bool bInitializeToZero = false;

	/** Automatically initialize output data from input before executing the kernel. When enabled the kernel only needs to write elements that differ from input.
	* Warning: Use with caution - unwritten data will have unitialized data and can intermittently fail or trigger undefined behaviour downstream. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bShowAutoInitializeOutput", EditConditionHides, HideEditConditionToggle))
	bool bAutoInitializeOutput = true;

	/** How to inherit attribute names, types, and values. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAllowEditInitMode && InitializationMode == EPCGPinInitMode::FromInputPins && !bShowTexturePinSettings", EditConditionHides))
	EPCGAttributeInheritanceMode AttributeInheritanceMode = EPCGAttributeInheritanceMode::CopyAttributeSetup;

	/** Add entries to create new attributes on data emitted by this pin. This is optional, as of 5.8 attributes are automatically created when written from HLSL. */
	UPROPERTY(EditAnywhere, DisplayName = "Attributes to Create", Category = Settings, meta = (EditCondition = "!bShowTexturePinSettings && bShowCreatedAttributesSettings", EditConditionHides, HideEditConditionToggle))
	TArray<FPCGKernelAttributeKey> CreatedKernelAttributeKeys;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	bool bAllowEditInitMode = false;

	/** Some types force single data. We should not allow data count to be edited for such types. */
	UPROPERTY(Transient)
	bool bAllowEditDataCount = false;

	UPROPERTY(Transient)
	bool bMultipleInitPins = false;

	UPROPERTY(Transient)
	bool bShowTexturePinSettings = false;

	UPROPERTY(Transient)
	bool bShowTextureArrayPinSettings = false;

	UPROPERTY(Transient)
	bool bShowInitializeToZeroSetting = false;

	UPROPERTY(Transient)
	bool bShowAutoInitializeOutput = false;

	/** Driven by UPCGCustomHLSLSettings::bPerPinAttributeCreationSettings. Shows/hides the CreatedKernelAttributeKeys array. */
	UPROPERTY(Transient)
	bool bShowCreatedAttributesSettings = false;
#endif // WITH_EDITORONLY_DATA
};

/** An extension of the pin properties that adds hints for GPU specific properties, such as buffer size and data layout. */
USTRUCT(BlueprintType)
struct FPCGPinPropertiesGPU : public FPCGPinProperties
{
	GENERATED_BODY()

public:
	FPCGPinPropertiesGPU() = default;

	explicit FPCGPinPropertiesGPU(const FName& InLabel, FPCGDataTypeIdentifier InAllowedTypes)
		: FPCGPinProperties(InLabel, InAllowedTypes)
	{}

	explicit FPCGPinPropertiesGPU(const FPCGPinProperties& InPinProps)
		: FPCGPinProperties(InPinProps.Label, InPinProps.AllowedTypes)
	{}

	UE_API uint32 GetElementCountMultiplier() const;

#if WITH_EDITOR
	UE_API bool CanEditChange(const FEditPropertyChain& PropertyChain) const;
#endif

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (DisplayName = "GPU Properties", DisplayAfter = "Tooltip", EditCondition = "bShowPropertiesGPU", EditConditionHides, HideEditConditionToggle))
	FPCGPinPropertiesGPUStruct PropertiesGPU;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	bool bShowPropertiesGPU = true;
#endif // WITH_EDITORONLY_DATA
};

template<>
struct TStructOpsTypeTraits<FPCGPinPropertiesGPU> : public TStructOpsTypeTraitsBase2<FPCGPinPropertiesGPU>
{
	enum
	{
		WithCanEditChange = true,
	};
};

#undef UE_API
