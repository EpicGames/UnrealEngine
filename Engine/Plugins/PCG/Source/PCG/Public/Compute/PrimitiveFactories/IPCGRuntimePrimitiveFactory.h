// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FPrimitiveSceneProxy;

/** Interface for helper object that creates and adds primitives to the scene. */
class IPCGRuntimePrimitiveFactory
{
public:
	virtual ~IPCGRuntimePrimitiveFactory() = default;

	virtual int32 GetNumPrimitives() const = 0;
	virtual FPrimitiveSceneProxy* GetSceneProxy(int32 InPrimitiveIndex) const = 0;
	virtual bool IsPrimitiveValid(int32 InPrimitiveIndex) const { ensureMsgf(false, TEXT("IsPrimitiveValid() should be overridden by classes implementing the IPCGRuntimePrimitiveFactory interface.")); return false; }
	PCG_API virtual int32 GetNumInstances(int32 InPrimitiveIndex, int32 InCellID) const;
	virtual int32 GetNumInstancesTotal(int32 InPrimitiveIndex) const = 0;
	virtual bool IsRenderStateCreated() const = 0;
	/** True if any owned primitive is awaiting render state recreate; callers must defer GPU-slot writes until this returns false. */
	PCG_API virtual bool IsAnyRenderStateDirty() const;

	UE_DEPRECATED(5.8, "Use version that takes cell ID instead")
	virtual int32 GetNumInstances(int32 InPrimitiveIndex) const { return 0; }
};
