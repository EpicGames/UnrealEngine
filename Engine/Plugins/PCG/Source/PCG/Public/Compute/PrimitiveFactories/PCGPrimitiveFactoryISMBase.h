// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PrimitiveFactories/IPCGRuntimePrimitiveFactory.h"

#include "Components/PCGProceduralISMComponentDescriptor.h"
#include "InstanceDataPackers/PCGInstanceDataPackerBase.h"

#include "InstanceDataSceneProxy.h"

class AActor;
struct FPCGContext;

struct FPCGInstanceRange
{
	FPCGInstanceRange(uint32 InNumInstances)
		: NumInstances(InNumInstances)
	{
	}

	FPCGInstanceRange(uint32 InNumInstances, int32 InCellID)
		: NumInstances(InNumInstances)
		, CellID(InCellID)
	{
	}

	uint32 GetNumInstances() const { return NumInstances; }
	int32 GetCellID() const { return CellID; }

private:
	uint32 NumInstances = 0;
	int32 CellID = 0;
};

struct FPCGPrimitiveInfo
{
	FPCGProceduralISMComponentDescriptor Descriptor;
	TArray<FPCGInstanceRange> InstanceRanges;
	float CullingCellExtent = 0.0f;
	FIntVector NumCullingCells = FIntVector::ZeroValue;
	/** Per-range world-space bounds from GPU readback, aligned 1:1 with InstanceRanges (same index, same length). Empty if no readback data is available. */
	TArray<FBox> CullingCellWorldBounds;
};

class IPCGPrimitiveFactoryISMBase : public IPCGRuntimePrimitiveFactory
{
public:
	struct FParameters
	{
		TArray<FPCGPrimitiveInfo> PrimitiveInfos;

		FPCGPackedCustomData CustomPrimitiveData;
		
		AActor* TargetActor = nullptr; // Optional, unless creating actor primitive components.

		UE_DEPRECATED(5.8, "Use PrimitiveInfos instead")
		TArray<FPCGProceduralISMComponentDescriptor> Descriptors;
	};

	virtual void Initialize(FParameters&& InParameters) = 0;
	virtual bool Create(FPCGContext* InContext) = 0;
	virtual FBox GetMeshBounds(int32 InPrimitiveIndex) const = 0;

protected:
	/** Produces spatial hash info which is an implicit cell in space plus instance counts. Also updates primitive bounds to tightly wrap populated cells. */
	PCG_API virtual void PopulateHashes(const FPCGPrimitiveInfo& InPrimitiveInfo, const FBox& InExecutionBounds, TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem>& OutHashes, FBox& OutBounds) const;
};

namespace PCGPrimitiveFactoryHelpers
{
	TSharedPtr<IPCGPrimitiveFactoryISMBase> GetFastGeoPrimitiveFactory();

	// todo_pcg: Remove and integrate FastGeo component spawning directly into PCG once FastGeo exits experimental.
	namespace Private
	{
		UE_EXPERIMENTAL(5.7, "This API is internal and not intended to be used.")
		PCG_API void SetupFastGeoPrimitiveFactory(TFunction<TSharedPtr<IPCGPrimitiveFactoryISMBase>()>&& Getter);

		UE_EXPERIMENTAL(5.7, "This API is internal and not intended to be used.")
		PCG_API void ResetFastGeoPrimitiveFactory();
	}
}
