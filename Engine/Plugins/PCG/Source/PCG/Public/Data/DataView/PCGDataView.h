// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataCommon.h"

#include "Serialization/CustomVersion.h"
#include "Templates/SubclassOf.h"

#include "PCGDataView.generated.h"

class UPCGDataViewBase;
struct FPCGAttributePropertySelector;
struct FPCGContext;

namespace PCGDataView::Constants
{
	// Custom PCG version for DataView data export
	struct FDataViewVersion
	{
		static constexpr TCHAR FriendlyName[] = TEXT("PCGDataView");

		enum Type
		{
			InitialVersion = 0,

			// New versions can be added above this line
			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};

		static FName GetFriendlyName() { return FName(FriendlyName); }
		const static FGuid GUID;
	};
} // namespace PCGDataView::Constants

UENUM()
enum class EPCGDataViewAttributeLayout : uint8
{
	ByElement UMETA(Tooltip = "Use the elements as the main data object. Each element contains all its attributes."),
	ByAttribute UMETA(Tooltip = "Use the attributes as the main data object. Each attribute grouping will contain the element values in sequential order.")
};

USTRUCT(MinimalAPI, BlueprintType, Blueprintable)
struct FPCGDataViewSelection
{
	GENERATED_BODY()

	// @todo_pcg: Can EPCGMetadataFilterMode be extended to include an "AllAttributes" mode?
	UPROPERTY(EditAnywhere, Category = Selection)
	bool bAllAttributes = true;

	// Limit the selection to a specific domain.
	UPROPERTY(EditAnywhere, Category = Selection, meta = (EditConditionHides, InlineEditConditionToggle))
	bool bLimitByDomain = false;

	// Ignore default attributes of the viewed data (properties). Ex. $Transform, $Density for Point Data.
	UPROPERTY(EditAnywhere, Category = Selection, DisplayName = "Ignore Static Attributes", meta = (EditCondition = "bAllAttributes", EditConditionHides))
	bool bIgnoreProperties = false;

	// @todo_pcg: Extend to use EPCGMetadataFilterMode for more complex selections
	UPROPERTY(EditAnywhere, Category = Selection, meta = (EditCondition = "!bAllAttributes", EditConditionHides))
	TArray<FPCGAttributePropertySelector> Attributes;

	// If "Limit By Domain" is enabled and a domain is provided, attributes from other domains will be filtered out.
	UPROPERTY(EditAnywhere, Category = Selection, DisplayName = "Limit by Domain", meta = (EditCondition = "bLimitByDomain"))
	FName Domain = TEXT("Data");
};

/** This wrapper class maintains a reference to unowned "viewed" data and also contains a "selection" of
 * metadata. This can be used with a variety of operations--usually exporting data--where a PCG Data type
 * needs to be abstracted and/or serialized into a more generic type or refined to an explicit selection
 * of properties or attributes on the data, to make it more accessible for processing.
 *
 * Ex. Convert the $Position property of each point in a PCG Point data into Json key:value objects. 
 */
USTRUCT(MinimalAPI, BlueprintType, Blueprintable)
struct FPCGDataView
{
	GENERATED_BODY()

	FPCGDataView() = default;
	virtual ~FPCGDataView() = default;

	// The data exists and the selection is not empty.
	PCG_API virtual bool IsValid() const;

	// Aggregate all domain IDs from the current selection
	PCG_API TArray<FPCGMetadataDomainID> GetAllMetadataDomainIDs() const;

	// Get the final selection. @todo_pcg: Could be cached.
	PCG_API TArray<FPCGAttributePropertySelector> GetResolvedSelection() const;

	// The PCG Data this will provide a 'view' onto.
	UPROPERTY(VisibleAnywhere, Category = Data)
	TObjectPtr<const UPCGData> ViewedData = nullptr;

	// An optionally filtered collection of attributes and properties to perform an operation on, such as data export.
	UPROPERTY(VisibleAnywhere, Category = Data, meta = (ShowOnlyInnerProperties))
	FPCGDataViewSelection Selection;
};
