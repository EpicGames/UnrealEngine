// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/DataView/PCGDataViewInterface.h"

#include "PCGDataViewCSVConverter.generated.h"

class FJsonObject;

UENUM()
enum class EPCGDataViewCSVOutput : uint8
{
	Name,
	Value,
	Both
};

USTRUCT()
struct FPCGDataViewCSVParameters
{
	GENERATED_BODY()

	// The comma-separated format, i.e. 'name' or 'value' or 'name:value'.
	UPROPERTY(EditAnywhere, Category = "CSV")
	EPCGDataViewCSVOutput Output = EPCGDataViewCSVOutput::Both;

	// The separating character(s) between the name and the value.
	UPROPERTY(EditAnywhere, Category = "CSV", meta = (EditCondition = "Output == EPCGDataViewCSVOutput::Both", EditConditionHides))
	FString NameValueSeparator = ":";

	// The character(s) between each value. By default, as a CSV, this will be the ',' character.
	UPROPERTY(EditAnywhere, Category = "CSV", meta = (InlineEditConditionToggle, EditCondition = "Output != EPCGDataViewCSVOutput::Name", EditConditionHides, MultiLine))
	FString ValueDelimiter = ",";

	// The character(s) between each attribute of the selection. By default--and via reset--this will be a platform-specific line terminator.
	UPROPERTY(EditAnywhere, Category = "CSV")
	FString AttributeDelimiter = LINE_TERMINATOR;

	// Omit the domain from the name of the attribute.
	UPROPERTY(EditAnywhere, Category = "CSV")
	bool bOmitDomain = true;
};

USTRUCT()
struct FPCGDataViewCSVOutput
{
	GENERATED_BODY()

	FString CSVFormattedOutput;
};

UCLASS()
class UPCGDataViewCSVConverter : public UPCGDataViewConverterBase
{
	GENERATED_BODY()

public:
	virtual TValueOrError<FInstancedStruct, FText> SerializeToTargetStruct(const FPCGDataView& InDataView, const FInstancedStruct& Parameters) const override;
	virtual TValueOrError<FString, FText> SerializeToString(const FPCGDataView& InDataView, const FInstancedStruct& Parameters) const override;
	virtual TValueOrError<UPCGData*, FText> ConstructDataFromTarget(const FInstancedStruct& InTargetStruct, UObject* Outer = nullptr) const override;
	virtual UScriptStruct* GetTargetStruct() const override { return FPCGDataViewCSVOutput::StaticStruct(); }
	virtual UScriptStruct* GetParameterStruct() const override { return FPCGDataViewCSVParameters::StaticStruct(); }
};
