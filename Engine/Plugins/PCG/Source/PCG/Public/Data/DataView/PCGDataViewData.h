// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "PCGDataViewInterface.h"
#include "PCGDataView.h"

#include "PCGDataViewData.generated.h"

// The Data View type info and description
USTRUCT()
struct FPCGDataTypeInfoDataView : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO()

	virtual bool SupportsConversionFrom(const FPCGDataTypeIdentifier& InputType, const FPCGDataTypeIdentifier& ThisType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings = nullptr, FText* OptionalOutCompatibilityMessage = nullptr) const override;
};

/** Acts as a type-erasing abstraction wrapper around other PCG Data types, as well as allowing for a limited
 * selection of attributes or properties to be defined for performing an operation on later. I.e. Specified
 * attributes or properties can be selectively serialized into another format, supported by implemented interfaces.
 *
 * Ex. A PCG Data View Data could point to a PCG Point Data, where only the $Position property is selected.
 * Because the point data implements the IPCGDataViewJson interface, the selection of Positions for each point
 * in the data may be serialized to/from a Json object/string form.
 *
 * Implementation Note: UPCGDataViewData should implement all available Data View interfaces to enforce and
 * maintain parity with consumer types. Calling an interface method on the Data View Data should essentially
 * forward the call to the underlying data, if the interface is supported.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDataViewData : public UPCGData
{
	GENERATED_BODY()

	// Used to cache the selector/accessor for ease of data retrieval into the data
	struct FAccessorCacheEntry
	{
		FPCGAttributePropertySelector Selector;
		TUniquePtr<const IPCGAttributeAccessor> Accessor = nullptr;
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = nullptr;
	};

public:
	PCG_ASSIGN_TYPE_INFO(FPCGDataTypeInfoDataView);

	// Set up the data view upon creation.
	void Initialize(FPCGDataView&& InDataView);

	//~ Begin UPCGData interface
	virtual const FPCGDataView& GetDataView() const;

	// Forwarding helper to get a domain ID from a selector, to the internal data.
	virtual FPCGMetadataDomainID GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const override;
	// Forwarding helper to get the default domain ID, to the internal data.
	virtual FPCGMetadataDomainID GetDefaultMetadataDomainID() const override;
	// Forwarding helper to get all supported domain IDs, to the internal data.
	virtual TArray<FPCGMetadataDomainID> GetAllSupportedMetadataDomainIDs() const override;
	// Forwarding helper to the underlying metadata.
	virtual const UPCGMetadata* ConstMetadata() const override;

protected:
	// Compute the Crc for the Data View and its underlying data.
	virtual FPCGCrc ComputeCrc(bool bFullDataCrc) const override;
	// Serialize the Data View selection components into the Crc.
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	virtual bool SupportsFullDataCrc() const override { return true; }
	//~ End UPCGData interface

public:
	// The selection is stored lazily. Get Selected Attributes will create the array of selectors. @todo_pcg: Look into caching results.
	TArray<FPCGAttributePropertySelector> GetSelectedAttributes() const;

private:
	UPROPERTY()
	FPCGDataView DataView;
};
