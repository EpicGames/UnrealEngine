// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/IO/PCGIOHelpers.h"
#include "Helpers/IO/PCGJsonHelpers.h"
#include "Templates/ValueOrError.h"

#include "PCGDataViewInterface.generated.h"

class FJsonObject;
class UPCGData;
struct FPCGDataView;

// @todo_pcg: Enable when loading from serialized data view.
// UENUM(BlueprintType)
// enum class EPCGDataViewVersionMismatchPolicy : uint8
// {
// 	Ignore = 0u,
// 	Log,
// 	Warn,
// 	Error
// };

/** A class can inherit from this interface to define a process for selecting properties of a PCG Data */
class IPCGDataViewPropertySelector
{
public:
	virtual ~IPCGDataViewPropertySelector() = default;

	/** Allows data types to set custom property selectors. */
	virtual TArray<FPCGAttributePropertySelector> GetSelection(const FPCGDataView& InDataView) const = 0;
};

/** The native base property selector for PCG. Covers spatial and any base types */
class FPCGDataViewPropertySelector : public IPCGDataViewPropertySelector
{
	virtual TArray<FPCGAttributePropertySelector> GetSelection(const FPCGDataView& InDataView) const override { return {}; }
};

UCLASS(Abstract)
class UPCGDataViewConverterBase : public UObject
{
	GENERATED_BODY()

public:
	virtual bool CanConvertToString() const { return false; }

	/** Override to serialize the Data View's underlying data and selection into an FInstancedStruct matching GetTargetStruct(). */
	virtual TValueOrError<FInstancedStruct, FText> SerializeToTargetStruct(const FPCGDataView& InDataView, const FInstancedStruct& Parameters) const PURE_VIRTUAL(UPCGDataViewConverterBase::SerializeToTargetStruct, { return MakeError(FText::GetEmpty()); })

	virtual TValueOrError<FString, FText> SerializeToString(const FPCGDataView& InDataView, const FInstancedStruct& Parameters) const PURE_VIRTUAL(UPCGDataViewConverterBase::SerializeToString, { return MakeError(FText::GetEmpty()); })

	/** Construct a new PCG Data subclass from an FInstancedStruct matching GetTargetStruct(). */
	virtual TValueOrError<UPCGData*, FText> ConstructDataFromTarget(const FInstancedStruct& InTargetStruct, UObject* Outer = nullptr) const PURE_VIRTUAL(UPCGDataViewConverterBase::ConstructDataFromTarget, { return MakeError(FText::GetEmpty()); })
	/** The custom UScriptStruct that contains the data after conversion in the target data format. */
	virtual UScriptStruct* GetTargetStruct() const PURE_VIRTUAL(UPCGDataViewConverterBase::GetTargetStruct, { return nullptr; })
	/** The custom UScriptStruct that contains the user defined parameters for the conversion. */
	virtual UScriptStruct* GetParameterStruct() const PURE_VIRTUAL(UPCGDataViewConverterBase::GetParameterStruct, { return nullptr; })

	/** Will leverage SerializeToString and save the result to the provided file path. */
	TValueOrError<void, FText> SerializeToFile(const FPCGDataView& InDataView, const FInstancedStruct& Parameters, const FString& FilePath) const;
};
