// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/DataView/PCGDataView.h"
#include "Data/DataView/PCGDataViewInterface.h"

#include "PCGDataViewNativeJsonConverters.generated.h"

class FJsonObject;

USTRUCT()
struct FPCGDataViewJsonParameters
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Json")
	EPCGDataViewAttributeLayout AttributeLayout = EPCGDataViewAttributeLayout::ByElement;

	UPROPERTY(EditAnywhere, Category = "Json")
	bool bPrettyJson = false;
};

/** Holds the root Json object for a PCG Data <-> Json conversion. @todo_pcg: consider a serializable option. */
USTRUCT()
struct FPCGDataViewJsonOutput
{
	GENERATED_BODY()

	TSharedPtr<FJsonObject> JsonObject = nullptr;
};

UCLASS()
class UPCGDataViewJsonConverter : public UPCGDataViewConverterBase
{
	GENERATED_BODY()

public:
	virtual TValueOrError<FInstancedStruct, FText> SerializeToTargetStruct(const FPCGDataView& InDataView, const FInstancedStruct& Parameters) const override;
	virtual TValueOrError<FString, FText> SerializeToString(const FPCGDataView& InDataView, const FInstancedStruct& Parameters) const override;
	virtual TValueOrError<UPCGData*, FText> ConstructDataFromTarget(const FInstancedStruct& InTargetStruct, UObject* Outer = nullptr) const override;
	virtual UScriptStruct* GetTargetStruct() const override { return FPCGDataViewJsonOutput::StaticStruct(); }
	virtual UScriptStruct* GetParameterStruct() const override { return FPCGDataViewJsonParameters::StaticStruct(); }

protected:
	/** Override to create a custom Json header. */
	virtual void BuildJsonHeader(const FPCGDataView& InDataView, TSharedPtr<FJsonObject>& InOutJsonObject) const;
};
