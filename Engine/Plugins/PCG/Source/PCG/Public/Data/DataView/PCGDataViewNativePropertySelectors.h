// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/DataView/PCGDataViewInterface.h"

namespace PCGDataView::Helpers
{
	// Append all non-property attributes to the provided array of selectors.
	void AppendAllAttributeSelectors(const UPCGData* InData, TArray<FPCGAttributePropertySelector>& InOutSelectors, TOptional<FPCGMetadataDomainID> OptionalDomain = {});

	// Append all relevant point properties avoiding redundancy, i.e. Transform already includes Positions, Rotation, Scale
	void AppendAllPointPropertySelectors(TArray<FPCGAttributePropertySelector>& InOutSelectors);

	// Append all relevant spline properties to the provided array of selectors.
	void AppendAllSplinePropertySelectors(TArray<FPCGAttributePropertySelector>& InOutSelectors);
} // namespace PCGDataView::Helpers

class FPCGDataViewBasePointDataPropertySelector : public FPCGDataViewPropertySelector
{
public:
	// ~Begin IPCGDataViewPropertySelector interface
	virtual TArray<FPCGAttributePropertySelector> GetSelection(const FPCGDataView& InDataView) const override;
	// ~End IPCGDataViewPropertySelector interface
};

class FPCGDataViewSplinePropertySelector : public FPCGDataViewPropertySelector
{
public:
	// ~Begin IPCGDataViewPropertySelector interface
	virtual TArray<FPCGAttributePropertySelector> GetSelection(const FPCGDataView& InDataView) const override;
	// ~End IPCGDataViewPropertySelector interface
};
