// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGSpatialData.h"

#include "Utils/PCGPointOctree.h"
#include "Utils/PCGValueRange.h"

#include "PCGBasePointData.generated.h"

#define UE_API PCG_API

namespace PCGPointDataConstants
{
	extern PCG_API const FName ActorReferenceAttribute;
	extern PCG_API const FName ElementsDomainName;
}

USTRUCT(BlueprintType)
struct FPCGPointInputRange
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	TObjectPtr<const UPCGBasePointData> PointData = nullptr;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	int32 RangeStartIndex = INDEX_NONE;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	int32 RangeSize = 0;
};

USTRUCT(BlueprintType)
struct FPCGPointOutputRange
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	TObjectPtr<UPCGBasePointData> PointData = nullptr;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	int32 RangeStartIndex = INDEX_NONE;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	int32 RangeSize = 0;
};

USTRUCT()
struct FPCGDataTypeInfoPoint : public FPCGDataTypeInfoConcrete
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Point)

	PCG_API virtual bool SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const override;
};

UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = (Procedural))
class UPCGBasePointData : public UPCGSpatialData
{
	GENERATED_BODY()

public:
	UE_API UPCGBasePointData(const FObjectInitializer& ObjectInitializer);

	// Get the functions to the accessor factory
	static UE_API FPCGAttributeAccessorMethods GetPointAccessorMethods();

	//~Begin UObject Interface
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~End UObject Interface

	// ~Begin UPCGSpatialData interface
	virtual int GetDimension() const override { return 0; }
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	UE_API virtual bool ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	// ~End UPCGSpatialData interface
		
	// ~Begin UPCGData Interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoPoint)
	
	/** Make a pass on Metadata to flatten parenting and only keep entries used by points. */
	UE_API virtual void Flatten() override;
	virtual bool SupportsFullDataCrc() const override { return true; }
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;

	/** Metadata specific */
	virtual FPCGMetadataDomainID GetDefaultMetadataDomainID() const override { return PCGMetadataDomainID::Elements; }
	virtual TArray<FPCGMetadataDomainID> GetAllSupportedMetadataDomainIDs() const override { return {PCGMetadataDomainID::Data, PCGMetadataDomainID::Elements}; }
	UE_API virtual FPCGMetadataDomainID GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const override;
	UE_API virtual bool SetDomainFromDomainID(const FPCGMetadataDomainID& InDomainID, FPCGAttributePropertySelector& InOutSelector) const;
	// ~End UPCGData Interface

	UE_API virtual bool ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata, bool bUseBounds) const;

	/** Initializes a single point based on the given actor */
	UE_API void InitializeFromActor(AActor* InActor, bool* bOutOptionalSanitizedTagAttributeName = nullptr);

	/** Adds a single point based on the given actor */
	UE_API void AddSinglePointFromActor(AActor* InActor, bool* bOutOptionalSanitizedTagAttributeName = nullptr);

	virtual bool IsValidRef(const PCGPointOctree::FPointRef& InPointRef) const PURE_VIRTUAL(UPCGBasePointData::IsValidRef, return false;);

	UFUNCTION(BlueprintCallable, Category = PointData, meta = (Keywords="Number"))
	virtual int32 GetNumPoints() const PURE_VIRTUAL(UPCGBasePointData::GetNumPoints, return 0;);

	UFUNCTION(BlueprintCallable, Category = PointData)
	bool IsEmpty() const { return GetNumPoints() == 0; }

	UFUNCTION(BlueprintCallable, Category = PointData, meta = (Keywords="Number"))
	virtual void SetNumPoints(int32 InNumPoints, bool bInitializeValues = true) PURE_VIRTUAL(UPCGBasePointData::SetNumPoints, return;);

	UFUNCTION(BlueprintCallable, Category = PointData, meta=(DisplayName="Allocate Properties"))
	void BP_AllocateProperties(UPARAM(meta = (BitMask, BitmaskEnum = "/Script/PCG.EPCGPointNativeProperties")) int32 Properties) { AllocateProperties((EPCGPointNativeProperties)Properties); }

	virtual void AllocateProperties(EPCGPointNativeProperties Properties) {}
	virtual void FreeProperties(EPCGPointNativeProperties Properties) {}
	virtual void MoveRange(int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements) PURE_VIRTUAL(UPCGBasePointData::MoveRange, return;);
	virtual void CopyUnallocatedPropertiesFrom(const UPCGBasePointData* InPointData) {}

	virtual TArray<FTransform> GetTransformsCopy() const PURE_VIRTUAL(UPCGBasePointData::GetTransformsCopy, return TArray<FTransform>(););
	
	/**
	 * GetValueRange / GetConstValueRange / Get*ValueRange / GetConst*ValueRange
	 * 
	 * Returns a value range that can be iterated and read from / written to abstracting the underlying point data structure.
	 * 
	 * Calling SetNumPoints/AllocateProperties/FreeProperties can and will probably invalidate ranges so make sure that you do those operations first or that you get a new range after you do.
	 * 
	 * @param bAllocate In the case where we return a non-const range by default the memory will be allocated but in some specific case we might not want to allocate (for SingleValue() ranges)
	 */

	// Keeping for backward compatibility (prefer using enum template version)
	template <typename T>
	TPCGValueRange<T> GetValueRange(EPCGPointNativeProperties NativeProperty, bool bAllocate = true);

	template<EPCGPointNativeProperties Property> 
	typename TPCGPointNativeProperty<Property>::ValueRange GetValueRange(bool bAllocate);

	virtual FPCGPointTransform::ValueRange GetTransformValueRange(bool bAllocate = true) PURE_VIRTUAL(UPCGBasePointData::GetTransformValueRange, return FPCGPointTransform::ValueRange(););
	virtual FPCGPointDensity::ValueRange GetDensityValueRange(bool bAllocate = true) PURE_VIRTUAL(UPCGBasePointData::GetDensityValueRange, return FPCGPointDensity::ValueRange(););
	virtual FPCGPointBoundsMin::ValueRange GetBoundsMinValueRange(bool bAllocate = true) PURE_VIRTUAL(UPCGBasePointData::GetBoundsMinValueRange, return FPCGPointBoundsMin::ValueRange(););
	virtual FPCGPointBoundsMax::ValueRange GetBoundsMaxValueRange(bool bAllocate = true) PURE_VIRTUAL(UPCGBasePointData::GetBoundsMaxValueRange, return FPCGPointBoundsMax::ValueRange(););
	virtual FPCGPointColor::ValueRange GetColorValueRange(bool bAllocate = true) PURE_VIRTUAL(UPCGBasePointData::GetColorValueRange, return FPCGPointColor::ValueRange(););
	virtual FPCGPointSteepness::ValueRange GetSteepnessValueRange(bool bAllocate = true) PURE_VIRTUAL(UPCGBasePointData::GetSteepnessValueRange, return FPCGPointSteepness::ValueRange(););
	virtual FPCGPointSeed::ValueRange GetSeedValueRange(bool bAllocate = true) PURE_VIRTUAL(UPCGBasePointData::GetSeedValueRange, return FPCGPointSeed::ValueRange(););
	virtual FPCGPointMetadataEntry::ValueRange GetMetadataEntryValueRange(bool bAllocate = true) PURE_VIRTUAL(UPCGBasePointData::GetMetadataEntryValueRange, return FPCGPointMetadataEntry::ValueRange(););

	// Keeping for backward compatibility (prefer using enum template version)
	template <typename T>
	TConstPCGValueRange<T> GetConstValueRange(EPCGPointNativeProperties NativeProperty) const;

	template<EPCGPointNativeProperties Property>
	typename TPCGPointNativeProperty<Property>::ConstValueRange GetConstValueRange() const;

	virtual FPCGPointTransform::ConstValueRange GetConstTransformValueRange() const PURE_VIRTUAL(UPCGBasePointData::GetConstTransformValueRange, return FPCGPointTransform::ConstValueRange(););
	virtual FPCGPointDensity::ConstValueRange GetConstDensityValueRange() const PURE_VIRTUAL(UPCGBasePointData::GetConstDensityValueRange, return FPCGPointDensity::ConstValueRange(););
	virtual FPCGPointBoundsMin::ConstValueRange GetConstBoundsMinValueRange() const PURE_VIRTUAL(UPCGBasePointData::GetConstBoundsMinValueRange, return FPCGPointBoundsMin::ConstValueRange(););
	virtual FPCGPointBoundsMax::ConstValueRange GetConstBoundsMaxValueRange() const PURE_VIRTUAL(UPCGBasePointData::GetConstBoundsMaxValueRange, return FPCGPointBoundsMax::ConstValueRange(););
	virtual FPCGPointColor::ConstValueRange GetConstColorValueRange() const PURE_VIRTUAL(UPCGBasePointData::GetConstColorValueRange, return FPCGPointColor::ConstValueRange(););
	virtual FPCGPointSteepness::ConstValueRange GetConstSteepnessValueRange() const PURE_VIRTUAL(UPCGBasePointData::GetConstSteepnessValueRange, return FPCGPointSteepness::ConstValueRange(););
	virtual FPCGPointSeed::ConstValueRange GetConstSeedValueRange() const PURE_VIRTUAL(UPCGBasePointData::GetConstSeedValueRange, return FPCGPointSeed::ConstValueRange(););
	virtual FPCGPointMetadataEntry::ConstValueRange GetConstMetadataEntryValueRange() const PURE_VIRTUAL(UPCGBasePointData::GetConstMetadataEntryValueRange, return FPCGPointMetadataEntry::ConstValueRange(););

	template<EPCGPointNativeProperties Property>
	void SetPointPropertyValues(const typename TPCGPointNativeProperty<Property>::Type& InValue)
	{
		FreeProperties(TPCGPointNativeProperty<Property>::EnumValue);
		typename TPCGPointNativeProperty<Property>::ValueRange PropertyValueRange = GetValueRange<TPCGPointNativeProperty<Property>::EnumValue>(/*bAllocate=*/false);
		PropertyValueRange.Set(InValue);
	}

	template<EPCGPointNativeProperties Property>
	const typename TPCGPointNativeProperty<Property>::Type& GetPointPropertyValue(int32 InPointIndex) const
	{
		const typename TPCGPointNativeProperty<Property>::ConstValueRange ConstPropertyValueRange = GetConstValueRange<TPCGPointNativeProperty<Property>::EnumValue>();
		if (ConstPropertyValueRange.IsValidIndex(InPointIndex))
		{
			return ConstPropertyValueRange[InPointIndex];
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Invalid point index : %d", InPointIndex);
			static typename TPCGPointNativeProperty<Property>::Type Invalid;
			return Invalid;
		}
	}

	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName = "Set All Transforms", ToolTip = "Set all transforms to same value"))
	UE_API virtual void SetTransform(const FTransform& InTransform);
	
	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName = "Set All Densities", ToolTip = "Set all densities to same value"))
	UE_API virtual void SetDensity(float InDensity);
	
	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName = "Set All Bounds Min", ToolTip = "Set all bounds min to same value"))
	UE_API virtual void SetBoundsMin(const FVector& InBoundsMin);
	
	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName = "Set All Bounds Max", ToolTip = "Set all bounds max to same value"))
	UE_API virtual void SetBoundsMax(const FVector& InBoundsMax);
	
	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName = "Set All Colors", ToolTip = "Set all colors to same value"))
	UE_API virtual void SetColor(const FVector4& InColor);
	
	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName = "Set All Steepness", ToolTip = "Set all steepness to same value"))
	UE_API virtual void SetSteepness(float InSteepness);
	
	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName = "Set All Seeds", ToolTip = "Set all seeds to same value"))
	UE_API virtual void SetSeed(int32 InSeed);
	
	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName = "Set All Metadata Entries", ToolTip = "Set all metadata entries to same value"))
	UE_API virtual void SetMetadataEntry(int64 InMetadataEntry);

	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName = "Set All Extents", ToolTip = "Set all extents to same value"))
	UE_API virtual void SetExtents(const FVector& InExtents);
	
	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName = "Set All Local Centers", ToolTip = "Set all local centers to same value"))
	UE_API virtual void SetLocalCenter(const FVector& InLocalCenter);

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual const FTransform& GetTransform(int32 InPointIndex) const;
	
	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual const float& GetDensity(int32 InPointIndex) const;
	
	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual const FVector& GetBoundsMin(int32 InPointIndex) const;
	
	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual const FVector& GetBoundsMax(int32 InPointIndex) const;
	
	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual const FVector4& GetColor(int32 InPointIndex) const;
	
	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual const float& GetSteepness(int32 InPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual const int32& GetSeed(int32 InPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual const int64& GetMetadataEntry(int32 InPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual FBoxSphereBounds GetDensityBounds(int32 InPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual FBox GetLocalDensityBounds(int32 InPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual FBox GetLocalBounds(int32 InPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual FVector GetLocalCenter(int32 InPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual FVector GetExtents(int32 InPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual FVector GetScaledExtents(int32 InPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual FVector GetLocalSize(int32 InPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = PointData)
	UE_API virtual FVector GetScaledLocalSize(int32 InPointIndex) const;
		
	/** Get the dirty status of the Octree. Note that the Point Octree can be rebuilt from another thread, so this info can be invalidated at anytime. */
	virtual bool IsPointOctreeDirty() const { return bOctreeIsDirty; }
	UE_API virtual const PCGPointOctree::FPointOctree& GetPointOctree() const;
	UE_API virtual FBox GetBounds() const override;
		
	UFUNCTION(BlueprintCallable, Category = PointData, meta = (DisplayName="Set Points From"))
	UE_API void BP_SetPointsFrom(const UPCGBasePointData* InData, const TArray<int32>& InDataIndices);
	
	UE_API void SetPointsFrom(const UPCGBasePointData* InData, const TArrayView<const int32>& InDataIndices);

	UE_API virtual void CopyPropertiesTo(UPCGBasePointData* To, int32 ReadStartIndex, int32 WriteStartIndex, int32 Count, EPCGPointNativeProperties Properties) const;
	UE_API virtual void CopyPropertiesTo(UPCGBasePointData* To, const TArrayView<const int32>& ReadIndices, const TArrayView<const int32>& WriteIndices, EPCGPointNativeProperties Properties) const;
	UE_API virtual void CopyPointsTo(UPCGBasePointData* To, int32 ReadStartIndex, int32 WriteStartIndex, int32 Count) const;
	UE_API virtual void CopyPointsTo(UPCGBasePointData* To, const TArrayView<const int32>& ReadIndices, const TArrayView<const int32>& WriteIndices) const;
	virtual EPCGPointNativeProperties GetAllocatedProperties(bool bWithInheritance = true) const { return EPCGPointNativeProperties::All; }

	static UE_API void SetPoints(const UPCGBasePointData* From, UPCGBasePointData* To, const TArrayView<const int32>& InDataIndices, bool bCopyAll);

	static EPCGPointNativeProperties GetPropertiesToAllocateFromPointData(const TArray<const UPCGBasePointData*>& PointDatas)
	{
		// Start by doing a union of all input allocated properties
		EPCGPointNativeProperties PropertiesToAllocate = EPCGPointNativeProperties::None;
		for (const UPCGBasePointData* PointInputData : PointDatas)
		{
			PropertiesToAllocate |= PointInputData->GetAllocatedProperties();
		}

		if (!EnumHasAnyFlags(PropertiesToAllocate, EPCGPointNativeProperties::Transform) && NeedToAllocateSingleValueProperty<EPCGPointNativeProperties::Transform>(PointDatas))
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::Transform;
		}

		if (!EnumHasAnyFlags(PropertiesToAllocate, EPCGPointNativeProperties::Density) && NeedToAllocateSingleValueProperty<EPCGPointNativeProperties::Density>(PointDatas))
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::Density;
		}

		if (!EnumHasAnyFlags(PropertiesToAllocate, EPCGPointNativeProperties::BoundsMin) && NeedToAllocateSingleValueProperty<EPCGPointNativeProperties::BoundsMin>(PointDatas))
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::BoundsMin;
		}

		if (!EnumHasAnyFlags(PropertiesToAllocate, EPCGPointNativeProperties::BoundsMax) && NeedToAllocateSingleValueProperty<EPCGPointNativeProperties::BoundsMax>(PointDatas))
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::BoundsMax;
		}

		if (!EnumHasAnyFlags(PropertiesToAllocate, EPCGPointNativeProperties::Color) && NeedToAllocateSingleValueProperty<EPCGPointNativeProperties::Color>(PointDatas))
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::Color;
		}

		if (!EnumHasAnyFlags(PropertiesToAllocate, EPCGPointNativeProperties::Steepness) && NeedToAllocateSingleValueProperty<EPCGPointNativeProperties::Steepness>(PointDatas))
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::Steepness;
		}

		if (!EnumHasAnyFlags(PropertiesToAllocate, EPCGPointNativeProperties::Seed) && NeedToAllocateSingleValueProperty<EPCGPointNativeProperties::Seed>(PointDatas))
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::Seed;
		}

		if (!EnumHasAnyFlags(PropertiesToAllocate, EPCGPointNativeProperties::MetadataEntry) && NeedToAllocateSingleValueProperty<EPCGPointNativeProperties::MetadataEntry>(PointDatas))
		{
			PropertiesToAllocate |= EPCGPointNativeProperties::MetadataEntry;
		}

		return PropertiesToAllocate;
	}

	UFUNCTION(BlueprintCallable, Category = PointData)
	static TArray<FTransform> GetTransformValuesFromRange(const FPCGPointInputRange& InputRange);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static TArray<float> GetDensityValuesFromRange(const FPCGPointInputRange& InputRange);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static TArray<FVector> GetBoundsMinValuesFromRange(const FPCGPointInputRange& InputRange);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static TArray<FVector> GetBoundsMaxValuesFromRange(const FPCGPointInputRange& InputRange);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static TArray<FVector4> GetColorValuesFromRange(const FPCGPointInputRange& InputRange);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static TArray<float> GetSteepnessValuesFromRange(const FPCGPointInputRange& InputRange);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static TArray<int32> GetSeedValuesFromRange(const FPCGPointInputRange& InputRange);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static TArray<int64> GetMetadataEntryValuesFromRange(const FPCGPointInputRange& InputRange);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static void SetTransformValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<FTransform>& Values);
		
	UFUNCTION(BlueprintCallable, Category = PointData)
	static void SetDensityValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<float>& Values);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static void SetBoundsMinValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<FVector>& Values);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static void SetBoundsMaxValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<FVector>& Values);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static void SetColorValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<FVector4>& Values);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static void SetSteepnessValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<float>& Values);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static void SetSeedValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<int32>& Values);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static void SetMetadataEntryValuesOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, const TArray<int64>& Values);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static FPCGPoint GetPointFromRange(const FPCGPointInputRange& InputRange, int32 Index);

	UFUNCTION(BlueprintCallable, Category = PointData)
	static void SetPointOnRange(UPARAM(ref) FPCGPointOutputRange& OutputRange, int32 Index, const FPCGPoint& Point);

protected:

	void RebuildOctreeIfNeeded() const
	{
		if (bOctreeIsDirty || !PCGPointOctree.IsValid())
		{
			RebuildOctree();
		}
	}

	UE_API void RebuildOctree() const;
	
	void RecomputeBoundsIfNeeded() const
	{
		if (bBoundsAreDirty)
		{
			RecomputeBounds();
		}
	}
	
	UE_API void RecomputeBounds() const;

	virtual void DirtyCache()
	{
		bOctreeIsDirty = true;
		bBoundsAreDirty = true;
	}

private:

	template<EPCGPointNativeProperties Property>
	static bool NeedToAllocateSingleValueProperty(const TArray<const UPCGBasePointData*>& PointDatas)
	{
		using PropertyType = typename TPCGPointNativeProperty<Property>::Type;
		// Here we are comparing single values (non-allocated properties) to see if we have single values that differ between inputs.
		// If they do differ, we need to allocate those properties because the output will have multiple values.
		TOptional<const PropertyType> DefaultValue;
		for (int32 i = 0; i < PointDatas.Num(); ++i)
		{
			const typename TPCGPointNativeProperty<Property>::ConstValueRange ValueRange = PointDatas[i]->GetConstValueRange<Property>();
			TOptional<const PropertyType> SingleValue = ValueRange.GetSingleValue();
			if (!DefaultValue.IsSet())
			{
				DefaultValue = SingleValue;
			}
			else if (DefaultValue.IsSet() && SingleValue.IsSet() && !PCG::Private::MetadataTraits<PropertyType>::Equal(DefaultValue.GetValue(), SingleValue.GetValue()))
			{
				return true;
			}
		}

		return false;
	}

protected:

	// Thread safety on Bounds and Octree is limited to lazy initialization, we do not support multi-threaded modification
	// of UPCGBasePointData while another thread is using the last generated Octree and/or Bounds.
	mutable PCG::FRecursiveLock CachedDataLock;
	mutable TUniquePtr<PCGPointOctree::FPointOctree> PCGPointOctree;
	mutable FBox Bounds;

	mutable bool bOctreeIsDirty = true;
	mutable bool bBoundsAreDirty = true;
};

template<EPCGPointNativeProperties Property>
typename TPCGPointNativeProperty<Property>::ValueRange UPCGBasePointData::GetValueRange(bool bAllocate)
{
	if constexpr (Property == EPCGPointNativeProperties::Transform)
	{
		return GetTransformValueRange(bAllocate);
	}
	else if constexpr (Property == EPCGPointNativeProperties::Density)
	{
		return GetDensityValueRange(bAllocate);
	}
	else if constexpr (Property == EPCGPointNativeProperties::Steepness)
	{
		return GetSteepnessValueRange(bAllocate);
	}
	else if constexpr (Property == EPCGPointNativeProperties::BoundsMin)
	{
		return GetBoundsMinValueRange(bAllocate);
	}
	else if constexpr (Property == EPCGPointNativeProperties::BoundsMax)
	{
		return GetBoundsMaxValueRange(bAllocate);
	}
	else if constexpr (Property == EPCGPointNativeProperties::Color)
	{
		return GetColorValueRange(bAllocate);
	}
	else if constexpr (Property == EPCGPointNativeProperties::Seed)
	{
		return GetSeedValueRange(bAllocate);
	}
	else if constexpr (Property == EPCGPointNativeProperties::MetadataEntry)
	{
		return GetMetadataEntryValueRange(bAllocate);
	}
	else
	{
		static_assert(TPCGPointUnsupportedProperty<Property>::Value, "Unsupported EPCGPointNativeProperties value");
		return TPCGPointNativeProperty<Property>::ValueRange();
	}
}

template<EPCGPointNativeProperties Property>
typename TPCGPointNativeProperty<Property>::ConstValueRange UPCGBasePointData::GetConstValueRange() const
{
	if constexpr (Property == EPCGPointNativeProperties::Transform)
	{
		return GetConstTransformValueRange();
	}
	else if constexpr (Property == EPCGPointNativeProperties::Density)
	{
		return GetConstDensityValueRange();
	}
	else if constexpr (Property == EPCGPointNativeProperties::Steepness)
	{
		return GetConstSteepnessValueRange();
	}
	else if constexpr (Property == EPCGPointNativeProperties::BoundsMin)
	{
		return GetConstBoundsMinValueRange();
	}
	else if constexpr (Property == EPCGPointNativeProperties::BoundsMax)
	{
		return GetConstBoundsMaxValueRange();
	}
	else if constexpr (Property == EPCGPointNativeProperties::Color)
	{
		return GetConstColorValueRange();
	}
	else if constexpr (Property == EPCGPointNativeProperties::Seed)
	{
		return GetConstSeedValueRange();
	}
	else if constexpr (Property == EPCGPointNativeProperties::MetadataEntry)
	{
		return GetConstMetadataEntryValueRange();
	}
	else
	{
		static_assert(TPCGPointUnsupportedProperty<Property>::Value, "Unsupported EPCGPointNativeProperties value");
		return TPCGPointNativeProperty<Property>::ConstValueRange();
	}
}

template <typename T>
inline TPCGValueRange<T> UPCGBasePointData::GetValueRange(EPCGPointNativeProperties NativeProperty, bool bAllocate)
{
	if constexpr (std::is_same_v<T, FTransform>)
	{
		if (NativeProperty == EPCGPointNativeProperties::Transform)
		{
			return GetTransformValueRange(bAllocate);
		}
	}
	else if constexpr (std::is_same_v<T, float>)
	{
		if (NativeProperty == EPCGPointNativeProperties::Density)
		{
			return GetDensityValueRange(bAllocate);
		}
		else if (NativeProperty == EPCGPointNativeProperties::Steepness)
		{
			return GetSteepnessValueRange(bAllocate);
		}
	}
	else if constexpr (std::is_same_v<T, FVector>)
	{
		if (NativeProperty == EPCGPointNativeProperties::BoundsMin)
		{
			return GetBoundsMinValueRange(bAllocate);
		}
		else if (NativeProperty == EPCGPointNativeProperties::BoundsMax)
		{
			return GetBoundsMaxValueRange(bAllocate);
		}
	}
	else if constexpr (std::is_same_v<T, FVector4>)
	{
		if (NativeProperty == EPCGPointNativeProperties::Color)
		{
			return GetColorValueRange(bAllocate);
		}
	}
	else if constexpr (std::is_same_v<T, int32>)
	{
		if (NativeProperty == EPCGPointNativeProperties::Seed)
		{
			return GetSeedValueRange(bAllocate);
		}
	}
	else if constexpr (std::is_same_v<T, int64>)
	{
		if (NativeProperty == EPCGPointNativeProperties::MetadataEntry)
		{
			return GetMetadataEntryValueRange(bAllocate);
		}
	}
	
	return TPCGValueRange<T>();
}

template <typename T>
inline TConstPCGValueRange<T> UPCGBasePointData::GetConstValueRange(EPCGPointNativeProperties NativeProperty) const
{
	if constexpr (std::is_same_v<T, FTransform>)
	{
		if (NativeProperty == EPCGPointNativeProperties::Transform)
		{
			return GetConstTransformValueRange();
		}
	}
	else if constexpr (std::is_same_v<T, float>)
	{
		if (NativeProperty == EPCGPointNativeProperties::Density)
		{
			return GetConstDensityValueRange();
		}
		else if (NativeProperty == EPCGPointNativeProperties::Steepness)
		{
			return GetConstSteepnessValueRange();
		}
	}
	else if constexpr (std::is_same_v<T, FVector>)
	{
		if (NativeProperty == EPCGPointNativeProperties::BoundsMin)
		{
			return GetConstBoundsMinValueRange();
		}
		else if (NativeProperty == EPCGPointNativeProperties::BoundsMax)
		{
			return GetConstBoundsMaxValueRange();
		}
	}
	else if constexpr (std::is_same_v<T, FVector4>)
	{
		if (NativeProperty == EPCGPointNativeProperties::Color)
		{
			return GetConstColorValueRange();
		}
	}
	else if constexpr (std::is_same_v<T, int32>)
	{
		if (NativeProperty == EPCGPointNativeProperties::Seed)
		{
			return GetConstSeedValueRange();
		}
	}
	else if constexpr (std::is_same_v <T, int64>)
	{
		if (NativeProperty == EPCGPointNativeProperties::MetadataEntry)
		{
			return GetConstMetadataEntryValueRange();
		}
	}
	
	return TConstPCGValueRange<T>();
}

struct FConstPCGPointValueRanges
{
	FConstPCGPointValueRanges() = default;

	FConstPCGPointValueRanges(const UPCGBasePointData* InBasePointData)
	{
		TransformRange = InBasePointData->GetConstTransformValueRange();
		DensityRange = InBasePointData->GetConstDensityValueRange();
		SteepnessRange = InBasePointData->GetConstSteepnessValueRange();
		BoundsMinRange = InBasePointData->GetConstBoundsMinValueRange();
		BoundsMaxRange = InBasePointData->GetConstBoundsMaxValueRange();
		ColorRange = InBasePointData->GetConstColorValueRange();
		SeedRange = InBasePointData->GetConstSeedValueRange();
		MetadataEntryRange = InBasePointData->GetConstMetadataEntryValueRange();
	}

	FPCGPoint GetPoint(int32 Index) const
	{
		FPCGPoint Point(TransformRange[Index], DensityRange[Index], SeedRange[Index]);
		Point.BoundsMin = BoundsMinRange[Index];
		Point.BoundsMax = BoundsMaxRange[Index];
		Point.Steepness = SteepnessRange[Index];
		Point.Color = ColorRange[Index];
		Point.MetadataEntry = MetadataEntryRange[Index];

		return Point;
	}

	TConstPCGValueRange<FTransform> TransformRange;
	TConstPCGValueRange<float> DensityRange;
	TConstPCGValueRange<float> SteepnessRange;
	TConstPCGValueRange<FVector> BoundsMinRange;
	TConstPCGValueRange<FVector> BoundsMaxRange;
	TConstPCGValueRange<FVector4> ColorRange;
	TConstPCGValueRange<int32> SeedRange;
	TConstPCGValueRange<int64> MetadataEntryRange;
};

struct FPCGPointValueRanges
{
	FPCGPointValueRanges() = default;

	FPCGPointValueRanges(UPCGBasePointData* InBasePointData, bool bAllocate = true)
	{
		EPCGPointNativeProperties AllocatedProperties = InBasePointData->GetAllocatedProperties(/*bWithInheritance=*/false);
		
		// In the very specific case where we have just a single point, that we don't inherit from a spatial parent (Orphan point data with a single point)
		// we can create a range that will allow to write into the default value of the point data.
		const bool bIsSingleOrphanPointData = !bAllocate && InBasePointData->GetNumPoints() == 1 && !InBasePointData->HasSpatialDataParent();
		const bool bMustCreateRange = bAllocate || bIsSingleOrphanPointData;

		if (bMustCreateRange || EnumHasAnyFlags(AllocatedProperties, EPCGPointNativeProperties::Transform))
		{
			TransformRange = InBasePointData->GetTransformValueRange(bAllocate);
		}

		if (bMustCreateRange || EnumHasAnyFlags(AllocatedProperties, EPCGPointNativeProperties::Density))
		{
			DensityRange = InBasePointData->GetDensityValueRange(bAllocate);
		}

		if (bMustCreateRange || EnumHasAnyFlags(AllocatedProperties, EPCGPointNativeProperties::Steepness))
		{
			SteepnessRange = InBasePointData->GetSteepnessValueRange(bAllocate);
		}

		if (bMustCreateRange || EnumHasAnyFlags(AllocatedProperties, EPCGPointNativeProperties::BoundsMin))
		{
			BoundsMinRange = InBasePointData->GetBoundsMinValueRange(bAllocate);
		}

		if (bMustCreateRange || EnumHasAnyFlags(AllocatedProperties, EPCGPointNativeProperties::BoundsMax))
		{
			BoundsMaxRange = InBasePointData->GetBoundsMaxValueRange(bAllocate);
		}

		if (bMustCreateRange || EnumHasAnyFlags(AllocatedProperties, EPCGPointNativeProperties::Color))
		{
			ColorRange = InBasePointData->GetColorValueRange(bAllocate);
		}

		if (bMustCreateRange || EnumHasAnyFlags(AllocatedProperties, EPCGPointNativeProperties::Seed))
		{
			SeedRange = InBasePointData->GetSeedValueRange(bAllocate);
		}

		if (bMustCreateRange || EnumHasAnyFlags(AllocatedProperties, EPCGPointNativeProperties::MetadataEntry))
		{
			MetadataEntryRange = InBasePointData->GetMetadataEntryValueRange(bAllocate);
		}
	}

	TPCGValueRange<FTransform> TransformRange;
	TPCGValueRange<float> DensityRange;
	TPCGValueRange<float> SteepnessRange;
	TPCGValueRange<FVector> BoundsMinRange;
	TPCGValueRange<FVector> BoundsMaxRange;
	TPCGValueRange<FVector4> ColorRange;
	TPCGValueRange<int32> SeedRange;
	TPCGValueRange<int64> MetadataEntryRange;

	void SetFromPoint(int32 Index, const FPCGPoint& Point)
	{
		if (!TransformRange.IsEmpty())
		{
			TransformRange[Index] = Point.Transform;
		}

		if (!DensityRange.IsEmpty())
		{
			DensityRange[Index] = Point.Density;
		}
		
		if (!SteepnessRange.IsEmpty())
		{
			SteepnessRange[Index] = Point.Steepness;
		}

		if (!BoundsMinRange.IsEmpty())
		{
			BoundsMinRange[Index] = Point.BoundsMin;
		}

		if (!BoundsMaxRange.IsEmpty())
		{
			BoundsMaxRange[Index] = Point.BoundsMax;
		}

		if (!ColorRange.IsEmpty())
		{
			ColorRange[Index] = Point.Color;
		}
		
		if (!SeedRange.IsEmpty())
		{
			SeedRange[Index] = Point.Seed;
		}

		if (!MetadataEntryRange.IsEmpty())
		{
			MetadataEntryRange[Index] = Point.MetadataEntry;
		}
	}
		
	void SetFromValueRanges(int32 WriteIndex, const FConstPCGPointValueRanges& ReadRange, int32 ReadIndex)
	{
		if (!TransformRange.IsEmpty())
		{
			TransformRange[WriteIndex] = ReadRange.TransformRange[ReadIndex];
		}

		if (!DensityRange.IsEmpty())
		{
			DensityRange[WriteIndex] = ReadRange.DensityRange[ReadIndex];
		}

		if (!SteepnessRange.IsEmpty())
		{
			SteepnessRange[WriteIndex] = ReadRange.SteepnessRange[ReadIndex];
		}

		if (!BoundsMinRange.IsEmpty())
		{
			BoundsMinRange[WriteIndex] = ReadRange.BoundsMinRange[ReadIndex];
		}

		if (!BoundsMaxRange.IsEmpty())
		{
			BoundsMaxRange[WriteIndex] = ReadRange.BoundsMaxRange[ReadIndex];
		}

		if (!ColorRange.IsEmpty())
		{
			ColorRange[WriteIndex] = ReadRange.ColorRange[ReadIndex];
		}

		if (!SeedRange.IsEmpty())
		{
			SeedRange[WriteIndex] = ReadRange.SeedRange[ReadIndex];
		}

		if (!MetadataEntryRange.IsEmpty())
		{
			MetadataEntryRange[WriteIndex] = ReadRange.MetadataEntryRange[ReadIndex];
		}
	}
};

#undef UE_API
