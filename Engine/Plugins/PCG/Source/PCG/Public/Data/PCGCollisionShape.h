// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollisionShape.h"

#include "PCGCollisionShape.generated.h"

#define UE_API PCG_API

struct FPCGContext;

UENUM(BlueprintType)
enum class EPCGCollisionShapeType : uint8
{
	Line,
	Box,
	Sphere,
	Capsule
};

/** Parameters for conducting a sweep with a specified shape against the physical world. */
USTRUCT(BlueprintType)
struct FPCGCollisionShape
{
	GENERATED_BODY()

	FPCGCollisionShape() = default;
	UE_API explicit FPCGCollisionShape(FCollisionShape InShape, const FPCGContext* InOptionalContext = nullptr);

	/** Shape that will be used in the collision detection. */
	UPROPERTY(BlueprintType, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGCollisionShapeType ShapeType = EPCGCollisionShapeType::Line;

	/** Half the size of the sweep box's X, Y, Z components in world space. */
	UPROPERTY(BlueprintType, EditAnywhere, Category = Settings, DisplayName = "Half Extent (Box)", meta = (EditCondition = "ShapeType == EPCGCollisionShapeType::Box", EditConditionHides, PCG_Overridable))
	FVector BoxHalfExtent = FVector::OneVector;

	/** Radius of the sweep's sphere in world space. */
	UPROPERTY(BlueprintType, EditAnywhere, Category = Settings, DisplayName = "Radius (Sphere)", meta = (EditCondition = "ShapeType == EPCGCollisionShapeType::Sphere", EditConditionHides, PCG_Overridable))
	float SphereRadius = 100.f;

	/** Radius of the spherical shape of the sweep's capsule in world space. */
	UPROPERTY(BlueprintType, EditAnywhere, Category = Settings, DisplayName = "Radius (Capsule)", meta = (EditCondition = "ShapeType == EPCGCollisionShapeType::Capsule", EditConditionHides, PCG_Overridable))
	float CapsuleRadius = 100.f;

	/** Half the length of the sweep's capsule in world space. */
	UPROPERTY(BlueprintType, EditAnywhere, Category = Settings, DisplayName = "Half Height (Capsule)", meta = (EditCondition = "ShapeType == EPCGCollisionShapeType::Capsule", EditConditionHides, PCG_Overridable))
	float CapsuleHalfHeight = 100.f;

	/** World space rotation applied to the collision shape prior to the sweep. */
	UPROPERTY(BlueprintType, EditAnywhere, Category = Settings, meta = (EditCondition = "ShapeType != EPCGCollisionShapeType::Line", EditConditionHides, PCG_Overridable))
	FRotator ShapeRotation = FRotator::ZeroRotator;

	UE_API FCollisionShape ToCollisionShape(const FPCGContext* InOptionalContext = nullptr) const;
};

#undef UE_API
