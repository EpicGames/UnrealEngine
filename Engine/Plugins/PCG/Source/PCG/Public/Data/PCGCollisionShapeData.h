// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPrimitiveData.h"
#include "PCGSpatialData.h"

#include "CollisionShape.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Physics/PhysicsInterfaceDeclares.h"

#include "PCGCollisionShapeData.generated.h"

class UShapeComponent;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCollisionShapeData : public UPCGSpatialDataWithPointCache
{
	GENERATED_BODY()

	friend class FPCGCollisionShapeDataVisualization;

public:
	PCG_API void Initialize(UShapeComponent* InShape);
	PCG_API static bool IsSupported(UShapeComponent* InShape);

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoPrimitive)
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	// ~Begin UPCGSpatialData interface
	virtual int GetDimension() const override { return 3; }
	virtual FBox GetBounds() const override { return CachedBounds; }
	virtual FBox GetStrictBounds() const override { return CachedStrictBounds; }
	virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	// TODO needs an implementation to support projection
	//virtual bool ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	// ~Begin UPCGSpatialDataWithPointCache implementation
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	// ~End UPCGSpatialDataWithPointCache implementation

	const FTransform& GetTransform() const { return Transform; }
	const FCollisionShape& GetCollisionShape() const { return Shape; }

private:
	virtual const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;

protected:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	FTransform Transform;

	// If we want to serialize this data, this will need to be broken up, and the shape adapter will need to be recreated as well
	FCollisionShape Shape;

	// Built from shape & transform, not serialized
	TUniquePtr<FPhysicsShapeAdapter> ShapeAdapter;

	UPROPERTY()
	FBox CachedBounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	FBox CachedStrictBounds = FBox(EForceInit::ForceInit);
};
