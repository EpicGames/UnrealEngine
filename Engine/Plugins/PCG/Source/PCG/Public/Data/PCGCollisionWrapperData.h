// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPrimitiveData.h"
#include "PCGSpatialData.h"
#include "Data/PCGBasePointData.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#include "Chaos/ChaosEngineInterface.h"

#include "PCGCollisionWrapperData.generated.h"

#define UE_API PCG_API

struct FBodyInstance;

UENUM(BlueprintType)
enum class EPCGCollisionQueryFlag : uint8
{
	Simple,
	Complex,
	SimpleFirst,
	ComplexFirst
};

struct FPCGCollisionWrapper
{
	FPCGCollisionWrapper() = default;
	UE_API ~FPCGCollisionWrapper();
	FPCGCollisionWrapper(const FPCGCollisionWrapper&) = delete;
	FPCGCollisionWrapper(FPCGCollisionWrapper&& Other) = default;
	FPCGCollisionWrapper& operator=(const FPCGCollisionWrapper&) = delete;
	FPCGCollisionWrapper& operator=(FPCGCollisionWrapper&& Other) = default;

	// Simple API - does both the prepare & create body instances in a single step
	UE_API bool Initialize(const IPCGAttributeAccessor* Accessor, const IPCGAttributeAccessorKeys* Keys);
	UE_API void Uninitialize();

	// Advanced API - allows to do async loading as we separate the mesh finding part from the body creation part
	UE_API bool Prepare(const IPCGAttributeAccessor* Accessor, const IPCGAttributeAccessorKeys* Keys, TArray<FSoftObjectPath>& MeshPathsToLoad);
	UE_API void CreateBodyInstances(const TArray<FSoftObjectPath>& MeshPaths);
	
	// Retrieves the body instance associated to the entry given by its index
	UE_API FBodyInstance* GetBodyInstance(int32 EntryIndex) const;

	// Retrieves the shape list for a given entry matching the query flag
	UE_API void GetShapeArray(int32 EntryIndex, EPCGCollisionQueryFlag QueryFlag, PhysicsInterfaceTypes::FInlineShapeArray& OutShapeArray) const;

	// Retrieves the shape list for a given body, matching the query flag. Returns false if we selected the other type for the 'SimpleFirst' or 'ComplexFirst' cases.
	static UE_API bool GetShapeArray(FBodyInstance* BodyInstance, EPCGCollisionQueryFlag QueryFlag, PhysicsInterfaceTypes::FInlineShapeArray& OutShapeArray);

	TArray<FBodyInstance*> BodyInstances;
	TArray<int32> IndexToBodyInstance;
	bool bInitialized = false;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCollisionWrapperData : public UPCGSpatialData
{
	GENERATED_BODY()

	friend class FPCGCollisionWrapperDataVisualization;

public:
	/** Inititializes the collision wrapper on a point data based on the provided attribute selector */
	PCG_API bool Initialize(const UPCGBasePointData* InPointData, const FPCGAttributePropertyInputSelector& InCollisionSelector, EPCGCollisionQueryFlag InCollisionQueryFlag);

	/** Advanced API for async loading */
	bool PreInitializeAndGatherMeshesEx(const UPCGBasePointData* InPointData, const FPCGAttributePropertyInputSelector& InCollisionSelector, EPCGCollisionQueryFlag InCollisionQueryFlag, TArray<FSoftObjectPath>& OutMeshesToLoad);
	void FinalizeInitializationEx(const TArray<FSoftObjectPath>& InMeshPaths);

	// ~Begin UPCGData Interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoPrimitive)
	virtual FPCGCrc ComputeCrc(bool bFullDataCrc) const override;
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	// ~End UPCGData Interface

	// ~Begin UPCGSpatialData interface
	virtual int GetDimension() const override { return 3; }
	virtual FBox GetBounds() const override;
	virtual FBox GetStrictBounds() const override;
	virtual const UPCGPointData* ToPointData(FPCGContext* Context, const FBox& InBounds = FBox(EForceInit::ForceInit)) const override;
	virtual const UPCGPointArrayData* ToPointArrayData(FPCGContext* Context, const FBox& InBounds = FBox(EForceInit::ForceInit)) const override;
	virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	// ~End UPCGSpatialData interface

public:
	// For performance reasons, we keep a raw pointer to the point data in editor.
#if WITH_EDITOR
	inline const UPCGBasePointData* GetPointData() const { return RawPointData; }
#else
	inline const UPCGBasePointData* GetPointData() const { return PointData.Get(); }
#endif

	// Some helper functions when this is used in different contexts
	EPCGCollisionQueryFlag GetCollisionFlag() const { return CollisionQueryFlag; }
	const FPCGCollisionWrapper& GetCollisionWrapper() const { return CollisionWrapper; }

private:
	const PhysicsInterfaceTypes::FInlineShapeArray& GetCachedShapes(int32 EntryIndex) const;

	UPROPERTY()
	TObjectPtr<const UPCGBasePointData> PointData;

	// Implementation note: in order to be able to duplicate this easily, we're keeping track of the arguments we used when calling Initialize (& derived functions)
	UPROPERTY()
	FPCGAttributePropertyInputSelector CollisionSelector;

	UPROPERTY()
	EPCGCollisionQueryFlag CollisionQueryFlag = EPCGCollisionQueryFlag::Simple;

	FPCGCollisionWrapper CollisionWrapper;

	TArray<PhysicsInterfaceTypes::FInlineShapeArray> CachedShapes;

#if WITH_EDITOR
	const UPCGBasePointData* RawPointData = nullptr;
#endif
};

#undef UE_API
