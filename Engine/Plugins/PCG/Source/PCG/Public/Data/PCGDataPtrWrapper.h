// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#include "UObject/ObjectPtr.h"

#include "PCGDataPtrWrapper.generated.h"

#define UE_API PCG_API

class UPCGData;

/** Wrapper to do ref-counting inside of FPCGDataCollection, so we can release temporary resources earlier. */
USTRUCT()
struct FPCGDataPtrWrapper
{
	friend struct FPCGDataCollection;

	GENERATED_BODY()

	FPCGDataPtrWrapper() = default;
	UE_API ~FPCGDataPtrWrapper();
	UE_API FPCGDataPtrWrapper(const FPCGDataPtrWrapper& InWrapper);
	UE_API FPCGDataPtrWrapper(FPCGDataPtrWrapper&& InWrapper);
	UE_API FPCGDataPtrWrapper& operator=(const FPCGDataPtrWrapper& InWrapper);
	UE_API FPCGDataPtrWrapper& operator=(FPCGDataPtrWrapper&& InWrapper);

	UE_API FPCGDataPtrWrapper(const UPCGData* InData);
	UE_API FPCGDataPtrWrapper& operator=(const UPCGData* InData);

	operator const UPCGData* () const { return Data; }
	const UPCGData* operator->() const { return Data; }
	const UPCGData* Get() const { return Data; }
	const TObjectPtr<const UPCGData> GetObjectPtr() const { return Data; }

	template<typename Class>
	bool IsA() const
	{
		return Data && Data.IsA<Class>();
	}

	UE_API bool Serialize(FArchive& Ar);
	UE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

protected:
	static UE_API void IncRefCount(const UPCGData* InData);
	static UE_API void DecRefCount(const TWeakObjectPtr<const UPCGData>& InWeakData);

	UE_API void IncRefCount();
	UE_API void DecRefCount();

protected:
	UPROPERTY(VisibleAnywhere, Category = Data)
	TObjectPtr<const UPCGData> Data = nullptr;

	// Weakptr to be able to make sure we don't try to decrement collection ref count if the data was already destroyed.
	// Normally, we don't really need it, except that in some cases the order of deletion isn't easily guaranteed, and enforcing BeginDestroy implementation
	// in every user of the FPCGDataCollection class seems like a tall order.
	TWeakObjectPtr<const UPCGData> WeakData;
};

template<>
struct TStructOpsTypeTraits<FPCGDataPtrWrapper> : public TStructOpsTypeTraitsBase2<FPCGDataPtrWrapper>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
		WithSerializer = true,
		WithImportTextItem = true
	};
};

template<typename To>
inline TCopyQualifiersFromTo_T<const TObjectPtr<const UPCGData>, To>* Cast(const FPCGDataPtrWrapper& InSrc)
{
	return Cast<To>(InSrc.GetObjectPtr());
}

template<typename To>
inline TCopyQualifiersFromTo_T<const TObjectPtr<const UPCGData>, To>* CastChecked(const FPCGDataPtrWrapper& InSrc, ECastCheckedType::Type CheckType = ECastCheckedType::NullChecked)
{
	return CastChecked<To>(InSrc.GetObjectPtr(), CheckType);
}

#undef UE_API
