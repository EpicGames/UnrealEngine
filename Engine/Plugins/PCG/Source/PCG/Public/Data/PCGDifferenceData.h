// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialData.h"

#include "PCGDifferenceData.generated.h"

#define UE_API PCG_API

struct FPropertyChangedEvent;

class UPCGUnionData;

UENUM()
enum class EPCGDifferenceDensityFunction : uint8
{
	Minimum,
	ClampedSubstraction UMETA(DisplayName = "Clamped Subtraction"),
	Binary
};

UENUM()
enum class EPCGDifferenceMode : uint8
{
	Inferred,
	Continuous,
	Discrete
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDifferenceData : public UPCGSpatialDataWithPointCache
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	UE_API void Initialize(const UPCGSpatialData* InData);

	UFUNCTION(BlueprintCallable, Category = SpatialData, meta = (DisplayName = "Add Difference"))
	UE_API void K2_AddDifference(const UPCGSpatialData* InDifference);

	UE_DEPRECATED(5.5, "Call/Implement version with FPCGContext parameter")
	void AddDifference(const UPCGSpatialData* InDifference) { AddDifference(nullptr, InDifference); }

	UE_API void AddDifference(FPCGContext* InContext, const UPCGSpatialData* InDifference);

	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetDensityFunction(EPCGDifferenceDensityFunction InDensityFunction);

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	bool bDiffMetadata = true;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	// ~Begin UObject interface
#if WITH_EDITOR
	UE_API virtual void PostLoad();
#endif
	// ~End UObject interface

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoComposite)
	UE_API virtual void VisitDataNetwork(TFunctionRef<void(const UPCGData*)> Action) const override;

protected:
	UE_API virtual FPCGCrc ComputeCrc(bool bFullDataCrc) const override;
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

public:
	//~Begin UPCGSpatialData interface
	UE_API virtual int GetDimension() const override;
	UE_API virtual FBox GetBounds() const override;
	UE_API virtual FBox GetStrictBounds() const override;
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	UE_API virtual bool HasNonTrivialTransform() const override;
	virtual const UPCGSpatialData* FindFirstConcreteShapeFromNetwork() const override { return GetSource() ? GetSource()->FindFirstConcreteShapeFromNetwork() : nullptr; }
	UE_API virtual void InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const override;
protected:
	UE_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	//~Begin UPCGSpatialDataWithPointCache interface
	UE_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialDataWithPointCache interface
protected:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	TObjectPtr<const UPCGSpatialData> Source = nullptr;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	TObjectPtr<const UPCGSpatialData> Difference = nullptr;

	UPROPERTY()
	TObjectPtr<UPCGUnionData> DifferencesUnion = nullptr;

	UPROPERTY(BlueprintSetter = SetDensityFunction, EditAnywhere, Category = Settings)
	EPCGDifferenceDensityFunction DensityFunction = EPCGDifferenceDensityFunction::Minimum;

	UE_API const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, const UPCGBasePointData* SourcePointData, TSubclassOf<UPCGBasePointData> PointDataClass) const;

#if WITH_EDITOR
	inline const UPCGSpatialData* GetSource() const { return RawPointerSource; }
	inline const UPCGSpatialData* GetDifference() const { return RawPointerDifference; }
	inline UPCGUnionData* GetDifferencesUnion() const { return RawPointerDifferencesUnion; }
#else
	inline const UPCGSpatialData* GetSource() const { return Source.Get(); }
	inline const UPCGSpatialData* GetDifference() const { return Difference.Get(); }
	inline UPCGUnionData* GetDifferencesUnion() const { return DifferencesUnion.Get(); }
#endif

#if WITH_EDITOR
private:
	// Cached pointers to avoid dereferencing object pointer which does access tracking and supports lazy loading, and can come with substantial
	// overhead (add trace marker to FObjectPtr::Get to see).
	const UPCGSpatialData* RawPointerSource = nullptr;
	const UPCGSpatialData* RawPointerDifference = nullptr;
	UPCGUnionData* RawPointerDifferencesUnion = nullptr;
#endif
};

#undef UE_API
