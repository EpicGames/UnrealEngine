// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGSpatialData.h"

#include "DynamicMesh/DynamicMeshOctree3.h"
#include "DynamicMesh/MeshIndexMappings.h"

#include "Misc/SpinLock.h"

#include "PCGDynamicMeshData.generated.h"

struct FPCGContext;
class UDynamicMesh;
class UDynamicMeshComponent;
class UMaterialInterface;

USTRUCT()
struct FPCGDataTypeInfoDynamicMesh: public FPCGDataTypeInfoConcrete
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::DynamicMesh);
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDynamicMeshData : public UPCGSpatialData
{
	GENERATED_BODY()

public:
	PCG_API UPCGDynamicMeshData(const FObjectInitializer& ObjectInitializer);
	
	PCG_API void Initialize(UDynamicMesh* InMesh, bool bCanTakeOwnership = false, const TArray<UMaterialInterface*>& InOptionalMaterials = {});
	PCG_API void Initialize(UE::Geometry::FDynamicMesh3&& InMesh, const TArray<UMaterialInterface*>& InOptionalMaterials = {});

	/**
	 * Initialize the dynamic mesh data from an input dynamic mesh object.
	 * If the input dynamic mesh is not meant to be re-used after this initialization, you can set Can Take Ownership to true. Be careful as it
	 * will put the previous object in an invalid state.
	 * You can also pass an array of materials that correspond to the referenced materials in the dynamic mesh.
	 */
	UFUNCTION(BlueprintCallable, Category="DynamicMesh", meta = (DisplayName = "Initialize", AutoCreateRefTerm = "InMaterials"))
	void K2_Initialize(UDynamicMesh* InMesh, const TArray<UMaterialInterface*>& InMaterials, bool bCanTakeOwnership = false) { Initialize(InMesh, bCanTakeOwnership, InMaterials); }
	
	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoDynamicMesh)
	PCG_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	// ~Begin UPCGSpatialData interface
	virtual int GetDimension() const override { return 3; }
	PCG_API virtual FBox GetBounds() const override;
	PCG_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	// TODO needs an implementation to support projection
	//virtual bool ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;
	//~End UPCGSpatialData interface

	PCG_API const UE::Geometry::FDynamicMeshOctree3& GetDynamicMeshOctree() const;
	
	UDynamicMesh* GetMutableDynamicMesh() { bDynamicMeshBoundsAreDirty = true; bDynamicMeshOctreeIsDirty = true; return DynamicMesh; }
	const UDynamicMesh* GetDynamicMesh() const { return DynamicMesh; }

	UFUNCTION(BlueprintCallable, Category="DynamicMesh")
	PCG_API void SetMaterials(const TArray<UMaterialInterface*>& InMaterials);

	PCG_API void SetMaterials(TArray<UMaterialInterface*>&& InMaterials);

	TArray<TObjectPtr<UMaterialInterface>>& GetMutableMaterials() { return Materials; }
	const TArray<TObjectPtr<UMaterialInterface>>& GetMaterials() const { return Materials; }

	// Copy the mesh of the data into the component and set the materials.
	PCG_API void InitializeDynamicMeshComponentFromData(UDynamicMeshComponent* InComponent) const;

protected:
	// ~Begin UPCGData interface
	virtual bool SupportsFullDataCrc() const override { return true; }
	// ~End UPCGData interface

	// ~Begin UPCGSpatialData interface
	PCG_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
public:
	PCG_API virtual const UPCGPointData* ToPointData(FPCGContext* Context, const FBox& InBounds) const override;
	PCG_API virtual const UPCGPointArrayData* ToPointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialData interface

private:
	// const but will set the mutable CachedBounds
	void ResetBounds() const;
	const UPCGBasePointData* ToBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const;

protected:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Instanced, Category="DynamicMesh")
	TObjectPtr<UDynamicMesh> DynamicMesh;
	
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Instanced, Category="DynamicMesh")
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	mutable UE::Geometry::FDynamicMeshOctree3 DynamicMeshOctree;
	mutable bool bDynamicMeshOctreeIsDirty = true;
	mutable PCG::FLock DynamicMeshOctreeLock;
	
	mutable FBox CachedBounds = FBox(EForceInit::ForceInit);
	mutable bool bDynamicMeshBoundsAreDirty = true;
	mutable PCG::FLock DynamicMeshBoundsLock;
};

#undef UE_API
