// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.7, "This header has moved to Data/Registry/PCGGetDataFunctionRegistry.h")

#include "Data/Registry/PCGGetDataFunctionRegistry.h"