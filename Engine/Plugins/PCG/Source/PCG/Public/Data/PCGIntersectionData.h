// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialData.h"

#include "PCGIntersectionData.generated.h"

#define UE_API PCG_API

UENUM()
enum class EPCGIntersectionDensityFunction : uint8
{
	Multiply UMETA(ToolTip="Multiplies the density values and results in the product."),
	Minimum UMETA(ToolTip="Chooses the minimum of the density values.")
};

/**
* Generic intersection class that delays operations as long as possible.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGIntersectionData : public UPCGSpatialDataWithPointCache
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	UE_API void Initialize(const UPCGSpatialData* InA, const UPCGSpatialData* InB);

	// ~Begin UObject interface
#if WITH_EDITOR
	UE_API virtual void PostLoad();
#endif
	// ~End UObject interface

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoComposite)
	UE_API virtual void VisitDataNetwork(TFunctionRef<void(const UPCGData*)> Action) const override;

protected:
	UE_API virtual FPCGCrc ComputeCrc(bool bFullDataCrc) const override;
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

public:
	//~Begin UPCGSpatialData interface
	UE_API virtual int GetDimension() const override;
	UE_API virtual FBox GetBounds() const override;
	UE_API virtual FBox GetStrictBounds() const override;
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	UE_API virtual bool HasNonTrivialTransform() const override;
	UE_API virtual const UPCGSpatialData* FindFirstConcreteShapeFromNetwork() const override;
	UE_API virtual void InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const override;
protected:
	UE_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	//~Begin UPCGSpatialDataWithPointCache interface
	UE_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialDataWithPointCache interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGIntersectionDensityFunction DensityFunction = EPCGIntersectionDensityFunction::Multiply;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	TObjectPtr<const UPCGSpatialData> A = nullptr;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	TObjectPtr<const UPCGSpatialData> B = nullptr;

protected:
	UE_API UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;
	UE_API UPCGBasePointData* CreateAndFilterPointData(FPCGContext* Context, const UPCGSpatialData* X, const UPCGSpatialData* Y, TSubclassOf<UPCGBasePointData> PointDataClass) const;

	UPROPERTY()
	FBox CachedBounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	FBox CachedStrictBounds = FBox(EForceInit::ForceInit);

#if WITH_EDITOR
	inline const UPCGSpatialData* GetA() const { return RawPointerA; }
	inline const UPCGSpatialData* GetB() const { return RawPointerB; }
#else
	inline const UPCGSpatialData* GetA() const { return A.Get(); }
	inline const UPCGSpatialData* GetB() const { return B.Get(); }
#endif

#if WITH_EDITOR
private:
	// Cached pointers to avoid dereferencing object pointer which does access tracking and supports lazy loading, and can come with substantial
	// overhead (add trace marker to FObjectPtr::Get to see).
	const UPCGSpatialData* RawPointerA = nullptr;
	const UPCGSpatialData* RawPointerB = nullptr;
#endif
};

#undef UE_API
