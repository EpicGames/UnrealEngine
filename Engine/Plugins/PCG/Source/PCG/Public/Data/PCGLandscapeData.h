// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSurfaceData.h"

#include "PCGLandscapeData.generated.h"

#define UE_API PCG_API

class UPCGSpatialData;
struct FPCGProjectionParams;

class ALandscapeProxy;
class ULandscapeInfo;
class UPCGLandscapeCache;

USTRUCT(BlueprintType)
struct FPCGLandscapeDataProps
{
	GENERATED_BODY()

	/** Controls whether the points projected on the landscape will return the normal/tangent (if false) or only the position (if true) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bGetHeightOnly = false;

	/** Controls whether data from landscape layers will be retrieved (turning it off is an optimization if that data is not needed) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bGetLayerWeights = true;

	/** Controls whether the points from this landscape will return the actor from which they originate (e.g. which Landscape Proxy) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bGetActorReference = false;

	/** Controls whether the points from the landscape will have their physical material added as the "PhysicalMaterial" attribute */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bGetPhysicalMaterial = false;

	/** Controls whether the component coordinates will be added the point as attributes ('CoordinateX', 'CoordinateY') */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bGetComponentCoordinates = false;

	/** Controls whether the landscape will try to sample from the landscape virtual textures (if they exist). Only relevant to GPU sampling. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = GPU, meta = (PCG_Overridable))
	bool bSampleVirtualTextures = true;

	/**
	 * Controls whether the landscape will try to sample normals from a normals virtual texture (if it exists), otherwise computes normals from multiple height samples. Only relevant to GPU sampling.
	 * Note that normal virtual textures may be detail normals and not match the actual landscape surface normals, so enable this with caution. Requires bSampleVirtualTextures to be true.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = GPU, meta = (PCG_Overridable, EditCondition = "bSampleVirtualTextures", EditConditionHides))
	bool bSampleVirtualTextureNormals = false;
};


USTRUCT()
struct FPCGDataTypeInfoLandscape : public FPCGDataTypeInfoSurface
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Landscape)
};

/**
* Landscape data access abstraction for PCG. Supports multi-landscape access, but it assumes that they are not overlapping.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGLandscapeData : public UPCGSurfaceData
{
	GENERATED_BODY()

public:
	UE_API void Initialize(const TArray<TWeakObjectPtr<ALandscapeProxy>>& InLandscapes, const FBox& InBounds, const FPCGLandscapeDataProps& InDataProps);

	// ~Begin UObject interface
	UE_API virtual void PostLoad();
	// ~End UObject interface

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoLandscape)
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	// ~Begin UPGCSpatialData interface
	UE_API virtual FBox GetBounds() const override;
	UE_API virtual FBox GetStrictBounds() const override;
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	UE_API virtual void SamplePoints(const TArrayView<const TPair<FTransform, FBox>>& Samples, const TArrayView<FPCGPoint>& OutPoints, UPCGMetadata* OutMetadata) const override;
	UE_API virtual bool ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	virtual bool HasNonTrivialTransform() const override { return true; }
	UE_API virtual TArray<FPCGTaskId> PrepareForSpatialQuery(FPCGContext* InContext, const FBox& InBounds) const override;
	UE_API virtual void InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const override;
protected:
	UE_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	// ~Begin UPCGSpatialDataWithPointCache interface
	virtual bool SupportsBoundedPointData() const { return true; }
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override { return CreatePointData(Context, FBox(EForceInit::ForceInit)); }
	UE_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context, const FBox& InBounds) const override;
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	// ~End UPCGSpatialDataWithPointCache interface

	// TODO: add on property changed to clear cached data. This is used to populate the LandscapeInfos array.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SourceData)
	TArray<TSoftObjectPtr<ALandscapeProxy>> Landscapes;

	bool IsUsingMetadata() const { return DataProps.bGetLayerWeights; }
	bool CanSampleVirtualTextures() const { return DataProps.bSampleVirtualTextures; }
	bool CanSampleVirtualTextureNormals() const { return CanSampleVirtualTextures() && DataProps.bSampleVirtualTextureNormals; }

protected:
	/** Returns the landscape info associated to the first landscape that contains the given position
	* Note that this implicitly removes support for overlapping landscapes, which might be a future TODO
	*/
	UE_API const ULandscapeInfo* GetLandscapeInfo(const FVector& InPosition) const;

	UE_API const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const;

	UPROPERTY()
	FBox Bounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	FPCGLandscapeDataProps DataProps;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bHeightOnly_DEPRECATED = false;

	UPROPERTY()
	bool bUseMetadata_DEPRECATED = true;
#endif // WITH_EDITORONLY_DATA

private:
	bool UseMetadata() const;

	// Transient data
	TArray<TPair<FBox, ULandscapeInfo*>> BoundsToLandscapeInfos;
	TArray<ULandscapeInfo*> LandscapeInfos;
	UPCGLandscapeCache* LandscapeCache = nullptr;
};

#undef UE_API
