// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPolyLineData.h"

#include "PCGLandscapeSplineData.generated.h"

#define UE_API PCG_API

class UPCGSpatialData;

class ULandscapeSplinesComponent;

USTRUCT()
struct FPCGDataTypeInfoLandscapeSpline : public FPCGDataTypeInfoPolyline
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::LandscapeSpline)
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGLandscapeSplineData : public UPCGPolyLineData
{
	GENERATED_BODY()
public:
	UE_API void Initialize(ULandscapeSplinesComponent* InSplineComponent);

	// ~Begin UObject interface
	UE_API virtual void PostLoad() override;
	// ~End UObject interface

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoLandscapeSpline)
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	//~Begin UPCGPolyLineData interface
	UE_API virtual FTransform GetTransform() const override;
	UE_API virtual int GetNumSegments() const override;
	UE_API virtual FVector::FReal GetSegmentLength(int SegmentIndex) const override;
	UE_API virtual FTransform GetTransformAtDistance(int SegmentIndex, FVector::FReal Distance, bool bWorldSpace = true, FBox* OutBounds = nullptr) const override;
	UE_API virtual FVector::FReal GetCurvatureAtDistance(int SegmentIndex, FVector::FReal Distance) const override;
	UE_API virtual float GetInputKeyAtDistance(int SegmentIndex, FVector::FReal Distance) const override;
	UE_API virtual void GetTangentsAtSegmentStart(int SegmentIndex, FVector& OutArriveTangent, FVector& OutLeaveTangent) const override;
	UE_API virtual FVector::FReal GetDistanceAtSegmentStart(int SegmentIndex) const override;
	UE_API virtual FVector GetLocationAtAlpha(float Alpha) const override;
	UE_API virtual FTransform GetTransformAtAlpha(float Alpha) const override;
	//~End UPCGPolyLineData interface

	//~Begin UPCGSpatialDataWithPointCache interface
	UE_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialDataWithPointCache interface

	//~Begin UPCGSpatialData interface
	UE_API virtual FBox GetBounds() const override;
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
protected:
	UE_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

	UE_API const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;

	/** Recompute the reparameterization of the spline by distance. */
	UE_API void UpdateReparamTable();

	/** Get the index of the first interp point before a given distance. Also computes the Alpha describing how far (in [0, 1]) the sample is between the left and right points. */
	UE_API void GetInterpPointAtDistance(int SegmentIndex, FVector::FReal Distance, int32& OutPointIndex, bool bComputeAlpha, FVector::FReal& OutAlpha) const;

protected:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SourceData)
	TWeakObjectPtr<ULandscapeSplinesComponent> Spline;

	/** Reparameterization of the spline by distance. Useful to query the InputKey at arbitrary distance. */
	FInterpCurveFloat ReparamTable;

	bool CheckSpline() const;
	mutable bool bLoggedInvalidSpline = false;
};

#undef UE_API
