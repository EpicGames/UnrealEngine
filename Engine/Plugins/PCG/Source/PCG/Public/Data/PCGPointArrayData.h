// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGBasePointData.h"
#include "PCGPointArray.h"

#include "PCGPointArrayData.generated.h"

extern PCG_API TAutoConsoleVariable<bool> CVarPCGEnablePointArrayDataParenting;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPointArrayData : public UPCGBasePointData
{
	GENERATED_BODY()
public:	
	//~Begin UObject Interface
	PCG_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~End UObject Interface

	PCG_API virtual void Flatten() override;

	PCG_API virtual void VisitDataNetwork(TFunctionRef<void(const UPCGData*)> Action) const override;
	PCG_API virtual const UPCGPointData* ToPointData(FPCGContext* Context, const FBox& InBounds = FBox(EForceInit::ForceInit)) const override;
	virtual const UPCGPointArrayData* ToPointArrayData(FPCGContext* Context, const FBox& InBounds = FBox(EForceInit::ForceInit)) const override { return this; }
	PCG_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	PCG_API virtual void CopyPropertiesTo(UPCGBasePointData* To, int32 ReadStartIndex, int32 WriteStartIndex, int32 Count, EPCGPointNativeProperties Properties) const override;
	PCG_API virtual EPCGPointNativeProperties GetAllocatedProperties(bool bWithInheritance = true) const override;
	PCG_API virtual bool SupportsSpatialDataInheritance() const override;
	virtual bool HasSpatialDataParent() const override { return ParentData != nullptr; }

	//~Begin UPCGBasePointData Interface
	virtual bool IsValidRef(const PCGPointOctree::FPointRef& InPointRef) const override { return InPointRef.Index >= 0 && InPointRef.Index < GetNumPoints(); }
	
	virtual int32 GetNumPoints() const override { return PointArray.GetNumPoints(); }
	PCG_API virtual void SetNumPoints(int32 InNumPoints, bool bInitializeValues = true) override;
	PCG_API void RemoveAt(int32 Index);
	
	PCG_API virtual void AllocateProperties(EPCGPointNativeProperties Properties) override;
	PCG_API virtual void FreeProperties(EPCGPointNativeProperties Properties) override;
	PCG_API virtual void MoveRange(int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements) override;
	PCG_API virtual void CopyUnallocatedPropertiesFrom(const UPCGBasePointData* InPointData) override;

	PCG_API virtual TArray<FTransform> GetTransformsCopy() const override;

	PCG_API virtual FPCGPointTransform::ValueRange GetTransformValueRange(bool bAllocate = true) override;
	PCG_API virtual FPCGPointDensity::ValueRange GetDensityValueRange(bool bAllocate = true) override;
	PCG_API virtual FPCGPointBoundsMin::ValueRange GetBoundsMinValueRange(bool bAllocate = true) override;
	PCG_API virtual FPCGPointBoundsMax::ValueRange GetBoundsMaxValueRange(bool bAllocate = true) override;
	PCG_API virtual FPCGPointColor::ValueRange GetColorValueRange(bool bAllocate = true) override;
	PCG_API virtual FPCGPointSteepness::ValueRange GetSteepnessValueRange(bool bAllocate = true) override;
	PCG_API virtual FPCGPointSeed::ValueRange GetSeedValueRange(bool bAllocate = true) override;
	PCG_API virtual FPCGPointMetadataEntry::ValueRange GetMetadataEntryValueRange(bool bAllocate = true) override;

	PCG_API virtual FPCGPointTransform::ConstValueRange GetConstTransformValueRange() const override;
	PCG_API virtual FPCGPointDensity::ConstValueRange GetConstDensityValueRange() const override;
	PCG_API virtual FPCGPointBoundsMin::ConstValueRange GetConstBoundsMinValueRange() const override;
	PCG_API virtual FPCGPointBoundsMax::ConstValueRange GetConstBoundsMaxValueRange() const override;
	PCG_API virtual FPCGPointColor::ConstValueRange GetConstColorValueRange() const override;
	PCG_API virtual FPCGPointSteepness::ConstValueRange GetConstSteepnessValueRange() const override;
	PCG_API virtual FPCGPointSeed::ConstValueRange GetConstSeedValueRange() const override;
	PCG_API virtual FPCGPointMetadataEntry::ConstValueRange GetConstMetadataEntryValueRange() const override;
	//~End UPCGBasePointData Interface
		
protected:
	PCG_API virtual void InitializeSpatialDataInternal(const FPCGInitializeFromDataParams& InParams) override;

private:
	void FlattenPropertiesIfNeeded(EPCGPointNativeProperties Property = EPCGPointNativeProperties::All);

	template<EPCGPointNativeProperties Property>
	FPCGPointArrayProperty<typename TPCGPointNativeProperty<Property>::Type>* GetProperty(bool bWithInheritance = true)
	{
		if (bWithInheritance)
		{
			if (EnumHasAnyFlags(static_cast<EPCGPointNativeProperties>(InheritedProperties), TPCGPointNativeProperty<Property>::EnumValue))
			{
				check(ParentData);
				return ParentData->GetProperty<Property>(bWithInheritance);
			}
		}

		if constexpr (Property == EPCGPointNativeProperties::Transform)
		{
			return &PointArray.Transform;
		}
		else if constexpr (Property == EPCGPointNativeProperties::Density)
		{
			return &PointArray.Density;
		}
		else if constexpr (Property == EPCGPointNativeProperties::Steepness)
		{
			return &PointArray.Steepness;
		}
		else if constexpr (Property == EPCGPointNativeProperties::BoundsMin)
		{
			return &PointArray.BoundsMin;
		}
		else if constexpr (Property == EPCGPointNativeProperties::BoundsMax)
		{
			return &PointArray.BoundsMax;
		}
		else if constexpr (Property == EPCGPointNativeProperties::Color)
		{
			return &PointArray.Color;
		}
		else if constexpr (Property == EPCGPointNativeProperties::Seed)
		{
			return &PointArray.Seed;
		}
		else if constexpr (Property == EPCGPointNativeProperties::MetadataEntry)
		{
			return &PointArray.MetadataEntry;
		}
		else
		{
			static_assert(TPCGPointUnsupportedProperty<Property>::Value, "Unsupported EPCGPointNativeProperties value");
			return nullptr;
		}
	}

	template<EPCGPointNativeProperties Property>
	const FPCGPointArrayProperty<typename TPCGPointNativeProperty<Property>::Type>* GetProperty(bool bWithInheritance = true) const
	{
		if (bWithInheritance)
		{
			if (EnumHasAnyFlags(static_cast<EPCGPointNativeProperties>(InheritedProperties), TPCGPointNativeProperty<Property>::EnumValue))
			{
				check(ParentData);
				return ParentData->GetProperty<Property>(bWithInheritance);
			}
		}

		if constexpr (Property == EPCGPointNativeProperties::Transform)
		{
			return &PointArray.Transform;
		}
		else if constexpr (Property == EPCGPointNativeProperties::Density)
		{
			return &PointArray.Density;
		}
		else if constexpr (Property == EPCGPointNativeProperties::Steepness)
		{
			return &PointArray.Steepness;
		}
		else if constexpr (Property == EPCGPointNativeProperties::BoundsMin)
		{
			return &PointArray.BoundsMin;
		}
		else if constexpr (Property == EPCGPointNativeProperties::BoundsMax)
		{
			return &PointArray.BoundsMax;
		}
		else if constexpr (Property == EPCGPointNativeProperties::Color)
		{
			return &PointArray.Color;
		}
		else if constexpr (Property == EPCGPointNativeProperties::Seed)
		{
			return &PointArray.Seed;
		}
		else if constexpr (Property == EPCGPointNativeProperties::MetadataEntry)
		{
			return &PointArray.MetadataEntry;
		}
		else
		{
			static_assert(TPCGPointUnsupportedProperty<Property>::Value, "Unsupported EPCGPointNativeProperties value");
			return nullptr;
		}
	}

	template<EPCGPointNativeProperties NativeProperty>
	bool FlattenPropertyIfNeeded()
	{
		using PropertyType = typename TPCGPointNativeProperty<NativeProperty>::Type;
		EPCGPointNativeProperties EnumValue = TPCGPointNativeProperty<NativeProperty>::EnumValue;
		EPCGPointNativeProperties InheritedPropertiesEnum = static_cast<EPCGPointNativeProperties>(InheritedProperties);

		if (!EnumHasAnyFlags(InheritedPropertiesEnum, EnumValue))
		{
			return false;
		}

		const FPCGPointArrayProperty<PropertyType>* InheritedProperty = GetProperty<NativeProperty>(/*bWithInheritance=*/true);
		FPCGPointArrayProperty<PropertyType>* Property = GetProperty<NativeProperty>(/*bWithInheritance=*/false);

		// Should not be equal since InheritedProperties contains that property
		if (ensure(InheritedProperty != Property))
		{
			if (InheritedProperty->IsAllocated())
			{
				Property->Allocate(/*bInitializeValues=*/false);
			}

			check(InheritedProperty->Num() == Property->Num());
			InheritedProperty->CopyTo(*Property, 0, 0, Property->Num());
		}

		EnumRemoveFlags(InheritedPropertiesEnum, EnumValue);
		InheritedProperties = static_cast<uint32>(InheritedPropertiesEnum);

		if (InheritedPropertiesEnum == EPCGPointNativeProperties::None)
		{
			ParentData = nullptr;
		}

		return true;
	};
	
	UPROPERTY()
	FPCGPointArray PointArray;

	UPROPERTY(VisibleAnywhere, Category = ParentData)
	TObjectPtr<UPCGPointArrayData> ParentData = nullptr;

	UPROPERTY(VisibleAnywhere, Category = ParentData, meta = (BitMask, BitmaskEnum = "/Script/PCG.EPCGPointNativeProperties"))
	uint32 InheritedProperties = 0;
};
