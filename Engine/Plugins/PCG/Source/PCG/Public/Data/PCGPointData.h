// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGBasePointData.h"
#include "Utils/PCGValueRange.h"

#include "Containers/StridedView.h"
#include "Math/GenericOctreePublic.h"
#include "Math/GenericOctree.h"

#include "PCGPointData.generated.h"

#define UE_API PCG_API

struct FPCGProjectionParams;

class AActor;

namespace PCGPointHelpers
{
	void Lerp(const FPCGPoint& A, const FPCGPoint& B, float Ratio, const UPCGMetadata* SourceMetadata, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata);
	void Bilerp(const FPCGPoint& X0Y0, const FPCGPoint& X1Y0, const FPCGPoint& X0Y1, const FPCGPoint& X1Y1, const UPCGMetadata* SourceMetadata, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata, float XFactor, float YFactor);
	void BilerpWithSnapping(const FPCGPoint& X0Y0, const FPCGPoint& X1Y0, const FPCGPoint& X0Y1, const FPCGPoint& X1Y1, const UPCGMetadata* SourceMetadata, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata, float XFactor, float YFactor);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

struct PCG_API UE_DEPRECATED(5.6, "Use PCGPointOctree::FPointRef instead") FPCGPointRef
{
	FPCGPointRef(const FPCGPoint& InPoint);
	FPCGPointRef(const FPCGPoint& InPoint, const FBox& InBoundsOverride);

	const FPCGPoint* Point;
	FBoxSphereBounds Bounds;
};

struct PCG_API UE_DEPRECATED(5.6, "Use PCGPointOctree::FPointRefSemantics instead") FPCGPointRefSemantics
{
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	inline static const FBoxSphereBounds& GetBoundingBox(const FPCGPointRef& InPoint)
	{
		return InPoint.Bounds;
	}

	inline static const bool AreElementsEqual(const FPCGPointRef& A, const FPCGPointRef& B)
	{
		// TODO: verify if that's sufficient
		return A.Point == B.Point;
	}

	inline static void ApplyOffset(FPCGPointRef& InPoint)
	{
		ensureMsgf(false, TEXT("Not implemented"));
	}

	inline static void SetElementId(const FPCGPointRef& Element, FOctreeElementId2 OctreeElementID)
	{
	}
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

// TODO: Split this in "concrete" vs "api" class (needed for views)
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPointData : public UPCGBasePointData
{
	GENERATED_BODY()

public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Use PCGPointOctree::FPointOctree instead")
	typedef TOctree2<FPCGPointRef, FPCGPointRefSemantics> PointOctree;
UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	//~Begin UObject Interface
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~End UObject Interface

	// Static helper to create an accessor on a data that doesn't yet exist, as accessors for point data don't rely on existing data.
	static UE_API TUniquePtr<IPCGAttributeAccessor> CreateStaticAccessor(const FPCGAttributePropertySelector& InSelector, bool bQuiet = false);

	// Get the functions to the accessor factory
	static UE_API FPCGAttributeAccessorMethods GetPointAccessorMethods();

	// ~Begin UPCGBasePointData interface
	virtual bool IsValidRef(const PCGPointOctree::FPointRef& InPointRef) const override { return Points.IsValidIndex(InPointRef.Index); }

	UE_API virtual TArray<FTransform> GetTransformsCopy() const override;

	UE_API virtual FPCGPointTransform::ValueRange GetTransformValueRange(bool bAllocate = true) override;
	UE_API virtual FPCGPointDensity::ValueRange GetDensityValueRange(bool bAllocate = true) override;
	UE_API virtual FPCGPointBoundsMin::ValueRange GetBoundsMinValueRange(bool bAllocate = true) override;
	UE_API virtual FPCGPointBoundsMax::ValueRange GetBoundsMaxValueRange(bool bAllocate = true) override;
	UE_API virtual FPCGPointColor::ValueRange GetColorValueRange(bool bAllocate = true) override;
	UE_API virtual FPCGPointSteepness::ValueRange GetSteepnessValueRange(bool bAllocate = true) override;
	UE_API virtual FPCGPointSeed::ValueRange GetSeedValueRange(bool bAllocate = true) override;
	UE_API virtual FPCGPointMetadataEntry::ValueRange GetMetadataEntryValueRange(bool bAllocate = true) override;

	UE_API virtual FPCGPointTransform::ConstValueRange GetConstTransformValueRange() const override;
	UE_API virtual FPCGPointDensity::ConstValueRange GetConstDensityValueRange() const override;
	UE_API virtual FPCGPointBoundsMin::ConstValueRange GetConstBoundsMinValueRange() const override;
	UE_API virtual FPCGPointBoundsMax::ConstValueRange GetConstBoundsMaxValueRange() const override;
	UE_API virtual FPCGPointColor::ConstValueRange GetConstColorValueRange() const override;
	UE_API virtual FPCGPointSteepness::ConstValueRange GetConstSteepnessValueRange() const override;
	UE_API virtual FPCGPointSeed::ConstValueRange GetConstSeedValueRange() const override;
	UE_API virtual FPCGPointMetadataEntry::ConstValueRange GetConstMetadataEntryValueRange() const override;
	
	UE_API virtual void MoveRange(int32 RangeStartIndex, int32 MoveToIndex, int32 NumElements) override;

	UE_API virtual void CopyPointsTo(UPCGBasePointData* OutData, int32 ReadStartIndex, int32 WriteStartIndex, int32 Count) const override;
	// ~End UPCGBasePointData interface

	// ~Begin UPCGSpatialData interface
	virtual const UPCGPointData* ToPointData(FPCGContext* Context, const FBox& InBounds = FBox(EForceInit::ForceInit)) const { return this; }
protected:
	UE_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	// ~End UPCGSpatialData interface

public:
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	const TArray<FPCGPoint>& GetPoints() const { return Points; }

	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = SpatialData)
	TArray<FPCGPoint> GetPointsCopy() const { return Points; }

	virtual int32 GetNumPoints() const override { return Points.Num(); }

	UE_API virtual void SetNumPoints(int32 InNumPoints, bool bInitializeValues = true) override;

	UFUNCTION(BlueprintCallable, Category = SpatialData)
	UE_API FPCGPoint GetPoint(int32 Index) const;

	UFUNCTION(BlueprintCallable, Category = SpatialData)
	UE_API void SetPoints(const TArray<FPCGPoint>& InPoints);

	UE_API TArray<FPCGPoint>& GetMutablePoints();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Use GetPointOctree instead")
	UE_API const PointOctree& GetOctree() const;
	UE_DEPRECATED(5.6, "Use IsPointOctreeDirty instead")
	bool IsOctreeDirty() const { return bOctreeOldIsDirty; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

protected:
	void RebuildOctreeOld() const;

	virtual void DirtyCache() override
	{
		Super::DirtyCache();
		bOctreeOldIsDirty = true;
	}
	
	UPROPERTY()
	TArray<FPCGPoint> Points;

	mutable bool bOctreeOldIsDirty = true;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// deprecated
	mutable PointOctree Octree;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

#undef UE_API
