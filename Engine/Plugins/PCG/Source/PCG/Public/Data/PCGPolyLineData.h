// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGSpatialData.h"

#include "PCGPolyLineData.generated.h"

#define UE_API PCG_API

USTRUCT()
struct FPCGDataTypeInfoPolyline : public FPCGDataTypeInfoConcrete
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::PolyLine)
};

UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = (Procedural))
class UPCGPolyLineData : public UPCGSpatialDataWithPointCache
{
	GENERATED_BODY()

public:
	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoPolyline)
	// ~End UPCGData interface

	//~Begin UPCGSpatialData interface
	virtual int GetDimension() const override { return 1; }
	UE_API virtual FBox GetBounds() const override;
	//~End UPCGSpatialData interface

	/** Get the world-space transform of the entire line. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual FTransform GetTransform() const { return FTransform::Identity; }

	/** Get the number of segments in this line. If the line is closed, this will be the same as the number of control points in the line. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual int GetNumSegments() const PURE_VIRTUAL(UPCGPolyLineData::GetNumSegments, return 0;);

	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	UE_API virtual int GetNumVertices() const;

	/** Get the length of a specific segment of the line. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual double GetSegmentLength(int SegmentIndex) const PURE_VIRTUAL(UPCGPolyLineData::GetSegmentLength, return 0;);

	/** Get the location of the point at the normalized [0, 1] parameter across the entire the poly line. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual FVector GetLocationAtAlpha(float Alpha) const PURE_VIRTUAL(UPCGPolyLine::GetLocationAtAlpha, return FVector::ZeroVector;)

	/** Get the full transform at the normalized [0, 1] parameter across the entire the poly line. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual FTransform GetTransformAtAlpha(float Alpha) const PURE_VIRTUAL(UPCGPolyLine::GetTransformAtAlpha, return FTransform::Identity;)

	/** Get the total length of the line. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	UE_API virtual double GetLength() const;

	/** Get the location at a distance along the line. */
	virtual FTransform GetTransformAtDistance(int SegmentIndex, double Distance, bool bWorldSpace = true, FBox* OutBounds = nullptr) const PURE_VIRTUAL(UPCGPolyLine::GetTransformAtDistance, return FTransform(););

	/** Get the location at a distance along the line. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData", meta = (DisplayName = "Get Transform At Distance", AutoCreateRefTerm = "OutBounds"))
	UE_API FTransform K2_GetTransformAtDistance(int SegmentIndex, double Distance, UPARAM(ref) FBox& OutBounds, bool bWorldSpace = true) const;
	
	/** Get the location at a distance along the line. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual FVector GetLocationAtDistance(int SegmentIndex, double Distance, bool bWorldSpace = true) const { return GetTransformAtDistance(SegmentIndex, Distance, bWorldSpace).GetLocation(); }

	/** Get the curvature at a distance along the line. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual double GetCurvatureAtDistance(int SegmentIndex, double Distance) const { return 0; }

	/**
	 * Get a value [0,1] representing how far along the point is to the end of the line. Each segment on the line represents a same-size interval.
	 * For example, if there are three segments, each segment will take up 0.333... of the interval.
	 */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	UE_API virtual float GetAlphaAtDistance(int SegmentIndex, double Distance) const;

	/** Get the input key at a distance along the line. InputKey is a float value in [0, N], where N is the number of control points. Each range [i, i+1] represents an interpolation from 0 to 1 across spline segment i. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual float GetInputKeyAtDistance(int SegmentIndex, double Distance) const { return 0; }

	/** Get the input key from the normalized distance of [0, 1] across the entire the poly line. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	UE_API virtual float GetInputKeyAtAlpha(float Alpha) const;

	/** Get the arrive and leave tangents for a control point via its segment index. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	UE_API virtual void GetTangentsAtSegmentStart(int SegmentIndex, FVector& OutArriveTangent, FVector& OutLeaveTangent) const;

	/** Get the cumulative distance along the line to the start of a segment. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual double GetDistanceAtSegmentStart(int SegmentIndex) const { return 0; }

	/** True if the line is a closed loop. */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual bool IsClosed() const { return false; }
	
	/** This function should be called in the Sample/Project point function, but can also be called if the sampling is done manually.
	 * This is meant for each child class to write its metadata given the InputKey, so interpolation can also be done.
	 */
	UFUNCTION(BlueprintCallable, Category = "PolylineData")
	virtual void WriteMetadataToPoint(float InputKey, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const {}

	virtual TConstArrayView<PCGMetadataEntryKey> GetConstVerticesEntryKeys() const { return {}; }
	virtual void AllocateMetadataEntries() {}
	virtual TArrayView<PCGMetadataEntryKey> GetMutableVerticesEntryKeys() { return {}; }
};

#undef UE_API
