// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGPolyLineData.h"

#include "PCGPointData.h"
#include "Metadata/PCGMetadataCommon.h"

#include "Curve/GeneralPolygon2.h"

#include "PCGPolygon2DData.generated.h"

UENUM()
enum class EPCGPolygon2DProperties : uint8
{
	Position UMETA(Tooltip = "Location of the vertex in world coordinates."),
	Rotation UMETA(Tooltip = "Rotation of the vertex in world coordinates.", PCG_PropertyReadOnly),
	SegmentIndex UMETA(Tooltip = "Segment index of the vertex in the current polygon (outer or hole)", PCG_PropertyReadOnly),
	HoleIndex UMETA(Tooltip = "Index of the hole the vertex is inside of. -1 on the Outer polygon.", PCG_PropertyReadOnly),
	SegmentLength UMETA(Tooltip = "Length of the segment starting from the vertex.", PCG_PropertyReadOnly),
	LocalPosition UMETA(Tooltip = "Location of the vertex in 2d coordinates."),
	LocalRotation UMETA(Tooltip = "Rotation of the vertex in 2d coordinates.", PCG_PropertyReadOnly)
};

UENUM()
enum class EPCGPolygon2DDataProperties : uint8
{
	Transform UMETA(Tooltip = "Transform of the 2d polygon.", PCG_MetadataDomain="Data"),
	Area UMETA(Tooltip = "Area of the 2d polygon.", PCG_MetadataDomain = "Data", PCG_PropertyReadOnly),
	Perimeter UMETA(Tooltip = "Perimeter of the 2d polygon.", PCG_MetadataDomain = "Data", PCG_PropertyReadOnly),
	BoundsMin UMETA(Tooltip = "Minimum point of the bounds of the 2d polygon in 2d space.", PCG_MetadataDomain = "Data", PCG_PropertyReadOnly),
	BoundsMax UMETA(Tooltip = "Minimum point of the bounds of the 2d polygon in 2d space.", PCG_MetadataDomain = "Data", PCG_PropertyReadOnly),
	SegmentCount UMETA(Tooltip = "Number of segments in the 2d polygon (includes hole segments).", PCG_MetadataDomain = "Data", PCG_PropertyReadOnly),
	OuterSegmentCount UMETA(Tooltip = "Number of segments in the 2d polygon (only the outer polygon).", PCG_MetadataDomain = "Data", PCG_PropertyReadOnly),
	HoleCount UMETA(Tooltip = "Number of holes in the 2d polygon.", PCG_MetadataDomain = "Data", PCG_PropertyReadOnly),
	LongestOuterSegmentIndex UMETA(Tooltip = "Index of the longest segment in the polygon.", PCG_MetadataDomain = "Data", PCG_PropertyReadOnly),
	IsClockwise UMETA(Tooltip = "Returns whether the 2d polygon is clockwise or counter-clocwise.", PCG_MetadataDomain = "Data", PCG_PropertyReadOnly)
};

namespace PCGPolygon2DData
{
	const FName VertexDomainName = "Vertices";

	/** Reverses the given polygon & its entry keys with proper hole support. */
	void ReversePolygon(UE::Geometry::FGeneralPolygon2d& Polygon, TArray<int64>& PolygonEntryKeys);

	/** Counts the number of vertices in a polygon, including the hole vertices. */
	int PolygonVertexCount(const UE::Geometry::FGeneralPolygon2d& InPoly);

	/** Returns the vertex index (0 to number of vertices - 1) from a segment index & hole index. */
	int GetVertexIndex(const UE::Geometry::FGeneralPolygon2d& InPoly, int InSegmentIndex, int InHoleIndex);

	/** Returns the vertex indices of a segment start and end. */
	void GetStartEndVertexIndices(const UE::Geometry::FGeneralPolygon2d& InPoly, int InSegmentIndex, int InHoleIndex, int& OutStartVertexIndex, int& OutEndVertexIndex);
}

USTRUCT()
struct FPCGDataTypeInfoPolygon2D : public FPCGDataTypeInfoPolyline
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Polygon2D);

	PCG_API virtual bool SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const override;
};

/** Data representing a single 2D polygon with a 3D transform (for spatial operations). */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPolygon2DData : public UPCGPolyLineData
{
	GENERATED_BODY()

public:
	PCG_API UPCGPolygon2DData(const FObjectInitializer& ObjectInitializer);

	// ~Begin UObject interface
	PCG_API virtual void Serialize(FArchive& InArchive) override;
	PCG_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	// ~End UObject interface

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoPolygon2D)
	PCG_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;

	virtual FPCGMetadataDomainID GetDefaultMetadataDomainID() const override { return PCGMetadataDomainID::Elements; }
	virtual TArray<FPCGMetadataDomainID> GetAllSupportedMetadataDomainIDs() const override { return { PCGMetadataDomainID::Data, PCGMetadataDomainID::Elements }; }
	PCG_API virtual FPCGMetadataDomainID GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const override;
	PCG_API virtual bool SetDomainFromDomainID(const FPCGMetadataDomainID& InDomainID, FPCGAttributePropertySelector& InOutSelector) const;

protected:
	virtual bool SupportsFullDataCrc() const override { return true; }
	// ~End UPCGData interface

public:
	// ~Begin UPCGSpatialData interface
	virtual int GetDimension() const override { return 2; }
	PCG_API virtual FBox GetBounds() const override;
	PCG_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	PCG_API virtual bool ProjectPoint(const FTransform& Transform, const FBox& Bounds, const FPCGProjectionParams& Params, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;

	PCG_API virtual void WriteMetadataToPoint(float InputKey, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;

protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	// ~End UPCGSpatialData interface

public:
	// ~Begin UPCGSpatialDataWithPointCache interface
	PCG_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	PCG_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	// ~End UPCGSpatialDataWithPointCache interface

	// ~Begin UPCGPolyLineData interface - all these functions will return world space information (e.g. including scale, ...) unless specified
	virtual FTransform GetTransform() const override { return Transform; }
	PCG_API virtual int GetNumSegments() const override;
	virtual int GetNumVertices() const override { return GetNumSegments(); }
	PCG_API virtual double GetSegmentLength(int SegmentIndex) const override;
	PCG_API virtual FVector GetLocationAtAlpha(float Alpha) const override;
	PCG_API virtual FTransform GetTransformAtAlpha(float Alpha) const override;
	PCG_API virtual FTransform GetTransformAtDistance(int SegmentIndex, double Distance, bool bWorldSpace = true, FBox* OutBounds = nullptr) const override;
	PCG_API virtual float GetInputKeyAtDistance(int SegmentIndex, double Distance) const override;
	PCG_API virtual double GetDistanceAtSegmentStart(int SegmentIndex) const override;
	virtual bool IsClosed() const { return true; }

	virtual TConstArrayView<PCGMetadataEntryKey> GetConstVerticesEntryKeys() const override { return VertexEntryKeys; }
	PCG_API virtual void AllocateMetadataEntries() override;
	virtual TArrayView<PCGMetadataEntryKey> GetMutableVerticesEntryKeys() override { return VertexEntryKeys; }
	// ~End UPCGPolyLineData interface

	/** Static helper to create an accessor on a data that doesn't yet exist, as accessors for spline data don't rely on existing data. */
	static PCG_API TUniquePtr<IPCGAttributeAccessor> CreateStaticAccessor(const UPCGPolygon2DData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet = false, bool bConst = false);

	/** Helper to create an accessor keys for data that doesn't yet exist. */
	PCG_API TUniquePtr<IPCGAttributeAccessorKeys> CreateAccessorKeys(const FPCGAttributePropertySelector& InSelector, bool bQuiet = false);
	PCG_API TUniquePtr<const IPCGAttributeAccessorKeys> CreateConstAccessorKeys(const FPCGAttributePropertySelector& InSelector, bool bQuiet = false) const;

	// Get the functions to the accessor factory
	static PCG_API FPCGAttributeAccessorMethods GetPolygon2DAccessorMethods();

	/**
	 * Utility function to be able to modify a vertex in place. Will assert if out of bounds.
	 * @param Vertex       Vertex value to update
	 * @param HoleIndex    Hole Index, -1 if targeting Outer
	 * @param VertexIndex Index of the vertex to update in the Hole/Outer polygon.
	 */
	void UpdateVertexInPlace(const FVector2d& Vertex, int32 HoleIndex, int32 VertexIndex)
	{
		Polygon.UpdateVertexInPlace(Vertex, HoleIndex, VertexIndex);
	}

public:
	const UE::Geometry::FGeneralPolygon2d& GetPolygon() const { return Polygon; }
	const TMap<int, TPair<int, int>>& GetSegmentIndexToSegmentAndHoleIndices() const { return SegmentIndexToSegmentAndHoleIndices; }

	PCG_API void SetPolygon(const UE::Geometry::FGeneralPolygon2d& InPolygon, TConstArrayView<PCGMetadataEntryKey>* InOptionalEntryKeys = nullptr);
	PCG_API void SetPolygon(UE::Geometry::FGeneralPolygon2d&& InPolygon, TConstArrayView<PCGMetadataEntryKey>* InOptionalEntryKeys = nullptr);
	PCG_API void SetTransform(const FTransform& InTransform, bool bCheckWinding = false);

	PCG_API void WriteMetadataToEntry(float InputKey, PCGMetadataEntryKey& OutEntryKey, UPCGMetadata* OutMetadata) const;

protected:
	const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;

private:
	void UpdateMappings();

protected:
	// Note: can't be a property
	UE::Geometry::FGeneralPolygon2d Polygon;

	// In order, outer then holes
	UPROPERTY()
	TArray<int64> VertexEntryKeys;

	UPROPERTY()
	FTransform Transform;

private:
	TMap<int, TPair<int, int>> SegmentIndexToSegmentAndHoleIndices;
	TMap<TPair<int, int>, int> SegmentAndHoleIndicesToSegmentIndex;
	int SegmentCount = 0;
};
