// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGSurfaceData.h"
#include "Data/PCGPolygon2DData.h"

#include "PCGPolygon2DInteriorData.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPolygon2DInteriorSurfaceData : public UPCGSurfaceData
{
	GENERATED_BODY()

public:
	PCG_API void Initialize(FPCGContext* Context, const UPCGPolygon2DData* InPolygonData);

	// ~Begin UPCGData interface
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	// ~Begin UPCGSpatialData interface
	PCG_API virtual FBox GetBounds() const override;
	PCG_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	PCG_API virtual bool ProjectPoint(const FTransform& Transform, const FBox& Bounds, const FPCGProjectionParams& Params, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	virtual bool HasNonTrivialTransform() const override { return true; }
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	// ~End UPCGSpatialData interface

public:
	// ~Begin UPCGSpatialDataWithPointCache interface
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	// ~End UPCGConcreteDataWithPointCache interface

	const UPCGPolygon2DData* GetPolygonData() const { return Polygon; }

protected:
	const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;

protected:
	UPROPERTY()
	TObjectPtr<const UPCGPolygon2DData> Polygon;
};