// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialData.h"

#include "PCGPrimitiveData.generated.h"

#define UE_API PCG_API

USTRUCT()
struct FPCGDataTypeInfoPrimitive : public FPCGDataTypeInfoConcrete
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Primitive)
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPrimitiveData : public UPCGSpatialDataWithPointCache
{
	GENERATED_BODY()

public:
	UE_API void Initialize(UPrimitiveComponent* InPrim);

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoPrimitive)
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	// ~Begin UPCGSpatialData interface
	virtual int GetDimension() const override { return 3; }
	virtual FBox GetBounds() const override { return CachedBounds; }
	virtual FBox GetStrictBounds() const override { return CachedStrictBounds; }
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	// TODO needs an implementation to support projection
	//virtual bool ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;

protected:
	UE_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	// ~Begin UPCGSpatialDataWithPointCache implementation
	UE_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	// ~End UPCGSpatialDataWithPointCache implementation

	TWeakObjectPtr<UPrimitiveComponent> GetComponent() const { return Primitive; }

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	FVector VoxelSize = FVector(100.0, 100.0, 100.0);

private:
	UE_API virtual const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;

protected:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Data")
	TWeakObjectPtr<UPrimitiveComponent> Primitive = nullptr;

	UPROPERTY()
	FBox CachedBounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	FBox CachedStrictBounds = FBox(EForceInit::ForceInit);
};

#undef UE_API
