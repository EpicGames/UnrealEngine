// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialData.h"

#include "PCGProjectionData.generated.h"

#define UE_API PCG_API

/**
* Generic projection class (A projected onto B) that intercepts spatial queries
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGProjectionData : public UPCGSpatialDataWithPointCache
{
	GENERATED_BODY()
public:
	UE_API void Initialize(const UPCGSpatialData* InSource, const UPCGSpatialData* InTarget, const FPCGProjectionParams& InProjectionParams);

	// ~Begin UObject interface
	UE_API virtual void PostLoad() override;
	// ~End UObject interface

	const FPCGProjectionParams& GetProjectionParams() const { return ProjectionParams; }

	// ~Begin UPCGData interface
	UE_API virtual FPCGCrc ComputeCrc(bool bFullDataCrc) const override;
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	//~Begin UPCGSpatialData interface
	UE_API virtual int GetDimension() const override;
	UE_API virtual FBox GetBounds() const override;
	UE_API virtual FBox GetStrictBounds() const override;
	UE_API virtual FVector GetNormal() const override;
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	UE_API virtual bool HasNonTrivialTransform() const override;
	UE_API virtual bool RequiresCollapseToSample() const override;
	UE_API virtual void InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const override;
protected:
	UE_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	//~Begin UPCGSpatialDataWithPointCache interface
	UE_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialDataWithPointCache interface

protected:
	UE_API const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;

	UE_API void CopyBaseProjectionClass(UPCGProjectionData* NewProjectionData) const;

	UE_API FBox ProjectBounds(const FBox& InBounds) const;

	/** Applies data from target point to projected point, conditionally according to the projection params. */
	UE_API void ApplyProjectionResult(const FPCGPoint& InTargetPoint, FPCGPoint& InOutProjected) const;

	UE_API void GetIncludeExcludeAttributeNames(TSet<FName>& OutAttributeNames) const;

	UE_API void SetupTargetMetadata(UPCGMetadata* MetadataToInitialize) const;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	TObjectPtr<const UPCGSpatialData> Source = nullptr;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	TObjectPtr<const UPCGSpatialData> Target = nullptr;

	UPROPERTY()
	FBox CachedBounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	FBox CachedStrictBounds = FBox(EForceInit::ForceInit);

	UPROPERTY(BlueprintReadwrite, VisibleAnywhere, Category = SpatialData)
	FPCGProjectionParams ProjectionParams;
};

#undef UE_API
