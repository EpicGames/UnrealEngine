// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGTextureData.h"

#include "RHI.h"

#include "PCGRenderTargetData.generated.h"

class UTexture;
class UTextureRenderTarget2D;

USTRUCT(meta = (PCG_DataTypeDisplayName = "Render Target 2D"))
struct FPCGDataTypeInfoRenderTarget2D : public FPCGDataTypeInfoTexture2DSingleBase
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::RenderTarget)
};

//TODO: It's possible that caching the result in this class is not as efficient as it could be
// if we expect to sample in different ways (e.g. channel) in the same render target
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGRenderTargetData : public UPCGTexture2DSingleBaseData
{
	GENERATED_BODY()

public:
	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoRenderTarget2D);
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	virtual bool IsCacheable() const { return Super::IsCacheable() && !bOwnsRenderTarget; }
	virtual void ReleaseTransientResources(const TCHAR* InReason = nullptr) override;
	// ~End UPCGData interface

	//~Begin UPCGTexture2DSingleBaseData interface
	virtual UTexture* GetTexture() const override;
	virtual FTextureRHIRef GetTextureRHI() const override;
	virtual EPCGTextureResourceType GetTextureResourceType() const override { return EPCGTextureResourceType::TextureObject; }
	PCG_API virtual bool RequestCPUReadback() override;
	//~End UPCGTexture2DSingleBaseData interface

	//~Begin UPCGSpatialData interface
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	PCG_API void Initialize(UTextureRenderTarget2D* InRenderTarget, const FTransform& InTransform, bool bInTakeOwnershipOfRenderTarget = false);

	UFUNCTION(BlueprintCallable, Category = RenderTarget, meta = (DisplayName = "Initialize"))
	void K2_Initialize(UTextureRenderTarget2D* InRenderTarget, const FTransform& InTransform, bool bInReadbackToCPU, bool bInTakeOwnershipOfRenderTarget = false);

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Properties)
	TObjectPtr<UTextureRenderTarget2D> RenderTarget = nullptr;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Properties)
	bool bOwnsRenderTarget = false;

protected:
	PCG_API void InitializeInternal(UTextureRenderTarget2D* InRenderTarget, const FTransform& InTransform, bool bInReadbackToCPU, bool bInTakeOwnershipOfRenderTarget);

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "bSkipReadbackToCPU has been removed. Use the overload without bSkipReadbackToCPU (readback is deferred). Call RequestCPUReadback to force immediate readback.")
	UFUNCTION(BlueprintCallable, Category = RenderTarget, meta = (DeprecatedFunction, DeprecationMessage = "Use the overload with bInReadbackToCPU instead."))
	PCG_API void Initialize(UTextureRenderTarget2D* InRenderTarget, const FTransform& InTransform, bool bInSkipReadbackToCPU, bool bInTakeOwnershipOfRenderTarget);
};
