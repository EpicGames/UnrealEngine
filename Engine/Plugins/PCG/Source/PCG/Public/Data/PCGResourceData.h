// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"

#include "Engine/StreamableManager.h"

#include "PCGResourceData.generated.h"

#define UE_API PCG_API

USTRUCT()
struct FPCGDataTypeInfoResource : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Resource)

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; }
#endif // WITH_EDITOR
};

/** Data that wrap/represent an asset, like a Static Mesh or Texture. */
UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = (Procedural))
class UPCGResourceData : public UPCGData
{
	GENERATED_BODY()

public:
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoResource)
	//~ End UPCGData interface

	virtual FSoftObjectPath GetResourcePath() const PURE_VIRTUAL(UPCGResourceData::GetResourcePath, return FSoftObjectPath(););

	UE_API TSharedPtr<FStreamableHandle> RequestResourceLoad(bool bAsynchronous = true) const;
};

#undef UE_API
