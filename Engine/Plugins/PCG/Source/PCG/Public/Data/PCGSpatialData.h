// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "PCGPoint.h" // IWYU pragma: keep
#include "Elements/PCGProjectionParams.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataCommon.h"

#include "PCGSpatialData.generated.h"

#define UE_API PCG_API

class AActor;
struct FPCGBlueprintContextHandle;
struct FPCGContext;
class UPCGSpatialData;
class UPCGBasePointData;
class UPCGPointData;
class UPCGPointArrayData;
class UPCGIntersectionData;
class UPCGUnionData;
class UPCGDifferenceData;
class UPCGProjectionData;

namespace PCGSpatialData
{
	extern TAutoConsoleVariable<bool> CVarEnablePrepareForSpatialQuery;
}

USTRUCT(MinimalAPI, BlueprintType)
struct FPCGInitializeFromDataParams
{
	GENERATED_BODY()

	UE_API FPCGInitializeFromDataParams();
	
	UE_API FPCGInitializeFromDataParams(const UPCGSpatialData* InSource);
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	TObjectPtr<const UPCGSpatialData> Source = nullptr;

	/** In the case of collapse of composite data, we need to inherit metadata from another source. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	TObjectPtr<const UPCGSpatialData> SourceOverride = nullptr;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	bool bInheritMetadata = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	bool bInheritAttributes = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	bool bInheritSpatialData = true;

	// Special boolean to be set to true when we duplicate data.
	bool bIsDuplicatingData = false;

	/** When initializing metadata, can provide an extra set of params to initialize (for filtering attributes for example) */
	FPCGMetadataInitializeParams MetadataInitializeParams;
};

USTRUCT()
struct FPCGDataTypeInfoSpatial : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Spatial)

	PCG_API virtual bool SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const override;
};

// Subclasses of UPCGSpatial
USTRUCT()
struct FPCGDataTypeInfoConcrete : public FPCGDataTypeInfoSpatial
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Concrete)

	PCG_API virtual bool SupportsConversionFrom(const FPCGDataTypeIdentifier& InputType, const FPCGDataTypeIdentifier& ThisType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const override;
};

USTRUCT()
struct FPCGDataTypeInfoComposite : public FPCGDataTypeInfoSpatial
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Composite)

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; }
#endif // WITH_EDITOR
};

/**
* "Concrete" data base class for PCG generation
* This will be the base class for data classes that actually represent
* concrete evidence of spatial data - points, surfaces, splines, etc.
* In opposition to settings/control type of data.
* 
* Conceptually, any concrete data can be decayed into points (potentially through transformations)
* which hold metadata and a transform, and this is the basic currency of the PCG framework.
*/
UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = (Procedural))
class UPCGSpatialData : public UPCGData
{
	GENERATED_BODY()

public:
	UE_API UPCGSpatialData(const FObjectInitializer& ObjectInitializer);

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoSpatial)

	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;

	UE_API virtual bool HasCachedLastSelector() const override;
	UE_API virtual FPCGAttributePropertyInputSelector GetCachedLastSelector() const override;
	UE_API virtual void SetLastSelector(const FPCGAttributePropertySelector& InSelector) override;

	/** Virtual call to allocate a new spacial data object, duplicate this spatial data into
	*   and parent the new metadata with this class metadata (if asked).
	*   Should be way cheaper than DuplicateObject, since we avoid duplicating metadata.
	*   It will not deep copy references.
	*   Some data are marked mutable and therefore are not threadsafe to copy, so they are not copied.
	*   They are mainly cached values (and octree for points).
	*   TODO: If we want to also copy those values (can be an optimization), we need to guard the copy.
	*/
	UE_API virtual UPCGSpatialData* DuplicateData(FPCGContext* Context, bool bInitializeMetadata = true) const override;
	// ~End UPCGData interface

	/** Returns the dimension of the data type, which has nothing to do with the dimension of its points */
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	virtual int GetDimension() const PURE_VIRTUAL(UPCGSpatialData::GetDimension, return 0;);

	/** Returns the full bounds (including density fall-off) of the data */
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	virtual FBox GetBounds() const PURE_VIRTUAL(UPCGSpatialData::GetBounds, return FBox(EForceInit::ForceInit););

	/** Returns whether a given spatial data is bounded as some data types do not require bounds by themselves */
	virtual bool IsBounded() const { return true; }

	/** Returns the bounds in which the density is always 1 */
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	virtual FBox GetStrictBounds() const { return FBox(EForceInit::ForceInit); }

	/** Returns the expected data normal (for surfaces) or eventual projection axis (for volumes) */
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	virtual FVector GetNormal() const { return FVector::UnitZ(); }

	/** Computes the density at a given location */
	UFUNCTION(BlueprintCallable, Category = Distribution)
	UE_API virtual float GetDensityAtPosition(const FVector& InPosition) const;

	/** Discretizes the data into points */
	UFUNCTION(BlueprintCallable, Category = SpatialData, meta = (DeprecatedFunction, DeprecationMessage = "The To Point Data function is deprecated - use To Point Data With Context instead."))
	const UPCGPointData* ToPointData() const { return ToPointData(nullptr); }

	/** Prefer using ToBasePointDataWithContext */
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	const UPCGPointData* ToPointDataWithContext(UPARAM(ref) FPCGContext& Context) const { return ToPointData(&Context); }

	UFUNCTION(BlueprintCallable, Category = SpatialData)
	const UPCGBasePointData* ToBasePointDataWithContext(const FPCGBlueprintContextHandle& ContextHandle) const;

	virtual const UPCGPointData* ToPointData(FPCGContext* Context, const FBox& InBounds = FBox(EForceInit::ForceInit)) const PURE_VIRTUAL(UPCGSpatialData::ToPointData, return nullptr;);

	UE_API virtual const UPCGPointArrayData* ToPointArrayData(FPCGContext* Context, const FBox& InBounds = FBox(EForceInit::ForceInit)) const;

	UE_API const UPCGBasePointData* ToBasePointData(FPCGContext* Context, const FBox& InBounds = FBox(EForceInit::ForceInit)) const;

	/** Sample rotation, scale and other attributes from this data at the query position. Returns true if Transform location and Bounds overlaps this data. */
	virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const PURE_VIRTUAL(UPCGSpatialData::SamplePoint, return false;);

	/** Performs multiple samples at the same time.
	* Contrary to the single SamplePoint call, this is expected to set the density to 0 for points that were not overlapping - but the other properties can be anything.
	* The OutPoints arrays is expected pre-allocated to the size of the Samples.
	*/
	UE_API virtual void SamplePoints(const TArrayView<const TPair<FTransform, FBox>>& Samples, const TArrayView<FPCGPoint>& OutPoints, UPCGMetadata* OutMetadata) const;

	/** Sample rotation, scale and other attributes from this data at the query position. Returns true if Transform location and Bounds overlaps this data. */
	UFUNCTION(BlueprintCallable, Category = SpatialData, meta = (DisplayName = "Sample Point"))
	UE_API bool K2_SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;

	/** Project the query point onto this data, and sample point and metadata information at the projected position. Returns true if successful. */
	UE_API virtual bool ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;
	
	/** Performs multiple projections of samples at the same time.
	* Contrary to the single ProjectPoint call, this is expected to set the density to 0 for points that were not overlapping - but the other properties can be anything.
	* The OutPoints arrays is expected pre-allocated to the size of the Samples.
	*/
	UE_API virtual void ProjectPoints(const TArrayView<const TPair<FTransform, FBox>>& Samples, const FPCGProjectionParams& InParams, const TArrayView<FPCGPoint>& OutPoints, UPCGMetadata* OutMetadata) const;

	UFUNCTION(BlueprintCallable, Category = SpatialData, meta = (DisplayName = "Project Point"))
	UE_API bool K2_ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;

	/** Returns true if the data has a non-trivial transform */
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	virtual bool HasNonTrivialTransform() const { return false; }

	/** Returns a specialized data to intersect with another data */
	UFUNCTION(BlueprintCallable, Category = SpatialData, meta = (DisplayName = "Intersect With"))
	UE_API UPCGIntersectionData* K2_IntersectWith(const UPCGSpatialData* InOther) const;

	UE_DEPRECATED(5.5, "Call/Implement version with FPCGContext parameter")
	virtual UPCGIntersectionData* IntersectWith(const UPCGSpatialData* InOther) const { return IntersectWith(nullptr, InOther); }

	UE_API virtual UPCGIntersectionData* IntersectWith(FPCGContext* InContext, const UPCGSpatialData* InOther) const;

	/** Returns a specialized data to project this on another data of equal or higher dimension. Returns copy of this data if projection fails. */
	UFUNCTION(BlueprintCallable, Category = SpatialData, meta = (AutoCreateRefTerm = "InParams", DisplayName  ="Project On"))
	UE_API UPCGSpatialData* K2_ProjectOn(const UPCGSpatialData* InOther, const FPCGProjectionParams& InParams = FPCGProjectionParams()) const;

	UE_DEPRECATED(5.5, "Call/Implement version with FPCGContext parameter")
	virtual UPCGSpatialData* ProjectOn(const UPCGSpatialData* InOther, const FPCGProjectionParams& InParams = FPCGProjectionParams()) const { return ProjectOn(nullptr, InOther, InParams); }

	UE_API virtual UPCGSpatialData* ProjectOn(FPCGContext* InContext, const UPCGSpatialData* InOther, const FPCGProjectionParams& InParams = FPCGProjectionParams()) const;

	/** Returns a specialized data to union this with another data */
	UFUNCTION(BlueprintCallable, Category = SpatialData, meta = (DisplayName = "Union With"))
	UE_API UPCGUnionData* K2_UnionWith(const UPCGSpatialData* InOther) const;

	UE_DEPRECATED(5.5, "Call/Implement version with FPCGContext parameter")
	virtual UPCGUnionData* UnionWith(const UPCGSpatialData* InOther) const { return UnionWith(nullptr, InOther); }

	UE_API virtual UPCGUnionData* UnionWith(FPCGContext* InContext, const UPCGSpatialData* InOther) const;

	UFUNCTION(BlueprintCallable, Category = SpatialData, meta = (DisplayName = "Subtract"))
	UE_API UPCGDifferenceData* K2_Subtract(const UPCGSpatialData* InOther) const;

	UE_DEPRECATED(5.5, "Call/Implement version with FPCGContext parameter")
	virtual UPCGDifferenceData* Subtract(const UPCGSpatialData* InOther) const { return Subtract(nullptr, InOther); }

	UE_API virtual UPCGDifferenceData* Subtract(FPCGContext* InContext, const UPCGSpatialData* InOther) const;

	UFUNCTION(BlueprintCallable, Category = Metadata, meta=(DeprecatedFunction, DeprecationMessage = "The Create Empty Metadata function is not needed anymore - it can safely be removed"))
	UE_API UPCGMetadata* CreateEmptyMetadata();

	/** Prefer using InitializeFromDataWithParams instead. Note that InMetadataParentOverride is deprecated on the code side, and should not be used anymore. */
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	UE_API void InitializeFromData(const UPCGSpatialData* InSource, const UPCGMetadata* InMetadataParentOverride = nullptr, bool bInheritMetadata = true, bool bInheritAttributes = true);

	UFUNCTION(BlueprintCallable, Category = SpatialData)
	UE_API void InitializeFromDataWithParams(const FPCGInitializeFromDataParams& InParams);

	/** True if this operation does not have an inverse and cannot be queried analytically/implicitly, and therefore must be collapsed to an explicit point representation. */
	virtual bool RequiresCollapseToSample() const { return false; }

	/** Find the first concrete (non-composite) shape in the network. Depth first search. */
	virtual const UPCGSpatialData* FindFirstConcreteShapeFromNetwork() const { return !!(GetDataType() & EPCGDataType::Concrete) ? this : nullptr; }

	/** True if subclass UPCGData instances support inheriting data from parent UPCGData */
	virtual bool SupportsSpatialDataInheritance() const { return false; }
	
	/** True if data inherits from parent */
	virtual bool HasSpatialDataParent() const { return false; }

	/** Optionally return a list of scheduled tasks that will prepare the data for spatial queries. Calling task can then wait on those tasks to finish before progressing further. */
	virtual TArray<FPCGTaskId> PrepareForSpatialQuery(FPCGContext* InContext, const FBox& InBounds) const { return {}; }

	/** Recipient of any artifacts generated using this data. */
	UPROPERTY(Transient, BlueprintReadWrite, EditAnywhere, Category = Data)
	TWeakObjectPtr<AActor> TargetActor = nullptr;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data)
	bool bKeepZeroDensityPoints = false;

	/**
	 * Initialize another metadata from this data. By default, it will just initialize/add attributes to the metadata to initialize.
	 * But it can also be overridden if there is specific logic when there is a cross domain boundary (like Surface -> Points).
	 * If MetadataToInitialize is not yet initialized, will initialize it. If it is already initialized, will add the attributes.
	 */
	UE_API virtual void InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const;

protected:
	UE_DEPRECATED(5.5, "Call/Implement version with FPCGContext parameter")
	virtual UPCGSpatialData* CopyInternal() const { return nullptr; }

	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const PURE_VIRTUAL(UPCGSpatialData::CopyInternal, return nullptr;);
	UE_API virtual void InitializeSpatialDataInternal(const FPCGInitializeFromDataParams& InParams);
	UE_API void InitializeMetadata(const FPCGInitializeFromDataParams& InParams);
	UE_API virtual void InitializeMetadataInternal(const FPCGInitializeFromDataParams& InParams);

private:
	/** Cache to keep track of the latest attribute manipulated on this data. */
	UPROPERTY()
	bool bHasCachedLastSelector = false;

	UPROPERTY()
	FPCGAttributePropertyInputSelector CachedLastSelector;
};

USTRUCT()
struct FPCGPointDataCache
{
	GENERATED_BODY()

	const UPCGBasePointData* ToBasePointDataInternal(FPCGContext* Context, const FBox& InBounds, bool bSupportsBoundedPointData, PCG::FLock& CacheLock, TFunctionRef<const UPCGBasePointData* (FPCGContext*, const FBox&)> CreatePointDataFunc);
	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);

	UPROPERTY(Transient)
	TObjectPtr<const UPCGBasePointData> CachedPointData;

	UPROPERTY(Transient)
	TArray<FBox> CachedBoundedPointDataBoxes;

	UPROPERTY(Transient)
	TArray<TObjectPtr<const UPCGBasePointData>> CachedBoundedPointData;
};

UCLASS(MinimalAPI, Abstract, ClassGroup = (Procedural))
class UPCGSpatialDataWithPointCache : public UPCGSpatialData
{
	GENERATED_BODY()

public:
	//~Begin UObject Interface
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~End UObject Interface

	// ~UPCGSpatialData implementation
	UE_API virtual const UPCGPointData* ToPointData(FPCGContext* Context, const FBox& InBounds = FBox(EForceInit::ForceInit)) const override;
	UE_API virtual const UPCGPointArrayData* ToPointArrayData(FPCGContext* Context, const FBox& InBounds = FBox(EForceInit::ForceInit)) const override;
	// ~End UPCGSpatialData implementation

protected:
	virtual bool SupportsBoundedPointData() const { return false; }
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const PURE_VIRTUAL(UPCGSpatialData::CreatePointData, return nullptr;);
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context, const FBox& InBounds) const { return CreatePointData(Context); }
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const; 

private:
	UPROPERTY(Transient)
	mutable FPCGPointDataCache PointDataCache;

	UPROPERTY(Transient)
	mutable FPCGPointDataCache PointArrayDataCache;

	mutable PCG::FLock CacheLock;
};

#undef UE_API
