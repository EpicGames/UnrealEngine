// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPoint.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGAsync.h"

#include "Algo/Transform.h"

namespace FPCGSpatialDataProcessing
{
	constexpr int32 DefaultSamplePointsChunkSize = 256;

	template<int ChunkSize, typename ProcessRangeFunc>
	void SampleBasedRangeProcessing(FPCGAsyncState* AsyncState, ProcessRangeFunc&& InProcessRange, const TArray<FPCGPoint>& SourcePoints, TArray<FPCGPoint>& OutPoints)
	{
		const int NumIterations = SourcePoints.Num();

		auto Initialize = [&OutPoints, NumIterations]()
		{
			OutPoints.SetNumUninitialized(NumIterations);
		};

		auto ProcessRange = [RangeFunc = MoveTemp(InProcessRange), &SourcePoints, &OutPoints](int32 StartReadIndex, int32 StartWriteIndex, int32 Count) -> int32
		{
			ensure(Count >= 0 && Count <= ChunkSize);

			TArrayView<const FPCGPoint> IterationPoints(SourcePoints.GetData() + StartReadIndex, Count);
			TArray<TPair<FTransform, FBox>, TInlineAllocator<ChunkSize>> Samples;
			TArray<FPCGPoint, TInlineAllocator<ChunkSize>> RangeOutputPoints;

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSpatialDataProcessing::SamplePoints::PrepareSamples)
				Algo::Transform(IterationPoints, Samples, [](const FPCGPoint& Point) { return TPair<FTransform, FBox>(Point.Transform, Point.GetLocalBounds()); });
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSpatialDataProcessing::SamplePoints::RangeFunc)
				RangeFunc(Samples, IterationPoints, RangeOutputPoints);
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSpatialDataProcessing::SamplePoints::RangeCopyResults)
				// Copy back the points
				FMemory::Memcpy((void*)(OutPoints.GetData() + StartWriteIndex), (void*)(RangeOutputPoints.GetData()), sizeof(FPCGPoint) * RangeOutputPoints.Num());
			}

			return RangeOutputPoints.Num();
		};

		auto MoveDataRange = [&OutPoints](int32 ReadIndex, int32 WriteIndex, int32 Count)
		{
			// Implementation note: if the array is to be moved to a range partially overlapping itself, it's important not to use Memcpy here
			FMemory::Memmove((void*)(OutPoints.GetData() + WriteIndex), (void*)(OutPoints.GetData() + ReadIndex), sizeof(FPCGPoint) * Count);
		};

		auto Finished = [&OutPoints](int32 Count)
		{
			// Shrinking can have a big impact on the performance, but without it, we can also hold a big chunk of wasted memory.
			// Might revisit later if the performance impact is too big.
			OutPoints.SetNum(Count);
		};

		ensure(FPCGAsync::AsyncProcessingRangeEx(AsyncState, NumIterations, Initialize, ProcessRange, MoveDataRange, Finished, /*bEnableTimeSlicing=*/false, ChunkSize, /*bAllowChunkSizeOverride=*/false));
	}

	template<int ChunkSize, typename ProcessRangeFunc>
	void SampleBasedRangeProcessing(FPCGAsyncState* AsyncState, ProcessRangeFunc&& InProcessRange, const UPCGBasePointData* SourceData, UPCGBasePointData* TargetData, EPCGPointNativeProperties PropertiesToAllocate)
	{
		const int NumIterations = SourceData->GetNumPoints();

		auto Initialize = [TargetData, SourceData, NumIterations, PropertiesToAllocate]()
		{
			TargetData->SetNumPoints(NumIterations, /*bInitializeValues=*/false);
			TargetData->AllocateProperties(PropertiesToAllocate);
			TargetData->CopyUnallocatedPropertiesFrom(SourceData);
		};

		auto ProcessRange = [RangeFunc = MoveTemp(InProcessRange), SourceData, TargetData](int32 StartReadIndex, int32 StartWriteIndex, int32 Count) -> int32
		{
			ensure(Count >= 0 && Count <= ChunkSize);

			TArray<TPair<FTransform, FBox>, TInlineAllocator<ChunkSize>> Samples;
			
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSpatialDataProcessing::SamplePoints::PrepareSamples)
				const TConstPCGValueRange<FTransform> SourceTransformRange = SourceData->GetConstTransformValueRange();
				const TConstPCGValueRange<FVector> SourceBoundsMinRange = SourceData->GetConstBoundsMinValueRange();
				const TConstPCGValueRange<FVector> SourceBoundsMaxRange = SourceData->GetConstBoundsMaxValueRange();

				Samples.Reserve(Count);
				for (int32 Index = 0; Index < Count; ++Index)
				{
					const int32 ReadIndex = StartReadIndex + Index;
					Samples.Add(TPair<FTransform, FBox>(SourceTransformRange[ReadIndex], PCGPointHelpers::GetLocalBounds(SourceBoundsMinRange[ReadIndex], SourceBoundsMaxRange[ReadIndex])));
				}
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSpatialDataProcessing::SamplePoints::RangeFunc)
				return RangeFunc(Samples, SourceData, StartReadIndex, TargetData, StartWriteIndex);
			}
		};

		auto MoveDataRange = [TargetData](int32 ReadIndex, int32 WriteIndex, int32 Count)
		{
			TargetData->MoveRange(ReadIndex, WriteIndex, Count);
		};

		auto Finished = [TargetData](int32 Count)
		{
			TargetData->SetNumPoints(Count);
		};

		ensure(FPCGAsync::AsyncProcessingRangeEx(AsyncState, NumIterations, Initialize, ProcessRange, MoveDataRange, Finished, /*bEnableTimeSlicing=*/false, ChunkSize, /*bAllowChunkSizeOverride=*/false));
	}
}