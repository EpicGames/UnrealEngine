// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/InterpCurve.h"
#include "Data/PCGPolyLineData.h"
#include "Data/PCGProjectionData.h"
#include "Data/PCGSplineStruct.h"
#include "Elements/PCGProjectionParams.h"

#include "PCGSplineData.generated.h"

#define UE_API PCG_API

class UPCGSpatialData;

class USplineComponent;
class UPCGSurfaceData;

namespace PCGSplineData
{
	const FName ControlPointDomainName = "ControlPoints";
}

USTRUCT()
struct FPCGDataTypeInfoSpline : public FPCGDataTypeInfoPolyline
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Spline)

	PCG_API virtual bool SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings, FText* OptionalOutCompatibilityMessage) const override;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSplineData : public UPCGPolyLineData
{
	GENERATED_BODY()

public:
	UE_API UPCGSplineData();

	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	/**
	 * Initialize a spline data from a Spline Component.
	 */
	UFUNCTION(BlueprintCallable, Category = "SplineData", meta = (DisplayName = "Initialize From Spline Component"))
	UE_API void Initialize(const USplineComponent* InSpline);

	/**
	 * Initialize a spline data
	 * @param InSplinePoints       List of control points.
	 * @param bInClosedLoop        If the spline is closed or not.
	 * @param InTransform          Transform of the spline
	 * @param InOptionalEntryKeys  Optional list of metadata entry keys to setup for the control points. Need to the same size as the number of control points.
	 */
	UE_API void Initialize(const TArray<FSplinePoint>& InSplinePoints, bool bInClosedLoop, const FTransform& InTransform, TArray<PCGMetadataEntryKey> InOptionalEntryKeys = {});

	/**
	 * Initialize a spline data
	 * @param InSplinePoints       List of control points.
	 * @param bInClosedLoop        If the spline is closed or not.
	 * @param InTransform          Transform of the spline
	 * @param InOptionalEntryKeys  Optional list of metadata entry keys to setup for the control points. Need to the same size as the number of control points.
	 */
	UFUNCTION(BlueprintCallable, Category = "SplineData", meta = (DisplayName = "Initialize From Spline Points", AutoCreateRefTerm = "InOptionalEntryKeys"))
	UE_API void K2_Initialize(const TArray<FSplinePoint>& InSplinePoints, bool bInClosedLoop, const FTransform& InTransform, TArray<int64> InOptionalEntryKeys);
	
	UE_API void Initialize(const FPCGSplineStruct& InSplineStruct);
	UE_API void ApplyTo(USplineComponent* InSpline) const;

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoSpline)
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;

	// To be enabled when we are sure Default translate well from Data to Elements (or we have a path for deprecation)
	//virtual FPCGMetadataDomainID GetDefaultMetadataDomainID() const override { return PCGMetadataDomainID::Elements; }
	virtual TArray<FPCGMetadataDomainID> GetAllSupportedMetadataDomainIDs() const override { return {PCGMetadataDomainID::Data, PCGMetadataDomainID::Elements}; }
	UE_API virtual FPCGMetadataDomainID GetMetadataDomainIDFromSelector(const FPCGAttributePropertySelector& InSelector) const override;
	UE_API virtual bool SetDomainFromDomainID(const FPCGMetadataDomainID& InDomainID, FPCGAttributePropertySelector& InOutSelector) const;

protected:
	virtual bool SupportsFullDataCrc() const override { return true; }
	// ~End UPCGData interface

public:
	// ~UPCGSpatialData interface
	UE_API virtual void InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const override;
	// ~End UPCGSpatialData interface
	
	//~Begin UPCGPolyLineData interface
	UE_API virtual FTransform GetTransform() const override;
	UE_API virtual int GetNumSegments() const override;
	UE_API virtual FVector::FReal GetSegmentLength(int SegmentIndex) const override;
	UE_API virtual FVector GetLocationAtDistance(int SegmentIndex, FVector::FReal Distance, bool bWorldSpace = true) const override;
	UE_API virtual FTransform GetTransformAtDistance(int SegmentIndex, FVector::FReal Distance, bool bWorldSpace = true, FBox* OutBounds = nullptr) const override;
	UE_API virtual FVector::FReal GetCurvatureAtDistance(int SegmentIndex, FVector::FReal Distance) const override;
	UE_API virtual float GetInputKeyAtDistance(int SegmentIndex, FVector::FReal Distance) const override;
	UE_API virtual float GetInputKeyAtAlpha(float Alpha) const override;
	UE_API virtual void GetTangentsAtSegmentStart(int SegmentIndex, FVector& OutArriveTangent, FVector& OutLeaveTangent) const override;
	UE_API virtual FVector::FReal GetDistanceAtSegmentStart(int SegmentIndex) const override;
	UE_API virtual FVector GetLocationAtAlpha(float Alpha) const override;
	UE_API virtual FTransform GetTransformAtAlpha(float Alpha) const override;
	virtual bool IsClosed() const override { return SplineStruct.bClosedLoop; }
	UE_API float GetLoopKeyOffset() const;
	UE_API virtual void WriteMetadataToPoint(float InputKey, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;

	virtual void AllocateMetadataEntries() override { SplineStruct.AllocateMetadataEntries(); }
	virtual TConstArrayView<PCGMetadataEntryKey> GetConstVerticesEntryKeys() const override { return SplineStruct.GetConstControlPointsEntryKeys(); }
	virtual TArrayView<PCGMetadataEntryKey> GetMutableVerticesEntryKeys() override { return SplineStruct.GetMutableControlPointsEntryKeys(); }
	//~End UPCGPolyLineData interface
	
	/** Static helper to create an accessor on a data that doesn't yet exist, as accessors for spline data don't rely on existing data. */
	static UE_API TUniquePtr<IPCGAttributeAccessor> CreateStaticAccessor(const FPCGAttributePropertySelector& InSelector, bool bQuiet = false);

	/** Helper to create an accessor keys for data that doesn't yet exist. */
	UE_API TUniquePtr<IPCGAttributeAccessorKeys> CreateAccessorKeys(const FPCGAttributePropertySelector& InSelector, bool bQuiet = false);
	UE_API TUniquePtr<const IPCGAttributeAccessorKeys> CreateConstAccessorKeys(const FPCGAttributePropertySelector& InSelector, bool bQuiet = false) const;
	
	// Get the functions to the accessor factory
	static UE_API FPCGAttributeAccessorMethods GetSplineAccessorMethods();

	//~Begin UPCGSpatialDataWithPointCache interface
	UE_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialDataWithPointCache interface

	//~Begin UPCGSpatialData interface
	UE_API virtual FBox GetBounds() const override;
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	UE_API virtual UPCGSpatialData* ProjectOn(FPCGContext* InContext, const UPCGSpatialData* InOther, const FPCGProjectionParams& InParams = FPCGProjectionParams()) const override;
protected:
	UE_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

	UE_API virtual void CopySplineData(UPCGSplineData* InCopy) const;

	UE_API const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;

public:
	UFUNCTION(BlueprintCallable, Category = "SplineData")
	UE_API TArray<FSplinePoint> GetSplinePoints() const;

	UFUNCTION(BlueprintCallable, Category = "SplineData")
	UE_API TArray<int64> GetMetadataEntryKeysForSplinePoints() const;

	UE_API void WriteMetadataToEntry(float InputKey, PCGMetadataEntryKey& OutEntryKey, UPCGMetadata* OutMetadata) const;

	// Minimal data needed to replicate the behavior from USplineComponent
	UPROPERTY()
	FPCGSplineStruct SplineStruct;

protected:
	UPROPERTY()
	FBox CachedBounds = FBox(EForceInit::ForceInit);
};

/* The projection of a spline onto a surface. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGSplineProjectionData : public UPCGProjectionData
{
	GENERATED_BODY()

public:
	PCG_API void Initialize(const UPCGSplineData* InSourceSpline, const UPCGSpatialData* InTargetSurface, const FPCGProjectionParams& InParams);

	// ~Begin UPCGData interface
	virtual EPCGDataType GetDataType() const override { return EPCGDataType::Spline; }
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	const UPCGSplineData* GetSpline() const;
	const UPCGSpatialData* GetSurface() const;

	//~Begin UPCGSpatialData interface
	virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	//~End UPCGSpatialData interface

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SpatialData)
	FInterpCurveVector2D ProjectedPosition;

protected:
	FVector2D Project(const FVector& InVector) const;

	//~Begin UPCGSpatialData interface
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface
};

#undef UE_API
