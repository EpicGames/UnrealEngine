// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGSurfaceData.h"
#include "Data/PCGSplineStruct.h"

#include "Polygon2.h"

#include "PCGSplineInteriorSurfaceData.generated.h"

class UPCGSpatialData;
class UPCGSplineData;
struct FPCGProjectionParams;

/**
* Represents a surface implicitly using the top-down 2D projection of a closed spline.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSplineInteriorSurfaceData : public UPCGSurfaceData
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.5, "Call version with FPCGContext parameter")
	void Initialize(const UPCGSplineData* InSplineData) { Initialize(nullptr, InSplineData); }

	PCG_API void Initialize(FPCGContext* Context, const UPCGSplineData* InSplineData);

	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface

	// ~Begin UPCGData interface
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	//~Begin UPCGSpatialData interface
	virtual FBox GetBounds() const override { return CachedBounds; }
	virtual FBox GetStrictBounds() const override { return CachedBounds; }
	virtual bool IsBounded() const override { return !!CachedBounds.IsValid; }
	virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	virtual bool ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	virtual bool HasNonTrivialTransform() const override { return true; }
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	// ~Begin UPCGSpatialDataWithPointCache interface
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	// ~End UPCGConcreteDataWithPointCache interface

protected:
	/** Recompute cached data used for sampling. */
	void CacheData(FPCGContext* Context);

	/** True if the given location falls in the top-down projection of the polygon given by our cached spline points. */
	bool PointInsidePolygon(const FTransform& InTransform, const FBox& InBounds) const;

	const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;

protected:
	/** Minimal data needed to replicate the behavior from USplineComponent. */
	UPROPERTY()
	FPCGSplineStruct SplineStruct;

	UPROPERTY(Transient)
	FBox CachedBounds = FBox(EForceInit::ForceInit);

	/** Cached list of points to describe the polygon given by the spline. */
	UPROPERTY(Transient)
	TArray<FVector> CachedSplinePoints;

	UE_DEPRECATED(5.8, "Not filled anymore, use CachedPolygon2D")
	UPROPERTY(Transient)
	TArray<FVector2D> CachedSplinePoints2D;

	/** Cached polygon given by the spline. */
	UE::Geometry::FPolygon2d CachedPolygon2D;

#if WITH_EDITORONLY_DATA
	/** Flag that indicates when we need to recompute our cached information. Preferable to serializing data that can be recreated on demand. */
	UPROPERTY(Transient)
	bool bNeedsToCache = true;
#endif
};
