// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Metadata/PCGMetadataCommon.h"

#include "CoreMinimal.h"
#include "Misc/SpinLock.h"
#include "Components/SplineComponent.h"

#include "PCGSplineStruct.generated.h"

#define UE_API PCG_API

UENUM()
enum class EPCGSplineStructProperties : uint8
{
	Position         UMETA(Tooltip = "Location component of the control point's transform in world coordinates."),
	Rotation         UMETA(Tooltip = "Rotation component of the control point's transform in world coordinates."),
	Scale            UMETA(Tooltip = "Scale component of the control point's transform in world coordinates."),
	Transform        UMETA(Tooltip = "Control point's world transform."),
	ArriveTangent    UMETA(Tooltip = "Arrive Tangent at the control point."),
	LeaveTangent     UMETA(Tooltip = "Leave Tangent at the control point."),
	InterpType       UMETA(Tooltip = "Type of interpolation at the control point for position (same as set on the spline control point). Underlying enum: ESplinePointType."),
	LocalPosition    UMETA(Tooltip = "Location component of the control point's transform."),
	LocalRotation    UMETA(Tooltip = "Rotation component of the control point's transform."),
	LocalScale       UMETA(Tooltip = "Scale component of the control point's transform."),
	LocalTransform   UMETA(Tooltip = "Control point's transform."),
};

UENUM()
enum class EPCGSplineDataProperties : uint8
{
	SplineTransform  UMETA(Tooltip = "Transform of the spline.", PCG_MetadataDomain="Data"),
	IsClosed         UMETA(Tooltip = "If the spline is closed.", PCG_MetadataDomain="Data"),
	Length           UMETA(Tooltip = "Length of the spline", PCG_MetadataDomain="Data")
};

enum class EPCGControlPointsAccessorTarget
{
	Location,
	Rotation,
	Scale,
	Transform,
	ArriveTangent,
	LeaveTangent,
	InterpMode
};

/** Subset of the Spline Component API in a standalone struct */
USTRUCT()
struct FPCGSplineStruct
{
	GENERATED_BODY()

	UE_API void Initialize(const USplineComponent* InSplineComponent);
	UE_API void Initialize(const TArray<FSplinePoint>& InSplinePoints, bool bIsClosedLoop, const FTransform& InTransform, TArray<PCGMetadataEntryKey> InOptionalEntryKeys = {});
	UE_API void ApplyTo(USplineComponent* InSplineComponent) const;

	FPCGSplineStruct() = default;
	UE_API FPCGSplineStruct(const FPCGSplineStruct& Other);
	UE_API FPCGSplineStruct(FPCGSplineStruct&& Other);
	UE_API FPCGSplineStruct& operator=(const FPCGSplineStruct& Other);
	UE_API FPCGSplineStruct& operator=(FPCGSplineStruct&& Other);
	
	FTransform GetTransform() const { return Transform; }

	// Spline-related methods
	UE_API void AddPoint(const FSplinePoint& InSplinePoint, bool bUpdateSpline, const int64* InOptionalEntryKey = nullptr); 
	UE_API void AddPoints(const TArray<FSplinePoint>& InSplinePoints, bool bUpdateSpline, const TArray<int64>* InOptionalEntryKeys = nullptr);

	UE_API void SetLocation(int32 Index, const FVector& InLocation);
	UE_API FVector GetLocation(const int32 Index) const;
	
	UE_API void SetArriveTangent(const int32 Index, const FVector& InArriveTangent);
	UE_API FVector GetArriveTangent(const int32 Index) const;
	
	UE_API void SetLeaveTangent(const int32 Index, const FVector& LeaveTangent);
	UE_API FVector GetLeaveTangent(const int32 Index) const;

	UE_API void SetRotation(int32 Index, const FQuat& InRotation);
	UE_API FQuat GetRotation(const int32 Index) const;
	
	UE_API void SetScale(int32 Index, const FVector& InScale);
	UE_API FVector GetScale(const int32 Index) const;
	
	UE_API void SetSplinePointType(int32 Index, EInterpCurveMode Type);
	UE_API EInterpCurveMode GetSplinePointType(int32 Index) const;
	
	// Gets a lexicographically sorted list of channel names.
	UE_API const TArray<FName>& GetAttributeChannelNames() const;

	// Evaluate the specified attribute channel at the specified spline input key.
	UE_API float GetAttributeValue(float InputKey, FName ChannelName) const;
	
	UE_API void UpdateSpline();
	UE_API void SetLoopKeyOffset(float LoopKeyOffset);

	UE_API int GetNumberOfSplineSegments() const;
	UE_API int GetNumberOfPoints() const;
	bool IsClosedLoop() const { return bClosedLoop; }
	UE_API float GetLoopKeyOffset() const;
	UE_API FVector::FReal GetSplineLength() const;
	UE_API FBox GetBounds() const;
	UE_API FBox GetSegmentBounds(int32 Index) const;

	UE_API const FInterpCurveVector& GetSplinePointsScale() const;
	UE_API const FInterpCurveQuat& GetSplinePointsRotation() const;
	UE_API const FInterpCurveVector& GetSplinePointsPosition() const;

	UE_API FVector GetRightVectorAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;
	UE_API FTransform GetTransformAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale = false) const;
	UE_API FVector GetLocationAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;
	UE_API FQuat GetQuaternionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;
	UE_API FVector GetScaleAtSplineInputKey(float InKey) const;
	UE_API FVector GetTangentAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace) const;

	UE_API FVector::FReal GetDistanceAlongSplineAtSplinePoint(int32 PointIndex) const;
	UE_API FVector GetLocationAtDistanceAlongSpline(FVector::FReal Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;
	UE_API FTransform GetTransformAtDistanceAlongSpline(FVector::FReal Distance, ESplineCoordinateSpace::Type CoordinateSpace, bool bUseScale = false) const;
	UE_API float GetInputKeyAtDistanceAlongSpline(FVector::FReal Distance) const;

	/** Given a threshold, returns a list of vertices along the spline segment that, treated as a list of segments (polyline), matches the spline shape. Taken from USplineComponent. */
	UE_API bool ConvertSplineSegmentToPolyLine(int32 SplinePointStartIndex, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const;

	/** Given a threshold, returns a list of vertices along the spline that, treated as a list of segments (polyline), matches the spline shape. Taken from USplineComponent. */
	UE_API bool ConvertSplineToPolyLine(ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints) const;
	
	UE_API float FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const;
	UE_API float FindInputKeyOnSegmentClosestToWorldLocation(const FVector& WorldLocation, int32 Index) const;

	/** Return the index of the segment for a given InputKey, and the input key at the segment start.
	 * It's for future-proof-ness as even if there is no guarantee a segment start is at an integer input key, right now it is not well supported.
	 * If the input key is invalid, returns {INDEX_NONE, 0}.
	 */
	UE_API TTuple<int, float> GetSegmentStartIndexAndKeyAtInputKey(float InKey) const;

	/** Return the input key at segment start for the given segment. Takes into account if the spline is closed. */
	UE_API float GetInputKeyAtSegmentStart(int InSegmentIndex) const;

	/** To be called at any moment we want to access individual metadata, like with accessors. Does nothing if it was already allocated. */
	UE_API void AllocateMetadataEntries();
	TConstArrayView<PCGMetadataEntryKey> GetConstControlPointsEntryKeys() const { return ControlPointsEntryKeys; }
	TArrayView<PCGMetadataEntryKey> GetMutableControlPointsEntryKeys() { return ControlPointsEntryKeys; }

	UE_API void PostSerialize(const FArchive& Ar);
	
	// Replaces the component transform
	UPROPERTY()
	FTransform Transform = FTransform::Identity;

	UPROPERTY()
	FVector DefaultUpVector = FVector::UpVector;

	UPROPERTY()
	int32 ReparamStepsPerSegment = 10;

	UPROPERTY()
	bool bClosedLoop = false;

	UPROPERTY()
	FBoxSphereBounds LocalBounds = FBoxSphereBounds(EForceInit::ForceInit);

	UPROPERTY()
	FBoxSphereBounds Bounds = FBoxSphereBounds(EForceInit::ForceInit);

protected:
	// todo: deprecate, serialize into FSpline
	UPROPERTY()
	FSplineCurves SplineCurves;
	
	UPROPERTY()
	TArray<int64> ControlPointsEntryKeys; // Needs to be int64 for UHT, but it is a PCGMetadataEntryKey

private:
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UPROPERTY()
	FSpline Spline;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	mutable PCG::FLock Lock;
	mutable TArray<FName> CachedSortedAttributeChannelNames;

	UE_API bool ShouldUseSplineCurves() const;
	
	// Remove when appending _DEPRECATED to SplineCurves.
	FSplineCurves& WarninglessSplineCurves()
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SplineCurves;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// Remove when appending _DEPRECATED to SplineCurves.
	const FSplineCurves& WarninglessSplineCurves() const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SplineCurves;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	// Internal helper function called by ConvertSplineSegmentToPolyLine -- assumes the input is within a half-segment, so testing the distance to midpoint will be an accurate guide to subdivision. Taken from USplineComponent.
	UE_API bool DivideSplineIntoPolylineRecursiveWithDistancesHelper(float StartDistanceAlongSpline, float EndDistanceAlongSpline, ESplineCoordinateSpace::Type CoordinateSpace, const float MaxSquareDistanceFromSpline, TArray<FVector>& OutPoints, TArray<double>& OutDistancesAlongSpline) const;
};

template<>
struct TStructOpsTypeTraits<FPCGSplineStruct> : public TStructOpsTypeTraitsBase2<FPCGSplineStruct>
{
	enum
	{
		WithPostSerialize = true,
	};
};

#undef UE_API
