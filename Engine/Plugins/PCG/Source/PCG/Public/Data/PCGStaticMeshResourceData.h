// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGResourceData.h"

#include "PCGStaticMeshResourceData.generated.h"

#define UE_API PCG_API

class UStaticMesh;

USTRUCT()
struct FPCGDataTypeInfoStaticMeshResource : public FPCGDataTypeInfoResource
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::StaticMeshResource)

#if WITH_EDITOR
	virtual bool Hidden() const override { return false; }
#endif // WITH_EDITOR
};

/** Data that wraps a Static Mesh soft object path. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGStaticMeshResourceData : public UPCGResourceData
{
	GENERATED_BODY()

public:
	UE_API void Initialize(TSoftObjectPtr<UStaticMesh> InStaticMesh);

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoStaticMeshResource)
	//~ End UPCGData interface

	//~ Begin UPCGResourceData interface
	UE_API virtual FSoftObjectPath GetResourcePath() const override;
	//~ End UPCGResourceData interface

protected:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = StaticMesh)
	TSoftObjectPtr<UStaticMesh> StaticMesh = nullptr;
};

#undef UE_API
