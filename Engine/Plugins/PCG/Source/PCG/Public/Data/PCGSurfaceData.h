// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialData.h"

#include "PCGSurfaceData.generated.h"

#define UE_API PCG_API

USTRUCT()
struct FPCGDataTypeInfoSurface : public FPCGDataTypeInfoConcrete
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Surface)
};

UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = (Procedural))
class UPCGSurfaceData : public UPCGSpatialDataWithPointCache
{
	GENERATED_BODY()

public:
	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoSurface)
	// ~End UPCGData interface

	//~Begin UPCGSpatialData interface
	virtual int GetDimension() const override { return 2; }
	virtual bool HasNonTrivialTransform() const override { return true; }
	//~End UPCGSpatialData interface

	const FTransform& GetTransform() const { return Transform; }
	const FBox& GetLocalBounds() const { return LocalBounds; }

protected:
	UE_API void CopyBaseSurfaceData(UPCGSurfaceData* NewSurfaceData) const;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	FTransform Transform;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	FBox LocalBounds = FBox(EForceInit::ForceInit);
};

#undef UE_API
