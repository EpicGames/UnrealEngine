// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGTexture2DBaseData.h"

#include "PCGCommon.h"

#include "RendererInterface.h"
#include "RHI.h"

#include "PCGTexture2DArrayData.generated.h"

class UPCGBasePointData;
class UPCGMetadata;
class UPCGPointArrayData;
class UPCGPointData;
class UTexture;
struct FPCGContext;
struct FPCGPoint;
struct IPooledRenderTarget;

USTRUCT(meta = (PCG_DataTypeDisplayName = "Texture 2D Array"))
struct FPCGDataTypeInfoTexture2DArray : public FPCGDataTypeInfoTexture2DBase
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO()
};

struct FPCGTexture2DArrayDataInitParams
{
	FTransform Transform = FTransform::Identity;
	EPCGTextureFilter Filter = EPCGTextureFilter::Bilinear;
};

UCLASS(MinimalAPI)
class UPCGTexture2DArrayData : public UPCGTexture2DBaseData
{
	GENERATED_BODY()

public:
	//~Begin UPCGData interface
	PCG_ASSIGN_TYPE_INFO(FPCGDataTypeInfoTexture2DArray);
	virtual bool IsCacheable() const { return Super::IsCacheable() && ResourceType != EPCGTextureResourceType::ExportedTexture; }
	virtual void ReleaseTransientResources(const TCHAR* InReason = nullptr) override { TextureHandle = nullptr; }
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	//~End UPCGData interface

	//~Begin UPCGSpatialData interface
	virtual bool SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override { return false; }

protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	//~Begin UPCGSpatialDataWithPointCache interface
	PCG_API virtual const UPCGPointData* CreatePointData(FPCGContext* InContext) const override;
	PCG_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* InContext, const FBox& InBounds) const override;
	//~End UPCGSpatialDataWithPointCache interface

	PCG_API bool Initialize(UTexture* InTexture, const FPCGTexture2DArrayDataInitParams& InTransform);
	PCG_API bool Initialize(TRefCountPtr<IPooledRenderTarget> InExportedTextureHandle, const FPCGTexture2DArrayDataInitParams& InTransform);
	UTexture* GetTexture() const { return Texture.Get(); }
	PCG_API FTextureRHIRef GetRHI() const;
	EPCGTextureResourceType GetResourceType() const { return ResourceType; }
	FIntPoint GetResolution() const override { return Resolution; }
	virtual uint16 GetArraySize() const override { return ArraySize; }
	EPCGTextureFilter GetFilter() const { return Filter; }

protected:
	PCG_API bool InitializeInternal(const FPCGTexture2DArrayDataInitParams& InParams);
	PCG_API const UPCGBasePointData* CreateBasePointData(FPCGContext* InContext, TSubclassOf<UPCGBasePointData> InPointDataClass) const;
	PCG_API void CopyTexture2DArrayData(UPCGTexture2DArrayData* NewTextureData) const;

protected:
	UPROPERTY()
	TWeakObjectPtr<UTexture> Texture = nullptr;

	/** The type of underlying resource that this texture data represents. */
	UPROPERTY()
	EPCGTextureResourceType ResourceType = EPCGTextureResourceType::TextureObject;

	UPROPERTY()
	FIntPoint Resolution = FIntPoint::ZeroValue;

	UPROPERTY()
	uint16 ArraySize = 1;
};
