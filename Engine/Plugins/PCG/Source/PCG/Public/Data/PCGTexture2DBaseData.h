// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGSurfaceData.h"

#include "PCGCommon.h"

#include "PixelFormat.h"
#include "RendererInterface.h"
#include "RHI.h"

#include "PCGTexture2DBaseData.generated.h"

/** Base type of 2D textures, 2D texture arrays, 2d render targets. */
USTRUCT(meta = (PCG_DataTypeDisplayName = "Texture 2D Base"))
struct FPCGDataTypeInfoTexture2DBase : public FPCGDataTypeInfoSurface
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO()
};

/** Base class of 2D textures, 2D texture arrays, 2d render targets. */
UCLASS(MinimalAPI, Abstract)
class UPCGTexture2DBaseData : public UPCGSurfaceData
{
	GENERATED_BODY()

public:
	//~ Begin UObject interface
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject interface

	// ~Begin UPCGData interface
	PCG_ASSIGN_TYPE_INFO(FPCGDataTypeInfoTexture2DBase)
	// ~End UPCGData interface

	//~ Begin UPCGSpatialData interface
	virtual FBox GetBounds() const override { return Bounds; }
	virtual FBox GetStrictBounds() const override { return Bounds; }
	//~ End UPCGSpatialData interface

	virtual EPixelFormat GetFormat() const { return Format; }
	virtual EPCGTextureFilter GetFilter() const { return Filter; }
	virtual uint16 GetArraySize() const { return 1; }
	virtual uint8 GetNumMips() const { return NumMips; }
	virtual FIntPoint GetResolution() const PURE_VIRTUAL(UPCGTexture2DBaseData::GetResolution, return FIntPoint::ZeroValue;);
	virtual TRefCountPtr<IPooledRenderTarget> GetRefCountedTexture() const { return TextureHandle; }

	/** Helper to compute the texture data transform based on the execution source transform and bounds. */
	static FTransform ComputeTransform(const IPCGGraphExecutionSource* InExecutionSource);

protected:
	PCG_API void CopyTexture2DBaseData(UPCGTexture2DBaseData* NewTextureData) const;

protected:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	TEnumAsByte<EPixelFormat> Format = EPixelFormat::PF_Unknown;

	UPROPERTY()
	uint8 NumMips = 0;

public:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	EPCGTextureFilter Filter = EPCGTextureFilter::Bilinear;

protected:
	/** If initialized from an exported texture this holds a reference to the resource. */
	TRefCountPtr<IPooledRenderTarget> TextureHandle = nullptr;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	FBox Bounds = FBox(EForceInit::ForceInit);
};
