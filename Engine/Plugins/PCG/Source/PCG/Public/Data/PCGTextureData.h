// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGTexture2DBaseData.h"

#include "PCGCommon.h"

#include "PixelFormat.h"
#include "RendererInterface.h"
#include "RHI.h"

#include <atomic>

#include "PCGTextureData.generated.h"

#define UE_API PCG_API

class UPCGSpatialData;
class UTexture;
class UTexture2D;

UENUM(BlueprintType)
enum class EPCGTextureColorChannel : uint8
{
	Red,
	Green,
	Blue,
	Alpha
};

UENUM(BlueprintType)
enum class UE_DEPRECATED(5.5, "EPCGTextureDensityFunction has been deprecated.") EPCGTextureDensityFunction : uint8
{
	Ignore,
	Multiply
};

UENUM()
enum class EPCGTextureAddressMode : uint8
{
	Clamp UMETA(ToolTip = "Clamps UV to 0-1."),
	Wrap UMETA(ToolTip = "Tiles the texture to fit.")
};

namespace PCGTextureSamplingHelpers
{
	/** Returns true if a texture is CPU-accessible. */
	TOptional<bool> IsTextureCPUAccessible(UTexture2D* Texture);

	/** Returns true if a texture is both GPU-accessible and reachable from CPU memory. */
	TOptional<bool> CanGPUTextureBeCPUAccessed(UTexture2D* Texture);
}

/** Base type of 2D textures/render targets. */
USTRUCT(meta = (PCG_DataTypeDisplayName = "Single Texture 2D Base"))
struct FPCGDataTypeInfoTexture2DSingleBase : public FPCGDataTypeInfoTexture2DBase
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::BaseTexture)
};

/** Base class for a 2D texture or render target. */
UCLASS(MinimalAPI, Abstract)
class UPCGTexture2DSingleBaseData : public UPCGTexture2DBaseData
{
	GENERATED_BODY()

public:
	//~Being UObject interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~End UObject interface

	//~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoTexture2DSingleBase)
	//~End UPCGData interface

	//~Begin UPCGSpatialData interface
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	//~End UPCGSpatialData interface

	//~Begin UPCGSpatialDataWithPointCache interface
	UE_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialDataWithPointCache interface

	//~Begin UPCGTexture2DBaseData interface
	virtual FIntPoint GetResolution() const override { return FIntPoint(Width, Height); }
	//~End UPCGTexture2DBaseData interface

	/** Sample using a local space 'UV' position. */
	UE_API bool SamplePointLocal(const FVector2D& LocalPosition, FVector4& OutColor, float& OutDensity) const;

	UE_API virtual bool IsValid() const;

	/** Returns true if the readback process has concluded (success, failure, or not needed). */
	bool IsReadbackComplete() const { return bIsReadbackComplete; }

	/** Trigger async readback if needed. Returns true if CPU data is available now. */
	UE_API bool RequestCPUReadback() const;

	virtual UTexture* GetTexture() const PURE_VIRTUAL(UPCGTexture2DSingleBaseData::GetTexture, return nullptr;)
	virtual FTextureRHIRef GetTextureRHI() const PURE_VIRTUAL(UPCGTexture2DSingleBaseData::GetTextureResource, return nullptr;);
	virtual EPCGTextureResourceType GetTextureResourceType() const PURE_VIRTUAL(UPCGTexture2DSingleBaseData::GetTextureResourceType, return EPCGTextureResourceType::Invalid;);
	virtual int GetTextureSlice() const { return 0; }

protected:
	virtual bool RequestCPUReadback() PURE_VIRTUAL(UPCGTexture2DSingleBaseData::RequestCPUReadback, return false;);
	UE_API const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UFUNCTION(BlueprintGetter, meta = (BlueprintInternalUseOnly = "true"))
	UE_API EPCGTextureDensityFunction GetDensityFunctionEquivalent() const;

	UFUNCTION(BlueprintSetter, meta = (BlueprintInternalUseOnly = "true"))
	UE_API void SetDensityFunctionEquivalent(EPCGTextureDensityFunction DensityFunction);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "DensityFunction has been deprecated in favor of bUseDensitySourceChannel.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(BlueprintGetter = GetDensityFunctionEquivalent, BlueprintSetter = SetDensityFunctionEquivalent, Category = SpatialData, meta = (DeprecatedProperty, DeprecatedMessage = "Density function on GetTextureData is deprecated in favor of bUseDensitySourceChannel."))
	EPCGTextureDensityFunction DensityFunction = EPCGTextureDensityFunction::Multiply; 
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bUseDensitySourceChannel = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (DisplayName = "Density Source Channel", EditCondition = "bUseDensitySourceChannel"))
	EPCGTextureColorChannel ColorChannel = EPCGTextureColorChannel::Alpha;

	/** The size of one texel in cm, used when calling ToPointData. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (UIMin = "1.0", ClampMin = "1.0"))
	float TexelSize = 50.0f;

	/** Whether to tile the source or to stretch it to fit target area. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bUseAdvancedTiling = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Tiling", meta = (EditCondition = "bUseAdvancedTiling"))
	FVector2D Tiling = FVector2D(1.0, 1.0);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Tiling", meta = (EditCondition = "bUseAdvancedTiling"))
	FVector2D CenterOffset = FVector2D::ZeroVector;

	/** Rotation to apply when sampling texture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (UIMin = -360, ClampMin = -360, UIMax = 360, ClampMax = 360, Units = deg, EditCondition = "bUseAdvancedTiling"))
	float Rotation = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Tiling", meta = (EditionCondition = "bUseAdvancedTiling"))
	bool bUseTileBounds = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Tiling", meta = (EditCondition = "bUseAdvancedTiling && bUseTileBounds"))
	FBox2D TileBounds = FBox2D(FVector2D(-0.5, -0.5), FVector2D(0.5, 0.5));

protected:
	UPROPERTY()
	TArray<FLinearColor> ColorData;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	int32 Height = 0;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	int32 Width = 0;

	/** Whether an async readback has been dispatched. */
	bool bReadbackInitiated = false;

	/** True once the readback process has concluded (success, failure, or not needed). */
	std::atomic<bool> bIsReadbackComplete = false;

	/** Used to make sure errors are only logged once when trying to sample points from a data which hasn't been read back into a CPU buffer. */
	mutable bool bEmittedNoReadbackDataError = false;

	UE_API void CopyBaseTextureData(UPCGTexture2DSingleBaseData* NewTextureData) const;

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "Use GetResolution instead")
	virtual FIntPoint GetTextureSize() const { return GetResolution(); }
	UE_DEPRECATED(5.8, "Use GetFormat instead")
	EPixelFormat GetTextureFormat() const { return Format; }

protected:
	UE_DEPRECATED(5.8, "bSkipReadbackToCPU has been removed. CPU readback is now always deferred.")
	bool bSkipReadbackToCPU = false;
};

USTRUCT(meta = (PCG_DataTypeDisplayName = "Texture 2D"))
struct FPCGDataTypeInfoTexture2D : public FPCGDataTypeInfoTexture2DSingleBase
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Texture)
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGTextureData : public UPCGTexture2DSingleBaseData // Texture2D
{
	GENERATED_BODY()

public:
	/**
	 * Initialize this data. CPU readback is always deferred, it will be triggered lazily when a downstream consumer needs CPU data.
	 * Can depend on async texture operations. Should be polled until it returns true signaling completion,
	 * and then IsSuccessfullyInitialized() is used to verify the initialization was successful and data is ready to use.
	 */
	PCG_API bool Initialize(UTexture* InTexture, uint32 InTextureIndex, const FTransform& InTransform, bool bCreateCPUDuplicateEditorOnly = false);
	PCG_API bool Initialize(TRefCountPtr<IPooledRenderTarget> InTextureHandle, uint32 InTextureIndex, const FTransform& InTransform);

	UE_DEPRECATED(5.8, "bSkipReadbackToCPU has been removed. CPU readback is now always deferred. Use the overload without bSkipReadbackToCPU.")
	PCG_API bool Initialize(UTexture* InTexture, uint32 InTextureIndex, const FTransform& InTransform, bool bCreateCPUDuplicateEditorOnly, bool bInSkipReadbackToCPU);
	UE_DEPRECATED(5.8, "bSkipReadbackToCPU has been removed. CPU readback is now always deferred. Use the overload without bSkipReadbackToCPU.")
	PCG_API bool Initialize(TRefCountPtr<IPooledRenderTarget> InTextureHandle, uint32 InTextureIndex, const FTransform& InTransform, bool bInSkipReadbackToCPU);

	//~Begin UPCGTexture2DSingleBaseData interface
	UE_API virtual bool RequestCPUReadback() override;
	//~End UPCGTexture2DSingleBaseData interface

	/** Data is successfully initialized and is ready to use. */
	bool IsSuccessfullyInitialized() const { return bSuccessfullyInitialized; }

	EPCGTextureResourceType GetTextureResourceType() const override { return ResourceType; }

	//~Begin UObject Interface
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~End UObject Interface

	//~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoTexture2D)
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	virtual bool IsCacheable() const { return Super::IsCacheable() && ResourceType != EPCGTextureResourceType::ExportedTexture; }
	virtual void ReleaseTransientResources(const TCHAR* InReason = nullptr) override { TextureHandle = nullptr; }
	//~End UPCGData interface

	//~Begin UPCGTexture2DSingleBaseData interface
	virtual UTexture* GetTexture() const override { return Texture.IsValid() ? Texture.Get() : nullptr; }
	virtual FTextureRHIRef GetTextureRHI() const override;
	virtual TRefCountPtr<IPooledRenderTarget> GetRefCountedTexture() const override { return TextureHandle; }
	virtual int GetTextureSlice() const override { return TextureIndex; }
	//~End UPCGTexture2DSingleBaseData interface
	
	//~Begin UPCGSpatialData interface
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

	void InitializeInternal(UTexture* InTexture, uint32 InTextureIndex, const FTransform& InTransform, bool* bOutInitializeDone, bool bCreateCPUDuplicateEditorOnly);

private:
	/** Attempts to populate ColorData directly from a CPU-accessible texture. Unset while waiting, true on success, false if path does not apply. */
	TOptional<bool> TryReadbackFromCPUTexture();

	/** Dispatch async GPU readback to populate ColorData. Returns false if readback is in-flight. */
	bool DispatchReadbackFromGPU();

#if WITH_EDITOR
	/** Attempts to populate ColorData from a CPU-accessible texture (or its duplicate) via direct BulkData access. Unset while waiting, true on success, false if path does not apply. */
	TOptional<bool> TryReadbackFromTextureBulkData();

	/** Sets up the CPU-accessible duplicate texture used by bCreateCPUDuplicateEditorOnly. Returns true when prep is complete. */
	bool PrepareCPUDuplicateTexture();
#endif

public:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Properties)
	TWeakObjectPtr<UTexture> Texture = nullptr;

#if WITH_EDITORONLY_DATA
	/** Transient CPU visible duplicate of Texture created and used only when initialized with bCreateCPUDuplicateEditorOnly. */
	UPROPERTY(Transient)
	TObjectPtr<UTexture2D> DuplicateTexture = nullptr;

	UPROPERTY(Transient)
	bool bDuplicateTextureInitialized = false;
#endif

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Properties)
	int TextureIndex = 0;

	UPROPERTY(Transient, BlueprintReadOnly, VisibleAnywhere, Category = Properties)
	bool bSuccessfullyInitialized = false;

	bool bReadbackFromGPUInitiated = false;

protected:
	/** The type of underlying resource that this texture data represents. */
	UPROPERTY()
	EPCGTextureResourceType ResourceType = EPCGTextureResourceType::TextureObject;

	bool bUpdatedReadbackTextureResource = false;
};

// Deprecated section
UCLASS(MinimalAPI, Abstract, meta = (DeprecationMessage = "Deprecated 5.8, use UPCGTexture2DSingleBaseData instead"))
class UPCGBaseTextureData : public UPCGTexture2DBaseData
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.8, "Use UPCGTexture2DSingleBaseData instead")
	UPCGBaseTextureData() {}
};

#undef UE_API
