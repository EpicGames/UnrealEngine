// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGModule.h"

#include "StructUtils/InstancedStruct.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "PCGToolData.generated.h"

class AActor;
class USplineComponent;
class UPCGPointArrayData;
class UPCGComponent;

USTRUCT()
struct FPCGInteractiveToolWorkingBaseData
{
	GENERATED_BODY()

	FPCGInteractiveToolWorkingBaseData() = default;
	virtual ~FPCGInteractiveToolWorkingBaseData() = default;

	PCG_API FName GetWorkingDataIdentifier() const;

protected:
	/** Used to identify this working data struct. Pattern: {ToolTag}.{DataInstanceIdentifier}.
	 *  If there is no DataInstanceIdentifier, will just use {ToolTag}. Changing this will remap data. */
	UPROPERTY(EditAnywhere, Category = "Tool")
	FName WorkingDataIdentifier = NAME_None;
};

USTRUCT()
struct FPCGGraphInstanceToolDataOverrides
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Tool)
	FText DisplayName;

	UPROPERTY(EditAnywhere, Category = Tool, meta=(MultiLine))
	FText Tooltip;
	
	/** Whether the graph instance this belongs to is a tool preset. */
	UPROPERTY(EditAnywhere, Category = Tool)
	bool bIsPreset = false;
};

/** The tool data stored on the graph itself. Informs the UI. */
USTRUCT(BlueprintType)
struct FPCGGraphToolData
{
	GENERATED_BODY()

	PCG_API FPCGGraphToolData();
	virtual ~FPCGGraphToolData() = default;
	
	UPROPERTY(EditAnywhere, Category = Tool)
	FText DisplayName;

	UPROPERTY(EditAnywhere, Category = Tool, meta=(MultiLine))
	FText Tooltip;
	
	UPROPERTY(EditAnywhere, Category = Tool)
	TArray<FString> CompatibleToolTags;

	UPROPERTY(EditAnywhere, Category = Tool)
	TSubclassOf<AActor> InitialActorClassToSpawn;

	UPROPERTY(EditAnywhere, Category = Tool)
	FName NewActorLabel;

	UPROPERTY(EditAnywhere, Category = Tool)
	bool bIsPreset = false;

	void ApplyOverrides(const FPCGGraphInstanceToolDataOverrides& Overrides)
	{
		// Only override the display name if it was actually specified
		if (Overrides.DisplayName.IsEmpty())
		{
			DisplayName = Overrides.DisplayName;
		}

		// Only override the tooltip if it was actually specified
		if (Overrides.Tooltip.IsEmpty())
		{
			Tooltip = Overrides.Tooltip;
		}

		// Always override the preset value
		bIsPreset = Overrides.bIsPreset;
	}
};

USTRUCT(BlueprintType)
struct FPCGInteractiveToolDataContainer
{
	GENERATED_BODY()

	FPCGInteractiveToolDataContainer() = default;
	virtual ~FPCGInteractiveToolDataContainer() = default;

	PCG_API const TInstancedStruct<FPCGInteractiveToolWorkingBaseData>* FindToolDataStruct(FName WorkingDataIdentifier) const;
	PCG_API TInstancedStruct<FPCGInteractiveToolWorkingBaseData>* FindMutableToolDataStruct(FName WorkingDataIdentifier);

	/** Removes all Tool Data with a given Working Data Identifier.
	 *  @returns number of elements removed. Should generally not be more than 1. */
	PCG_API int32 RemoveToolData(FName WorkingDataIdentifier);
	
	template<typename T = FPCGInteractiveToolWorkingBaseData>
	const T* GetTypedWorkingData(FName WorkingDataIdentifier = NAME_None) const
	{
		static_assert(TIsDerivedFrom<T, FPCGInteractiveToolWorkingBaseData>::IsDerived, "T needs to be derived from FPCGInteractiveToolWorkingBaseData.");

		if (const TInstancedStruct<FPCGInteractiveToolWorkingBaseData>* Data = FindToolDataStruct(WorkingDataIdentifier))
		{
			if (Data->IsValid() && Data->GetScriptStruct()->IsChildOf(T::StaticStruct()))
			{
				return Data->GetPtr<T>();
			}
		}

		return nullptr;
	}
	
	template<typename T = FPCGInteractiveToolWorkingBaseData>
	T* GetMutableTypedWorkingData(FName WorkingDataIdentifier = NAME_None)
	{
		static_assert(TIsDerivedFrom<T, FPCGInteractiveToolWorkingBaseData>::IsDerived, "T needs to be derived from FPCGInteractiveToolWorkingBaseData.");

		if (TInstancedStruct<FPCGInteractiveToolWorkingBaseData>* Data = FindMutableToolDataStruct(WorkingDataIdentifier))
		{
			if (Data->IsValid() && Data->GetScriptStruct()->IsChildOf(T::StaticStruct()))
			{
				return Data->GetMutablePtr<T>();
			}
		}

		return nullptr;
	}
	
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Tool Data")
	TArray<TInstancedStruct<FPCGInteractiveToolWorkingBaseData>> ToolData;
};