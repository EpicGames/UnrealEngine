// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialData.h"

#include "PCGUnionData.generated.h"

#define UE_API PCG_API

UENUM()
enum class EPCGUnionType : uint8
{
	LeftToRightPriority,
	RightToLeftPriority,
	KeepAll,
};

UENUM()
enum class EPCGUnionDensityFunction : uint8
{
	Maximum,
	ClampedAddition,
	Binary
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGUnionData : public UPCGSpatialDataWithPointCache
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = SpatialData)
	UE_API void Initialize(const UPCGSpatialData* InA, const UPCGSpatialData* InB);

	UFUNCTION(BlueprintCallable, Category = SpatialData)
	UE_API void AddData(const UPCGSpatialData* InData);

	void SetType(EPCGUnionType InUnionType) { UnionType = InUnionType; }
	void SetDensityFunction(EPCGUnionDensityFunction InDensityFunction) { DensityFunction = InDensityFunction; }

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoComposite)
	UE_API virtual void VisitDataNetwork(TFunctionRef<void(const UPCGData*)> Action) const override;

protected:
	UE_API virtual FPCGCrc ComputeCrc(bool bFullDataCrc) const override;
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

public:
	//~Begin UPCGSpatialData interface
	UE_API virtual int GetDimension() const override;
	UE_API virtual FBox GetBounds() const override;
	UE_API virtual FBox GetStrictBounds() const override;
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	UE_API virtual bool HasNonTrivialTransform() const override;
	UE_API virtual const UPCGSpatialData* FindFirstConcreteShapeFromNetwork() const override;
	UE_API virtual void InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const override;
protected:
	UE_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	//~Begin UPCGSpatialDataWithPointCache interface
	UE_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialDataWithPointCache interface

protected:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = SpatialData)
	TArray<TObjectPtr<const UPCGSpatialData>> Data;

	UPROPERTY()
	TObjectPtr<const UPCGSpatialData> FirstNonTrivialTransformData = nullptr;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGUnionType UnionType = EPCGUnionType::LeftToRightPriority;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGUnionDensityFunction DensityFunction = EPCGUnionDensityFunction::Maximum;

	UPROPERTY()
	FBox CachedBounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	FBox CachedStrictBounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	int CachedDimension = 0;

private:
	UE_API const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass, TFunctionRef<const UPCGBasePointData* (FPCGContext*, const UPCGSpatialData*)> ToPointDataFunc) const;
	UE_API void CreateSequentialPointData(FPCGContext* Context, TArray<const UPCGSpatialData*>& DataRawPtr, TArray<const UPCGMetadata*>& InputMetadatas, UPCGBasePointData* PointData, UPCGMetadata* OutMetadata, bool bLeftToRight, TFunctionRef<const UPCGBasePointData* (FPCGContext*, const UPCGSpatialData*)> ToPointDataFunc) const;
};

#undef UE_API
