// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"

#include "StructUtils/InstancedStruct.h"

#include "PCGUserParametersData.generated.h"

class UPCGGraphInterface;

/**
* PCG Data meant only to be used internally.
* It contains a copy of UserParameters for a given graph instance, with overrides in it.
* The idea is to have a structure to hold our overrides, provided by the override pins on the Subgraph
* and use this as input to PCGUserParametersGetElement. By doing so, we are able to provide the right
* parameters to the getter node.
*/
UCLASS(BlueprintType, HideDropdown, ClassGroup = (Procedural))
class UPCGUserParametersData : public UPCGData
{
	GENERATED_BODY()

public:
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoOther)

	// ~Begin UPCGData interface
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override { AddUIDToCrc(Ar); }
	// ~End UPCGData interface

	UPROPERTY()
	FInstancedStruct UserParameters;

	UPROPERTY()
	TSoftObjectPtr<UPCGUserParametersData> UpstreamData = nullptr;
};