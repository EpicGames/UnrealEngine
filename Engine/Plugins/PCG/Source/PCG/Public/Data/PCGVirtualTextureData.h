// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGSurfaceData.h"

#include "PCGVirtualTextureData.generated.h"

class URuntimeVirtualTexture;
class URuntimeVirtualTextureComponent;

USTRUCT()
struct FPCGDataTypeInfoVirtualTexture : public FPCGDataTypeInfoSurface
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::VirtualTexture)
};

UCLASS()
class UPCGVirtualTextureData : public UPCGSurfaceData
{
	GENERATED_BODY()

public:
	void Initialize(const URuntimeVirtualTextureComponent* InVirtualTextureComponent);

	//~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoVirtualTexture)
	//~End UPCGData interface

	//~Begin UPCGSpatialData interface
	virtual FBox GetBounds() const override { return FBox(EForceInit::ForceInit); }
	virtual bool SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override { return false; }
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	//~Begin UPCGSpatialDataWithPointCache interface
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialDataWithPointCache interface

	TObjectPtr<const URuntimeVirtualTexture> GetRuntimeVirtualTexture() const { return RuntimeVirtualTexture; }

protected:
	UPROPERTY()
	TObjectPtr<URuntimeVirtualTexture> RuntimeVirtualTexture = nullptr;

	// @todo_pcg: Support SVT as well?
	//UPROPERTY()
	//TObjectPtr<class UVirtualTextureBuilder> StreamingVirtualTexture = nullptr;
};
