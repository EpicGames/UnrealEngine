// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSpatialData.h"

#include "PCGVolumeData.generated.h"

#define UE_API PCG_API

class AVolume;
struct FBodyInstance;

USTRUCT()
struct FPCGDataTypeInfoVolume : public FPCGDataTypeInfoConcrete
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Volume)
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGVolumeData : public UPCGSpatialDataWithPointCache
{
	GENERATED_BODY()

	friend class FPCGVolumeDataVisualization;

public:
	UE_API ~UPCGVolumeData();
	UE_API void Initialize(AVolume* InVolume);
	UE_API void Initialize(const FBox& InBounds);

	// ~Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPCGDataTypeInfoVolume)
	UE_API virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	// ~Begin UPGCSpatialData interface
	virtual int GetDimension() const override { return 3; }
	UE_API virtual FBox GetBounds() const override;
	UE_API virtual FBox GetStrictBounds() const override;
	UE_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	// TODO what should this do - closest point on volume?
	//virtual bool ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;
protected:
	UE_API virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	// ~Begin UPCGSpatialDataWithPointCache interface
	UE_API virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override;
	UE_API virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	// ~End UPCGSpatialDataWithPointCache interface

	TWeakObjectPtr<AVolume> GetVolumeActor() const { return Volume; }

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	FVector VoxelSize = FVector(100.0, 100.0, 100.0);

private:
	UE_API const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass) const;

protected:
	UE_API void CopyBaseVolumeData(UPCGVolumeData* NewVolumeData) const;
	UE_API void ReleaseInternalBodyInstance();
	UE_API void SetupVolumeBodyInstance();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SourceData)
	TWeakObjectPtr<AVolume> Volume = nullptr;

	UPROPERTY()
	FBox Bounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	FBox StrictBounds = FBox(EForceInit::ForceInit);

	// Internal body instance to perform queries faster, used in static cases only
	FBodyInstance* VolumeBodyInstance = nullptr;
};

#undef UE_API
