// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCollisionShape.h"
#include "PCGVolumeData.h"
#include "PCGSurfaceData.h"
#include "Helpers/PCGWorldQueryHelpers.h"

#include "CollisionQueryParams.h"
#include "Engine/EngineTypes.h"
#include "UObject/ObjectKey.h"

#include "PCGWorldData.generated.h"

class UPCGSpatialData;

class UWorld;
class UPCGMetadata;
class UPCGComponent;

UENUM()
enum class UE_DEPRECATED(5.6, "Not used anymore, replaced by EPCGWorldQueryFilter.") EPCGWorldQueryFilterByTag
{
	NoTagFilter,
	IncludeTagged,
	ExcludeTagged
};

UENUM(Blueprintable)
enum class EPCGWorldQueryFilter : uint8
{
	None    UMETA(Tooltip = "Filter disabled"),
	Include UMETA(Tooltip = "Includes the actor if no other filter explicitly filters it out (either by exclusion or by requiring an unmet criteria)."),
	Exclude UMETA(Tooltip = "Always exclude an actor if it matches this filter."),
	Require UMETA(Tooltip = "Requires the actor to match to this filter to be included."),

	// Deprecation values to support deserializing from EPCGWorldQueryFilterByTag, since it is deserialized by name. To be removed when EPCGWorldQueryFilterByTag is removed.
	NoTagFilter = None UMETA(Hidden),
	IncludeTagged = Include UMETA(Hidden),
	ExcludeTagged = Exclude UMETA(Hidden)
};

UENUM()
enum class EPCGWorldQuerySelectLandscapeHits : uint8
{
	Exclude = 0 UMETA(ToolTip="Excludes hits from the landscape."),
	Include UMETA(ToolTip="Will report hits on the landscape."),
	Require UMETA(ToolTip="Will return only hits on the landscape.")
};

namespace PCGWorldRayHitConstants
{
	UE_DEPRECATED(5.5, "Please use 'PCGWorldQueryConstants::PhysicalMaterialReferenceAttribute' instead.")
	const FName PhysicalMaterialReferenceAttribute = PCGWorldQueryConstants::PhysicalMaterialReferenceAttribute;
	
	const FName FilterActorPinLabel = TEXT("FilterActors");
}

USTRUCT(BlueprintType)
struct FPCGWorldCommonQueryParams
{
	GENERATED_BODY()

#if WITH_EDITOR
	void CommonPostLoad();
#endif

	/** If true, will ignore hits/overlaps on content created from PCG. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Filtering", meta = (PCG_Overridable))
	bool bIgnorePCGHits = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Filtering", meta = (PCG_Overridable))
	bool bIgnoreSelfHits = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Advanced", meta = (PCG_Overridable))
	TEnumAsByte<ECollisionChannel> CollisionChannel = ECC_WorldStatic;

	/** Queries against complex collision if enabled, performance warning */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Advanced", meta = (PCG_Overridable))
	bool bTraceComplex = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Filtering", meta = (PCG_Overridable))
	EPCGWorldQueryFilter ActorTagFilter = EPCGWorldQueryFilter::None;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Filtering", meta = (PCG_Overridable, EditCondition = "ActorTagFilter != EPCGWorldQueryFilter::None"))
	FString ActorTagsList;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Filtering", meta = (PCG_Overridable))
	EPCGWorldQueryFilter ActorClassFilter = EPCGWorldQueryFilter::None;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Filtering", meta = (PCG_Overridable, EditCondition = "ActorClassFilter != EPCGWorldQueryFilter::None"))
	TSubclassOf<AActor> ActorClass;

	/** Will add an input pin to pass a list of actor references for filtering if this value is not set to None. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Filtering", meta = (PCG_Overridable))
	EPCGWorldQueryFilter ActorFilterFromInput = EPCGWorldQueryFilter::None;

	/** Input source for the attribute to read from the Filter Actor pin. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Filtering", meta = (PCG_Overridable, EditCondition = "ActorFilterFromInput != EPCGWorldQueryFilter::None"))
	FPCGAttributePropertyInputSelector ActorFilterInputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Filtering", meta = (PCG_Overridable, PCG_OverrideAliases="bIgnoreLandscapeHits"))
	EPCGWorldQuerySelectLandscapeHits SelectLandscapeHits = EPCGWorldQuerySelectLandscapeHits::Include;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="IgnoreLandscapeHits has been deprecated in favor of SelectLandscapeHits"))
	bool bIgnoreLandscapeHits_DEPRECATED = false;
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	bool bGetReferenceToActorHit = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	bool bGetReferenceToPhysicalMaterial = false;

	// Not exposed, will be filled in when initializing this
	UPROPERTY()
	TSet<FName> ParsedActorTagsList;

	/** Utility function to add a filter pin with the right label and tooltip to the pin properties. */
	PCG_API void AddFilterPinIfNeeded(TArray<FPCGPinProperties>& PinProperties) const;

	/** Utility function to extract the Object Filters from the incoming data using the selector in this struct. Return true if the extraction succeeded. */
	PCG_API bool ExtractObjectFiltersIfNeeded(const UPCGData* InData, TArray<FSoftObjectPath>& OutArray, FPCGContext* InContext = nullptr) const;

	/** Utility function to extract the Object Filters (from loaded actors)  from the incoming data using the selector in this struct. Return true if the extraction succeeded. */
	PCG_API bool ExtractLoadedObjectFiltersIfNeeded(const UPCGData* InData, TSet<FObjectKey>& OutSet, FPCGContext* InContext = nullptr) const;

	UE_DEPRECATED(5.8, "Use ExtractObjectFiltersIfNeeded instead")
	PCG_API bool ExtractActorFiltersIfNeeded(const UPCGData* InData, TArray<TSoftObjectPtr<AActor>>& OutArray, FPCGContext* InContext = nullptr) const;

	UE_DEPRECATED(5.8, "Use ExtractLoadedObjectFiltersIfNeeded instead")
	PCG_API bool ExtractLoadedActorFiltersIfNeeded(const UPCGData* InData, TSet<TObjectKey<AActor>>& OutSet, FPCGContext* InContext = nullptr) const;

protected:
	/** Sets up the data we need to efficiently perform the queries */
	void Initialize();
};

USTRUCT(BlueprintType)
struct FPCGWorldRaycastQueryParams : public FPCGWorldCommonQueryParams
{
	GENERATED_BODY()

	void Initialize();
	void PostSerialize(const FArchive& Ar);

	/** Ignore rays that hit backfaces. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Filtering", meta = (PCG_Overridable))
	uint8 bIgnoreBackfaceHits : 1 = false;

	/** Create an attribute for whether the raycast resulted in a hit. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetImpact : 1 = false;

	/** Create an attribute for the impact location in world space. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetImpactPoint : 1 = false;

	/** Create an attribute for the impact normal. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetImpactNormal : 1 = false;

	/** Create an attribute for the reflection vector based on the ray incoming direction and the impact normal. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetReflection : 1 = false;

	/** Create an attribute for the distance between the ray origin and the impact point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetDistance : 1 = false;

	/** Create an attribute for the impact point in the hit object's local space. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetLocalImpactPoint : 1 = false;

	/** Create an attribute for the render material. Requires 'bTraceComplex' for use with Primitive Components. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetReferenceToRenderMaterial : 1 = false;

	/** Create an attribute for the static mesh. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetReferenceToStaticMesh : 1 = false;

	/** Create an attribute for index of the hit face. Note: Will only work in complex traces. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable, EditCondition = "bTraceComplex"))
	uint8 bGetFaceIndex : 1 = false;

	/** Create an attribute for UV Coordinates of the surface hit. Note: Will only work in complex traces and must have 'Project Settings->Physics->Support UV From Hit Results' set to true. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable, EditCondition = "bTraceComplex"))
	uint8 bGetUVCoords : 1 = false;

	/** Create an attribute for the index of the element hit. Unique to the hit primitive. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable))
	uint8 bGetElementIndex : 1 = false;

	/** Create an attribute for the index of the section hit. Currently only works for Static Meshes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable, EditCondition = "bTraceComplex"))
	uint8 bGetSectionIndex : 1 = false;

	/** Will apply landscape layers and their values at the impact point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable, EditCondition = "SelectLandscapeHits != EPCGWorldQuerySelectLandscapeHits::Exclude"))
	uint8 bApplyMetadataFromLandscape : 1 = false;

	/** Retrieve the material index explicitly. If false, the render material will be assumed from the primitive. Currently only works for Static Meshes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", DisplayName = "Set Render Material Index", meta = (PCG_Overridable, EditCondition = "bTraceComplex && bGetReferenceToRenderMaterial", EditConditionHides, DisplayAfter = "bGetReferenceToRenderMaterial"))
	uint8 bUseRenderMaterialIndex : 1 = false;

	/** Define the index of the render material to retrieve when a primitive is hit. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable, EditCondition = "bTraceComplex && bUseRenderMaterialIndex", EditConditionHides, DisplayAfter = "bUseRenderMaterialIndex"))
	int32 RenderMaterialIndex = 0;

	/** The index of the render material to query when a primitive is hit. Currently only works for Static Meshes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable, EditCondition = "bTraceComplex && !bUseRenderMaterialIndex", EditConditionHides, DisplayAfter = "bUseRenderMaterialIndex"))
	uint8 bGetRenderMaterialIndex : 1 = false;

	/** This UV Channel will be selected when retrieving UV Coordinates from a raycast query. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data|Attributes", meta = (PCG_Overridable, EditCondition = "bTraceComplex && bGetUVCoords", EditConditionHides, DisplayAfter = "bGetUVCoords"))
	int32 UVChannel = 0;

	// Helper function to ensure cohesive conversion of FPCGWorldRaycastQueryParams->FCollisionQueryParams.
	FCollisionQueryParams ToCollisionQuery() const
	{
		FCollisionQueryParams Params;
		Params.bTraceComplex = bTraceComplex;
		Params.bReturnPhysicalMaterial = bGetReferenceToPhysicalMaterial;
		// We need the face index to calculate UVs, Render Material, and Mesh information.
		Params.bReturnFaceIndex =
			bGetFaceIndex
			|| bGetUVCoords
			|| bGetSectionIndex
			|| bGetRenderMaterialIndex
			|| bGetReferenceToRenderMaterial;

		return Params;
	}
};

template<>
struct TStructOpsTypeTraits<FPCGWorldRaycastQueryParams> : public TStructOpsTypeTraitsBase2<FPCGWorldRaycastQueryParams>
{
	enum
	{
		WithPostSerialize = true,
	};
};

USTRUCT(BlueprintType)
struct FPCGWorldVolumetricQueryParams : public FPCGWorldCommonQueryParams
{
	GENERATED_BODY()

	void Initialize();
	void PostSerialize(const FArchive& Ar);

	/** Controls whether we are trying to find an overlap with physical objects (true) or to find empty spaces that do not contain anything (false) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable))
	bool bSearchForOverlap = true;
};

template<>
struct TStructOpsTypeTraits<FPCGWorldVolumetricQueryParams> : public TStructOpsTypeTraitsBase2<FPCGWorldVolumetricQueryParams>
{
	enum
	{
		WithPostSerialize = true,
	};
};

USTRUCT(BlueprintType)
struct FPCGWorldRayHitQueryParams : public FPCGWorldRaycastQueryParams
{
	GENERATED_BODY()

	void Initialize();
	void PostSerialize(const FArchive& Ar);

	/** Set ray parameters including origin, direction and length explicitly rather than deriving these from the generating actor bounds. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (DisplayName = "Set Ray Parameters", PCG_Overridable))
	bool bOverrideDefaultParams = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, EditCondition = "bOverrideDefaultParams", EditConditionHides))
	FVector RayOrigin = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, EditCondition = "bOverrideDefaultParams", EditConditionHides))
	FVector RayDirection = FVector(0.0, 0.0, -1.0);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, EditCondition = "bOverrideDefaultParams", EditConditionHides))
	double RayLength = 1.0e+5; // 100m

	// TODO: see in FCollisionQueryParams if there are some flags we want to expose
	// examples: bReturnFaceIndex, bReturnPhysicalMaterial, some ignore patterns
};

template<>
struct TStructOpsTypeTraits<FPCGWorldRayHitQueryParams> : public TStructOpsTypeTraitsBase2<FPCGWorldRayHitQueryParams>
{
	enum
	{
		WithPostSerialize = true,
	};
};

USTRUCT()
struct UE_DEPRECATED(5.8, "Use FPCGWorldQueryFilterCache instead") FPCGWorldQueryActorFilterCache
{
	GENERATED_BODY();

public:
	// No longer computed, use GetCachedFilteredObjects()
	PCG_API const TSet<TObjectKey<AActor>>& GetCachedFilterActors() const { return CachedFilterActors; }
	TArray<TSoftObjectPtr<AActor>>& GetFilterActorsMutable() { CachedFilterActorsDirty = true; return FilterActors_DEPRECATED; }
	const TArray<TSoftObjectPtr<AActor>>& GetFilterActorsConst() const { return FilterActors_DEPRECATED; }

private:
	UPROPERTY()
	TArray<TSoftObjectPtr<AActor>> FilterActors_DEPRECATED;

	// Transient data to cache filter actors
	mutable TSet<TObjectKey<AActor>> CachedFilterActors;
	mutable bool CachedFilterActorsDirty = false;

	friend class UPCGWorldVolumetricData;
	friend class UPCGWorldRayHitData;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template<>
struct TStructOpsTypeTraits<FPCGWorldQueryActorFilterCache> : public TStructOpsTypeTraitsBase2<FPCGWorldQueryActorFilterCache>
{
	enum
	{
		WithCopy = false,
	};
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Queries volume for presence of world collision or not. Can be used to voxelize environment. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGWorldVolumetricData : public UPCGVolumeData
{
	GENERATED_BODY()

public:
	PCG_API void Initialize(IPCGGraphExecutionSource* InOriginatingSource, const FBox& InBounds = FBox(EForceInit::ForceInit));

	PCG_API IPCGGraphExecutionSource* GetOriginatingSource() const;
	PCG_API void SetOriginatingSource(IPCGGraphExecutionSource* InExecutionSource);
	PCG_API UWorld* GetWorld() const;
	
	TArray<FSoftObjectPath>& GetMutableFilteredObjects() { CachedFilteredObjectsDirty = true; return FilteredObjects; }
	const TArray<FSoftObjectPath>& GetConstFilteredObjects() const { return FilteredObjects; }
	PCG_API const TSet<FObjectKey>& GetCachedFilteredObjects() const;

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSpatialData interface
	virtual bool IsBounded() const override { return !!Bounds.IsValid; }
	virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	// TODO not sure what this would mean. Without a direction, this means perhaps finding closest point on any collision surface? Should we implement this disabled?
	//virtual bool ProjectPoint(const FTransform& InTransform, const FBox& InBounds, const FPCGProjectionParams& InParams, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

public:
	//~Begin UPCGSpatialDataWithPointCache
	virtual bool SupportsBoundedPointData() const { return true; }
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override { return CreatePointData(Context, FBox(EForceInit::ForceInit)); }
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context, const FBox& InBounds) const override;
	virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialDataWithPointCache
private:
	const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (ShowOnlyInnerProperties))
	FPCGWorldVolumetricQueryParams QueryParams;
	
private:
	UPROPERTY()
	FSoftObjectPath OriginatingObject;

	UPROPERTY()
	TArray<FSoftObjectPath> FilteredObjects;

	// transient
	mutable bool CachedFilteredObjectsDirty = false;
	mutable PCG::FLock CachedFilteredObjectsLock;
	mutable TSet<FObjectKey> CachedFilteredObjects;

public:
	UE_DEPRECATED(5.8, "Use Initialize taking a IPCGGraphExecutionSource parameter instead")
	void Initialize(UWorld* InWorld, const FBox& InBounds = FBox(EForceInit::ForceInit)) {}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use GetWorld() instead")
	UPROPERTY(Transient)
	TWeakObjectPtr<UWorld> World = nullptr;

	UE_DEPRECATED(5.8, "Use OriginatingObject instead")
	UPROPERTY()
	TWeakObjectPtr<UPCGComponent> OriginatingComponent = nullptr;

	UE_DEPRECATED(5.8, "Use GetObjectFilter methods instead")
	UPROPERTY()
	FPCGWorldQueryActorFilterCache ActorFilter;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/** Executes collision queries against world collision. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGWorldRayHitData : public UPCGSurfaceData
{
	GENERATED_BODY()

public:
	PCG_API void Initialize(IPCGGraphExecutionSource* InExecutionSource, const FTransform& InTransform, const FBox& InBounds = FBox(EForceInit::ForceInit), const FBox& InLocalBounds = FBox(EForceInit::ForceInit));

	PCG_API IPCGGraphExecutionSource* GetOriginatingSource() const;
	PCG_API void SetOriginatingSource(IPCGGraphExecutionSource* InExecutionSource);
	PCG_API UWorld* GetWorld() const;
	
	TArray<FSoftObjectPath>& GetMutableFilteredObjects() { CachedFilteredObjectsDirty = true; return FilteredObjects; }
	const TArray<FSoftObjectPath>& GetConstFilteredObjects() const { return FilteredObjects; }
	PCG_API const TSet<FObjectKey>& GetCachedFilteredObjects() const;

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	// ~Begin UPCGData interface
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	//~Begin UPCGSpatialData interface
	virtual FBox GetBounds() const override { return Bounds; }
	virtual FBox GetStrictBounds() const override { return Bounds; }
	virtual bool IsBounded() const override { return !!Bounds.IsValid; }
	PCG_API virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	virtual bool HasNonTrivialTransform() const override { return true; }
	virtual FVector GetNormal() const override { return Transform.GetRotation().GetUpVector(); }
	PCG_API virtual void InitializeTargetMetadata(const FPCGInitializeFromDataParams& InParams, UPCGMetadata* MetadataToInitialize) const override;
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

	const UPCGBasePointData* CreateBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const;
public:
	// ~Begin UPCGSpatialDataWithPointCache interface
	virtual bool SupportsBoundedPointData() const { return true; }
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context) const override { return CreatePointData(Context, FBox(EForceInit::ForceInit)); }
	virtual const UPCGPointData* CreatePointData(FPCGContext* Context, const FBox& InBounds) const override;
	virtual const UPCGPointArrayData* CreatePointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	// ~End UPCGSpatialDataWithPointCache interface

	UPROPERTY()
	FBox Bounds = FBox(EForceInit::ForceInit);

	/** Parameters for either using a line trace or specifying a collision shape for a sweep. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (ShowOnlyInnerProperties))
	FPCGCollisionShape CollisionShape;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (ShowOnlyInnerProperties))
	FPCGWorldRayHitQueryParams QueryParams;
	
	/** Attributes related to landscape layers are added when we initialize the target metadata, so we need to cache them when this data gets initialized. */
	UPROPERTY()
	TSet<FName> CachedLandscapeLayerNames;

private:
	UPROPERTY()
	FSoftObjectPath OriginatingObject = nullptr;

	UPROPERTY()
	TArray<FSoftObjectPath> FilteredObjects;

	// transient
	mutable bool CachedFilteredObjectsDirty = false;
	mutable PCG::FLock CachedFilteredObjectsLock;
	mutable TSet<FObjectKey> CachedFilteredObjects;

public:
	UE_DEPRECATED(5.8, "Use Initialize taking a IPCGGraphExecutionSource parameter instead")
	void Initialize(UWorld* InWorld, const FTransform& InTransform, const FBox& InBounds = FBox(EForceInit::ForceInit), const FBox& InLocalBounds = FBox(EForceInit::ForceInit)) {}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use GetWorld() instead")
	UPROPERTY(Transient)
	TWeakObjectPtr<UWorld> World = nullptr;

	UE_DEPRECATED(5.8, "Use OriginatingObject instead")
	UPROPERTY()
	TWeakObjectPtr<UPCGComponent> OriginatingComponent = nullptr;

	UE_DEPRECATED(5.8, "Use ObjectFilter instead")
	UPROPERTY()
	FPCGWorldQueryActorFilterCache ActorFilter;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
