// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#define UE_API PCG_API

class AActor;
class UPCGBasePointData;
class UPCGMetadata;
class FPCGMetadataAttributeBase;
template<typename T> class FPCGMetadataAttribute;

/**
 * Context passed to FPCGActorMeshParsingContext::ParseActorComponents.
 * When parsing multiple actors into the same PointData, this struct should be shared
 * across all calls so that tag attribute creation is deduplicated.
 */
struct FPCGActorMeshParsingContext
{
	// Required output target
	UPCGBasePointData* PointData     = nullptr;
	UPCGMetadata*      PointMetadata = nullptr; // if null, derived from PointData->MutableMetadata()

	// Attribute pointers - null means that attribute will not be written
	FPCGMetadataAttribute<FSoftObjectPath>* MeshAttribute = nullptr;
	FPCGMetadataAttribute<FSoftObjectPath>* SkeletalMeshAttribute = nullptr;
	FPCGMetadataAttribute<FSoftObjectPath>* MaterialAttribute = nullptr;
	FPCGMetadataAttribute<int64>* ActorIndexAttribute = nullptr;
	FPCGMetadataAttribute<int64>* ParentIndexAttribute = nullptr;
	FPCGMetadataAttribute<FTransform>* RelativeTransformAttribute = nullptr;
	FPCGMetadataAttribute<int64>* HierarchyDepthAttribute = nullptr;
	FPCGMetadataAttribute<FSoftObjectPath>* ActorReferenceAttribute = nullptr;
	FPCGMetadataAttribute<FSoftObjectPath>* ComponentReferenceAttribute = nullptr;

	// Per-actor values - caller must set these before each call
	int64 ActorIndex = -1;
	int64 ParentIndex = 0;
	FTransform RelativeTransform = FTransform::Identity;
	int64 HierarchyDepth = 1; // only used if HierarchyDepthAttribute != nullptr
	const TArray<FName>* ActorTags = nullptr; // if null, Actor->Tags is used

	// Tag attribute tracking - shared across all calls for the same PointData
	TMap<FName, FPCGMetadataAttributeBase*>* TagToAttributeMap = nullptr;
	TSet<FName>* SanitizedAttributeNames = nullptr;

	// Processing flags
	bool bOnlyMeshComponents = false; // restricts to SMC/ISMC/SKMC/ISKMC only when true (LevelToAsset behavior)
	bool bIgnorePCGCreatedComponents = false;

	/**
	 * Creates metadata attributes on PointMetadata for any of the actor's tags that are not yet
	 * tracked in TagToAttributeMap, skipping any that would collide with pre-existing attribute pointers.
	 * No-op if TagToAttributeMap or SanitizedAttributeNames are null.
	 */
	UE_API void CreateTagAttributes(AActor* InActor);

	/**
	 * Parses the primitive components of InActor and writes points into PointData.
	 * For UStaticMeshComponent/UInstancedStaticMeshComponent/USkinnedMeshComponent/UInstancedSkinnedMeshComponent, creates one point per mesh instance.
	 * For other UPrimitiveComponent types (when bOnlyMeshComponents is false), creates a single
	 * point at the component transform using the component's local bounds.
	 * Returns the number of points added.
	 */
	UE_API int ParseActorComponents(AActor* InActor);
};

#undef UE_API
