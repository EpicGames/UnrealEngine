// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Data/Registry/PCGDataTypeCommon.h"

#include "PCGDataType.generated.h"

struct FPCGDataTypeBaseId;
struct FPCGDataTypeIdentifier;
class UPCGData;
class UPCGGraph;
class UPCGNode;
class UPCGPin;
class UPCGSettings;

/**
 * To declare a new type:
 * - Make a new UClass inheriting from UPCGData (or any of its children)
 * - Make a new UStruct inheriting from FPCGDataTypeInfo, and use the PCG_DECLARE_TYPE_INFO(DLL_EXPORT_NAME) in the header, replacing the DLL_EXPORT_NAME by your lib dll export (ie. PCG_API). Leave it empty if you don't want to export it.
 * - In the cpp, use the PCG_DEFINE_TYPE_INFO(PCGDataTypeInfoClass, PCGDataClass), replacing PCGDataClass by your new UClass and PCGDataTypeInfoClass by your new UStruct
 * - In the UClass, use the PCG_ASSIGN_TYPE_INFO(PCGDataTypeInfoClass) replacing PCGDataTypeInfoClass by your new UStruct
 * - Implement the virtual classes in the type info for any support for different colors/icons/conversions etc...
 *
 * If the new type doesn't have an associated PCGData class, use PCG_DEFINE_TYPE_INFO_WITHOUT_CLASS
 */
#define PCG_DECLARE_TYPE_INFO(DLL_EXPORT_NAME) \
	DLL_EXPORT_NAME static TSubclassOf<UPCGData> GetStaticAssociatedClass(); \
	DLL_EXPORT_NAME virtual TSubclassOf<UPCGData> GetAssociatedClass() const override; \
	static const FPCGDataTypeBaseId& AsId() { static FPCGDataTypeBaseId Id(StaticStruct()); return Id; }

#define PCG_DEFINE_TYPE_INFO(PCGDataTypeInfoClass, PCGDataClass) \
	TSubclassOf<UPCGData> PCGDataTypeInfoClass::GetStaticAssociatedClass() { return PCGDataClass::StaticClass(); } \
	TSubclassOf<UPCGData> PCGDataTypeInfoClass::GetAssociatedClass() const { return PCGDataClass::StaticClass(); };

#define PCG_DEFINE_TYPE_INFO_WITHOUT_CLASS(PCGDataTypeInfoClass) \
	TSubclassOf<UPCGData> PCGDataTypeInfoClass::GetStaticAssociatedClass() { return nullptr; } \
	TSubclassOf<UPCGData> PCGDataTypeInfoClass::GetAssociatedClass() const { return nullptr; };

#define PCG_ASSIGN_TYPE_INFO(PCGDataTypeInfoClass) \
	using TypeInfo = PCGDataTypeInfoClass; \
	virtual const FPCGDataTypeBaseId& GetDataTypeId() const override { return PCGDataTypeInfoClass::AsId(); }

// Convenient macro for default types in PCG (that are defined in EPCGDataType)
#define PCG_DECLARE_LEGACY_TYPE_INFO(LegacyDataType) \
	PCG_DECLARE_TYPE_INFO(PCG_API) \
	virtual EPCGDataType GetAssociatedLegacyType() const override{ return LegacyDataType; } \
	static EPCGDataType GetStaticAssociatedLegacyType() { return LegacyDataType; }

#define PCG_ASSIGN_DEFAULT_TYPE_INFO(PCGDataTypeInfoClass) \
	PCG_ASSIGN_TYPE_INFO(PCGDataTypeInfoClass); \
	PRAGMA_DISABLE_DEPRECATION_WARNINGS \
	virtual EPCGDataType GetDataType() const { return PCGDataTypeInfoClass::GetStaticAssociatedLegacyType(); } \
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
 * Store the information about a type.
 * Used by FPCGDataTypeBaseId as the UScriptStruct (the meta-struct) and will have a "default object" initialized in the registry for the virtual calls.
 * You can also add a "PCG_DataTypeDisplayName" meta tag to the UStruct to have a custom display name. Otherwise by default, it will keep what is at the right of `DataTypeInfo`.
 */
USTRUCT()
struct FPCGDataTypeInfo
{
	GENERATED_BODY()

	virtual ~FPCGDataTypeInfo() = default;

	static const FPCGDataTypeBaseId& AsId();

	PCG_API static TSubclassOf<UPCGData> GetStaticAssociatedClass();
	PCG_API virtual TSubclassOf<UPCGData> GetAssociatedClass() const;

	virtual EPCGDataType GetAssociatedLegacyType() const { return EPCGDataType::Any; }
	static EPCGDataType GetStaticAssociatedLegacyType() { return EPCGDataType::Any; }

	/**
	 * Can be overriden to have more restrictive compatibility based on the subtype. InType and OutType are expected to be already compatible without their subtype.
	 * InType is expected to be the type associated with this type info.
	 * Can also provide a custom compatibility message, instead of the generic "Not compatible"
	 */
	virtual EPCGDataTypeCompatibilityResult IsCompatibleForSubtype(const FPCGDataTypeIdentifier& InType, const FPCGDataTypeIdentifier& OutType, FText* OptionalOutCompatibilityMessage = nullptr) const { return EPCGDataTypeCompatibilityResult::Compatible; }

	/**
	 * Can be overriden by a type info to indicate if we can convert from InputType to ThisType.
	 * Since we have a hierarchy of type, ThisType is by construction a child of the current type info ; But it can be a child and not the same type.
	 * Can also return an optional simple conversion settings if the conversion is valid.
	 * Can also provide a custom message to provide the user more information.
	 */
	PCG_API virtual bool SupportsConversionFrom(const FPCGDataTypeIdentifier& InputType, const FPCGDataTypeIdentifier& ThisType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings = nullptr, FText* OptionalOutCompatibilityMessage = nullptr) const;
	
	/**
	 * Can be overriden by a type info to indicate if we can convert from ThisType to OutputType.
	 * Since we have a hierarchy of type, ThisType is by construction a child of the current type info ; But it can be a child and not the same type.
	 * Can also return an optional simple conversion settings if the conversion is valid.
	 * Can also provide a custom message to provide the user more information.
	 */
	PCG_API virtual bool SupportsConversionTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, TSubclassOf<UPCGSettings>* OptionalOutConversionSettings = nullptr, FText* OptionalOutCompatibilityMessage = nullptr) const;

	/** If SupportsConversionFrom returns true, can provide a function to add conversion nodes. Returns an optional array of added nodes. */
	PCG_API virtual TOptional<TArray<UPCGNode*>> AddConversionNodesFrom(const FPCGDataTypeIdentifier& InputType, const FPCGDataTypeIdentifier& ThisType, UPCGGraph* InGraph, UPCGPin* InUpstreamPin, UPCGPin* InDownstreamPin) const;
	
	/** If SupportsConversionTo returns true, can provide a function to add conversion nodes. Returns an optional array of added nodes. */
	PCG_API virtual TOptional<TArray<UPCGNode*>> AddConversionNodesTo(const FPCGDataTypeIdentifier& ThisType, const FPCGDataTypeIdentifier& OutputType, UPCGGraph* InGraph, UPCGPin* InUpstreamPin, UPCGPin* InDownstreamPin) const;

#if WITH_EDITOR
	/** Can provide a custom tooltip for subtype that will be shown when hovering a pin of this type. */
	PCG_API virtual TOptional<FText> GetSubtypeTooltip(const FPCGDataTypeIdentifier& ThisType) const;

	/** Can provide a custom extra tooltip that will be shown when hovering a pin of this type. */
	PCG_API virtual TOptional<FText> GetExtraTooltip(const FPCGDataTypeIdentifier& ThisType) const;

	// If the type should be hidden to the user.
	PCG_API virtual bool Hidden() const;
#endif // WITH_EDITOR
};

/**
 * Wrapper around an object ptr of a UScriptStruct, that must be a child of FPCGDataTypeInfo
 * Represent a single type, and is aggregated into a FPCGDataTypeIdentifier.
 * Have a mapping from and to the EPCGDataType, for all legacy data types.
 */
USTRUCT(BlueprintType)
struct FPCGDataTypeBaseId
{
	GENERATED_BODY()

	PCG_API FPCGDataTypeBaseId(const UScriptStruct* InStruct = nullptr);

	PCG_API static FPCGDataTypeBaseId MakeFromLegacyType(EPCGDataType LegacyType);
	PCG_API EPCGDataType AsLegacyType() const;

	template <typename T> requires std::is_base_of_v<FPCGDataTypeInfo, T>
	static FPCGDataTypeBaseId Construct()
	{
		return FPCGDataTypeBaseId(T::StaticStruct());
	}

	friend uint32 GetTypeHash(const FPCGDataTypeBaseId& This)
	{
		return GetTypeHash(This.Struct);
	}

	bool operator==(const FPCGDataTypeBaseId& Other) const
	{
		return Struct == Other.Struct;
	}

	PCG_API bool IsValid() const;
	PCG_API bool IsChildOf(const FPCGDataTypeBaseId& Other) const;

	template <typename T> requires std::is_base_of_v<FPCGDataTypeInfo, T>
	bool IsChildOf() const
	{
		return IsChildOf(FPCGDataTypeBaseId(T::StaticStruct()));
	}

	PCG_API FString ToString() const;

	const UScriptStruct* GetStruct() const { return Struct; }

private:
	UPROPERTY()
	TObjectPtr<const UScriptStruct> Struct = nullptr;
};

USTRUCT()
struct FPCGDataTypeInfoOther: public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_LEGACY_TYPE_INFO(EPCGDataType::Other);
};

inline const FPCGDataTypeBaseId& FPCGDataTypeInfo::AsId()
{
	static FPCGDataTypeBaseId Id(StaticStruct());
	return Id;
}