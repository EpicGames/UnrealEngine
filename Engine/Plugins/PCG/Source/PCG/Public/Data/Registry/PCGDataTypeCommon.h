// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

enum class EPCGDataTypeCompatibilityResult
{
	// Types are not compatible, they can't be connected together.
	NotCompatible = 0,

	// Types are fully compatible, they can be connected as-is.
	Compatible = 1,

	// Types are compatible, but requires a filter to be added for the connection.
	RequireFilter = 2,

	// Types are compatible, but require conversion nodes to be added for the connection.
	RequireConversion = 3,

	// Error if the identifier is not found
	UnknownType = 4,

	// For deprecation purposes, we do not want to break overridable pins that were mismatched in types, as it will
	// make the dynamic error go away, and "fail" silently.
	// This will allow to catch when types are compatible but not their subtype.
	TypeCompatibleSubtypeNotCompatible = 5,

	Count,
};

namespace PCGDataTypeCompatibilityResult
{
	inline bool IsValid(const EPCGDataTypeCompatibilityResult Result)
	{
		return Result != EPCGDataTypeCompatibilityResult::NotCompatible
		&& Result != EPCGDataTypeCompatibilityResult::UnknownType
		&& Result != EPCGDataTypeCompatibilityResult::TypeCompatibleSubtypeNotCompatible;
	}
}
