// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGDataType.h"
#include "Algo/AllOf.h"

#include "Containers/Array.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"

#include "PCGDataTypeIdentifier.generated.h"

// A concept to know what we can construct a FPCGDataTypeIdentifier from.
template <typename T>
concept CPCGDataTypeIdentifierConstructible =
	std::is_same_v<std::decay_t<T>, FPCGDataTypeIdentifier>
	|| std::is_same_v<std::decay_t<T>, FPCGDataTypeBaseId>
	|| std::is_convertible_v<std::decay_t<T>, TSubclassOf<UPCGData>>
	|| std::is_same_v<std::decay_t<T>, EPCGDataType>;

class FPCGDataTypeRegistry;
class UPCGData;

USTRUCT(BlueprintType)
struct FPCGDataTypeIdentifier
{
	friend FPCGDataTypeRegistry;
	
	GENERATED_BODY()

	// Default constructors
	FPCGDataTypeIdentifier() = default;
	FPCGDataTypeIdentifier(const FPCGDataTypeIdentifier&) = default;
	FPCGDataTypeIdentifier(FPCGDataTypeIdentifier&&) = default;
	FPCGDataTypeIdentifier& operator=(const FPCGDataTypeIdentifier&) = default;
	FPCGDataTypeIdentifier& operator=(FPCGDataTypeIdentifier&&) = default;

	/* Constructors from other types */

	// Implicit to facilitate deprecation
	PCG_API FPCGDataTypeIdentifier(EPCGDataType LegacyType);

	// Implicit since it can be used interchangeably
	PCG_API FPCGDataTypeIdentifier(const FPCGDataTypeBaseId& BaseId);

	PCG_API explicit FPCGDataTypeIdentifier(const TSubclassOf<UPCGData>& DataClass);

    /** Composition static constructors. */

	// Can compose from multiple FPCGDataTypeIdentifier, FPCGDataTypeBaseId, TSubclassOf<UPCGData> or EPCGDataType
	template <typename ...T> requires (CPCGDataTypeIdentifierConstructible<T> && ...)
	static FPCGDataTypeIdentifier Construct(T&& ...InIds)
	{
		FPCGDataTypeIdentifier Result{};

		([&InIds, &Result]()
		{
			if constexpr (std::is_same_v<std::decay_t<T>, FPCGDataTypeIdentifier>)
			{
				Result |= std::forward<T>(InIds);
			}
			else if constexpr (std::is_same_v<std::decay_t<T>, FPCGDataTypeBaseId>)
			{
				Result.ComposeFromBaseId(std::forward<T>(InIds));
			}
			else if constexpr (std::is_convertible_v<std::decay_t<T>, TSubclassOf<UPCGData>>)
			{
				Result.ComposeFromPCGDataSubclass(std::forward<T>(InIds));
			}
			else if constexpr(std::is_same_v<std::decay_t<T>, EPCGDataType>)
			{
				Result.ComposeFromLegacyType(std::forward<T>(InIds));
			}
			else
			{
				static_assert(!std::is_same_v<T, T>, "Incompatible argument");
			}
		}(), ...);

		return Result;
	}

    PCG_API static FPCGDataTypeIdentifier Construct(TConstArrayView<FPCGDataTypeIdentifier> InIds);
    PCG_API static FPCGDataTypeIdentifier Construct(TConstArrayView<TSubclassOf<UPCGData>> Classes);

	template <typename ...T> requires (std::is_base_of_v<UPCGData, T> && ...)
	static FPCGDataTypeIdentifier Construct()
	{
		return FPCGDataTypeIdentifier::Construct(T::StaticClass()...);
	}

	// ------- Begin deprecation with EPCGDataType --------
	// @todo_pcg: Implicit for now, should be explicit before next release with deprecation path
	UE_DEPRECATED(5.7, "FPCGDataTypeIdentifier is replacing EPCGDataType")
	PCG_API operator EPCGDataType() const;
	
	// @todo_pcg: Implicit for now, should be explicit before next release with deprecation path
	UE_DEPRECATED(5.7, "FPCGDataTypeIdentifier is replacing EPCGDataType")
	operator uint32() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return static_cast<uint32>(static_cast<EPCGDataType>(*this));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FPCGDataTypeIdentifier& operator=(EPCGDataType LegacyType)
	{
		*this = FPCGDataTypeIdentifier(LegacyType);
		return *this;
	}

	bool operator==(EPCGDataType OtherLegacyType) const
	{
		return IsSameType(FPCGDataTypeIdentifier(OtherLegacyType));
	}
	
	PCG_API bool operator!() const;
	
	PCG_API FPCGDataTypeIdentifier& operator|=(EPCGDataType OtherLegacyType);
	PCG_API FPCGDataTypeIdentifier operator|(EPCGDataType OtherLegacyType) const;
	PCG_API FPCGDataTypeIdentifier& operator&=(EPCGDataType OtherLegacyType);
	PCG_API FPCGDataTypeIdentifier operator&(EPCGDataType OtherLegacyType) const;
	
	PCG_API bool SupportsType(EPCGDataType OtherLegacyType) const;
	// ------- End deprecation with EPCGDataType --------

	// To mimic flags
	bool operator==(const FPCGDataTypeIdentifier& Other) const
	{
		return IsIdentical(Other);
	}

	PCG_API FPCGDataTypeIdentifier& operator|=(const FPCGDataTypeIdentifier& Other);
	PCG_API FPCGDataTypeIdentifier& operator|=(FPCGDataTypeIdentifier&& Other);
	PCG_API FPCGDataTypeIdentifier operator|(const FPCGDataTypeIdentifier& Other) const;
	PCG_API FPCGDataTypeIdentifier operator|(FPCGDataTypeIdentifier&& Other) const;

	PCG_API FPCGDataTypeIdentifier& operator&=(const FPCGDataTypeIdentifier& Other);
	PCG_API FPCGDataTypeIdentifier operator&(const FPCGDataTypeIdentifier& Other) const;

	// Faster to determine if there is intersection instead of !!(A & B)
	PCG_API bool Intersects(const FPCGDataTypeIdentifier& Other) const;

	// Return the composition with other types, with reduction. (ie. if Data B and C are child of Data A, Composition of B and C gives A)
	// CAREFUL: This can't be used before the Data Type Registry is done initializing (so can't be used before the Engine Post Init).
	PCG_API FPCGDataTypeIdentifier Compose(const FPCGDataTypeIdentifier& Other);
	PCG_API static FPCGDataTypeIdentifier Compose(TConstArrayView<FPCGDataTypeIdentifier> IDs);

	/** Can't be called when the identifier is a composition. Need to use GetIds in that case. */
	PCG_API FPCGDataTypeBaseId GetId() const;
	
	PCG_API TConstArrayView<FPCGDataTypeBaseId> GetIds() const;

	/** A type identifier can be a composition of multiple types. */
	bool IsComposition() const {return Ids.Num() > 1; }
	
	bool IsValid() const { return !Ids.IsEmpty() && Algo::AllOf(Ids, [](const FPCGDataTypeBaseId& Id) { return Id.IsValid(); }); }

	/** Return true if the IDs are the same, but not necessary the subtype. */
	PCG_API bool IsSameType(const FPCGDataTypeIdentifier& Other) const;

	/** Equivalent to IsSameType if it is not a composition. Otherwise, return true if the OtherId is in the Ids. */
	PCG_API bool SupportsType(const FPCGDataTypeIdentifier& Other) const;

	/** Return true if this identifier supports different/more types than Other. Equivalent of !!(*this & ~Other) in bit flags form. */
	PCG_API bool IsWider(const FPCGDataTypeIdentifier& Other) const;

	/** Return true if the IDs and subtypes are the same. */
	PCG_API bool IsIdentical(const FPCGDataTypeIdentifier& Other) const;

	/** A type is a child of another type if and only if their intersection gives self. */
	PCG_API bool IsChildOf(const FPCGDataTypeIdentifier& Other) const;

	/** To support previously saved types, that used EPCGDataType, we need to define this function to de-serialize the new class using the old. And add a trait (see below). */
	PCG_API bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	PCG_API friend uint32 GetTypeHash(const FPCGDataTypeIdentifier& Id);

	PCG_API FString ToString() const;
	PCG_API FText ToDisplayText() const;

#if WITH_EDITOR
	/**
	 * Return a subtype tooltip associated with this type.
	 * If it is a combination it will just return "ToDisplayText()".
	 * If it is not a combination, it will query the Data Registry for the underlying FPCGDataTypeInfo, if it has anything.
	 * If the FPCGDataTypeInfo doesn't provide a custom tooltip, it will just return "ToDisplayText()"
	 */
	PCG_API FText GetSubtypeTooltip() const;

	/**
	 * Return an extra tooltip associated with this type.
	 * If it is a combination it will return an empty text.
	 * If it is not a combination, it will query the Data Registry for the underlying FPCGDataTypeInfo, if it has anything.
	 * If the FPCGDataTypeInfo doesn't provide a custom tooltip, it will just return empty.
	 */
	PCG_API FText GetExtraTooltip() const;
#endif // WITH_EDITOR

private:
	PCG_API void ComposeFromLegacyType(EPCGDataType LegacyType);
	PCG_API void ComposeFromPCGDataSubclass(TSubclassOf<UPCGData> Subclass);
	PCG_API void ComposeFromBaseId(const FPCGDataTypeBaseId& BaseId);
	PCG_API void ComposeFromBaseId(FPCGDataTypeBaseId&& BaseId);
	
	bool PrepareForCompose(const FPCGDataTypeBaseId& BaseId);

private:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "PCG", meta = (AllowPrivateAccess))
	TArray<FPCGDataTypeBaseId> Ids;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "PCG")
	int32 CustomSubtype = -1;
};

template<>
struct TStructOpsTypeTraits<FPCGDataTypeIdentifier> : public TStructOpsTypeTraitsBase2<FPCGDataTypeIdentifier>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true
	};
};

/**
* Helper class to allow the BP to call the custom setters and getters on FPCGDataTypeIdentifier.
*/
UCLASS()
class UPCGDataTypeIdentifierHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "PCG|PCGDataTypeIdentifier")
	static FPCGDataTypeIdentifier GetIdentifierFromClass(TSubclassOf<UPCGData> Class) { return FPCGDataTypeIdentifier(Class); };

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGDataTypeIdentifier")
	static PCG_API FPCGDataTypeIdentifier GetIdentifierFromLegacyType(EPCGExclusiveDataType LegacyDataType);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGDataTypeIdentifier", meta = (ScriptMethod))
	static bool IsValid(UPARAM(Ref) const FPCGDataTypeIdentifier& This) { return This.IsValid(); }

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGDataTypeIdentifier", meta = (ScriptMethod))
	static bool IsComposition(UPARAM(Ref) const FPCGDataTypeIdentifier& This) { return This.IsComposition(); }

	/** Return true if the IDs are the same, but not necessary the subtype. */
	UFUNCTION(BlueprintCallable, Category = "PCG|PCGDataTypeIdentifier", meta = (ScriptMethod))
	static bool IsSameType(UPARAM(Ref) const FPCGDataTypeIdentifier& This, UPARAM(Ref) const FPCGDataTypeIdentifier& Other) { return This.IsSameType(Other); }

	/** Return true if the IDs and subtypes are the same. */
	UFUNCTION(BlueprintCallable, Category = "PCG|PCGDataTypeIdentifier", meta = (ScriptMethod))
	static bool IsIdentical(UPARAM(Ref) const FPCGDataTypeIdentifier& This, UPARAM(Ref) const FPCGDataTypeIdentifier& Other) { return This.IsIdentical(Other); }
};
