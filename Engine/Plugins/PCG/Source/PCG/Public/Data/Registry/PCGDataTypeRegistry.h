// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Data/Registry/PCGDataTypeCommon.h"
#include "Data/Registry/PCGDataTypeIdentifier.h"

#include "StructUtils/InstancedStruct.h"

struct FPCGPinProperties;
class FPCGDataTypeRegistry;
// Friend for tests
class FPCGDataTypeRegistryTestBase;
class FPCGHierarchyTest;
class FPCGModule;

#if WITH_EDITOR
// For custom icons
struct FSlateBrush;
#endif // WITH_EDITOR

namespace PCGDataTypeHierarchy
{
	struct FTree;
}

/**
 * Registry to hold PCG data types and the hierarchy.
 */
class FPCGDataTypeRegistry
{
public:
	// Moveable but not copyable
	FPCGDataTypeRegistry();
	FPCGDataTypeRegistry(const FPCGDataTypeRegistry&) = delete;
	FPCGDataTypeRegistry(FPCGDataTypeRegistry&&) = default;
	FPCGDataTypeRegistry& operator=(const FPCGDataTypeRegistry&) = delete;
	FPCGDataTypeRegistry& operator=(FPCGDataTypeRegistry&&) = default;
	
	~FPCGDataTypeRegistry();

	/** Is it possible to connect the InType into a OutType. Can also provide an optional FText for a custom message. */
	PCG_API EPCGDataTypeCompatibilityResult IsCompatible(const FPCGDataTypeIdentifier& InType, const FPCGDataTypeIdentifier& OutType, FText* OptionalOutCompatibilityMessage = nullptr) const;

	/** Return the type info, to allow to call virtual functions. */
	PCG_API const FPCGDataTypeInfo* GetTypeInfo(const FPCGDataTypeBaseId& ID) const;

	/** Returns the data type for the union of all the identifiers passed as input. It is the common ancestor of all of them. */
	PCG_API FPCGDataTypeIdentifier GetIdentifiersUnion(TConstArrayView<FPCGDataTypeIdentifier> Identifiers) const;

	/** Returns the data type for the composition of all the identifiers passed as input. It will be simplified if possible (like composition of spline and landscape spline gives a polyline). */
	PCG_API FPCGDataTypeIdentifier GetIdentifiersComposition(TConstArrayView<FPCGDataTypeIdentifier> Identifiers) const;

	struct FGetIdentifiersDifferenceParams
	{
		const FPCGDataTypeIdentifier* SourceIdentifier = nullptr;
		TConstArrayView<FPCGDataTypeIdentifier> DifferenceIdentifiers;

		enum EFilter
		{
			IncludeFilteredTypes,
			ExcludeFilteredTypes
		};

		EFilter Filter = ExcludeFilteredTypes;
		TConstArrayView<FPCGDataTypeBaseId> FilteredTypes;
	};

	/** Returns the data type for the difference between the Source and all the differences. Example: Difference(Spatial, Composite) gives Concrete. If there is no intersection between both, it returns SourceIdentifier. */
	PCG_API FPCGDataTypeIdentifier GetIdentifiersDifference(const FGetIdentifiersDifferenceParams& Params) const;

	enum class ESearchCommand
	{
		ExpandAndContinue,
		Continue,
		Stop
	};
	
	/** Do a depth search in the hierarchy tree. Functor should return ESearchCommand to continue/stop. ExpandAndContinue will continue searching deeper in a branch. */
	PCG_API void HierarchyDepthSearch(TFunctionRef<ESearchCommand(const FPCGDataTypeBaseId& Id, int32 Depth)> Callback) const;

#if WITH_EDITOR
	using FPCGPinColorQueryFunction = TFunction<FLinearColor(const FPCGDataTypeIdentifier& InId)>;
	/** Can register a custom pin color for a given Id. Can be a composition or a single type. Custom Types will be ignored. */
	PCG_API void RegisterPinColorFunction(const FPCGDataTypeIdentifier& InId, FPCGPinColorQueryFunction InFunction);
	PCG_API void UnregisterPinColorFunction(const FPCGDataTypeIdentifier& InId);

	PCG_API FLinearColor GetPinColor(const FPCGDataTypeIdentifier& InId) const;

	using FPCGPinIconsQueryFunction = TFunction<TTuple<const FSlateBrush*, const FSlateBrush*>(const FPCGDataTypeIdentifier& InId, const FPCGPinProperties& InProperties, bool bIsInput)>;
	/** Can register a custom pin icon for a given Id. Can be a composition or a single type. Custom Types will be ignored. Callback should return Slate Brush for connected and disconnected pins. */
	PCG_API void RegisterPinIconsFunction(const FPCGDataTypeIdentifier& InId, FPCGPinIconsQueryFunction InFunction);
	PCG_API void UnregisterPinIconsFunction(const FPCGDataTypeIdentifier& InId);
	
	PCG_API TTuple<const FSlateBrush*, const FSlateBrush*> GetPinIcons(const FPCGDataTypeIdentifier& InId, const FPCGPinProperties& InProperties, bool bIsInput) const;
#endif // WITH_EDITOR
	
private:
	friend FPCGDataTypeRegistryTestBase;
	friend FPCGHierarchyTest;
	friend FPCGModule;
	
	void RegisterKnownTypes();
	void RegisterEntry(const FPCGDataTypeBaseId& BaseID);
	
	bool IsRegistered(const FPCGDataTypeBaseId& BaseID) const { return Mapping.Contains(BaseID); }

	// To be called by the FPCGModule on PreExit as we have to make sure the structs FPCGDataTypeInfo are still alive.
	void Shutdown();
	
	TMap<FPCGDataTypeBaseId, TInstancedStruct<FPCGDataTypeInfo>> Mapping;
	TUniquePtr<PCGDataTypeHierarchy::FTree> HierarchyTree;

#if WITH_EDITOR
	TArray<TTuple<FPCGDataTypeIdentifier, FPCGPinColorQueryFunction>> CombinationPinColors;
	TArray<TTuple<FPCGDataTypeIdentifier, FPCGPinColorQueryFunction>> SingleTypePinColors;

	TArray<TTuple<FPCGDataTypeIdentifier, FPCGPinIconsQueryFunction>> CombinationPinIcons;
	TArray<TTuple<FPCGDataTypeIdentifier, FPCGPinIconsQueryFunction>> SingleTypePinIcons;
#endif // WITH_EDITOR
};

