// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"

class UPCGDataViewConverterBase;
class IPCGDataViewPropertySelector;
class UPCGData;
struct FPCGDataView;

class FPCGDataViewRegistry
{
public:
	FPCGDataViewRegistry();

	FPCGDataViewRegistry(FPCGDataViewRegistry&&);
	FPCGDataViewRegistry& operator=(FPCGDataViewRegistry&&);

	FPCGDataViewRegistry(const FPCGDataViewRegistry&) = delete;
	FPCGDataViewRegistry& operator=(const FPCGDataViewRegistry&) = delete;

	virtual ~FPCGDataViewRegistry();

	PCG_API void RegisterPropertySelector(const TSubclassOf<UPCGData> InDataClass, TUniquePtr<const IPCGDataViewPropertySelector> PropertySelector);
	PCG_API void UnregisterPropertySelector(const TSubclassOf<UPCGData> InDataClass);

	PCG_API void RegisterConverter(const TSubclassOf<UPCGData> InDataClass, TSubclassOf<UPCGDataViewConverterBase> InConverterClass);
	PCG_API void UnregisterConverter(const TSubclassOf<UPCGData> InDataClass, const TSubclassOf<UPCGDataViewConverterBase> InConverterClass);

	PCG_API const IPCGDataViewPropertySelector* GetPropertySelector(const FPCGDataView& InDataView) const;
	PCG_API const UPCGDataViewConverterBase* GetConverter(const FPCGDataView& InDataView, TSubclassOf<UPCGDataViewConverterBase> InConverterClass) const;

private:
	TMap<const TSubclassOf<UPCGData>, TUniquePtr<const IPCGDataViewPropertySelector>> PropertySelectorRegistry;

	using FConverterRegistryType = TMap<const TSubclassOf<UPCGData>, TArray<TSubclassOf<UPCGDataViewConverterBase>>>;
	FConverterRegistryType ConverterRegistry;
};
