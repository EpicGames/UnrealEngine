// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGData.h"

#define UE_API PCG_API

class AActor;
class UActorComponent;
struct FPCGContext;
struct FPCGComponentSelectorSettings;

/** Struct to hold selection criteria */
struct FPCGGetDataFunctionRegistryParams
{
	const IPCGGraphExecutionSource* Source = nullptr;
	const FPCGComponentSelectorSettings* ComponentSelector = nullptr;
	EPCGDataType DataTypeFilter = EPCGDataType::Any;
	bool bParseActor = true;
	bool bIgnorePCGGeneratedComponents = true;
	bool bAddActorTags = true;

	UE_DEPRECATED(5.8, "Use Source instead")
	const UPCGComponent* SourceComponent = nullptr;
};

struct FPCGGetDataFunctionRegistryOutput
{
	FPCGDataCollection Collection;
	bool bSanitizedTagAttributeNames = false;
};

/** Registry to hold actor & component to PCG data construction mapping */
struct FPCGGetDataFunctionRegistry
{
public:
	using FDataFromActorFunction = TFunction<bool(FPCGContext*, const FPCGGetDataFunctionRegistryParams&, AActor*, FPCGGetDataFunctionRegistryOutput&)>;
	using FDataFromComponentFunction = TFunction<bool(FPCGContext*, const FPCGGetDataFunctionRegistryParams&, UActorComponent*, FPCGGetDataFunctionRegistryOutput&)>;
	using FFunctionHandle = uint64;

	FPCGGetDataFunctionRegistry() = default;
	FPCGGetDataFunctionRegistry(const FPCGGetDataFunctionRegistry&) = delete;
	FPCGGetDataFunctionRegistry(FPCGGetDataFunctionRegistry&&) = default;
	FPCGGetDataFunctionRegistry& operator=(const FPCGGetDataFunctionRegistry&) = delete;
	FPCGGetDataFunctionRegistry& operator=(FPCGGetDataFunctionRegistry&&) = default;
	virtual ~FPCGGetDataFunctionRegistry() = default;

	/** Register an actor to PCG data function. */
	UE_API FFunctionHandle RegisterDataFromActorFunction(const FDataFromActorFunction& InFunction);

	/** Unregisters an actor to PCG data function. */
	UE_API void UnregisterDataFromActorFunction(FFunctionHandle InFunctionHandle);

	/** Appends actor data to the given collection and returns the number of data added. */
	UE_API int GetDataFromActor(FPCGContext* InContext, const FPCGGetDataFunctionRegistryParams& InParams, AActor* InActor, FPCGGetDataFunctionRegistryOutput& OutDataCollection) const;

	/** Register a component to PCG data function. */
	UE_API FFunctionHandle RegisterDataFromComponentFunction(const FDataFromComponentFunction& InFunction);

	/** Unregisters a component to PCG data function. */
	UE_API void UnregisterDataFromComponentFunction(FFunctionHandle InFunctionHandle);

	/** Append component data to the given collection, returns the number of data added. */
	UE_API int GetDataFromComponent(FPCGContext* InContext, const FPCGGetDataFunctionRegistryParams& InParams, UActorComponent* InComponent, FPCGGetDataFunctionRegistryOutput& OutDataCollection) const;

private:
	UE_API int DefaultDataFromActor(FPCGContext* InContext, const FPCGGetDataFunctionRegistryParams& InParams, AActor* InActor, FPCGGetDataFunctionRegistryOutput& OutDataCollection) const;
	UE_API int DefaultDataFromComponent(FPCGContext* InContext, const FPCGGetDataFunctionRegistryParams& InParams, UActorComponent* InComponent, FPCGGetDataFunctionRegistryOutput& OutDataCollection) const;

	FFunctionHandle NextFunctionHandle = 0;

	TArray<TPair<FDataFromActorFunction, FFunctionHandle>> ActorParsingFunctions;
	TArray<TPair<FDataFromComponentFunction, FFunctionHandle>> ComponentParsingFunctions;
};

#undef UE_API
