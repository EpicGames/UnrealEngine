// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/IPCGEditorProgressNotification.h"
#include "Editor/PCGSyntaxTokenizerParams.h"
#include "Utils/PCGNodeVisualLogs.h"

#include "Internationalization/Text.h"
#include "Modules/ModuleInterface.h"

class UPropertyBagHierarchyRoot;
class IPCGBaseSubsystem;
struct FPCGStackContext;
class APCGWorldActor;
class UPCGComponent;

enum class EPCGEditorPermissionMode : uint8
{
	All = 0,
	Limited,
	None
};

class IPCGEditorModule : public IModuleInterface
{
public:
	PCG_API static IPCGEditorModule* Get();

	virtual TWeakPtr<IPCGEditorProgressNotification> CreateProgressNotification(const FTextFormat& TextFormat, bool bCanCancel) = 0;
	virtual void ReleaseProgressNotification(TWeakPtr<IPCGEditorProgressNotification> InNotification) = 0;
	virtual void SetOutlinerUIRefreshDelay(float InDelay) = 0;
	
#if WITH_EDITOR
	virtual const FPCGNodeVisualLogs& GetNodeVisualLogs() const = 0;
	virtual FPCGNodeVisualLogs& GetNodeVisualLogsMutable() = 0;
	virtual bool CanSelectPartitionActors() const = 0;
	virtual void SetPermissionMode(EPCGEditorPermissionMode InPermissionMode) = 0;
	virtual EPCGEditorPermissionMode GetPermissionMode() const = 0;
	virtual void TransferPropertyBagMetadataIntoHierarchy(UPCGGraph* Graph) const = 0;
	virtual UObject* CreatePropertyBagHierarchyRootObject(UPCGGraph* Graph) const = 0;
	virtual void ConnectPropertyBagHierarchyRootObjectModifyDelegate(UPCGGraph* Graph) const = 0;
	virtual void GraphIsBeingDestroyed(UPCGGraph* Graph) const = 0;

	DECLARE_EVENT_OneParam(IPCGEditorModule, FPermissionModeChanged, EPCGEditorPermissionMode);
	virtual FPermissionModeChanged& OnPermissionModeChanged() = 0;

	static bool ShouldTreatViewportAsGenerationSource(const APCGWorldActor* InPCGWorldActor);
#endif

protected:
	friend class IPCGBaseSubsystem;
	friend class FPCGEditor;
	friend class FPCGGraphExecutor;
	friend class SPCGEditorGraphDebugObjectTree;
	friend class SPCGEditorGraphFind;
	friend class UPCGGraph;
	friend class UPCGSubsystem;
	friend class UPCGEngineSubsystem;

	virtual void OnScheduleGraph(const FPCGStackContext& StackContext) = 0;
	virtual void OnGraphPreSave(UPCGGraph* Graph, FObjectPreSaveContext ObjectSaveContext) = 0;

	/** Clear any data collected during execution, normally called prior to generating the component. */
	virtual void ClearExecutionMetadata(IPCGGraphExecutionSource* InSource) = 0;

	virtual void ClearExecutedStacks(const IPCGGraphExecutionSource* InRootSource) = 0;
	virtual void ClearExecutedStacks(const UPCGGraph* InContainingGraph) = 0;

	/** Get a list of stacks that were executed during the last execution. */
	virtual TArray<FPCGStackSharedPtr> GetExecutedStacksPtrs(const FPCGStack& BeginningWithStack) = 0;
	virtual TArray<FPCGStackSharedPtr> GetExecutedStacksPtrs(const IPCGGraphExecutionSource* InSource, const UPCGGraph* InSubgraph, bool bOnlyWithSubgraphAsCurrentFrame = true) = 0;

	virtual void NotifyGraphChanged(UPCGGraph* InGraph, EPCGChangeType ChangeType) = 0;

#if WITH_EDITOR
	virtual bool ShouldTreatViewportAsGenerationSourceInternal(const APCGWorldActor* InPCGWorldActor) const = 0;
#endif

	PCG_API static void SetEditorModule(IPCGEditorModule* InModule);
};