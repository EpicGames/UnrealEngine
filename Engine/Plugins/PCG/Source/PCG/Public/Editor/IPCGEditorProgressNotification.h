// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

class IPCGEditorProgressNotification : public TSharedFromThis<IPCGEditorProgressNotification>
{
public:
	virtual ~IPCGEditorProgressNotification() = default;
	
	virtual void Update(uint32 NumRemainingTasks) = 0;

	virtual uint32 GetNumTasks() const = 0;
	virtual uint32 GetNumTasksDone() const = 0;
	virtual const FTextFormat& GetTextFormat() const = 0;

	virtual void Cancel() = 0;
	virtual bool CanCancel() const = 0;

	DECLARE_EVENT(IPCGEditorNotification, FCancelTasksEvent);
	virtual FCancelTasksEvent& OnCancelTasks() = 0;
};
