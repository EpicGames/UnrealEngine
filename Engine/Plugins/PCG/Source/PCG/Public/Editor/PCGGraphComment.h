// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "EdGraphNode_Comment.h"
#endif // WITH_EDITOR

#include "PCGGraphComment.generated.h"


/**
 * Mimic UEdGraphNode_Comment class layout to store the information in the backend (PCG Graph).
 * Default values come from the default ctor of UEdGraphNode_Comment.
 */
USTRUCT(meta=(Hidden))
struct FPCGGraphCommentNodeData
{
	GENERATED_BODY();

#if WITH_EDITOR
	PCG_API void InitializeFromCommentNode(const UEdGraphNode_Comment& CommentNode);
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/** X position of node in the editor */
	UPROPERTY()
	int32 NodePosX = 0;

	/** Y position of node in the editor */
	UPROPERTY()
	int32 NodePosY = 0;

	/** Width of node in the editor; only used when the node can be resized */
	UPROPERTY()
	int32 NodeWidth = 400;

	/** Height of node in the editor; only used when the node can be resized */
	UPROPERTY()
	int32 NodeHeight = 100;

	UPROPERTY()
	FString NodeComment;
	
	/** Color to style comment with */
	UPROPERTY()
	FLinearColor CommentColor = FLinearColor::White;

	/** Size of the text in the comment box */
	UPROPERTY()
	int32 FontSize = 18;

	/** Whether to show a zoom-invariant comment bubble when zoomed out (making the comment readable at any distance). */
	UPROPERTY()
	uint32 bCommentBubbleVisible_InDetailsPanel:1 = 1;

	/** Whether to use Comment Color to color the background of the comment bubble shown when zoomed out. */
	UPROPERTY()
	uint32 bColorCommentBubble:1 = 0;

	/** Whether the comment should move any fully enclosed nodes around when it is moved.
	 * Underlying enum: ECommentBoxMode::Type
	 * Implementation node: This needs to be type erased for UHT since ECommentBoxMode::Type is not visible outside of editor. */
	UPROPERTY()
	uint8 MoveMode = 0;

	/** Details field if more info is needed to be communicated (will show up in tooltip) */
	UPROPERTY()
	FText NodeDetails;

	/** comment Depth */
	UPROPERTY()
	int32 CommentDepth = -1;

	/** Extra GUID to be unique. */
	UPROPERTY()
	FGuid GUID;

	/** Comment bubble pinned state */
	UPROPERTY()
	uint8 bCommentBubblePinned : 1 = 1;

	/** Comment bubble visibility */
	UPROPERTY()
	uint8 bCommentBubbleVisible : 1 = 1;
#endif // WITH_EDITORONLY_DATA
};