// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGNode.h"
#include "PCGSettings.h"

#include "PCGGraphCustomization.generated.h"

#define UE_API PCG_API

UENUM()
enum class EPCGGraphEditorFiltering
{
	Allow,
	Disallow
};

/** 
* Container struct for editor customization of a specific graph. 
* This will grow/evolve as we expose more options to adapt the PCG editor to specific use cases.
* This can be used to restrict choices and to make the usage more streamlined when it calls for it.
* It will be extended to support hierarchy of customization (for things that make sense).
*/
USTRUCT()
struct FPCGGraphEditorCustomization
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Filtering)
	bool bFilterNodesByCategory = false;

	UPROPERTY(EditAnywhere, Category = Filtering, meta = (EditCondition = "bFilterNodesByCategory"))
	EPCGGraphEditorFiltering NodeFilterType = EPCGGraphEditorFiltering::Allow;

	UPROPERTY(EditAnywhere, Category = Filtering, meta = (EditCondition = "bFilterNodesByCategory"))
	TSet<FString> FilteredCategories;

	UPROPERTY(EditAnywhere, Category = Filtering)
	bool bFilterSubgraphs = false;

	UPROPERTY(EditAnywhere, Category = Filtering, meta = (EditCondition = "bFilterSubgraphs"))
	EPCGGraphEditorFiltering SubgraphFilterType = EPCGGraphEditorFiltering::Allow;

	UPROPERTY(EditAnywhere, Category = Filtering, meta = (EditCondition = "bFilterSubgraphs"))
	TSet<TSoftObjectPtr<UPCGGraph>> FilteredSubgraphTypes;

	UPROPERTY(EditAnywhere, Category = Filtering)
	bool bFilterSettings = false;

	UPROPERTY(EditAnywhere, Category = Filtering, meta = (EditCondition = "bFilterSettings"))
	EPCGGraphEditorFiltering SettingsFilterType = EPCGGraphEditorFiltering::Allow;

	UPROPERTY(EditAnywhere, Category = Filtering, meta = (EditCondition = "bFilterSettings"))
	TArray<TSubclassOf<UPCGSettings>> FilteredSettingsTypes;

	UE_API bool Accepts(const FText& InCategory) const;
	UE_API bool FiltersSubgraphs() const;
	// Returns false if path is accepted by the customization
	UE_API bool FilterSubgraph(const FSoftObjectPath& InSubgraphPath) const;

	UE_API bool FiltersSettings() const;
	// Returns false if settings is accepted by the customization
	UE_API bool FilterSettings(const TSubclassOf<UPCGSettings>& InSettingsClass) const;

private:
	const FPCGGraphEditorCustomization* GetParent() const;
};

#undef UE_API
