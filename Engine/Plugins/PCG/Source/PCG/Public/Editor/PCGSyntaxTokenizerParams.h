// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"

/** Parameters to create a PCG syntax tokenizer. */
struct FPCGSyntaxTokenizerParams
{
	TArray<FString> AdditionalKeywords;
};
#endif
