// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Helpers/PCGAssetHelpers.h"

#include "PCGBlueprintBaseElement.generated.h"

class UWorld;
class UPCGBlueprintElement;
class UPCGBlueprintBaseElement;

#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPCGBlueprintElementChanged, UPCGBlueprintBaseElement*);

namespace PCGBlueprintHelper
{
	TSet<TObjectPtr<UObject>> GetDataDependencies(UPCGBlueprintBaseElement* InElement);
}
#endif // WITH_EDITOR

UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, hidecategories = (Object))
class UPCGBlueprintBaseElement : public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR
	friend class FPCGAssetHelpers;
#endif

public:
	// ~Begin UObject interface
	PCG_API virtual void PostLoad() override;
	PCG_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	PCG_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif
	// ~End UObject interface

	/**
	 * Main execution function that will contain the logic for this PCG Element. Use GetContextHandle to have access to the context.
	 * @param Input  - Input collection containing all the data passed as input to the node.
	 * @param Output - Data collection that will be passed as the output of the node, with pins matching the ones provided during the execution.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "PCG|Execution")
	PCG_API void Execute(const FPCGDataCollection& Input, FPCGDataCollection& Output);

	/** Override for the default node name */
	UFUNCTION(BlueprintNativeEvent, Category = "PCG|Node Customization")
	PCG_API FName NodeTitleOverride() const;

	/** Override for the default node color. */
	UFUNCTION(BlueprintNativeEvent, Category = "PCG|Node Customization")
	PCG_API FLinearColor NodeColorOverride() const;

	/** Override to change the node type. */
	UFUNCTION(BlueprintNativeEvent, Category = "PCG|Node Customization")
	PCG_API EPCGSettingsType NodeTypeOverride() const;

	/** If Dynamic Pins is enabled in the BP settings, override this function to provide the type for the given pin. You can use "GetTypeUnionOfIncidentEdges" from the settings to get the union of input types on a given pin. Use the bitwise OR to combine multiple types together.*/
	UFUNCTION(BlueprintNativeEvent, Category = "PCG|Node Customization")
	PCG_API int32 DynamicPinTypesOverride(const UPCGSettings* InSettings, const UPCGPin* InPin) const;

	/** Override for the IsCacheable node property when it depends on the settings in your node. If true, the node will be cached, if not it will always be executed. */
	UFUNCTION(BlueprintNativeEvent, Category = "PCG|Execution")
	PCG_API bool IsCacheableOverride() const;

	/** Apply the preconfigured settings specified in the class default. Used to create nodes that are configured with pre-defined settings. Use InPreconfigureInfo index to know which settings it is. */
	UFUNCTION(BlueprintImplementableEvent, Category = "PCG|Preconfigure Settings", meta = (ForceAsFunction))
	PCG_API void ApplyPreconfiguredSettings(UPARAM(ref) const FPCGPreConfiguredSettingsInfo& InPreconfigureInfo);

	// Returns the labels of custom input pins only
	UFUNCTION(BlueprintCallable, Category = "PCG|Input & Output")
	PCG_API TSet<FName> CustomInputLabels() const;

	// Returns the labels of custom output pins only
	UFUNCTION(BlueprintCallable, Category = "PCG|Input & Output")
	PCG_API TSet<FName> CustomOutputLabels() const;

	UFUNCTION(BlueprintCallable, Category = "PCG|Input & Output", meta = (HideSelfPin = "true"))
	PCG_API TArray<FPCGPinProperties> GetInputPins() const;

	UFUNCTION(BlueprintCallable, Category = "PCG|Input & Output", meta = (HideSelfPin = "true"))
	PCG_API TArray<FPCGPinProperties> GetOutputPins() const;

	/** Returns true if there is an input pin with the matching label. If found, will copy the pin properties in OutFoundPin */
	UFUNCTION(BlueprintCallable, Category = "PCG|Input & Output", meta = (HideSelfPin = "true"))
	PCG_API bool GetInputPinByLabel(FName InPinLabel, FPCGPinProperties& OutFoundPin) const;

	/** Returns true if there is an output pin with the matching label. If found, will copy the pin properties in OutFoundPin */
	UFUNCTION(BlueprintCallable, Category = "PCG|Input & Output", meta = (HideSelfPin = "true"))
	PCG_API bool GetOutputPinByLabel(FName InPinLabel, FPCGPinProperties& OutFoundPin) const;

	/** Gets the seed from the associated settings & source component */
	UFUNCTION(BlueprintCallable, Category = "PCG|Random")
	PCG_API int32 GetSeedWithContext(const FPCGBlueprintContextHandle& InContextHandle) const;

	/** Creates a random stream from the settings & source component */
	UFUNCTION(BlueprintCallable, Category = "PCG|Random")
	PCG_API FRandomStream GetRandomStreamWithContext(const FPCGBlueprintContextHandle& InContextHandle) const;

	/** Called after object creation to setup the object callbacks */
	PCG_API void Initialize();

	UFUNCTION(BlueprintCallable, Category = "PCG|Advanced", meta = (HideSelfPin = "true"))
	PCG_API FPCGBlueprintContextHandle GetContextHandle() const;

	/** Called after the element duplication during execution to be able to get the context easily - internal call only */
	PCG_API void SetCurrentContext(FPCGContext* InCurrentContext);

	/** Tries to resolve current Context from thread local BP stack */
	static PCG_API FPCGContext* ResolveContext();

#if WITH_EDITOR
	// ~Begin UObject interface
	PCG_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// ~End UObject interface

	/** Used for filtering */
	static PCG_API FString GetParentClassName();
#endif

	/** Needed to be able to call certain blueprint functions */
	PCG_API virtual UWorld* GetWorld() const override;

#if !WITH_EDITOR
	void SetInstanceWorld(UWorld* World) { InstanceWorld = World; }
#endif

#if WITH_EDITOR
	FOnPCGBlueprintElementChanged OnBlueprintElementChangedDelegate;
#endif

	/** Controls whether results can be cached so we can bypass execution if the inputs & settings are the same in a subsequent execution.
	* If you have implemented the IsCacheableOverride function, then this value is ignored.
	* Note that if your node relies on data that is not directly tracked by PCG or creates any kind of artifact (adds components, creates actors, etc.) then it should not be cacheable.
	*/
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, AdvancedDisplay, Category = Settings)
	bool bIsCacheable = false;

	/** In cases where your node is non-cacheable but is likely to yield the same results on subsequent executions, this controls whether we will do a deep & computationally intensive CRC computation (true),
	* which will allow cache usage in downstream nodes in your graph, or, by default (false), a shallow but quick crc computation which will not be cache-friendly. */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, AdvancedDisplay, Category = Settings)
	bool bComputeFullDataCrc = false;

	/** Controls whether this node execution can be run from a non-game thread. This is not related to the Loop functions provided/implemented in this class, which should always run on any thread. */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = Settings)
	bool bRequiresGameThread = true;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = "Settings|Input & Output")
	TArray<FPCGPinProperties> CustomInputPins;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = "Settings|Input & Output")
	TArray<FPCGPinProperties> CustomOutputPins;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = "Settings|Input & Output")
	bool bHasDefaultInPin = true;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = "Settings|Input & Output")
	bool bHasDefaultOutPin = true;

	/** If enabled, by default, the Out pin type will have the union of In pin types. Default only works if the pins are In and Out. For custom behavior, implement DynamicPinTypesOverride. */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = "Settings|Input & Output")
	bool bHasDynamicPins = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = AssetInfo, AssetRegistrySearchable)
	bool bExposeToLibrary = false;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = "AssetInfo|Preconfigured Settings", AssetRegistrySearchable)
	bool bEnablePreconfiguredSettings = false;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = "AssetInfo|Preconfigured Settings", AssetRegistrySearchable, meta = (EditCondition = bEnablePreconfiguredSettings, EditConditionHides))
	bool bOnlyExposePreconfiguredSettings = false;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = "AssetInfo|Preconfigured Settings", meta = (EditCondition = bEnablePreconfiguredSettings, EditConditionHides))
	TArray<FPCGPreConfiguredSettingsInfo> PreconfiguredInfo;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = AssetInfo, AssetRegistrySearchable)
	FText Category;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = AssetInfo, AssetRegistrySearchable)
	FText Description;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, AdvancedDisplay, Category = "Settings")
	int32 DependencyParsingDepth = 1;

private:
	/** Cached pin data to get type information without loading. */
	UPROPERTY(AssetRegistrySearchable)
	FPCGAssetCachedPins CachedPins;

	/** Builds and returns list of pins as visible for graph creation purposes. */
	FPCGAssetCachedPins BuildCachedPins() const;
#endif

protected:
	void ThrowRequiresGameThreadException(FPCGContext* InContext, const FString& InFunctionName) const;

#if WITH_EDITOR
	PCG_API void OnDependencyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);
	PCG_API void UpdateDependencies();
	TSet<TWeakObjectPtr<UObject>> DataDependencies;
#endif

#if !WITH_EDITORONLY_DATA
	UWorld* InstanceWorld = nullptr;
#endif

	// Since we duplicate the blueprint elements prior to execution, they will be unique
	// and have a 1:1 match with their context, which allows us to store it here
	FPCGContext* CurrentContext = nullptr;
};


