// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGBlueprintBaseElement.h"
#include "PCGContext.h"
#include "PCGSettings.h"
#include "Delegates/Delegate.h"

#include "PCGBlueprintDeprecatedElement.generated.h"

class UPCGBlueprintElement;
class UPCGMetadata;
class UPCGPointData;
class UPCGSpatialData;
struct FPCGPoint;

class UWorld;

#if WITH_EDITOR

// Deprecated
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPCGBlueprintChanged, UPCGBlueprintElement*);

#endif // WITH_EDITOR

/**
 * Class is no longer supported and doesn't support PCGBasePointData. 
 * Please sub-class UPCGBlueprintBaseElement instead or use one of its existing child classes (ex: PCGBlueprintPointProcessorElement, PCGBlueprintSimplePointProcessorElement).
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, hidecategories = (Object), meta = (DisplayName = "Deprecated Blueprint Element"))
class UPCGBlueprintElement : public UPCGBlueprintBaseElement
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	UE_DEPRECATED(5.7, "Use OnBlueprintElementChangedDelegate instead")
	FOnPCGBlueprintChanged OnBlueprintChangedDelegate;
#endif

	/**
	 * Main execution function that will contain the logic for this PCG Element, with the context as parameter.
	 * @param InContext - Context of the execution
	 * @param Input     - Input collection containing all the data passed as input to the node.
	 * @param Output    - Data collection that will be passed as the output of the node, with pins matching the ones provided during the execution.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "PCG|Execution")
	PCG_API void ExecuteWithContext(UPARAM(ref)FPCGContext& InContext, const FPCGDataCollection& Input, FPCGDataCollection& Output);

	/**
	 * Multi-threaded loop that will iterate on all points in InData. All points will be added in the same order than in input. Will be called by Point Loop function.
	 * @param InContext   - Context of the execution
	 * @param InData      - Input point data. Constant, must not be modified.
	 * @param InPoint     - Point for the current iteration. Constant, must not be modified.
	 * @param OutPoint    - Point that will be added to the output data. Can be modified.
	 * @param OutMetadata - Output metadata to write attribute to. Can be modified.
	 * @param Iteration   - Index of the current point. Must only be used to access input data, as this call is multi-threaded. It is not safe to access output data.
	 * @returns True if the point should be kept, False if not.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "PCG|Flow Control")
	PCG_API bool PointLoopBody(const FPCGContext& InContext, const UPCGPointData* InData, const FPCGPoint& InPoint, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata, int64 Iteration) const;

	/**
	 * Multi-threaded loop that will be called on all points in InData. Can return a variable number of output points.
	 * All points will be added in the same order than in input. Will be called by Variable Loop function.
	 * @param InContext   - Context of the execution
	 * @param InData      - Input point data. Constant, must not be modified.
	 * @param InPoint     - Point for the current iteration. Constant, must not be modified.
	 * @param OutMetadata - Output metadata to write attribute to. Can be modified.
	 * @param Iteration   - Index of the current point. Must only be used to access input data, as this call is multi-threaded. It is not safe to access output data.
	 * @returns Array of new points that will be added to the output point data.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "PCG|Flow Control")
	PCG_API TArray<FPCGPoint> VariableLoopBody(const FPCGContext& InContext, const UPCGPointData* InData, const FPCGPoint& InPoint, UPCGMetadata* OutMetadata, int64 Iteration) const;

	/**
	 * Multi-threaded loop that will iterate on all nested loop pairs (e.g. (o, i) for all o in Outer, i in Inner).
	 * All points will be added in the same order than in input (e.g: (0,0), (0,1), (0,2), ...). Will be called by Nested Loop function.
	 * @param InContext      - Context of the execution
	 * @param InOuterData    - Outer point data. Constant, must not be modified.
	 * @param InInnerData    - Inner point data. Constant, must not be modified.
	 * @param InOuterPoint   - Outer Point for the current iteration. Constant, must not be modified.
	 * @param InInnerPoint   - Inner Point for the current iteration. Constant, must not be modified.
	 * @param OutPoint       - Point that will be added to the output data. Can be modified.
	 * @param OutMetadata    - Output metadata to write attribute to. Can be modified.
	 * @param OuterIteration - Index of the current point in outer data. Must only be used to access input data, as this call is multi-threaded. It is not safe to access output data.
	 * @param InnerIteration - Index of the current point in inner data. Must only be used to access input data, as this call is multi-threaded. It is not safe to access output data.
	 * @returns True if the point should be kept, False if not.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "PCG|Flow Control")
	PCG_API bool NestedLoopBody(const FPCGContext& InContext, const UPCGPointData* InOuterData, const UPCGPointData* InInnerData, const FPCGPoint& InOuterPoint, const FPCGPoint& InInnerPoint, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata, int64 OuterIteration, int64 InnerIteration) const;

	/**
	 * Multi-threaded loop that will be called N number of times (defined by Iteration Loop parameter NumIterations).
	 * All points will be added in order (iteration 0 will be before iteration 1 in the final array).
	 * @param InContext   - Context of the execution
	 * @param Iteration   - Index of the current iteration. Must only be used to access input data, as this call is multi-threaded. It is not safe to access output data.
	 * @param InA         - Optional input spatial data, can be null. Constant, must not be modified.
	 * @param InB         - Optional input spatial data, can be null. Constant, must not be modified.
	 * @param OutPoint    - Point that will be added to the output data. Can be modified.
	 * @param OutMetadata - Output metadata to write attribute to. Can be modified.
	 * @returns True if the point should be kept, False if not.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "PCG|Flow Control")
	PCG_API bool IterationLoopBody(const FPCGContext& InContext, int64 Iteration, const UPCGSpatialData* InA, const UPCGSpatialData* InB, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;

	/** Calls the PointLoopBody function on all points */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "PCG|Flow Control", meta = (HideSelfPin = "true"))
	PCG_API void PointLoop(UPARAM(ref) FPCGContext& InContext, const UPCGPointData* InData, UPCGPointData*& OutData, UPCGPointData* OptionalOutData = nullptr, bool bGameThread = false) const;

	/** Calls the VariableLoopBody function on all points, each call can return a variable number of points */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "PCG|Flow Control", meta = (HideSelfPin = "true"))
	PCG_API void VariableLoop(UPARAM(ref) FPCGContext& InContext, const UPCGPointData* InData, UPCGPointData*& OutData, UPCGPointData* OptionalOutData = nullptr, bool bGameThread = false) const;

	/** Calls the NestedLoopBody function on all nested loop pairs (e.g. (o, i) for all o in Outer, i in Inner) */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "PCG|FLow Control", meta = (HideSelfPin = "true"))
	PCG_API void NestedLoop(UPARAM(ref) FPCGContext& InContext, const UPCGPointData* InOuterData, const UPCGPointData* InInnerData, UPCGPointData*& OutData, UPCGPointData* OptionalOutData = nullptr, bool bGameThread = false) const;

	/** Calls the IterationLoopBody a fixed number of times, optional parameters are used to potentially initialized the Out Data, but otherwise are used to remove the need to have variables */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "PCG|Flow Control", meta = (HideSelfPin = "true"))
	PCG_API void IterationLoop(UPARAM(ref) FPCGContext& InContext, int64 NumIterations, UPCGPointData*& OutData, const UPCGSpatialData* OptionalA = nullptr, const UPCGSpatialData* OptionalB = nullptr, UPCGPointData* OptionalOutData = nullptr, bool bGameThread = false) const;

	/** Creates a random stream from the settings & source component */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "PCG|Random")
	PCG_API FRandomStream GetRandomStream(UPARAM(ref) FPCGContext& InContext) const;

	/** Gets the seed from the associated settings & source component */
	UFUNCTION(BlueprintCallable, Category = "PCG|Random")
	PCG_API int GetSeed(UPARAM(ref) FPCGContext& InContext) const;

	/** Retrieves the execution context - note that this will not be valid outside of the Execute functions */
	UFUNCTION(BlueprintCallable, Category = "PCG|Advanced", meta = (HideSelfPin = "true"))
	PCG_API FPCGContext& GetContext() const;
};
