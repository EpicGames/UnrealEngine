// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGBlueprintBaseElement.h"

#include "PCGBlueprintPointProcessorElement.generated.h"

class UPCGBasePointData;
struct FPCGPointInputRange;
struct FPCGPointOutputRange;

/**
 * Point processor element that supports the following two types of loop:
 * - PointLoop: requires PointLoopBody to be overridden with optional override of InitalizePointLoop.
 * - IterationLoop: requires IterationLoopBody to be overridden with optional override of InitializeIterationLoop.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, hidecategories = (Object))
class UPCGBlueprintPointProcessorElement : public UPCGBlueprintBaseElement
{
	GENERATED_BODY()

public:
	/**
	 * Calls the PointLoopInitialize once and PointLoopBody a fixed number of times.
	 * @param InputPointData	Input point data. Constant, must not be modified.
	 * @param OutputPointData	PointLoop output point data. Can be modified.
	 * @param bGameThread		If true, loop will run single-thread on the game thread.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "PCG|Flow Control", meta = (HideSelfPin = "true"))
	PCG_API void PointLoop(const UPCGBasePointData* InputPointData, UPCGBasePointData*& OutputPointData, bool bGameThread = false);

	/**
	 * Initialization of the PointLoop output point data. This is called once before the PointLoopBody calls.
	 * Native default implementation inherits spatial data & metadata from input.
	 * 
	 * @param InputPointData	Input point data. Constant, must not be modified.
	 * @param OutputPointData	PointLoop output point data. Can be modified.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "PCG|Flow Control", meta = (ForceAsFunction))
	void PointLoopInitialize(const UPCGBasePointData* InputPointData, UPCGBasePointData* OutputPointData);

	/**
	 * Multi-threaded range based loop that will be called in chunks a number of times.
	 *
	 * @param InputRange		Input range data created from InputPointData.
	 * @param OutputRange		PointLoop output range data used to write to the output point data.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "PCG|Flow Control")
	int32 PointLoopBody(const FPCGPointInputRange& InputRange, UPARAM(Ref) FPCGPointOutputRange& OutputRange);

	/** 
	 * Calls the IterationLoopInitialize once and IterationLoopBody a fixed number of times. 
	 * @param NumIteration		Number of iterations.
	 * @param InputPointDataA	Optional input point data, can be null. Constant, must not be modified.
	 * @param InputPointDataB	Optional input point data, can be null. Constant, must not be modified.
	 * @param OutputPointData	IterationLoop output point data. Can be modified.
	 * @param bGameThread		If true, loop will run single-thread on the game thread.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "PCG|Flow Control", meta = (HideSelfPin = "true"))
	PCG_API void IterationLoop(int32 NumIterations, const UPCGBasePointData* InputPointDataA, const UPCGBasePointData* InputPointDataB, UPCGBasePointData*& OutputPointData, bool bGameThread = false);

	/** 
	 * Initialization of the IterationLoop output point data. This is called once before the IterationLoopBody calls. 
	 * Native default implementation does not inherit spatial data and inherits metadata from its first non-null input if any.
	 * 
	 * @param NumIteration		Number of iterations.
	 * @param InputPointDataA	Optional input point data, can be null. Constant, must not be modified.
	 * @param InputPointDataB	Optional input point data, can be null. Constant, must not be modified.
	 * @param OutputPointData	IterationLoop output point data to initialize. Can be modified.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "PCG|Flow Control", meta = (ForceAsFunction))
	void IterationLoopInitialize(int32 NumIterations, const UPCGBasePointData* InputPointDataA, const UPCGBasePointData* InputPointDataB, UPCGBasePointData* OutputPointData);

	/** 
	 * Multi-threaded range based loop that will be called in chunks a number of times.
	 * Use the OutputRange.RangeSize to get the number of iterations to process in the IterationLoopBody [IterationIndex, IterationIndex + OutputRange.RangeSize [
	 * 
	 * @param IterationIndex	Current IterationLoopBody index.
	 * @param InputRangeA		Optional input range data if IterationLoop was called with a valid InputPointDataA.
	 * @param InputRangeB		Optional input range data if IterationLoop was called with a valid InputPointDataB.
	 * @param OutputRange		IterationLoop output range data used to write to the output point data.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "PCG|Flow Control")
	int32 IterationLoopBody(int32 IterationIndex, const FPCGPointInputRange& InputRangeA, const FPCGPointInputRange& InputRangeB, UPARAM(Ref) FPCGPointOutputRange& OutputRange);
};
