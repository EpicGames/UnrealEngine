// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGBlueprintBaseElement.h"
#include "Data/PCGBasePointData.h"

#include "PCGBlueprintPointProcessorSimpleElement.generated.h"

/**
 * Simple point processor element that supports the following two overrides:
 * - RangePointLoopBody is the more efficient but more complex loop operating on a point range per call.
 * - PointLoopBody is the less efficient but simplest loop operating on a single point per call.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, hidecategories = (Object))
class UPCGBlueprintPointProcessorSimpleElement : public UPCGBlueprintBaseElement
{
	GENERATED_BODY()

public:
	UPCGBlueprintPointProcessorSimpleElement();

	/** Execute implementation will call the proper loop body based on which is implemented */
	virtual void Execute_Implementation(const FPCGDataCollection & Input, FPCGDataCollection & Output) override;

	/** Allows initialization of the output point data before the loop body calls. Called once per pin input - output pair. */
	UFUNCTION(BlueprintImplementableEvent, Category = "PCG|Flow Control", meta = (ForceAsFunction))
	void Initialize(const UPCGBasePointData* InputPointData, UPCGBasePointData* OutputPointData);

	/**
	 * Multi-threaded loop that will iterate on all points in InPointData. All points will be added in the same order than in input.
	 * @param InPointData  - Input point data. Constant, must not be modified.
	 * @param InPoint      - Point for the current iteration. Constant, must not be modified.
	 * @param OutPointData - Output point data. Can be modified.
	 * @param OutPoint     - Point that will be added to the output data. Can be modified.
	 * @param PointIndex   - Index of the current point. Must only be used to access input data, as this call is multi-threaded. It is not safe to access output data.
	 * @returns True if the point should be kept, False if not.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "PCG|Flow Control")
	bool PointLoopBody(const UPCGBasePointData* InPointData, const FPCGPoint& InPoint, UPCGBasePointData* OutPointData, FPCGPoint& OutPoint, int64 PointIndex);

	/**
	 * Multi-threaded loop that will iterate on all points. All points will be added in the same order than in input.
	 * Each call will work on a subset of the data (ranges).
	 * @param InputRange  - Input Range. Constant, must not be modified. Can be used to get property value ranges on the Input point data.
	 * @param OutputRange - Output Range. Can be modified. Can be used to set property value ranges on the Output point data.
	 * @returns Number of points that were written.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "PCG|Flow Control")
	int32 RangePointLoopBody(const FPCGPointInputRange& InputRange, UPARAM(Ref) FPCGPointOutputRange& OutputRange);
	
	/** If true, output will inherit metadata from inputs */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = Settings)
	bool bInheritMetadata = true;

	/** If true, output will inherit spatial data from inputs */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = Settings)
	bool bInheritSpatialData = true;

	/** Specify which properties to allocate on the outputs */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = Settings, meta = (Bitmask, BitmaskEnum = "/Script/PCG.EPCGPointNativeProperties"))
	int32 PropertiesToAllocate = 0;
};