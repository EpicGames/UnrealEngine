// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGControlFlow.h"

#include "PCGBooleanSelect.generated.h"

/**
 * Selects data from either input pin, based on a boolean condition.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural), meta=(Keywords = "if bool select"))
class UPCGBooleanSelectSettings : public UPCGControlFlowSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Select")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::ControlFlow; }
#endif

	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* Pin) const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings, meta=(PCG_Overridable))
	bool bUseInputB = false;
};

class FPCGBooleanSelectElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
