// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGControlFlow.h"

#include "PCGBranch.generated.h"

namespace PCGBranchConstants
{
	extern PCG_API const FName OutputLabelA;
	extern PCG_API const FName OutputLabelB;
}

/**
 * Routes input data to one of two outputs, based on a boolean condition.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural), meta=(Keywords = "if bool switch"))
class UPCGBranchSettings : public UPCGControlFlowSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Branch")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::ControlFlow; }
#endif
	virtual bool HasDynamicPins() const override { return true; }
	virtual bool OutputPinsCanBeDeactivated() const override { return true; }
	virtual bool IsPinStaticallyActive(const FName& PinLabel) const override;
	
	virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings, meta=(PCG_Overridable))
	bool bOutputToB = false;
};

class FPCGBranchElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
};
