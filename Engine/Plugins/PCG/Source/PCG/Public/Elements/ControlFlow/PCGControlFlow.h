// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGControlFlow.generated.h"

#define UE_API PCG_API

UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = (Procedural))
class UPCGControlFlowSettings : public UPCGSettings
{
	GENERATED_BODY()

	//~Begin UPCGSettings interface
protected:
	virtual bool HasExecutionDependencyPin() const override { return false; }
	//~End UPCGSettings interface
};

USTRUCT(BlueprintType, meta = (Hidden))
struct FEnumSelector
{
	GENERATED_BODY()

	UPROPERTY(DisplayName="Enum Class", meta=(PCG_NotOverridable))
	TObjectPtr<UEnum> Class = nullptr;

	UPROPERTY(EditAnywhere, DisplayName="Enum Value", Category="Settings")
	int64 Value = 0;

	UE_API FText GetDisplayName() const;
	UE_API FString GetCultureInvariantDisplayName() const;
};

UENUM()
enum class EPCGControlFlowSelectionMode : uint8
{
	Integer,
	Enum,
	String
};

namespace PCGControlFlowConstants
{
	inline const FText SubtitleInt = NSLOCTEXT("FPCGControlFlow", "SubtitleInt", "Integer Selection");
	inline const FText SubtitleEnum = NSLOCTEXT("FPCGControlFlow", "SubtitleEnum", "Enum Selection");
	inline const FText SubtitleString = NSLOCTEXT("FPCGControlFlow", "SubtitleString", "String Selection");
	inline const FName DefaultPathPinLabel = TEXT("Default");
}

#undef UE_API
