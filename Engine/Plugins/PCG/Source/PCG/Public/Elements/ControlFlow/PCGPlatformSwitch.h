// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/ControlFlow/PCGPlatformSwitchBase.h"

#include "PCGPlatformSwitch.generated.h"

/** Statically activates/deactivates output pins based on the current platform/platform group. */
UCLASS(BlueprintType, ClassGroup = (Procedural), meta=(Keywords = "if branch platform"))
class UPCGPlatformSwitchSettings : public UPCGPlatformSwitchSettingsBase
{
	GENERATED_BODY()

public:
	//~ Begin UObject interface
#if WITH_EDITOR
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif
	//~ End UObject interface

	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PlatformSwitch")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("FPCGPlatformSwitchElement", "NodeTitle", "Platform Switch"); }
	virtual FText GetNodeTooltipText() const override;
#endif // WITH_EDITOR
	virtual bool IsPinStaticallyActive(const FName& PinLabel) const override;
	virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~ End UPCGSettings interface

public:
	/** Get the users selection, sanitized to remove None and duplicates. */
	TArray<FName> GetSanitizedPlatforms() const;

	/** Returns current platform and platform group, either by querying from editor or by using cooked values. */
	bool GetCurrentPlatformInfo(FName& OutPlatform, FName& OutPlatformGroup) const;

	/** Returns true if platform or platform group included in user's selection. */
	bool IsPlatformSelected(FName InPlatform) const;
	bool IsPlatformSelected(FName InPlatform, FName InCurrentPlatform, FName InCurrentPlatformGroup) const;

protected:
#if WITH_EDITOR
	UFUNCTION()
	TArray<FName> GetPlatformOptions() const;
#endif

protected:
	UPROPERTY(EditAnywhere, Category=Settings, meta = (GetOptions = "GetPlatformOptions"))
	TArray<FName> PlatformOutputs;

	/** The current platform used in cooked builds. */
	UPROPERTY()
	FName CookedPlatform;

	/** The current platform group used in cooked builds. */
	UPROPERTY()
	FName CookedPlatformGroup;
};

class FPCGPlatformSwitchElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
};
