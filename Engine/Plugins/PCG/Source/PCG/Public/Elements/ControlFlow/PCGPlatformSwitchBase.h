// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGControlFlow.h"
#include "RHIShaderPlatform.h"

#include "PCGPlatformSwitchBase.generated.h"

UCLASS(MinimalAPI, BlueprintType, Abstract, ClassGroup = (Procedural))
class UPCGPlatformSwitchSettingsBase : public UPCGControlFlowSettings
{
	GENERATED_BODY()

public:
	//~ Begin UObject interface
#if WITH_EDITOR
	virtual void PostInitProperties();
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostEditImport() override;
	virtual void BeginDestroy() override;
#endif
	//~ End UObject interface

	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::ControlFlow; }
#endif // WITH_EDITOR
	virtual bool HasDynamicPins() const override { return true; }
	virtual bool HasFlippedTitleLines() const override { return true; }
	virtual bool CanCullTaskIfUnwired() const override { return false; } // Useful to run unwired to only cull downstream nodes.
	virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const override;
	//~ End UPCGSettings interface

protected:
#if WITH_EDITOR
	void SetupPreviewPlatformChangeEvents();
	void TearDownPreviewPlatformChangeEvents();
	void OnPreviewPlatformChanged(EShaderPlatform NewShaderPlatform);
#endif

private:
#if WITH_EDITOR
	FDelegateHandle PreviewPlatformChangedHandle;
	FDelegateHandle PreviewFeatureLevelChangedHandle;
#endif
};
