// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGControlFlow.h"

#include "PCGQualitySelect.generated.h"

/**
 * Selects from input pins based on 'pcg.Quality' setting.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural), meta = (Keywords = "if bool switch quality"))
class UPCGQualitySelectSettings : public UPCGControlFlowSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("RuntimeQualitySelect")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("FPCGQualitySelectElement", "NodeTitle", "Runtime Quality Select"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::ControlFlow; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;
	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* Pin) const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseLowPin = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseMediumPin = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseHighPin = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseEpicPin = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseCinematicPin = false;
};

class FPCGQualitySelectElement : public IPCGElement
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
