// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/ControlFlow/PCGPlatformSwitchBase.h"

#include "RHIFeatureLevel.h"

#include "PCGShaderFeatureLevelSwitch.generated.h"

/** Statically activates/deactivates output pins based on the maximum shader feature level of the current platform. */
UCLASS(BlueprintType, ClassGroup = (Procedural), meta=(Keywords = "if branch platform shader"))
class UPCGShaderFeatureLevelSwitchSettings : public UPCGPlatformSwitchSettingsBase
{
	GENERATED_BODY()

public:
	//~ Begin UObject interface
#if WITH_EDITOR
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif
	//~ End UObject interface

	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ShaderFeatureLevelSwitch")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGShaderFeatureLevelSwitchElement", "NodeTitle", "Max Shader Feature Level Switch"); }
	virtual FText GetNodeTooltipText() const override;
#endif // WITH_EDITOR
	virtual bool IsPinStaticallyActive(const FName& PinLabel) const override;
	virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~ End UPCGSettings interface

public:
	/** Get the users selection, sanitized to remove None and duplicates. */
	TArray<ERHIFeatureLevel::Type> GetSanitizedRHIFeatureLevels() const;
	TArray<FName> GetSanitizedRHIFeatureLevelNames() const;

	/** Returns current platform and platform group, either by querying from editor or by using cooked values. */
	ERHIFeatureLevel::Type GetCurrentShaderFeatureLevel() const;

	/** Returns true if given shader feature level included in users selection. */
	bool IsRHIFeatureLevelSelected(ERHIFeatureLevel::Type InFeatureLevel, ERHIFeatureLevel::Type InCurrentFeatureLevel) const;
	bool IsRHIFeatureLevelSelected(FName InFeatureLevelName, ERHIFeatureLevel::Type InCurrentFeatureLevel) const;

protected:
#if WITH_EDITOR
	UFUNCTION()
	TArray<FName> GetRHIFeatureLevelOptions() const;
#endif

protected:
	UPROPERTY(EditAnywhere, Category=Settings, DisplayName = "Shader Feature Levels", meta = (GetOptions = "GetRHIFeatureLevelOptions"))
	TArray<FName> RHIFeatureLevelOutputs;

	/** The current platform group used in cooked builds. */
	UPROPERTY()
	int CookedFeatureLevel = INDEX_NONE;
};

class FPCGShaderFeatureLevelSwitchElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
};
