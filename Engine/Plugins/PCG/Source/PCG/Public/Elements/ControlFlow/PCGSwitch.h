// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGControlFlow.h"

#include "PCGSwitch.generated.h"

class FPCGSwitchElement;

/**
 * Routes data from the input pin, to a specific output pin based on a selection criteria (Int/String/Enum) 
 */
UCLASS(BlueprintType, ClassGroup = (Procedural), meta=(Keywords = "if multi enum branch"))
class UPCGSwitchSettings : public UPCGControlFlowSettings
{
	GENERATED_BODY()

	friend FPCGSwitchElement;

public:
	//~Begin UObject interface
	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~End UObject interface

	//~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Switch")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::ControlFlow; }
#endif // WITH_EDITOR
	virtual bool HasDynamicPins() const override { return true; }
	virtual bool OutputPinsCanBeDeactivated() const override { return true; }
	virtual bool IsPinStaticallyActive(const FName& PinLabel) const override;
	virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const override;
	
	virtual FString GetAdditionalTitleInformation() const override;
	virtual bool HasFlippedTitleLines() const override { return true; }

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	virtual void OnOverrideSettingsDuplicatedInternal(bool bSkippedPostLoad) override;
	//~End UPCGSettings interface

	/** Switch is dynamic if selection value is overridden / cannot be determined prior to execution. */
	virtual bool IsSwitchDynamic() const;

public:
	/** Determines the type of value to be used to select an output. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings)
	EPCGControlFlowSelectionMode SelectionMode = EPCGControlFlowSelectionMode::Integer;

	/** Determines which output will be selected if the selection mode is Integer. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings, meta=(PCG_Overridable, EditConditionHides, EditCondition="SelectionMode == EPCGControlFlowSelectionMode::Integer"))
	int32 IntegerSelection = 0;

	/** Determines the available output pin selection options. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings, meta=(EditConditionHides, EditCondition="SelectionMode == EPCGControlFlowSelectionMode::Integer"))
	TArray<int32> IntOptions = {0};

	/** Determines which output will be selected if the selection mode is String. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings, meta=(PCG_Overridable, EditConditionHides, NoResetToDefault, EditCondition="SelectionMode == EPCGControlFlowSelectionMode::String"))
	FString StringSelection;

	/** Determines the available output pin selection options. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings, meta=(EditConditionHides, EditCondition="SelectionMode == EPCGControlFlowSelectionMode::String"))
	TArray<FString> StringOptions;

	/** Determines which output pin will be selected if the selection mode is Enum. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings, meta=(PCG_Overridable, EditConditionHides, EditCondition="SelectionMode == EPCGControlFlowSelectionMode::Enum"))
	FEnumSelector EnumSelection;

	/** Since we use the Display name as the output pins for enums, this can be different in non-editor build. We need to keep a mapping between the enum value and its associated pin name. */
	UPROPERTY()
	mutable TMap<int64, FName> EnumValueToPinLabelMapping;

	/** Cached pin labels for use during the selection process. */
	UPROPERTY(Transient)
	TArray<FName> CachedPinLabels;

	/** To be called after the settings are modified directly in native code. */
	PCG_API void UpdateSettings(); 

protected:
	/** Returns true if the integer value exists in the user defined options. */
	bool IsValuePresent(int32 Value) const;

	/** Returns true if the string value exists in the user defined options. */
	bool IsValuePresent(const FString& Value) const;

	/** Returns true if the enum value exists within the selected enum class. */
	bool IsValuePresent(int64 Value) const;

	/** Returns the index of the currently selected pin. */
	int GetSelectedOutputPinIndex() const;

	/** Helper function to use the appropriate selection value to determine the current selection. Returns true if it succeeds or false if the value is invalid. */
	bool GetSelectedPinLabel(FName& OutSelectedPinLabel) const;

	void CachePinLabels();
	
	void CacheEnumValueToPinLabelsMapping() const;
};

class FPCGSwitchElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
};
