// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"

#include "PCGWait.generated.h"

/**
* Simple node to wait some amount of time and/or some amount of frames. Simply forwards inputs.
*/
UCLASS(BlueprintType, ClassGroup=(Procedural))
class UPCGWaitSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Wait")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGWaitSettings", "NodeTitle", "Wait"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGWaitSettings", "NodeTooltip", "Waits some time and/or frames. Not a node that should be used in production except in very special cases."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin="0.0", PCG_Overridable))
	double WaitTimeInSeconds = 1.0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin="0", PCG_Overridable))
	int64 WaitTimeInEngineFrames = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", PCG_Overridable))
	int64 WaitTimeInRenderFrames = 0;

	// Controls whether all conditions are needed or any condition is sufficient to end the wait.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bEndWaitWhenAllConditionsAreMet = true;
};

struct FPCGWaitContext : public FPCGContext
{
	double StartTime = -1.0;
	uint64 StartEngineFrame = 0;
	uint64 StartRenderFrame = 0;
	bool bQueriedTimers = false;
};

class FPCGWaitElement : public IPCGElementWithCustomContext<FPCGWaitContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};