// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGReplaceDataByTags.generated.h"

class FPCGReplaceDataByTagElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};

/**
* Selects data from primary input or from replacement input based on common tags or tag values.
*/
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGReplaceDataByTagSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ReplaceDataByTag")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGReplaceDataByTagElement", "NodeTitle", "Replace Data By Tag"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGReplaceDataByTagElement", "NodeTooltip", "Replaces data in the input data collection by data from the replacement collection based on common tags."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif
	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGReplaceDataByTagElement>(); }
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	TArray<FString> Tags;
};