// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Elements/Grammar/PCGSubdivisionBase.h"

#include "PCGDuplicateCrossSections.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDuplicateCrossSectionsSettings : public UPCGSubdivisionBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	void PostLoad() override;
	//~End UObject interface
	
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Set it to true if you want the extrude vector to be taken from the input spline as attribute, or fixed in the settings. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bExtrudeVectorAsAttribute = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bExtrudeVectorAsAttribute", EditConditionHides, DisplayAfter = bExtrudeVectorAsAttribute, PCG_Overridable))
	FVector ExtrudeVector = FVector(0.0, 0.0, 1000.0);

	/** Attribute to be taken from the input spline containing the extrude vector for the slicing. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bExtrudeVectorAsAttribute", EditConditionHides, DisplayAfter = bExtrudeVectorAsAttribute, PCG_Overridable))
	FPCGAttributePropertyInputSelector ExtrudeVectorAttribute;

	/** Choose if the Extra Output Attributes are added to the Data domain or the Elements domain. If ForwardAttributesFromModulesInfo is true, module info attributes will also be pushed to the data domain. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable))
	bool bExtraOutputAttributesOnDataDomain = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputSplineIndexAttribute = true;

	/** Name of the spline index output attribute name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputSplineIndexAttribute"))
	FName SplineIndexAttributeName = TEXT("SplineIndex");

	/** Do not post node errors when the grammar does not fit the intended cross-sections. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	bool bSilenceGrammarDoesNotFitErrors = false;
};

class FPCGDuplicateCrossSectionsElement : public FPCGSubdivisionBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
