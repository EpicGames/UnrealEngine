// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGPrintGrammar.generated.h"

/**
 * Prints the result of an interpreted grammar.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPrintGrammarSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PrintGrammar")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGPrintGrammarElement", "NodeTitle", "Print Grammar"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGPrintGrammarElement", "NodeTooltip", "Prints the result of an interpreted grammar."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Debug; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return {}; }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** The grammar to interpret. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (MultiLine, PCG_Overridable))
	FString Grammar;
};

class FPCGPrintGrammarElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};