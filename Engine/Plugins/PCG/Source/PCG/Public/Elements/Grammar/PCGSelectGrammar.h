// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "PCGSubdivisionBase.h"

#include "PCGSelectGrammar.generated.h"

/** An enum representation of basic numerical comparisons for the Select Grammar operation. */
UENUM()
enum class EPCGSelectGrammarComparator : uint16
{
	// 32 options
	BinaryOps = 1 << 5 UMETA(Hidden),
	Select UMETA(DisplayName = "Select", ToolTip = "Select this grammar. Always evaluates to true."),
	LessThan UMETA(DisplayName = "<", ToolTip = "Select if 'less than' (A < B)"),
	LessThanEqualTo UMETA(DisplayName = "<=", ToolTip = "Select if 'less than or equal to' (A <= B)"),
	EqualTo UMETA(DisplayName = "==", ToolTip = "Select if 'equal to' (A == B)"),
	GreaterThanEqualTo UMETA(DisplayName = ">=", ToolTip = "Select if 'greater than or equal to' (A >= B)"),
	GreaterThan UMETA(DisplayName = ">", ToolTip = "Select if 'greater than' (A > B)"),

	// 64 options
	TernaryOps = 1 << 6 UMETA(Hidden),
	RangeExclusive UMETA(DisplayName = "Range (exclusive)", ToolTip = "Select if 'within exclusive range' (B < A < C)"),
	RangeInclusive UMETA(DisplayName = "Range (inclusive)", ToolTip = "Select if 'within inclusive range' (B <= A <= C)"),
};

/** Criteria to compare against an input value for conditionally selecting a grammar. */
USTRUCT(BlueprintType)
struct FPCGSelectGrammarCriterion
{
	GENERATED_BODY()

	/** Key representation of a potential grammar selection. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName Key;

	/** Comparison operator of a selected input value against a criterion's value. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGSelectGrammarComparator Comparator = EPCGSelectGrammarComparator::LessThan;

	/** Compared against the input value. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Comparator != EPCGSelectGrammarComparator::Select", EditConditionHides))
	double FirstValue = 0.0;

	/** Used for comparison in range-based or ternary conditions. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Comparator == EPCGSelectGrammarComparator::RangeExclusive || Comparator == EPCGSelectGrammarComparator::RangeInclusive", EditConditionHides))
	double SecondValue = 0.0;

	/** The selected grammar result if the condition evaluates to true. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (MultiLine))
	FString Grammar;
};

/** Used for directly mapping custom attribute names to a criteria attribute set. */
USTRUCT(BlueprintType)
struct FPCGSelectGrammarCriteriaAttributeNames
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName KeyAttributeName = TEXT("Key");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName ComparatorAttributeName = TEXT("Comparator");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName FirstValueAttributeName = TEXT("FirstValue");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName SecondValueAttributeName = TEXT("SecondValue");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName GrammarAttributeName = TEXT("Grammar");
};

/** Select a grammar by comparing an input attribute against a provided set criteria one-by-one in a sequential order. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSelectGrammarSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGSelectGrammarSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif
	virtual bool HasDynamicPins() const override { return true; }
	virtual void PostLoad() override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Select the key with an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bKeyAsAttribute = true;

	/** Each criterion has an associated key, this node will only evaluate (in order) the criteria that match this value. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bKeyAsAttribute", EditConditionHides))
	FName Key;

	/** Attribute selector to read a key attribute from the input data. Each criterion has an associated key, this node will only evaluate (in order) the criteria that match the value of this attribute for each element. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bKeyAsAttribute", EditConditionHides))
	FPCGAttributePropertyInputSelector KeyAttribute;

	/** The attribute on the input data to be compared against. Will be numerically evaluated. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector ComparedValueAttribute;

	/** Toggle for an additional input "Selection Criteria Data" which accepts criteria info as an attribute set matching the structure type 'FPCGSelectGrammarCriteria'. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bCriteriaAsInput = false;

	/** Selection criteria that will be evaluated in order. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bCriteriaAsInput", EditConditionHides))
	TArray<FPCGSelectGrammarCriterion> Criteria;

	/** If no grammar is selected for a given point, pass through the key value. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bCopyKeyForUnselectedGrammar = false;

	/** Remap expected attribute names for the comparison criteria. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle, EditCondition = "bCriteriaAsInput", EditConditionHides))
	bool bRemapCriteriaAttributeNames = false;

	/** The attribute names expected for the comparison criteria. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, DisplayName = "Remap Criteria Attribute Names", EditCondition = "bRemapCriteriaAttributeNames", DisplayAfter = "bRemapCriteriaAttributeNames"))
	FPCGSelectGrammarCriteriaAttributeNames CriteriaAttributeNames;

	/** The attribute to output the selected grammar. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyOutputSelector OutputGrammarAttribute;
};

// TODO: For V2, this element could be async and/or timesliced
class FPCGSelectGrammarElement : public FPCGSubdivisionBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};