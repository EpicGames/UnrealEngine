// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGSplineToSegment.generated.h"

/**
* Take a spline as input and create a point data, with each point being a segment defined by 2 connected control points.
* The point position will be in the middle of those 2 control points, and the extents of the point will be half the difference between those 2 control points.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSplineToSegmentSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Can extract the tangents with previous segment and next segment. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bExtractTangents = false;

	/** Can extract the angle with previous tangent and next tangent. Will be 0 at the extremity for non-closed splines. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bExtractAngles = true;

	/** Can set the index of the previous and next segment (to keep information on connectivity). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bExtractConnectivityInfo = true;

	/** Can output a global attribute to know if the spline points order is clockwise or counterclockwise. (Only for closed loops). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bExtractClockwiseInfo = true;
};

class FPCGSplineToSegmentElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};

