// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Elements/Grammar/PCGSubdivisionBase.h"

#include "Elements/PCGSplitPoints.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGSubdivideSegment.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSubdivideSegmentSettings : public UPCGSubdivisionBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Subdivision direction in point local space. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGSplitAxis SubdivisionAxis = EPCGSplitAxis::X;

	/** Use an attribute to determine whether we should flip axis. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bFlipAxisAsAttribute = false;

	/** If we need to flip axis. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bFlipAxisAsAttribute", EditConditionHides, ShowAfter = "bFlipAxisAsAttribute"))
	bool bShouldFlipAxis = false;

	/** Name of the attribute to know if we need to flip axis. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bFlipAxisAsAttribute", EditConditionHides, ShowAfter="bFlipAxisAsAttribute"))
	FPCGAttributePropertyInputSelector FlipAxisAttribute;

	/** If the subdivision with a given grammar doesn't fill the entire segment, setting it to true makes it a valid case. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bAcceptIncompleteSubdivision = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputModuleIndexAttribute = false;

	/** Name of the module index output attribute name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputModuleIndexAttribute"))
	FName ModuleIndexAttributeName = TEXT("ModuleIndex");

	/** Output attributes labeling the first and final points after subdivision. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable))
	bool bOutputExtremityAttributes = false;

	/** Name of the attribute labeling the first output point from the first module. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputExtremityAttributes", EditConditionHides))
	FName IsFirstAttributeName = TEXT("IsFirst");

	/** Name of the attribute labeling the final output point from the final module. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputExtremityAttributes", EditConditionHides))
	FName IsFinalAttributeName = TEXT("IsFinal");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputExtremityNeighborIndexAttribute = false;

	/** Name of the extremity neighbor index output attribute name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputExtremityNeighborIndexAttribute"))
	FName ExtremityNeighborIndexAttributeName = TEXT("ExtremityNeighborIndex");

	/** Do not post node errors when the grammar does not fit the segment. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	bool bSilenceGrammarDoesNotFitErrors = false;
};

class PCGSubdivideSegmentHelpers;

class FPCGSegmentSubdivisionElement : public FPCGSubdivisionBaseElement
{
public:
	friend class PCGSubdivideSegmentHelpers;
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
