// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Elements/Grammar/PCGSubdivisionBase.h"

#include "PCGSubdivideSpline.generated.h"

namespace PCGSubdivideSplineConstants
{
	constexpr double MinimumBisectionTolerance = 1.0e-6;
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSubdivideSplineSettings : public UPCGSubdivisionBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SubdivideSpline")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSubdivideSplineElement", "NodeTitle", "Subdivide Spline"); }
#endif // WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** If the subdivision with a given grammar doesn't fill the entire spline, setting it to true makes it a valid case. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bAcceptIncompleteSubdivision = false;

	/** Select the module height from an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bModuleHeightAsAttribute = false;

	/** The height of each placed module. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bModuleHeightAsAttribute", EditConditionHides))
	double ModuleHeight = 100.0;

	/** Selection will be used as the module height for placed modules. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bModuleHeightAsAttribute", EditConditionHides))
	FPCGAttributePropertyInputSelector ModuleHeightAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputModuleIndexAttribute = false;

	/** Name of the module index output attribute name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputModuleIndexAttribute"))
	FName ModuleIndexAttributeName = TEXT("ModuleIndex");

	/** Output attributes labeling the first and final points after subdivision. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable))
	bool bOutputExtremityAttributes = false;

	/** Name of the attribute labeling the first output point from the first module. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputExtremityAttributes", EditConditionHides))
	FName IsFirstAttributeName = TEXT("IsFirst");

	/** Name of the attribute labeling the final output point from the final module. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputExtremityAttributes", EditConditionHides))
	FName IsFinalAttributeName = TEXT("IsFinal");

	/** Do not post node errors when the grammar does not fit the spline. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	bool bSilenceGrammarDoesNotFitErrors = false;

	/** For deprecation purposes only. If this is false, the bounds will be on Z between 0 and Height (previous behavior). If this is true, bounds will be between -Height/2 and Height/2. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (EditCondition="!bCenterBounds", EditConditionHides))
	bool bCenterBounds = true;

	/** Maximum acceptable distance tolerance between placed modules (overlap or gap) - a smaller number is more precise. Affects performance if the number is very small. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable, ClampMin="0.000001"))
	double ModulePlacementTolerance = 0.01;
};

class FPCGSubdivideSplineElement : public FPCGSubdivisionBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
