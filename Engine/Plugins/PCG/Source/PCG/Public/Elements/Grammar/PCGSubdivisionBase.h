// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Grammar/PCGGrammar.h"
#include "Utils/PCGLogErrors.h"

#include "Algo/Copy.h"

#include "PCGSubdivisionBase.generated.h"

class UPCGData;
class UPCGBasePointData;

USTRUCT(BlueprintType)
struct FPCGSubdivisionSubmodule
{
	GENERATED_BODY()

	/** Symbol for the grammar. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "")
	FName Symbol = NAME_None;

	/** Size of the block, aligned on the segment direction. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "")
	double Size = 100.0;

	/** If the volume can be scaled to fit the remaining space or not. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "")
	bool bScalable = false;

	/** For easier debugging, using Point color in conjunction with PCG Debug Color Material. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Debug")
	FVector4 DebugColor = FVector4::One();
};

namespace PCGSubdivisionBase::Constants
{
	const FName ModulesInfoPinLabel = TEXT("ModulesInfo");
	const FName SymbolAttributeName = TEXT("Symbol");
	const FName SizeAttributeName = TEXT("Size");
	const FName ScalableAttributeName = TEXT("Scalable");
	const FName DebugColorAttributeName = TEXT("DebugColor");
}

USTRUCT(BlueprintType)
struct FPCGSubdivisionModuleAttributeNames
{
	GENERATED_BODY()

public:
	/** Mandatory. Expected type: FName. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "")
	FName SymbolAttributeName = PCGSubdivisionBase::Constants::SymbolAttributeName;

	/** Mandatory. Expected type: double. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "")
	FName SizeAttributeName = PCGSubdivisionBase::Constants::SizeAttributeName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "", meta = (InlineEditConditionToggle))
	bool bProvideScalable = false;

	/** Optional. Expected type: bool. If disabled, default value will be false. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "", meta = (EditCondition = "bProvideScalable"))
	FName ScalableAttributeName = PCGSubdivisionBase::Constants::ScalableAttributeName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "", meta = (InlineEditConditionToggle))
	bool bProvideDebugColor = false;

	/** Optional. Expected type: Vector4. If disabled, default value will be (1.0, 1.0, 1.0, 1.0). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "", meta = (EditCondition = "bProvideDebugColor"))
	FName DebugColorAttributeName = PCGSubdivisionBase::Constants::DebugColorAttributeName;
};

UCLASS(MinimalAPI, Abstract, ClassGroup = (Procedural))
class UPCGSubdivisionBaseSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

	virtual bool UseSeed() const override { return true; }
	//~End UPCGSettings interface

	virtual void PostLoad() override;

public:
	/** Set it to true to pass the info as attribute set. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bModuleInfoAsInput = false;

	/** Fixed array of modules used for the subdivision. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bModuleInfoAsInput", EditConditionHides, DisplayAfter = bModuleInfoAsInput))
	TArray<FPCGSubdivisionSubmodule> ModulesInfo;

	/** Fixed array of modules used for the subdivision. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bModuleInfoAsInput", EditConditionHides, DisplayAfter = bModuleInfoAsInput, DisplayName = "Attribute Names for Module Info"))
	FPCGSubdivisionModuleAttributeNames ModulesInfoAttributeNames;

	/** An encoded string that represents how to apply a set of rules to a series of defined modules. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGGrammarSelection GrammarSelection;

	/** Controls whether we'll use an attribute to drive random seeding for stochastic processes in the subdivision. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bUseSeedAttribute = false;

	/** Attribute to use to drive seed selection. It should be convertible to an integer. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bUseSeedAttribute", PCG_Overridable))
	FPCGAttributePropertyInputSelector SeedAttribute;

	/** Do a match and set with the incoming modules info, only if the modules info is passed as input. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (EditCondition = "bModuleInfoAsInput", EditConditionHides, PCG_Overridable))
	bool bForwardAttributesFromModulesInfo = false;

	/** Name of the Symbol output attribute name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable))
	FName SymbolAttributeName = PCGSubdivisionBase::Constants::SymbolAttributeName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputSizeAttribute = true;

	/** Name of the Size output attribute name, ignored if Forward Attributes From Modules Info is true. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputSizeAttribute"))
	FName SizeAttributeName = PCGSubdivisionBase::Constants::SizeAttributeName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputScalableAttribute = true;

	/** Name of the Scalable output attribute name, ignored if Forward Attributes From Modules Info is true. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputScalableAttribute"))
	FName ScalableAttributeName = PCGSubdivisionBase::Constants::ScalableAttributeName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputDebugColorAttribute = false;

	/** Name of the Debug Color output attribute name, ignored if Forward Attributes From Modules Info is true. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra Output Attributes", meta = (PCG_Overridable, EditCondition = "bOutputDebugColorAttribute"))
	FName DebugColorAttributeName = PCGSubdivisionBase::Constants::DebugColorAttributeName;

private:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "Use 'GrammarSelection' instead.")
	UPROPERTY()
	bool bGrammarAsAttribute_DEPRECATED = false;

	UE_DEPRECATED(5.5, "Use 'GrammarSelection' instead.")
	UPROPERTY()
	FString Grammar_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};

namespace PCGSubdivisionBase
{
	using FModuleInfoMap = TMap<FName, FPCGSubdivisionSubmodule>;

	// Materialized modules created from a tokenized grammar.
	template <typename T>
	struct TModuleInstance
	{
		TModuleInstance() = default;
		explicit TModuleInstance(const T* InModule) : Module(InModule) {}

		const T* Module = nullptr;
		double ExtraScale = 0.0;
		bool bIsValid = true;
		bool bSkipExpansion = false; // FIXME used to mark a token so it cannot be repeated - TODO rename this to bCanBeRepeated
	};

	PCGGrammar::FTokenizedGrammar GetTokenizedGrammar(FPCGContext* InContext, const FString& InGrammar, const FModuleInfoMap& InModulesInfo, double& OutMinSize);

	template <typename T>
	bool Subdivide(const T& Root, double Length, TArray<TModuleInstance<T>>& OutModuleInstances, double& RemainingLength, FPCGContext* InOptionalContext = nullptr, int32 InOptionalAdditionalSeed = 0, const bool bQuiet = false)
	{
		OutModuleInstances.Reset();
		RemainingLength = Length;

		if (!Root.IsValid() || FMath::IsNearlyZero(Length))
		{
			return true;
		}

		FRandomStream RandomStream((InOptionalContext ? InOptionalContext->GetSeed() : 42) + InOptionalAdditionalSeed);
		TArray<TModuleInstance<T>> CurrentModules;

		// Start with root
		CurrentModules.Emplace(&Root);
		RemainingLength -= Root.GetMinSize(); // Here we use the min size because we'll consume concrete size only during expansion

		if (RemainingLength < 0)
		{
			return false;
		}

		// Working data sets
		TArray<TModuleInstance<T>> ExpandedModules;

		auto ExpandSubmodule = [&ExpandedModules](const T& Submodule)
		{
			if (!Submodule.IsValid())
			{
				return;
			}

			int NumConcreteModules = Submodule.GetNumRepeat();
			bool bHasRepeatModule = false;

			if (NumConcreteModules == PCGGrammar::InfiniteRepetition)
			{
				NumConcreteModules = 0;
				bHasRepeatModule = true;
			}
			else if (NumConcreteModules == PCGGrammar::AtLeastOneRepetition)
			{
				NumConcreteModules = 1;
				bHasRepeatModule = true;
			}

			for (int C = 0; C < NumConcreteModules; ++C)
			{
				TModuleInstance<T>& ExpandedSubmodule = ExpandedModules.Emplace_GetRef(&Submodule);
				ExpandedSubmodule.bSkipExpansion = false;
			}

			if (bHasRepeatModule)
			{
				TModuleInstance<T>& ExpandedSubmodule = ExpandedModules.Emplace_GetRef(&Submodule);
				ExpandedSubmodule.bSkipExpansion = true;
			}
		};

		bool bNotDone = true;
		while (bNotDone)
		{
			bNotDone = false;

			// 1. Expand "concrete" symbols, e.g. root, sequence, stochastic, priority
			// Implementation note; since we've already consumed the min size, there's no need to update anything when expanding nodes, only when replacing with another choice.
			ExpandedModules.Reset();
			ExpandedModules.Reserve(CurrentModules.Num());

			int NumExpandedModules = 0;

			for (TModuleInstance<T>& CurrentModule : CurrentModules)
			{
				if (!CurrentModule.bIsValid)
				{
					// Module was discarded, could be culled. Normally should have no size
					continue;
				}
				else if (CurrentModule.bSkipExpansion)
				{
					// Do not expand repetition modules until they are concretized
					ExpandedModules.Add(CurrentModule);
				}
				else if (CurrentModule.Module->GetType() == PCGGrammar::EModuleType::Root || CurrentModule.Module->GetType() == PCGGrammar::EModuleType::Sequence)
				{
					for (const T& Submodule : CurrentModule.Module->Submodules)
					{
						ExpandSubmodule(Submodule);
					}

					++NumExpandedModules;
				}
				else if (CurrentModule.Module->GetType() == PCGGrammar::EModuleType::Priority)
				{
					// Replace current module by the first of its childen that has a min size that fits in with the remaining length
					bool bModuleExpanded = false;
					for (const T& Submodule : CurrentModule.Module->Submodules)
					{
						if (!Submodule.IsValid())
						{
							continue;
						}

						const double DeltaMinSize = (Submodule.GetMinSize() - CurrentModule.Module->GetUnitSize());
						check(DeltaMinSize >= 0);
						if (RemainingLength >= DeltaMinSize)
						{
							bModuleExpanded = true;
							ExpandSubmodule(Submodule);
							RemainingLength -= DeltaMinSize;
							break;
						}
					}

					// Implementation note: if the module doesn't pick anything (here) it will be removed automatically.
					check(bModuleExpanded || CurrentModule.Module->GetMinSize() == 0);
					++NumExpandedModules;
				}
				else if (CurrentModule.Module->GetType() == PCGGrammar::EModuleType::Stochastic)
				{
					// TODO: investigate if doing expansion of stochastic modules in a random order would be a better idea.
					// Replace current module by a random pick according to total weights of "valid" choices we can still make
					int TotalValidWeight = 0;

					// Compute total weight for all valid permutations
					for (const T& Submodule : CurrentModule.Module->Submodules)
					{
						if (!Submodule.IsValid())
						{
							continue;
						}

						const double DeltaMinSize = Submodule.GetMinSize() - CurrentModule.Module->GetUnitSize();
						check(DeltaMinSize >= 0);
						if (DeltaMinSize <= RemainingLength)
						{
							TotalValidWeight += Submodule.GetWeight();
						}
					}

					int WeightPick = RandomStream.RandRange(0, TotalValidWeight - 1);
					bool bModuleExpanded = false;

					for (const T& Submodule : CurrentModule.Module->Submodules)
					{
						if (!Submodule.IsValid())
						{
							continue;
						}

						const double DeltaMinSize = Submodule.GetMinSize() - CurrentModule.Module->GetUnitSize();
						check(DeltaMinSize >= 0);
						if (DeltaMinSize <= RemainingLength)
						{
							if (Submodule.GetWeight() > WeightPick)
							{
								bModuleExpanded = true;
								ExpandSubmodule(Submodule);
								RemainingLength -= DeltaMinSize;
								break;
							}
							else
							{
								WeightPick -= Submodule.GetWeight();
							}
						}
					}

					check(bModuleExpanded);
					++NumExpandedModules;
				}
				else // Literals - copy as-is
				{
					ExpandedModules.Add(CurrentModule);
				}
			}

			// Move expanded modules to current modules
			CurrentModules = MoveTemp(ExpandedModules);

			bNotDone |= (NumExpandedModules > 0);

			// 2. Concretize repetitions as needed
			int NumConcretizedModules = 0;
			for (int ModuleIndex = 0; ModuleIndex < CurrentModules.Num(); ++ModuleIndex)
			{
				TModuleInstance<T>& CurrentModule = CurrentModules[ModuleIndex];
				if (!CurrentModule.bSkipExpansion)
				{
					continue;
				}

				check(CurrentModule.Module->GetMinConcreteSize() >= CurrentModule.Module->GetUnitSize());
				check(CurrentModule.Module->GetUnitSize() >= CurrentModule.Module->GetMinSize());
				if(CurrentModule.Module->GetMinConcreteSize() > RemainingLength)
				{
					CurrentModule.bIsValid = false;
				}
				else
				{
					// Update length
					RemainingLength -= CurrentModule.Module->GetUnitSize();

					// Duplicate this module, mark it non-repeatable
					TModuleInstance<T> ModuleToDuplicate = CurrentModule;
					ModuleToDuplicate.bSkipExpansion = false;

					// Insert left
					CurrentModules.Insert(ModuleToDuplicate, ModuleIndex);
					++ModuleIndex;
					++NumConcretizedModules;
				}
			}

			bNotDone |= (NumConcretizedModules > 0);
		}

		// Remove invalid modules
		TArray<TModuleInstance<T>> PreviousModules = MoveTemp(CurrentModules);
		Algo::CopyIf(PreviousModules, CurrentModules, [](const TModuleInstance<T>& InModule) { return InModule.bIsValid; });

#if WITH_EDITOR
		// Perform some early validation and see if there's a mismatch on the reported size and the one actually placed
		double CountedLength = RemainingLength;
		for (TModuleInstance<T>& CurrentModule : CurrentModules)
		{
			CountedLength += CurrentModule.Module->GetUnitSize();
		}

		ensure(FMath::Abs(CountedLength-Length) < 1.0);
#endif

		// 7. Finally, apply adjusted scales to modules that support it.
		check(RemainingLength >= 0);
		if (!FMath::IsNearlyZero(RemainingLength))
		{
			int32 NumScalableModules = 0;
			double ScalableLength = 0;
			for (TModuleInstance<T>& CurrentModule : CurrentModules)
			{
				if (CurrentModule.Module->IsScalable())
				{
					++NumScalableModules;
					// implementation note: at this point we have only unit-literals, so we need to ignore repetitions, if any here, hence using the unit size
					ScalableLength += CurrentModule.Module->GetUnitSize();
				}
			}

			if (ScalableLength > 0)
			{
				for (TModuleInstance<T>& CurrentModule : CurrentModules)
				{
					if (CurrentModule.Module->IsScalable())
					{
						CurrentModule.ExtraScale = (RemainingLength / ScalableLength);
					}
				}

				RemainingLength = 0;
			}
		}

		OutModuleInstances = MoveTemp(CurrentModules);

		return true;
	}
}

class FPCGSubdivisionBaseElement : public IPCGElement
{
public:
	// Worth computing a full CRC in case we can halt change propagation/re-executions
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override { return true; }

protected:
	using FModuleInfoMap = PCGSubdivisionBase::FModuleInfoMap;

	FModuleInfoMap GetModulesInfoMap(FPCGContext* InContext, const TArray<FPCGSubdivisionSubmodule>& SubmodulesInfo, const UPCGParamData*& OutModuleInfoParamData) const;
	FModuleInfoMap GetModulesInfoMap(FPCGContext* InContext, const FPCGSubdivisionModuleAttributeNames& InSubdivisionModuleAttributeNames, const UPCGParamData*& OutModuleInfoParamData) const;
	FModuleInfoMap GetModulesInfoMap(FPCGContext* InContext, const UPCGSubdivisionBaseSettings* InSettings, const UPCGParamData*& OutModuleInfoParamData) const;
	PCGGrammar::FTokenizedGrammar GetTokenizedGrammar(FPCGContext* InContext, const UPCGData* InputData, const UPCGSubdivisionBaseSettings* InSettings, const FModuleInfoMap& InModulesInfo, double& OutMinSize) const;
	TMap<FString, PCGGrammar::FTokenizedGrammar> GetTokenizedGrammarForPoints(FPCGContext* InContext, const UPCGBasePointData* InputData, const UPCGSubdivisionBaseSettings* InSettings, const FModuleInfoMap& InModulesInfo, double& OutMinSize) const;
	bool MatchAndSetAttributes(const TArray<FPCGTaggedData>& InputData, TArray<FPCGTaggedData>& OutputData, const UPCGParamData* InModuleInfoParamData, const UPCGSubdivisionBaseSettings* InSettings, const FPCGMetadataDomainID& InTargetDomain = PCGMetadataDomainID::Default) const;
};
