// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGExternalData.h"

#include "PCGCommon.h"

class UDataTable;

#include "PCGDataTableElement.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGLoadDataTableSettings : public UPCGExternalDataSettings
{
	GENERATED_BODY()

public:
	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("LoadDataTable")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
#endif

	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TSoftObjectPtr<UDataTable> DataTable;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ValidEnumValues = "Point, Param"))
	EPCGExclusiveDataType OutputType = EPCGExclusiveDataType::Point;

	/** By default, data table loading is asynchronous, can force it synchronous if needed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

class FPCGLoadDataTableElement : public FPCGExternalDataElement
{
protected:
	virtual bool PrepareLoad(FPCGExternalDataContext* Context) const override;
	virtual bool ExecuteLoad(FPCGExternalDataContext* Context) const override;
};