// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Data/DataView/PCGDataView.h"
#include "Data/DataView/PCGDataViewCSVConverter.h"

#include "PCGDataViewToStringElement.generated.h"

/** Serialize a PCG Data via a Data View into a formatted FString, i.e. CSV, Json, etc. */
UCLASS(BlueprintType, MinimalAPI, ClassGroup = (Procedural))
class UPCGDataViewToStringSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGDataViewToStringSettings();

	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PCGDataViewToString")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDataViewToStringElement", "NodeTitle", "Data View To String"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGDataViewToStringElement", "NodeTooltip", "Convert the Data View selection to a string attribute."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual TArray<FPCGSettingsOverridableParam> GatherOverridableParams() const override;
#endif // WITH_EDITOR
	virtual void FixingOverridableParamPropertyClass(FPCGSettingsOverridableParam& Param) const override;
	//~ End UPCGSettings interface

public:
	// The converter format to construct from the Data View
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, NoClear)
	TSubclassOf<UPCGDataViewConverterBase> ConverterClass = UPCGDataViewCSVConverter::StaticClass();

	// Parameters for the conversion
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ShowOnlyInnerProperties, StructTypeConst))
	FInstancedStruct ConverterParameters;
};

class FPCGDataViewToStringExecutionContext : public FPCGContext
{
protected:
	virtual FStructView GetExternalStructContainerForOverridableParam(const FPCGSettingsOverridableParam& InParam) override;
};

class FPCGDataViewToStringElement final : public IPCGElementWithCustomContext<FPCGDataViewToStringExecutionContext>
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;
};
