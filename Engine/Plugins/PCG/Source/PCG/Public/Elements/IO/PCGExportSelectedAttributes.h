// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "Serialization/CustomVersion.h"

#include "PCGExportSelectedAttributes.generated.h"

UENUM(BlueprintType)
enum class EPCGExportAttributesFormat : uint8
{
	Binary UMETA(Tooltip = "Export binary data to file using the Unreal archive system."),
	Json UMETA(Tooltip = "Output the asset values into a JSON format.")
};

UENUM(BlueprintType)
enum class EPCGExportAttributesLayout : uint8
{
	ByElement UMETA(Tooltip = "Use the elements as the main data object. Each element contains all its attributes."),
	ByAttribute UMETA(Tooltip = "Use the attributes as the main data object. Each attribute grouping will contain the element values in sequential order.")
};

/** @todo_pcg:
 * - Provide a tooltip or other feedback to dynamically generate the expected format to make it easier for the user to consume.
 * - Abstract this out to an API and have the node leverage the API.
 * - Each Data Type can be in charge of the default "selection" of properties/attributes.
 */

/**
 * Exports the selected attributes directly to file in a specified format.
 * Note: This node is only operational on traditional development platforms (Windows, Linux, Mac) where the Editor can be used.
 */
UCLASS(BlueprintType, MinimalAPI, ClassGroup = (Procedural))
class UPCGExportSelectedAttributesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGExportSelectedAttributesSettings();

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PCGExportSelectedAttributes")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGExportSelectedAttributesElement", "NodeTitle", "Export Selected Attributes"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGExportSelectedAttributesElement", "NodeTooltip", "Exports selected attributes directly to a file in a specified format."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
#endif // WITH_EDITOR

	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

protected:
	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface

public:
	/** Data will be exported to a local file in this format. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	EPCGExportAttributesFormat Format = EPCGExportAttributesFormat::Binary;

	/** Determines how the data will be laid out in the export. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "Format != EPCGExportAttributesFormat::Binary", PCG_Overridable))
	EPCGExportAttributesLayout Layout = EPCGExportAttributesLayout::ByElement;

	/** The directory to save the data within. If none is selected a dialog will open by default. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	FDirectoryPath Path;

	/** The file name (without extension) to export the data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	FString FileName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	bool bExportAllAttributes = true;

	/** The attributes to use as sources for the data export. Only those selected will be exported from the input data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "!bExportAllAttributes", EditConditionHides, PCG_Overridable))
	TArray<FPCGAttributePropertyInputSelector> AttributeSelectors;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Advanced)
	bool bAddCustomDataVersion = false;

	/** Extra user version for any special requirements. Must be >= 0.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Advanced, meta = (EditCondition = "bAddCustomDataVersion", EditConditionHides, ClampMin = 0))
	int32 CustomVersion = 0;
};

class FPCGExportSelectedAttributesElement final : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	// @todo_pcg: Crc the file bytes and version
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
};
