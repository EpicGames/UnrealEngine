// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGExternalData.generated.h"

#define UE_API PCG_API

struct FPCGExternalDataContext;

/** Base class for external data input settings */
UCLASS(MinimalAPI, Abstract, ClassGroup = (Procedural))
class UPCGExternalDataSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return DefaultPointOutputPinProperties(); }
	// ~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	TMap<FString, FPCGAttributePropertyInputSelector> AttributeMapping;
};

class FPCGExternalDataElement : public IPCGElement
{
public:
	// Loading needs to be done on the main thread and accessing objects outside of PCG might not be thread safe, so taking the safe approach
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	UE_API virtual FPCGContext* CreateContext() override;
	UE_API virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	UE_API virtual bool ExecuteInternal(FPCGContext* InContext) const override;

	virtual bool PrepareLoad(FPCGExternalDataContext* Context) const = 0;
	UE_API virtual bool ExecuteLoad(FPCGExternalDataContext* Context) const;
	virtual bool PostExecuteLoad(FPCGExternalDataContext* Context) const { return true; }
};

#undef UE_API
