// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Data/PCGPointData.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

struct FPCGExternalDataContext : public FPCGContext, public IPCGAsyncLoadingContext
{
	// Used for data table parsing
	struct FRowToPointAccessors
	{
		TUniquePtr<const IPCGAttributeAccessor> RowAccessor;
		TUniquePtr<IPCGAttributeAccessor> PointAccessor;
		FPCGAttributePropertyOutputSelector Selector;

		FRowToPointAccessors() = default;
		FRowToPointAccessors(const FRowToPointAccessors&) = delete;
		FRowToPointAccessors(FRowToPointAccessors&&) = default;
		FRowToPointAccessors& operator=(FRowToPointAccessors&&) = default;
		FRowToPointAccessors& operator=(const FRowToPointAccessors&) = delete;
		FRowToPointAccessors(TUniquePtr<const IPCGAttributeAccessor> InRow, TUniquePtr<IPCGAttributeAccessor> InPoint, const FPCGAttributePropertySelector& InSelector)
			: RowAccessor(MoveTemp(InRow)), PointAccessor(MoveTemp(InPoint))
		{
			Selector.ImportFromOtherSelector(InSelector);
		}
	};

	struct FPointDataAccessorsMapping
	{
		UPCGData* Data = nullptr;
		UPCGMetadata* Metadata = nullptr;
		TArray<FRowToPointAccessors> RowToPointAccessors;
		TUniquePtr<const IPCGAttributeAccessorKeys> RowKeys;
	};

	TArray<FPointDataAccessorsMapping> PointDataAccessorsMapping;

	// Set to true if the prepare data succeeded
	bool bDataPrepared = false;
};