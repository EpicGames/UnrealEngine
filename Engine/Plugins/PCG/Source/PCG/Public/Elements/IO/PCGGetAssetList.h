// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "PCGGetAssetList.generated.h"

UENUM(BlueprintType)
enum class EPCGAssetListSource : uint8
{
	Folder UMETA(Tooltip="Retrieves all the assets from the provided folder."),
	Collection UMETA(Tooltip="Retrieves all the assets from the provided collection name.")
};

class FPCGGetAssetListElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const;
};

/** Returns the contents of a collection or a folder in an attribute list. Note that this does not track contents and as such is not a cacheable node. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetAssetListSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetAssetList")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetAssetListElement", "NodeTitle", "Get Asset List"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGGetAssetListElement", "NodeTooltip", "Returns the list of asset, with options (class, bp generated class, etc.) from a source - collection or folder."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
#endif // WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGGetAssetListElement>(); }
	// ~End UPCGSettings interface implementation

public:
	/** Controls whether we will retrieve the files from a specified directory or from a collection. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGAssetListSource AssetListSource = EPCGAssetListSource::Folder;

	/** Directory path to parse for assets. In 'long package name' format, e.g. '/Game/' and so on. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (LongPackageName, EditCondition = "AssetListSource == EPCGAssetListSource::Folder", EditConditionHides, PCG_Overridable))
	FDirectoryPath Directory;

	/** Name of the collection to parse for assets. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "AssetListSource == EPCGAssetListSource::Collection", EditConditionHides, PCG_Overridable))
	FName Collection;

	/** Controls whether the class path will also be exported as part of this node in the 'ClassPath' attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bGetClassPath = false;

	/** Controls whether inexistant or empty collections will log a warning. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	bool bQuiet = false;
};