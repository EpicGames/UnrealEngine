// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGDataAsset.h"
#include "PCGSettings.h"
#include "Elements/PCGLoadObjectsContext.h"
#include "Helpers/PCGTagHelpers.h"

struct FAssetData;

#include "PCGLoadAssetElement.generated.h"

/** Loader/Executor of PCG data assets */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGLoadDataAssetSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	PCG_API UPCGLoadDataAssetSettings();

	//~UObject interface implementation
	PCG_API virtual void PostLoad() override;
#if WITH_EDITOR
	PCG_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject interface implementation

	//~UPCGSettings interface implementation
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PCGLoadDataAsset")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGLoadDataAssetSettings", "NodeTitle", "Load PCG Data Asset"); }
	virtual FText GetNodeTooltipText() const override { return AssetDescription; }
	virtual FLinearColor GetNodeTitleColor() const override { return AssetColor; }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	PCG_API virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
#endif // WITH_EDITOR

	virtual bool HasDynamicPins() const override { return true; }
	virtual bool HasFlippedTitleLines() const override { return true; }
	PCG_API virtual FString GetAdditionalTitleInformation() const override;
	PCG_API virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const;

protected:
	PCG_API virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	PCG_API virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	// TODO tracking if data is stored in external asset

	PCG_API virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Pins; }
	// ~End UPCGSettings interface

public:
	PCG_API virtual void SetFromAsset(const FAssetData& InAsset);
	PCG_API virtual void UpdateFromData(bool bShouldLoadAsset = true);

#if WITH_EDITOR
	PCG_API void OnPreObjectSaved(UObject* Object, FObjectPreSaveContext SaveContext);
#endif // WITH_EDITOR

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable))
	TSoftObjectPtr<UPCGDataAsset> Asset;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Data|Asset Info", meta = (NoResetToDefault))
	TArray<FPCGPinProperties> Pins;

	// Cached from the data when loaded
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Data|Asset Info")
	FString AssetName;

#if WITH_EDITORONLY_DATA
	// Cached from the data when loaded
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Data|Asset Info")
	FText AssetDescription = FText::GetEmpty();

	// Cached from the data when loaded
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Data|Asset Info")
	FLinearColor AssetColor = FLinearColor::White;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bLoadFromInput = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bLoadFromInput", EditConditionHides, PCG_DiscardPropertySelection))
	FPCGAttributePropertyInputSelector AssetReferenceSelector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bLoadFromInput", EditConditionHides))
	FName InputIndexTag = NAME_None;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bLoadFromInput", EditConditionHides))
	FName DataIndexTag = NAME_None;

	// Exposes an attribute set pin to override defaults of the loaded data assets.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bSetDefaultAttributeOverridesFromInput = false;

	// List of Tag:Value default value overrides to apply on the loaded data assets.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (Tooltip="Tag:Value default overrides to be applied to the loaded data.", EditCondition = "!bSetDefaultAttributeOverridesFromInput", EditConditionHides))
	TArray<FString> DefaultAttributeOverrides;

	UPROPERTY(meta = (Tooltip="Overridable-only value to set multiple tags from a single string. If provided, the entries in the Default Value Overrides will be ignored.", PCG_Overridable))
	FString CommaSeparatedDefaultAttributeOverrides;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (Tooltip="Warns if asset is null or couldn't be loaded"))
	bool bWarnIfNoAsset = true;

	/** Controls whether the data output from the loaded asset will be passed to the default pin with tags or on the proper pins. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Settings", meta = (NoResetToDefault))
	bool bTagOutputsBasedOnOutputPins = true;

	/** By default, data table loading is asynchronous, can force it synchronous if needed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

struct FPCGLoadDataAssetContext : public FPCGLoadObjectsFromPathContext 
{
	TArray<TObjectPtr<const UPCGParamData>> DefaultProviders;
	TArray<PCG::Private::FParseTagResult> DefaultValueTags;

	bool bDefaultsMatchInput = false;
	bool bShouldApplyDefaults = false;
};

class FPCGLoadDataAssetElement : public IPCGElementWithCustomContext<FPCGLoadDataAssetContext>
{
public:
	// Loading needs to be done on the main thread and accessing objects outside of PCG might not be thread safe, so taking the safe approach
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	PCG_API virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	PCG_API virtual bool ExecuteInternal(FPCGContext* Context) const override;
};

// Extra class to handle when a PCG Data Asset is changed to update all loaded Load PCG Data Asset nodes
// Will be registered by the PCG Editor.
#if WITH_EDITOR
class FPCGLoadDataAssetChangeTracker
{
public:
	PCG_API FPCGLoadDataAssetChangeTracker();
	PCG_API ~FPCGLoadDataAssetChangeTracker();

	PCG_API static void OnPreObjectSaved(UObject* InObject, FObjectPreSaveContext InSaveContext);
private:
	FDelegateHandle Handle;
};
#endif // WITH_EDITOR
