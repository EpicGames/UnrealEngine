// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGAssetExporter.h"
#include "PCGContext.h"
#include "PCGDataAsset.h"
#include "PCGSettings.h"

struct FAssetData;

#include "PCGSaveAssetElement.generated.h"

/** Default exporter to save data collections, with no capacity for update. */
UCLASS(MinimalAPI, BlueprintType)
class UPCGDataCollectionExporter : public UPCGAssetExporter
{
	GENERATED_BODY()

protected:
	//~Begin UPCGAssetExporter interface
	PCG_API virtual bool ExportAsset(const FString& PackageName, UPCGDataAsset* Asset);
	PCG_API virtual UPackage* UpdateAsset(const FAssetData& PCGAsset);
	//~End UPCGAssetExporter interface

public:
	// @todo_pcg: Should it be a base class function? 
	PCG_API bool ShouldSave(const UPCGDataAsset* Asset);
	
	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category = Data)
	FPCGDataCollection Data;

#if WITH_EDITORONLY_DATA
	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category = Data)
	FString AssetDescription;

	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category = Data)
	FLinearColor AssetColor = FLinearColor::White;
#endif
};

/** Node that will save input data to a PCG data asset */
UCLASS(MinimalAPI, BlueprintType, CLassGroup=(Procedural))
class UPCGSaveDataAssetSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	PCG_API UPCGSaveDataAssetSettings();

	//~UPCGSettings interface implementation
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PCGSaveDataAsset")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSaveDataAssetElement", "NodeTitle", "Save PCG Data Asset"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGSaveDataAssetElement", "NodeTooltip", "Exports the input data to a PCG Data Asset."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
#endif // WITH_EDITOR

	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Pins; }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

protected:
	PCG_API virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data)
	TArray<FPCGPinProperties> Pins;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data)
	TSubclassOf<UPCGDataCollectionExporter> CustomDataCollectionExporterClass;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGAssetExporterParameters Params;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable))
	FString AssetDescription;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable))
	FLinearColor AssetColor = FLinearColor::White;
};

class FPCGSaveDataAssetElement : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	PCG_API virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
};
