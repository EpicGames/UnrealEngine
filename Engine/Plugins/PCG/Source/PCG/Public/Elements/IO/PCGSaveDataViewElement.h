// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Data/DataView/PCGDataView.h"
#include "Data/DataView/PCGDataViewNativeJsonConverters.h"

#include "Serialization/CustomVersion.h"

#include "PCGSaveDataViewElement.generated.h"

/** Serialize a PCG Data via a Data View into a target format, i.e. Json or Binary. */
UCLASS(BlueprintType, MinimalAPI, ClassGroup = (Procedural))
class UPCGSaveDataViewSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGSaveDataViewSettings();

	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PCGSaveDataView")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSaveDataViewElement", "NodeTitle", "Save Data View"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGSaveDataViewElement", "NodeTooltip", "Export the Data View selection to a selected file."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual bool IsEditorOnly() const override { return true; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual TArray<FPCGSettingsOverridableParam> GatherOverridableParams() const override;
#endif // WITH_EDITOR
	virtual void FixingOverridableParamPropertyClass(FPCGSettingsOverridableParam& Param) const override;
	//~ End UPCGSettings interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, NoClear)
	TSubclassOf<UPCGDataViewConverterBase> ConverterClass = UPCGDataViewJsonConverter::StaticClass();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ShowOnlyInnerProperties, StructTypeConst))
	FInstancedStruct ConverterParameters;

	/** The file path to save the data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	FFilePath FilePath;

#if WITH_EDITORONLY_DATA
	/** Open a file dialogue on empty path. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	bool bOpenDialogueOnEmptyPath = false;
#endif // WITH_EDITORONLY_DATA

	/** Overwrite the existing file, if it exists. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	bool bOverwriteExistingFile = false;
};

class FPCGSaveDataViewExecutionContext : public FPCGContext
{
protected:
	virtual FStructView GetExternalStructContainerForOverridableParam(const FPCGSettingsOverridableParam& InParam) override;
};

class FPCGSaveDataViewElement final : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	// @todo_pcg: Checksum the file?
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual FPCGContext* CreateContext() override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
