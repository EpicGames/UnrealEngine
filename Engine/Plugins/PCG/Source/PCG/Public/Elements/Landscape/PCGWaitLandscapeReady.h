// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"	

#include "PCGWaitLandscapeReady.generated.h"

class ALandscape;

/**
* Used to wait until the landscape is done updating anything.
*/
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGWaitLandscapeReadySettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("WaitUntilLandscapeReady")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGWaitLandscapeReadySettings", "NodeTitle", "Wait Until Landscape Is Ready"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGWaitLandscapeReadySettings", "NodeTooltip", "Waits until landscape is ready, then passes data downstream."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

struct FPCGWaitLandscapeReadyElementContext : public FPCGContext
{
	bool bLandscapeQueryDone = false;
	bool bLandscapeReady = false;
	TArray<TWeakObjectPtr<ALandscape>> Landscapes;
};

class FPCGWaitLandscapeReadyElement : public IPCGElementWithCustomContext<FPCGWaitLandscapeReadyElementContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
