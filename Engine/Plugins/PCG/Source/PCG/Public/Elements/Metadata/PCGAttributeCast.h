// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGElement.h"
#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataAttributeTraits.h"

#include "PCGAttributeCast.generated.h"

/**
* Cast an attribute to another type. Support broadcastable cast (like double -> FVector) and constructible cast (like double -> float)
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAttributeCastSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGMetadataTypes OutputType = EPCGMetadataTypes::Float;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyOutputSelector OutputTarget;

	//~Begin UPCGSettings interface
public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName("AttributeCast"); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGAttributeCastElement", "NodeTitle", "Attribute Cast"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
#endif
	virtual bool HasDynamicPins() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;
	virtual FName AdditionalTaskName() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo) override;
	virtual FPCGElementPtr CreateElement() const override;

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const;
	//~End UPCGSettings interface
};

class FPCGAttributeCastElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
