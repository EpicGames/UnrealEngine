// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/Metadata/PCGMetadataOpElementBase.h"
#include "Elements/PCGTimeSlicedElementBase.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Curves/CurveFloat.h"
#include "Templates/UniquePtr.h"

#include "PCGAttributeRemap.generated.h"

UENUM()
enum class EPCGAttributeRemapMode
{
	Ranges,
	Curve
};

/**
* Remap attribute values from one range to another.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAttributeRemapSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	UPCGAttributeRemapSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool GroupPreconfiguredSettings() const override { return false; }
	virtual bool HasFlippedTitleLines() const override { return false; }
#endif

	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	//~Begin UPCGMetadataSettingsBase interface
	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;
	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	//~End UPCGMetadataSettingsBase interface

	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override { return EPCGMetadataTypes::Double; }
	//~End IPCGSettingsDefaultValueProvider interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGAttributeRemapMode Mode = EPCGAttributeRemapMode::Ranges;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	/** If InRangeMin = InRangeMax, then that attribute value is mapped to the average of OutRangeMin and OutRangeMax */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "Mode == EPCGAttributeRemapMode::Ranges", EditConditionHides))
	double InRangeMin = 0.f;

	/** If InRangeMin = InRangeMax, then that attribute value is mapped to the average of OutRangeMin and OutRangeMax */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "Mode == EPCGAttributeRemapMode::Ranges", EditConditionHides))
	double InRangeMax = 1.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "Mode == EPCGAttributeRemapMode::Ranges", EditConditionHides))
	double OutRangeMin = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "Mode == EPCGAttributeRemapMode::Ranges", EditConditionHides))
	double OutRangeMax = 1.f;

	/** If checked, outside values will be clamped between 0 and 1. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "Mode == EPCGAttributeRemapMode::Ranges", EditConditionHides))
	bool bClampToUnitRange = false;

	/** Attribute values outside of the input range will be unaffected by the remapping */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "Mode == EPCGAttributeRemapMode::Ranges", EditConditionHides))
	bool bIgnoreValuesOutsideInputRange = false;

	/** Allow remapping when Min is larger than Max, e.g. from [0.0, 1.0] -> [1.0, 0.0]. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "Mode == EPCGAttributeRemapMode::Ranges", EditConditionHides))
	bool bAllowInverseRange = false; // Note that this is no longer the default value for new nodes, it is now 'true'

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverridableChildProperties="ExternalCurve", EditCondition = "Mode == EPCGAttributeRemapMode::Curve", EditConditionHides))
	FRuntimeFloatCurve Curve;
};

class FPCGAttributeRemapElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};

