// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGDataAttributesAndTags.generated.h"

UCLASS(Abstract, MinimalAPI, ClassGroup = (Procedural))
class UPCGDataAttributesAndTagsSettingsBase : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif
	
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface

public:
	// Map between input attribute/tags to output attribute/tags. Can use @Source to keep the name. If empty, copies everything.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TMap<FString, FPCGAttributePropertyOutputSelector> AttributesTagsMapping;

	// After the operation, can delete the input attributes/tags
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (PCG_Overridable))
	bool bDeleteInputsAfterOperation = false;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDataAttributesToTagsSettings : public UPCGDataAttributesAndTagsSettingsBase
{
	GENERATED_BODY()
	
public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif // WITH_EDITOR
	
	virtual FPCGElementPtr CreateElement() const override;

	// If the type is not parseable (like a Quaternion for example), it can be discarded or just added as name.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Settings", meta = (PCG_Overridable))
	bool bDiscardNonParseableAttributeTypes = true;

	// Do not output the value of the attribute in the tag (e.g. MyAttr:1), but only the attribute name (e.g. MyAttr)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Settings", meta = (PCG_Overridable))
	bool bDiscardAttributeValue = false;

	// Controls whether string attribute will be broken down as their contents only, and supports CSV too.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Settings", meta = (PCG_Overridable))
	bool bTreatStringAttributesAsTags = false;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGTagsToDataAttributesSettings : public UPCGDataAttributesAndTagsSettingsBase
{
	GENERATED_BODY()
	
public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif // WITH_EDITOR
	
	virtual FPCGElementPtr CreateElement() const override;
};

class FPCGDataAttributesToTagsElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};

class FPCGTagsToDataAttributesElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};

