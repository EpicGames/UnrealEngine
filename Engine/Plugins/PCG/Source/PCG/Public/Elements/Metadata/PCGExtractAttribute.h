// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/SubclassOf.h"

#include "PCGExtractAttribute.generated.h"

struct FPCGAttributePropertyInputSelector;
struct FPCGAttributePropertyOutputSelector;
struct FPCGContext;
struct FPCGTaggedData;
class UPCGData;

namespace PCGExtractAttribute
{
	struct FExtractAttributeParams
	{
		/** Context that contains the input data. Cannot be null when passed to ExtractAttribute. */
		FPCGContext* Context = nullptr;
		
		/** Selector for the attribute to extract. Cannot be null when passed to ExtractAttribute. */
		const FPCGAttributePropertyInputSelector* InputSource = nullptr;
		
		/** Index of the element to extract the attribute from. */
		const int32 Index = 0;
		
		/** Selector for the output attribute to write into. Cannot be null when passed to ExtractAttribute. */
		const FPCGAttributePropertyOutputSelector* OutputAttributeName = nullptr;
		
		/** Optional class to check the input data. */
		TOptional<TSubclassOf<UPCGData>> OptionalClassRequirement = {};
		
		/** Input pin label to read the data from. */
		FName InputLabel = PCGPinConstants::DefaultInputLabel;
		
		/** Output pin label to output the result attribute set to. */
		FName OutputLabel = PCGPinConstants::DefaultOutputLabel;
		
		/** Extra function that can be called after the extraction succeeded with the input data. */
		TFunction<void(const FPCGTaggedData&)> OnSuccessExtractionCallback;

		/** 
		* For deprecation, old behavior was discarding the output domain and was always writing to the Elements domain.
		* Since ExtractAttribute function was not exported, external users could not use it, so default to false in the general case.
		*/
		bool bForceOutputAttributeToBeInElementsDomain = false;
	};

	/**
	 * Go through all inputs from the context input data, and try to extract an attribute at the provided index into a new Attribute Set.
	 * Input are read from the context and output is written in the context too.
	 */
	void ExtractAttribute(const FExtractAttributeParams& Params);
}

/**
 * Extract an attribute at a given index into a new attribute set.
 * Support any domain. Index needs to be in range of valid indexes for the given domain.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGExtractAttributeSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGExtractAttributeSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	int32 Index = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyOutputSelector OutputAttributeName;

	/** For deprecation, old node was discarding the output domain and was always writing to the Elements domain. This will be false for new nodes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Deprecation", meta = (EditCondition="bForceOutputAttributeToBeInElementsDomain", EditConditionHides))
	bool bForceOutputAttributeToBeInElementsDomain = true;
};

class FPCGExtractAttributeElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};

