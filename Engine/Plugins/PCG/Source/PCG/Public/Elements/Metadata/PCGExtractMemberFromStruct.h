// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/SubclassOf.h"

#include "PCGExtractMemberFromStruct.generated.h"

struct FPCGAttributePropertyInputSelector;
struct FPCGAttributePropertyOutputSelector;
struct FPCGContext;
struct FPCGTaggedData;
class UPCGData;

namespace PCGExtractMemberFromStruct
{
	struct FExtractMemberFromStructParams
	{
		/** Context that contains the input data. Cannot be null when passed to ExtractMemberFromStruct. */
		FPCGContext* Context = nullptr;

		/** Selector for the attribute to extract. Must be a path including the struct member name(s) in extra names. Cannot be null when passed to ExtractMemberFromStruct. */
		const FPCGAttributePropertyInputSelector* InputSource = nullptr;
		
		/** If we want to extract ALL members from the struct. OutputAttributeName will be ignored. */
		bool bExtractAll = false;

		/** Selector for the output attribute to write into. Cannot be null when passed to ExtractMemberFromStruct. */
		const FPCGAttributePropertyOutputSelector* OutputAttributeName = nullptr;
		
		/** After the extraction, delete the original attribute. */
		bool bDeleteSourceAttribute = true;

		/** Optional class to check the input data. */
		TOptional<TSubclassOf<UPCGData>> OptionalClassRequirement = {};

		/** Input pin label to read the data from. */
		FName InputLabel = PCGPinConstants::DefaultInputLabel;

		/** Output pin label to output the result attribute set to. */
		FName OutputLabel = PCGPinConstants::DefaultOutputLabel;

		/** Extra function that can be called after the extraction succeeded with the input data. */
		TFunction<void(const FPCGTaggedData&)> OnSuccessExtractionCallback;
	};

	/**
	 * Go through all inputs from the context input data, and try to extract a struct member into a new attribute on a duplicated input data.
	 * Input are read from the context and output is written in the context too.
	 */
	void ExtractMemberFromStruct(const FExtractMemberFromStructParams& Params);
}

/**
 * Extract a member (or all members) from a struct attribute into a new attribute on the input data.
 * Support any domain.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGExtractMemberFromStructSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
#endif
	virtual bool HasDynamicPins() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** The source to extract the attribute from. Must be a path including the struct member to extract. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;
	
	/** To extract all members from the struct. If the property path doesn't point to a structure, this will be ignored. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bExtractAll = false;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection, EditCondition = "!bExtractAll", EditConditionHides))
	FPCGAttributePropertyOutputSelector OutputAttributeName;

	/** After the extraction, delete the original attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bDeleteSourceAttribute = true;
};

class FPCGExtractMemberFromStructElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
