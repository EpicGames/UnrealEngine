// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGMetadataOpElementBase.h"
#include "PCGSettings.h"

#include "PCGHashAttribute.generated.h"

/** Hash an attribute into a 32-bit unsigned integer, expressed as a 64-bit signed integer. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGHashAttributeSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	//~Begin UPCGMetadataSettingsBase
	PCG_API virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;

	virtual FName GetInputPinLabel(uint32 Index) const override { return PCGPinConstants::DefaultInputLabel; }
	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override { return true; }
	virtual uint32 GetOperandNum() const override { return 1; }
	virtual uint16 GetOutputType(uint16 InputTypeId) const override { return static_cast<uint16>(EPCGMetadataTypes::Integer32); }
	//~End UPCGMetadataSettingsBase

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName("HashAttribute"); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGHashAttributeElement", "NodeTitle", "Hash Attribute"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
#endif
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override { return EPCGMetadataTypes::String; }
	//~End IPCGSettingsDefaultValueProvider interface

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, DisplayName="Input Source"))
	FPCGAttributePropertyInputSelector InputSource1;
};

class FPCGHashAttributeElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& InOperationData) const override;
};
