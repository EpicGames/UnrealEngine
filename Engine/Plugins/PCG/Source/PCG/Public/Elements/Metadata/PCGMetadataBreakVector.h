// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#include "PCGMetadataBreakVector.generated.h"

namespace PCGMetadataBreakVectorConstants
{
	const FName XLabel = TEXT("X");
	const FName YLabel = TEXT("Y");
	const FName ZLabel = TEXT("Z");
	const FName WLabel = TEXT("W");
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataBreakVectorSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif // WITH_EDITOR
	// ~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
#endif


	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;

	virtual FName GetOutputPinLabel(uint32 Index) const override;
	virtual uint32 GetResultNum() const override;

	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	virtual uint16 GetOutputType(uint16 InputTypeId) const override;
	virtual FName GetOutputAttributeName(FName BaseName, uint32 Index) const override;

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin IPCGSettingsDefaultValueProvider interface
#if WITH_EDITOR
	virtual FString GetPinInitialDefaultValueString(FName PinLabel) const override { return PCG::Private::MetadataTraits<FVector>::ZeroValueString(); }
#endif // WITH_EDITOR
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override { return EPCGMetadataTypes::Vector; }
	//~End IPCGSettingsDefaultValueProvider interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName InputAttributeName_DEPRECATED = NAME_None;
#endif
};

class FPCGMetadataBreakVectorElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};
