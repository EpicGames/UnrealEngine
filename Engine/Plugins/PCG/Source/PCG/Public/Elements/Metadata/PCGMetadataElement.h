// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGCopyAttributes.h"

#include "PCGMetadataElement.generated.h"

/** Deprecated, use UPCGCopyAttributesSettings */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UE_DEPRECATED(5.5, "Use UPCGCopyAttributeSettings") UPCGMetadataOperationSettings : public UPCGCopyAttributesSettings
{
	GENERATED_BODY()

public:
	UPCGMetadataOperationSettings()
		: UPCGCopyAttributesSettings()
	{
#if WITH_EDITOR
		bExposeToLibrary = false;
#endif // WITH_EDITOR
	}
};