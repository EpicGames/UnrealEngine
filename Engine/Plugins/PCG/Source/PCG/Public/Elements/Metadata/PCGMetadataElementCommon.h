// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Algo/AnyOf.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h" // IWYU pragma: keep
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"
#include "Utils/PCGPreconfiguration.h"

#include "Misc/ScopeExit.h" // For ON_SCOPE_EXIT
#include "Templates/MemoryOps.h" // For DestructItem

class IPCGAttributeAccessor;
template <typename T> class FPCGMetadataAttribute;

struct FPCGTaggedData;

namespace PCGMetadataElementCommon
{
	UE_DEPRECATED(5.5, "Call/Implement version with FPCGContext parameter")
	PCG_API void DuplicateTaggedData(const FPCGTaggedData& InTaggedData, FPCGTaggedData& OutTaggedData, UPCGMetadata*& OutMetadata);

	PCG_API void DuplicateTaggedData(FPCGContext* InContext, const FPCGTaggedData& InTaggedData, FPCGTaggedData& OutTaggedData, UPCGMetadata*& OutMetadata);

	/** Copies the entry to value key relationship stored in the given Metadata, including its parents */
	PCG_API void CopyEntryToValueKeyMap(const UPCGMetadata* MetadataToCopy, const FPCGMetadataAttributeBase* AttributeToCopy, FPCGMetadataAttributeBase* OutAttribute);

	/** Creates a new attribute, or clears the attribute if it already exists and is a 'T' type. If default value not provided, will take the zero value for that type. */
	template<typename T>
	FPCGMetadataAttribute<T>* ClearOrCreateAttribute(FPCGMetadataDomain* Metadata, const FName& DestinationAttribute, T DefaultValue = PCG::Private::MetadataTraits<T>::ZeroValue())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGMetadataElementCommon::ClearOrCreateAttribute);

		if (!Metadata)
		{
			return nullptr;
		}

		if (Metadata->HasAttribute(DestinationAttribute))
		{
			UE_LOGF(LogPCG, Verbose, "Attribute %ls already exists and will be overwritten", *DestinationAttribute.ToString());
			Metadata->DeleteAttribute(DestinationAttribute);
		}

		Metadata->CreateAttribute<T>(DestinationAttribute, DefaultValue, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);

		return static_cast<FPCGMetadataAttribute<T>*>(Metadata->GetMutableAttribute(DestinationAttribute));
	}
	
	/** Creates a new attribute, or clears the attribute if it already exists and is a 'T' type. If default value not provided, will take the zero value for that type. */
	template<typename T>
	FPCGMetadataAttribute<T>* ClearOrCreateAttribute(UPCGMetadata* Metadata, const FName& DestinationAttribute, T DefaultValue = PCG::Private::MetadataTraits<T>::ZeroValue())
	{
		if (!Metadata)
		{
			return nullptr;
		}
		
		return ClearOrCreateAttribute<T>(Metadata->GetDefaultMetadataDomain(), DestinationAttribute, std::move(DefaultValue));
	}

	/** Creates a new attribute, or clears the attribute if it already exists and is a 'T' type. If default value not provided, will take the zero value for that type. */
	template<typename T>
	FPCGMetadataAttribute<T>* ClearOrCreateAttribute(UPCGMetadata* Metadata, const FPCGAttributePropertySelector& DestinationAttribute, T DefaultValue = PCG::Private::MetadataTraits<T>::ZeroValue(), FPCGContext* InOptionalContext = nullptr)
	{
		if (!Metadata)
		{
			return nullptr;
		}

		FPCGMetadataDomain* MetadataDomain = Metadata->GetMetadataDomainFromSelector(DestinationAttribute);
		if (!MetadataDomain)
		{
			PCGLog::Metadata::LogInvalidMetadataDomain(DestinationAttribute, InOptionalContext);
			return nullptr;
		}
		
		return ClearOrCreateAttribute<T>(MetadataDomain, DestinationAttribute.GetAttributeName(), std::move(DefaultValue));
	}

	constexpr int32 DefaultChunkSize = 256;

	/**
	* Iterate over the full range of the keys (if Count is negative, otherwise, as many times as Count), calling the callback with values range get from the accessor.
	* ChunkSize influence the max number of values to get in one go with GetRange.
	* Callback should have a signature: void(const TArrayView<T>& View, int32 Start, int32 Range) or bool(const TArrayView<T>& View, int32 Start, int32 Range) which returns true if we continue, false otherwise.
	* Return false if it process nothing.
	*/
	template <typename T, typename Func>
	bool ApplyOnAccessorRange(const IPCGAttributeAccessorKeys& Keys, const IPCGAttributeAccessor& Accessor, Func&& Callback, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType, const int32 ChunkSize = DefaultChunkSize, const int32 Count = -1)
	{
		const int32 NumberOfEntries = Count < 0 ? Keys.GetNum() : Count;

		if (NumberOfEntries == 0)
		{
			return false;
		}

		TArray<T, TInlineAllocator<DefaultChunkSize>> TempValues;
		if constexpr (std::is_trivially_copyable_v<T>)
		{
			TempValues.SetNumUninitialized(ChunkSize);
		}
		else
		{
			TempValues.SetNum(ChunkSize);
		}

		using ReturnType = decltype(Callback(TArrayView<T>{}, 0, 0));

		const int32 NumberOfIterations = (NumberOfEntries + ChunkSize - 1) / ChunkSize;

		for (int32 i = 0; i < NumberOfIterations; ++i)
		{
			const int32 StartIndex = i * ChunkSize;
			const int32 Range = FMath::Min(NumberOfEntries - StartIndex, ChunkSize);
			TArrayView<T> View(TempValues.GetData(), Range);

			if (!Accessor.GetRange(View, StartIndex, Keys, Flags))
			{
				return false;
			}

			if constexpr (std::is_same_v<ReturnType, bool>)
			{
				if (!Callback(View, StartIndex, Range))
				{
					break;
				}
			}
			else
			{
				Callback(View, StartIndex, Range);
			}
		}

		return true;
	}

	/**
	* Iterate over the full range of the keys (if Count is negative, otherwise, as many times as Count), calling the callback with values get from the accessor.
	* ChunkSize influence the max number of values to get in one go with GetRange.
	* Callback should have a signature: void(const T& Value, int32 Index) / void(T Value, int32 Index). Can also return a bool to know if we need to continue or not.
	* Return false if it process nothing.
	*/
	template <typename T, typename Func>
	bool ApplyOnAccessor(const IPCGAttributeAccessorKeys& Keys, const IPCGAttributeAccessor& Accessor, Func&& InCallback, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType, const int32 ChunkSize = DefaultChunkSize, const int32 Count = -1)
	{
		auto RangeCallback = [Callback = std::forward<Func>(InCallback)](const TArrayView<T>& View, int32 Start, int32 Range)
		{
			using ReturnType = decltype(Callback(T{}, 0));
			
			for (int32 j = 0; j < Range; ++j)
			{
				// We can move the value since we don't use it afterward. If the callback is taking a const ref, it's fine.
				if constexpr (std::is_same_v<ReturnType, bool>)
				{
					if (!Callback(std::move(View[j]), Start + j))
					{
						return false;
					}
				}
				else
				{
					Callback(std::move(View[j]), Start + j);
				}
			}

			if constexpr (std::is_same_v<ReturnType, bool>)
			{
				return true;
			}
		};

		return ApplyOnAccessorRange<T>(Keys, Accessor, std::move(RangeCallback), Flags, ChunkSize, Count);
	}

	/**
	* Iterate over the full range of the keys (if Count is negative, otherwise, as many times as Count), calling the callback with values ranges get from the multiple accessors.
	* There can be 1 Keys for all accessors or 1 per accessor, but in that case if Count is set to -1, it will use the first key in the list with more than 1 element for the number of elements to process.
	* ChunkSize influence the max number of values to get in one go with GetRange.
	* Callback should have a signature: void(const TArrayView<T>&... Args, int32 Start, int32 Range). Can also return a bool to know if we need to continue.
	* Return false if it processes nothing.
	*/
	template <typename... T, typename Func = TFunction<void(const TArrayView<T>&..., int32 Start, int32 Range)>>
	bool ApplyOnMultiAccessorsRange(const TConstArrayView<IPCGAttributeAccessorKeys const*> MultiKeys, const TConstArrayView<IPCGAttributeAccessor const*> Accessors, Func&& InCallback, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType, const int32 ChunkSize = DefaultChunkSize, const int32 Count = -1)
	{
		// We use C++20 functionality here, so make sure we are at this version at least
		static_assert(__cplusplus >= 202002L);

		// We support 1 key for all accessors or 1 key per accessor.
		if (MultiKeys.IsEmpty() || Accessors.IsEmpty() || (MultiKeys.Num() != Accessors.Num() && MultiKeys.Num() != 1) || !ensure(Algo::AllOf(MultiKeys))|| !ensure(Algo::AllOf(Accessors)))
		{
			return false;
		}

		// Also make sure that the keys are matching in num (or just have a single entry)
		int32 NumElements = 1;
		for (const IPCGAttributeAccessorKeys* Keys : MultiKeys)
		{
			const int32 KeysNum = Keys->GetNum();
			if (NumElements == 1)
			{
				NumElements = KeysNum;
			}

			if (NumElements != KeysNum && KeysNum != 1)
			{
				PCGLog::LogErrorOnGraph(NSLOCTEXT("PCGApplyOnMultiAccessorsRange", "InvalidKeyCardinality", "Try to use ApplyOnMultiAccessorsRange with keys nums that doesn't match."));
				return false;
			}
		}

		const int32 NumberOfEntries = Count < 0 ? NumElements : Count;

		if (NumElements == 0 || NumberOfEntries == 0)
		{
			return false;
		}

		constexpr int32 NumArgs = static_cast<int32>(sizeof...(T));

		// Buffers for our get range cannot be stored in a TArray, since we have different T types. So buffers will be raw buffers of bytes.
		// Since every type can be different, we pre-allocate chunk size times the size of FVector for all, which should cover most usecases.
		// Making sure we can accommodate our big types that have different alignment like FTransform or FVector4
		constexpr int32 MaxAlignment = FMath::Max(alignof(T)...);
		constexpr int32 AlignedBlockSize = MaxAlignment;
		constexpr int32 MaxInlineMem = DefaultChunkSize * sizeof(FVector) / AlignedBlockSize;
		static_assert(MaxInlineMem > 0, "MaxInlineMem must be greater than 0");

		using RawBuffer = TArray<TAlignedBytes<AlignedBlockSize, MaxAlignment>, TInlineAllocator<MaxInlineMem>>;

		RawBuffer RawBuffers[NumArgs]{};

		// First fold expression: Initialize all the buffers. We unfold T to initialize the buffer to 
		// ChunkSize element of type T. If T is not POD, we need to construct the object, so we do a placement new.
		int i = 0;
		([&]
		{
			const int32 AllocationSize = FMath::DivideAndRoundUp<int32>(sizeof(T) * ChunkSize, AlignedBlockSize);
			RawBuffers[i].SetNumUninitialized(AllocationSize);
			if constexpr (!std::is_trivially_copyable_v<T>)
			{
				for (int32 j = 0; j < ChunkSize; ++j)
				{
					new(reinterpret_cast<T*>(RawBuffers[i].GetData()) + j) T;
				}
			}
			++i;
		}(), ...);

		ON_SCOPE_EXIT
		{
			// Second fold expression, to be executed when we exit the function, so that if we constructed our objects while allocating the buffer
			// we need to explicitly call their destructor. Since we are in a fold expression, we can't use `.~T` as ~T is not recognized as a function name,
			// so we use DestructItem.
			i = 0;
			([&]
			{
				if constexpr (!std::is_trivially_copyable_v<T>)
				{
					for (int32 j = 0; j < ChunkSize; ++j)
					{
						DestructItem(reinterpret_cast<T*>(RawBuffers[i].GetData()) + j);
					}
				}
				++i;
			}(), ...);
		};

		const int32 NumberOfIterations = (NumberOfEntries + ChunkSize - 1) / ChunkSize;

		for (int32 j = 0; j < NumberOfIterations; ++j)
		{
			const int32 StartIndex = j * ChunkSize;
			const int32 Range = FMath::Min(NumberOfEntries - StartIndex, ChunkSize);

			// Third fold expression, that will unroll T to get all ranges.
			// Since we manage our memory ourselves, it is safe to do a reinterpret_cast here.
			bool bSuccess = true;
			i = 0;
			(
			[&]
			{
				TArrayView<T> View(reinterpret_cast<T*>(RawBuffers[i].GetData()), Range);
				bSuccess &= Accessors[i]->GetRange<T>(View, StartIndex, *MultiKeys[i % MultiKeys.Num()], Flags);
				++i;
			}(), ...);

			if (!bSuccess)
			{
				return false;
			}

			// Finally, we need to pass the views to the callback. We'll have to have an indirection by first unpacking T to create the views and
			// then unpack again the views to the callback.
			// We can't do that with a fold expression, so we need to use a Tuple. We create a Tuple of views, and use a lambda unpacker to construct our views
			// then we unpack it again when calling our callback. ApplyBefore will first unpack the views and then pass the rest of the parameters (here StartIndex and Range)
			using TupleType = TTuple<TArrayView<T>...>;
			TupleType TempTuple = [&]<std::size_t ...I>(std::index_sequence<I...>)
			{
				return TupleType(TArrayView<T>(reinterpret_cast<T*>(RawBuffers[I].GetData()), Range)...);
			}(std::make_index_sequence<NumArgs>{});

			using ReturnType = decltype(TempTuple.ApplyBefore(InCallback, StartIndex, Range));

			if constexpr (std::is_same_v<ReturnType, bool>)
			{
				if (!TempTuple.ApplyBefore(InCallback, StartIndex, Range))
				{
					break;
				}
			}
			else
			{
				TempTuple.ApplyBefore(InCallback, StartIndex, Range);
			}
		}

		return true;
	}

	/**
	* Iterate over the full range of the keys (if Count is negative, otherwise, as many times as Count), calling the callback with values ranges get from the multiple accessors.
	* ChunkSize influence the max number of values to get in one go with GetRange.
	* Callback should have a signature: void(const TArrayView<T>&... Args, int32 Start, int32 Range)
	* Return false if it processes nothing.
	*/
	template <typename... T, typename Func = TFunction<void(const TArrayView<T>&..., int32 Start, int32 Range)>>
	bool ApplyOnMultiAccessorsRange(const IPCGAttributeAccessorKeys& Keys, const TConstArrayView<IPCGAttributeAccessor const*> Accessors, Func&& InCallback, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType, const int32 ChunkSize = DefaultChunkSize, const int32 Count = -1)
	{
		const IPCGAttributeAccessorKeys* KeysPtr = &Keys;
		return ApplyOnMultiAccessorsRange<T...>(TConstArrayView<IPCGAttributeAccessorKeys const*>(&KeysPtr, 1), Accessors, std::forward<Func>(InCallback), Flags, ChunkSize, Count);
	}

	/**
	* Iterate over the full range of the keys (if Count is negative, otherwise, as many times as Count), calling the callback with values get from the multiple accessors.
	* There can be 1 keys for all accessors or 1 per accessor, but in that case if Count is set to -1, it will use the first key for the number of elements to process.
	* ChunkSize influence the max number of values to get in one go with GetRange.
	* Callback should have a signature: void(const T&... Values, int32 Index) / void(T... Values, int32 Index). Can return a bool to know if we need to continue.
	* Return false if it process nothing.
	*/
	template <typename ...T, typename Func = TFunction<void(const T&..., int32 Index)>>
	bool ApplyOnMultiAccessors(const TConstArrayView<IPCGAttributeAccessorKeys const*> MultiKeys, const TConstArrayView<IPCGAttributeAccessor const*> Accessors, Func&& InCallback, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType, const int32 ChunkSize = DefaultChunkSize, const int32 Count = -1)
	{
		auto RangeCallback = [Callback = std::forward<Func>(InCallback)](const TArrayView<T>& ...Views, int32 Start, int32 Range)
		{
			using ReturnType = decltype(Callback(Views[0]..., 0));

			for (int32 j = 0; j < Range; ++j)
			{
				// We can move the value since we don't use it afterward. If the callback is taking a const ref, it's fine.
				if constexpr (std::is_same_v<ReturnType, bool>)
				{
					if (!Callback(std::move(Views[j])..., Start + j))
					{
						return false;
					}
				}
				else
				{
					Callback(std::move(Views[j])..., Start + j);
				}
			}

			if constexpr (std::is_same_v<ReturnType, bool>)
			{
				return true;
			}
		};

		return ApplyOnMultiAccessorsRange<T...>(MultiKeys, Accessors, std::move(RangeCallback), Flags, ChunkSize, Count);
	}

	/**
	* Iterate over the full range of the keys (if Count is negative, otherwise, as many times as Count), calling the callback with values get from the multiple accessors.
	* ChunkSize influence the max number of values to get in one go with GetRange.
	* Callback should have a signature: void(const T&... Values, int32 Index) / void(T... Values, int32 Index). Can return a bool to know if we need to continue.
	* Return false if it process nothing.
	*/
	template <typename ...T, typename Func = TFunction<void(const T&..., int32 Index)>>
	bool ApplyOnMultiAccessors(const IPCGAttributeAccessorKeys& Keys, const TConstArrayView<IPCGAttributeAccessor const*> Accessors, Func&& InCallback, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType, const int32 ChunkSize = DefaultChunkSize, const int32 Count = -1)
	{
		const IPCGAttributeAccessorKeys* KeysPtr = &Keys;
		return ApplyOnMultiAccessors<T...>(TConstArrayView<IPCGAttributeAccessorKeys const*>(&KeysPtr, 1), Accessors, std::forward<Func>(InCallback), Flags, ChunkSize, Count);
	}

	/**
	* Read all the values from in accessor and write to out accessor. Iterate as many times than the out accessor keys
	*/
	struct FCopyFromAccessorToAccessorParams
	{
		enum EIterationCount
		{
			In, // Iterate as many times than the in accessor keys
			Out, // Iterate as many times than the out accessor keys
			Min, // Iterate as many times than the min of accessor keys
			Max, // Iterate as many times than the max of accessor keys
		};

		EIterationCount IterationCount;
		const IPCGAttributeAccessor* InAccessor = nullptr;
		const IPCGAttributeAccessorKeys* InKeys = nullptr;
		IPCGAttributeAccessor* OutAccessor = nullptr;
		IPCGAttributeAccessorKeys* OutKeys = nullptr;
		EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType;
		int32 ChunkSize = DefaultChunkSize;
	};

	PCG_API bool CopyFromAccessorToAccessor(FCopyFromAccessorToAccessorParams& Params);


	template <typename EnumOperation>
	UE_DEPRECATED(5.6, "Please use FPCGPreConfiguredSettingsInfo::PopulateFromEnum instead.")
	TArray<FPCGPreConfiguredSettingsInfo> FillPreconfiguredSettingsInfoFromEnum(const TSet<EnumOperation>& InValuesToSkip = {}, const FText& InOptionalPrefix = FText())
	{
		FTextFormat Format = FTextFormat(FText::Format(INVTEXT("{0}{1}"), InOptionalPrefix, INVTEXT("{0}")));
		return FPCGPreConfiguredSettingsInfo::PopulateFromEnum<EnumOperation>(InValuesToSkip, std::move(Format));
	}
}
