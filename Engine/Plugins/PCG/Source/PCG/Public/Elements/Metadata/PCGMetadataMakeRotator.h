// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#include "PCGMetadataMakeRotator.generated.h"

namespace PCGMetadataMakeRotatorConstants
{
	const FName XLabel = TEXT("X");
	const FName YLabel = TEXT("Y");
	const FName ZLabel = TEXT("Z");
	const FName ForwardLabel = TEXT("Forward");
	const FName RightLabel = TEXT("Right");
	const FName UpLabel = TEXT("Up");
	const FName YawLabel = TEXT("Yaw");
	const FName PitchLabel = TEXT("Pitch");
	const FName RollLabel = TEXT("Roll");
}

UENUM()
enum class EPCGMetadataMakeRotatorOp : uint8
{
	MakeRotFromX,
	MakeRotFromY,
	MakeRotFromZ,
	MakeRotFromXY,
	MakeRotFromYX,
	MakeRotFromXZ,
	MakeRotFromZX,
	MakeRotFromYZ,
	MakeRotFromZY,
	MakeRotFromAxes,
	MakeRotFromAngles UMETA(DisplayName = "Make Rot From Euler Angles")
};

/* Create a Rotator from 1, 2 or 3 axis. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataMakeRotatorSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo) override;
	//~End UPCGSettings interface

	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;

	virtual FName GetInputPinLabel(uint32 Index) const override;
	virtual uint32 GetOperandNum() const override;

	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	virtual uint16 GetOutputType(uint16 InputTypeId) const override;

protected:
	//~Begin UPCGSettings interface
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin IPCGSettingsDefaultValueProvider interface
#if WITH_EDITOR
	virtual FString GetPinInitialDefaultValueString(FName PinLabel) const override;
#endif // WITH_EDITOR
	virtual bool CreateInitialDefaultValueAttribute(FName PinLabel, UPCGMetadata* OutMetadata) const override;
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	//~End IPCGSettingsDefaultValueProvider interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (EditCondition = "Operation != EPCGMetadataMakeRotatorOp::MakeRotFromX && Operation != EPCGMetadataMakeRotatorOp::MakeRotFromY && Operation != EPCGMetadataMakeRotatorOp::MakeRotFromZ", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource2;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (EditCondition = "Operation == EPCGMetadataMakeRotatorOp::MakeRotFromAxes || Operation == EPCGMetadataMakeRotatorOp::MakeRotFromAngles", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource3;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGMetadataMakeRotatorOp Operation = EPCGMetadataMakeRotatorOp::MakeRotFromAxes;
};

class FPCGMetadataMakeRotatorElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};
