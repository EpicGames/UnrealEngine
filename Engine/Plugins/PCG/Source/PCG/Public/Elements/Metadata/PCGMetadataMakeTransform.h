// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#include "PCGMetadataMakeTransform.generated.h"

namespace PCGMetadataTransformConstants
{
	const FName Translation = TEXT("Translation");
	const FName Rotation = TEXT("Rotation");
	const FName Scale = TEXT("Scale");
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataMakeTransformSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface

#if WITH_EDITOR
	//~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif
	//~End UPCGSettings interface

	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;

	virtual FName GetInputPinLabel(uint32 Index) const override;
	virtual uint32 GetOperandNum() const override;

	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	virtual uint16 GetOutputType(uint16 InputTypeId) const override;

protected:
	//~Begin UPCGSettings interface
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin IPCGSettingsDefaultValueProvider interface
#if WITH_EDITOR
	virtual FString GetPinInitialDefaultValueString(FName PinLabel) const override;
#endif // WITH_EDITOR
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	virtual bool CreateInitialDefaultValueAttribute(FName PinLabel, UPCGMetadata* OutMetadata) const override;
	//~End IPCGSettingsDefaultValueProvider interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource2;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource3;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName Input1AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input2AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input3AttributeName_DEPRECATED = NAME_None;
#endif
};

class FPCGMetadataMakeTransformElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};
