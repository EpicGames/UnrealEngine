// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#include "PCGMetadataMakeVector.generated.h"

namespace PCGMetadataMakeVectorConstants
{
	const FName XLabel = TEXT("X");
	const FName YLabel = TEXT("Y");
	const FName ZLabel = TEXT("Z");
	const FName WLabel = TEXT("W");

	const FName XYLabel = TEXT("XY");
	const FName ZWLabel = TEXT("ZW");

	const FName XYZLabel = TEXT("XYZ");
}

UENUM()
enum class EPCGMetadataMakeVector3 : uint8
{
	ThreeValues,
	Vector2AndValue
};

UENUM()
enum class EPCGMetadataMakeVector4 : uint8
{
	FourValues,
	Vector2AndTwoValues,
	TwoVector2,
	Vector3AndValue
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataMakeVectorSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	// ~End UObject interface

#if WITH_EDITOR
	//~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif
	//~End UPCGSettings interface

	//~Begin UPCGMetadataSettingsBase interface
	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;

	virtual FName GetInputPinLabel(uint32 Index) const override;
	virtual uint32 GetOperandNum() const override;

	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	virtual uint16 GetOutputType(uint16 InputTypeId) const override;
	//~End UPCGMetadataSettingsBase interface

protected:
	//~Begin UPCGSettings interface
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin IPCGSettingsDefaultValueProvider interface
#if WITH_EDITOR
	virtual FString GetPinInitialDefaultValueString(FName PinLabel) const override;
#endif // WITH_EDITOR
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	//~End IPCGSettingsDefaultValueProvider interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource2;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource3;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource4;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ValidEnumValues = "Vector2, Vector, Vector4"))
	EPCGMetadataTypes OutputType = EPCGMetadataTypes::Vector2;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "OutputType == EPCGMetadataTypes::Vector", EditConditionHides))
	EPCGMetadataMakeVector3 MakeVector3Op = EPCGMetadataMakeVector3::ThreeValues;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "OutputType == EPCGMetadataTypes::Vector4", EditConditionHides))
	EPCGMetadataMakeVector4 MakeVector4Op = EPCGMetadataMakeVector4::FourValues;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName Input1AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input2AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input3AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input4AttributeName_DEPRECATED = NAME_None;
#endif
};

class FPCGMetadataMakeVectorElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};
