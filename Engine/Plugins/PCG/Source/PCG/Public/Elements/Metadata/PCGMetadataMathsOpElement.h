// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#include "PCGMetadataMathsOpElement.generated.h"

UENUM(Meta=(Bitflags))
enum class EPCGMetadataMathsOperation : uint16
{
	// Unary op
	UnaryOp = 1 << 10 UMETA(Hidden),
	Sign,
	Frac,
	Truncate,
	Round,
	Sqrt UMETA(SearchHints = "square root"),
	Abs UMETA(SearchHints = "|"),
	Floor,
	Ceil,
	OneMinus UMETA(Tooltip = "1 - X operation", SearchHints = "1-"),
	Inc UMETA(Tooltip = "X + 1 operation", SearchHints = "++"),
	Dec UMETA(Tooltip = "X - 1 operation", SearchHints = "--"),
	Negate UMETA(Tooltip = "-X operation", SearchHints = "-"),

	// Binary op
	BinaryOp = 1 << 11 UMETA(Hidden),
	Add UMETA(SearchHints = "+"),
	Subtract UMETA(SearchHints = "-"),
	Multiply UMETA(SearchHints = "*"),
	Divide UMETA(SearchHints = "/"),
	Max,
	Min,
	Pow UMETA(SearchHints = "**"),
	// ClampMin is just Max and ClampMax is just Min, so hide them.
	ClampMin UMETA(Hidden),
	ClampMax UMETA(Hidden),
	Modulo UMETA(SearchHints = "%"),
	Set UMETA(SearchHints = "="),

	// Ternary op
	TernaryOp = 1 << 12 UMETA(Hidden),
	Clamp,
	Lerp,
	MulAdd UMETA(Tooltip = "Multiply Add (A + B * C)", SearchHints = "*+"),
	AddModulo UMETA(Tooltip = "Add Modulo (A + B) % C. If B is negative: (A + C + B) % C", SearchHints = "+%")
};
ENUM_CLASS_FLAGS(EPCGMetadataMathsOperation);

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataMathsSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;
	//~End UPCGSettings interface

	//~Begin UPCGMetadataSettingsBase interface
	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;

	virtual FName GetInputPinLabel(uint32 Index) const override;
	virtual uint32 GetOperandNum() const override;
	virtual uint16 GetOutputType(uint16 InputTypeId) const override;

	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	//~End UPCGMetadataSettingsBase interface

	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override { return EPCGMetadataTypes::Double; }
#if WITH_EDITOR
	virtual bool IsPinDefaultValueMetadataTypeValid(FName PinLabel, EPCGMetadataTypes DataType) const override;
#endif
	//~End IPCGSettingsDefaultValueProvider interface

protected:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGMetadataMathsOperation Operation = EPCGMetadataMathsOperation::Add;

	/** For rounding operation, if the input type is float or double, use this option to force the output attribute to be int64. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditConditionHides, EditCondition = "Operation == EPCGMetadataMathsOperation::Round || Operation == EPCGMetadataMathsOperation::Truncate || Operation == EPCGMetadataMathsOperation::Floor || Operation == EPCGMetadataMathsOperation::Ceil"))
	bool bForceRoundingOpToInt = false;

	/** For operations that can yield floating point values, if the input type are ints, use this option to force the output attribute to be double. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditConditionHides, EditCondition = "Operation == EPCGMetadataMathsOperation::Divide || Operation == EPCGMetadataMathsOperation::Sqrt || Operation == EPCGMetadataMathsOperation::Pow || Operation == EPCGMetadataMathsOperation::Lerp"))
	bool bForceOpToDouble = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (EditCondition = "(Operation & '/Script/PCG.EPCGMetadataMathsOperation::BinaryOp') || (Operation & '/Script/PCG.EPCGMetadataMathsOperation::TernaryOp')", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource2;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (EditCondition = "Operation & '/Script/PCG.EPCGMetadataMathsOperation::TernaryOp'", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource3;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName Input1AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input2AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input3AttributeName_DEPRECATED = NAME_None;
#endif

private:
	bool ShouldForceOutputToInt(uint16 InputTypeId) const;
	bool ShouldForceOutputToDouble(uint16 InputTypeId) const;
};

class FPCGMetadataMathsElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};
