// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Data/PCGPointData.h"
#include "Elements/PCGTimeSlicedElementBase.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGDefaultValueContainer.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGDefaultValueInterface.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Containers/StaticArray.h"

#include "PCGMetadataOpElementBase.generated.h"

#define UE_API PCG_API

class FPCGMetadataAttributeBase;

// FIXME: To be removed when we are confident Metadata is stable in MT.
namespace PCGMetadataBase
{
	extern TAutoConsoleVariable<bool> CVarMetadataOperationInMT;
	extern TAutoConsoleVariable<int> CVarMetadataOperationChunkSize;
	extern TAutoConsoleVariable<bool> CVarMetadataOperationReserveValues;
}

namespace PCGMetadataSettingsBaseConstants
{
	const FName DoubleInputFirstLabel = TEXT("InA");
	const FName DoubleInputSecondLabel = TEXT("InB");
	const FName DoubleInputThirdLabel = TEXT("InC");

	const FName ClampMinLabel = TEXT("Min");
	const FName ClampMaxLabel = TEXT("Max");
	const FName LerpRatioLabel = TEXT("Ratio");
	const FName TransformLabel = TEXT("Transform");

	const FName DefaultOutputDataFromPinName = TEXT("Default");
}

// Defines behavior when number of entries doesn't match in inputs
UENUM()
enum class EPCGMetadataSettingsBaseMode
{
	Inferred     UMETA(Tooltip = "Broadcast for ParamData and no broadcast for SpatialData."),
	NoBroadcast  UMETA(ToolTip = "If number of entries doesn't match, will use the default value."),
	Broadcast    UMETA(ToolTip = "If there is no entry or a single entry, will repeat this value.")
};

UENUM()
enum class EPCGMetadataSettingsBaseTypes
{
	AutoUpcastTypes,
	StrictTypes
};

/**
 * Base class for all Metadata operations.
 * Metadata operation can work with attributes or properties. For example you could compute the addition between all points density and a constant from
 * a param data.
 * The output will be the duplication of the first spatial input (by default - can be overridden by OutputDataFromPin), 
 * with the same metadata + the result of the operation (either in an attribute or a property).
 * 
 * The new attribute can collide with one of the attributes in the incoming metadata. In this case, the attribute value will be overridden by the result
 * of the operation. It will also override the type of the attribute if it doesn't match the original.
 * 
 * We only support operations between points and between spatial data. They all need to match (or be a param data)
 * For example, if input 0 is a point data and input 1 is a spatial data, we fail.
 * 
 * You can specify the name of the attribute for each input and for the output.
 * If the input name is None, it will take the lastest attribute in the input metadata.
 * If the output name is None, it will take the input name.
 * 
 * Each operation has some requirements for the input types, and can broadcast some values into others (example Vector + Float -> Vector).
 * For example, if the op only accept booleans, all other value types will throw an error.
 * 
 * If there are multiple values for an attribute, the operation will be done on all values. If one input has N elements and the second has 1 element,
 * the second will be repeated for each element of the first for the operation. We only support N-N operations and N-1 operation (ie. The number of values
 * needs to be all the same or 1)
 * 
 * If the node doesn't provide an output, check the logs to know why it failed.
 */
UCLASS(MinimalAPI, BlueprintType, Abstract, ClassGroup = (Procedural))
class UPCGMetadataSettingsBase : public UPCGSettings, public IPCGSettingsDefaultValueProvider
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	// ~End UObject interface

	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual bool DefaultValuesAreEnabled() const override { return true; }
	UE_API virtual bool IsPinDefaultValueEnabled(FName PinLabel) const override;
	UE_API virtual bool IsPinDefaultValueActivated(FName PinLabel) const override;
	UE_API virtual EPCGMetadataTypes GetPinDefaultValueType(FName PinLabel) const override;
#if WITH_EDITOR
	UE_API virtual bool IsPinDefaultValueMetadataTypeValid(FName PinLabel, EPCGMetadataTypes DataType) const override;
	UE_API virtual void SetPinDefaultValue(FName PinLabel, const FString& DefaultValue, bool bCreateIfNeeded = false) override;
	UE_API virtual void ConvertPinDefaultValueMetadataType(FName PinLabel, EPCGMetadataTypes DataType) override;
	UE_API virtual void SetPinDefaultValueIsActivated(FName PinLabel, bool bIsActivated, bool bDirtySettings = true) override;
	UE_API virtual void ResetDefaultValues() override;
	virtual FString GetPinInitialDefaultValueString(FName PinLabel) const override { return PCG::Private::MetadataTraits<double>::ZeroValueString(); }
	UE_API virtual FString GetPinDefaultValueAsString(FName PinLabel) const override;
	UE_API virtual void ResetDefaultValue(FName PinLabel) override;
#endif // WITH_EDITOR

protected:
	UE_API virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	//~End IPCGSettingsDefaultValueProvider interface

public:
	/** Creates a Param Data with the inline constant default value properties inserted as metadata. */
	UE_API const UPCGParamData* CreateDefaultValueParamData(FPCGContext* Context, FName PinLabel) const;

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	UE_API virtual bool GetPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip) const override;
	UE_API virtual FText GetNodeTooltipText() const override;
	UE_API virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
#endif
	virtual bool HasFlippedTitleLines() const override { return true; }
	virtual bool HasDynamicPins() const override { return true; }
	UE_API virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;
	UE_API virtual bool DoesPinSupportPassThrough(UPCGPin* InPin) const override;

	virtual bool CanCullTaskIfUnwired() const override { return false; }
	UE_API virtual bool IsInputPinRequiredByExecution(const UPCGPin* InPin) const override;

	UE_DEPRECATED(5.8, "No longer needed - default values are included in settings CRC automatically via UPROPERTY serialization.")
	void AddDefaultValuesToCrc(FArchiveCrc32& Crc32) const {}

protected:
	UE_API virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	UE_API virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

#if WITH_EDITOR
	/**
	 * Helper to check if the InputSource property should be hidden to the user.
	 * NumSources should match the number of InputSource's the node has in its current configuration.
	 */
	UE_API bool CanEditInputSource(const FProperty* InProperty, int32 NumSources = 1) const;
#endif // WITH_EDITOR
	//~End UPCGSettings interface

public:
	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const PURE_VIRTUAL(UPCGMetadataSettingsBase::GetInputSource, return FPCGAttributePropertyInputSelector(); )

	virtual FName GetInputPinLabel(uint32 Index) const { return PCGPinConstants::DefaultInputLabel; }
	virtual uint32 GetOperandNum() const { return 1; }

	virtual FName GetOutputPinLabel(uint32 Index) const { return PCGPinConstants::DefaultOutputLabel; }
	virtual uint32 GetResultNum() const { return 1; }

	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const { return false; }
	virtual uint16 GetOutputType(uint16 InputTypeId) const { return InputTypeId; }
	virtual FName GetOutputAttributeName(FName BaseName, uint32 Index) const { return BaseName; }

	virtual bool HasDifferentOutputTypes() const { return false; }
	virtual TArray<uint16> GetAllOutputTypes() const { return TArray<uint16>(); }

	/* Can be overriden by child class to support default values on unplugged pins. */
	UE_DEPRECATED(5.6, "Override 'UPCGSettings::IsPinDefaultValueEnabled' and 'UPCGSettings::IsPinDefaultValueActivated' instead.")
	virtual bool DoesInputSupportDefaultValue(uint32 Index) const { return false; }

	UE_DEPRECATED(5.5, "Call/Implement version with FPCGContext parameter")
	virtual UPCGParamData* CreateDefaultValueParam(uint32 Index) const { return nullptr; }

	UE_DEPRECATED(5.6, "Replaced by inline DefaultValue system. Use CreateDefaultValueParamData instead.")
	virtual UPCGParamData* CreateDefaultValueParam(FPCGContext* Context, uint32 Index) const { return nullptr; }

	/* Return the current input pin to forward to the output. */
	UE_API uint32 GetInputPinToForward() const;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output", meta = (PCG_Overridable))
	FPCGAttributePropertyOutputSelector OutputTarget;

	/* By default, output is taken from first non-param pin (aka if the second pin is a point data, the output will be this point data). You can change it to any available input pin. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category = "Output", meta = (GetOptions = GetOutputDataFromPinOptions))
	FName OutputDataFromPin = PCGMetadataSettingsBaseConstants::DefaultOutputDataFromPinName;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName OutputAttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	EPCGMetadataSettingsBaseMode Mode_DEPRECATED = EPCGMetadataSettingsBaseMode::Inferred;
#endif // WITH_EDITORONLY_DATA

	static constexpr uint32 MaxNumberOfInputs = 4;
	static constexpr uint32 MaxNumberOfOutputs = 4;

#if WITH_EDITORONLY_DATA
	// Useful for unit tests. Allow to force a connection to allow the node to do its operation, even if nothing is connected to it.
	TStaticArray<bool, MaxNumberOfOutputs> ForceOutputConnections{};
#endif // WITH_EDITORONLY_DATA

protected:
	UE_DEPRECATED(5.7, "Use GetInputPinTypeID")
	UE_API EPCGDataType GetInputPinType(uint32 Index) const;
	
	/* Return the type union from incident edges with the support for default values. */
	UE_API FPCGDataTypeIdentifier GetInputPinTypeID(uint32 Index) const;

	/* Return the index of the given input pin label. INDEX_NONE if not found*/
	UE_API uint32 GetInputPinIndex(FName InPinLabel) const;

	/* Return the list of all the Input pins */
	UFUNCTION()
	UE_API TArray<FName> GetOutputDataFromPinOptions() const;

private:
	/** Stores the default values for the pins to be used as inline constants. */
	UPROPERTY()
	FPCGDefaultValueContainer DefaultValues;
};

namespace PCGMetadataOps
{
	struct FInputStateData
	{
		TArray<TArray<FPCGTaggedData>> OperandPinData;
		TArray<bool, TFixedAllocator<UPCGMetadataSettingsBase::MaxNumberOfInputs>> DefaultValueOverriddenPins;
	};

	struct FOperationData
	{
		int32 NumberOfElementsToProcess = -1;
		uint16 MostComplexInputType = static_cast<uint16>(EPCGMetadataTypes::Unknown);
		uint16 OutputType;
		const UPCGMetadataSettingsBase* Settings = nullptr;

		FPCGContext* Context;

		TArray<FPCGAttributePropertyInputSelector> InputSources;

		TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> InputKeys;
		TArray<TUniquePtr<IPCGAttributeAccessorKeys>> OutputKeys;

		TArray<TUniquePtr<const IPCGAttributeAccessor>> InputAccessors;
		TArray<TUniquePtr<IPCGAttributeAccessor>> OutputAccessors;

		template <int32 NbInputs, int32 NbOutputs>
		void Validate();
	};
}

class FPCGMetadataElementBase : public TPCGTimeSlicedElementBase<PCGMetadataOps::FInputStateData, PCGMetadataOps::FOperationData>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }

	virtual bool DoOperation(PCGMetadataOps::FOperationData& InOperationData) const = 0;

	/**
	* Generic method to factorise all the boilerplate code for a variable number of inputs/outputs
	*/
	template <typename... InputTypes, typename... Callbacks>
	inline bool DoNAryOp(PCGMetadataOps::FOperationData& InOperationData, TTuple<Callbacks...>&& InCallbacks) const;

	/* All operations can have a fixed number of inputs and a variable number of outputs.
	* Each output need to have its own callback, all taking the exact number of "const InType&" as input
	* and each can return a different output type.
	*/
	template <typename InType, typename... Callbacks>
	bool DoUnaryOp(PCGMetadataOps::FOperationData& InOperationData, Callbacks&& ...InCallbacks) const;

	template <typename InType1, typename InType2, typename... Callbacks>
	bool DoBinaryOp(PCGMetadataOps::FOperationData& InOperationData, Callbacks&& ...InCallbacks) const;

	template <typename InType1, typename InType2, typename InType3, typename... Callbacks>
	bool DoTernaryOp(PCGMetadataOps::FOperationData& InOperationData, Callbacks&& ...InCallbacks) const;

	template <typename InType1, typename InType2, typename InType3, typename InType4, typename... Callbacks>
	bool DoQuaternaryOp(PCGMetadataOps::FOperationData& InOperationData, Callbacks&& ...InCallbacks) const;

	/** To be called if we have no data to perform any operation, it will passthrough the input. The per execution state should be initialized before calling this, or performance issues might ensue if there is a large amount of inputs. */
	void PassthroughInput(FPCGContext* Context, TArray<FPCGTaggedData>& Outputs, const int32 Index) const;
};

/**
* Heavy templated code to factorize the gathering of inputs, the application of callbacks and the set of outputs.
* Note that the order of calls is from the bottom to the top: Operation -> Gather -> Apply.
* So start at the bottom!
*/
namespace PCG
{
namespace Private
{
namespace NAryOperation
{
	/**
	* Empty struct to pack/unpack types in templates.
	*/
	template <typename... T>
	struct Signature {};

	constexpr int32 DefaultChunkSize = 256;

	/**
	* Set of options to know if we need to use the default key + flags for get and set.
	*/
	struct Options
	{
		EPCGAttributeAccessorFlags GetFlags;
		EPCGAttributeAccessorFlags SetFlags;
		bool bUseDefaultKey = false;
	};

	/**
	* Finally we call our callback with the packed input values from InArgs, and set the output value in its accessor.
	* We use OutputIndex to know which callback to get (and therefore which output to set).
	* Note that we go in reverse order, and stop when OutputIndex is negative.
	*/
	template<int OutputIndex, typename ...Callbacks, typename ...Args>
	bool Apply(PCGMetadataOps::FOperationData& InOperationData, int32 StartIndex, int32 Range, const Options& InOptions, const TTuple<Callbacks...>& InCallbacks, Args&& ...InArgs)
	{
		if constexpr (OutputIndex < 0)
		{
			return true;
		}
		else
		{
			if (Range <= 0)
			{
				return true;
			}

			// First compute the first element to get its type.
			// We use argument unpacking, since our InArgs contains all the arrays with our values.
			// If in InArgs we have TArray<int>& IntValues, TArray<float>& FloatValues, the unpack will do:
			// (IntValues[0], FloatValues[0])

			// Implementation note: If we have just a single callback, it is safe to move InArgs, as it will only be used once.
			// But if we have multiple callbacks, it is not until the last one. So only move if we are at the last callback.
			// Since we move backward, it is if OutputIndex == 0
			auto InvokeCallback = [&InCallbacks, &InArgs...](const int32 i)
			{
				constexpr bool bCanMove = OutputIndex == 0;
				if constexpr (bCanMove)
				{
					return InCallbacks.template Get<OutputIndex>()(MoveTemp(InArgs[i])...);
				}
				else
				{
					return InCallbacks.template Get<OutputIndex>()(InArgs[i]...);
				}
			};
			
			auto OutValue = InvokeCallback(0);
			using OutType = decltype(OutValue);

			TArray<OutType, TInlineAllocator<DefaultChunkSize>> OutValues;
			if constexpr (std::is_trivially_copyable_v<OutType>)
			{
				OutValues.SetNumUninitialized(Range);
			}
			else
			{
				OutValues.SetNum(Range);
			}
			
			OutValues[0] = OutValue;

			for (int32 i = 1; i < Range; ++i)
			{
				OutValues[i] = InvokeCallback(i);
			}

			bool bSuccess = true;
			TArrayView<const OutType> OutValuesView(OutValues);

			if (InOptions.bUseDefaultKey)
			{
				FPCGAttributeAccessorKeysEntries DefaultAccessorKey(PCGInvalidEntryKey);
				bSuccess = InOperationData.OutputAccessors[OutputIndex]->SetRange(OutValuesView, /*Index=*/0, DefaultAccessorKey, InOptions.SetFlags);
			}
			else
			{
				bSuccess = InOperationData.OutputAccessors[OutputIndex]->SetRange(OutValuesView, StartIndex, *InOperationData.OutputKeys[OutputIndex], InOptions.SetFlags);
			}

			if (!bSuccess)
			{
				return false;
			}

			return Apply<OutputIndex - 1>(InOperationData, StartIndex, Range, InOptions, InCallbacks, std::forward<Args>(InArgs)...);
		}
	}

	/**
	* When Signature doesn't have any templated types anymore (Signature<>), we got all our input values packed in "InArgs", so it's time to compute our operation and set the outputs.
	* To do so, we use Apply templated with the LastOutputIndex, and go backwards (last output to first output).
	*/
	template <typename... Callbacks, typename... Args>
	bool Gather(PCGMetadataOps::FOperationData& InOperationData, int32 StartIndex, int32 Range, const Options& InOptions, const TTuple<Callbacks...>& InCallbacks, int InputIndex, Signature<> S, Args&& ...InArgs)
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(PCG::Private::NAryOperation::Apply);

		constexpr int LastOutputIndex = (int)sizeof...(Callbacks) - 1;
		return Apply<LastOutputIndex>(InOperationData, StartIndex, Range, InOptions, InCallbacks, std::forward<Args>(InArgs)...);
	}

	/**
	* The idea of gather is to pack all the input values at the end of the function call.
	* We use InputType, from our Signature struct to extract the current InputType, get a range of values from our accessor (using the InputIndex)
	* then call Gather recursively, with the rest of our input types in Signature, InputIndex incremented by one and our newly InputValues moved at the end and packed into InArgs.
	* We specialize the case where Signature is empty, meaning we got all our input values.
	* 
	* For example, if InputTypes = <int, float>, we have 3 calls:
	* Gather(InOperationData, StartIndex, Range, InOptions, InCallbacks, 0, Signature<int, float>);
	* Gather(InOperationData, StartIndex, Range, InOptions, InCallbacks, 1, Signature<float>, IntValues);
	* Gather(InOperationData, StartIndex, Range, InOptions, InCallbacks, 2, Signature<>, IntValues, FloatValues);
	*/
	template <typename... Callbacks, typename InputType, typename... InputTypes, typename... Args>
	bool Gather(PCGMetadataOps::FOperationData& InOperationData, int32 StartIndex, int32 Range, const Options& InOptions, const TTuple<Callbacks...>& InCallbacks, int InputIndex, Signature<InputType, InputTypes...> S, Args&& ...InArgs)
	{
		bool bSuccess = true;

		TArray<InputType, TInlineAllocator<DefaultChunkSize>> InputValues;
		if constexpr (std::is_trivially_copyable_v<InputType>)
		{
			InputValues.SetNumUninitialized(Range);
		}
		else
		{
			InputValues.SetNum(Range);
		}
		
		if (InOptions.bUseDefaultKey)
		{
			FPCGAttributeAccessorKeysEntries DefaultAccessorKey(PCGInvalidEntryKey);
			bSuccess = InOperationData.InputAccessors[InputIndex]->GetRange<InputType>(InputValues, /*Index=*/0, DefaultAccessorKey, InOptions.GetFlags);
		}
		else
		{
			bSuccess = InOperationData.InputAccessors[InputIndex]->GetRange<InputType>(InputValues, StartIndex, *InOperationData.InputKeys[InputIndex], InOptions.GetFlags);
		}

		if (!bSuccess)
		{
			return false;
		}

		return Gather(InOperationData, StartIndex, Range, InOptions, InCallbacks, InputIndex + 1, Signature<InputTypes...>(), std::forward<Args>(InArgs)..., std::move(InputValues));
	}

	/**
	* First we need to gather all our input values.
	* We use an index, starting at 0, and also use our empty struct to pack all our input types.
	*/
	template <typename... InputTypes, typename... Callbacks>
	bool Operation(PCGMetadataOps::FOperationData& InOperationData, int32 StartIndex, int32 Range, const Options& InOptions, const TTuple<Callbacks...>& InCallbacks)
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(PCG::Private::NAryOperation::Operation);
		return Gather(InOperationData, StartIndex, Range, InOptions, InCallbacks, 0, Signature<InputTypes...>());
	}
}
}
}

template <int32 NbInputs, int32 NbOutputs>
void PCGMetadataOps::FOperationData::Validate()
{
	check(OutputKeys.Num() == NbOutputs);

	for (int32 i = 0; i < NbInputs; ++i)
	{
		check(InputAccessors[i].IsValid());
		check(InputKeys[i].IsValid());
	}

	for (int32 j = 0; j < NbOutputs; ++j)
	{
		check(OutputAccessors[j].IsValid());
		check(OutputKeys[j].IsValid());
	}
}

template <typename... InputTypes, typename... Callbacks>
inline bool FPCGMetadataElementBase::DoNAryOp(PCGMetadataOps::FOperationData& InOperationData, TTuple<Callbacks...>&& InCallbacks) const
{
	// Validate that all is good
	constexpr uint32 NbInputs = (uint32)sizeof...(InputTypes);
	constexpr uint32 NbOutputs = (uint32)sizeof...(Callbacks);

	static_assert(NbInputs <= UPCGMetadataSettingsBase::MaxNumberOfInputs);
	static_assert(NbOutputs <= UPCGMetadataSettingsBase::MaxNumberOfOutputs);

	InOperationData.Validate<NbInputs, NbOutputs>();

	EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible;

	// First set the default value (only on first pass)
	PCG::Private::NAryOperation::Options Options{ Flags, Flags | EPCGAttributeAccessorFlags::AllowSetDefaultValue, true };
	if (!InOperationData.Context->AsyncState.bStarted)
	{
		if (PCGMetadataBase::CVarMetadataOperationReserveValues.GetValueOnAnyThread())
		{
			for (int32 j = 0; j < NbOutputs; ++j)
			{
				// We can't re-use entry keys yet, it can be dangerous in some situations where some points share their entry key.
				InOperationData.OutputAccessors[j]->Prepare(*InOperationData.OutputKeys[j], InOperationData.NumberOfElementsToProcess, /*bCanReuseEntryKeys=*/false);
			}
		}

		PCG::Private::NAryOperation::Operation<InputTypes...>(InOperationData, /*StartIndex=*/0, /*Range=*/1, Options, InCallbacks);
	}

	// If nothing to do now, we can early out.
	if (InOperationData.NumberOfElementsToProcess == 0)
	{
		return true;
	}

	// Then iterate over all the values
	Options.SetFlags = Flags;
	Options.bUseDefaultKey = false;

	const int32 ChunkSize = PCGMetadataBase::CVarMetadataOperationChunkSize.GetValueOnAnyThread();

	if (PCGMetadataBase::CVarMetadataOperationInMT.GetValueOnAnyThread())
	{
		return FPCGAsync::AsyncProcessingOneToOneRangeEx(&InOperationData.Context->AsyncState, InOperationData.NumberOfElementsToProcess, []() {},
			[&InOperationData, &InCallbacks, &Options](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			PCG::Private::NAryOperation::Operation<InputTypes...>(InOperationData, StartReadIndex, Count, Options, InCallbacks);
			return Count;
		}, /*bEnableTimeSlicing=*/true, ChunkSize);
	}
	else
	{
		const int32 NumberOfIterations = (InOperationData.NumberOfElementsToProcess + PCG::Private::NAryOperation::DefaultChunkSize - 1) / PCG::Private::NAryOperation::DefaultChunkSize;
		for (int32 i = 0; i < NumberOfIterations; ++i)
		{
			int32 StartIndex = i * PCG::Private::NAryOperation::DefaultChunkSize;
			int32 Range = FMath::Min(InOperationData.NumberOfElementsToProcess - StartIndex, PCG::Private::NAryOperation::DefaultChunkSize);
			PCG::Private::NAryOperation::Operation<InputTypes...>(InOperationData, StartIndex, Range, Options, InCallbacks);
		}

		return true;
	}
}

template <typename InType, typename... Callbacks>
inline bool FPCGMetadataElementBase::DoUnaryOp(PCGMetadataOps::FOperationData& InOperationData, Callbacks&& ...InCallbacks) const
{
	return DoNAryOp<InType>(InOperationData, ForwardAsTuple(std::forward<Callbacks>(InCallbacks)...));
}

template <typename InType1, typename InType2, typename... Callbacks>
inline bool FPCGMetadataElementBase::DoBinaryOp(PCGMetadataOps::FOperationData& InOperationData, Callbacks&& ...InCallbacks) const
{
	return DoNAryOp<InType1, InType2>(InOperationData, ForwardAsTuple(std::forward<Callbacks>(InCallbacks)...));
}

template <typename InType1, typename InType2, typename InType3, typename... Callbacks>
inline bool FPCGMetadataElementBase::DoTernaryOp(PCGMetadataOps::FOperationData& InOperationData, Callbacks&& ...InCallbacks) const
{
	return DoNAryOp<InType1, InType2, InType3>(InOperationData, ForwardAsTuple(std::forward<Callbacks>(InCallbacks)...));
}

template <typename InType1, typename InType2, typename InType3, typename InType4, typename... Callbacks>
inline bool FPCGMetadataElementBase::DoQuaternaryOp(PCGMetadataOps::FOperationData& InOperationData, Callbacks&& ...InCallbacks) const
{
	return DoNAryOp<InType1, InType2, InType3, InType4>(InOperationData, ForwardAsTuple(std::forward<Callbacks>(InCallbacks)...));
}

#undef UE_API
