// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGMetadataPartition.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataPartitionSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual bool DisplayExecuteOnGPUSetting() const override { return true; }
	virtual void CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const override;
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("AttributePartition")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGMetadataPartitionSettings", "NodeTitle", "Attribute Partition"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
#endif
	virtual bool HasDynamicPins() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	// TODO: Should be overridable once array override is supported
	/** The data will be partitioned on these selected attributes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FPCGAttributePropertyInputSelector> PartitionAttributeSelectors = {FPCGAttributePropertyInputSelector()};

	// TODO: Should be deprecated once array override is supported
	/** Comma-separated list of attributes to partition against. */
	UPROPERTY(meta = (PCG_Overridable))
	FString PartitionAttributeNames;

	/** Enables deprecated behavior using spaces as separators. Disable to update the node to current behavior. */
	UE_DEPRECATED(5.5, "bTokenizeOnWhiteSpace has been deprecated.")
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = "bTokenizeOnWhiteSpace", EditConditionHides, DeprecationMessage = "bTokenizeOnWhiteSpace has been deprecated."))
	bool bTokenizeOnWhiteSpace = false;

	/** Assign an index partition as an extra attribute. */
	UPROPERTY(EditAnywhere, DisplayName = "Output Partition Index", Category = Settings, meta = (PCG_Overridable))
	bool bAssignIndexPartition = false;

	/** If we assign an index, we can also not partition (and only assign the partition index to the original data). */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bAssignIndexPartition", EditConditionHides))
	bool bDoNotPartition = true;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bAssignIndexPartition", EditConditionHides))
	FName PartitionIndexAttributeName = TEXT("PartitionIndex");

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "PartitionAttribute has been deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "PartitionAttribute has been deprecated."))
	FName PartitionAttribute_DEPRECATED = NAME_None;

	UE_DEPRECATED(5.5, "PartitionAttributeSource has been deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "PartitionAttributeSource has been deprecated."))
	FPCGAttributePropertyInputSelector PartitionAttributeSource_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};

class FPCGMetadataPartitionElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
