// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeKernel.h"

#include "PCGMetadataPartitionKernel.generated.h"

class UPCGMetadataPartitionSettings;

namespace PCGMetadataPartitionConstants
{
	const FName ElementCountsPinLabel = TEXT("ElementCounts");
}

UCLASS()
class UPCGMetadataPartitionKernel : public UPCGComputeKernel
{
	GENERATED_BODY()

public:
	virtual bool IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const override;
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const override;
#if WITH_EDITOR
	virtual const TCHAR* GetSourceFilePath() const override { return TEXT("/Plugin/PCG/Private/Elements/PCGMetadataPartition.usf"); }
	virtual FString GetEntryPoint() const override { return TEXT("PCGMetadataPartitionCS"); }
	void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
	// Split graph to update element counts on CPU before proceeding. Compacts data for downstream kernels.
	virtual bool SplitGraphAtOutput() const override { return true; }
#endif
	virtual void GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const override;
	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;

protected:
#if WITH_EDITOR
	virtual bool PerformStaticValidation() override;
#endif
};
