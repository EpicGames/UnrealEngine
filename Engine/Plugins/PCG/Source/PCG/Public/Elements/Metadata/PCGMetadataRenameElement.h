// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGMetadataRenameElement.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataRenameSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGMetadataRenameSettings(const FObjectInitializer& ObjectInitializer);
	
	//~Begin UObject interface
	void PostLoad() override;
	//~End UObject interface
	
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("AttributeRename")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGMetadataRenameSettings", "NodeTitle", "Attribute Rename"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
#endif

	virtual bool HasDynamicPins() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Selector to the attribute to rename. Can also specify a metadata domain. Can't be a property or contains extractors. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector AttributeToRename;

	/** Name of the new attribute, rename is in place. It will be on the same domain specified by the AttributeToRename selector. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName NewAttributeName = NAME_None;
};

class FPCGMetadataRenameElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
