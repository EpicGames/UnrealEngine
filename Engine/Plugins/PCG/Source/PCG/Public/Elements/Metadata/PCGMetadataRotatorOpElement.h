// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#include "PCGMetadataRotatorOpElement.generated.h"

UENUM()
enum class EPCGMetadataRotatorOperation : uint16
{
	RotatorOp = 0 UMETA(Hidden),
	Combine,
	Invert,
	Lerp,
	Normalize,

	TransformOp = 100 UMETA(Hidden),
	TransformRotation,
	InverseTransformRotation
};

namespace PCGMetadataRotatorHelpers
{
	// Taken from Kismet Math Library
	FRotator RLerp(const FRotator& A, const FRotator& B, double Alpha, bool bShortestPath);
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataRotatorSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;
	//~End UPCGSettings interface

	//~Begin UPCGMetadataSettingsBase interface
	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;

	virtual FName GetInputPinLabel(uint32 Index) const override;
	virtual uint32 GetOperandNum() const override;

	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	//~End UPCGMetadataSettingsBase interface

protected:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin IPCGSettingsDefaultValueProvider interface
#if WITH_EDITOR
	virtual bool IsPinDefaultValueMetadataTypeValid(FName PinLabel, EPCGMetadataTypes DataType) const override;
#endif
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	//~End IPCGSettingsDefaultValueProvider interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGMetadataRotatorOperation Operation = EPCGMetadataRotatorOperation::Combine;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (EditCondition = "Operation != EPCGMetadataRotatorOperation::Invert", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource2;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (EditCondition = "Operation == EPCGMetadataRotatorOperation::Lerp", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource3;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName Input1AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input2AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input3AttributeName_DEPRECATED = NAME_None;
#endif
};

class FPCGMetadataRotatorElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};
