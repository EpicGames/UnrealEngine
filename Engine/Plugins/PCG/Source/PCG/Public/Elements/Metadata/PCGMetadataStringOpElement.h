// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#include "PCGMetadataStringOpElement.generated.h"

UENUM()
enum class EPCGMetadataStringOperation : uint16
{
	Append UMETA(DisplayName="Append String"),
	Replace UMETA(DisplayName="Replace String"),
	Substring UMETA(DisplayName="Substring", Tooltip="True if string A contains substring B.", SearchHints="contains"),
	Matches UMETA(DisplayName="Matches Wildcard", Tooltip="True if string A matches substring B exactly. B can have wildcards.\n*?-type wildcard: '*' matches 0 or more arbitrary characters, '?' matches either an arbitrary character or nothing, all other characters match themselves.", SearchHints="equal"),
	ToUpper UMETA(DisplayName="To Upper", Tooltip="Convert all characters to upper case."),
	ToLower UMETA(DisplayName="To Lower", Tooltip="Convert all characters to lower case."),
	TrimStart UMETA(DisplayName="Trim Start", Tooltip="Trim whitespace from the beginning of the string."),
	TrimEnd UMETA(DisplayName="Trim End", Tooltip="Trim whitespace from the end of the string."),
	TrimStartAndEnd UMETA(DisplayName="Trim Start and End", Tooltip="Trim whitespace from the beginning and end of the string."),
	Left UMETA(ToolTip="Returns the left most given number of characters"),
	LeftChop UMETA(DisplayName="Left Chop", ToolTip="Returns the left most characters from the string chopping the given number of characters from the end"),
	Right UMETA(ToolTip="Returns the right most given number of characters"),
	RightChop UMETA(DisplayName="Right Chop", ToolTip="Returns the right most characters from the string chopping the given number of characters from the start"),
	Mid UMETA(ToolTip="Returns the substring from Start position for Count characters."),
	RemoveFromStart UMETA(DisplayName="Remove from Start", ToolTip="Removes the text from the start of the string if it exists."),
	RemoveFromEnd UMETA(DisplayName="Remove from End", ToolTip="Removes the text from the end of the string if it exists."),
	Find UMETA(ToolTip="Return the index of the first occurence of B in A. -1 if not found"),
	FindLast UMETA(DisplayName = "Find Last", ToolTip="Return the index of the last occurence of B in A. -1 if not found")
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataStringOpSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface
	
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	//~Begin UPCGMetadataSettingsBase interface
	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;

	virtual FName GetInputPinLabel(uint32 Index) const override;
	virtual uint32 GetOperandNum() const override;

	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	virtual uint16 GetOutputType(uint16 InputTypeId) const override;
	//~End UPCGMetadataSettingsBase interface

	//~Begin IPCGSettingsDefaultValueProvider interface
#if WITH_EDITOR
	virtual FString GetPinInitialDefaultValueString(FName PinLabel) const override;
#endif // WITH_EDITOR
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	//~End IPCGSettingsDefaultValueProvider interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGMetadataStringOperation Operation = EPCGMetadataStringOperation::Append;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "HasSearchCase()", EditConditionHides))
	TEnumAsByte<ESearchCase::Type> SearchCase = ESearchCase::CaseSensitive;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable, EditCondition = "HasTwoOperands() || HasThreeOperands()", EditConditionHides))
	FPCGAttributePropertyInputSelector InputSource2;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable, EditCondition = "HasThreeOperands()", EditConditionHides))
	FPCGAttributePropertyInputSelector InputSource3;

	/** For deprecation, for Matches and Substring, allow to output the result in String instead of bool. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Output, AdvancedDisplay, meta=(EditCondition = "OutputsBool()", EditConditionHides))
	bool OutputTypeAsString = false;

private:
#if WITH_EDITOR
	UFUNCTION()
	bool HasSearchCase() const;
#endif // WITH_EDITOR

	UFUNCTION()
	bool HasTwoOperands() const;

	UFUNCTION()
	bool HasThreeOperands() const;

	UFUNCTION()
	bool OutputsBool() const;
};

class FPCGMetadataStringOpElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};