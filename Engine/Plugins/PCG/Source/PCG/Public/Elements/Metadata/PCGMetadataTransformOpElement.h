// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#include "PCGMetadataTransformOpElement.generated.h"

UENUM()
enum class EPCGMetadataTransformOperation : uint16
{
	Compose,
	Invert,
	Lerp,
};

// Taken from Kismet Math Library
UENUM()
enum class EPCGTransformLerpMode : uint16
{
	/** Shortest Path or Quaternion interpolation for the rotation. */
	QuatInterp,

	/** Rotor or Euler Angle interpolation. */
	EulerInterp,

	/** Dual quaternion interpolation, follows helix or screw-motion path between keyframes.   */
	DualQuatInterp
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataTransformSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;
	//~End UPCGSettings interface

	//~Begin UPCGMetadataSettingsBase interface
	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;

	virtual FName GetInputPinLabel(uint32 Index) const override;
	virtual uint32 GetOperandNum() const override;

	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	//~End UPCGMetadataSettingsBase interface

	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual bool IsPinDefaultValueEnabled(FName PinLabel) const override;
	virtual EPCGMetadataTypes GetPinDefaultValueType(FName PinLabel) const override;
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	//~End IPCGSettingsDefaultValueProvider interface
protected:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGMetadataTransformOperation Operation = EPCGMetadataTransformOperation::Compose;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Operation == EPCGMetadataTransformOperation::Lerp", EditConditionHides))
	EPCGTransformLerpMode TransformLerpMode = EPCGTransformLerpMode::QuatInterp;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (EditCondition = "Operation != EPCGMetadataTransformOperation::Invert", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource2;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (EditCondition = "Operation == EPCGMetadataTransformOperation::Lerp", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource3;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName Input1AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input2AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input3AttributeName_DEPRECATED = NAME_None;
#endif
};

class FPCGMetadataTransformElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};
