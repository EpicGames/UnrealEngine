// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Metadata/PCGMetadataOpElementBase.h"

#include "PCGMetadataVectorOpElement.generated.h"

UENUM()
enum class EPCGMetadataVectorOperation : uint16
{
	VectorOp = 0 UMETA(Hidden),
	Cross,
	Dot,
	Distance,
	Normalize,
	Length,
	RotateAroundAxis,
	
	// Angles between vectors
	RotatorBetweenVectors,
	EulerBetweenVectors,
	AngleBetweenVector2D,

	TransformOp = 100 UMETA(Hidden),
	TransformDirection,
	TransformLocation,
	InverseTransformDirection,
	InverseTransformLocation
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMetadataVectorSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;
	//~End UPCGSettings interface

	//~Begin UPCGMetadataSettingsBase interface
	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;

	virtual FName GetInputPinLabel(uint32 Index) const override;
	virtual uint32 GetOperandNum() const override;

	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	virtual uint16 GetOutputType(uint16 InputTypeId) const override;
	//~End UPCGMetadataSettingsBase interface

protected:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual bool IsPinDefaultValueEnabled(FName PinLabel) const override;
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	//~End IPCGSettingsDefaultValueProvider interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGMetadataVectorOperation Operation = EPCGMetadataVectorOperation::Cross;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (EditCondition = "Operation != EPCGMetadataVectorOperation::Normalize && Operation != EPCGMetadataVectorOperation::Length", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource2;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Input, meta = (EditCondition = "Operation == EPCGMetadataVectorOperation::RotateAroundAxis", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource3;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName Input1AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input2AttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName Input3AttributeName_DEPRECATED = NAME_None;
#endif
};

class FPCGMetadataVectorElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};
