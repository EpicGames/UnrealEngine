// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Metadata/PCGMetadataOpElementBase.h"
#include "Metadata/PCGMetadataAttributeTraits.h"

#include "PCGParseString.generated.h"

/**
* Convert a string to a numerical value.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGParseStringSettings : public UPCGMetadataSettingsBase
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

	virtual bool HasFlippedTitleLines() const override { return false; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	//~Begin UPCGMetadataSettingsBase interface
	virtual FPCGAttributePropertyInputSelector GetInputSource(uint32 Index) const override;
	virtual bool IsSupportedInputType(uint16 TypeId, uint32 InputIndex, bool& bHasSpecialRequirement) const override;
	virtual bool HasDifferentOutputTypes() const { return true; }
	virtual TArray<uint16> GetAllOutputTypes() const;
	//~End UPCGMetadataSettingsBase interface

	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual bool DefaultValuesAreEnabled() const override { return false; }
	//~End IPCGSettingsDefaultValueProvider interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Input", meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable))
	EPCGMetadataTypes TargetType = EPCGMetadataTypes::Integer32;
};

class FPCGParseStringElement : public FPCGMetadataElementBase
{
protected:
	virtual bool DoOperation(PCGMetadataOps::FOperationData& OperationData) const override;
};

