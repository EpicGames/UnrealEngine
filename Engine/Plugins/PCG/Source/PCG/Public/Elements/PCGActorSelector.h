// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGAttributePropertySelector.h"

#include "Misc/Guid.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"

#include "PCGActorSelector.generated.h"

#define UE_API PCG_API

class AActor;
class IPCGGraphExecutionSource;
class UActorComponent;
class UPCGComponent;
class UWorld;
struct FPCGActorSelectorSettings;

UENUM()
enum class EPCGActorSelection : uint8
{
	ByTag,
	// Deprecated - actor labels are unavailable in shipping builds
	ByName UMETA(Hidden),
	ByClass,
	ByPath UMETA(Hidden), // Hidden because actors are not tracked by paths.
	Unknown UMETA(Hidden)
};

UENUM()
enum class EPCGComponentSelection : uint8
{
	ByTag,
	ByClass,
	Unknown UMETA(Hidden)
};

UENUM()
enum class EPCGActorFilter : uint8
{
	/** This actor (either the original PCG actor or the partition actor if partitioning is enabled). */
	Self,
	/** The parent of this actor in the hierarchy. */
	Parent,
	/** The top most parent of this actor in the hierarchy. */
	Root,
	/** All actors in world. */
	AllWorldActors,
	/** The source PCG actor (rather than the generated partition actor). */
	Original,
	/** Consider only the provided list of actors */
	FromInput
};

UENUM()
enum class EPCGIsolatedActorIncludeFlag : uint8
{
	/** Ignores isolated actors that stem from the selection. */
	Ignore,
	/** Includes isolated actors that were extracted from the selection. */
	Include,
	/** Returns only isolated actors extracted from the original selection. */
	Only,
};

/**
 * Base struct to extend FPCGSelectionKey to allow more fine grained and custom selection
 */
USTRUCT()
struct FPCGCustomSelectionKey
{
public:
	GENERATED_BODY()

	FPCGCustomSelectionKey() = default;
	virtual ~FPCGCustomSelectionKey() = default;
		
	bool IsMatching(const FPCGCustomSelectionKey& InSelectionKey) const
	{
		if (GetScriptStruct() != InSelectionKey.GetScriptStruct())
		{
			return false;
		}

		return IsMatchingInternal(InSelectionKey);
	}

	bool Equals(const FPCGCustomSelectionKey& InSelectionKey) const
	{
		if (GetScriptStruct() != InSelectionKey.GetScriptStruct())
		{
			return false;
		}

		return EqualsInternal(InSelectionKey);
	}

	template <typename T>
	T* CastTo()
	{
		return GetScriptStruct() == T::StaticStruct() ? StaticCast<T*>(this) : nullptr;
	}
	
	template <typename T>
	const T* CastTo() const
	{
		return GetScriptStruct() == T::StaticStruct() ? StaticCast<const T*>(this) : nullptr;
	}

	PCG_API friend uint32 GetTypeHash(const FPCGCustomSelectionKey& In);
	
	virtual UScriptStruct* GetScriptStruct() const { return nullptr; }

protected:
	virtual bool IsMatchingInternal(const FPCGCustomSelectionKey&) const PURE_VIRTUAL(FPCGCustomSelectionKey::IsMatchingInternal, return false;)
	virtual bool EqualsInternal(const FPCGCustomSelectionKey&) const PURE_VIRTUAL(FPCGCustomSelectionKey::EqualsInternal, return false;)
	virtual uint32 GetTypeHashInternal() const PURE_VIRTUAL(FPCGCustomSelectionKey::GetTypeHashInternal, return 0;)
};

template<>
struct TStructOpsTypeTraits<FPCGCustomSelectionKey> : public TStructOpsTypeTraitsBase2<FPCGCustomSelectionKey>
{
	enum
	{
		WithPureVirtual = true,
	};
};

/**
* Structure to specify a selection criteria for an object/actor
* Object can be selected using the EPCGActorSelection::ByClass or EPCGActorSelection::ByPath
* Actors have more options for selection with Self/Parent/Root/Original and also EPCGActorSelection::ByTag
*/
USTRUCT()
struct FPCGSelectionKey
{
	friend class FPCGTrackingManager;

	GENERATED_BODY()

	FPCGSelectionKey() = default;

	// For all filters except FromInput
	UE_API explicit FPCGSelectionKey(const FPCGActorSelectorSettings& InActorSelector);

	// For all filters others than AllWorldActor. For AllWorldActors Filter, use the other constructors.
	UE_API explicit FPCGSelectionKey(EPCGActorFilter InFilter);
	UE_API explicit FPCGSelectionKey(FName InTag);
	UE_API explicit FPCGSelectionKey(TSubclassOf<UObject> InSelectionClass);

	static UE_API FPCGSelectionKey CreateFromPath(const FSoftObjectPath& InObjectPath);
	static UE_API FPCGSelectionKey CreateFromPath(FSoftObjectPath&& InObjectPath);
	static UE_API FPCGSelectionKey CreateFromObjectPtr(const UObject* InObject);

	UE_API bool operator==(const FPCGSelectionKey& InOther) const;

	// Friend functions need to be explicitly exported.
	PCG_API friend uint32 GetTypeHash(const FPCGSelectionKey& In);

	UE_API bool IsMatching(const FPCGSelectionKey& InSelectionKey, const IPCGGraphExecutionSource* InExecutionSource) const;
	UE_API bool IsMatching(const FPCGSelectionKey& InSelectionKey, const TSet<FName>& InRemovedTags, const TSet<IPCGGraphExecutionSource*>& InExecutionSources, TSet<IPCGGraphExecutionSource*>* OptionalExecutionSources = nullptr) const;

	UE_API void SetExtraDependency(const UClass* InExtraDependency);
	UE_API void UpdateAfterTagChange();

	UE_API const UObject* GetObjectFromPath() const;

	UPROPERTY()
	EPCGActorFilter ActorFilter = EPCGActorFilter::AllWorldActors;

	UPROPERTY()
	EPCGActorSelection Selection = EPCGActorSelection::Unknown;

	UPROPERTY()
	FName Tag = NAME_None;

	UPROPERTY()
	TSubclassOf<UObject> SelectionClass;

	// If the Selection is ByPath, contain the path to select.
	UPROPERTY()
	FSoftObjectPath ObjectPath;

	// If it should track a specific object dependency instead of an actor. For example, GetActorData with GetPCGComponent data.
	UPROPERTY()
	TObjectPtr<const UClass> OptionalExtraDependency = nullptr;

	UPROPERTY()
	TInstancedStruct<FPCGCustomSelectionKey> CustomKey;

	/** If both keys matching have optional bounds, then the IsMatching will test that there is an overlap between any pair of bounds. */
	UPROPERTY()
	TArray<FBox> OptionalBounds;

	// Deprecated methods
	UE_DEPRECATED(5.8, "Replaced by the version using IPCGGraphExecutionSource.")
	UE_API bool IsMatching(const FPCGSelectionKey& InSelectionKey, const TSet<FName>& InRemovedTags, const TSet<UPCGComponent*>& InComponents, TSet<UPCGComponent*>* OptionalMatchedComponents = nullptr) const;
	
	UE_DEPRECATED(5.7, "Replaced by the version with a FPCGSelectionKey.")
	UE_API bool IsMatching(const UObject* InObject, const FSoftObjectPath& InObjectPath, const UPCGComponent* InComponent) const;
	UE_DEPRECATED(5.7, "Replaced by the version with a FPCGSelectionKey.")
	UE_API bool IsMatching(const UObject* InObject, const FSoftObjectPath& InObjectPath, const TSet<FName>& InRemovedTags, const TSet<UPCGComponent*>& InComponents, TSet<UPCGComponent*>* OptionalMatchedComponents = nullptr) const;

protected:
	FString CachedTagString;
	bool bTagContainsWildcard = false;
	mutable const UObject* CachedObject;
};

PCG_API FArchive& operator<<(FArchive& Ar, FPCGSelectionKey& Key);

/** Helper struct for organizing queries against the world to gather actors. */
USTRUCT(BlueprintType)
struct FPCGActorSelectorSettings
{
	GENERATED_BODY()

	/** Which actors to consider. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings", meta = (EditCondition = "bShowActorFilter", EditConditionHides, HideEditConditionToggle))
	EPCGActorFilter ActorFilter = EPCGActorFilter::Self;

	/** Filters out actors that do not overlap the source component bounds. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings", meta = (EditCondition = "ActorFilter==EPCGActorFilter::AllWorldActors", EditConditionHides))
	bool bMustOverlapSelf = false;

	/** Whether to consider child actors. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings", meta = (EditCondition = "bShowIncludeChildren && ActorFilter!=EPCGActorFilter::AllWorldActors", EditConditionHides))
	bool bIncludeChildren = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings")
	EPCGIsolatedActorIncludeFlag IncludeIsolatedActors = EPCGIsolatedActorIncludeFlag::Ignore;

	/** Enables/disables fine-grained actor filtering options. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings", meta = (PCG_NotOverridable, EditCondition = "ActorFilter!=EPCGActorFilter::AllWorldActors && bIncludeChildren", EditConditionHides))
	bool bDisableFilter = false;

	/** How to select when filtering actors. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings", meta = (EditCondition = "bShowActorSelection && (ActorFilter==EPCGActorFilter::AllWorldActors || (bIncludeChildren && !bDisableFilter))", EditConditionHides))
	EPCGActorSelection ActorSelection = EPCGActorSelection::ByTag;

	/** Tag to match against when filtering actors. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings", meta = (EditCondition = "bShowActorSelection && (ActorFilter==EPCGActorFilter::AllWorldActors || (bIncludeChildren && !bDisableFilter)) && ActorSelection==EPCGActorSelection::ByTag", EditConditionHides))
	FName ActorSelectionTag;

	/** Actor class to match against when filtering actors. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings", meta = (EditCondition = "bShowActorSelectionClass && bShowActorSelection && (ActorFilter==EPCGActorFilter::AllWorldActors || (bIncludeChildren && !bDisableFilter)) && ActorSelection==EPCGActorSelection::ByClass", EditConditionHides, AllowAbstract = "true"))
	TSubclassOf<AActor> ActorSelectionClass;

	/** Controls what attribute to read from when the actor selector uses the "FromInput" actor filter. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings", meta = (EditCondition = "ActorFilter==EPCGActorFilter::FromInput", EditConditionHides))
	FPCGAttributePropertyInputSelector ActorReferenceSelector;

	/** If true processes all matching actors, otherwise returns data from first match. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings", meta = (EditCondition = "bShowSelectMultiple && ActorFilter==EPCGActorFilter::AllWorldActors && ActorSelection!=EPCGActorSelection::ByName", EditConditionHides))
	bool bSelectMultiple = false;

	/** If true, ignores results found from within this actor's hierarchy. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor Selector Settings", meta = (EditCondition = "bShowIgnoreSelfAndChildren && ActorFilter==EPCGActorFilter::AllWorldActors", EditConditionHides))
	bool bIgnoreSelfAndChildren = false;

	// Properties used to hide some fields when used in different contexts
	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bShowActorFilter = true;

	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bShowIncludeChildren = true;

	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bShowActorSelection = true;

	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bShowActorSelectionClass = true;

	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bShowSelectMultiple = true;

	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bShowIgnoreSelfAndChildren = true;

#if WITH_EDITOR
	UE_API FText GetTaskName() const;
	UE_API FText GetTaskNameSuffix() const;
#endif

	UE_API FPCGSelectionKey GetAssociatedKey() const;
	static UE_API FPCGActorSelectorSettings ReconstructFromKey(const FPCGSelectionKey& InKey);

	UE_API void PrepareForFiltering(bool bForce = false) const;
	UE_API bool MatchesTag(AActor* Actor) const;

protected:
	mutable FString ActorSelectionTagString;
	mutable bool bTagContainsWildcards = false;
	mutable bool bHasPreparedTag = false;
};

USTRUCT(BlueprintType)
struct FPCGComponentSelectorSettings
{
	GENERATED_BODY()

	/** How to select when filtering actors. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Component Selector Settings", meta = (EditCondition = "bShowComponentSelection", EditConditionHides, HideEditConditionToggle))
	EPCGComponentSelection ComponentSelection = EPCGComponentSelection::ByTag;

	/** Tag to match against when filtering actors. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Component Selector Settings", meta = (EditCondition = "bShowComponentSelection && ComponentSelection==EPCGComponentSelection::ByTag", EditConditionHides))
	FName ComponentSelectionTag;

	/** Actor class to match against when filtering actors. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Component Selector Settings", meta = (EditCondition = "bShowComponentSelection && bShowComponentSelectionClass && ComponentSelection==EPCGComponentSelection::ByClass", EditConditionHides, AllowAbstract = "true"))
	TSubclassOf<UActorComponent> ComponentSelectionClass;

	UPROPERTY(Transient)
	bool bShowComponentSelection = true;

	UPROPERTY(Transient)
	bool bShowComponentSelectionClass = true;

	TArray<UActorComponent*> ComponentList;

	UE_API bool FilterComponent(UActorComponent* InComponent) const;
	UE_API bool FilterActor(AActor* InActor) const;
	UE_API TArray<UActorComponent*> FilterComponents(TArrayView<UActorComponent*> InComponents) const;

	UE_API void PrepareForFiltering(bool bForce = false) const;

protected:
	mutable FString ComponentSelectionTagString;
	mutable bool bTagContainsWildcards = false;
	mutable bool bHasPreparedTag = false;
};

namespace PCGActorSelector
{
	// Simple actor filtering
	PCG_API TArray<AActor*> FindActors(const FPCGActorSelectorSettings& Settings, const IPCGGraphExecutionSource* InExecutionSource, const TFunction<bool(const AActor*)>& BoundsCheck, const TFunction<bool(const AActor*)>& SelfIgnoreCheck, TArrayView<AActor*> InputActors = TArrayView<AActor*>());
	PCG_API AActor* FindActor(const FPCGActorSelectorSettings& InSettings, const IPCGGraphExecutionSource* InExecutionSource, const TFunction<bool(const AActor*)>& BoundsCheck, const TFunction<bool(const AActor*)>& SelfIgnoreCheck, TArrayView<AActor*> InputActors = TArrayView<AActor*>());

	// Actor + Component filtering
	PCG_API TArray<AActor*> FindActors(const FPCGActorSelectorSettings* ActorSettings, const FPCGComponentSelectorSettings* ComponentSettings, const IPCGGraphExecutionSource* InExecutionSource, const TFunction<bool(const AActor*)>& BoundsCheck, const TFunction<bool(const AActor*)>& SelfIgnoreCheck, TArrayView<AActor*> InputActors = TArrayView<AActor*>());
	
	// Additional filter methods
	PCG_API TArray<AActor*> FilterActors(const FPCGComponentSelectorSettings& ComponentSettings, TArrayView<AActor*> InActors);
}

#undef UE_API
