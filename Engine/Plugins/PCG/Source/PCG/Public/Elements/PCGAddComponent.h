// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Metadata/PCGObjectPropertyOverride.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Components/ActorComponent.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"

#include "PCGAddComponent.generated.h"

namespace PCGAddComponentConstants
{
	const FName ComponentReferenceAttribute = TEXT("ComponentReference");
	const FName TargetActorPinLabel = TEXT("TargetActor");
}

/**
* Creates components and adds them to specified actors.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGAddComponentSettings : public UPCGSettings
{
	GENERATED_BODY()

	friend class FPCGAddComponentElement;
public:
	UPCGAddComponentSettings(const FObjectInitializer& ObjectInitializer);

	//~Begin UObject interface
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
#endif
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("AddComponent")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGAddComponentElement", "NodeTitle", "Add Component"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGAddComponentElement", "NodeTooltip", "Adds component(s) to specified target actor(s)."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	PCG_API void SetTemplateComponentClass(const TSubclassOf<UActorComponent>& InTemplateComponentClass);
	PCG_API void SetAllowTemplateComponentEditing(bool bInAllowTemplateComponentEditing);
	const TSubclassOf<UActorComponent>& GetTemplateComponentClass() const { return TemplateComponentClass; }
	bool GetAllowTemplateComponentEditing() const { return bAllowTemplateComponentEditing; }

private:
#if WITH_EDITOR
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances);
	void SetupOnBlueprintChangedEvent();
	void TeardownOnBlueprintChangedEvent();
	void RefreshTemplateComponent();
#endif

public:
	/** Controls whether component class selection will be done by attribute or from a constant defined in this node. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bUseClassAttribute = false;

	/** Specifies component class selection */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition="bUseClassAttribute", EditConditionHides))
	FPCGAttributePropertyInputSelector ClassAttribute;

protected:
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (EditCondition = "!bUseClassAttribute", EditConditionHides))
	TSubclassOf<UActorComponent> TemplateComponentClass;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bUseClassAttribute", EditConditionHides))
	bool bAllowTemplateComponentEditing = false;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Instanced, Category = Settings, meta = (ShowInnerProperties, EditCondition = "!bUseClassAttribute && bAllowTemplateComponentEditing", EditConditionHides))
	TObjectPtr<UActorComponent> TemplateComponent;

	/** Specifies what attribute is used to derive actor reference */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition="CanEditActorReference()", PCG_Overridable))
	FPCGAttributePropertyInputSelector ActorReferenceAttribute;

	/** Specifies what attribute to write the component reference to. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyOutputNoSourceSelector ComponentReferenceAttribute;

	/** Overrides to apply on the spawned component. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings")
	TArray<FPCGObjectPropertyOverrideDescription> PropertyOverrideDescriptions;

protected:
#if WITH_EDITOR
	UFUNCTION()
	bool CanEditActorReference() const;
#endif // WITH_EDITOR
};

struct FPCGAddComponentContext : public FPCGContext
{
	int32 CurrentInputIndex = 0;
};

class FPCGAddComponentElement : public IPCGElementWithCustomContext<FPCGAddComponentContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }

private:
	void AddComponents(FPCGAddComponentContext* Context,
		const UPCGData* OriginalData,
		const UPCGAddComponentSettings* Settings,
		const IPCGAttributeAccessor* TransformAccessor,
		const IPCGAttributeAccessorKeys* TransformKeys,
		const IPCGAttributeAccessor* ClassAccessor,
		const IPCGAttributeAccessorKeys* ClassKeys,
		const IPCGAttributeAccessor* TargetActorAccessor,
		const IPCGAttributeAccessorKeys* TargetActorKeys,
		IPCGAttributeAccessor* OutputComponentAccessor,
		IPCGAttributeAccessorKeys* OutputComponentKeys) const;
};
