// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGAddTag.generated.h"

/**
 * Applies the specified tags on the output data.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGAddTagSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;

	virtual FName GetDefaultNodeName() const override { return FName(TEXT("AddTags")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGAddTagElement", "NodeTitle", "Add Tags"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGAddTagElement", "NodeTooltip", "Applies the specified tags on the output data."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

public:
	/** Comma-separated list of tags to apply to the node. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	FString TagsToAdd;

	/** Common prefix to add to all tags, can be left empty. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	FString Prefix;

	/** Common suffix to add to all tags, can be left empty. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	FString Suffix;

	/** Controls whether tags are not considered to be key-value pairs, e.g. that the prefix/suffix will be added before the ':' (if any) or not. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	bool bIgnoreTagValueParsing = false;

	/** Enables deprecated behavior using spaces as separators. Disable to update the node to current behavior. */
	UE_DEPRECATED(5.5, "bTokenizeOnWhiteSpace has been deprecated.")
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = "bTokenizeOnWhiteSpace", EditConditionHides, DeprecationMessage = "bTokenizeOnWhiteSpace has been deprecated."))
	bool bTokenizeOnWhiteSpace = false;
};

class FPCGAddTagElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
