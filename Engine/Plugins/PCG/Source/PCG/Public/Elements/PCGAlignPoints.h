// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGAlignPoints.generated.h"

namespace PCGAlignPointsConstants
{
	const FName SourcePointsLabel = TEXT("Source");
	const FName TargetPointsLabel = TEXT("Target");
}

/** Controls which edge of a bounding box is used as the reference value on a given axis. */
UENUM(BlueprintType)
enum class EPCGAlignPointsAxisReferential : uint8
{
	Min,
	Max,
	Center
};

/** Defines which coordinate frame is used to compute bounding box extents for alignment. */
UENUM(BlueprintType)
enum class EPCGAlignPointsSpatialReferential : uint8
{
	/** Express bounds along the target point's local axes. */
	Target,
	/** Align source to target's rotation, then affect using the target referential. */
	LocalToTarget,
	/** Express bounds along the source point's local axes. */
	Source,
	/** Express bounds along world-space axes (standard AABB). */
	World
};

/** Settings controlling alignment along a single axis. */
USTRUCT(BlueprintType)
struct FPCGAlignPointsAxisSettings
{
	GENERATED_BODY()

	/** Whether to apply alignment on this axis. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "")
	bool bApplyToAxis = false;

	/** Which edge of the source point's bounding box to align from. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "", meta = (EditCondition = "bApplyToAxis", EditConditionHides))
	EPCGAlignPointsAxisReferential SourceReferential = EPCGAlignPointsAxisReferential::Center;

	/** Which edge of the target point's bounding box to align to. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "", meta = (EditCondition = "bApplyToAxis", EditConditionHides))
	EPCGAlignPointsAxisReferential TargetReferential = EPCGAlignPointsAxisReferential::Center;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAlignPointsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("AlignPoints")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/**
	 * Coordinate frame used to compute bounding box extents when determining alignment values.
	 * Target: bounds are measured along the target point's local axes.
	 * LocalToTarget: source point is rotated like the target point, then bounds are measured along the tàrget point's local axes.
	 * Source: bounds are measured along the source point's own local axes.
	 * World:  bounds are measured along world-aligned axes (standard AABB).
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGAlignPointsSpatialReferential SpatialReferential = EPCGAlignPointsSpatialReferential::World;

	/** Alignment settings along the X axis of the chosen spatial referential. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable))
	FPCGAlignPointsAxisSettings XAxis;

	/** Alignment settings along the Y axis of the chosen spatial referential. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable))
	FPCGAlignPointsAxisSettings YAxis;

	/** Alignment settings along the Z axis of the chosen spatial referential. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable))
	FPCGAlignPointsAxisSettings ZAxis;
};

class FPCGAlignPointsElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
