// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/PCGTimeSlicedElementBase.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/Accessors/PCGPropertyAccessor.h"

#include "PCGApplyHierarchy.generated.h"

UENUM(BlueprintType)
enum class EPCGApplyHierarchyOption : uint8
{
	Always,
	Never,
	OptInByAttribute,
	OptOutByAttribute
};

/** Applies hierarchy transformations based on a hierarchy depth, point index & parent index scheme.
* This is used in the context of PCG Data Assets that have these fields by default.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGApplyHierarchySettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGApplyHierarchySettings();

	//~Begin UPCGSettings interface
public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ApplyHierarchy")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGApplyHierarchyElement", "NodeTitle", "Apply Hierarchy"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Attributes that constitute a unique key representing the point. All attributes must be int32 at this time. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TArray<FPCGAttributePropertyInputSelector> PointKeyAttributes;

	/** Attributes that constitute a unique key representing the point's parent in the hierarchy. All attributes must be int32 at this time. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TArray<FPCGAttributePropertyInputSelector> ParentKeyAttributes;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector HierarchyDepthAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector RelativeTransformAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGApplyHierarchyOption ApplyParentRotation;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "ApplyParentRotation==EPCGApplyHierarchyOption::OptInByAttribute||ApplyParentRotation==EPCGApplyHierarchyOption::OptOutByAttribute", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector ApplyParentRotationAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGApplyHierarchyOption ApplyParentScale;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "ApplyParentScale==EPCGApplyHierarchyOption::OptInByAttribute||ApplyParentScale==EPCGApplyHierarchyOption::OptOutByAttribute", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector ApplyParentScaleAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Advanced, meta = (PCG_Overridable))
	EPCGApplyHierarchyOption ApplyHierarchy;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Advanced, meta = (EditCondition = "ApplyHierarchy==EPCGApplyHierarchyOption::OptInByAttribute||ApplyHierarchy==EPCGApplyHierarchyOption::OptOutByAttribute", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector ApplyHierarchyAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Advanced, meta = (PCG_Overridable))
	bool bWarnOnPointsWithInvalidParent = true;
};

namespace PCGApplyHierarchyElement
{
	struct IterationState
	{
		const UPCGBasePointData* InputData = nullptr;
		UPCGBasePointData* OutputData = nullptr;
		int OutputDataIndex = INDEX_NONE;
		
		TArray<TUniquePtr<const IPCGAttributeAccessor>> PointIndexAccessors;
		TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> PointIndexKeys;

		TArray<TUniquePtr<const IPCGAttributeAccessor>> ParentIndexAccessors;
		TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> ParentIndexKeys;

		FPCGAttributePropertyInputSelector HierarchyDepthSelector;
		TUniquePtr<const IPCGAttributeAccessor> HierarchyDepthAccessor;
		TUniquePtr<const IPCGAttributeAccessorKeys> HierarchyDepthKeys;
		
		TUniquePtr<const IPCGAttributeAccessor> RelativeTransformAccessor;
		TUniquePtr<const IPCGAttributeAccessorKeys> RelativeTransformKeys;
		
		TUniquePtr<const IPCGAttributeAccessor> ApplyRotationAccessor;
		bool bInvertApplyRotation = false;

		TUniquePtr<const IPCGAttributeAccessor> ApplyScaleAccessor;
		bool bInvertApplyScale = false;

		TUniquePtr<const IPCGAttributeAccessor> ApplyHierarchyAccessor;
		bool bInvertApplyHierarchy = false;
		
		// Built during execution
		TArray<int32> ParentIndices;
		TArray<TArray<int32>> HierarchyPartition;
		TArray<int32> HierarchyPartitionOrder;

		// Execution state variables
		bool bParentMappingDone = false;
		bool bHierarchyDepthPartitionDone = false;
		int CurrentDepth = 0;

		bool bHasPointsWithInvalidParent = false;
	};
}

class FPCGApplyHierarchyElement : public TPCGTimeSlicedElementBase<PCGTimeSlice::FEmptyStruct, PCGApplyHierarchyElement::IterationState>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};