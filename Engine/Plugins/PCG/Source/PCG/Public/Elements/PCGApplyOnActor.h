// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/PCGLoadObjectsContext.h"
#include "Metadata/PCGObjectPropertyOverride.h"

#include "PCGApplyOnActor.generated.h"

class AActor;

/**
* Apply property overrides and executes functions on a target actor.
*/
UCLASS(BlueprintType)
class UPCGApplyOnActorSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ApplyOnActor")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGApplyOnActorElement", "NodeTitle", "Apply On Object"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGApplyOnActorElement", "NodeTooltip", "Applies property overrides and executes functions on a target object."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** If something is connected in the In pin, it will look for this attribute values to load, representing the object reference. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector ObjectReferenceAttribute;

#if WITH_EDITORONLY_DATA
	/** Provides a single target actor to apply on, used only if the 'In' pin is not connected. */
	UE_DEPRECATED(5.5, "TargetActor has been deprecated; pass data directly to the 'In' pin instead.")
	UPROPERTY(BlueprintReadWrite, Category = Settings, meta = (DeprecatedProperty, DeprecationMessage = "TargetActor has been deprecated; pass data directly to the 'In' pin instead."))
	TSoftObjectPtr<AActor> TargetActor;
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, DisplayName="Tags to add on actors/components"))
	TArray<FName> TagsToAddOnActors;

	/** Override the default property values on the target actor. Applied before post-process functions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TArray<FPCGObjectPropertyOverrideDescription> PropertyOverrideDescriptions;

	/** Specify a list of functions to be called on the target actor. Functions need to be parameter-less and with "CallInEditor" flag enabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FName> PostProcessFunctionNames;

	/**
	 * In case of container override, allow for each element in input, to use the full override data in the Property Overrides pin.
	 * In that case it is expected to have as many data in the Property Overrides pin as there are elements in the In pin.
	 * (it's also allowed to have just a single data in Property Overrides, in that case, this data will be used for all the elements in the In pin)
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bUseOneOverrideDataPerElement = false;

	/** Opt-in option to silence errors when the path is Empty or nothing to extract. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bSilenceErrorOnEmptyObjectPath = false;

#if WITH_EDITORONLY_DATA
	/** Overrides can propagate the change to the object if it is necessary. May trigger expensive downstream computation or infinite refresh loop if PCG listens to changes to this object. Only works in Editor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", AdvancedDisplay)
	bool bPropagateObjectChangeEvent = false;
#endif // WITH_EDITORONLY_DATA

	/** By default, object loading is asynchronous, can force it synchronous if needed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

class FPCGApplyOnObjectElement : public IPCGElementWithCustomContext<FPCGLoadObjectsFromPathContext>
{
public:
	// Calling a function on an actor might not be threadsafe, so taking the safe approach.
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
