// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGSettings.h"
#include "Elements/PCGPointOperationElementBase.h"

#include "PCGApplyScaleToBounds.generated.h"

/**
 * Applies the scale of each point to its bounds and resets the scale.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGApplyScaleToBoundsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ApplyScaleToBounds")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGApplyScaleToBoundsElement", "NodeTitle", "Apply Scale To Bounds"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGApplyScaleToBoundsElement", "NodeTooltip", "Applies the scale of each point to its bounds and resets the scale."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGApplyScaleToBoundsElement : public FPCGPointOperationElementBase
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGPointNativeProperties GetPropertiesToAllocate(FPCGContext* InContext) const override;
	virtual bool ShouldCopyPoints() const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};