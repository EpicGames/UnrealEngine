// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Elements/PCGTimeSlicedElementBase.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/Accessors/PCGPropertyAccessor.h"

#include "PCGAttractElement.generated.h"

class UPCGBasePointData;
class UPCGPointData;

UENUM(BlueprintType)
enum class EPCGAttractMode : uint8
{
	Closest UMETA(Tooltip = "Attract to nearest target."),
	MinAttribute UMETA(Tooltip = "Attract to the minimum target inside the search radius."),
	MaxAttribute UMETA(Tooltip = "Attract to the maximum target inside the search radius."),
	FromIndex UMETA(Tooltip = "Attracts to a fixed target point based on an attribute.")
};

/** Attracts points (interpolates) from the source towards points from the target. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAttractSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGAttractSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("AttractElement")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGAttractElement", "NodeTitle", "Attract"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Controls the criteria used for the attract operation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	EPCGAttractMode Mode = EPCGAttractMode::Closest;

	/** Index attribute on the source that maps a point to a point from the target. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "Mode == EPCGAttractMode::FromIndex", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector AttractorIndexAttribute;

	/** Will be used to determine which points to attract. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "Mode != EPCGAttractMode::FromIndex", ClampMin = "0.01", PCG_Overridable))
	double Distance = 100.0;

	/** Can optionally remove points that weren't attracted to points on the target. Will have no effect when the source is the target. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	bool bRemoveUnattractedPoints = false;

	/** The target attribute used when attracted by attribute value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "Mode == EPCGAttractMode::MinAttribute || Mode == EPCGAttractMode::MaxAttribute", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector TargetAttribute;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (InlineEditConditionToggle, PCG_Overridable))
	bool bUseSourceWeight = false;

	/** This attribute will determine the weight of the fusion result for the source point. It will be normalized to the range of [0..1]. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bUseSourceWeight", PCG_Overridable))
	FPCGAttributePropertyInputSelector SourceWeightAttribute;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (InlineEditConditionToggle, PCG_Overridable))
	bool bUseTargetWeight = false;

	/** This attribute will determine the weight of the fusion result for the target point. It will be normalized to the range of [0..1]. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bUseTargetWeight", PCG_Overridable))
	FPCGAttributePropertyInputSelector TargetWeightAttribute;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "-1.0", ClampMax = "1.0", EditCondition = "!bUseSourceWeight && !bUseTargetWeight", PCG_Overridable))
	double Weight = 1.0;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TMap<FPCGAttributePropertyInputSelector, FPCGAttributePropertyInputSelector> SourceAndTargetAttributeMapping;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (InlineEditConditionToggle, PCG_Overridable))
	bool bOutputAttractIndex = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bOutputAttractIndex", PCG_Overridable))
	FPCGAttributePropertyOutputNoSourceSelector OutputAttractIndexAttribute;
};

namespace PCGAttractElement
{
	struct FAttractState
	{
		// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FAttractState() = default;
		~FAttractState() = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		const UPCGBasePointData* TargetData = nullptr;
		const UPCGBasePointData* SourceData = nullptr;
		UPCGBasePointData* OutputData = nullptr;

		UE_DEPRECATED(5.6, "Use TargetData instead")
		const UPCGPointData* TargetPointData = nullptr;
		UE_DEPRECATED(5.6, "Use SourceData instead")
		const UPCGPointData* SourcePointData = nullptr;
		UE_DEPRECATED(5.6, "Use OutputData instead")
		UPCGPointData* OutPointData = nullptr;

		TArray<int32> Mapping;
		TArray<int32> SortedTargetIndices;

		TUniquePtr<IPCGAttributeAccessorKeys> OutputKeys;
		TUniquePtr<IPCGAttributeAccessor> OutputAttractIndexAccessor;

		TArray<double> SourceWeights;
		TArray<double> TargetWeights;

		struct FAttributeInterpolationData
		{
			TUniquePtr<const IPCGAttributeAccessor> SourceAccessor;
			TUniquePtr<const IPCGAttributeAccessorKeys> SourceKeys;
			TUniquePtr<const IPCGAttributeAccessor> TargetAccessor;
			TUniquePtr<const IPCGAttributeAccessorKeys> TargetKeys;

			FPCGAttributePropertyOutputSelector OutputSelector;
			TUniquePtr<IPCGAttributeAccessor> OutputAccessor;
		};

		TArray<FAttributeInterpolationData> Interpolators;

		int32 IterationIndex = INDEX_NONE;
		bool bTargetIsSource = false;
		bool bAttractPhaseDone = false;
		int32 InterpolationIndex = 0;
	};

	struct FExecState
	{
		using AttractSignature = bool(*)(const FPCGContext* InContext, FAttractState& AttractData, const UPCGAttractSettings* AttractSettings);
		AttractSignature AttractFunction = nullptr;
	};
}

class FPCGAttractElement : public TPCGTimeSlicedElementBase<PCGAttractElement::FExecState, PCGAttractElement::FAttractState>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
