// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.6, "Use /Elements/Metadata/PCGAttributeCast.h instead of /Elements/PCGAttributeCast.h")

#include "Metadata/PCGAttributeCast.h"