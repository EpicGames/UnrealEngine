// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataTypesConstantStruct.h"

#include "PCGAttributeFilter.generated.h"

UENUM(BlueprintType)
enum class EPCGAttributeFilterOperator : uint8
{
	Greater UMETA(DisplayName=">"),
	GreaterOrEqual UMETA(DisplayName=">="),
	Lesser UMETA(DisplayName="<"),
	LesserOrEqual UMETA(DisplayName="<="),
	Equal UMETA(DisplayName="="),
	NotEqual UMETA(DisplayName="!="),
	InRange UMETA(Hidden),
	Substring,
	Matches
};

USTRUCT(BlueprintType)
struct FPCGAttributeFilterThresholdSettings
{
	GENERATED_BODY()

#if WITH_EDITOR
	PCG_API void OnPostLoad();
#endif

	/** If the threshold in included or excluded from the range. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bInclusive = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseConstantThreshold = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bUseConstantThreshold", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector ThresholdAttribute;

	// This value is now false by default (changed in the ctor of the filtering settings)
	/** For Point Data, enabling this option will use sampling rather than comparing points 1 to 1 directly. For other spatial data, this is always true, and for attribute sets, always false. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bUseConstantThreshold", EditConditionHides, PCG_Overridable))
	bool bUseSpatialQuery = true;
	
	/** If there are multiple filtering data, do a N:N filtering (Input 0 with filter 0, Input 1 with filter 1, etc...). Otherwise, only use the first data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bUseConstantThreshold", EditConditionHides))
	bool bFilterEachInputWithEachFilterDataRespectively = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bUseConstantThreshold", EditConditionHides, ShowOnlyInnerProperties, DisplayAfter = "bUseConstantThreshold", PCG_NotOverridable))
	FPCGMetadataTypesConstantStruct AttributeTypes;
};

/**
* Filter elements by attribute that allows to do "A op B" type filtering, where A is the input spatial data or Attribute set,
* and B is either a constant, another spatial data (if input is a spatial data), an Attribute set (in filter) or the input itself.
* The filtering can be done either on properties or attributes.
* Some examples:
* - Threshold on property by constant (A.Density > 0.5)
* - Threshold on attribute by constant (A.aaa != "bob")
* - Threshold on property by metadata attribute(A.density >= B.bbb)
* - Threshold on property by property(A.density <= B.steepness)
* - Threshold on attribute by metadata attribute(A.aaa < B.bbb)
* - Threshold on attribute by property(A.aaa == B.color)
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAttributeFilteringSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGAttributeFilteringSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("AttributeFilter")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGAttributeFilteringElement", "NodeTitle", "Filter Attribute Elements"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Filter; }

	// Expose 2 nodes: Attribute filter and Point Filter that will not have the same defaults.
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
	virtual bool GroupPreconfiguredSettings() const override { return false; }
#endif
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;
	virtual FString GetAdditionalTitleInformation() const override;
	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGAttributeFilterOperator Operator = EPCGAttributeFilterOperator::Greater;

	/** Target property/attribute related properties */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector TargetAttribute;

	/** Threshold property/attribute/constant related properties */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseConstantThreshold = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bUseConstantThreshold", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector ThresholdAttribute;

	// This value is now false by default (changed in the ctor)
	/** If the threshold data is Point data, it will sample input points in threshold data. Always true with Spatial data.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bUseConstantThreshold", EditConditionHides, PCG_Overridable))
	bool bUseSpatialQuery = true;
	
	/** If there is multiple filtering data, do a N:N filtering (Input 0 with filter 0, Input 1 with filter 1, etc...). Otherwise, only use the first data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bUseConstantThreshold", EditConditionHides))
	bool bFilterEachInputWithEachFilterDataRespectively = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bUseConstantThreshold", EditConditionHides, ShowOnlyInnerProperties, DisplayAfter = "bUseConstantThreshold", PCG_NotOverridable))
	FPCGMetadataTypesConstantStruct AttributeTypes;

	/** Controls whether the node will emit a warning when the input data or the filter data doesn't have the attribute to filter on. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bWarnOnDataMissingAttribute = true;

	/** Always generate output data (possibly empty) for both the in and out filters, even if all/none of the elements were filtered. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bGenerateOutputDataEvenIfEmpty = true;

	// Hidden value to indicate that Spatial -> Point deprecation is on where pins are not explicitly points.
	UPROPERTY()
	bool bHasSpatialToPointDeprecation = false;
};


/**
* Attribute filter on range that allows to do "A op B" type filtering, where A is the input spatial data or Attribute set,
* and B is either a constant, another spatial data (if input is a spatial data), an Attribute set (in filter) or the input itself.
* The filtering can be done either on properties or attributes.
* Some examples (that might not make sense, but are valid):
* - Threshold on property by constant (A.Density in [0.2, 0.5])
* - Threshold on attribute by constant (A.aaa in [0.4, 0.6])
* - Threshold on property by metadata attribute(A.density in [B.bbmin, B.bbmax])
* - Threshold on property by property(A.density in [B.position.x, B.steepness])
* - Threshold on attribute by metadata attribute(A.aaa in [B.bbmin, B.bbmax])
* - Threshold on attribute by property(A.aaa in [B.position, B.scale])
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAttributeFilteringRangeSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGAttributeFilteringRangeSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("AttributeFilterRange")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGAttributeFilteringElement", "NodeTitleRange", "Filter Attribute Elements by Range"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Filter; }

	// Expose 2 nodes: "Attribute Filter Range" and "Point Filter Range" that will not have the same defaults.
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
	virtual bool GroupPreconfiguredSettings() const override { return false; }
#endif
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;
	virtual FString GetAdditionalTitleInformation() const override;
	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Target property/attribute related properties */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector TargetAttribute;

	/** Threshold property/attribute/constant related properties */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributeFilterThresholdSettings MinThreshold;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributeFilterThresholdSettings MaxThreshold;

	/** Controls whether the node will emit a warning when the input data or the filter data doesn't have the attribute to filter on. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bWarnOnDataMissingAttribute = true;

	/** Always generate output data (possibly empty) for both the in and out filters, even if all/none of the elements were filtered. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bGenerateOutputDataEvenIfEmpty = true;

	// Hidden value to indicate that Spatial -> Point deprecation is on where pins are not explicitly points.
	UPROPERTY()
	bool bHasSpatialToPointDeprecation = false;
};

class FPCGAttributeFilterElementBase : public IPCGElement
{
protected:
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }

	/** Performs the filter operation. Returns true when the operation is complete. */
	bool DoFiltering(FPCGContext* Context, EPCGAttributeFilterOperator InOperation, const FPCGAttributePropertyInputSelector& InTargetAttribute, bool bHasSpatialToPointDeprecation, bool bWarnOnDataMissingAttribute, bool bAlwaysGenerateOutputData, const FPCGAttributeFilterThresholdSettings& FirstThreshold, const FPCGAttributeFilterThresholdSettings* SecondThreshold = nullptr) const;
};

class FPCGAttributeFilterElement : public FPCGAttributeFilterElementBase
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};

class FPCGAttributeFilterRangeElement : public FPCGAttributeFilterElementBase
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};

namespace PCGAttributeFilterConstants
{
	const FName FilterLabel = TEXT("Filter");
	const FName FilterMinLabel = TEXT("FilterMin");
	const FName FilterMaxLabel = TEXT("FilterMax");
}

namespace PCGAttributeFilterHelpers
{
	template <typename T>
	bool ApplyCompare(const T& Input1, const T& Input2, EPCGAttributeFilterOperator Operation)
	{
		if (Operation == EPCGAttributeFilterOperator::Equal)
		{
			return PCG::Private::MetadataTraits<T>::Equal(Input1, Input2);
		}
		else if (Operation == EPCGAttributeFilterOperator::NotEqual)
		{
			return !PCG::Private::MetadataTraits<T>::Equal(Input1, Input2);
		}

		if constexpr (PCG::Private::MetadataTraits<T>::CanCompare)
		{
			switch (Operation)
			{
			case EPCGAttributeFilterOperator::Greater:
				return PCG::Private::MetadataTraits<T>::Greater(Input1, Input2);
			case EPCGAttributeFilterOperator::GreaterOrEqual:
				return PCG::Private::MetadataTraits<T>::GreaterOrEqual(Input1, Input2);
			case EPCGAttributeFilterOperator::Lesser:
				return PCG::Private::MetadataTraits<T>::Less(Input1, Input2);
			case EPCGAttributeFilterOperator::LesserOrEqual:
				return PCG::Private::MetadataTraits<T>::LessOrEqual(Input1, Input2);
			default:
				break;
			}
		}

		if constexpr (PCG::Private::MetadataTraits<T>::CanSearchString)
		{
			switch (Operation)
			{
			case EPCGAttributeFilterOperator::Substring:
				return PCG::Private::MetadataTraits<T>::Substring(Input1, Input2);
			case EPCGAttributeFilterOperator::Matches:
				return PCG::Private::MetadataTraits<T>::Matches(Input1, Input2);
			default:
				break;
			}
		}

		return false;
	}

	template <typename T>
	bool ApplyRange(const T& Input, const T& InMin, const T& InMax, bool bMinIncluded, bool bMaxIncluded)
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanCompare)
		{
			return (bMinIncluded ? PCG::Private::MetadataTraits<T>::GreaterOrEqual(Input, InMin) : PCG::Private::MetadataTraits<T>::Greater(Input, InMin)) &&
				(bMaxIncluded ? PCG::Private::MetadataTraits<T>::LessOrEqual(Input, InMax) : PCG::Private::MetadataTraits<T>::Less(Input, InMax));
		}
		else
		{
			return false;
		}
	}
}
