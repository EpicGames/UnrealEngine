// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataAttribute.h"

#include "PCGAttributeGetFromPointIndexElement.generated.h"

namespace PCGAttributeGetFromPointIndexConstants
{
	const FName OutputAttributeLabel = TEXT("Attribute");
	const FName OutputPointLabel = TEXT("Point");
}

/**
* Get the attribute/property of a point given its index. The result will be in a ParamData.
* There is also a second output that will output the selected point. This point will be output
* even if the property/attribute doesn't exist.
* 
* The Index can be overridden by a second Params input.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAttributeGetFromPointIndexSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGAttributeGetFromPointIndexSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	int32 Index = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyOutputSelector OutputAttributeName;

	/** For deprecation, old node was discarding the output domain and was always writing to the Elements domain. This will be false for new nodes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Deprecation", meta = (EditCondition = "bForceOutputAttributeToBeInElementsDomain", EditConditionHides))
	bool bForceOutputAttributeToBeInElementsDomain = true;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName InputAttributeName_DEPRECATED = NAME_None;
#endif // WITH_EDITORONLY_DATA
};


class FPCGAttributeGetFromPointIndexElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
