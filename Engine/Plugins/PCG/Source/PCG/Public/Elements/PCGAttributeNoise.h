// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Templates/UniquePtr.h"

#include "PCGAttributeNoise.generated.h"

UENUM(BlueprintType)
enum class EPCGAttributeNoiseMode : uint8
{
	Set,
	Minimum,
	Maximum,
	Add,
	Multiply
};

/**
* Apply some noise to an attribute/property. You can select the mode you want and a noise range.
* Support all numerical types and vectors/rotators.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAttributeNoiseSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGAttributeNoiseSettings();

	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins);
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("AttributeNoise")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGAttributeNoiseSettings", "NodeTitle", "Attribute Noise"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }

	// Expose 2 nodes: Density noise and Attribute Noise that will not have the same defaults.
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
	virtual bool GroupPreconfiguredSettings() const override { return false; }
#endif
	virtual bool UseSeed() const override { return true; }
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;
	virtual bool HasDynamicPins() const override { return true; }
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyOutputSelector OutputTarget;

	/** Attribute = (Original op Noise), Noise in [NoiseMin, NoiseMax] */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases = "DensityMode"))
	EPCGAttributeNoiseMode Mode = EPCGAttributeNoiseMode::Set;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases = "DensityNoiseMin"))
	float NoiseMin = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases = "DensityNoiseMax"))
	float NoiseMax = 1.f;

	/** Attribute = 1 - Attribute before applying the operation */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases = "bInvertSourceDensity"))
	bool bInvertSource = false;

	// Clamp the result between 0 and 1. Always applied if we apply noise to the density.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bClampResult = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bHasCustomSeedSource = false;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bHasCustomSeedSource"))
	FPCGAttributePropertyInputSelector CustomSeedSource;

	// Previous names
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	EPCGAttributeNoiseMode DensityMode_DEPRECATED = EPCGAttributeNoiseMode::Set;

	UPROPERTY()
	float DensityNoiseMin_DEPRECATED = 0.f;

	UPROPERTY()
	float DensityNoiseMax_DEPRECATED = 1.f;

	UPROPERTY()
	bool bInvertSourceDensity_DEPRECATED = false;

	UPROPERTY()
	bool bOutputTargetDifferentFromInputSource_DEPRECATED = false;
#endif // WITH_EDITORDATA_ONLY

	// Hidden value to indicate that Spatial -> Point deprecation is on where pins are not explicitly points.
	UPROPERTY()
	bool bHasSpatialToPointDeprecation = false;
};

struct FPCGAttributeNoiseContext : public FPCGContext
{
	int32 CurrentInput = 0;
	bool bDataPreparedForCurrentInput = false;
	bool bInputsGathered = false; 
	TArray<FPCGTaggedData> CachedInputs;
	FPCGAttributePropertyInputSelector InputSource;
	FPCGAttributePropertyInputSelector CustomSeedSource;
	FPCGAttributePropertyOutputSelector OutputTarget;
	TUniquePtr<const IPCGAttributeAccessor> InputAccessor;
	TUniquePtr<const IPCGAttributeAccessor> CustomSeedAccessor;
	TUniquePtr<IPCGAttributeAccessor> OutputAccessor;
	TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys;
	TUniquePtr<const IPCGAttributeAccessorKeys> CustomSeedKeys;
	TUniquePtr<IPCGAttributeAccessorKeys> OutputKeys;
};

class FPCGAttributeNoiseElement : public IPCGElementWithCustomContext<FPCGAttributeNoiseContext>
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
