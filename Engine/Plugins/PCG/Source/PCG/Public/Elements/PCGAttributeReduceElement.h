// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataAttribute.h"

#include "PCGAttributeReduceElement.generated.h"

UENUM()
enum class EPCGAttributeReduceOperation
{
	Average,
	Max,
	Min,
	Sum,
	Join
};

/**
* Take all the entries/points from the input and perform a reduce operation on the given attribute/property
* and output the result into a ParamData.
* Note: Special case for average on Quaternion since they are not trivially averageable. We have a simplistic approximation
* that would be accurate only if the quaternions are close to each other. The accurate version of the average is using eigenvectors/eigenvalues
* which is way more complicated and computationally expensive. Quaternion will also be normatilzed at the end. Beware if you are using this average.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAttributeReduceSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
#endif
	virtual bool HasDynamicPins() const override;

	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfiguredInfo) override;

	virtual FString GetAdditionalTitleInformation() const override;

	bool ShouldMergeOutputAttributes() const { return !bWriteToDataDomain && bMergeOutputAttributes; }

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	/** By default the reduce output the result into an new attribute set, but it can also be written into the data domain of the input. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bWriteToDataDomain = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName OutputAttributeName = NAME_None;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGAttributeReduceOperation Operation = EPCGAttributeReduceOperation::Average;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "Operation==EPCGAttributeReduceOperation::Join", EditConditionHides))
	FString JoinDelimiter = FString(", ");

	/** Option to merge all results into a single attribute set with multiple entries, instead of multiple attribute sets with a single value in them.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bWriteToDataDomain", EditConditionHides))
	bool bMergeOutputAttributes = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName InputAttributeName_DEPRECATED = NAME_None;
#endif // WITH_EDITORONLY_DATA

protected:
	virtual FPCGElementPtr CreateElement() const override;
};


class FPCGAttributeReduceElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
