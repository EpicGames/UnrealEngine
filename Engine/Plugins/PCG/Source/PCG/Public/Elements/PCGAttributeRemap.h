// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.6, "Use /Elements/Metadata/PCGAttributeRemap.h instead of /Elements/PCGAttributeRemap.h")

#include "Metadata/PCGAttributeRemap.h"