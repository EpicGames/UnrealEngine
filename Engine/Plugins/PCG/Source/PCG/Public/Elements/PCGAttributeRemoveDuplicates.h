// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGAttributeRemoveDuplicates.generated.h"

/**
* Remove duplicates for given attributes
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAttributeRemoveDuplicatesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return TEXT("AttributeRemoveDuplicates"); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGAttributeRemoveDuplicatesElement", "NodeTitle", "Attribute Remove Duplicates"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
#endif

	virtual bool HasDynamicPins() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	// TODO: Should be overridable once array override is supported
	/** The data will be partitioned on these selected attributes, and we will only keep the first entry for each partition. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FPCGAttributePropertyInputSelector> AttributeSelectors = { FPCGAttributePropertyInputSelector() };

	// TODO: Should be deprecated once array override is supported
	/** Comma-separated list of attributes to remove duplicates from, such that only the first entry of each partition will be kept. */
	UPROPERTY(meta = (PCG_Overridable))
	FString AttributeNamesToRemoveDuplicates;
};

class FPCGAttributeRemoveDuplicatesElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
