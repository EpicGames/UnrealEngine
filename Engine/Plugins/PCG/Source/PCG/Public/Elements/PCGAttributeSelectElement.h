// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataAttribute.h"

#include "PCGAttributeSelectElement.generated.h"

namespace PCGAttributeSelectConstants
{
	const FName OutputAttributeLabel = TEXT("Attribute");
	const FName OutputPointLabel = TEXT("Point");
}

UENUM()
enum class EPCGAttributeSelectOperation
{
	Min,
	Max,
	Median
};

UENUM()
enum class EPCGAttributeSelectAxis
{
	X,
	Y,
	Z,
	W,
	CustomAxis
};

/**
* Take all the entries/points from the input and perform a select operation on the given attribute/property on the given axis 
* (if the attribute/property is a vector) and output the result into a ParamData.
* It will also output the selected point if the input is a PointData.
* 
* Only support vector attributes and scalar attributes.
* 
* CustomAxis is overridable.
* 
* In case of the median operation, and the number of elements is even, we arbitrarily chose a point (Index = Num / 2)
*
* If the OutputAttributeName is None, we will use InputSource.GetName().
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAttributeSelectSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
#endif
	
	virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName OutputAttributeName = NAME_None;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGAttributeSelectOperation Operation = EPCGAttributeSelectOperation::Min;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGAttributeSelectAxis Axis = EPCGAttributeSelectAxis::X;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Axis == EPCGAttributeSelectAxis::CustomAxis", PCG_Overridable))
	FVector4 CustomAxis = FVector4::Zero();

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName InputAttributeName_DEPRECATED = NAME_None;
#endif // WITH_EDITORONLY_DATA

protected:
	virtual FPCGElementPtr CreateElement() const override;
};


class FPCGAttributeSelectElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
