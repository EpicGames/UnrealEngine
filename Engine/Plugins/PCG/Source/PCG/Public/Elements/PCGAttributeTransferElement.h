// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGCopyAttributes.h"

#include "PCGAttributeTransferElement.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UE_DEPRECATED(5.5, "Use UPCGCopyAttributeSettings") UPCGAttributeTransferSettings : public UPCGCopyAttributesSettings
{
	GENERATED_BODY()

public:
	UPCGAttributeTransferSettings()
		: UPCGCopyAttributesSettings()
	{
#if WITH_EDITOR
		bExposeToLibrary = false;
#endif // WITH_EDITOR

		// Reset default values to the ones of Attribute Transfer, not inherited from CopyAttributes
		InputSource = FPCGAttributePropertyInputSelector();
		OutputTarget = FPCGAttributePropertyOutputSelector();
	}
};
