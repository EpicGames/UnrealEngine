// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Elements/PCGTimeSlicedElementBase.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "PCGBlurElement.generated.h"

class UPCGPointData;

UENUM()
enum class EPCGBlurElementMode
{
	/** Same weight for each point, which will be 1 / N, N being the number of points found in the given distance. */
	Constant,
	/** Weight for Point will be 1 - Dist(Point, Center) / SearchDistance. */
	Linear,
	/** Weight will be a gaussian distribution. */
	Gaussian
};

/**
* Select an attribute on a point data and blur it using the values from neighbors within some distance, center to center, and can be done over multiple iterations.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGBlurSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Attribute to use as a base value. Needs to be numerical. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	/** Attribute where the result will be written. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyOutputSelector OutputTarget;

	/** Number of iterations to apply the blur. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = 1))
	int NumIterations = 1;

	/** Radius for search. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = 0))
	double SearchDistance = 1000.0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGBlurElementMode BlurMode = EPCGBlurElementMode::Constant;

	/** By default, the standard deviation will be SearchDistance / 3, so that at search distance from the point it corresponds to 3 std deviation. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Gaussian", meta = (PCG_Overridable, EditConditionHides, EditCondition = "BlurMode == EPCGBlurElementMode::Gaussian"))
	bool bUseCustomStandardDeviation = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Gaussian", meta = (PCG_Overridable, EditConditionHides, EditCondition = "BlurMode == EPCGBlurElementMode::Gaussian && bUseCustomStandardDeviation"))
	double CustomStandardDeviation = 1.0;
};

struct FPCGBlurIterState
{
	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPCGBlurIterState() = default;
	~FPCGBlurIterState() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TUniquePtr<const IPCGAttributeAccessor> InputAccessor;
	TUniquePtr<IPCGAttributeAccessor> OutputAccessor;
	TUniquePtr<const IPCGAttributeAccessorKeys> InputKeys;
	TUniquePtr<IPCGAttributeAccessorKeys> OutputKeys;

	UE_DEPRECATED(5.6, "Use InputData instead")
	const UPCGPointData* InputPointData = nullptr;

	const UPCGBasePointData* InputData = nullptr;
	
	// Need two working buffers as we are working in parallel. One for read, one for write.
	TArray<uint8> WorkingBuffer1;
	TArray<uint8> WorkingBuffer2;
	int32 CurrentIteration = 0;
};

class FPCGBlurElement : public TPCGTimeSlicedElementBase<PCGTimeSlice::FEmptyStruct, FPCGBlurIterState>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};

