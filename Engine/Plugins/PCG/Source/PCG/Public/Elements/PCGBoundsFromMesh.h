// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "PCGContext.h"
#include "Async/PCGAsyncLoadingContext.h"

#include "PCGBoundsFromMesh.generated.h"

class UStaticMesh;

/**
* Sets the bounds according to the static or skeletal mesh(es) provided in the mesh pin.
*/
UCLASS(MinimalAPI, BlueprintType)
class UPCGBoundsFromMeshSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; };
#endif //WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Attribute from which to source the meshes to use. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_DiscardPropertySelection))
	FPCGAttributePropertyInputSelector MeshAttribute;

	/** Will not produce warnings when the input data does not have the required attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bSilenceAttributeNotFoundErrors = false;

	/** By default, will use async loading for the meshes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

struct FPCGBoundsFromMeshContext : public FPCGContext, public IPCGAsyncLoadingContext
{
	struct InputMeshesData
	{
		int InputIndex = 0;
		TArray<int32> MeshValueIndex;
	};

	TArray<FPCGTaggedData> CachedInputs;
	TArray<InputMeshesData> PerInputData;

	FSoftObjectPath SingleMesh;
	TMap<int, FBox> MeshToBoundsMap;

	TArray<FSoftObjectPath> MeshesToLoad;
	bool bInputsGathered = false;
	bool bPrepareDone = false;
	bool bBoundsQueried = false;
	int CurrentPrepareIndex = 0;
	int CurrentExecuteIndex = 0;
};

class FPCGBoundsFromMeshElement : public IPCGElementWithCustomContext<FPCGBoundsFromMeshContext>
{
public:
	// Loading needs to be done on the main thread and accessing objects outside of PCG might not be thread safe, so taking the safe approach
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
