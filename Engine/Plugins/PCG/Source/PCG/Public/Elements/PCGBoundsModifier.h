// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPointOperationElementBase.h"
#include "PCGSettings.h"

#include "PCGBoundsModifier.generated.h"

UENUM()
enum class EPCGBoundsModifierMode : uint8
{
	Set,
	Intersect,
	Include,
	Translate,
	Scale
};

/**
* This class controls/sets up a node that modifies the min/max bounds of the input points.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGBoundsModifierSettings : public UPCGSettings
{
	GENERATED_BODY()

public:

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("BoundsModifier")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGBoundsModifierSettings", "NodeTitle", "Bounds Modifier"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGBoundsModifierMode Mode = EPCGBoundsModifierMode::Scale;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FVector BoundsMin = FVector::One();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FVector BoundsMax = FVector::One();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bAffectSteepness = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bAffectSteepness", ClampMin = "0", ClampMax = "1", PCG_Overridable))
	float Steepness = 1.0f;
};

class FPCGBoundsModifier : public FPCGPointOperationElementBase
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGPointNativeProperties GetPropertiesToAllocate(FPCGContext* InContext) const override;
	virtual bool ShouldCopyPoints() const override { return true; }
};