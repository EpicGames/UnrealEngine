// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGCleanSpline.generated.h"

class UPCGSplineData;
class UPCGPolygon2DData;

UENUM()
enum class EPCGControlPointFuseMode
{
	KeepFirst UMETA(Tooltip = "If index 0 and 1 are colocated, keep 0."),
	KeepSecond UMETA(Tooltip = "If index 0 and 1 are colocated, keep 1."),
	Merge UMETA(Tooltip = "Fuse the two control points into a new control point, located halfway between the original two."),
	Auto UMETA(Tooltip = "Will generally keep the second of both control points, except keep the first when it would otherwise alter the length of the spline--ie. final control point.")
};

/* Remove superfluous control points along the spline, such as those that are co-located or collinear. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCleanSplineSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CleanSpline")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCleanSplineElement", "NodeTitle", "Clean Spline"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGCleanSplineElement", "NodeTooltip", "Remove superfluous control points along the spline, such as those that are co-located or collinear."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Fuse control points that share the same location in world space, within a distance threshold. Colocated points are inherently collinear, so this will automatically be applied when removing collinear points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (PCG_Overridable))
	bool bFuseColocatedControlPoints = true;

	/** Control points will be considered co-located if they are within this distance from one another. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bFuseColocatedControlPoints", EditConditionHides, PCG_Overridable))
	double ColocationDistanceThreshold = UE_KINDA_SMALL_NUMBER;

	/** Use spline local space for the distance calculation, rather than world space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bFuseColocatedControlPoints", EditConditionHides, PCG_Overridable))
	bool bUseSplineLocalSpace = false;

	/** Controls how two co-located points will be fused together. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bFuseColocatedControlPoints", EditConditionHides, PCG_Overridable))
	EPCGControlPointFuseMode FuseMode = EPCGControlPointFuseMode::Auto;

	/** Remove control points on linear sections of the spline that would otherwise have no effect on the final spline calculation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (PCG_Overridable))
	bool bRemoveCollinearControlPoints = true;

	/** A control point will be considered collinear if it is within this angle from the segment between its previous and next control points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bRemoveCollinearControlPoints", EditConditionHides, PCG_Overridable, ClampMin = "0"))
	double CollinearAngleThreshold = 5;

	/** Use radians directly, instead of degrees. The current 'CollinearAngleThreshold' value will be automatically converted when toggled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bRemoveCollinearControlPoints", EditConditionHides))
	bool bUseRadians = false;
};

class FPCGCleanSplineElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;

private:
	void ProcessSpline(FPCGContext* InContext, const UPCGCleanSplineSettings* Settings, const FPCGTaggedData& InputData, const UPCGSplineData* InputSplineData) const;
	void ProcessPolygon(FPCGContext* InContext, const UPCGCleanSplineSettings* Settings, const FPCGTaggedData& InputData, const UPCGPolygon2DData* InputPolygonData) const;
};
