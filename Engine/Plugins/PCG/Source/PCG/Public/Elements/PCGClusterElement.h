// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Elements/PCGTimeSlicedElementBase.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"

#include "PCGClusterElement.generated.h"

class UPCGPointData;
class UPCGBasePointData;

UENUM()
enum class EPCGClusterAlgorithm
{
	KMeans UMETA(Tooltip = "K-Means - Categorizes points by their distance from a predefined number of continuously updated centroids."),
	EM UMETA(Tooltip = "Expectation-Maximization - Categorizes points by using a Gaussian Mixture Model.")
};

/** Given a desired number of clusters (categories), find the best fit cluster for each point by distance, using one of various clustering algorithms. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGClusterSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGClusterSettings(const FObjectInitializer& ObjectInitializer);

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ClusterElement")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGClusterElement", "NodeTitle", "Cluster"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
	virtual FText GetNodeTooltipText() const override;
#endif
	virtual bool UseSeed() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Mathematical algorithm for selecting clusters. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	EPCGClusterAlgorithm Algorithm = EPCGClusterAlgorithm::KMeans;

	/** Number of clusters (segments) to group the points into. Each point will be assigned a cluster at the end. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, DisplayName = "Clusters", meta = (ClampMin = "1", PCG_Overridable))
	int32 NumClusters = 3;

	/** Cluster IDs will be written to this attribute on the main output. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyOutputNoSourceSelector ClusterAttribute;

	/** A limit on the number of iterations to run on each algorithm, if it doesn't otherwise converge. A higher number can offer more accuracy at the cost of processing time. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (ClampMin = "1", PCG_Overridable))
	int32 MaxIterations = 100;

	/** For EM, the maximum allowed difference between the last two iterations' "Log Likelihood"--which converges from positive infinity to 0 in relation to point-to-cluster probabilities.
	 * It is exponential, so raising this number can offer faster iteration if exact precision isn't needed.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (ClampMin = "0.0001", EditCondition = "Algorithm == EPCGClusterAlgorithm::EM", EditConditionHides, PCG_Overridable))
	double Tolerance = UE_DOUBLE_KINDA_SMALL_NUMBER;

	/** Output the final location of the centroids or gaussians. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay)
	bool bOutputFinalCentroids = false;

	/** Output the element count for each of the final centroids. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (EditCondition = "bOutputFinalCentroids", EditConditionHides, PCG_Overridable))
	bool bOutputFinalCentroidElementCount = false;

	/** Final centroid element count will be written to this attribute on the Final Centroid output. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (EditCondition = "bOutputFinalCentroidElementCount", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyOutputNoSourceSelector FinalCentroidElementCountAttribute;
};

namespace PCGClusterElement
{
	struct FClusteringData
	{
		// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FClusteringData() = default;
		~FClusteringData() = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		UE_DEPRECATED(5.6, "Use OutputData instead")
		UPCGPointData* OutPointData = nullptr;

		UPCGBasePointData* OutputData = nullptr;

		TUniquePtr<IPCGAttributeAccessor> Accessor;
		TUniquePtr<IPCGAttributeAccessorKeys> Keys;

		// Shared cached data
		int32 Iteration = 0;
		// The stage cached for alternating async processing. Stage 0 == Complete.
		uint8 ProcessingStage = 1;
		bool bConverged = false;
		TArray<FVector, TInlineAllocator<16>> Centroids;
		TArray<int32> PointToClusterAssignments;

		// Expectation-Maximization cached data
		struct FExpectationMaximizationData
		{
			struct FGaussian
			{
				FGaussian() = default;

				FGaussian(const FVector& InMean, const FMatrix& InCovariance, const double InWeight) : Mean(InMean), Covariance(InCovariance), Weight(InWeight)
				{}

				FVector Mean = FVector::ZeroVector;
				FMatrix Covariance = FMatrix::Identity;
				FMatrix InvCovariance = FMatrix::Identity;
				double NormalizationFactor = 1.0;
				double Weight = 0.0;
				bool bDead = false;
			};

			struct FProbabilityArray
			{
				void Initialize(const int32 InNumPoints, const int32 InNumCentroids)
				{
					check(InNumPoints > 0 && InNumCentroids > 0);
					NumPoints = InNumPoints;
					NumCentroids = InNumCentroids;
					Array.SetNumZeroed(NumPoints * NumCentroids);
				}

				TArrayView<double> operator[](const int32 Index)
				{
					check(Array.IsValidIndex(Index * NumCentroids));
					return TArrayView<double>(&Array[Index * NumCentroids], NumCentroids);
				}

			private:
				int32 NumPoints = 0;
				int32 NumCentroids = 0;
				TArray<double, TInlineAllocator<1024>> Array;
			};

			TArray<FGaussian, TInlineAllocator<16>> Gaussians;

			FProbabilityArray Probabilities;

			TArray<FVector> NormalizedPointLocations;
			double ScalingFactor = 1.0;
			double Tolerance = UE_DOUBLE_KINDA_SMALL_NUMBER;
			// A value from -inf to some positive finite value to represent how well the model fits the data. It will be used to recognize convergence.
			std::atomic<double> LogLikelihoodSum = 0;
			double PreviousLogLikelihood = std::numeric_limits<double>::max();
		};

		FExpectationMaximizationData EMData;
	};

	struct FExecState
	{
		using AlgoSignature = bool(*)(FPCGContext* Context, FClusteringData& ClusteringData, const int MaxIterations);
		AlgoSignature ClusteringFunction = nullptr;

		bool bSelectRandomCentroids = true;
		TArray<FVector> InitialCentroids;
	};
}

class FPCGClusterElement : public TPCGTimeSlicedElementBase<PCGClusterElement::FExecState, PCGClusterElement::FClusteringData>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
