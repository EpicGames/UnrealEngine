// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "PCGCollapseElement.generated.h"

/** Convert input to point data, performing sampling with default settings if necessary */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCollapseSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ToPoint")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCollapseElement", "NodeTitle", "To Point"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual bool ShouldDrawNodeCompact() const override { return true; }
	virtual bool GetCompactNodeIcon(FName& OutCompactNodeIcon) const override;
	virtual bool CanUserEditTitle() const override { return false; }
#endif
	virtual bool HasExecutionDependencyPin() const override { return false; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Before version PCGAttributeSetToPointAlwaysConverts (5.6) we passed through empty attribute sets without conversion.
	* This toggle preserves old behavior. */
	UPROPERTY()
	bool bPassThroughEmptyAttributeSets = false;
};

/** Converts attribute sets to point data */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGConvertToPointDataSettings : public UPCGCollapseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("AttributeSetToPoint")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGConvertToPointElement", "NodeTitle", "Attribute Set To Point"); }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	//~End UPCGSettings interface

public:
	/** Try to match attribute names that matches point property names to automatically set them. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable))
	bool bMatchAttributeNamesWithPropertyNames = false;
	
	/** Can remap attribute to point properties at the same time. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable))
	TMap<FPCGAttributePropertyInputSelector, FPCGAttributePropertyOutputSelector> AttributeToConvert;

	/** When an attribute is remapped, delete the original. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable))
	bool bDeleteOriginalRemappedAttribute = true;
};

struct FPCGCollapseContext : public FPCGContext
{
public:
	bool bShouldComputeFullOutputDataCrc = false;
};

class FPCGCollapseElement : public IPCGElementWithCustomContext<FPCGCollapseContext>
{
public:
	// Might be sampling spline/landscape or other external data, worth computing a full CRC in case we can halt change propagation/re-executions
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override;

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
