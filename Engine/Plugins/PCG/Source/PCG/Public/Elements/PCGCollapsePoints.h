// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Data/PCGPointData.h"
#include "Utils/PCGPointOctree.h"
#include "Elements/PCGTimeSlicedElementBase.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/Accessors/PCGPropertyAccessor.h"

#include "PCGCollapsePoints.generated.h"

UENUM(BlueprintType)
enum class EPCGCollapseMode : uint8
{
	PairwiseClosest UMETA(Tooltip="At every iteration, will select exclusive pairs that are closest."),
	AbsoluteClosest UMETA(Hidden, Tooltip="At every iteration, will select the closest pair to merge. Performance warning.")
};

UENUM(BlueprintType)
enum class EPCGCollapseComparisonMode : uint8
{
	Position UMETA(Tooltip="Uses point position only for distance testing, regardless of bounds."),
	Center UMETA(Tooltip="Uses point centers (e.g. center of bounds) for distance testing, and ignore bounds otherwise."),
};

UENUM(BlueprintType)
enum class EPCGCollapseVisitOrder : uint8
{
	Ordered UMETA(Tooltip="Will create pairs based on original point order."),
	Random UMETA(Tooltip="Will generate a random ordering that will drive pair order creation."),
	MinAttribute UMETA(Tooltip="Will create pairs according to attribute value order (minimum value first)."),
	MaxAttribute UMETA(Tooltip = "Will create pairs according to attribute value order (maximum value first).")
};

/** Collapses (decimates) points on a closest-point basis, allowing weighted averages to be computed as required. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCollapsePointsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGCollapsePointsSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CollapsePoints")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCollapsePointsElement", "NodeTitle", "Collapse Points"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
	virtual FText GetNodeTooltipText() const override;
#endif
	virtual bool UseSeed() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Distance at which we will stop collapsing points. E.g. Points will continue collapsing until every point is at least this distance from each other. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.01", PCG_Overridable))
	double DistanceThreshold = 100.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	EPCGCollapseMode Mode = EPCGCollapseMode::PairwiseClosest;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	EPCGCollapseComparisonMode ComparisonMode = EPCGCollapseComparisonMode::Position;

	/** Determines order in which we will collapse points pair-wise. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition="Mode == EPCGCollapseMode::PairwiseClosest", EditConditionHides, PCG_Overridable))
	EPCGCollapseVisitOrder VisitOrder = EPCGCollapseVisitOrder::Ordered;

	/** Attribute to drive visit order. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition="Mode == EPCGCollapseMode::PairwiseClosest && (VisitOrder == EPCGCollapseVisitOrder::MinAttribute || VisitOrder == EPCGCollapseVisitOrder::MaxAttribute)", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector VisitOrderAttribute;

	/** Controls whether input points will use a weight driven by an attribute */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (InlineEditConditionToggle, PCG_Overridable))
	bool bUseMergeWeightAttribute = false;

	/** Attribute that will drive relative weight when merging points (only in the Weighted mode). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bUseMergeWeightAttribute", PCG_Overridable))
	FPCGAttributePropertyInputSelector MergeWeightAttribute;

	/** List of attributes to merge on the final points, based on the weights. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TArray<FPCGAttributePropertyOutputNoSourceSelector> AttributesToMerge;
};

namespace PCGCollapsePoints
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	struct FCollapsePointsState
	{
		TArray<FTransform> PointTransforms;
		PCGPointOctree::FPointOctree PointOctree;

		TArray<double> Weights;
		TArray<int> Merged;
		
		// Selection state
		TArray<TPair<int32, int32>> Selections;
		TArray<bool> Visited;

		// Computed once in the preparation stage
		TArray<int> VisitOrder;
		TArray<double> OriginalWeights;

		const UPCGBasePointData* SourcePointData = nullptr;
		UPCGBasePointData* OutputPointData = nullptr;

		// Output data
		UE_DEPRECATED(5.6, "Use SourcePointData instead")
		const UPCGPointData* SourceData = nullptr;

		UE_DEPRECATED(5.6, "Use OutputPointData instead")
		UPCGPointData* OutData = nullptr;

		TArray<TUniquePtr<const IPCGAttributeAccessor>> SourceMergeAccessors;
		TUniquePtr<const IPCGAttributeAccessorKeys> SourceMergeKeys;
		TArray<TUniquePtr<IPCGAttributeAccessor>> OutputMergeAccessors;
		TArray<FPCGAttributePropertyOutputSelector> OutputMergeSelectors;

		UE_DEPRECATED(5.6, "Use PointTransforms instead")
		TArray<FPCGPoint> Points;

		UE_DEPRECATED(5.6, "Use PointOctree instead")
		UPCGPointData::PointOctree Octree;
	};
		
	struct FCollapsePointsSettings
	{
		const UPCGCollapsePointsSettings* Settings = nullptr;

		using PairSelectionFuncType = bool(*)(FPCGContext* /*InContext*/, const FCollapsePointsSettings& /*Settings*/, FCollapsePointsState& /*OutState*/);
		PairSelectionFuncType PairSelectionFunc = nullptr;

		using MergeSelectionFuncType = void(*)(FPCGContext* /*InContext*/, const FCollapsePointsSettings& /*Settings*/, FCollapsePointsState& /*OutState*/);
		MergeSelectionFuncType MergeSelectionFunc = nullptr;

		using GetPointPositionFuncType = FVector(*)(const TArray<FTransform>& /*InTransforms*/,const UPCGBasePointData* /*InPointData*/, int32 /*InPointIndex*/);
		GetPointPositionFuncType GetPointPositionFunc = nullptr;

		using GetPointReferenceFuncType = PCGPointOctree::FPointRef(*)(const TArray<FTransform>& /*InTransforms*/, const UPCGBasePointData* /*InPointData*/, int32 /*InPointIndex*/);
		GetPointReferenceFuncType GetPointReferenceFunc = nullptr;

		using GetPointSearchBoundsFuncType = FBoxCenterAndExtent(*)(const TArray<FTransform>& /*InTransforms*/, const UPCGBasePointData* /*InPointData*/, int32 /*InPointIndex*/, const double& /*InExtents*/);
		GetPointSearchBoundsFuncType GetPointSearchBoundsFunc = nullptr;

		double DistanceThreshold = 0.0;

		using GetPositionFuncType = FVector(*)(const FPCGPoint&);
		
		UE_DEPRECATED(5.6, "Use GetPointPositionFunc instead")
		GetPositionFuncType GetPositionFunc = nullptr;

		using GetPointRefFuncType = FPCGPointRef(*)(const FPCGPoint&);
		
		UE_DEPRECATED(5.6, "Use GetPointSearchBoundsFunc instead")
		GetPointRefFuncType GetPointRefFunc = nullptr;

		using GetSearchBoundsFuncType = FBoxCenterAndExtent(*)(const FPCGPoint&, const double&);
		
		UE_DEPRECATED(5.6, "Use GetPointSearchBoundsFunc instead")
		GetSearchBoundsFuncType GetSearchBoundsFunc = nullptr;
	};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
} // namespace PCGCollapsePoints

class FPCGCollapsePointsElement : public TPCGTimeSlicedElementBase<PCGCollapsePoints::FCollapsePointsSettings, PCGCollapsePoints::FCollapsePointsState>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
