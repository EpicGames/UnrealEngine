// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/PCGPointOperationElementBase.h"

#include "PCGCombinePoints.generated.h"

/**
 * Combines each point to share a singular bound extent.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGCombinePointsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CombinePoints")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCombinePointsElement", "NodeTitle", "Combine Points"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGCombinePointsElement", "NodeTooltip", "Combines each point to share a singular bound extent."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Places the point at the center of the combined bounds. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bCenterPivot = true;

	/** Use the transform of the initial point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseFirstPointTransform = true;

	/** Transform the point and adjust the bounds. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bUseFirstPointTransform"))
	FTransform PointTransform = FTransform();
};

class FPCGCombinePointsElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};