// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGConvertToAttributeSet.generated.h"

/** Converts point data to an attribute set with one entry per point and the same attributes. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGConvertToAttributeSetSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PointToAttributeSet")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGConvertToAttributeSetElement", "NodeTitle", "Point To Attribute Set"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGConvertToAttributeSetElement", "NodeTooltip", "Converts point data to an attribute set with one entry per point and the same attributes."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Param; }
	virtual bool ShouldDrawNodeCompact() const override { return true; }
	virtual bool GetCompactNodeIcon(FName& OutCompactNodeIcon) const override;
#endif
	virtual bool HasExecutionDependencyPin() const override { return false; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Can add properties to convert at the same time, with a potential remap. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable))
	TMap<FPCGAttributePropertyInputSelector, FPCGAttributePropertyOutputSelector> PropertiesToConvert;

	/** If properties are converted and there is already an attribute with the same name, it can be overridden by the property. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable))
	bool bOverrideExistingAttributes = false;
};

class FPCGConvertToAttributeSetElement : public IPCGElement
{
public:
	// Since this is downstream from points, possibly sampling, it's worth to compute a full CRC so if there are no changes to the attributes, we can maximize cache usage
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};

/** Converts tags on data collections to an attribute set (per input data), akin to how it's done on Get Actor Data with the Single Point option. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGTagsToAttributeSetSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("TagsToAttributeSet")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGTagsToAttributeSetElement", "NodeTitle", "Data Tags To Attribute Set"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGTagsToAttributeSetElement", "NodeTooltip", "Extracts the tags on the data to an attribute set."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Param; }
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface
};

class FPCGTagsToAttributeSetElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
