// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Data/DataView/PCGDataView.h"

#include "PCGConvertToDataView.generated.h"

/** Conversion node from any PCG Data Type to a Data View Data.
 * Constructs a view onto the PCG Data, allowing future operations to be performed on a selection of
 * internal data or metadata.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGConvertToDataViewSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ConvertToDataView")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGConvertToDataViewElement", "NodeTitle", "To Data View"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGConvertToDataViewElement", "NodeTooltip", "Construct a view onto the PCG Data, which allows for extraction or conversion of selected internal data."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual bool ShouldDrawNodeCompact() const override;
#endif // WITH_EDITOR
	virtual bool HasExecutionDependencyPin() const override { return false; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Provide the selection as an attribute set of selector strings directly into a secondary input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bSelectionAsInput = false;

	/** The selection sources as a comma-separated string. E.g. "$Index, $Position, CustomTangent", etc. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bSelectionAsInput", EditConditionHides, PCG_Overridable))
	FString SelectionSources;

	/** Manually select attributes or properties for this data view. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "!bSelectionAsInput", EditConditionHides))
	FPCGDataViewSelection Selection;
};

class FPCGConvertToDataViewElement : public IPCGElement
{
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
