// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGConvexHull2D.generated.h"

/**
* Return the convex hull of a set of points on the XY plane
*/
UCLASS(BlueprintType)
class UPCGConvexHull2DSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("FindConvexHull2D")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGConvexHullElement", "NodeTitle", "Find Convex Hull 2D"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGConvexHullElement", "NodeTooltip", "Return the 2D convex hull of a set of points on the XY plane."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGConvexHull2DElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};