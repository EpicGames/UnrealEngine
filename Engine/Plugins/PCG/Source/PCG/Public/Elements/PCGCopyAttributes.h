// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGCopyAttributes.generated.h"

UENUM()
enum class EPCGCopyAttributesOperation
{
	/** 
	* N:N operation, N:1 or 1:N Operation. If there are SourceA/SourceB and TargetA/TagretB, SourceA will be copied to TargetA and SourceB to TargetB.
	* If there are SourceA and TargetA/TargetB, SourceA will be copied to TargetA and TargetB.
	* If there are SourceA/SourceB and TargetA, SourceA will be copied to TargetA and SourceB to a copy of TargetA.
	* Produces Max(N,M) data, N being the number of targets and M being the number of sources. Either N == M, or N == 1 or M == 1.
	*/
	CopyEachSourceToEachTargetRespectively,

	/**
	* N:M operation. If there are SourceA/SourceB and TargetA/TagretB, SourceA and SourceB will be copied to TargetA and also to TargetB.
	* Copy will be sequential, so if attribute names are clashing, they will be overwritten by the last Source.
	* Produces N data, N being the number of targets.
	*/
	MergeSourcesAndCopyToAllTargets,

	/**
	* N:M operation. If there are SourceA/SourceB and TargetA/TagretB, we will have those copies: SourceA -> TargetA, SourceA -> TargetB, SourceB -> TargetA, SourceB -> TargetB.
	* Produces N*M data, N being the number of targets and M the number of sources.
	*/
	CopyEachSourceOnEveryTarget
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCopyAttributesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGCopyAttributesSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CopyAttributes")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGMetadataOperationSettings", "NodeTitle", "Copy Attributes"); }
	virtual FText GetNodeTooltipText() const;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
	virtual TArray<FText> GetNodeTitleAliases() const override;
#endif
	virtual bool HasDynamicPins() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGCopyAttributesOperation Operation = EPCGCopyAttributesOperation::CopyEachSourceToEachTargetRespectively;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bCopyAllAttributes = false;

	/** If checked, it is copying all attributes from all domains, as long as the source domain is supported on the target data.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition="bCopyAllAttributes", PCG_Overridable))
	bool bCopyAllDomains = false;

	/** When copying all attributes, a mapping can be specified. If it is empty, it's going to be Default -> Default. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition="bCopyAllAttributes && !bCopyAllDomains", EditConditionHides))
	TMap<FName, FName> MetadataDomainsMapping;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition="!bCopyAllAttributes", PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bCopyAllAttributes", PCG_Overridable))
	FPCGAttributePropertyOutputSelector OutputTarget;

private:
#if WITH_EDITOR
	// To know if we need to reconnect Target to Source.
	bool bWasDeprecated = false;
#endif // WITH_EDITOR
};

class FPCGCopyAttributesElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
