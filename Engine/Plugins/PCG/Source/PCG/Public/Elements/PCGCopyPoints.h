// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGCopyPoints.generated.h"

namespace PCGCopyPointsConstants
{
	const FName SourcePointsLabel = TEXT("Source");
	const FName TargetPointsLabel = TEXT("Target");
	const FName SelectedFlagsPinLabel = TEXT("SelectedFlags");
	const FName SelectedFlagAttributeName = TEXT("SelectForCopy");
}

UENUM()
enum class EPCGCopyPointsInheritanceMode : uint8
{
	Relative,
	Source,
	Target
};

UENUM()
enum class EPCGCopyPointsTagInheritanceMode : uint8
{
	Both,
	Source,
	Target,
};

UENUM()
enum class EPCGCopyPointsMetadataInheritanceMode : uint8
{
	SourceFirst UMETA(Tooltip = "Points will inherit from source metadata and apply only unique attributes from target."),
	TargetFirst UMETA(Tooltip = "Points will inherit from target metadata and apply only unique attributes from source."),
	SourceOnly  UMETA(Tooltip = "Points will inherit metadata only from the source."),
	TargetOnly  UMETA(Tooltip = "Points will inherit metadata only from the target."),
	None        UMETA(Tooltip = "Points will have no metadata.")
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCopyPointsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CopyPoints")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCopyPointSettings", "NodeTitle", "Copy Points"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Sampler; }
	virtual bool DisplayExecuteOnGPUSetting() const override { return true; }
	virtual void CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** The method used to determine output point rotation */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU))
	EPCGCopyPointsInheritanceMode RotationInheritance = EPCGCopyPointsInheritanceMode::Relative;

	/** If this option is set, points will have their source position transformed using the target transform with rotation */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU, EditCondition = "RotationInheritance==EPCGCopyPointsInheritanceMode::Source", EditConditionHides))
	bool bApplyTargetRotationToPositions = true;

	/** The method used to determine output point scale */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU))
	EPCGCopyPointsInheritanceMode ScaleInheritance = EPCGCopyPointsInheritanceMode::Relative;

	/** If this option is set, points will have their source position transformed using the target transform with scale */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU, EditCondition="ScaleInheritance==EPCGCopyPointsInheritanceMode::Source", EditConditionHides))
	bool bApplyTargetScaleToPositions = true;

	/** The method used to determine output point color */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPUWithReadback))
	EPCGCopyPointsInheritanceMode ColorInheritance = EPCGCopyPointsInheritanceMode::Relative;

	/** The method used to determine output seed values. Relative recomputes the seed from the new location. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPUWithReadback))
	EPCGCopyPointsInheritanceMode SeedInheritance = EPCGCopyPointsInheritanceMode::Relative;

	/** The method used to determine output data attributes */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPUWithReadback))
	EPCGCopyPointsMetadataInheritanceMode AttributeInheritance = EPCGCopyPointsMetadataInheritanceMode::SourceFirst;

	/** The method used to determine the output data tags */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPUWithReadback))
	EPCGCopyPointsTagInheritanceMode TagInheritance = EPCGCopyPointsTagInheritanceMode::Both;

	/** If this option is set, each source point data will be copied to every target point data (cartesian product), producing N * M point data. Otherwise, will do a N:N (or N:1 or 1:N) operation, producing N point data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPUWithReadback))
	bool bCopyEachSourceOnEveryTarget = true;

	// Note: Not overridable currently. Used to make compile time decision about which kernels to emit. If this needs to be overridable then we
	// need to investigate how to deal with this at execution time.
	/** Perform a conditional copy point where data pairs must have a matching attribute in order to participate in the copy operation.
	* EXPERIMENTAL: This feature is work in progress and is currently GPU-only. It may not be stable, and may be removed in the future.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (EditCondition = "bExecuteOnGPU", EditConditionHides))
	bool bMatchBasedOnAttribute = false;

	// @todo_pcg: Could be overridden when Name overrides are supported.
	/** Attribute that must be present on both the source data and the target point, and have the same value.
	* EXPERIMENTAL: This feature is work in progress and is currently GPU-only. It may not be stable, and may be removed in the future.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (EditCondition = "bExecuteOnGPU && bMatchBasedOnAttribute", EditConditionHides))
	FName MatchAttribute = NAME_None;
};

class FPCGCopyPointsElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
