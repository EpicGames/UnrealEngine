// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeKernel.h"

#include "PCGCopyPointsAnalysisKernel.generated.h"

class UPCGCopyPointsSettings;

UCLASS()
class UPCGCopyPointsAnalysisKernel : public UPCGComputeKernel
{
	GENERATED_BODY()

public:
	virtual bool IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const override;
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const override;
#if WITH_EDITOR
	virtual const TCHAR* GetSourceFilePath() const override { return TEXT("/Plugin/PCG/Private/Elements/PCGCopyPointsAnalysis.usf"); }
	virtual FString GetEntryPoint() const override { return TEXT("PCGCopyPointsAnalysisCS"); }
	void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
	// Split graph to read back analysis results.
	virtual bool SplitGraphAtOutput() const override { return true; }
#endif
	virtual void GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const override;

	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;
};
