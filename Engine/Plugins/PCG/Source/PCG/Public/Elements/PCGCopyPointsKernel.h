// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeKernel.h"

#include "Compute/PCGPinPropertiesGPU.h"

#include "PCGCopyPointsKernel.generated.h"

class UPCGCopyPointsSettings;

UCLASS()
class UPCGCopyPointsKernel : public UPCGComputeKernel
{
	GENERATED_BODY()

public:
	//~ Begin UPCGComputeKernel Interface
	virtual bool IsKernelDataValid(const UPCGDataBinding* InDataBinding, FPCGContext* InContext) const override;
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const override;
#if WITH_EDITOR
	virtual const TCHAR* GetSourceFilePath() const override { return TEXT("/Plugin/PCG/Private/Elements/PCGCopyPoints.usf"); }
	virtual FString GetEntryPoint() const override { return TEXT("PCGCopyPointsCS"); }
	void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
#endif
	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;
	//~ End UPCGComputeKernel Interface

	void SetMatchBasedOnAttribute(bool bInMatchBasedOnAttribute) { bMatchBasedOnAttribute = bInMatchBasedOnAttribute; }

protected:
	UPROPERTY()
	bool bMatchBasedOnAttribute = false;
};
