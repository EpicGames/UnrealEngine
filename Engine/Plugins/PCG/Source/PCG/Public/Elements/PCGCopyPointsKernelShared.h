// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UPCGComputeKernel;
class UPCGDataBinding;
struct FPCGContext;

namespace PCGCopyPointsKernel
{
	/** Performs data validation common to all copy points kernels. */
	bool IsKernelDataValid(const UPCGComputeKernel* InKernel, const UPCGDataBinding* InDataBinding, FPCGContext* InContext = nullptr);
}
