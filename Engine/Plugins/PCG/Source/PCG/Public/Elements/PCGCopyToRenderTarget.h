// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Async/PCGAsyncLoadingContext.h"

#include "PCGCopyToRenderTarget.generated.h"

class UPCGTexture2DSingleBaseData;
class UTextureRenderTarget2D;

/** Copy texture data into render target assets. Input textures should be 1:1 with the render target assets and must match in extents and format. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCopyToRenderTargetSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CopyToRenderTarget")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCopyToRenderTargetElement", "NodeTitle", "Copy to Render Target"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGCopyToRenderTargetElement", "NodeTooltip", "Copy texture data into render target assets. Input textures should be 1:1 with the render target assets and must match in extents and format."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;

#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

public:
	/** Override render targets from input. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bOverrideFromInput = false;

	/** Render targets to copy texture data into. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bOverrideFromInput"))
	TArray<TSoftObjectPtr<UTextureRenderTarget2D>> RenderTargets;

	/** Input attribute to pull render targets from. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bOverrideFromInput", PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector RenderTargetAttribute;

	/** By default, will use async loading for the render targets. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

struct FPCGCopyToRenderTargetContext : public FPCGContext, public IPCGAsyncLoadingContext
{
protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;

public:
	bool bEnqueuedCopyCommand = false;
	TArray<FSoftObjectPath> RenderTargetsToLoad;
	TArray<TPair<const UPCGTexture2DSingleBaseData*, TObjectPtr<UTextureRenderTarget2D>>> TexturesToCopy;
};

class FPCGCopyToRenderTargetElement : public IPCGElementWithCustomContext<FPCGCopyToRenderTargetContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
