// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataTypesConstantStruct.h"
#include "Metadata/PCGMetadataCommon.h"

#include "PCGCreateAttribute.generated.h"

class FPCGMetadataAttributeBase;
class UPCGMetadata;


/** Add a new attribute to a spatial data or an attribute set.
* New attribute can be a constant, hardcoded in the node, or can come from another Attribute Set.
* Can also add all the attributes coming from the other Attribute Set.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAddAttributeSettings : public UPCGSettings
{
	GENERATED_BODY()

	UPCGAddAttributeSettings();

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Param; }
	virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const override;
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void ApplyStructuralDeprecation(UPCGNode* InOutNode) override;
#endif // WITH_EDITOR
	virtual bool HasDynamicPins() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bCopyAllAttributes", HideEditConditionToggle, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection, PCG_Overridable))
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bCopyAllAttributes", PCG_DiscardPropertySelection, PCG_DiscardExtraSelection, PCG_Overridable))
	FPCGAttributePropertyOutputSelector OutputTarget;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties))
	FPCGMetadataTypesConstantStruct AttributeTypes;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bCopyAllAttributes = false;

	/** If checked, it is copying all attributes from all domains, as long as the source domain is supported on the target data.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition="bCopyAllAttributes", PCG_Overridable))
	bool bCopyAllDomains = false;

	/** When copying all attributes, a mapping can be specified. If it is empty, it's going to be Default -> Default. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition="bCopyAllAttributes && !bCopyAllDomains", EditConditionHides))
	TMap<FName, FName> MetadataDomainsMapping;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName SourceParamAttributeName_DEPRECATED = NAME_None;

	UPROPERTY()
	FName OutputAttributeName_DEPRECATED = NAME_None;
#endif // WITH_EDITORONLY_DATA

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
};

/* Creates a new Attribute Set. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCreateAttributeSetSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Param; }
	virtual TArray<FText> GetNodeTitleAliases() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool GroupPreconfiguredSettings() const override { return true; }
	virtual bool OnlyExposePreconfiguredSettings() const override { return false; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
	virtual TArray<FPCGPreconfiguredInfo> GetConversionInfo() const override;
#endif
	virtual bool ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo) override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties))
	FPCGMetadataTypesConstantStruct AttributeTypes;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_DiscardPropertySelection, PCG_DiscardExtraSelection, PCG_Overridable))
	FPCGAttributePropertyOutputNoSourceSelector OutputTarget;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName OutputAttributeName_DEPRECATED = NAME_None;
#endif // WITH_EDITORONLY_DATA

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
};

class FPCGCreateAttributeElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};

class FPCGAddAttributeElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};