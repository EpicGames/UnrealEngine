// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Data/PCGCollisionWrapperData.h"

#include "PCGCreateCollisionData.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCreateCollisionDataSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CreateCollisionData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCreateCollisionDataSettings", "NodeTitle", "Create Collision Data"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif // WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector CollisionAttribute;

	/** Controls how shapes are selected from collision. Performance warning on using complex shapes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases="bUseComplexCollision"))
	EPCGCollisionQueryFlag CollisionQueryFlag = EPCGCollisionQueryFlag::Simple;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bWarnIfAttributeCouldNotBeUsed = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;

private:
#if WITH_EDITORONLY_DATA
	// Implementation note: was introduced during UE 5.5 development and replaced, does not require a full public API deprecation mechanism
	UPROPERTY()
	bool bUseComplexCollision_DEPRECATED = false;
#endif
};

struct FPCGCreateCollisionContext : public FPCGContext, public IPCGAsyncLoadingContext
{
	struct InputMeshData
	{
		int InputIndex = INDEX_NONE;
		TArray<FSoftObjectPath> MeshPaths;
		TObjectPtr<UPCGCollisionWrapperData> Data = nullptr;
	};

	TArray<InputMeshData> PerInputData;

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;
};

class FPCGCreateCollisionDataElement : public IPCGElementWithCustomContext<FPCGCreateCollisionContext>
{
public:
	// Loading needs to be done on the main thread and accessing objects outside of PCG might not be thread safe, so taking the safe approach
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};