// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGSettings.h"

#include "PCGCommon.h"
#include "Data/PCGPointData.h"

#include "PCGCreatePoints.generated.h"

UENUM()
enum class UE_DEPRECATED(5.4, "Not used anymore, replaced by EPCGCoordinateSpace.") EPCGLocalGridPivot : uint8
{
	Global,
	OriginalComponent,
	LocalComponent
};

/**
 * Creates point data from a provided list of points.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGCreatePointsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGCreatePointsSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CreatePoints")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCreatePointsElement", "NodeTitle", "Create Points"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGCreatePointsElement", "NodeTooltip", "Creates point data from a provided list of points."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TArray<FPCGPoint> PointsToCreate;

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Sets the points transform to world or local space*/
	UPROPERTY()
	EPCGLocalGridPivot GridPivot_DEPRECATED = EPCGLocalGridPivot::Global;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	/** Sets the generation referential of the points */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases="GridPivot"))
	EPCGCoordinateSpace CoordinateSpace = EPCGCoordinateSpace::World;

	/** If true, points are removed if they are outside of the volume */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bCullPointsOutsideVolume = false;
};

namespace PCGCreatePointsHelper
{
	UPCGData* GetDependenciesData(IPCGGraphExecutionSource* InExecutionSource, EPCGCoordinateSpace InCoordinateSpace, bool bInCullPointsOutsideVolume);
}

class FPCGCreatePointsElement : public IPCGElement
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override;
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};