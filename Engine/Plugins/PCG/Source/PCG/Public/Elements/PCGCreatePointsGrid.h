// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGSettings.h"

#include "Data/PCGPointData.h"
#include "PCGCreatePointsGrid.generated.h"

UENUM()
enum class EPCGPointPosition : uint8
{
	CellCenter,
	CellCorners
};

UENUM()
enum class UE_DEPRECATED(5.4, "Not used anymore, replaced by EPCGCoordinateSpace.") EPCGGridPivot : uint8
{
	Global,
	OriginalComponent,
	LocalComponent
};

/**
 * Creates a 2D or 3D grid of points.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGCreatePointsGridSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGCreatePointsGridSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CreatePointsGrid")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCreatePointsGridElement", "NodeTitle", "Create Points Grid"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGCreatePointsGridElement", "NodeTooltip", "Creates a 2D or 3D grid of points."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = "0.0", UIMin = "0.0"))
	FVector GridExtents = FVector(500.0, 500.0, 50.0);
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = "0.0", UIMin = "0.0"))
	FVector CellSize = FVector(100.0, 100.0, 100.0);

	/** Each PCG point represents a discretized, volumetric region of world space. The points' Steepness value [0.0 to
	 * 1.0] establishes how "hard" or "soft" that volume will be represented. From 0, it will ramp up linearly
	 * increasing its influence over the density from the point's center up to two times the bounds. At 1, it will
	 * represent a binary box function with the size of the point's bounds.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", ClampMax = "1", PCG_Overridable))
	float PointSteepness = 0.5f;

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Sets the points transform to world or local space*/
	UPROPERTY()
	EPCGGridPivot GridPivot_DEPRECATED = EPCGGridPivot::Global;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	/** Sets the generation referential of the points */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases="GridPivot"))
	EPCGCoordinateSpace CoordinateSpace = EPCGCoordinateSpace::World;

	/** If true, the bounds of the points are set to 50.0, if false, 1.0 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bSetPointsBounds = true;

	/** If true, points are removed if they are outside of the volume */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bCullPointsOutsideVolume = false;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGPointPosition PointPosition = EPCGPointPosition::CellCenter;
};

class FPCGCreatePointsGridElement : public IPCGElement
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};