// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGSettings.h"

#include "PCGCreatePointsSphere.generated.h"

struct ISphereGenerator;

UENUM()
enum class EPCGSphereGeneration : uint8
{
	Geodesic UMETA(ToolTip = "Points generated from subdivided equilateral triangles across the sphere's surface."),
	Angle UMETA(ToolTip = "Points generated radially along longitude and latitude lines to form a sphere with user defined angles."),
	Segments UMETA(ToolTip = "Points generated radially along longitude and latitude lines to form a sphere with user defined number of segments."),
	Random UMETA(ToolTip = "Points are generated with a uniform distribution on the surface of the sphere."),
	Poisson UMETA(ToolTip = "Sample points along the sphere's surface with a minimal distance between any given two points.")
};

UENUM()
enum class EPCGGeodesicSphereRepresentation : uint8
{
	Icosahedron UMETA(ToolTip = "Twenty sided polyhedron."),
	// TODO: Tetrahedron, Octahedron
};

UENUM()
enum class EPCGSpherePointOrientation : uint8
{
	Radial UMETA(ToolTip = "Points' Up vector is aligned outward from the origin (the normal of the sphere's surface)."),
	Centric UMETA(ToolTip = "Points' Up vector is aligned inward toward the origin (inverse of the normal)."),
	None UMETA(ToolTip = "Points will not be explicitly aligned.")
};

/**
 * Generate a grid of points along the surface of a sphere.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGCreatePointsSphereSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGCreatePointsSphereSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CreatePointsSphere")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCreatePointsSphereElement", "NodeTitle", "Create Points Sphere"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGCreatePointsSphereElement", "NodeTooltip", "Generate points on the surface of a sphere."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif // WITH_EDITOR
	virtual bool UseSeed() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return {}; }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Determines the type of sphere generated. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGSphereGeneration SphereGeneration = EPCGSphereGeneration::Geodesic;

	/** Sets the generation referential of the points. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGCoordinateSpace CoordinateSpace = EPCGCoordinateSpace::World;

	/** Will determine the points' orientation, once generated. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGSpherePointOrientation PointOrientation = EPCGSpherePointOrientation::Radial;

	/** The sphere's origin, around which the points will be generated. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FVector Origin = FVector::ZeroVector;

	/** The sphere's radius. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	double Radius = 100.0;

	/** Determines the number of subdivisions of the geodesic sphere. Becomes exponentially more expensive as it gets higher. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "SphereGeneration == EPCGSphereGeneration::Geodesic", EditConditionHides, ClampMin = 0, ClampMax = 8))
	int32 GeodesicSubdivisions = 2;

	/** The latitudinal angle (in degrees) between generated segments on the standard sphere grid. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, DisplayName = "Latitudinal Segment (Degrees)", Category = Settings, meta = (PCG_Overridable, EditCondition = "SphereGeneration == EPCGSphereGeneration::Angle", EditConditionHides, ClampMin = 0.001, ClampMax = 90))
	double Theta = 12;

	/** The longitudinal angle (in degrees) between generated segments on the standard sphere grid. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, DisplayName = "Longitudinal Segment (Degrees)", Category = Settings, meta = (PCG_Overridable, EditCondition = "SphereGeneration == EPCGSphereGeneration::Angle", EditConditionHides, ClampMin = 0.001, ClampMax = 180))
	double Phi = 12;

	/** Will determine the latitudinal angle between segments needed to generate this number of latitudinal segments. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "SphereGeneration == EPCGSphereGeneration::Segments", EditConditionHides, ClampMin = 1, ClampMax = 90000))
	int32 LatitudinalSegments = 15;

	/** Will determine the longitudinal angle between segments needed to generate this number of longitudinal segments. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "SphereGeneration == EPCGSphereGeneration::Segments", EditConditionHides, ClampMin = 1, ClampMax = 180000))
	int32 LongitudinalSegments = 30;

	/** Determines the number of samples generated for the random generation and the poisson sampling. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "SphereGeneration == EPCGSphereGeneration::Random || SphereGeneration == EPCGSphereGeneration::Poisson", EditConditionHides, ClampMin = 1))
	int32 SampleCount = 100;

	/** The maximum world distance between points sampled on the sphere's surface during a Poisson sampling, in cm. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "SphereGeneration == EPCGSphereGeneration::Poisson", EditConditionHides, ClampMin = 1))
	double PoissonDistance = 100.0;

	/** Poisson sampling will continue to search for open positions until this limit is reached. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "SphereGeneration == EPCGSphereGeneration::Poisson", EditConditionHides, ClampMin = 1))
	int32 PoissonMaxAttempts = 32;

	/** Points will be generated on the sphere's surface beginning at this equatorial angle. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = -90, ClampMax = 90))
	double LatitudinalStartAngle = -90.0;

	/** Points will cease to be generated on the sphere's surface after this equatorial angle. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = -90, ClampMax = 90))
	double LatitudinalEndAngle = 90.0;

	/** Points will be generated on the sphere's surface beginning at this meridional angle. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = -180, ClampMax = 180))
	double LongitudinalStartAngle = -180.0;

	/** Points will cease to be generated on the sphere's surface after this meridional angle. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = -180, ClampMax = 180))
	double LongitudinalEndAngle = 180.0;

	/** Adds randomization (in the range of [-Jitter, Jitter]) of the angles of a generated point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "SphereGeneration == EPCGSphereGeneration::Angle || SphereGeneration == EPCGSphereGeneration::Segments", EditConditionHides, ClampMin = 0.0, ClampMax = 0.5))
	double Jitter = 0.0;

	/** Directly set as the output points' steepness value. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", ClampMax = "1", PCG_Overridable))
	float PointSteepness = 0.5f;

	/** Points are removed if they are outside the volume. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bCullPointsOutsideVolume = false;

	/** Adds a graph error and prevents generation if the generated point count would exceed this amount. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (ClampMin = 1))
	int32 PointLimit = 1000000;
};

class FPCGCreatePointsSphereElement : public IPCGElement
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;

protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
