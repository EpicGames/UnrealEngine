// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGElement.h"
#include "PCGSettings.h"
#include "Data/PCGSplineData.h"

#include "PCGCreateSpline.generated.h"

struct FPCGContext;

UENUM()
enum class EPCGCreateSplineMode : uint8
{
	CreateDataOnly,
	CreateComponent,
};

/** PCG node that creates a spline presentation from the input points data, with optional tangents */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCreateSplineSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGCreateSplineSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CreateSpline")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCreateSplineSettings", "NodeTitle", "Create Spline"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif
	

	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGCreateSplineMode Mode = EPCGCreateSplineMode::CreateDataOnly;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bClosedLoop = false;

	// Controls whether the segment between control points is a curve (when false) or a straight line (when true).
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bLinear = false;

	/** Allow to specify custom tangents for each point, as an attribute. Can't be set if the spline is linear. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bLinear"))
	bool bApplyCustomTangents = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bLinear && bApplyCustomTangents"))
	FName ArriveTangentAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bLinear && bApplyCustomTangents"))
	FName LeaveTangentAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bUseInterpTypeAttribute = false;

	/** Name of the attribute containing the interpolation type, if any. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bUseInterpTypeAttribute"))
	FName InterpTypeAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "Mode!= EPCGCreateSplineMode::CreateDataOnly", EditConditionHides))
	TArray<FName> TagsToAddOnComponents;

	UPROPERTY(meta = (PCG_Overridable))
	TSoftObjectPtr<AActor> TargetActor;

	/** Specify a list of functions to be called on the target actor after spline creation. Functions need to be parameter-less and with "CallInEditor" flag enabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FName> PostProcessFunctionNames;
};

class FPCGCreateSplineElement : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override;

protected:	
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};