// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGCreateSurfaceFromSpline.generated.h"

/**
 * Create an implicit surface for each given spline. The surface is given by the top-down 2D projection of the spline. Each spline must be closed.
 * 
 * Note that by default a low resolution polyline is used to represent the spline. If you observe that your interior samples are not sufficiently accurate,
 * you can resample the spline with a SplineSampler at your desired resolution, and convert the points back to a spline with the CreateSpline node.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCreateSurfaceFromSplineSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CreateSurfaceFromSpline")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCreateSurfaceFromSplineElement", "NodeTitle", "Create Surface From Spline"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual bool ShouldDrawNodeCompact() const override { return bShouldDrawNodeCompact; }
#endif
	virtual bool HasExecutionDependencyPin() const override { return false; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY()
	bool bShouldDrawNodeCompact = false;
#endif
};

class FPCGCreateSurfaceFromSplineElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
