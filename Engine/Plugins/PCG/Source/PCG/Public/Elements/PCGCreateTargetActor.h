// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Helpers/PCGDataLayerHelpers.h"
#include "Helpers/PCGHLODHelpers.h"
#include "Metadata/PCGObjectPropertyOverride.h"

#include "WorldPartition/DataLayer/DataLayerAsset.h"

#include "PCGCreateTargetActor.generated.h"

class AActor;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCreateTargetActor : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGCreateTargetActor(const FObjectInitializer& ObjectInitializer);

	//~Begin UObject interface
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
#endif
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CreateTargetActor")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spawner; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	//~End UPCGSettings interface

	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances);
	void SetupBlueprintEvent();
	void TeardownBlueprintEvent();

private:
	void RefreshTemplateActor();
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, Category = Settings, meta = (ShowInnerProperties, EditCondition = "bAllowTemplateActorEditing", EditConditionHides))
	TObjectPtr<AActor> TemplateActor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGAttachOptions AttachOptions = EPCGAttachOptions::Attached; // Note that this is no longer the default value for new nodes, it is now EPCGAttachOptions::InFolder

	/** Actor to attach to if the option is Attached. Default to the Component owner. */
	UPROPERTY(meta = (PCG_Overridable))
	TSoftObjectPtr<AActor> RootActor;

	/** Name of the actor that will be created. */
	UPROPERTY(meta = (PCG_Overridable))
	FString ActorLabel;

	/** Transform of the actor that will be created. */
	UPROPERTY(meta = (PCG_Overridable))
	FTransform ActorPivot;

	/** Comma-separated list of tags to add to the newly created target actor. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, DisplayName = "Comma Separated Actor Tags (Deprecated)", meta = (PCG_Overridable))
	FString CommaSeparatedActorTags;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TArray<FName> TagsToAddOnActor;

	/** Override the default property values on the created target actor. Applied before post-process functions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TArray<FPCGObjectPropertyOverrideDescription> PropertyOverrideDescriptions;

	/** Specify a list of functions to be called on the target actor after creation. Functions need to be parameter-less and with "CallInEditor" flag enabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FName> PostProcessFunctionNames;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming, meta = (PCG_Overridable))
	FPCGDataLayerSettings DataLayerSettings;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming, DisplayName = "HLOD Settings", meta = (PCG_Overridable))
	FPCGHLODSettings HLODSettings;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bDeleteActorsBeforeGeneration = false;
	
	/** 
	 * Override the template actor class to spawn. If there is an actor template, it will only be valid if the override class is a child class (or the exact class) of the template actor.
	 * If it is not, the node will fail.
	 */
	UPROPERTY(meta = (PCG_Overridable))
	TSubclassOf<AActor> OverrideTemplateActorClass;

	PCG_API void SetTemplateActorClass(const TSubclassOf<AActor>& InTemplateActorClass);
	PCG_API void SetAllowTemplateActorEditing(bool bInAllowTemplateActorEditing);
	const TSubclassOf<AActor>& GetTemplateActorClass() const { return TemplateActorClass; }
	bool GetAllowTemplateActorEditing() const { return bAllowTemplateActorEditing; }

protected:
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (OnlyPlaceable, DisallowCreateNew))
	TSubclassOf<AActor> TemplateActorClass;

	// TODO: make this InlineEditConditionToggle, not done because property changed event does not propagate correctly so we can't track accurately the need to create the target actor
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bAllowTemplateActorEditing = false;

	friend class FPCGCreateTargetActorElement;
};

class FPCGCreateTargetActorElement : public IPCGElement
{
protected:
	// Since this element creates an actor, it needs to run on the game thread and cannot be cached
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* Settings) const override { return false; }
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
