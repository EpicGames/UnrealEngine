// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Elements/PCGActorSelector.h"
#include "Elements/PCGLoadObjectsContext.h"

#include "UObject/ObjectKey.h"

#include "PCGDataFromActor.generated.h"

#define UE_API PCG_API

UENUM()
enum class EPCGGetDataFromActorMode : uint8
{
	ParseActorComponents UMETA(Tooltip = "Parse the found actor(s) for relevant components such as Primitives, Splines, and Volumes."),
	GetSinglePoint UMETA(Tooltip = "Produces a single point per actor with the actor transform and bounds."),
	GetDataFromProperty UMETA(DisplayName = "Get PCG Data From Property", Tooltip = "Gets a data collection from an actor property."),
	GetDataFromPCGComponent UMETA(Tooltip = "Copy generated output from other PCG components on the found actor(s)."),
	GetDataFromPCGComponentOrParseComponents UMETA(Tooltip = "Attempts to copy generated output from other PCG components on the found actor(s), otherwise, falls back to parsing actor components."),
	GetActorReference UMETA(Tooltip = "Produces one entry per actor with only the actor reference."),
	GetComponentsReference UMETA(Tooltip = "Produces one entry per component within the actor selection."),
	GetPointData UMETA(Tooltip = "Produces a point data containing a point per component, except for ISMs that will have a point per instance. Similar behavior to the PCG Data Asset format.")
};

/** Builds a collection of PCG-compatible data from the selected actors. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDataFromActorSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetActorData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDataFromActorSettings", "NodeTitle", "Get Actor Data"); }
	UE_API virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	UE_API virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
	UE_API virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
#endif
	virtual bool HasDynamicPins() const override { return true; }
	UE_API virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

	UE_API virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	UE_API virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
#endif
	UE_API virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	UE_API virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	UE_API virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings

public:
	//~Begin UObject interface
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject interface

public:
	/** Override this to filter what kinds of data should be retrieved from the actor(s). */
	virtual EPCGDataType GetDataFilter() const { return EPCGDataType::Any; }

	/** Override this to change the default value the selector will revert to when changing the actor selection type */
	UE_API virtual TSubclassOf<AActor> GetDefaultActorSelectorClass() const;

protected:
#if WITH_EDITOR
	UFUNCTION()
	UE_API virtual bool DisplayModeSettings() const;
#endif

public:
	/** Describes which actors to select for data collection. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ShowOnlyInnerProperties))
	FPCGActorSelectorSettings ActorSelector;

	/** Describes which components to select for the data collection. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ShowOnlyInnerProperties))
	FPCGComponentSelectorSettings ComponentSelector;

	/** Describes what kind of data we will collect from the found actor(s). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta = (EditCondition = "DisplayModeSettings()", EditConditionHides, HideEditConditionToggle))
	EPCGGetDataFromActorMode Mode = EPCGGetDataFromActorMode::ParseActorComponents;

	/** Ignores any component that was spawned by PCG. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta = (EditCondition = "Mode == EPCGGetDataFromActorMode::ParseActorComponents || Mode == EPCGGetDataFromActorMode::GetPointData || Mode == EPCGGetDataFromActorMode::GetComponentsReference || Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents", EditConditionHides))
	bool bIgnorePCGGeneratedComponents = true;

	/** Also produces a single point data at the actor location. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta = (EditCondition = "Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents", EditConditionHides))
	bool bAlsoOutputSinglePointData = false;

	/** Only get data from components which overlap with the bounds of your source component. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta = (EditCondition = "Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents", EditConditionHides))
	bool bComponentsMustOverlapSelf = true;

	/** Get data from all grid sizes if there is a partitioned PCG component on the actor, instead of a specific set of grid sizes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta = (EditCondition = "Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents", EditConditionHides))
	bool bGetDataOnAllGrids = true;

	/** Select which grid sizes to consider when collecting data from partitioned PCG components (without grid scaling applied). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta = (Bitmask, BitmaskEnum = "/Script/PCG.EPCGHiGenGrid", EditCondition = "!bGetDataOnAllGrids && Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents", EditConditionHides))
	int32 AllowedGrids = int32(EPCGHiGenGrid::Uninitialized);

	/** Merges all the single point data outputs into a single point data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta = (DisplayName = "Merge Simple Data", EditCondition = "Mode == EPCGGetDataFromActorMode::GetSinglePoint || Mode == EPCGGetDataFromActorMode::GetActorReference || Mode == EPCGGetDataFromActorMode::GetComponentsReference", EditConditionHides))
	bool bMergeSinglePointData = false;

	/** Provide pin names to match against the found component output pins. Data will automatically be wired to the expected pin if the name comparison succeeds. All unmatched pins will go into the standard out pin. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta = (EditCondition = "Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponent || Mode == EPCGGetDataFromActorMode::GetDataFromPCGComponentOrParseComponents", EditConditionHides))
	TArray<FName> ExpectedPins;

	/** The property name on the found actor to create a data collection from. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta = (EditCondition = "Mode == EPCGGetDataFromActorMode::GetDataFromProperty", EditConditionHides))
	FName PropertyName = NAME_None;

	/** If this is true, we will never put this element in cache, and will always try to re-query the actors and read the latest data from them. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings")
	bool bAlwaysRequeryActors = false;

	/** Silence warnings that attribute names were sanitized to replace invalid characters. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", AdvancedDisplay)
	bool bSilenceSanitizedAttributeNameWarnings = false;

	/** Silence warnings that attribute names were rejected because they clash with reserved names. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", AdvancedDisplay)
	bool bSilenceReservedAttributeNameWarnings = false;

#if WITH_EDITORONLY_DATA
	/** If this is checked, found actors that are outside component bounds will not trigger a refresh. Only works for tags for now in editor. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Data Retrieval Settings")
	bool bTrackActorsOnlyWithinBounds = true;
#endif // WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.5, "No longer in use, override DisplayModeSettings() instead.")
	UPROPERTY(Transient, meta = (EditCondition = false, EditConditionHides))
	bool bDisplayModeSettings = true;
};

struct FPCGDataFromActorContext : public FPCGLoadObjectsFromPathContext
{
	TArray<AActor*> FoundActors;
	FPCGComponentSelectorSettings ComponentSelector;
	bool bPerformedQuery = false;
	bool bWaitingOnProcessActors = false;

#if WITH_EDITOR
	/** Any change origin ignores we added, to solve dependency issues (like upstream execution cancelling downstream graph). */
	TArray<TObjectKey<UObject>> IgnoredChangeOrigins;
#endif
};

class FPCGDataFromActorElement : public IPCGElementWithCustomContext<FPCGDataFromActorContext>
{
public:
	UE_API virtual bool IsCacheable(const UPCGSettings* InSettings) const override;
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	UE_API virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override { return true; }

protected:
	UE_API virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	UE_API virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	UE_API void GatherWaitTasks(AActor* FoundActor, FPCGContext* InContext, TArray<FPCGTaskId>& OutWaitTasks) const;

	UE_API virtual void ProcessActors(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, const TArray<AActor*>& FoundActors, TArray<FPCGTaskId>& OutDynamicDependencies) const;
	UE_API virtual void ProcessActors(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, const TArray<AActor*>& FoundActors) const;
	UE_API virtual void ProcessActor(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, AActor* FoundActor) const;

	UE_API void MergeActorsIntoData(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, const TArray<AActor*>& FoundActors) const;
	UE_API void CreateReferenceData(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, const TArray<AActor*>& Actors) const;
};

#undef UE_API
