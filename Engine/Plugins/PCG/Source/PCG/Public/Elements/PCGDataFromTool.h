// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Elements/PCGActorSelector.h"
#include "Elements/PCGLoadObjectsContext.h"

#include "UObject/ObjectKey.h"

#include "PCGDataFromTool.generated.h"

/** Builds a collection of PCG-compatible data from the specified editor tools. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDataFromTool : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetToolData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDataFromTool", "NodeTitle", "Get Tool Data"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif

	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** The tool tag on the pcg component to create a data collection from.
	 * Example given: "PaintTool" will allow you to retrieve points created by the PaintTool in the level viewport in PCG mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta=(PCG_Overridable))
	FName ToolTag = NAME_None;

	/** The optional name of the data this node should retrieve. If you want to support multiple tool outputs of the same type, differentiate using this name.
	 * Example given: "Trees" or "Bushes" will allow you to paint onto two different layers so you can process them differently in the graph.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data Retrieval Settings", meta=(PCG_Overridable))
	FName DataInstance = NAME_None;
};

class FPCGDataFromToolElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
protected:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
