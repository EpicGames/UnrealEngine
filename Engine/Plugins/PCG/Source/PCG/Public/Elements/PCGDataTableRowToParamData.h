// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Async/PCGAsyncLoadingContext.h"

#include "PCGDataTableRowToParamData.generated.h"

class UDataTable;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDataTableRowToParamDataSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

#if WITH_EDITOR
	//~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("DataTableRowToAttributeSet")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDataTableRowToParamDataSettings", "NodeTitle", "Data Table Row To Attribute Set"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins);
#endif

	virtual FString GetAdditionalTitleInformation() const override;

	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;

	// The name of the row to copy from
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName RowName = NAME_None;	

	// the data table to copy from
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases = "PathOverride"))
	TSoftObjectPtr<UDataTable> DataTable;

	/** By default, data table loading is asynchronous, can force it synchronous if needed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString PathOverride_DEPRECATED = FString();
#endif // 


protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface
};

struct FPCGDataTableRowToParamDataContext : public FPCGContext, public IPCGAsyncLoadingContext {};

class FPCGDataTableRowToParamData : public IPCGElementWithCustomContext<FPCGDataTableRowToParamDataContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
