// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "GameFramework/Actor.h"

#include "PCGDebugElement.generated.h"

#define UE_API PCG_API

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDebugSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	// ~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Debug")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDebugSettings", "NodeTitle", "Debug"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Debug; }
#endif
	

protected:
	UE_API virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	UE_API virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	UE_API virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	TSoftObjectPtr<AActor> TargetActor;
};

class FPCGDebugElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};

#undef UE_API
