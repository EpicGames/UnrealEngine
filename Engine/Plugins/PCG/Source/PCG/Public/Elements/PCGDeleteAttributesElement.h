// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGDeleteAttributesElement.generated.h"

UENUM()
enum class EPCGAttributeFilterOperation
{
	KeepSelectedAttributes UMETA(DisplayName = "Keep Only Selected Attributes"),
	DeleteSelectedAttributes
};

/**
* Removes attributes from a given input metadata.
* Either removes specifically named attributes or remove all attributes not in a given list.
* 
* The output will be the original data with the updated metadata.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDeleteAttributesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGDeleteAttributesSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;

	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual TArray<FText> GetNodeTitleAliases() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
#endif

	virtual bool HasDynamicPins() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	// Implementation note: the default operation has been changed to DeleteSelected for new objects
	/** Controls if only the attributes that are selected in this node will be kept or deleted from the input data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGAttributeFilterOperation Operation = EPCGAttributeFilterOperation::KeepSelectedAttributes;

	/** Criteria to match the attribute name against the selected attributes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGStringMatchingOperator Operator = EPCGStringMatchingOperator::Equal;

	/** Comma-separated list of attributes to keep or remove from the input data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FString SelectedAttributes;

	/** Enables deprecated behavior using spaces as separators. Disable to update the node to current behavior. */
	UE_DEPRECATED(5.5, "bTokenizeOnWhiteSpace has been deprecated.")
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = "bTokenizeOnWhiteSpace", EditConditionHides, DeprecationMessage = "bTokenizeOnWhiteSpace has been deprecated."))
	bool bTokenizeOnWhiteSpace = false;

	/** When deleting attributes, only the specified domain will be visited. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName MetadataDomain = PCGDataConstants::DefaultDomainName;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FName> AttributesToKeep_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};


class FPCGDeleteAttributesElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
