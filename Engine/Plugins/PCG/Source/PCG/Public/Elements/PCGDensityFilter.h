// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGDensityFilter.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDensityFilterSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("DensityFilter")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDensityFilterSettings", "NodeTitle", "Density Filter"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Filter; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", ClampMax = "1", PCG_Overridable))
	float LowerBound = 0.5f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", ClampMax = "1", PCG_Overridable))
	float UpperBound = 1.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bInvertFilter = false;
	
	/** 
	 * If checked, the result density will be remapped from [LowerBound, UpperBound] to [0, 1]. 
	 * If LowerBound == UpperBound, then the density will be set to 0.5. 
	 * Only available when we do not invert the filter, as it is not clear how to normalize disjoint ranges.
	 * Also not available if bKeepZeroDensityPoints is true, as they will not be different from points that have their density equal to lower bound.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bInvertFilter && !bKeepZeroDensityPoints"))
	bool bNormalizeOutputDensity = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug", meta = (PCG_Overridable))
	bool bKeepZeroDensityPoints = false;
#endif
};

class FPCGDensityFilterElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
