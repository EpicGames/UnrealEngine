// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPointOperationElementBase.h"
#include "PCGSettings.h"

#include "PCGDensityRemapElement.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UE_DEPRECATED(5.5, "Superseded by UPCGAttributeRemapSettings") UPCGDensityRemapSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGDensityRemapSettings();

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("DensityRemap")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDensityRemapSettings", "NodeTitle", "Density Remap"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Density; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** If InRangeMin = InRangeMax, then that density value is mapped to the average of OutRangeMin and OutRangeMax */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", ClampMax = "1", PCG_Overridable))
	float InRangeMin = 0.f;

	/** If InRangeMin = InRangeMax, then that density value is mapped to the average of OutRangeMin and OutRangeMax */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", ClampMax = "1", PCG_Overridable))
	float InRangeMax = 1.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", ClampMax = "1", PCG_Overridable))
	float OutRangeMin = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", ClampMax = "1", PCG_Overridable))
	float OutRangeMax = 1.f;

	/** Density values outside of the input range will be unaffected by the remapping */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bExcludeValuesOutsideInputRange = false;
};

class FPCGDensityRemapElement : public FPCGPointOperationElementBase
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGPointNativeProperties GetPropertiesToAllocate(FPCGContext* InContext) const override;
	virtual bool ShouldCopyPoints() const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
