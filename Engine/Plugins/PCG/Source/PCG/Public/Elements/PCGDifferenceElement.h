// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Data/PCGDifferenceData.h"

#include "PCGDifferenceElement.generated.h"

namespace PCGDifferenceConstants
{
	const FName SourceLabel = TEXT("Source");
	const FName DifferencesLabel = TEXT("Differences");
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDifferenceSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Difference")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDifferenceSettings", "NodeTitle", "Difference"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual void ApplyStructuralDeprecation(UPCGNode* InOutNode) override;
#endif
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** The density function to use when recalculating the density after the operation. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGDifferenceDensityFunction DensityFunction = EPCGDifferenceDensityFunction::Minimum;

	/** Describes how the difference operation will treat the output data:
	 * Continuous - Non-destructive data output will be maintained.
	 * Discrete - Output data will be discrete points, or explicitly converted to points.
	 * Inferred - Output data will choose from Continuous or Discrete, based on the source and operation.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGDifferenceMode Mode = EPCGDifferenceMode::Inferred;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bDiffMetadata = true;

	/** If enabled, the output will not automatically filter out points with zero density. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bKeepZeroDensityPoints = false;
};

class FPCGDifferenceElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
