// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGSettings.h"

#include "PCGDistance.generated.h"

UENUM()
enum class PCGDistanceShape
{
	SphereBounds,
	BoxBounds,
	Center,
};

namespace PCGDistance
{
	extern const FName SourceLabel;
	extern const FName TargetLabel;
}

namespace PCGDistanceConstants
{
	const FName DefaultOutputAttributeName = TEXT("Distance");
}

/**
 * Calculates the distance between two points (inherently a n*n operation)
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGDistanceSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Distance")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDistanceSettings", "NodeTitle", "Distance"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	//~End UPCGSettings interface

public:
	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface

	/** Output the distance or distance vector to an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bOutputToAttribute = true;

	/** The attribute output for the resulting distance value. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bOutputToAttribute", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertySelector OutputAttribute = FPCGAttributePropertySelector::CreateAttributeSelector(PCGDistanceConstants::DefaultOutputAttributeName);

	/** Controls whether the attribute will be a scalar or a vector */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bOutputToAttribute", EditConditionHides, PCG_Overridable))
	bool bOutputDistanceVector = false;

	/** If true, will also set the density to be 0 - 1 based on MaximumDistance */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bSetDensity = false;

	/** A maximum distance to search, which is used as an optimization */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "1", PCG_Overridable))
	double MaximumDistance = 20000.0;

	/** What shape is used on the 'source' points */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	PCGDistanceShape SourceShape = PCGDistanceShape::SphereBounds;

	/** What shape is used on the 'target' points */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	PCGDistanceShape TargetShape = PCGDistanceShape::SphereBounds;

	/** If this option is on, each source will be tested against its respective target (for a N:N operation). Source and Target num must be the same (or 1).  */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bCheckSourceAgainstRespectiveTarget = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use OutputAttribute selector instead."))
	FName AttributeName_DEPRECATED = PCGDistanceConstants::DefaultOutputAttributeName;
#endif // WITH_EDITORONLY_DATA
};

class FPCGDistanceElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};