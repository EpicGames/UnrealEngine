// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGDownloadFromGPU.generated.h"

/** Forces a readback of any GPU-resident input data to the CPU. Passthrough for data already on the CPU. */
UCLASS(BlueprintType, ClassGroup = (Procedural), meta = (Keywords = "readback"))
class UPCGDownloadFromGPUSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("DownloadFromGPU")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDownloadFromGPUElement", "NodeTitle", "Download From GPU"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGDownloadFromGPUElement", "NodeTooltip", "Forces a readback of any GPU-resident input data to the CPU. Passthrough for data already on the CPU."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::GPU; }
#endif
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGDownloadFromGPUElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return false; }
};
