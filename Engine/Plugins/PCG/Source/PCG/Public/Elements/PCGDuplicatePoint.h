// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGDuplicatePoint.generated.h"

/**
 * Creates duplicates of each point with optional transform offsets.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGDuplicatePointSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("DuplicatePoint")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGDuplicatePointElement", "NodeTitle", "Duplicate Point"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGDuplicatePointElement", "NodeTooltip", "Creates duplicates of each point with optional transform offsets."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Number of duplicates to produce. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "1", PCG_Overridable))
	int Iterations = 1;
	
	/** Direction to stack point duplicates. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "-1.0", ClampMax = "1.0", PCG_Overridable))
	FVector Direction = FVector(0.0, 0.0, 1.0);

	/** Controls whether the axis displacement will be made in relative space or not */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bDirectionAppliedInRelativeSpace = false;
	
	/** Include the source point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bOutputSourcePoint = true;

	/** Transform offset for each point duplicate */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FTransform PointTransform;
};

class FPCGDuplicatePointElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};