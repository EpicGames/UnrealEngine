// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGEditPoints.generated.h"

/**
 * Pass-through node that allows interactive point editing anywhere in the graph.
 * Connect it after any node that produces PCGPointData to enable the Edit Points viewport tool on those points.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGEditPointsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("EditPoints")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGEditPointsElement", "NodeTitle", "Edit Points"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGEditPointsElement", "NodeTooltip", "Pass-through node that allows interactive editing of incoming points."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGEditPointsElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return true; }
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
