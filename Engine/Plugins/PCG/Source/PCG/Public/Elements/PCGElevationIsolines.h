// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGElevationIsolines.generated.h"

/**
* Compute the elevation isolines of a surface, can output either points or splines.
* Currently only work for Z-up surfaces.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGElevationIsolinesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Minimum elevation of the isolines. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings, meta = (PCG_Overridable))
	double ElevationStart = 0.0;

	/** Maximum elevation of the isolines. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	double ElevationEnd = 1000.0;

	/** Increment elevation between each isolines. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	double ElevationIncrement = 100.0;

	/** Resolution of the grid for the discretization of the surface. This is the size of one cell, in cm. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	double Resolution = 100.0;

	/** Can add a tag (integer) to group output data that are at the same elevation. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bAddTagOnOutputForSameElevation = false;

	/** Option to either have Z up or project the surface normal at this position (similar to project rotations on the projection node). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bProjectSurfaceNormal = false;

	/** Will output splines rather than points. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bOutputAsSpline = false;

	/** Spline can either be curved or linear. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bOutputAsSpline", EditConditionHides))
	bool bLinearSpline = false;
};

class FPCGElevationIsolinesElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};

