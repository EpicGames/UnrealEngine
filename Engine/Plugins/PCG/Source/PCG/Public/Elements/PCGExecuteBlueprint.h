// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "Blueprint/PCGBlueprintBaseElement.h"
#include "Blueprint/PCGBlueprintDeprecatedElement.h"

#include "PCGExecuteBlueprint.generated.h"

/**
 * Helper class which can be used before calling a Blueprint function with a FPCGContext& parameter.
 *
 * Since BP calls will copy the incoming FPCGContext reference we need to clear out a couple of fields before the BP copy is made to prevent issues when the copy is destroyed.
 */
class FPCGContextBlueprintScope
{
public:
	PCG_API explicit FPCGContextBlueprintScope(FPCGContext* InContext);
	PCG_API ~FPCGContextBlueprintScope();

private:
	FPCGContext* Context = nullptr;
	TSharedPtr<FPCGContextHandle> ContextHandle = nullptr;
	TWeakPtr<FPCGGraphExecutor> GraphExecutor = nullptr;
};


UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGBlueprintSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	PCG_API UPCGBlueprintSettings();

	friend class FPCGExecuteBlueprintElement;

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ExecuteBlueprint")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGBlueprintSettings", "NodeTitle", "Execute Blueprint"); }
	PCG_API virtual FLinearColor GetNodeTitleColor() const override;
	PCG_API virtual EPCGSettingsType GetType() const override;
	PCG_API virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	PCG_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	PCG_API virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
	PCG_API virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	PCG_API virtual bool OnlyExposePreconfiguredSettings() const override;
#endif
	virtual bool UseSeed() const override { return true; }
	PCG_API virtual bool HasDynamicPins() const override;
	PCG_API virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

	PCG_API virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& InPreconfiguredsInfo) override;
	PCG_API virtual FString GetAdditionalTitleInformation() const override;
	virtual bool HasFlippedTitleLines() const override { return true; }
	// This node may have side effects, don't assume we can cull even when unwired.
	virtual bool CanCullTaskIfUnwired() const { return false; }
	PCG_API virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	PCG_API virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	/** To be removed when we support automatic override of BP params. For now always return true to force params pin. */
	virtual bool HasOverridableParams() const override { return true; }
protected:
	PCG_API virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
	PCG_API virtual TArray<FPCGSettingsOverridableParam> GatherOverridableParams() const override;
#endif // WITH_EDITOR
	PCG_API virtual void FixingOverridableParamPropertyClass(FPCGSettingsOverridableParam& Param) const override;
	// ~End UPCGSettings interface

public:
	// ~Begin UObject interface
	PCG_API virtual void PostLoad() override;
	PCG_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	PCG_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	PCG_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// ~End UObject interface
#endif

	/** Deprecated: use SetBlueprintElementType instead. */
	UFUNCTION(BlueprintCallable, Category = "Settings|Template", meta = (DeterminesOutputType = "InElementType", DynamicOutputParam = "ElementInstance"))
	PCG_API void SetElementType(TSubclassOf<UPCGBlueprintElement> InElementType, UPCGBlueprintElement*& ElementInstance);

	UFUNCTION(BlueprintCallable, Category = "Settings|Template", meta = (DeterminesOutputType = "InElementType", DynamicOutputParam = "ElementInstance"))
	PCG_API void SetBlueprintElementType(TSubclassOf<UPCGBlueprintBaseElement> InElementType, UPCGBlueprintBaseElement*& ElementInstance);

	UFUNCTION(BlueprintCallable, Category = "Settings|Template")
	TSubclassOf<UPCGBlueprintBaseElement> GetElementType() const { return BlueprintElementType; }

#if WITH_EDITOR
	UE_DEPRECATED(5.7, "Use GetElementObject instead")
	TObjectPtr<UPCGBlueprintBaseElement> GetElementInstance() const { return nullptr; }

	TObjectPtr<UPCGBlueprintBaseElement> GetElementObject() const { return BlueprintElementInstance; }
#endif

protected:
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Template)
	TSubclassOf<UPCGBlueprintBaseElement> BlueprintElementType;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Instanced, Category = "Instance", meta = (ShowOnlyInnerProperties))
	TObjectPtr<UPCGBlueprintBaseElement> BlueprintElementInstance;

#if WITH_EDITORONLY_DATA
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FName> TrackedActorTags;

	/** If this is checked, found actors that are outside component bounds will not trigger a refresh. Only works for tags for now in editor. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings)
	bool bTrackActorsOnlyWithinBounds = false;
#endif

protected:
#if WITH_EDITOR
	UE_DEPRECATED(5.6, "No longer supported")
	void OnBlueprintChanged(UBlueprint* InBlueprint) {}

	PCG_API void OnBlueprintElementChanged(UPCGBlueprintBaseElement* InElement);
	PCG_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances);
#endif

	PCG_API void RefreshBlueprintElement();
	PCG_API void SetupBlueprintEvent();
	PCG_API void TeardownBlueprintEvent();
	PCG_API void SetupBlueprintElementEvent();
	PCG_API void TeardownBlueprintElementEvent();
};

struct FPCGBlueprintExecutionContext : public FPCGContext
{
	TObjectPtr<UPCGBlueprintBaseElement> BlueprintElementInstance = nullptr;

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;
	virtual UObject* GetExternalContainerForOverridableParam(const FPCGSettingsOverridableParam& InParam) override { return BlueprintElementInstance; }
};

class FPCGExecuteBlueprintElement : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override;
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override;
	/** Set it to true by default, if there is a performance concern, we can expose a bool in the element class. */
	virtual bool ShouldVerifyIfOutputsAreUsedMultipleTimes(const UPCGSettings* InSettings) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual void PostExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* Context) const override;
	virtual FPCGContext* Initialize(const FPCGInitializeElementParams& InParams) override;
};