// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGAttributeFilter.h"
#include "Elements/PCGFilterDataBase.h"
#include "Metadata/PCGMetadataTypesConstantStruct.h"

#include "PCGFilterByAttribute.generated.h"

UENUM()
enum class EPCGFilterByAttributeMode
{
	FilterByExistence UMETA(Tooltip="Filter the data if they have a specific attribute."),
	FilterByValue UMETA(Tooltip="Filter the data by comparing the values of an attribute."),
	FilterByValueRange UMETA(Tooltip="Filter the data by comparing the values of an attribute within a range.")
};

UENUM()
enum class EPCGFilterByAttributeValueMode
{
	AnyOf UMETA(Tooltip="Any of the values satisfy the filter"),
	AllOf UMETA(Tooltip="All of the values satisfy the filter"),
};

USTRUCT(BlueprintType)
struct FPCGFilterByAttributeThresholdSettings
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseConstantThreshold = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bUseConstantThreshold", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector ThresholdAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bUseConstantThreshold", EditConditionHides, ShowOnlyInnerProperties, DisplayAfter = "bUseConstantThreshold", PCG_NotOverridable))
	FPCGMetadataTypesConstantStruct AttributeTypes;
};

USTRUCT(BlueprintType)
struct FPCGFilterByAttributeThresholdSettingsRange : public FPCGFilterByAttributeThresholdSettings
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "ShowInclusive()", EditConditionHides))
	bool bInclusive = false;
};

/** Separates data on whether they have a specific metadata attribute, depending on some criteria on attribute values. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGFilterByAttributeSettings : public UPCGFilterDataBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("FilterDataByAttribute")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGFilterByAttributeElement", "NodeTooltip", "Separates input data by whether they have the specified attribute or not, or on the data attribute value."); }
#endif
	
	virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif

	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGFilterByAttributeMode FilterMode = EPCGFilterByAttributeMode::FilterByExistence;

	/** Comma-separated list of attributes to look for */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (DisplayName = "Attributes", PCG_Overridable, EditCondition = "FilterMode == EPCGFilterByAttributeMode::FilterByExistence", EditConditionHides))
	FName Attribute;
	
	/** Domain to target for filtering existence */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "FilterMode == EPCGFilterByAttributeMode::FilterByExistence", EditConditionHides))
	FName MetadataDomain = PCGDataConstants::DefaultDomainName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "FilterMode == EPCGFilterByAttributeMode::FilterByExistence", EditConditionHides))
	EPCGStringMatchingOperator Operator = EPCGStringMatchingOperator::Equal;

	/** Controls whether properties (denoted by $) will be considered in the filter or not. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "FilterMode == EPCGFilterByAttributeMode::FilterByExistence", EditConditionHides))
	bool bIgnoreProperties = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "FilterMode != EPCGFilterByAttributeMode::FilterByExistence", EditConditionHides))
	EPCGFilterByAttributeValueMode FilterByValueMode = EPCGFilterByAttributeValueMode::AnyOf;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "FilterMode != EPCGFilterByAttributeMode::FilterByExistence", EditConditionHides))
	FPCGAttributePropertyInputSelector TargetAttribute;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "FilterMode == EPCGFilterByAttributeMode::FilterByValue", EditConditionHides))
	EPCGAttributeFilterOperator FilterOperator = EPCGAttributeFilterOperator::Greater;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "FilterMode == EPCGFilterByAttributeMode::FilterByValue", EditConditionHides))
	FPCGFilterByAttributeThresholdSettings Threshold;

	/** Threshold property/attribute/constant related properties */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "FilterMode == EPCGFilterByAttributeMode::FilterByValueRange", EditConditionHides))
	FPCGFilterByAttributeThresholdSettingsRange MinThreshold;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "FilterMode == EPCGFilterByAttributeMode::FilterByValueRange", EditConditionHides))
	FPCGFilterByAttributeThresholdSettingsRange MaxThreshold;
};

class FPCGFilterByAttributeElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
