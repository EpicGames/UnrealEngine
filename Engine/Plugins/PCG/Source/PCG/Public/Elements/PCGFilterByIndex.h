// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGFilterDataBase.h"

#include "PCGFilterByIndex.generated.h"

/** Filters a data collection based on a user defined index range expression. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGFilterByIndexSettings : public UPCGFilterDataBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif
	
	virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Will invert which indices will be included and excluded. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bInvertFilter = false;

	/**
	 * Selected individual indices or index ranges to include or exclude. Negative end indices allowed.
	 * For example, on an array of size 10: '0,2,4:5,7:-1' will include indices: 0,2,4,7,8 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FString SelectedIndices;
};

class FPCGFilterByIndexElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
