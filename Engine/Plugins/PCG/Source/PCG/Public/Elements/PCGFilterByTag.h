// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGFilterDataBase.h"

#include "PCGFilterByTag.generated.h"

UENUM()
enum class EPCGFilterByTagOperation
{
	KeepTagged,
	RemoveTagged
};

/** Filters a data collection based on some tag criterion */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGFilterByTagSettings : public UPCGFilterDataBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;

	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif
	
	virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGFilterByTagOperation Operation = EPCGFilterByTagOperation::KeepTagged;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGStringMatchingOperator Operator = EPCGStringMatchingOperator::Equal;

	/** Comma-separated list of tags */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FString SelectedTags;

	/** Enables deprecated behavior using spaces as separators. Disable to update the node to current behavior. */
	UE_DEPRECATED(5.5, "bTokenizeOnWhiteSpace has been deprecated.")
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = "bTokenizeOnWhiteSpace", EditConditionHides, DeprecationMessage = "bTokenizeOnWhiteSpace has been deprecated."))
	bool bTokenizeOnWhiteSpace = false;
};

class FPCGFilterByTagElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
