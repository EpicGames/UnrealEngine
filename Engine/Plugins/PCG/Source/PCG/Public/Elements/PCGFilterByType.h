// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGFilterDataBase.h"

#include "PCGFilterByType.generated.h"

/** Filters an input collection based on data type. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGFilterByTypeSettings : public UPCGFilterDataBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return TEXT("FilterDataByType"); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual bool ShouldDrawNodeCompact() const override { return true; }
	virtual bool GetCompactNodeIcon(FName& OutCompactNodeIcon) const override;
	virtual bool CanUserEditTitle() const override { return false; }
#endif
	virtual bool HasExecutionDependencyPin() const override { return false; }
	
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FPCGDataTypeIdentifier TargetType = EPCGDataType::Any;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bShowOutsideFilter = false;
};

class FPCGFilterByTypeElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
