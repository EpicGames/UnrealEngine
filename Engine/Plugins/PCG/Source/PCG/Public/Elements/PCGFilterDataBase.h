// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGFilterDataBase.generated.h"

#define UE_API PCG_API

UCLASS(MinimalAPI, BlueprintType, Abstract, ClassGroup = (Procedural))
class UPCGFilterDataBaseSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Filter; }
	UE_API virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif // WITH_EDITOR

	virtual bool HasDynamicPins() const override { return true; }

protected:
	UE_API virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface
};

#undef UE_API
