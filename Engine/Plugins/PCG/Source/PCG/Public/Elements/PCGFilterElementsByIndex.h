// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGFilterElementsByIndex.generated.h"

/** Filters points or the elements of an attribute set based on a second input of points, attribute sets, or a user-defined index range expression. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGFilterElementsByIndexSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("FilterElementsByIndex")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGFilterElementsByIndexElement", "NodeTitle", "Filter Elements By Index"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGFilterElementsByIndexElement", "NodeTooltip", "Filters points or the elements of an attribute set based on a second input of points, attribute sets, or a user-defined index range expression."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Filter; }
#endif // WITH_EDITOR
	virtual FString GetAdditionalTitleInformation() const override;
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** A second input will define the indices to filter. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bSelectIndicesByInput = true;

	/** The attribute which will define the indices to filter. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bSelectIndicesByInput", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector IndexSelectionAttribute;

	/** Selected individual indices or index ranges to include or exclude. Negative end indices allowed.
	* For example, on an array of size 10: '0,2,4:5,7:-1' will include indices: 0,2,4,7,8.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bSelectIndicesByInput", EditConditionHides, PCG_Overridable))
	FString SelectedIndices = TEXT(":");

	/** An additional output for discarded elements. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bOutputDiscardedElements = false;

	/** Will invert which indices will be included and excluded. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bInvertFilter = false;
};

class FPCGFilterElementsByIndexElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
