// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGSettingsWithDynamicInputs.h"

#include "PCGGather.generated.h"

/**
 * Used to wrangle multiple input wires into one output wire for organizational purposes.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGGatherSettings : public UPCGSettingsWithDynamicInputs
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Gather")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGatherSettings", "NodeTitle", "Gather"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGGatherSettings", "NodeTooltip", "Gathers multiple data in a single collection. Can also be used to order execution through the dependency-only pin."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif

	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* Pin) const override;

protected:
	virtual TArray<FPCGPinProperties> StaticInputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

	//~Begin PCGSettingsWithDynamicInputs interface
	virtual FName GetDynamicInputPinsBaseLabel() const override;

public:
#if WITH_EDITOR
	virtual void AddDefaultDynamicInputPin() override;
#endif // WITH_EDITOR
	//~End PCGSettingsWithDynamicInputs interface
};

class FPCGGatherElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool ShouldVerifyIfOutputsAreUsedMultipleTimes(const UPCGSettings* InSettings) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
};

namespace PCGGather
{
	/** Gathers the input data into a single data collection and updates the tags */
	FPCGDataCollection GatherDataForPin(const FPCGDataCollection& InputData, const FName InputLabel = PCGPinConstants::DefaultInputLabel, const FName OutputLabel = PCGPinConstants::DefaultOutputLabel);
}
