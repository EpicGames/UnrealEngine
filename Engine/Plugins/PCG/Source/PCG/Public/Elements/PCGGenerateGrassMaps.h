// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGenerateLandscapeTextures.h"

#include "PCGGenerateGrassMaps.generated.h"

UCLASS(MinimalAPI, BlueprintType, Deprecated, meta = (DeprecationMessage = "Deprecated 5.7. Use UPCGGenerateLandscapeTexturesSettings instead."))
class UDEPRECATED_PCGGenerateGrassMapsSettings : public UPCGGenerateLandscapeTexturesSettings
{
	GENERATED_BODY()
};
