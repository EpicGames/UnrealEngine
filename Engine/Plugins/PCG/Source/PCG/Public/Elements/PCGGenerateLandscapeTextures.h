// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "RendererInterface.h"

#include "PCGGenerateLandscapeTextures.generated.h"

class ALandscape;
class ALandscapeProxy;
class FLandscapeGrassWeightExporter;
class ULandscapeComponent;
class ULandscapeGrassType;
class UPCGTextureData;
class UPCGTexture2DArrayData;
class UTexture;
struct IPooledRenderTarget;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGenerateLandscapeTexturesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGGenerateLandscapeTexturesSettings();

	//~Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GenerateLandscapeTextures")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGenerateLandscapeTexturesElement", "NodeTitle", "Generate Landscape Textures"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGGenerateLandscapeTexturesElement", "NodeTooltip", "Generates landscape height texture and grass maps on the GPU."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::GPU; }
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
	virtual bool GroupPreconfiguredSettings() const override { return false; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual bool IsInputPinRequiredByExecution(const UPCGPin* InPin) const override { return true; }
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	//~End UPCGSettings interface

public:
	/** Select which grass types to generate. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bOverrideFromInput"))
	TArray<FName> SelectedGrassTypes;

	/** Override grass types from input. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bOverrideFromInput = false;

	/** Input attribute to pull grass type strings from. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bOverrideFromInput", PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector GrassTypesAttribute;

	/** Only generate grass types which are not selected. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bExcludeSelectedGrassTypes = true;

	/** Output texture data for each selected grass type. Outputs black for grass types that are not present. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bExcludeSelectedGrassTypes"))
	bool bOutputDataForEachSelectedGrassType = false;

	/** Generate a landscape height texture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bGenerateHeightMap = false;

	/** Output a single texture 2D array data rather than a texture 2D data for each slice. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bOutputTextureArray = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(ClampMin=1))
	int32 NumLandscapeComponentsGeneratedPerFrame = 1;

	/** Use the Landscape Grass Type asset's names, rather than the grass names to sample the grass maps. Mostly there for deprecation purposes : it used to be that these were the only way to address grass maps, but that is not the case anymore : the grass input's name in the Landscape Grass Output material expression is what drives the grass maps, so they should be used from now on (for example, the material can now use the same Landscape Grass Type asset for 2 different grass maps, so addressing them by asset name is not sufficient to differentiate them). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bTagDataWithLandscapeGrassTypeAssetNames = false;

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "bSkipReadbackToCPU has been removed. CPU readback is now always deferred.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecatedMessage = "bSkipReadbackToCPU has been removed. CPU readback is now always deferred."))
	bool bSkipReadbackToCPU = false;
};

struct FPCGGenerateLandscapeTexturesContext : public FPCGContext
{
public:
	/** Output height texture data. */
	TObjectPtr<UPCGTextureData> HeightTextureData;

	/** Output grass map texture data, either a texture array data or multiple texture datas, depending on settings. */
	TArray<TObjectPtr<UPCGTextureData>> GrassMapTextureDatas;
	TObjectPtr<UPCGTexture2DArrayData> GrassMapTextureArrayData;
	bool bCreatedOutputGrassTextures = false;

	struct FOutputTextureData
	{
		// Initialized and consumed on render thread during render thread, then read on game thread after all rendering is complete.
		TRefCountPtr<IPooledRenderTarget> HeightMapHandle;
		TRefCountPtr<IPooledRenderTarget> GrassMapHandle;
		bool bRenderingComplete = false;
	};

	/** Exported result textures. */
	TSharedPtr<FOutputTextureData, ESPMode::ThreadSafe> OutputTextureData;

	/** The grass names passed to the grass exporter. Each of these grass names is present on at least one of the landscape components. */
	TArray<FName> GrassNamesToExport;

	/** The grass names to output from the node. Grass names that were not present on the overlapping landscape components will emit empty/black texture data. */
	TArray<FName> GrassNamesToOutput;

	/** Maps output grass name indices to exported grass name indices. */
	TArray<int32> OutputIndexToExportIndex;

	TWeakObjectPtr<ALandscape> Landscape;
	TArray<TWeakObjectPtr<ULandscapeComponent>> LandscapeComponents;

	double LandscapeScaleZ = 1.0;
	double LandscapeRootZ = 0.0;

	/** World-space bounds containing all of the landscape components given to the grass weight exporter. */
	FBox TotalLandscapeComponentBounds = FBox(EForceInit::ForceInit);

	FBox GenerationBounds = FBox(EForceInit::ForceInit);

	/** Extent (side length) of each landscape component. */
	double LandscapeComponentExtent = 0.0;

	/** True when we have filtered all of the incoming landscape components down to the ones which overlap the given bounds. */
	bool bLandscapeComponentsFiltered = false;

	/** True when the landscape components are ready for rendering. */
	bool bReadyToRender = false;

	/** Textures that we wait to be streamed before generating the grass maps. */
	TArray<TObjectPtr<UTexture>> TexturesToStream;

	/** True when streaming has been requested on landscape textures. */
	bool bTextureStreamingRequested = false;

	/** True when grass map generation has been scheduled on the render thread. */
	bool bGenerationScheduled = false;

	int32 NumGenerationPassesScheduled = 0;

	double TexelSizeWorld = 0.0;

	FTransform OutputTextureTransform;

	FBox OutputTextureBounds = FBox(EForceInit::ForceInit);

	FIntPoint OutputResolution = FIntPoint::ZeroValue;

	TObjectPtr<UTexture> BlackTexture;

	/** Indirection table to support bTagDataWithLandscapeGrassTypeAssetNames, where grass names for PCG(that used to be the Landscape Grass Type asset names) don't match the actual landscape grass names
	 (that come from the input pins of the Landscape Grass Output Material Expression) */
	TMap<FName, FName> GrassTypeToGrassNameMap;

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "No longer used")
	int32 NumGrassTypes = 0;
	UE_DEPRECATED(5.8, "Use Landscape")
	TWeakObjectPtr<ALandscapeProxy> LandscapeProxy = nullptr;
	UE_DEPRECATED(5.8, "No longer used")
	TArray<TTuple<TWeakObjectPtr<ULandscapeGrassType>, int32>> SelectedGrassTypes;
	UE_DEPRECATED(5.8, "No longer used")
	TArray<TObjectPtr<ULandscapeGrassType>> GlobalGrassTypes;
	UE_DEPRECATED(5.8, "No longer used")
	FLandscapeGrassWeightExporter* LandscapeGrassWeightExporter = nullptr;
	UE_DEPRECATED(5.8, "Replaced with FOutputTextureData")
	TRefCountPtr<IPooledRenderTarget> HeightHandle = nullptr;
	UE_DEPRECATED(5.8, "Replaced with FOutputTextureData")
	TRefCountPtr<IPooledRenderTarget> GrassMapHandle = nullptr;
	UE_DEPRECATED(5.8, "Replaced with GrassNamesToExport")
	TArray<TObjectPtr<ULandscapeGrassType>> GrassTypesToExport;
	UE_DEPRECATED(5.8, "Replaced with GrassNamesToOutput")
	TArray<TObjectPtr<ULandscapeGrassType>> GrassTypesToOutput;
};

class FPCGGenerateLandscapeTexturesElement : public IPCGElement
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;
	/** FLandscapeGrassWeightExporter expects to exist in scope only on the game thread. */
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const { return true; }

protected:
	virtual FPCGContext* CreateContext() override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
