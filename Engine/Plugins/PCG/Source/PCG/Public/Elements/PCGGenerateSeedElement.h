// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGGenerateSeedElement.generated.h"

UENUM()
enum class EPCGGenerateSeedSource : uint8
{
	RandomStream UMETA(Tooltip = "Creates a random stream using the node's seed, and generates new seeds from that. Quick solution for seed values, but may be identical across duplicate input data."),
	HashEachSourceAttribute UMETA(Tooltip = "Hashes each value in a source attribute to determine the target seeds. Can be used to differentiate results across duplicate input data."),
	HashStringConstant UMETA(Tooltip = "Creates a target seed by combining a string's hash value and the previous seed value. Quick, human-readable solution for seed values, but may be identical across duplicate input data."),
};

/** Generate a seed from either a random stream, a constant string, or a source attribute. */
UCLASS(BlueprintType, ClassGroup = (Procedural), meta = (Keywords = "seed"))
class UPCGGenerateSeedSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGGenerateSeedSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GenerateSeed")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGenerateSeed", "NodeTitle", "Generate Seed"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGGenerateSeed", "NodeTooltip", "Generate a seed attribute "); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	virtual TArray<FText> GetNodeTitleAliases() const override { return {NSLOCTEXT("PCGGenerateSeed", "NodeTitleAlias", "Seed From Value")}; }
#endif // WITH_EDITOR
	virtual bool UseSeed() const override { return true; }
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** The source method seed attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	EPCGGenerateSeedSource GenerationSource = EPCGGenerateSeedSource::RandomStream;

	/** This value will be hashed and applied to generate each new seed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", DisplayName = "String", meta = (EditCondition = "GenerationSource == EPCGGenerateSeedSource::HashStringConstant", EditConditionHides))
	FString SourceString;

	/** The source attribute to hash to generate each new seed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (EditCondition = "GenerationSource == EPCGGenerateSeedSource::HashEachSourceAttribute", EditConditionHides))
	FPCGAttributePropertyInputSelector SeedSource;

	/** Reset the seed at the beginning of each input's generation to stay order agnostic.  */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable, EditCondition = "GenerationSource != EPCGGenerateSeedSource::HashEachSourceAttribute", EditConditionHides))
	bool bResetSeedPerInput = true;

	/** The target attribute output of the generated seed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable))
	FPCGAttributePropertyOutputSelector OutputTarget;
};

class FPCGGenerateSeedElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
