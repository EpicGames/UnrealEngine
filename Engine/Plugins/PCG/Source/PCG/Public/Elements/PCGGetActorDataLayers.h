// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGGetActorDataLayers.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetActorDataLayersSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGGetActorDataLayersSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::DataLayers; }
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PCGGetActorDataLayers")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetActorDataLayersSettings", "NodeTitle", "Get Actor Data Layers"); }
#endif
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Actor reference Attribute to use from input to resolve actors */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector ActorReferenceAttribute;

	/** Data Layer reference Attribute to use as output */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default, meta = (PCG_Overridable))
	FPCGAttributePropertyOutputSelector DataLayerReferenceAttribute;
};

class FPCGGetActorDataLayersElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};