// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/PCGActorSelector.h"
#include "Elements/PCGLoadObjectsContext.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Metadata/PCGMetadataAttribute.h"

#include "PCGGetActorProperty.generated.h"

class UActorComponent;

/**
* Extract a property value from an actor/component into a ParamData.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetActorPropertySettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGGetActorPropertySettings(const FObjectInitializer& ObjectInitializer);
	
	//~Begin UObject interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetActorProperty")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGPropertyToParamDataSettings", "NodeTitle", "Get Actor Property"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Param; }
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
#endif
	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

	virtual FString GetAdditionalTitleInformation() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
#endif
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ShowOnlyInnerProperties))
	FPCGActorSelectorSettings ActorSelector;

	/** Allow to look for an actor component instead of an actor. It will need to be attached to the found actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bSelectComponent = false;

	/** If we are looking for an actor component, the class can be specified here. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bSelectComponent", EditConditionHides))
	TSubclassOf<UActorComponent> ComponentClass;

	/** Process all Actor components. If not set, only the first component found will be processed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bSelectComponent", EditConditionHides))
	bool bProcessAllComponents = false;
	
	/** Controls whether a component reference attribute will be added to the result */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bSelectComponent", EditConditionHides))
	bool bOutputComponentReference = false;

	/** Property name to extract. Can only extract properties that are compatible with metadata types. If None, extract the actor/component directly. Can be a comma-separated list, assuming they have the same cardinality. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName PropertyName = NAME_None;

	/** In the case of multiple properties being extracted, will be ignored. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bForceObjectAndStructExtraction", EditConditionHides, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyOutputSelector OutputAttributeName;

	/** If the output attribute name has special characters, remove them. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bSanitizeOutputAttributeName = true;

	/** Controls whether an actor reference attribute will be added to the result */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bOutputActorReference = false;

	/** If this is true, we will never put this element in cache, and will always try to re-query the actors and read the latest properties from them. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bAlwaysRequeryActors = true;
	
	/** How we should extract the structs. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGStructExtractorBehavior StructExtractorBehavior = EPCGStructExtractorBehavior::ExtractRootOnly;
		
	/** How we should extract the objects. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGObjectExtractorBehavior ObjectExtractorBehavior = EPCGObjectExtractorBehavior::NoExtract;
		
	/** How we should extract the containers. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGContainerExtractorBehavior ContainerExtractorBehavior = EPCGContainerExtractorBehavior::FlattenLastAndDiscardNested;
	
	/** If the property is a struct/object supported by metadata, this option can be toggled to force extracting all (compatible) properties contained in this property. Automatically true if unsupported by metadata. For now, only supports direct child properties (and not deeper). */
	UE_DEPRECATED(5.8, "Use the different extractor behavior options")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the different extractor behavior options"))
	bool bForceObjectAndStructExtraction = false;

#if WITH_EDITORONLY_DATA
	/** If this is checked, found actors that are outside component bounds will not trigger a refresh. Only works for tags for now in editor. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings)
	bool bTrackActorsOnlyWithinBounds = false;
#endif // WITH_EDITORONLY_DATA

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	EPCGActorSelection ActorSelection_DEPRECATED;

	UPROPERTY()
	FName ActorSelectionTag_DEPRECATED;

	UPROPERTY()
	FName ActorSelectionName_DEPRECATED;

	UPROPERTY()
	TSubclassOf<AActor> ActorSelectionClass_DEPRECATED;

	UPROPERTY()
	EPCGActorFilter ActorFilter_DEPRECATED = EPCGActorFilter::Self;

	UPROPERTY()
	bool bIncludeChildren_DEPRECATED = false;
#endif
};

class FPCGGetActorPropertyElement : public IPCGElementWithCustomContext<FPCGLoadObjectsFromPathContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override;
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;
protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
