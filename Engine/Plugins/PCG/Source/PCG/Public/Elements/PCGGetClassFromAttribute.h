// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGElement.h"
#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGGetClassFromAttribute.generated.h"

/**
 * Reads a Soft Object Path, Soft Class Path, or String attribute on any PCG data entry,
 * resolves the asset class without loading the asset, and writes the short class name
 * as a string (e.g. "StaticMesh") to the output attribute.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetClassFromAttributeSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGGetClassFromAttributeSettings(const FObjectInitializer& Initializer);
	
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName("GetClassFromAttribute"); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Param; }
	virtual FString GetAdditionalTitleInformation() const override;
#endif
	virtual FPCGElementPtr CreateElement() const override;
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface

public:
	/** Attribute to read from. Accepts Soft Object Path, Soft Class Path, or String values. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector InputSource;

	/** Attribute to write the class name string to. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyOutputSelector OutputTarget;
	
	/** If we also want to output the full Soft Class Path of the class. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bAlsoOutputClassPath = false;
	
	/** Optional attribute to write the full Soft Class Path. Useful to get the class path of the input object (if it is an actor for example). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection, EditCondition = "bAlsoOutputClassPath"))
	FPCGAttributePropertyOutputSelector ClassPathOutputTarget;

	/** If true, suppresses warnings when an input path cannot be resolved to a valid class name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	bool bSilenceClassNotFoundWarnings = false;
};

class FPCGGetClassFromAttributeElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
