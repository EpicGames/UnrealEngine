// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGGetConsoleVariable.generated.h"

/**
 * Reads the given console variable and writes the value to an attribute set.
 * Note: Setting the console variable will not trigger a regeneration.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetConsoleVariableSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetConsoleVariable")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetConsoleVariableElement", "NodeTitle", "Get Console Variable"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif
	virtual FString GetAdditionalTitleInformation() const override { return ConsoleVariableName.ToString(); }

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable))
	FName ConsoleVariableName = NAME_None;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable))
	FName OutputAttributeName = NAME_None;
};

class FPCGGetConsoleVariableElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const { return false; }
};
