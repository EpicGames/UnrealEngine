// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGGetDataInfo.generated.h"

/** Extracts tags on data to an attribute set with one entry per tag. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetTagsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetTags")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetTagsElement", "NodeTitle", "Get Tags"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGGetTagsElement", "NodeTooltip", "Creates an attribute set with one entry per tag"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Param; }
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

public:
	/** Creates a 'Values' attribute and stores the values of the valued tags (e.g. 'Tag:Value' tags) as a string. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bExtractTagValues = false;

	/** Creates a 'Tags' attribute that stores the full tag string (e.g. "Tag:Value") as a string. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bExtractFullTagStrings = false;
};

class FPCGGetTagsElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};

/** Extracts the list of attribute on data to an attribute set with one entry per attribute. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetAttributesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetAttributeList")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetAttributesElement", "NodeTitle", "Get Attribute List"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGGetAttributesElement", "NodeTooltip", "Creates an attribute set with one entry per attribute on the input data."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Param; }
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Controls whether the attribute types will be extracted into the 'Type' attribute (as a name). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bGetType = false;

	/** Controls whether the default value for the attribute will be extracted into the 'DefaultValue' attribute (as a string). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bGetDefaultValue = false;
};

class FPCGGetAttributesElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};