// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "PCGGetExecutionContext.generated.h"

UENUM(BlueprintType)
enum class EPCGGetExecutionContextMode : uint8
{
	IsEditor UMETA(Tooltip="Returns whether the execution happens in the editor (not in PIE and not runtime generation (including editor viewport generation)."),
	IsRuntime UMETA(Tooltip="Returns whether the execution happens in a runtime environment (PIE or cooked build or any runtime generation (including editor viewport generation)."),
	IsOriginal UMETA(Tooltip="Returns whether the executing context is the original context (e.g. not partitioned)."),
	IsLocal UMETA(Tooltip="Returns whether the executing context is partitioned (in opposition to original)."),
	IsPartitioned UMETA(Tooltip = "Returns whether the executing context is the original context and that it is partitioned."),
	IsRuntimeGeneration UMETA(Tooltip="Returns whether the executing context is doing runtime generation."),
	IsDedicatedServer UMETA(Tooltip="Returns whether the executing context is running on a dedicated server."),
	IsListenServer UMETA(Tooltip="Returns whether the executing context is running on a listen server."),
	HasAuthority UMETA(Tooltip="Returns whether the execution context is running with network authority."),
	IsBuilder UMETA(Tooltip="Returns whether the execution source is generated by the PCG World Partition Builder. Always false in non-editor."),
	TrackingPriority UMETA(Tooltip = "Gives tracking priority of the graph instance (lower values are unaffected by changes triggered by higher values). Always 0 in non-editor."),
	IsGameWorld UMETA(Tooltip="Returns whether the execution happens in a game world.")
};

/** Returns some context-specific information in the form of an attribute set. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGGetExecutionContextSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface implementation
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetExecutionContext")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetExecutionContext", "NodeTitle", "Get Execution Context Info"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif // WITH_EDITOR

	virtual bool HasFlippedTitleLines() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface implementation

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGGetExecutionContextMode Mode = EPCGGetExecutionContextMode::IsRuntime;
};

class FPCGGetExecutionContextElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};