// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "PCGGetLoopIndex.generated.h"

/** Returns the current loop iteration index of the "nearest" subgraph in the execution stack */
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGGetLoopIndexSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface implementation
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetLoopIndex")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetLoopIndexElement", "NodeTitle", "Get Loop Index"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif // WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface implementation

public:
	/** Controls whether this node will create a warning when not called from within a loop. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bWarnIfCalledOutsideOfLoop = true;
};

class FPCGGetLoopIndexElement : public IPCGElement
{
public:
	// A loop index node is never cacheable because it needs to execute for every iteration
	virtual bool IsCacheable(const UPCGSettings* InSettings) const { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};