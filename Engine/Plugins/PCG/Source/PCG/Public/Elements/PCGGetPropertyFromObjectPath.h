// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/PCGLoadObjectsContext.h"
#include "Helpers/PCGPropertyHelpers.h"

#include "PCGGetPropertyFromObjectPath.generated.h"

/**
* Extract property from a list of soft object paths.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetPropertyFromObjectPathSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGGetPropertyFromObjectPathSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Param; }
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const override;
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
#endif
	virtual bool CanCullTaskIfUnwired() const override { return false; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** If nothing is connected in the In pin, it will use these static paths to load.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	TArray<FSoftObjectPath> ObjectPathsToExtract;

	/** If something is connected in the In pin, it will look for this attribute value for the object to load.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector InputSource;

	/** Property name to extract. Can only extract properties that are compatible with metadata types. If None, extract the object. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName PropertyName = NAME_None;

	/** In the case of multiple properties being extracted, will be ignored. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bForceObjectAndStructExtraction", EditConditionHides, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyOutputSelector OutputAttributeName;

	/** If the output attribute name has special characters, remove them. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bSanitizeOutputAttributeName = true;

	/** By default, object loading is asynchronous, can force it synchronous if needed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;

	/** Opt-in option to create empty data when there is nothing to extract or property is not found, to have the same number of inputs than outputs. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bPersistAllData = false;

	/** Opt-in option to silence errors when the path is Empty or nothing to extract. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bSilenceErrorOnEmptyObjectPath = false;
	
	/** How we should extract the structs. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGStructExtractorBehavior StructExtractorBehavior = EPCGStructExtractorBehavior::ExtractRootOnly;
		
	/** How we should extract the objects. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGObjectExtractorBehavior ObjectExtractorBehavior = EPCGObjectExtractorBehavior::NoExtract;
		
	/** How we should extract the containers. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGContainerExtractorBehavior ContainerExtractorBehavior = EPCGContainerExtractorBehavior::FlattenLastAndDiscardNested;
	
	/** If the property is a struct/object supported by metadata, this option can be toggled to force extracting all (compatible) properties contained in this property. Automatically true if unsupported by metadata. For now, only supports direct child properties (and not deeper). */
	UE_DEPRECATED(5.8, "Use the different extractor behavior options")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the different extractor behavior options"))
	bool bForceObjectAndStructExtraction = false;
};

class FPCGGetPropertyFromObjectPathElement : public IPCGElementWithCustomContext<FPCGLoadObjectsFromPathContext>
{
public:
	// Loading needs to be done on the main thread and accessing objects outside of PCG might not be thread safe, so taking the safe approach
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	// FIXME: Disable the caching to be sure that we have the most up-to-date data from external objects
	// To be re-enabled when we have more control on the cache, to be able to remove entries that were generated
	// but no longer tracked, because if this object changes but it's not tracked anymore, then it will not properly remove itself from the 
	// cache and not be caught up in future executions
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};

