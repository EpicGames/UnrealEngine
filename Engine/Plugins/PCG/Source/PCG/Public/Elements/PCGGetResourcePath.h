// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "PCGGetResourcePath.generated.h"

/** Converts a resource data to an attribute set containing the resource path. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetResourcePath : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetResourcePath")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetResourcePathElement", "NodeTitle", "Get Resource Path"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Resource; }
#endif
	virtual bool HasExecutionDependencyPin() const override { return false; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~ End UPCGSettings interface
};

class FPCGGetResourcePathElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
