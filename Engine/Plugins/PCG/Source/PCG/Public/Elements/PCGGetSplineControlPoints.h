// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "PCGSplineSampler.h"

#include "PCGGetSplineControlPoints.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetSplineControlPointsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
    virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FName ArriveTangentAttributeName = PCGSplineSamplerConstants::ArriveTangentAttributeName;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FName LeaveTangentAttributeName = PCGSplineSamplerConstants::LeaveTangentAttributeName;

	/** Name of the attribute that will contain the interpolation type. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FName InterpTypeAttributeName = PCGSplineSamplerConstants::InterpTypeAttributeName;
};

class FPCGGetSplineControlPointsElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
