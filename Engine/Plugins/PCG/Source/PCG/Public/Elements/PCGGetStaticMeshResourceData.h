// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Data/PCGStaticMeshResourceData.h"

#include "PCGGetStaticMeshResourceData.generated.h"

class UStaticMesh;

/** Creates static mesh resource data from the given soft object paths. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetStaticMeshResourceDataSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetStaticMeshResourceData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetStaticMeshResourceDataElement", "NodeTitle", "Get Static Mesh Resource Data"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Resource; }
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return bOverrideFromInput; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;

#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	//~ End UPCGSettings interface

public:
	/** Produces one resource data per entry. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bOverrideFromInput"))
	TArray<TSoftObjectPtr<UStaticMesh>> StaticMeshes;

	/** Override static meshes from input. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bOverrideFromInput = false;

	/** Input attribute to pull meshes from. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bOverrideFromInput", PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector MeshAttribute;
};

class FPCGGetStaticMeshResourceDataElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
