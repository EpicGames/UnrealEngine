// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "PCGGetSubgraphDepth.generated.h"

UENUM(BlueprintType)
enum class EPCGSubgraphDepthMode : uint8
{
	Depth UMETA(Tooltip="Depth of the dynamic subgraph with respect to the top level graph."),
	RecursiveDepth UMETA(Tooltip="Recursive depth of the current subgraph, e.g. the number of times this graph is in the execution stack.")
};

/** Returns the current call or recursion depth in the execution stack. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGGetSubgraphDepthSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface implementation
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetSubgraphDepth")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetSubgraphDepthElement", "NodeTitle", "Get Subgraph Depth"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif // WITH_EDITOR
	
	virtual bool HasFlippedTitleLines() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface implementation

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGSubgraphDepthMode Mode = EPCGSubgraphDepthMode::Depth;

	// In the case of recursive depth, it is possible to target the current graph (0), the parent graph (1) or other graphs upstream (2+).
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", EditCondition = "Mode == EPCGSubgraphDepthMode::RecursiveDepth", EditConditionHides, PCG_Overridable))
	int DistanceRelativeToUpstreamGraph = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode == EPCGSubgraphDepthMode::RecursiveDepth", EditConditionHides))
	bool bQuietInvalidDepthQueries = false;
};

class FPCGGetSubgraphDepthElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};