// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "Async/PCGAsyncLoadingContext.h"
#include "Data/PCGTexture2DArrayData.h"

#include "PCGGetTexture2DArrayData.generated.h"

class UTexture;

/** Creates texture 2D array data from a soft object path. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetTexture2DArrayDataSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~ Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetTexture2DArrayData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetTexture2DArrayDataElement", "NodeTitle", "Get Texture 2D Array Data"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGGetTexture2DArrayDataElement", "NodeTooltip", "Creates texture 2D array data from a soft object path."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~ End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bOverrideFromInput", AllowedClasses = "/Script/Engine.Texture2DArray, /Script/Engine.TextureRenderTarget2DArray", PCG_Overridable))
	TSoftObjectPtr<UTexture> Texture;

	/** Method used to determine the value for a sample based on the value of nearby texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGTextureFilter Filter = EPCGTextureFilter::Bilinear;

	/** By default, texture loading is asynchronous, can force it synchronous if needed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

struct FPCGGetTexture2DArrayDataContext : public FPCGContext, public IPCGAsyncLoadingContext {};

class FPCGGetTexture2DArrayDataElement : public IPCGElementWithCustomContext<FPCGGetTexture2DArrayDataContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* InContext) const override;
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;

protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
