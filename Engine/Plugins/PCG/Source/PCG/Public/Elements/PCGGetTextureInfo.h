// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGGetTextureInfo.generated.h"

/** Returns texture metadata as attributes. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetTextureInfoSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputWidth = true;

	/** Output attribute name for the texture width in pixels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bOutputWidth"))
	FName WidthAttributeName = TEXT("Width");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputHeight = true;

	/** Output attribute name for the texture height in pixels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bOutputHeight"))
	FName HeightAttributeName = TEXT("Height");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputNumMips = true;

	/** Output attribute name for the number of mip levels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bOutputNumMips"))
	FName NumMipsAttributeName = TEXT("NumMips");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputArraySize = true;

	/** Output attribute name for the texture array size (1 for non-array textures). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bOutputArraySize"))
	FName ArraySizeAttributeName = TEXT("ArraySize");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputFormat = true;

	/** Output attribute name for the pixel format as a human-readable string (e.g. "PF_B8G8R8A8"). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bOutputFormat"))
	FName FormatAttributeName = TEXT("Format");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputFormatIndex = true;

	/** Output attribute name for the pixel format as an integer (EPixelFormat enum value). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bOutputFormatIndex"))
	FName FormatIndexAttributeName = TEXT("FormatIndex");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputComputeFormat = true;

	/** Output attribute name for the PCG compute format as a human-readable string (e.g. "RGBA16f"). This is the pixel format narrowed to the subset of formats that support UAV access, used by PCG compute nodes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bOutputComputeFormat"))
	FName ComputeFormatAttributeName = TEXT("ComputeFormat");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputComputeFormatIndex = true;

	/** Output attribute name for the PCG compute format as an integer (EPCGRenderTargetFormat enum value). This is the pixel format narrowed to the subset of formats that support UAV access. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bOutputComputeFormatIndex"))
	FName ComputeFormatIndexAttributeName = TEXT("ComputeFormatIndex");
};

class FPCGGetTextureInfoElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
