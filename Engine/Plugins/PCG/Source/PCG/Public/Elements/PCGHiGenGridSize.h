// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGElement.h"
#include "PCGSettings.h"

#include "PCGHiGenGridSize.generated.h"

/**
 * Set the execution grid size for downstream nodes. Enables executing a single graph across a hierarchy of grids.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGHiGenGridSizeSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	uint32 GetScaledGridSize() const;

	bool ShouldOutputCellVolumeData() const;

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::HierarchicalGeneration; }
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
#endif
	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;
	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual TArray<FPCGPreconfiguredInfo> GetConversionInfo() const override;
#endif
	virtual bool HasExecutionDependencyPin() const override { return bShowInputPin; }
	virtual bool ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo) override;
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (DisplayName = "HiGen Grid Size"))
	EPCGHiGenGrid HiGenGridSize = EPCGHiGenGrid::Grid256;

	/** This property is hidden from the user and drives the behavior of the Grid Size node. */
	UPROPERTY()
	bool bShowInputPin = true;
};

class FPCGHiGenGridSizeElement : public IPCGElement
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const;
};
