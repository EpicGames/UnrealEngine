// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "PCGContext.h"

#include "PCGIndirectionElement.generated.h"

class UPCGBlueprintBaseElement;

UENUM()
enum class EPCGProxyInterfaceMode : uint8
{
	ByNativeElement = 0u UMETA(Tooltip = "Select a native element to define the pin interface"),
	ByBlueprintElement UMETA(Tooltip = "Select a custom blueprint element to define the pin interface"),
	BySettings UMETA(Tooltip = "User defined settings will define the pin interface")
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGIndirectionSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
#endif // WITH_EDITOR
	virtual FString GetAdditionalTitleInformation() const override;
	virtual bool HasFlippedTitleLines() const override { return true; }
	// TODO: This is a safe default and could likely be refined based on the actual indirection target (especially if static).
	virtual bool CanCullTaskIfUnwired() const { return false; }

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif // WITH_EDITOR
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Defines which interface to use for populating pins */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGProxyInterfaceMode ProxyInterfaceMode = EPCGProxyInterfaceMode::BySettings;

	/** The element settings class used to define the pin interface for this node instance */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = "ProxyInterfaceMode == EPCGProxyInterfaceMode::ByNativeElement", EditConditionHides))
	TSubclassOf<UPCGSettings> SettingsClass;

	/** The blueprint element class used to define the pin interface for this node instance */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = "ProxyInterfaceMode == EPCGProxyInterfaceMode::ByBlueprintElement", EditConditionHides))
    TSubclassOf<UPCGBlueprintBaseElement> BlueprintElementClass;

	/** The element settings, which can be overriden, that will be used during the proxy execution */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	TObjectPtr<UPCGSettings> Settings;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bTagOutputsBasedOnOutputPins = true;
};

struct FPCGIndirectionContext : public FPCGContext
{
public:
	virtual ~FPCGIndirectionContext();

	void SetupInnerContext();
protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;

public:
	FPCGElementPtr InnerElement;
	FPCGContext* InnerContext = nullptr;
	bool bShouldActAsPassthrough = false;	

	TObjectPtr<UPCGSettings> InnerSettings = nullptr;
};

class FPCGIndirectionElement : public IPCGElement
{
public:
	// TODO: investigate how we could make this cacheable, might require to pass in context instead of settings
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;

protected:
	virtual FPCGContext* CreateContext() override;
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};