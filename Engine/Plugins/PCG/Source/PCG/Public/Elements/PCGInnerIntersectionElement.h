// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Data/PCGIntersectionData.h"

#include "PCGInnerIntersectionElement.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGInnerIntersectionSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Inner Intersection")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGInnerIntersectionSettings", "NodeTitle", "Inner Intersection"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif
	

	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	// If node disabled, don't intersect - pass through first edge
	virtual bool OnlyPassThroughOneEdgeWhenDisabled() const { return true; }

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGIntersectionDensityFunction DensityFunction = EPCGIntersectionDensityFunction::Multiply;

	/** If enabled, output points with a density value of 0 will NOT be automatically filtered out. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bKeepZeroDensityPoints = false;
};

class FPCGInnerIntersectionElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
