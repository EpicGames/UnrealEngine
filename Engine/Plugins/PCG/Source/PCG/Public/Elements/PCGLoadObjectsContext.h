// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Metadata/PCGAttributePropertySelector.h"

struct FPCGLoadObjectsFromPathContext : public FPCGContext, public IPCGAsyncLoadingContext
{
	PCG_API bool InitializeAndRequestLoad(
		FName InputPinName,
		const FPCGAttributePropertyInputSelector& InputAttributeSelector,
		const TArray<FSoftObjectPath>& StaticObjectPaths,
		bool bPersistAllData,
		bool bSilenceErrorOnEmptyObjectPath,
		bool bSynchronousLoad
	);

	TArray<TTuple<FSoftObjectPath, int32, int32>> PathsToObjectsAndDataIndex;
};