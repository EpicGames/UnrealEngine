// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSubgraph.h"

#include "PCGLoopElement.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGLoopSettings : public UPCGSubgraphSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface implementation
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject interface implementation

	//~Begin UPCGSettings interface implementation
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Loop")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
	virtual bool GetPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip) const override;
#endif

protected:
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override { return Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic; }
#endif
	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface implementation

public:
	//~Begin UPCGBaseSubgraphSettings interface
	virtual bool IsDynamicGraph() const override { return true; }
	//~End UPCGBaseSubgraphSettings interface

	/** This method is used to retrieve the loop & feedback pin names either from the graph (if using the default pin usage) or from the comma-separated list otherwise. */
	void GetLoopPinNames(FPCGContext* Context, TArray<FName>& LoopPinNames, TArray<FName>& FeedbackPinNames, bool bQuiet) const;

	/** Controls whether the pin usage (normal, loop, feedback) will be taken from the subgraph to execute or from the manually provided list. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data)
	bool bUseGraphDefaultPinUsage = true;

	/** 
	* Comma-separated list of pin names on which we will loop by-element in a step-wise fashion; if more than one is provided, it is expected that they all have the same number of data.
	* If none are provided, the first connected pin will taken as the pin to loop on. 
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (EditCondition = "!bUseGraphDefaultPinUsage"))
	FString LoopPins;

	/** 
	* Comma-separated list of pin names that will act as feedback pins, namely that in a given iteration it will receive the data from the output pin of the same name of the previous loop iteration.
	* These pins can have initial data, in which case only the first iteration will get this data.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (EditCondition = "!bUseGraphDefaultPinUsage"))
	FString FeedbackPins;

	/** Enables deprecated behavior using spaces as separators. Disable to update the node to current behavior. */
	UE_DEPRECATED(5.5, "bTokenizeOnWhiteSpace has been deprecated.")
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = "bTokenizeOnWhiteSpace", EditConditionHides, DeprecationMessage = "bTokenizeOnWhiteSpace has been deprecated."))
	bool bTokenizeOnWhiteSpace = false;
};

class FPCGLoopElement : public FPCGSubgraphElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	void PrepareLoopDataCollections(FPCGContext* Context, const UPCGLoopSettings* Settings, TArray<FPCGDataCollection>& LoopDataCollection, FPCGDataCollection& FeedbackDataCollection, TArray<FName>& FeedbackPinNames, FPCGDataCollection& FixedInputDataCollection) const;
};

/** This element does the same static input forwarding mechanism as the FPCGInputForwardingElement, but it also will go fetch the previous iteration data, if any. */
class FPCGLoopInputForwardingElement : public FPCGInputForwardingElement
{
public:
	FPCGLoopInputForwardingElement(const FPCGDataCollection& StaticInputToForward, FPCGTaskId InPreviousIterationTaskId, const TArray<FName>& InFeedbackPinNames);

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;

	FPCGTaskId PreviousIterationTaskId = InvalidPCGTaskId;
	TArray<FName> FeedbackPinNames;
};