// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGMakeConcreteElement.generated.h"

/** Makes data Concrete, collapsing to Point if necessary. Discards non-Spatial data. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMakeConcreteSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MakeConcrete")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual bool ShouldDrawNodeCompact() const override { return true; }
#endif
	virtual bool HasExecutionDependencyPin() const override { return false; }

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGMakeConcreteElement : public IPCGElement
{
public:
	// Might be sampling spline/landscape or other external data, worth computing a full CRC in case we can halt change propagation/re-executions
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* InContext) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
