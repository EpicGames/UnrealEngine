// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "PCGSettings.h"
#include "Elements/PCGTimeSlicedElementBase.h"
#include "Metadata/PCGMetadataTypesConstantStruct.h"

#include "PCGMatchAndSetAttributes.generated.h"

// Defined in the cpp file
class FPCGMatchAndSetPartition;

struct FPCGMatchAndSetAttributesExecutionState
{
	~FPCGMatchAndSetAttributesExecutionState();

	FPCGMatchAndSetPartition* Partition = nullptr;
};

struct FPCGMatchAndSetAttributesIterationState
{
	~FPCGMatchAndSetAttributesIterationState();

	int CurrentIndex = 0;
	const UPCGData* InData = nullptr;
	UPCGData* OutData = nullptr;

	FPCGMatchAndSetPartition* Partition = nullptr;
	bool bIsPartitionOwner = false;
};

UENUM()
enum class EPCGMatchMaxDistanceMode
{
	NoMaxDistance UMETA(DisplayName="No maximum distance"),
	UseConstantMaxDistance UMETA(DisplayName="Use constant maximum distance"),
	AttributeMaxDistance
};

/** This class creates a PCG node that can match, select by weight or match & select by weight 
* a 'matching' entry in a provided Attribute Set with multiple entries.
* E.g. for a given point, if the point has the same specified attribute as the matching attribute in the attribute set,
* then we will copy all the other non-selection attributes to the point.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMatchAndSetAttributesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGMatchAndSetAttributesSettings();

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	virtual bool HasDynamicPins() const { return true; }
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif // WITH_EDITOR
	virtual bool UseSeed() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	// ~End UPCGSettings interface

public:
	/** Controls whether selection of the attribute set values to copy will be done by matching point-to-attribute set (true) or done randomly (false). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bMatchAttributes = false;

	/** Attribute from the point data to select & match. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bMatchAttributes", PCG_Overridable))
	FPCGAttributePropertyInputSelector InputAttribute;

	/** Attribute from the attribute set to match against. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bMatchAttributes", PCG_Overridable, PCG_DiscardPropertySelection))
	FPCGAttributePropertyInputSelector MatchAttribute;

	/** Controls whether points that have no valid match in the attribute set are kept as is (default values) or removed from the output. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bMatchAttributes", PCG_Overridable))
	bool bKeepUnmatched = true;

	/** Controls whether the match operation will return the nearest match and not only match on equality. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bMatchAttributes", PCG_Overridable))
	bool bFindNearest = false;

	/** Controls whether the match operation has a maximum distance on which to reject points that would be too far from the nearest value. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bFindNearest", EditConditionHides, PCG_Overridable))
	EPCGMatchMaxDistanceMode MaxDistanceMode = EPCGMatchMaxDistanceMode::NoMaxDistance;

	/** Constant value that establishes the maximum distance an entry can be from its nearest match to be selected */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bFindNearest && MaxDistanceMode==EPCGMatchMaxDistanceMode::UseConstantMaxDistance", EditConditionHides))
	FPCGMetadataTypesConstantStruct MaxDistanceForNearestMatch;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bFindNearest && MaxDistanceMode==EPCGMatchMaxDistanceMode::AttributeMaxDistance", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector MaxDistanceInputAttribute;

	/** Controls whether we will use the attribute provided in the Input Weight Attribute to perform entry selection. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bUseInputWeightAttribute = false;

	/** Input weight from the points, assumed to be in the [0, 1] range. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bUseInputWeightAttribute", PCG_Overridable))
	FPCGAttributePropertyInputSelector InputWeightAttribute;

	/** Controls whether we will consider the weights, as determined by the Weight Attribute values on the attribute set. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (DisplayName = "Use Match Weight", InlineEditConditionToggle, PCG_Overridable))
	bool bUseWeightAttribute = false;

	/** Attribute to weight more or less some entries from the attribute set. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (DisplayName="Match Weight Attribute", EditCondition = "bUseWeightAttribute", PCG_Overridable, PCG_DiscardPropertySelection))
	FPCGAttributePropertyInputSelector WeightAttribute;

	/** Controls whether we will emit a warning and return nothing if there is no provided attribute set. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bWarnIfNoMatchData = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bWarnOnAttributeCast = true;
};

class FPCGMatchAndSetAttributesElement : public TPCGTimeSlicedElementBase<FPCGMatchAndSetAttributesExecutionState, FPCGMatchAndSetAttributesIterationState>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::PrimaryPinAndBroadcastablePins; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
