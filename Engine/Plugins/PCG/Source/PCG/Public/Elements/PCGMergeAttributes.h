// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettingsWithDynamicInputs.h"
#include "PCGMergeElement.h"

#include "PCGMergeAttributes.generated.h"

/** Merges multiple attribute sets together into a single attribute set */
UCLASS(MinimalAPI, BlueprintType, Classgroup = (Procedural))
class UPCGMergeAttributesSettings : public UPCGSettingsWithDynamicInputs
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return TEXT("MergeAttributes"); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
#endif // WITH_EDITOR
	

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin UPCGDynamicSettings interface
	virtual FName GetDynamicInputPinsBaseLabel() const override;
	/** The input pin properties that are statically defined */
	virtual TArray<FPCGPinProperties> StaticInputPinProperties() const override;

public:
#if WITH_EDITOR
	virtual void AddDefaultDynamicInputPin() override;
#endif // WITH_EDITOR
	//~End UPCGDynamicSettings interface

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGMergeTagsMode MergeTagsMode = EPCGMergeTagsMode::MergeAllTags;
};

class FPCGMergeAttributesElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};