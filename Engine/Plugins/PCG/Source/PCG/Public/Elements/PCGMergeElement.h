// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettingsWithDynamicInputs.h"

#include "PCGMergeElement.generated.h"

UENUM(BlueprintType)
enum class EPCGMergeTagsMode : uint8
{
	MergeAllTags,
	MergeTagsFromFirstPin,
	SelectTagsFromFirstData,
	None
};

/** Merges multiple data sources (currently only points supported) into a single output. */
UCLASS(MinimalAPI, BlueprintType, Classgroup = (Procedural))
class UPCGMergeSettings : public UPCGSettingsWithDynamicInputs
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MergePoints")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGMergeSettings", "NodeTitle", "Merge Points"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin UPCGDynamicSettings interface
	virtual FName GetDynamicInputPinsBaseLabel() const override;
	/** The input pin properties that are statically defined */
	virtual TArray<FPCGPinProperties> StaticInputPinProperties() const override;

public:
#if WITH_EDITOR
	virtual void AddDefaultDynamicInputPin() override;
#endif // WITH_EDITOR
	//~End UPCGDynamicSettings interface

	/** Controls whether the resulting merge data will have any metadata */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bMergeMetadata = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGMergeTagsMode MergeTagsMode = EPCGMergeTagsMode::MergeAllTags;
};

class FPCGMergeElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
