// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPointOperationElementBase.h"
#include "PCGSettings.h"

#include "PCGMutateSeed.generated.h"

/**
 * Generates a new seed for each point using its position and user seed input and applies to all points.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGMutateSeedSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MutateSeed")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGMutateSeedElement", "NodeTitle", "Mutate Seed"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGMutateSeedElement", "NodeTooltip", "Applies a new random seed from point input."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif
	virtual bool UseSeed() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGMutateSeedElement : public FPCGPointOperationElementBase
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGPointNativeProperties GetPropertiesToAllocate(FPCGContext* InContext) const override;
	virtual bool ShouldCopyPoints() const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
