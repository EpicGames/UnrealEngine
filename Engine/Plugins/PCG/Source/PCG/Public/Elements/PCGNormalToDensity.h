// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGPointOperationElementBase.h"
#include "PCGSettings.h"

#include "PCGNormalToDensity.generated.h"

UENUM()
enum class PCGNormalToDensityMode : int8
{
	Set,
	Minimum,
	Maximum,
	Add,
	Subtract,
	Multiply,
	Divide
};

/**
 * Finds the angle against the specified direction and applies that to the density
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGNormalToDensitySettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("NormalToDensity")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGNormalToDensitySettings", "NodeTitle", "Normal To Density"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }

	virtual bool DisplayExecuteOnGPUSetting() const override { return true; }
	virtual void CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const override;
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:

	// The normal to compare against
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FVector Normal = FVector::UpVector;

	// This biases the value towards or against the normal (positive or negative)
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	double Offset = 0.0;

	// This applies a curve to scale the result density with Result = Result^(1/Strength)
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = "0.0001", ClampMax = "100.0"))
	double Strength = 1.0;

	// The operator to apply to the output density 
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	PCGNormalToDensityMode DensityMode = PCGNormalToDensityMode::Set;
};

class FPCGNormalToDensityElement : public FPCGPointOperationElementBase
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGPointNativeProperties GetPropertiesToAllocate(FPCGContext* InContext) const override;
	virtual bool ShouldCopyPoints() const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};

