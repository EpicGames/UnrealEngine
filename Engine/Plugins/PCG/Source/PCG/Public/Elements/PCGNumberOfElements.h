// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGNumberOfElements.generated.h"

class UPCGParamData;
class UPCGBasePointData;

/**
* Elements for getting the number of elements in a point data or a param data. Since the whole logic is identical
* except for getting the number of elements, it is factorized in a base class.
*/

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGNumberOfElementsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetElementsCount")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGNumberOfElementsSettings", "NodeTitleElement", "Get Element Count"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGNumberOfElementsSettings", "NodeTooltipElement", "Return the number of elements in the input data."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	virtual TArray<FText> GetNodeTitleAliases() const override { return {NSLOCTEXT("PCGNumberOfElementsElement", "NodeTitleAlias", "Get Points Count")}; }
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName OutputAttributeName = "NumEntries";
};

class FPCGNumberOfElementsElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
