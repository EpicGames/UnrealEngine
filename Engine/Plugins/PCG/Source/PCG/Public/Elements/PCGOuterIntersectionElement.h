// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettingsWithDynamicInputs.h"
#include "Data/PCGIntersectionData.h"

#include "PCGOuterIntersectionElement.generated.h"

namespace PCGIntersectionConstants
{
	const FName PrimaryLabel = TEXT("Primary Source");
	const FName SecondaryLabel = TEXT("Source");
	const FText SecondaryTooltip = NSLOCTEXT("PCGOuterIntersectionElement", "SecondaryPinLabel",
		"Secondary pin inputs will be implicitly unioned together before being compared to the primary pin's input for calculation of the intersection operation. "
		"Empty data passed along on one of the secondary pins will result in an empty intersection output, unless the 'Ignore Empty Secondary Input' flag is enabled.");
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGOuterIntersectionSettings : public UPCGSettingsWithDynamicInputs
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	//~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Intersection")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGOuterIntersectionElement", "NodeTitle", "Intersection"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif // WITH_EDITOR
	// If node disabled, don't intersect - pass through all primary edges
	virtual bool OnlyPassThroughOneEdgeWhenDisabled() const override { return false; }

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin UPCGDynamicSettings interface
	virtual FName GetDynamicInputPinsBaseLabel() const override;
	/** The input pin properties that are statically defined */
	virtual TArray<FPCGPinProperties> StaticInputPinProperties() const override;

#if WITH_EDITOR
public:
	virtual void AddDefaultDynamicInputPin() override;
#endif // WITH_EDITOR
	//~End UPCGDynamicSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGIntersectionDensityFunction DensityFunction = EPCGIntersectionDensityFunction::Multiply;

	/** If enabled, dynamic input pins that have no incoming data will be bypassed during the intersection operation
	 *  calculation, which would otherwise result in an empty result. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bIgnorePinsWithNoInput = false;

	/** If enabled, output points with a density value of 0 will NOT be automatically filtered out. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bKeepZeroDensityPoints = false;
};

class FPCGOuterIntersectionElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};