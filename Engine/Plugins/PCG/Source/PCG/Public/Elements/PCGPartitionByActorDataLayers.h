// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Helpers/PCGDataLayerHelpers.h"

#include "PCGPartitionByActorDataLayers.generated.h"

class UDataLayerAsset;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPartitionByActorDataLayersSettings : public UPCGSettings
{
	GENERATED_BODY()

	UPCGPartitionByActorDataLayersSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::DataLayers; }
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PartitionByActorDataLayers")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGPartitionByActorDataLayersSettings", "NodeTitle", "Partition by Actor Data Layers"); }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	//~End UPCGSettings interface
		
public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector ActorReferenceAttribute;
		
	/** Data Layer reference Attribute to use as output for Data Layer Partitions */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyOutputSelector DataLayerReferenceAttribute;

	/** When left empty, all Data Layers are included, if any Data Layers are specified, only those will be included */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FPCGDataLayerReferenceSelector IncludedDataLayers;

	/** Specified Data Layers will get excluded */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FPCGDataLayerReferenceSelector ExcludedDataLayers;
};

class FPCGPartitionByActorDataLayersElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};