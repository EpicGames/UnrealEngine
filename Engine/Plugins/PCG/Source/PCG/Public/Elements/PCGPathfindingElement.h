// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Data/PCGWorldData.h"
#include "Elements/PCGTimeSlicedElementBase.h"
#include "SpatialAlgo/PCGAStar.h"

#include "PCGPathfindingElement.generated.h"

class UPCGPointData;

UENUM(BlueprintType)
enum class EPCGPathfindingSplineMode : uint8
{
	Curve UMETA(Tooltip = "Interpret the spline as a continuous curve."),
	Linear UMETA(Tooltip = "Interpret the spline as a conjunction of linear segments."),
};

UENUM(BlueprintType)
enum class EPCGPathfindingCostFunctionMode : uint8
{
	Distance UMETA(Tooltip = "Pathfinding cost will be the distance only."),
	FitnessScore UMETA(Tooltip = "Pathfinding cost will be driven by a fitness score (0-1 range), with a maximum penalty applied at fitness = 0."),
	CostMultiplier UMETA(Tooltip = "Pathfinding cost will be the distance multiplied by the provided factor. Note that multipliers below 1 will be clamped to 1.")
};

UENUM(BlueprintType)
enum class EPCGPathfindingGoalMappingMode : uint8
{
	EachStartToNearestGoal UMETA(Tooltip = "For each starting location, find the optimal path to any of the goal locations. There will be one attempted path from each starting location."),
	EachStartToEachGoal UMETA(Tooltip = "For each starting location, find the optimal path to each of the goal locations. There will be an attempted path from each starting location to every goal. Ex. S1->G1, S1->G2, S2->G1, S2->G2."),
	EachStartToPairwiseGoal UMETA(Tooltip = "Map each starting location to a consecutively corresponding goal location as a pair, and find the optimal path between them. Input count must match one-to-one.")
};

/** Finds the optimal path across the points of a given point cloud--should one exist--when provided a start and goal
 * location, and a maximum jump distance between points. Can return a partial path.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural), hidecategories="Data|Attributes")
class UPCGPathfindingSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGPathfindingSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PathfindingElement")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGPathfindingElement", "NodeTitle", "Pathfinding"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGPathfindingElement", "NodeTooltip", "Finds the optimal path across the points of a given point cloud--should one exist--when provided a start and goal location, and a maximum jump distance between points. Can return a partial path."); }
#endif // WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
#endif
	//~End UPCGSettings interface

public:
	/** The max distance from each point to search for the next viable point in the path. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.01", PCG_Overridable))
	double SearchDistance = 1000;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bStartLocationsAsInput = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bStartLocationsAsInput", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector StartLocationAttribute;

	/** The location the pathfinding should attempt to reach. Not used when using start locations from an input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "!bStartLocationsAsInput", EditConditionHides, PCG_Overridable))
	FVector Start = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bGoalLocationsAsInput = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bGoalLocationsAsInput", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector GoalLocationAttribute;

	/** The location the pathfinding should attempt to reach. Not used when using goal locations from an input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "!bGoalLocationsAsInput", EditConditionHides, PCG_Overridable))
	FVector Goal = FVector::ZeroVector;

	/** How each goal location correlates to each start location. Only relevant when using multiple start and goal locations as input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	EPCGPathfindingGoalMappingMode GoalMappingMode = EPCGPathfindingGoalMappingMode::EachStartToNearestGoal;

	/** The heuristic estimates a faster path to speed up processing. A higher than 1 heuristic weight can be faster, but it may cease being the optimal path. A weight of 0 is essentially flood fill. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "0.0", PCG_Overridable))
	double HeuristicWeight = 1.0;

	/** Controls whether the cost function will use a given attribute as a scalar wrt to the distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	EPCGPathfindingCostFunctionMode CostFunctionMode = EPCGPathfindingCostFunctionMode::Distance;

	/** Attribute to use as part of the cost function - it's meaning will depend on the cost function mode (fitness value, scalar multiplier, or else). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "CostFunctionMode != EPCGPathfindingCostFunctionMode::Distance", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector CostAttribute;

	/** Fitness penalty scalar (maximum penalty applied when fitness is zero.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin = "1.0", EditCondition = "CostFunctionMode == EPCGPathfindingCostFunctionMode::FitnessScore", EditConditionHides, PCG_Overridable))
	double MaximumFitnessPenaltyFactor = 10.0;

	/** Controls whether raycasts will be used to test for collisions along the path (hit results will be considered obstacles for the pathfinding). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	bool bUsePathTraces = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bUsePathTraces", EditConditionHides, PCG_Overridable, ShowOnlyInnerProperties))
	FPCGWorldRaycastQueryParams PathTraceParams;

	/** Even if the path is not complete, return a viable partial path to the point closest to the goal. Output data will be tagged with "CompletePath" or "PartialPath", depending on the result, if enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	bool bAcceptPartialPath = true;

	/** The final path will be a spline. If false, the final path will be an ordered point data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bOutputAsSpline = true;

	/** Determines how the output spline's curves will be calculated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bOutputAsSpline", EditConditionHides, PCG_Overridable))
	EPCGPathfindingSplineMode SplineMode = EPCGPathfindingSplineMode::Curve;

	/** Copy the properties and attributes from the originating point input to the output points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "!bOutputAsSpline", EditConditionHides, PCG_Overridable))
	bool bCopyOriginatingPoints = false;
};

namespace PCGPathfindingElement
{
	struct FExecutionState
	{
		/** The starting points of the paths to search. One search iteration will happen for each starting point. */
		FPCGPointInputRange StartPoints;
		/** The goal points of the paths to search for. If more than one goal exists, the heuristic will be ignored. */
		FPCGPointInputRange GoalPoints;

		/** The total number of paths to find. */
		int32 IterationCount = 0;
		PCGSpatialAlgo::AStar::FSearchSettings Settings;
	};

	struct FIterationState
	{
		/** Tracks which path this iteration should follow based on the start points. */
		int32 PathIterationIndex = 0;
		/** To compare against PathIterationIndex to know if the iteration has changed, for re-initialization of the search. */
		int32 LastPathIterationIndex = INDEX_NONE;

		/** The starting points for the current iteration. needs to stay alive until the end of iteration */
		FPCGPointInputRange StartPoints;
		/** The goal points for the current iteration. needs to stay alive until the end of iteration */
		FPCGPointInputRange GoalPoints;

		PCGSpatialAlgo::AStar::FSearchState SearchState;
	};
}

class FPCGPathfindingElement : public TPCGTimeSlicedElementBase<PCGPathfindingElement::FExecutionState, PCGPathfindingElement::FIterationState>
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override;

protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;

	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }

};
