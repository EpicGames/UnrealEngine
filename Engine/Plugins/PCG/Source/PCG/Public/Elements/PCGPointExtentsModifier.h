// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPointOperationElementBase.h"
#include "PCGSettings.h"

#include "PCGPointExtentsModifier.generated.h"

UENUM()
enum class EPCGPointExtentsModifierMode : uint8
{
	Set,
	Minimum,
	Maximum,
	Add,
	Multiply
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPointExtentsModifierSettings : public UPCGSettings
{
	GENERATED_BODY()

public:

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ExtentsModifier")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGPointExtentsModifierSettings", "NodeTitle", "Extents Modifier"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FVector Extents = FVector::One();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGPointExtentsModifierMode Mode = EPCGPointExtentsModifierMode::Set;
};

class FPCGPointExtentsModifier : public FPCGPointOperationElementBase
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGPointNativeProperties GetPropertiesToAllocate(FPCGContext* InContext) const override;
	virtual bool ShouldCopyPoints() const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};