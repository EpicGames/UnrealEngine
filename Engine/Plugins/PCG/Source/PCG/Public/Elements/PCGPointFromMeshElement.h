// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Async/PCGAsyncLoadingContext.h"

#include "UObject/SoftObjectPtr.h"

#include "PCGPointFromMeshElement.generated.h"

class UStaticMesh;

// PointFromMesh creates a single point at the origin with an attribute containing a SoftObjectPath to the selected UStaticMesh/USkeletalMesh
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPointFromMeshSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
	void PostLoad();
	//~End UPCGSettings interface
	
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PointFromMesh")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; };
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases="StaticMesh", AllowedClasses="/Script/Engine.StaticMesh, /Script/Engine.SkeletalMesh"))
	TSoftObjectPtr<UObject> Mesh;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Mesh property instead"))
	TSoftObjectPtr<UStaticMesh> StaticMesh;
#endif // WITH_EDITORONLY_DATA

	/** Name of the string attribute to be created and hold a SoftObjectPath to the Mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName MeshPathAttributeName = NAME_None;

	/** By default, mesh loading is asynchronous, can force it synchronous if needed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;
};

struct FPCGPointFromMeshContext : public FPCGContext, public IPCGAsyncLoadingContext {};

class FPCGPointFromMeshElement : public IPCGElementWithCustomContext<FPCGPointFromMeshContext>
{
public:
	// Loading needs to be done on the main thread and accessing objects outside of PCG might not be thread safe, so taking the safe approach
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
