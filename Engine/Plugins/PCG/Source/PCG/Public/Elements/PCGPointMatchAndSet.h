// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataTypesConstantStruct.h"
#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGPointMatchAndSet.generated.h"

class UPCGMatchAndSetBase;

/** This settings class is used to create a PCG node that will apply a "Match and Set" operation
* on the point data it consumes as input.
* E.g. for a given point, if it matches with something in the Match & Set object, it will set a value on the point.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural), meta = (PrioritizeCategories = "Settings"))
class UPCGPointMatchAndSetSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGPointMatchAndSetSettings(const FObjectInitializer& ObjectInitializer);

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PointMatchAndSet")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGPointMatchAndSetSettings", "NodeTitle", "Point Match And Set"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Metadata; }
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
#endif // WITH_EDITOR
	virtual bool UseSeed() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	// ~Begin UObject interface
	void PostLoad();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// ~End UObject interface

	/** Recreates the match & set instance stored in this settings object if needed. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	PCG_API void SetMatchAndSetType(TSubclassOf<UPCGMatchAndSetBase> InMatchAndSetType);

protected:
	void RefreshMatchAndSet();

public:
	/** Defines the type of Match & Set object to use. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, NoClear, Category = MatchAndSet)
	TSubclassOf<UPCGMatchAndSetBase> MatchAndSetType;

	/** Instance of MatchAndSetType, stores the data that will be used in these settings. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Instanced, Category = MatchAndSet)
	TObjectPtr<UPCGMatchAndSetBase> MatchAndSetInstance;

	/** "Set" part of the Match & Set - defines what will be changed in the operation */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyOutputSelector SetTarget;

	/** If the "Set" part is an attribute, then the type must be provided */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bSetTargetIsAttribute", HideEditConditionToggle, EditConditionHides))
	EPCGMetadataTypes SetTargetType = EPCGMetadataTypes::Double;

private:
#if WITH_EDITORONLY_DATA
	// Property used to sidestep edit condition issues - reflects SetTarget.Selection == Attribute
	UPROPERTY()
	bool bSetTargetIsAttribute = true;

	/** For string types, the subtype is used to cleanup the UI. */
	UPROPERTY()
	EPCGMetadataTypesConstantStructStringMode SetTargetStringMode_DEPRECATED;
#endif
};

class FPCGPointMatchAndSetElement : public IPCGElement
{
public:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
