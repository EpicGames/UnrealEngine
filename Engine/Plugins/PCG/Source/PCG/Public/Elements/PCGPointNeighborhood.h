// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGSettings.h"

#include "PCGPointNeighborhood.generated.h"

UENUM()
enum class EPCGPointNeighborhoodDensityMode
{
	None,
	SetNormalizedDistanceToDensity,
	SetAverageDensity
};

/**
* Computes quantities from nearby neighbor points, such as average density, color, and position.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGPointNeighborhoodSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PointNeighborhood")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGPointNeighborhoodElement", "NodeTitle", "Point Neighborhood"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGPointNeighborhoodElement", "NodeTooltip", "Computes quantities from nearby neighbor points, such as average density, color, and position."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", PCG_Overridable))
	double SearchDistance = 500.0;

	/** Allows the non-normalized distance to be output into a user-generated attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bSetDistanceToAttribute = false;
	
	/** The output attribute name to write the non-normalized distance, if not "None". */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bSetDistanceToAttribute", PCG_Overridable))
	FName DistanceAttribute = TEXT("Distance");

	/** Allows the average position to be output into a user-generated attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bSetAveragePositionToAttribute = false;

	/** The output attribute name to write the average positions, if not "None". */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bSetAveragePositionToAttribute", PCG_Overridable))
	FName AveragePositionAttribute = TEXT("AvgPosition");

	/** Writes either the normalized distance or the average density to the point density. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGPointNeighborhoodDensityMode SetDensity = EPCGPointNeighborhoodDensityMode::None;

	/** Writes the average position to the point transform. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bSetAveragePosition = false;

	/** Writes the target color to the point color if true, otherwise keeps the source color. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bSetAverageColor = false;

	/** Takes the bounds into account when projecting points. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bWeightedAverage = false;
};

class FPCGPointNeighborhoodElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
