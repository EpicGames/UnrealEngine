// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGTimeSlicedElementBase.h"

#define UE_API PCG_API

class UPCGPointData;
struct FPCGPoint;

namespace PCGPointOperation
{
	namespace Constants
	{
		/** The default minimum number of points to execute per async slice */
		static constexpr int32 PointsPerChunk = 4096;
	}

	/** Stores the input and output data as the state of the time sliced execution */
	struct IterationState
	{
		// This will be set to the same value as InputData when using UPCGPointData type to support deprecated code
		UE_DEPRECATED(5.6, "Use InputData instead")
		const UPCGPointData* InputPointData = nullptr;

		// This will be set to the same value as OutputData when using UPCGPointData type to support deprecated code
		UE_DEPRECATED(5.6, "Use OutputData instead")
		UPCGPointData* OutputPointData = nullptr;

		const UPCGBasePointData* InputData = nullptr;

		UPCGBasePointData* OutputData = nullptr;

		int32 NumPoints = 0;
	};
}

/** Simplified, time-sliced, and point by point operation class. A function or lambda may be passed into the `ExecutePointOperation` at execution time to
 * invoke a customized update operation on all incoming points, individually.
 */
class FPCGPointOperationElementBase : public TPCGTimeSlicedElementBase<PCGTimeSlice::FEmptyStruct, PCGPointOperation::IterationState>
{
protected:
	//~Begin IPCGElement interface
	/** Conveniently calls PreparePointOperationData to prepare the time sliced element for execution. May be overridden, but PreparePointOperationData must be called. */
	UE_API virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	//~End IPCGElement interface

	/** Executes the PointFunction function/lambda for every point copied from PreparePointOperationData */
	template <typename Func = TFunctionRef<bool(const FPCGPoint& InPoint, FPCGPoint& OutPoint)>>
	bool ExecutePointOperation(ContextType* Context, Func Callback, int32 PointsPerChunk = PCGPointOperation::Constants::PointsPerChunk) const
	{
		if constexpr (std::is_invocable_v<Func, const FPCGPoint&, FPCGPoint&>)
		{
			return ExecutePointOperationWithPoints(Context, Callback, PointsPerChunk);
		}
		else
		{
			return ExecutePointOperationWithIndices(Context, Callback, PointsPerChunk);
		}
	}

	/** Mandatory call. Using the context, prepares the state data for time slice execution */
	UE_API bool PreparePointOperationData(ContextType* Context, FName InputPinLabel = PCGPinConstants::DefaultInputLabel) const;

	virtual EPCGPointNativeProperties GetPropertiesToAllocate(FPCGContext* InContext) const { return EPCGPointNativeProperties::All; }
	
	/** If True, input points will be copied into output points before each point operation. The copy will only happen if output doesn't support inheritance */
	virtual bool ShouldCopyPoints() const { return false; }

private:
	UE_API bool ExecutePointOperationWithPoints(ContextType* Context, TFunctionRef<bool(const FPCGPoint& InPoint, FPCGPoint& OutPoint)> Callback, int32 PointsPerChunk) const;
	UE_API bool ExecutePointOperationWithIndices(ContextType* Context, TFunctionRef<bool(const UPCGBasePointData* InPointData, UPCGBasePointData* OutPointData, int32 StartIndex, int32 Count)> Callback, int32 PointsPerChunk) const;
};

#undef UE_API
