// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGElement.h"

struct FPCGPoint;
struct FPCGTaggedData;

class UE_DEPRECATED(5.5, "FPCGPointProcessingElementBase is deprecated. Please use 'FPCGPointOperationElementBase' instead.") PCG_API FPCGPointProcessingElementBase : public IPCGElement
{
protected:
	void ProcessPoints(FPCGContext* Context, const TArray<FPCGTaggedData>& Inputs, TArray<FPCGTaggedData>& Outputs, const TFunction<bool(const FPCGPoint&, FPCGPoint&)>& PointFunc) const;
};
