// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGElement.h"
#include "PCGSettings.h"

#include "PCGManagedResource.h"

#include "Logging/LogVerbosity.h"

#include "PCGPrintElement.generated.h"

UENUM(BlueprintType)
enum class EPCGPrintVerbosity : uint8
{
	NoLogging = ELogVerbosity::NoLogging, // 0
	Log = ELogVerbosity::Log,
	Warning = ELogVerbosity::Warning,
	Error = ELogVerbosity::Error,
	Display = ELogVerbosity::Display
};

/** Used to track the debug message to properly remove it upon regen or clean up. */
UCLASS(ClassGroup = (Procedural))
class UPCGManagedDebugStringMessageKey : public UPCGManagedResource
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	// Debug should always be transient
	virtual void ChangeTransientState(EPCGEditorDirtyMode NewEditingMode) override {}
#endif // WITH_EDITOR

	virtual bool Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	virtual bool CanBeUsed() const override { return false; }

	UPROPERTY()
	uint64 HashKey = (uint64)-1;
};

/**
 * Issues a specified message to the log, and optionally to the graph and/or screen.
 * Note: This node will not function in shipping builds.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural), meta = (Keywords = "debug print string"))
class UPCGPrintElementSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Print String")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGPrintElement", "NodeTitle", "Print String"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGPrintElement", "NodeTooltip", "Issues a specified message to the log, and optionally to the graph and/or screen."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Debug; }
#endif
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** The core message to print to the logger, graph, and/or screen. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (MultiLine, PCG_Overridable))
	FString PrintString;

	/** The verbosity level of the printed message. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGPrintVerbosity Verbosity = EPCGPrintVerbosity::Log;

	/** A prefix to which the core message will be appended. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FString CustomPrefix;

	/** Display warnings or errors on this node. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bDisplayOnNode = false;

	/** Use the component as part of the key hash and print a message for each component with this node. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bPrintPerComponent = true;

#if WITH_EDITORONLY_DATA
	/** Print the message to the editor viewport. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bPrintToScreen = false;

	/** The duration (in seconds) of the on screen message. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bPrintToScreen", EditConditionHides, ClampMin = "0.0"))
	double PrintToScreenDuration = 15.0;

	/** The color of the on screen message. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bPrintToScreen", EditConditionHides))
	FColor PrintToScreenColor = FColor::Cyan;
#endif // WITH_EDITORONLY_DATA

	/** Prefix the message with the name of the component's owner. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bPrintPerComponent", EditConditionHides))
	bool bPrefixWithOwner = false;

	/** Prefix the message with the name of the component. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bPrintPerComponent", EditConditionHides))
	bool bPrefixWithComponent = false;

	/** Prefix the message with the name of the graph. */
    UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bPrintPerComponent", EditConditionHides))
    bool bPrefixWithGraph = true;

	/** Prefix the message with the name of the node. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bPrefixWithNode = true;

	/** Enable the functionality of this node. Disable to bypass printing. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bEnablePrint = true;
};

class FPCGPrintElement : public IPCGElement
{
public:
	// Print to screen is perhaps safe to be called from outside of the main thread, but taking the safe approach here.
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
