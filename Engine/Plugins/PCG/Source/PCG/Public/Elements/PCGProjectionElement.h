// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGProjectionParams.h"
#include "PCGSettings.h"

#include "PCGProjectionElement.generated.h"

namespace PCGProjectionConstants
{
	const FName ProjectionTargetLabel = TEXT("Projection Target");
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGProjectionSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Projection")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGProjectionSettings", "NodeTitle", "Projection"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif
	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGProjectionParams ProjectionParams;

	/** Force the result to be sampled to points, equivalent to having a To Point node after the projection node. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bForceCollapseToPoint = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bKeepZeroDensityPoints = false;

#if WITH_EDITOR
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
#endif

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGProjectionElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
