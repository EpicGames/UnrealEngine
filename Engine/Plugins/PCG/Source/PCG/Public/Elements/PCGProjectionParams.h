// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataCommon.h"

#include "PCGProjectionParams.generated.h"

UENUM()
enum class EPCGProjectionColorBlendMode : uint8
{
	SourceValue,
	TargetValue,
	Add,
	Subtract,
	Multiply
};

UENUM()
enum class EPCGProjectionTagMergeMode : uint8
{
	Source,
	Target,
	Both
};

/** Parameters that control projection behaviour. */
USTRUCT(BlueprintType)
struct FPCGProjectionParams
{
	GENERATED_BODY()

	// TODO [DEPRECATED_IN_5_4]: This rule of five necessary to avoid compile-time warnings. To be removed in when the deprecated member has expired
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPCGProjectionParams() = default;
	~FPCGProjectionParams() = default;
	FPCGProjectionParams(const FPCGProjectionParams&) = default;
	FPCGProjectionParams(FPCGProjectionParams&&) = default;
	FPCGProjectionParams& operator=(const FPCGProjectionParams&) = default;
	FPCGProjectionParams& operator=(FPCGProjectionParams&&) = default;

	void ApplyDeprecation()
	{
#if WITH_EDITOR
		// To maintain backwards compatibility, a 'true' here would've multiplied as the blend mode
		if (bProjectColors_DEPRECATED)
		{
			ColorBlendMode = EPCGProjectionColorBlendMode::Multiply;
			bProjectColors_DEPRECATED = false;
		}
#endif // WITH_EDITOR
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Project positions. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Apply Data")
	bool bProjectPositions = true;

	/** Project rotations. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Apply Data")
	bool bProjectRotations = true;

	/** Project scales. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Apply Data")
	bool bProjectScales = false;

	/** The blend mode for colors during the projection */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Apply Data")
	EPCGProjectionColorBlendMode ColorBlendMode = EPCGProjectionColorBlendMode::SourceValue;

	/** Attributes to either explicitly exclude or include in the projection operation, depending on the Attribute Mode setting. Leave empty to gather all attributes and their values. Format is comma separated list like: Attribute1,Attribute2 .*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Apply Data")
	FString AttributeList = TEXT("");

	/** How the attribute list is used. Exclude Attributes will ignore these attributes and their values on the projection target. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Apply Data")
	EPCGMetadataFilterMode AttributeMode = EPCGMetadataFilterMode::ExcludeAttributes;

	/** Operation to use to combine attributes that reside on both source and target data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Apply Data")
	EPCGMetadataOp AttributeMergeOperation = EPCGMetadataOp::TargetValue;

	/** Controls whether the data tags are taken from the source, the target or both. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Apply Data")
	EPCGProjectionTagMergeMode TagMergeOperation = EPCGProjectionTagMergeMode::Source;

	// TODO [DEPRECATED_IN_5_4]: Remove the 'rule of five' needed above upon deprecation
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.4, "This property will be deprecated in 5.4. Please use `ColorBlendMode` instead.")
	UPROPERTY(meta = (DeprecatedProperty))
	bool bProjectColors_DEPRECATED = false;
#endif // WITH_EDITORONLY_DATA
};
