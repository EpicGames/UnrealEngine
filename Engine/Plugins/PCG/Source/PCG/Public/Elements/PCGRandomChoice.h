// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGRandomChoice.generated.h"

namespace PCGRandomChoiceConstants
{
	const FName ChosenEntriesLabel = TEXT("Chosen");
	const FName DiscardedEntriesLabel = TEXT("Discarded");
}

/**
* Chooses entries randomly through ratio or a fixed number of entries.
* Chosen/Discarded entries will be in the same order than they appear in the input data.
*/
UCLASS(BlueprintType)
class UPCGRandomChoiceSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Filter; }
#endif
	virtual bool UseSeed() const override { return true; }
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface


public:
	// Either choose a fixed number of entries, or a ratio of entries.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bFixedMode = true;

	// Defines the number of entries to keep.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bFixedMode", ClampMin = "0"))
	int FixedNumber = 1;

	// Defines the ratio of entries to keep.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "!bFixedMode", ClampMin = "0", ClampMax = "1"))
	float Ratio = 0.5;

	// By default, we output discarded entries. If you don't need them, disable this option.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bOutputDiscardedEntries = true;

	/** Use an attribute as a source for generating the seed, i.e. similar to or replacing the $Seed property on points. Mostly useful for attribute sets as points have this unique seed by default. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bHasCustomSeedSource = false;

	/** Attribute to generate the selection seed from. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bHasCustomSeedSource", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector CustomSeedSource;

	/** Controls whether only the first element's attribute will be used to generate the selection seed. Otherwise, all values will be used to compute it. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bHasCustomSeedSource", EditConditionHides, PCG_Overridable))
	bool bUseFirstAttributeOnly = true;
};

class FPCGRandomChoiceElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
