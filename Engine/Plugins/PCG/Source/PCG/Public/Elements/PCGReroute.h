// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "PCGElement.h"
#include "PCGSettings.h"

#include "PCGReroute.generated.h"

namespace PCGNamedRerouteConstants
{
	const FName InvisiblePinLabel = TEXT("InvisiblePin");
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGRerouteSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGRerouteSettings();

	//~Begin UPCGSettingsInterface interface
	virtual bool CanBeDisabled() const override { return false; }
	//~End UPCGSettingsInterface interface

	//~Begin UPCGSettings interface
	virtual bool HasDynamicPins() const override { return true; }
	virtual FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName("Reroute"); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGRerouteElement", "NodeTitle", "Reroute"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Reroute; }
	virtual bool CanUserEditTitle() const override { return false; }
#endif

	static PCG_API TOptional<FName> GetCollisionFreeNodeName(const UPCGGraph* InGraph, FName BaseName);

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	virtual bool HasExecutionDependencyPin() const override { return false; }
#if WITH_EDITOR
	virtual TArray<FPCGPreconfiguredInfo> GetConversionInfo() const override;
#endif // WITH_EDITOR
	virtual bool ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo) override;
	//~End UPCGSettings interface
};

/** Base class for both reroute declaration and usage to share implementation, but also because they use the same visual node representation in the editor. */
UCLASS(MinimalAPI, ClassGroup = (Procedural))
class UPCGNamedRerouteBaseSettings : public UPCGRerouteSettings
{
	GENERATED_BODY()

public:
	UPCGNamedRerouteBaseSettings();

#if WITH_EDITOR
	virtual bool CanUserEditTitle() const override { return true; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif

};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGNamedRerouteDeclarationSettings : public UPCGNamedRerouteBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	//~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override { return FName("NamedRerouteDeclaration"); }
	//~End UPCGSettings interface
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
#if WITH_EDITOR
	virtual TArray<FPCGPreconfiguredInfo> GetConversionInfo() const override;
#endif // WITH_EDITOR
	virtual bool ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo) override;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGNamedRerouteUsageSettings : public UPCGNamedRerouteBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	//~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override { return FName("NamedRerouteUsage"); }
	//~End UPCGSettings interface
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
#if WITH_EDITOR
	virtual TArray<FPCGPreconfiguredInfo> GetConversionInfo() const override { return {}; }
#endif // WITH_EDITOR
	virtual bool ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo) override { return false; }

public:
	/** Very counter-intuitive but reroute nodes are normally culled by other means, if they aren't we want to make sure they log errors. */
	virtual bool CanCullTaskIfUnwired() const { return false; }
	FPCGDataTypeIdentifier GetCurrentPinTypesID(const UPCGPin* InPin) const override;

public:
	UPROPERTY(BlueprintReadWrite, Category = Settings)
	TObjectPtr<const UPCGNamedRerouteDeclarationSettings> Declaration;
};

class FPCGRerouteElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }
};
