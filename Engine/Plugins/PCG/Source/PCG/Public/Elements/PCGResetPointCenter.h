// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/PCGPointOperationElementBase.h"

#include "PCGResetPointCenter.generated.h"

/**
 * Modify the position of a point within its bounds, while keeping its bounds the same.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGResetPointCenterSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ResetPointCenter")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGResetPointCenterElement", "NodeTitle", "Reset Point Center"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGResetPointCenterElement", "NodeTooltip", "Modify the position of a point within its bounds, while keeping its bounds the same."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Set the normalized center of the point */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FVector PointCenterLocation = FVector(0.5, 0.5, 0.5);
};

class FPCGResetPointCenterElement : public FPCGPointOperationElementBase
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGPointNativeProperties GetPropertiesToAllocate(FPCGContext* InContext) const override;
	virtual bool ShouldCopyPoints() const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};