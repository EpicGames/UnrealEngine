// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Data/PCGTextureData.h"

#include "PCGSampleTexture.generated.h"

namespace PCGSampleTextureConstants
{
	const FName InputPointLabel = TEXT("Point");
	const FName InputTextureLabel = TEXT("BaseTexture");
}

UENUM()
enum class EPCGTextureMappingMethod : uint8
{
	Planar UMETA(DisplayName = "Planar From Texture Data"),
	UVCoordinates UMETA(DisplayName = "Use Explicit Points UV Coordinates")
};

/** Samples color of texture at each point. */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGSampleTextureSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SampleTexture")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSampleTextureElement", "NodeTitle", "Sample Texture"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGSampleTextureElement", "NodeTooltip", "Samples color of texture at each point."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Sampler; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Whether to treat the sample positions as being in 0-1 UV space. If method is Planar then the coordinates will be transformed according to the texture settings. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGTextureMappingMethod TextureMappingMethod = EPCGTextureMappingMethod::Planar;
	
	/** The attribute that provides sample positions for sampling the texture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, DisplayName = "UV Coordinates Attribute", EditCondition = "TextureMappingMethod == EPCGTextureMappingMethod::UVCoordinates", EditConditionHides))
	FPCGAttributePropertyInputSelector UVCoordinatesAttribute;

	/** Overrides the texture's tiling to wrap or clamp its UVs. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "TextureMappingMethod == EPCGTextureMappingMethod::UVCoordinates", EditConditionHides))
	EPCGTextureAddressMode TilingMode = EPCGTextureAddressMode::Wrap;

	/** Controls the behavior of density computation with respect to initial data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGDensityMergeOperation DensityMergeFunction = EPCGDensityMergeOperation::Set;

	/** Controls whether the output density should be clamped or not. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bClampOutputDensity = true;
};

class FPCGSampleTextureElement : public IPCGElement
{
public:
	// Loading needs to be done on the main thread and accessing objects outside of PCG might not be thread safe, so taking the safe approach
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
