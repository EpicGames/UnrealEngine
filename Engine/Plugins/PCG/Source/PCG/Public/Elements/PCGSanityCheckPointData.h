// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGSanityCheckPointData.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSanityCheckPointDataSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SanityCheckPointData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSanityCheckPointDataSettings", "NodeTitle", "Sanity Check Point Data"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Debug; }
#endif

	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	int32 MinPointCount = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	int32 MaxPointCount = 100;
};

class FPCGSanityCheckPointDataElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
};
