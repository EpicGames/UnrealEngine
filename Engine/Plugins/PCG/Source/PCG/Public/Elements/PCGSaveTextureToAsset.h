// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGAssetExporter.h"
#include "PCGContext.h"
#include "PCGSettings.h"
#include "PixelFormat.h"

#include "PCGSaveTextureToAsset.generated.h"

class UPCGTexture2DSingleBaseData;
class UTexture2D;

/** Save the input texture to a UTexture2D asset. Output format is auto-detected from the input texture (BGRA8, RGBA16F, or RGBA32F). Outputs a soft object path attribute for the saved texture. Editor only. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSaveTextureToAssetSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SaveTextureToAsset")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSaveTextureToAssetElement", "NodeTitle", "Save Texture to Asset"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGSaveTextureToAssetElement", "NodeTooltip", "Save the input texture to a UTexture2D asset. Output format is auto-detected from the input texture (BGRA8, RGBA16F, or RGBA32F). Note: This node is editor only."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override { return true; }
#endif
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGAssetExporterParameters ExporterParams;
};

struct FPCGSaveTextureToAssetContext : public FPCGContext
{
public:
	virtual ~FPCGSaveTextureToAssetContext();

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;

public:
	TObjectPtr<const UPCGTexture2DSingleBaseData> InputTextureData = nullptr;
	TObjectPtr<UTexture2D> ExportedTexture = nullptr;
	uint8* RawReadbackData = nullptr;
	int32 ReadbackWidth = INDEX_NONE;
	int32 ReadbackHeight = INDEX_NONE;
	bool bUpdatedReadbackTextureResource = false;
	EPixelFormat ReadbackFormat = PF_B8G8R8A8;
	bool bReadbackDispatched = false;
	bool bReadbackComplete = false;
};

class FPCGSaveTextureToAssetElement : public IPCGElementWithCustomContext<FPCGSaveTextureToAssetContext>
{
public:
	/** Creating asset and initializing texture resource must occur on main thread. */
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsGPUResidentData(FPCGContext* InContext) const override { return true; }

	bool ReadbackInputTexture(FPCGSaveTextureToAssetContext* InContext) const;
};
