// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Data/PCGTextureData.h"

#include "Components/SceneCaptureComponent2D.h"
#include "Engine/TextureRenderTarget2D.h"

#include "PCGSceneCapture.generated.h"

UENUM()
enum class EPCGSceneCaptureOrientationMode : uint8
{
	FromExecutionSource UMETA(ToolTip="Derive scene capture orientation and extents from the PCG execution source."),
	FromBoundingShape UMETA(ToolTip="Override scene capture orientation and extents using a spatial bounding shape. Note, bounding shape overrides will always become an axis aligned bounding box with top down projection (rotation is ignored)."),
	Explicit UMETA(ToolTip="Control scene capture orientation and extents using explicit values."),
};

/** Perform a 2D orthographic scene capture and write the result to a render target data. Can be costly, use with caution with runtime generation. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSceneCaptureSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface
#endif

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SceneCapture")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSceneCaptureElement", "NodeTitle", "Scene Capture"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGSceneCaptureElement", "NodeTooltip", "Perform a 2D orthographic scene capture and write the result to a render target data. Can be costly, use with caution with runtime generation."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Sampler; }
#endif
	virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	//~End UPCGSettings interface

public:
	/** Subset of EPixelFormat exposed to UTextureRenderTarget2D. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(PCG_Overridable))
	TEnumAsByte<enum ETextureRenderTargetFormat> PixelFormat = ETextureRenderTargetFormat::RTF_RGBA16f;

	/** Specifies which component of the scene rendering should be output to the render target. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(PCG_Overridable))
	TEnumAsByte<enum ESceneCaptureSource> CaptureSource = ESceneCaptureSource::SCS_SceneColorHDR;

	/** Only capture actors and primitive components that have tags listed in Included Tags. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Capture Filtering", meta = (PCG_Overridable))
	bool bOnlyCaptureContentMatchingIncludedTags = false;

	/** Tags to match against actors/components when selecting for inclusion in the scene capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Capture Filtering", meta = (EditCondition = "bOnlyCaptureContentMatchingIncludedTags", EditConditionHides, PCG_Overridable))
	TArray<FName> IncludedTags;

	/** Tags to match against actors/components when selecting for exclusion from the scene capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Capture Filtering", meta = (PCG_Overridable))
	TArray<FName> ExcludedTags;

	/** Exclude content created from PCG. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Capture Filtering", DisplayName = "Exclude PCG Content", meta = (PCG_Overridable))
	bool bExcludePCGContent = true;

	/** Defines how the scene capture bounds and transform are computed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(PCG_Overridable))
	EPCGSceneCaptureOrientationMode OrientationMode = EPCGSceneCaptureOrientationMode::FromExecutionSource;

	/** Positions the scene capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "OrientationMode==EPCGSceneCaptureOrientationMode::Explicit", EditConditionHides, PCG_Overridable))
	FVector CaptureLocation = FVector::ZeroVector;

	/** Rotates the scene capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "OrientationMode==EPCGSceneCaptureOrientationMode::Explicit", EditConditionHides, PCG_Overridable))
	FQuat CaptureRotation = FQuat::Identity;

	/** Used to determine the orthographic width of the scene capture. Extents on Z are used for the resulting texture data's Z extents. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "OrientationMode==EPCGSceneCaptureOrientationMode::Explicit", EditConditionHides, PCG_Overridable))
	FVector CaptureHalfExtents = FVector(1000.0);

	/** Scene capture position will be translated such that the capture is taken from the top of the capture extents rather than from the center. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bCaptureFromTopOfExtents = true;

	/** Size of a texel in the render target in world units (cm). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(UIMin = "1.0", ClampMin = "1.0", PCG_Overridable))
	float TexelSize = 50.0f;

	/** Method used to determine the value for a sample based on the value of nearby texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGTextureFilter Filter = EPCGTextureFilter::Bilinear;

	// Deprecated section
	UE_DEPRECATED(5.8, "bSkipReadbackToCPU has been removed. CPU readback is now always deferred.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecatedMessage = "bSkipReadbackToCPU has been removed. CPU readback is now always deferred."))
	bool bSkipReadbackToCPU = false;
};

struct FPCGSceneCaptureContext : public FPCGContext
{
protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;

public:
	TObjectPtr<USceneCaptureComponent2D> SceneCaptureComponent = nullptr;
	TObjectPtr<UTextureRenderTarget2D> RenderTarget = nullptr;
	FTransform RenderTargetTransform = FTransform::Identity;
	bool bSubmittedSceneCapture = false;
};

class FPCGSceneCaptureElement : public IPCGElementWithCustomContext<FPCGSceneCaptureContext>
{
public:
	/** Required to create the scene capture component. */
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
