// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGSelectPoints.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSelectPointsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SelectPoints")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSelectPointsElement", "NodeTitle", "Select Points"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Sampler; }
#endif
	virtual bool UseSeed() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(ClampMin="0", ClampMax="1", PCG_Overridable))
	float Ratio = 0.1f;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug", meta = (PCG_Overridable))
	bool bKeepZeroDensityPoints = false;
#endif
};

class FPCGSelectPointsElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
};
