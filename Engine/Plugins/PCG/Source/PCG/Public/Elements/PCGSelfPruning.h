// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Data/PCGCollisionWrapperData.h"
#include "Data/PCGPointData.h"
#include "Elements/PCGTimeSlicedElementBase.h"

#include "Containers/Array.h"

#include "PCGSelfPruning.generated.h"

class UPCGBasePointData;
class UPCGPointData;

UENUM()
enum class EPCGSelfPruningType : uint8
{
	LargeToSmall,
	SmallToLarge,
	AllEqual,
	None,
	RemoveDuplicates
};

USTRUCT(BlueprintType)
struct FPCGSelfPruningParameters
{
	GENERATED_BODY()

public:
	void PostLoad();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGSelfPruningType PruningType = EPCGSelfPruningType::LargeToSmall;

	/** By default, it will prune according to the extents of each point, but you can provide another comparison like Density, or a dynamic attribute. Only support numeric values or vector (vector will be reduced to its squared length). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "PruningType == EPCGSelfPruningType::LargeToSmall || PruningType == EPCGSelfPruningType::SmallToLarge", PCG_Overridable))
	FPCGAttributePropertyInputSelector ComparisonSource;

	/** Similarity factor to consider 2 points "equal". (For example, if a point extents squared length is 10 and factor is 0.25, all points between 7.5 and 12.5 will be considered "the same"). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = 0.0f, EditCondition = "PruningType == EPCGSelfPruningType::LargeToSmall || PruningType == EPCGSelfPruningType::SmallToLarge", PCG_Overridable))
	float RadiusSimilarityFactor = 0.25f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bRandomizedPruning = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "PruningType != EPCGSelfPruningType::RemoveDuplicates"))
	bool bUseCollisionAttribute = false;

	/** Specifies to use the collision from a given mesh instead of the point; note that this will be ignored in the Remove Duplicates mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection, EditCondition = "bUseCollisionAttribute && PruningType != EPCGSelfPruningType::RemoveDuplicates", EditConditionHides))
	FPCGAttributePropertyInputSelector CollisionAttribute;

	/** Controls whether queries will be done against complex collisions or not. If enabled, performance warning. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases="bUseComplexCollision", EditCondition = "bUseCollisionAttribute && PruningType != EPCGSelfPruningType::RemoveDuplicates", EditConditionHides))
	EPCGCollisionQueryFlag CollisionQueryFlag = EPCGCollisionQueryFlag::Simple;

#if WITH_EDITORONLY_DATA
	// Implementation note: was introduced during UE 5.5 development and replaced, does not require a full public API deprecation mechanism
	UPROPERTY()
	bool bUseComplexCollision_DEPRECATED = false;
#endif
};

namespace PCGSelfPruningElement
{
	// all points are in contiguous memory so this will create a bitset to just look up the index to set/test a bit
	// this should reduce the 'set' cost to O[1]
	struct FPointBitSet
	{
		TArray<uint32> Bits;

		void Initialize(int32 NumPoints);
		void Add(int32 Index);
		bool Contains(int32 Index);

		UE_DEPRECATED(5.6, "Use int32 param version")
		void Initialize(const TArray<FPCGPoint>& Points) {}
		UE_DEPRECATED(5.6, "Use int32 param version")
		void Add(const FPCGPoint* Point) {}
		UE_DEPRECATED(5.6, "Use int32 param version")
		bool Contains(const FPCGPoint* Point) { return false; }
	};

	struct FIterationState
	{
		const UPCGBasePointData* InputData = nullptr;
		UPCGBasePointData* OutputData = nullptr;

		TArray<PCGPointOctree::FPointRef> SortedPointRefs;
		FPointBitSet ExcludedPoints;
		FPointBitSet ExclusionPoints;
		int32 CurrentPointIndex = 0;
		bool bSortDone = false;

		// In the case of the self-pruning using the collision, we'll need instances of the unique bodies that are used in the point data.
		// Additionally, in the cases where two points to compare use the same instance, we'll have to do a temporary copy.
		FPCGCollisionWrapper CollisionWrapper;
		TMap<FBodyInstance*, FBodyInstance*> TemporaryBodyInstances;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.6, "Use SortedPointRefs instead")
		TArray<FPCGPointRef> SortedPoints;

		UE_DEPRECATED(5.6, "Use InputData instead")
		const UPCGPointData* InputPointData = nullptr;

		UE_DEPRECATED(5.6, "Use OutputData instead")
		UPCGPointData* OutputPointData = nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	/** Will do the self pruning on all inputs in the context. Blocking call (no time slicing). Be careful, it can be very costly. */
	void Execute(FPCGContext* Context, EPCGSelfPruningType PruningType, float RadiusSimilarityFactor, bool bRandomizedPruning);

	/** Will do the self pruning on all inputs in the context. Blocking call (no time slicing). Be careful, it can be very costly. */
	PCG_API void Execute(FPCGContext* Context, const FPCGSelfPruningParameters& InParameters);

	/** Will execute self pruning according to the info from FIterationState. Time sliced call, will return true if it is done. */
	PCG_API bool ExecuteSlice(FIterationState& InState, const FPCGSelfPruningParameters& InParameters, FPCGContext* InOptionalContext = nullptr);
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGSelfPruningSettings : public UPCGSettings
{
	GENERATED_BODY()

	UPCGSelfPruningSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGSelfPruningParameters Parameters;

#if WITH_EDITOR
	//~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SelfPruning")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSelfPruningSettings", "NodeTitle", "Self Pruning"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Filter; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	EPCGSelfPruningType PruningType_DEPRECATED = EPCGSelfPruningType::LargeToSmall;

	UPROPERTY()
	float RadiusSimilarityFactor_DEPRECATED = 0.25f;

	UPROPERTY()
	bool bRandomizedPruning_DEPRECATED = true;
#endif // WITH_EDITORONLY_DATA
};

class FPCGSelfPruningElement : public TPCGTimeSlicedElementBase<PCGTimeSlice::FEmptyStruct, PCGSelfPruningElement::FIterationState>
{
protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
