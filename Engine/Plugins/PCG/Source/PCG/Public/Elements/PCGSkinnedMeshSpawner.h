// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGObjectPropertyOverride.h"
#include "Engine/CollisionProfile.h"
#include "PCGSkinnedMeshSpawner.generated.h"

#define UE_API PCG_API

class UPCGSkinnedMeshInstanceDataPackerBase;
class UPCGSkinnedMeshSelector;
class UPCGSpatialData;
struct FPCGContext;
struct FPCGSkinnedMeshInstanceList;
struct FPCGSkinnedMeshPackedCustomData;
struct FPCGSkinnedMeshSpawnerContext;

class UAnimBank;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural), meta = (PrioritizeCategories = "Settings"))
class UPCGSkinnedMeshSpawnerSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UE_API UPCGSkinnedMeshSpawnerSettings(const FObjectInitializer& ObjectInitializer);

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	//virtual bool DisplayExecuteOnGPUSetting() const override { return true; } // Disabled pending testing.
	virtual void CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const override;
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SkinnedMeshSpawner")); }
	UE_API virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spawner; }
	UE_API virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
#endif
	virtual bool UseSeed() const override { return true; }

protected:
	UE_API virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	UE_API virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface

public:
	// ~Begin UObject interface
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	// ~End UObject interface
#endif

	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetInstancePackerType(TSubclassOf<UPCGSkinnedMeshInstanceDataPackerBase> InInstancePackerType);

public:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Instanced, Category = MeshSelector)
	TObjectPtr<UPCGSkinnedMeshSelector> MeshSelectorParameters;

	/**
	 * Defines the method of custom data packing for spawned anim bank meshes. Note, Rotators are treated as 3 floats, while Quaternions are
	 * treated as 4 floats. You can see an attribute's type in the 'Attribute List View' window, and use an 'Attribute Cast' node to cast to the desired type.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = InstanceDataPacker)
	TSubclassOf<UPCGSkinnedMeshInstanceDataPackerBase> InstanceDataPackerType;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Instanced, Category = InstanceDataPacker)
	TObjectPtr<UPCGSkinnedMeshInstanceDataPackerBase> InstanceDataPackerParameters = nullptr;

	/** Map an attribute directly to an anim bank descriptor property, the value of which will be overriden when generated.
	 * Note: Currently only enabled using SelectByAttribute mesh selection.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FPCGObjectPropertyOverrideDescription> SkinnedMeshComponentPropertyOverrides;

	/** Sets the BoundsMin and BoundsMax attributes of each point to reflect the AnimBank mesh spawned at its location */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bApplyMeshBoundsToPoints = true;

	UPROPERTY(BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	TSoftObjectPtr<AActor> TargetActor;

	/** Specify a list of functions to be called on the target actor after instances are spawned. Functions need to be parameter-less and with "CallInEditor" flag enabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FName> PostProcessFunctionNames;

	/** Meshes/Materials will be synchronously loaded before spawning instead of asynchronously. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;

	/** Opt-in option to silence errors when the property override attributes are not found. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bSilenceOverrideAttributeNotFoundErrors = false;

	/** Adds a warning to the node on repeated spawning with identical conditions (ie. same mesh descriptor at same spawn location, etc). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bWarnOnIdenticalSpawn = true;

protected:
	void RefreshInstancePacker();
};

class FPCGSkinnedMeshSpawnerElement : public IPCGElement
{
public:
	UE_API virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	UE_API virtual FPCGContext* CreateContext() override;
	UE_API virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	UE_API virtual bool ExecuteInternal(FPCGContext* Context) const override;
	UE_API virtual void AbortInternal(FPCGContext* Context) const override;
	
	UE_API void SpawnSkinnedMeshInstances(
		FPCGSkinnedMeshSpawnerContext* Context,
		const FPCGSkinnedMeshInstanceList& InstanceList,
		AActor* TargetActor,
		const FPCGSkinnedMeshPackedCustomData* InPackedCustomData) const;
};

#undef UE_API
