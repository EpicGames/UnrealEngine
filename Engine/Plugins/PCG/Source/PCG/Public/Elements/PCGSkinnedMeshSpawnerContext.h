// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGManagedResource.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "MeshSelectors/PCGSkinnedMeshSelector.h"
#include "InstanceDataPackers/PCGSkinnedMeshInstanceDataPackerBase.h"

class AActor;
class UPCGSpatialData;

struct FPCGSkinnedMeshSpawnerContext : public FPCGContext, public IPCGAsyncLoadingContext
{
	struct FPackedInstanceListData
	{
		FPackedInstanceListData();
		~FPackedInstanceListData();
		AActor* TargetActor = nullptr;
		const UPCGSpatialData* SpatialData = nullptr;
		TArray<FPCGSkinnedMeshInstanceList> MeshInstances;
		TArray<FPCGSkinnedMeshPackedCustomData> PackedCustomData;
	};

	TArray<FPackedInstanceListData> MeshInstancesData;
	// Index of input in the context (for selection)
	int32 CurrentInputIndex = 0;
	// Index of MeshInstances/PackedCustomData in last MeshInstancesData element
	int32 CurrentDataIndex = 0;

	// Whole-execution variables
	bool bReuseCheckDone = false;
	bool bSkippedDueToReuse = false;

	// Per-input context variables
	bool bCurrentInputSetup = false;
	bool bSelectionDone = false;
	bool bPartitionDone = false;

	const UPCGPointData* CurrentPointData = nullptr;
	UPCGPointData* CurrentOutputPointData = nullptr;
	FPCGMeshMaterialOverrideHelper MaterialOverrideHelper;
	int32 CurrentPointIndex = 0;

	// Used in all selectors if we have to change the out points bounds by the mesh bounds. Will be empty otherwise.
	// We need to keep all point indices that will spawn this mesh, in all output point data.
	TMap<TSoftObjectPtr<USkinnedAsset>, TMap<UPCGPointData*, TArray<int32>>> MeshToOutPoints;

	TMap<TSoftObjectPtr<USkinnedAsset>, PCGMetadataValueKey> MeshToValueKey;

	// Used for mesh property overrides
	TArray<TArray<int32>> AttributeOverridePartition;
	TArray<FPCGSoftSkinnedMeshComponentDescriptor> OverriddenDescriptors;

	// Keeping track of all touched resources to allow for correct cleanup on abort
	TArray<TWeakObjectPtr<UPCGManagedISKMComponent>> TouchedResources;

	TArray<FSoftObjectPath> PointProviderValues;
	TArray<int32> PointAnimationValues;

	void ResetInputIterationData();
};