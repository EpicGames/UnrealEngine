// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeKernel.h"

#include "PCGSkinnedMeshSpawnerKernel.generated.h"

class UPCGSkinnedMeshSpawnerSettings;

namespace PCGSkinnedMeshSpawnerConstants
{
	const FName InstanceCountsPinLabel = TEXT("InstanceCounts");
}

UCLASS()
class UPCGSkinnedMeshSpawnerKernel : public UPCGComputeKernel
{
	GENERATED_BODY()

public:
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const override;
#if WITH_EDITOR
	virtual const TCHAR* GetSourceFilePath() const override { return TEXT("/Plugin/PCG/Private/Elements/PCGStaticMeshSpawner.usf"); }
	virtual FString GetCookedSource(FPCGGPUCompilationContext& InOutContext) const override;
	virtual FString GetEntryPoint() const override { return TEXT("PCGStaticMeshSpawnerCS"); }
	void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
	void CreateAdditionalOutputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
#endif

	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;

	bool GetProduceOutputPoints() const { return bProduceOutputPoints; }
	void SetProduceOutputPoints(bool bInProduceOutputPoints) { bProduceOutputPoints = bInProduceOutputPoints; }

protected:
#if WITH_EDITOR
	virtual bool PerformStaticValidation() override;
#endif

	UPROPERTY()
	bool bProduceOutputPoints = true;
};
