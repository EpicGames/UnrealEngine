// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGSortAttributes.generated.h"

/**
 * Sorts points based on an attribute.
 */

UENUM()
enum class EPCGSortMethod : uint8
{
	Ascending,
	Descending
};

/** An individual entry in a multi sort operation. */
USTRUCT(BlueprintType)
struct FPCGSortAttributeEntry
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGSortMethod SortMethod = EPCGSortMethod::Ascending;
};

/** Sorts data lexicographically by one or more attributes. */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGSortAttributesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGSortAttributesSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual TArray<FText> GetNodeTitleAliases() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif

	virtual bool HasDynamicPins() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TArray<FPCGSortAttributeEntry> SortAttributes = { FPCGSortAttributeEntry() };

	// Whether to preserve input order for elements that compare equal across all sort attributes.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bUseStableSort = false;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "InputSource has been deprecated. Use SortAttributes.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "InputSource has been deprecated. Use SortAttributes."))
	FPCGAttributePropertyInputSelector InputSource;

	UE_DEPRECATED(5.8, "SortMethod has been deprecated. Use SortAttributes.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "SortMethod has been deprecated. Use SortAttributes."))
	EPCGSortMethod SortMethod = EPCGSortMethod::Ascending;
#endif // WITH_EDITORONLY_DATA
};

class FPCGSortAttributesElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
