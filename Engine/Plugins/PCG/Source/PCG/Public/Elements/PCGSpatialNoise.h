// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGSpatialNoise.generated.h"

UENUM()
enum class PCGSpatialNoiseMode
{
	/** Your classic perlin noise. */
	Perlin2D,
	/** Based on underwater fake caustic rendering, gives swirly look. */
	Caustic2D,
	/** Voronoi noise, result a the distance to edge and cell ID. */
	Voronoi2D,
	/** Based on fractional brownian motion. */
	FractionalBrownian2D,
	/** Used to create masks to blend out edges. */
	EdgeMask2D,
};

UENUM()
enum class PCGSpatialNoiseMask2DMode
{
	/** Your classic perlin noise. */
	Perlin,
	/** Based on underwater fake caustic rendering, gives swirly look. */
	Caustic,
	/** Based on fractional brownian motion. */
	FractionalBrownian,
};

namespace PCGSpatialNoise
{
	struct FLocalCoordinates2D
	{
		// coordinates to sample for the repeating edges (not 0 to 1, is scaled according to the settings scale)
		double X0;
		double Y0;
		double X1;
		double Y1;

		// how much to interpolate between the corners
		double FracX;
		double FracY;
	};

	FLocalCoordinates2D CalcLocalCoordinates2D(const FBox& ActorLocalBox, const FTransform& ActorTransformInverse, FVector2D Scale, const FVector& Position);
	double CalcEdgeBlendAmount2D(const FLocalCoordinates2D& LocalCoords, double EdgeBlendDistance);
}

/**
 * Various fractal noises that can be used to filter points
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGSpatialNoiseSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGSpatialNoiseSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Spatial Noise")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSpatialNoise", "NodeTitle", "Spatial Noise"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
#endif
	virtual bool UseSeed() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	// The noise method used
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	PCGSpatialNoiseMode Mode = PCGSpatialNoiseMode::Perlin2D;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode == PCGSpatialNoiseMode::EdgeMask2D", EditConditionHides, PCG_Overridable))
	PCGSpatialNoiseMask2DMode EdgeMask2DMode = PCGSpatialNoiseMask2DMode::Perlin;

	// How many times the fractal method recurses. A higher number will mean more detail
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "1", ClampMax = "100", EditCondition = "Mode != PCGSpatialNoiseMode::Voronoi2D", EditConditionHides, PCG_Overridable))
	int32 Iterations = 4;

	// If true, will generate results that tile along the bounding box size of the PCG source.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bTiling = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	float Brightness = 0.0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	float Contrast = 1.0;

	// The output attribute name to write, if not 'None'
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyOutputNoSourceSelector ValueTarget;

	// Adds a random amount of offset up to this amount
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FVector RandomOffset = FVector(100000.0);

	// this will apply a transform to the points before calculating noise
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode != PCGSpatialNoiseMode::Voronoi2D || !bTiling", EditConditionHides, PCG_Overridable))
	FTransform Transform = FTransform::Identity;

	// The less random this is, the more it returns to being a grid (at zero).
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0", EditCondition = "Mode == PCGSpatialNoiseMode::Voronoi2D", EditConditionHides, PCG_Overridable))
	double VoronoiCellRandomness = 1.0;

	// The output attribute name to write the voronoi cell id, if not 'None'
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode == PCGSpatialNoiseMode::Voronoi2D", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyOutputNoSourceSelector VoronoiCellIDTarget;
	
	// If true it will orient the output points to point towards the cell edges, which can be used for effects
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode == PCGSpatialNoiseMode::Voronoi2D", EditConditionHides, PCG_Overridable))
	bool bVoronoiOrientSamplesToCellEdge = false;
	
	// The cell resolution of the tiled voronoi (across the bounds)
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode == PCGSpatialNoiseMode::Voronoi2D && bTiling", EditConditionHides, PCG_Overridable, ClampMin = "1"))
	int32 TiledVoronoiResolution = 8;

	// How many cells around the edge will tile
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode == PCGSpatialNoiseMode::Voronoi2D && bTiling", EditConditionHides, PCG_Overridable))
	int32 TiledVoronoiEdgeBlendCellCount = 2;

	// If > 0, we blend to a tiling edge value
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode == PCGSpatialNoiseMode::EdgeMask2D", EditConditionHides, PCG_Overridable))
	float EdgeBlendDistance = 1.0;

	// Adjust the center point of the curve (where x = curve(x) crosses over)
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode == PCGSpatialNoiseMode::EdgeMask2D", EditConditionHides, ClampMin = "0", PCG_Overridable))
	float EdgeBlendCurveOffset = 1.0;

	// Will make the falloff harsher or softer
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode == PCGSpatialNoiseMode::EdgeMask2D", EditConditionHides, ClampMin = "0", PCG_Overridable))
	float EdgeBlendCurveIntensity = 1.0;

private:
	// Private field to fix a deprecation issue related to seed usage between UE 5.3 and UE 5.4
	UPROPERTY()
	bool bForceNoUseSeed = false;
};

class FPCGSpatialNoise : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
