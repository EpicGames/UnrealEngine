// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSubgraph.h"
#include "Helpers/PCGDataLayerHelpers.h"
#include "Helpers/PCGHLODHelpers.h"
#include "Metadata/PCGObjectPropertyOverride.h"

#include "PCGSpawnActor.generated.h"

class AActor;
class UPCGGraphInterface;
class UPCGBasePointData;

UENUM()
enum class EPCGSpawnActorOption : uint8
{
	CollapseActors UMETA(Tooltip="Extracts the mesh(es) from the actor class and creates a single merged representation instead and merges graph execution if needed."),
	MergePCGOnly UMETA(Tooltip="Spawns actors as-is but any PCG component on the actor will merge into a single graph execution."),
	NoMerging UMETA(Tooltip="Spawns one actor per point.")
};

UENUM()
enum class EPCGSpawnActorGenerationTrigger : uint8
{
	Default UMETA(Tooltip="Default; will trigger generation if a PCG Component exists and has the 'Generate On Load' trigger."),
	ForceGenerate UMETA(Tooltip = "Will trigger generation if a PCG Component exists, regardless of generation trigger."),
	DoNotGenerateInEditor UMETA(Tooltip = "Will not trigger generation in the editor but decays to Default otherwise."),
	DoNotGenerate UMETA(Tooltip = "Will never call generation on any PCG components")
};

/*
* PCG settings class that allows spawning actors with some options to perform the work more efficiently.
* Note that depending on the options, any PCG components on the spawned actors can be also generated,
* which is why this class derives from UPCGBaseSubgraphSettings - it has similar inner-workings to the subgraph node
* as far as data passing and dispatch go.
* Note that at this point in time, results from the underlying graphs being generated is not propagated back as results of this node.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSpawnActorSettings : public UPCGBaseSubgraphSettings
{
	GENERATED_BODY()

public:
	UPCGSpawnActorSettings(const FObjectInitializer& ObjectInitializer);

	/** 
	 * Can specify a list of functions from the template class to be called on each actor spawned, in order. Need to have "CallInEditor" flag enabled
	 * and have either no parameters or exactly the parameters PCGPoint and PCGMetadata
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Option!=EPCGSpawnActorOption::CollapseActors", EditConditionHides))
	TArray<FName> PostSpawnFunctionNames;

	/** 
	* Controls how actors are spawned; collapsed in more efficient components or as-is with generation or not. 
	* Note that new nodes now have the 'No Merging' option by default.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGSpawnActorOption Option = EPCGSpawnActorOption::CollapseActors; 

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Option==EPCGSpawnActorOption::NoMerging", EditConditionHides))
	bool bForceDisableActorParsing = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Option==EPCGSpawnActorOption::NoMerging", EditConditionHides))
	EPCGSpawnActorGenerationTrigger GenerationTrigger = EPCGSpawnActorGenerationTrigger::Default;

	/** Warning: inheriting parent actor tags work only in non-collapsed actor hierarchies */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Option!=EPCGSpawnActorOption::CollapseActors", EditConditionHides))
	bool bInheritActorTags = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Option!=EPCGSpawnActorOption::CollapseActors", EditConditionHides))
	TArray<FName> TagsToAddOnActors;

	/** Overrides the default label that would be set on the spawned actors (class based) by a user-specified one. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Option!=EPCGSpawnActorOption::CollapseActors", EditConditionHides, PCG_Overridable))
	FString ActorLabel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, Category = Settings, meta = (ShowInnerProperties, EditCondition = "GetAllowTemplateActorEditing()", EditConditionHides))
	TObjectPtr<AActor> TemplateActor;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "Option != EPCGSpawnActorOption::CollapseActors", EditConditionHides))
	TArray<FPCGObjectPropertyOverrideDescription> SpawnedActorPropertyOverrideDescriptions;

	UPROPERTY(meta = (PCG_Overridable))
	TSoftObjectPtr<AActor> RootActor;

	/** 
	* Controls where spawned actors will appear in the Outliner. Note that attaching actors to an actor couples their streaming. 
	* Note that new nodes now have the 'In Folder' option by default.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Option != EPCGSpawnActorOption::CollapseActors", EditConditionHides))
	EPCGAttachOptions AttachOptions = EPCGAttachOptions::Attached; 

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bSpawnByAttribute = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bSpawnByAttribute", PCG_Overridable))
	FName SpawnAttribute = NAME_None;

	/** Adds a warning to the node on repeated spawning with identical conditions (ie. same actor at same spawn location, etc). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bWarnOnIdenticalSpawn = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bDeleteActorsBeforeGeneration = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming, meta = (PCG_Overridable))
	FPCGDataLayerSettings DataLayerSettings;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming, DisplayName = "HLOD Settings", meta = (PCG_Overridable))
	FPCGHLODSettings HLODSettings;

protected:
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (OnlyPlaceable, DisallowCreateNew))
	TSubclassOf<AActor> TemplateActorClass;

	/** Can modify the template actor for the expected class. When spawning by attribute, it is mandatory for that template actor to be a parent class of ALL actors that will be spawned. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditConditionHides, EditCondition = "Option != EPCGSpawnActorOption::CollapseActors"))
	bool bAllowTemplateActorEditing = false;

public:
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;

#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
#endif // WITH_EDITOR

	//~Begin UPCGSettings interface
	virtual UPCGNode* CreateNode() const override;

	PCG_API void SetTemplateActorClass(const TSubclassOf<AActor>& InTemplateActorClass);
	PCG_API void SetAllowTemplateActorEditing(bool bInAllowTemplateActorEditing);
	const TSubclassOf<AActor>& GetTemplateActorClass() const { return TemplateActorClass; }

	UFUNCTION()
	PCG_API bool GetAllowTemplateActorEditing() const;

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SpawnActor")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSpawnActorSettings", "NodeTitle", "Spawn Actor"); }
	virtual EPCGSettingsType GetType() const override;
#endif

protected:
#if WITH_EDITOR	
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	//~Begin UPCGBaseSubgraphSettings interface
	// When using spawn by attribute, the potential execution of subgraphs will be done in a dynamic manner
	virtual bool IsDynamicGraph() const { return bSpawnByAttribute; }
	virtual UPCGGraphInterface* GetSubgraphInterface() const override;

	static UPCGGraphInterface* GetGraphInterfaceFromActorSubclass(TSubclassOf<AActor> InTemplateActorClass);

protected:
#if WITH_EDITOR
	//~End UPCGBaseSubgraphSettings interface

	void SetupBlueprintEvent();
	void TeardownBlueprintEvent();
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances);
private:
	void RefreshTemplateActor();
#endif

	friend class FPCGSpawnActorElement;
};

UCLASS(ClassGroup = (Procedural))
class UPCGSpawnActorNode : public UPCGBaseSubgraphNode
{
	GENERATED_BODY()
public:
	/** ~Begin UPCGBaseSubgraphNode interface */
	virtual TObjectPtr<UPCGGraphInterface> GetSubgraphInterface() const override;
	/** ~End UPCGBaseSubgraphNode interface */
};

class FPCGSpawnActorElement : public FPCGSubgraphElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool IsPassthrough(const UPCGSettings* InSettings) const override { return !InSettings; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;

private:
	bool SpawnAndPrepareSubgraphs(FPCGSubgraphContext* Context, const UPCGSpawnActorSettings* Settings) const;

	TArray<FName> GetNewActorTags(FPCGContext* Context, AActor* TargetActor, bool bInheritActorTags, const TArray<FName>& AdditionalTags) const;

	void CollapseIntoTargetActor(FPCGSubgraphContext* Context, AActor* TargetActor, TSubclassOf<AActor> TemplateActorClass, const UPCGBasePointData* PointData) const;
	void SpawnActors(FPCGSubgraphContext* Context, UObject* TargetWorldObject, TSubclassOf<AActor> TemplateActorClass, AActor* TemplateActor, FPCGTaggedData& Output, const UPCGBasePointData* PointData, UPCGBasePointData* OutPointData) const;
};
