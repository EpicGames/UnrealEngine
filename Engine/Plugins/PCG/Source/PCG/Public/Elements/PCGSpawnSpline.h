// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Components/SplineComponent.h"
#include "Elements/PCGAddComponent.h"
#include "Metadata/PCGObjectPropertyOverride.h"

#include "PCGSpawnSpline.generated.h"

/**
* Spawn a spline component from a spline data.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSpawnSplineSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spawner; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Class of the component to spawn, must be a subclass of Spline Component. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (OnlyPlaceable, PCG_Overridable))
	TSubclassOf<USplineComponent> SplineComponent = USplineComponent::StaticClass();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bSpawnComponentFromAttribute = false;

	/** If the class of the component to spawn is coming from an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable, EditCondition = bSpawnComponentFromAttribute, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyInputSelector SpawnComponentFromAttributeName;

	UPROPERTY(meta = (PCG_Overridable))
	TSoftObjectPtr<AActor> TargetActor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TArray<FName> TagsToAddOnComponents;

	/** Specify a list of functions to be called on the target actor after spline creation. Functions need to be parameter-less and with "CallInEditor" flag enabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FName> PostProcessFunctionNames;

	/** Overrides to apply on the spawned component. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings")
	TArray<FPCGObjectPropertyOverrideDescription> PropertyOverrideDescriptions;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bOutputSplineComponentReference = true;

	/** Can output the spawned component reference in an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable, EditCondition = bOutputSplineComponentReference))
	FName ComponentReferenceAttributeName = PCGAddComponentConstants::ComponentReferenceAttribute;
};

class FPCGSpawnSplineElement : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext*) const override { return true; };
	virtual bool IsCacheable(const UPCGSettings*) const override { return false; };
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

