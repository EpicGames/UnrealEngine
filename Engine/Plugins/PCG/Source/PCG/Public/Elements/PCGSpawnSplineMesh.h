// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"
#include "PCGSplineMeshParams.h"
#include "PCGTimeSlicedElementBase.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Helpers/PCGActorHelpers.h"
#include "Metadata/PCGObjectPropertyOverride.h"

#include "Components/SplineMeshComponent.h"
#include "Engine/SplineMeshComponentDescriptor.h"

#include "PCGSpawnSplineMesh.generated.h"

struct FPCGContext;
class UPCGLandscapeSplineData;
class UPCGPolyLineData;

/** Create a USplineMeshComponent for each segment along a given spline. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSpawnSplineMeshSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SpawnSplineMesh")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSpawnSplineMeshElement", "NodeTitle", "Spawn Spline Mesh"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spawner; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(EditAnywhere, Category = Settings)
	FSoftSplineMeshComponentDescriptor SplineMeshDescriptor;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGSplineMeshParams SplineMeshParams;

	UPROPERTY(meta = (PCG_Overridable))
	TSoftObjectPtr<AActor> TargetActor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TArray<FName> TagsToAddOnComponents;

	/** Specify a list of functions to be called on the target actor after spline mesh creation. Functions need to be parameter-less and with "CallInEditor" flag enabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FName> PostProcessFunctionNames;

	/** Force meshes/materials to load synchronously. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;

	/** Overrides for spline mesh descriptor. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TArray<FPCGObjectPropertyOverrideDescription> SplineMeshOverrideDescriptions;

	/** Overrides for spline mesh params. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TArray<FPCGObjectPropertyOverrideDescription> SplineMeshParamsOverride;

	/** Overrides for the spline mesh component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TArray<FPCGObjectPropertyOverrideDescription> SplineMeshComponentOverride;
};

struct FPCGSpawnSplineMeshPerExecutionState
{
	AActor* TargetActor = nullptr;
};

struct FPCGSpawnSplineMeshPerIterationState
{
	FPCGSplineMeshComponentBuilderParameters SMCBuilderParams;
	FPCGObjectOverrides<FSplineMeshComponentDescriptor> DescriptionOverrides = FPCGObjectOverrides(&SMCBuilderParams.Descriptor);
	FPCGObjectOverrides<FPCGSplineMeshParams> ParamsOverrides = FPCGObjectOverrides(&SMCBuilderParams.SplineMeshParams);
	FPCGObjectOverrides<USplineMeshComponent> ComponentOverrides{nullptr};
	const UPCGPolyLineData* SplineData = nullptr;
	const UPCGLandscapeSplineData* LandscapeSplineData = nullptr;
	int32 ElementIndex = 0;
};

struct FPCGSpawnSplineMeshContext : public FPCGContext, public IPCGAsyncLoadingContext {};

class FPCGSpawnSplineMeshElement : public TPCGTimeSlicedElementBase<FPCGSpawnSplineMeshPerExecutionState, FPCGSpawnSplineMeshPerIterationState, FPCGSpawnSplineMeshContext>
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
