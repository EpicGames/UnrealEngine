// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGSplineDirection.generated.h"

class UPCGSplineData;

namespace PCGSplineDirection
{
	/** Test spline controls points to know if they are clockwise on the XY plane (clockwise around the Z axis). */
	PCG_API bool IsClockwiseXY(const UPCGSplineData* InputSplineData);

	/** Return the same spline data, but with its control points in reverse order. */
	PCG_API UPCGSplineData* Reverse(const UPCGSplineData* InputSplineData, FPCGContext* Context);
}

UENUM()
enum class EPCGReverseSplineOperation
{
	/** Will reverse the spline points regardless of their orientation. */
	Reverse,

	/** Will reverse the spline points if they are counter clockwise. */
	ForceClockwise,

	/** Will reverse the spline points if they are clockwise. */
	ForceCounterClockwise
};

class FPCGSplineDirectionElement;

/**
* Direct the order of a spline's control points.
* This can be conditional to force a given orientation (clockwise or counter clockwise).
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGReverseSplineSettings : public UPCGSettings
{
	friend FPCGSplineDirectionElement;

	GENERATED_BODY()

public:
	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface
	
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif

	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	EPCGReverseSplineOperation Operation = EPCGReverseSplineOperation::Reverse;
	
private:
	// UE_DEPRECATED(5.6, "Deprecation to be removed at some point")
	/* Prior to 5.6, the CW/CCW were reversed. This is added to make sure we don't break data existing prior (e.g. from 5.5), but will throw a warning to be fixed. */
	UPROPERTY()
	bool bFlipClockwiseComputationResult = false;
};

class FPCGSplineDirectionElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
