// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGSplineIntersection.generated.h"

UENUM()
enum class EPCGSplineIntersectionType : uint8
{
	/** Performs self-intersection only on a per-spline basis. */
	Self UMETA(Hidden),

	/** Performs intersection test for each spline against each other spline. */
	AgainstOtherSplines
};

UENUM()
enum class EPCGSplineIntersectionOutput : uint8
{
	/** Returns only the intersection positions. */
	IntersectionPointsOnly,

	/** Adds control points at the intersections on the input splines. */
	OriginalSplinesWithIntersections
};

UENUM(BlueprintType)
enum class EPCGSplineIntersectionPointType : uint8
{
	NotAnIntersection = 0,
	SinglePointIntersection = 1,
	IntersectionStart = 2,
	IntersectionEnd = 3
};

/** 
* Intersects splines against other splines (or themselves) and returns varied results
* based on user need.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGSplineIntersectionSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGSplineIntersectionSettings();
	
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Controls type of intersection test that will be done. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGSplineIntersectionType Type = EPCGSplineIntersectionType::AgainstOtherSplines;

	/** Controls type of output result. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGSplineIntersectionOutput Output = EPCGSplineIntersectionOutput::OriginalSplinesWithIntersections;

	/** Controls whether the output will contain the index of the intersecting splines */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(EditCondition="Type!=EPCGSplineIntersectionType::Self"))
	bool bOutputSplineIndices = false;

	/** Attribute that will contain the first spline index (only in the output case of intersection points.) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection, EditCondition="bOutputSplineIndices && Type!=EPCGSplineIntersectionType::Self && Output==EPCGSplineIntersectionOutput::IntersectionPointsOnly"))
	FPCGAttributePropertyOutputNoSourceSelector OriginatingSplineIndexAttribute;

	/** Attribute that will contain the first intersecting spline index, or -1 if not an intersection control point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection, EditCondition="bOutputSplineIndices"))
	FPCGAttributePropertyOutputNoSourceSelector IntersectingSplineIndexAttribute;

	/** Controls whether we'll output intersection ranges (start/end) when relevant instead of singular points. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(PCG_Overridable))
	bool bOutputComplexIntersections = false;

	/** Controls whether intersection types will be written to the attributes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(PCG_Overridable, InlineEditConditionToggle))
	bool bOutputIntersectionType = false;

	/** Attribute that will contain the intersection type (0: not an intersection, 1: point intersection, 2: intersection start, 3: intersection end.) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection, EditCondition="bOutputIntersectionType"))
	FPCGAttributePropertyOutputNoSourceSelector IntersectionTypeAttribute;

	/** Maximum distance at which we report an intersection on the splines. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(ClampMin="0.0001", PCG_Overridable))
	double DistanceThreshold = 10.0;

	/** Controls whether the output for intersection points is oriented using a best fit box (using PCA - Principal Component Analysis) or if it's aligned on the longest intersection. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(PCG_Overridable, EditCondition="Output==EPCGSplineIntersectionOutput::IntersectionPointsOnly", EditConditionHides))
	bool bUseBestFitBox = true;
};

class FPCGSplineIntersectionElement : public IPCGElement
{
public:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};