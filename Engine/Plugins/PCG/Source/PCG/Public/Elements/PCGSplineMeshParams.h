// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSplineMeshParams.generated.h"

UENUM()
enum class EPCGSplineMeshForwardAxis : uint32
{
	X,
	Y,
	Z
};

USTRUCT(BlueprintType)
struct FPCGSplineMeshParams
{
	GENERATED_BODY()

	/** Chooses the forward axis for the spline mesh orientation. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Spline Mesh Params", meta = (PCG_Overridable))
	EPCGSplineMeshForwardAxis ForwardAxis = EPCGSplineMeshForwardAxis::X;

	/** Scale mesh to the spline control point bounds. Especially useful on Landscape Splines, where the bounds come from the width. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Spline Mesh Params", meta = (PCG_Overridable))
	bool bScaleMeshToBounds = false;

	/** Scale the mesh to the full width of the Landscape Spline (including Falloff). Only applies to Landscape Splines. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Spline Mesh Params", meta = (PCG_Overridable, EditCondition = "bScaleMeshToBounds"))
	bool bScaleMeshToLandscapeSplineFullWidth = false;

	/** Axis (in component space) that is used to determine X axis for coordinates along spline. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Spline Mesh Params", meta = (PCG_Overridable))
	FVector SplineUpDir = FVector(0.0, 0.0, 1.0);

	/**
	 * How much to scale the calculated culling bounds of Nanite clusters after deformation.
	 * NOTE: This should only be set greater than 1.0 if it fixes visible issues with clusters being
	 * incorrectly culled.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Spline Mesh Params", meta = (ClampMin = 1.0, PCG_Overridable))
	float NaniteClusterBoundsScale = 1.0f;

	/** Minimum coordinate along the spline forward axis which corresponds to start of spline. If set to 0.0, will use bounding box to determine bounds. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Spline Mesh Params", meta = (PCG_Overridable))
	float SplineBoundaryMin = 0.0f;

	/** Maximum coordinate along the spline forward axis which corresponds to end of spline. If set to 0.0, will use bounding box to determine bounds. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Spline Mesh Params", meta = (PCG_Overridable))
	float SplineBoundaryMax = 0.0f;

	/** If true, will use smooth interpolation (ease in/out) for Scale, Roll, and Offset along this section of spline. If false, uses linear. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Spline Mesh Params", meta = (PCG_Overridable))
	bool bSmoothInterpRollScale = true;

	/** Starting offset of the mesh from the spline, in component space. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Spline Mesh Params", meta = (PCG_Overridable))
	FVector2D StartOffset = FVector2D::ZeroVector;

	/** Ending offset of the mesh from the spline, in component space. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Spline Mesh Params", meta = (PCG_Overridable))
	FVector2D EndOffset = FVector2D::ZeroVector;

	// Below properties are not exposed for edit because they are computed internally.
	UPROPERTY(meta = (PCG_NotOverridable))
	FVector StartPosition = FVector::ZeroVector;

	UPROPERTY(meta = (PCG_NotOverridable))
	FVector StartTangent = FVector::ZeroVector;

	/** Start roll of the mesh. Computed automatically so can't be set there, but can be overridden in the Spline Mesh Params Overrides, if needed. */
	UPROPERTY(VisibleAnywhere, Category = "Spline Mesh Params", meta = (PCG_NotOverridable))
	float StartRollDegrees = 0.0f;

	UPROPERTY(meta = (PCG_NotOverridable))
	FVector2D StartScale = FVector2D::ZeroVector;

	UPROPERTY(meta = (PCG_NotOverridable))
	FVector EndPosition = FVector::ZeroVector;

	UPROPERTY(meta = (PCG_NotOverridable))
	FVector EndTangent = FVector::ZeroVector;

	/** End roll of the mesh. Computed automatically so can't be set there, but can be overridden in the Spline Mesh Params Overrides, if needed. */
	UPROPERTY(VisibleAnywhere, Category = "Spline Mesh Params", meta = (PCG_NotOverridable))
	float EndRollDegrees = 0.0f;

	UPROPERTY(meta = (PCG_NotOverridable))
	FVector2D EndScale = FVector2D::ZeroVector;

	friend uint32 GetTypeHash(const FPCGSplineMeshParams& Key);
	bool operator==(const FPCGSplineMeshParams& Other) const;

private:
	uint32 ComputeHash() const;
	mutable TOptional<uint32> Hash;
};
