// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Polygon2.h"
#include "Curves/CurveFloat.h"

#include "PCGSplineSampler.generated.h"

struct FPCGContext;
struct FPCGProjectionParams;
class UPCGBasePointData;
class UPCGPolyLineData;
class UPCGSpatialData;

namespace PCGSplineSamplerConstants
{
	const FName SplineLabel = TEXT("Spline");
	const FName BoundingShapeLabel = TEXT("Bounding Shape");
	const FName NextDirectionDeltaAttributeName = TEXT("NextDirectionDelta");
	const FName CurvatureAttributeName = TEXT("Curvature");
	const FName SegmentIndexAttributeName = TEXT("SegmentIndex");
	const FName SubsegmentIndexAttributeName = TEXT("SubsegmentIndex");
	const FName ArriveTangentAttributeName = TEXT("ArriveTangent");
	const FName LeaveTangentAttributeName = TEXT("LeaveTangent");
	const FName AlphaAttributeName = TEXT("Alpha");
	const FName DistanceAttributeName = TEXT("Distance");
	const FName InputKeyAttributeName = TEXT("InputKey");
	const FName InterpTypeAttributeName = TEXT("InterpType");
}

UENUM()
enum class EPCGSplineSamplingMode : uint8
{
	Subdivision = 0,
	Distance,
	NumberOfSamples UMETA(Tooltip = "Samples a specified number of times, evenly spaced around the spline.")
};

UENUM()
enum class EPCGSplineSamplingDimension : uint8
{
	OnSpline = 0,
	OnHorizontal,
	OnVertical,
	OnVolume,
	OnInterior
};

UENUM()
enum class EPCGSplineSamplingFill : uint8
{
	Fill = 0,
	EdgesOnly = 1
};

UENUM()
enum class EPCGSplineSamplingInteriorOrientation : uint8
{
	Uniform = 0,
	FollowCurvature = 1
};

UENUM()
enum class EPCGSplineSamplingSeedingMode : uint8
{
	SeedFromPosition = 0 UMETA(Tooltip = "Compute seed from sample position."),
	SeedFromIndex = 1 UMETA(Tooltip = "Compute seed from sample index.")
};

USTRUCT(BlueprintType)
struct FPCGSplineSamplerParams
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGSplineSamplingDimension Dimension = EPCGSplineSamplingDimension::OnSpline;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	EPCGSplineSamplingMode Mode = EPCGSplineSamplingMode::Subdivision;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Dimension!=EPCGSplineSamplingDimension::OnSpline&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	EPCGSplineSamplingFill Fill = EPCGSplineSamplingFill::Fill;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", EditCondition = "Mode==EPCGSplineSamplingMode::Subdivision&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	int32 SubdivisionsPerSegment = 1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0.1", EditCondition = "Mode==EPCGSplineSamplingMode::Distance&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	float DistanceIncrement = 100.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", EditCondition = "Mode==EPCGSplineSamplingMode::NumberOfSamples&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	int32 NumSamples = 8;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", EditCondition = "Dimension==EPCGSplineSamplingDimension::OnHorizontal||Dimension==EPCGSplineSamplingDimension::OnVolume", EditConditionHides))
	int32 NumPlanarSubdivisions = 8;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", EditCondition = "Dimension==EPCGSplineSamplingDimension::OnVertical||Dimension==EPCGSplineSamplingDimension::OnVolume", EditConditionHides))
	int32 NumHeightSubdivisions = 8;

	/** Distance (in cm) along the spline at which sampling will begin. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", EditCondition = "Mode!=EPCGSplineSamplingMode::Subdivision&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	float StartOffset = 0.0f;

	/** Distance (in cm) from the end of the spline at which sampling will end. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", EditCondition = "Mode!=EPCGSplineSamplingMode::Subdivision&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	float EndOffset = 0.0f;

	/** Normalized value for the maximum possible offset for each sample point. 0.0 means no offset, and 1.0 means DistanceIncrement / 2.0. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", EditCondition = "Mode!=EPCGSplineSamplingMode::Subdivision&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	float MaxRandomOffsetNormalized = 0.0f;

	/**
	 * If the length of the spline does not divide evenly into the DistanceIncrement, the final sample point will not land on the end of the spline. Enable this to force
	 * the DistanceIncrement to be rounded up to the nearest value which would yield evenly spaced samples across the entire length of the spline.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode==EPCGSplineSamplingMode::Distance&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bFitToCurve = false;
	
	/** The space between each sample point */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0.1", EditCondition = "Dimension==EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	float InteriorSampleSpacing = 100.0f;

	/** The space between each sample point on the spline boundary. Used for computation; lower spacing is more expensive but more accurate. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0.1", EditCondition = "Dimension==EPCGSplineSamplingDimension::OnInterior&&!bTreatSplineAsPolyline", EditConditionHides))
	float InteriorBorderSampleSpacing = 100.0f;

	/** Use the spline points to form a polyline, instead of computing many sample points along the spline. This is more accurate if your spline is linear. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Dimension==EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bTreatSplineAsPolyline = false;

	/** Determines the orientation of interior points. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Dimension==EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	EPCGSplineSamplingInteriorOrientation InteriorOrientation = EPCGSplineSamplingInteriorOrientation::Uniform;

	/** Project sample points onto one possible surface given by the spline boundary. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Dimension==EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bProjectOntoSurface = false;

	// TODO: DirtyCache for OnDependencyChanged when this float curve is an external asset
	/** Defines the density for each sample based on its distance from the spline. X axis is normalized distance to boundary (0-1), Y axis is density value. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverridableChildProperties="ExternalCurve", EditCondition = "Dimension==EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	FRuntimeFloatCurve InteriorDensityFalloffCurve;

	/** Compute the delta angle to the next point on the spline and write it to an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bComputeDirectionDelta = false;

	/** Attribute that will contain the delta angle to the next point on the spline w.r.t to the current's point Up vector. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "bComputeDirectionDelta&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	FName NextDirectionDeltaAttribute = PCGSplineSamplerConstants::NextDirectionDeltaAttributeName;

	/** Compute curvature along the spline and write it to an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bComputeCurvature = false;

	/** Attribute that will contain the curvature. Note that the radius of curvature is defined as 1/Curvature, and might need you to scale to world units. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "bComputeCurvature&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	FName CurvatureAttribute = PCGSplineSamplerConstants::CurvatureAttributeName;

	/** Compute the spline segment index and write it to an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bComputeSegmentIndex = false;

	/** Attribute that will contain the spline segment index. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "bComputeSegmentIndex&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	FName SegmentIndexAttribute = PCGSplineSamplerConstants::SegmentIndexAttributeName;

	/** Compute the sub-segment index of a point on the spline and write it to an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "Mode==EPCGSplineSamplingMode::Subdivision&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bComputeSubsegmentIndex = false;

	/** Attribute that will contain the sub-segment index of a point on the spline. When the sub-segment index is 0, the point is a control point on the actual spline. Only applies to Subdivision mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "bComputeSubsegmentIndex&&Mode==EPCGSplineSamplingMode::Subdivision&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	FName SubsegmentIndexAttribute = PCGSplineSamplerConstants::SubsegmentIndexAttributeName;

	/** Compute arrive and leave tangents along the spline and write them to attributes. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bComputeTangents = false;

	/** Attribute that will contain the arrive tangent vector. For control points, this will be the actual arrive tangent. For non-control points, this will only be the normalized tangent at this point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "bComputeTangents&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	FName ArriveTangentAttribute = PCGSplineSamplerConstants::ArriveTangentAttributeName;

	/** Attribute that will contain the leave tangent vector. For control points, this will be the actual leave tangent. For non-control points, this will only be the normalized tangent at this point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "bComputeTangents&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	FName LeaveTangentAttribute = PCGSplineSamplerConstants::LeaveTangentAttributeName;

	/** Compute an alpha value along the spline and write it to an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bComputeAlpha = false;

	/**
	 * Attribute that will contain a value in [0,1] representing how far along the point is to the end of the line. Each segment on the line represents a same-size interval.
	 * For example, if there are three segments, each segment will take up 0.333... of the interval.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "bComputeAlpha&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	FName AlphaAttribute = PCGSplineSamplerConstants::AlphaAttributeName;

	/** Compute distance along the spline and write it to an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bComputeDistance = false;

	/** Attribute that will contain the distance along the spline at the sample point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "bComputeDistance&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	FName DistanceAttribute = PCGSplineSamplerConstants::DistanceAttributeName;

	/** Compute an input key for each point along the spline and write the key to an attribute. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	bool bComputeInputKey = false;

	/** Attribute that will contain the spline input key, a float value between [0, N], where N is the number of control points. Each range [i, i+1] represents an interpolation from 0 to 1 across spline segment i. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Attributes", meta = (EditCondition = "bComputeInputKey&&Dimension!=EPCGSplineSamplingDimension::OnInterior", EditConditionHides))
	FName InputKeyAttribute = PCGSplineSamplerConstants::InputKeyAttributeName;

	/** If no Bounding Shape input is provided, the actor bounds are used to limit the sample generation domain.
	* This option allows ignoring the actor bounds and generating over the entire spline. Use with caution as this
	* may generate a lot of points.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUnbounded = false; 

	/** Each PCG point represents a discretized, volumetric region of world space. The points' Steepness value [0.0 to
	 * 1.0] establishes how "hard" or "soft" that volume will be represented. From 0, it will ramp up linearly
	 * increasing its influence over the density from the point's center up to two times the bounds. At 1, it will
	 * represent a binary box function with the size of the point's bounds.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Points", meta=(ClampMin="0", ClampMax="1", PCG_Overridable))
	float PointSteepness = 0.5f;

	/** Controls the mode for computing a sample point's seed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Seeding");
	EPCGSplineSamplingSeedingMode SeedingMode = EPCGSplineSamplingSeedingMode::SeedFromPosition;

	/** Controls whether we will seed the sampled points using the final world position or the local position. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Seeding", meta=(EditCondition = "SeedingMode==EPCGSplineSamplingSeedingMode::SeedFromPosition", EditConditionHides))
	bool bSeedFromLocalPosition = false;

	/** Controls whether we will seed the sampled points using the 3D position or the 2D (XY) position. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Seeding", meta=(EditCondition = "SeedingMode==EPCGSplineSamplingSeedingMode::SeedFromPosition", EditConditionHides))
	bool bSeedFrom2DPosition = false;
};

namespace PCGSplineSamplerHelpers
{
	/** Samples on spline or within volume around it. */
	void SampleLineData(FPCGContext* Context, const UPCGPolyLineData* LineData, const UPCGSpatialData* InBoundingShape, const UPCGSpatialData* InProjectionTarget, const FPCGProjectionParams& InProjectionParams, const FPCGSplineSamplerParams& Params, UPCGBasePointData* OutPointData);

	/** Samples 2D region bounded by spline. */
	void SampleInteriorData(FPCGContext* Context, const UPCGPolyLineData* LineData, const UPCGSpatialData* InBoundingShape, const UPCGSpatialData* InProjectionTarget, const FPCGProjectionParams& InProjectionParams, const FPCGSplineSamplerParams& Params, UPCGBasePointData* OutPointData);

	const UPCGPolyLineData* GetPolyLineData(const UPCGSpatialData* InSpatialData);

	/**
	 * Tests if a point lies inside the given polygon. Polygon MUST be CounterClockwise otherwise the result will be incorrect.
	 */
	bool PointInsidePolygon2D(const UE::Geometry::FPolygon2d& PolygonPoints, const FVector2D& Point);

	/** Projects a point in space onto the approximated surface defined by a closed spline. Returns the height of the point after projection. */
	FVector::FReal ProjectOntoSplineInteriorSurface(const TArray<FVector>& SplinePoints, const FVector& PointToProject);
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSplineSamplerSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGSplineSamplerSettings();

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SplineSampler")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSplineSamplerSettings", "NodeTitle", "Spline Sampler"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Sampler; }
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif
	virtual bool UseSeed() const override { return true; }


protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGSplineSamplerParams SamplerParams;
};

class FPCGSplineSamplerElement : public IPCGElement
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;
	// Worth computing a full CRC in case we can halt change propagation/re-executions
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
