// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGSplitPoints.generated.h"

UENUM()
enum class EPCGSplitAxis
{
	X = 0,
	Y,
	Z
};

/**
 * Splits each point into two separate points, and sets bounds based on the position and axis of the cut.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGSplitPointsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SplitPoints")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSplitPointsElement", "NodeTitle", "Split Points"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGSplitPointsElement", "NodeTooltip", "Splits each input point into two separate points and sets bounds based on the position and axis of the cut."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
#endif
	

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties();  }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0", ClampMax = "1", PCG_Overridable))
	float SplitPosition = 0.5f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGSplitAxis SplitAxis = EPCGSplitAxis::Z;
};

class FPCGSplitPointsElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};