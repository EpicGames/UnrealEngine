// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Data/PCGSplineData.h"

#include "PCGSplitSplines.generated.h"

UENUM()
enum class EPCGSplitSplineMode
{
	ByKey,
	ByDistance,
	ByAlpha,
	ByPredicateOnControlPoints,
};

UENUM()
enum class EPCGSplitSplineDataBehavior
{
	KeepAllSplines,
	KeepCutSplines,
	KeepInteriorSplinesOnly
};

namespace PCGSplitSpline
{
	enum class EStatus
	{
		Split,
		Skipped,
		Error
	};

	EStatus SplitSpline(const UPCGSplineData* Spline, UPCGSplineData* SplitSpline, double StartKey, double EndKey);
	EStatus SplitSpline(const UPCGSplineData* Spline, UPCGSplineData* SplitSpline, TConstArrayView<FSplinePoint> SplinePoints, TConstArrayView<PCGMetadataEntryKey> SplinePointsEntryKeys, double StartKey, double EndKey);
}

/** Splits spline at a specific distance(s), key(s) or at certain values. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGSplitSplinesSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGSplitSplinesSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Split criteria. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGSplitSplineMode Mode = EPCGSplitSplineMode::ByAlpha;

	/** Behavior of whether we'll keep uncut, cut on one side or only interior splines. Note that for closed splines, all cut splines are interior splines. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGSplitSplineDataBehavior CutDataBehavior = EPCGSplitSplineDataBehavior::KeepAllSplines;

	/** Controls whether the input splines will be cut using a single constant or values driven either by an additional input or based on a predicate on the spline control points. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition ="Mode != EPCGSplitSplineMode::ByPredicateOnControlPoints"))
	bool bUseConstant = true;

	/** Constant (either Key, Distance or Alpha) that will be used to split the splines. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition ="bUseConstant && Mode != EPCGSplitSplineMode::ByPredicateOnControlPoints", ClampMin = "0", PCG_Overridable))
	double Constant = 0.5;

	/** Attribute identifying either the provenance of the split constants (key, distance or alpha) or the predicate on the control points. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition="!bUseConstant || Mode == EPCGSplitSplineMode::ByPredicateOnControlPoints", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector Attribute;

	/** Controls whether the output spline will have an attribute containing the index of the originating spline. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle, PCG_Overridable))
	bool bShouldOutputOriginatingSplineIndex = true;

	/** Attribute to write the originating spline index to. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition="bShouldOutputOriginatingSplineIndex", PCG_DiscardPropertySelection, PCG_DiscardExtraSelection, PCG_Overridable))
	FPCGAttributePropertyOutputSelector OutputOriginatingSplineIndex;
};

class FPCGSplitSplineElement : public IPCGElement
{
public:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override;
};