// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/PCGStaticMeshSpawnerContext.h"
#include "Metadata/PCGObjectPropertyOverride.h"

#include "Engine/CollisionProfile.h"

#include "PCGStaticMeshSpawner.generated.h"

#define UE_API PCG_API

class UPCGInstanceDataPackerBase;
class UPCGMeshSelectorBase;
class UPCGSpatialData;
struct FPCGContext;
class UStaticMesh;

namespace PCGStaticMeshSpawner
{
	static constexpr FLazyName PrimitiveTablePinLabel = "PrimitiveTable";
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural), meta = (PrioritizeCategories = "Settings"))
class UPCGStaticMeshSpawnerSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UE_API UPCGStaticMeshSpawnerSettings(const FObjectInitializer &ObjectInitializer);

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual bool DisplayExecuteOnGPUSetting() const override { return true; }
	UE_API virtual void CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const override;
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("StaticMeshSpawner")); }
	UE_API virtual FText GetDefaultNodeTitle() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spawner; }
	UE_API virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
#endif // WITH_EDITOR
	virtual bool UseSeed() const override { return true; }

	int32 GetCullingCellSize() const;

protected:
	UE_API virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	UE_API virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	UE_API virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif
	// ~End UPCGSettings interface

public:
	// ~Begin UObject interface
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	// ~End UObject interface
#endif

	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetMeshSelectorType(TSubclassOf<UPCGMeshSelectorBase> InMeshSelectorType);

	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetInstancePackerType(TSubclassOf<UPCGInstanceDataPackerBase> InInstancePackerType);

public:
	/** Defines the method of mesh selection per input data */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, NoClear, Category = MeshSelector)
	TSubclassOf<UPCGMeshSelectorBase> MeshSelectorType;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Instanced, Category = MeshSelector)
	TObjectPtr<UPCGMeshSelectorBase> MeshSelectorParameters;

	/** Allows PCG to make some changes on the descriptors as situation arises (using ISM instead of HISM for nanite meshes, etc.) */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = MeshSelector)
	bool bAllowDescriptorChanges = true;

	/**
	 * Defines the method of custom data packing for spawned (H)ISMCs. Note, Rotators are treated as 3 floats, while Quaternions are
	 * treated as 4 floats. You can see an attribute's type in the 'Attribute List View' window, and use an 'Attribute Cast' node to cast to the desired type.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = InstanceDataPacker)
	TSubclassOf<UPCGInstanceDataPackerBase> InstanceDataPackerType;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Instanced, Category = InstanceDataPacker)
	TObjectPtr<UPCGInstanceDataPackerBase> InstanceDataPackerParameters = nullptr;

	/** Map an attribute directly to an ISM Descriptor property, the value of which will be overriden when generated.
	 * Note: Currently only enabled using SelectByAttribute mesh selection.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FPCGObjectPropertyOverrideDescription> StaticMeshComponentPropertyOverrides;

	/** Attribute name to store mesh SoftObjectPaths inside if the output pin is connected. Note: Will overwrite existing data if the attribute name already exists. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FName OutAttributeName = NAME_None;

	/** Sets the BoundsMin and BoundsMax attributes of each point to reflect the StaticMesh spawned at its location */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bApplyMeshBoundsToPoints = true;

	UPROPERTY(BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	TSoftObjectPtr<AActor> TargetActor;

	/** Specify a list of functions to be called on the target actor after instances are spawned. Functions need to be parameter-less and with "CallInEditor" flag enabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FName> PostProcessFunctionNames;

	/** Meshes/Materials will be synchronously loaded before spawning instead of asynchronously. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;

	/** Controls whether instances stemming from different data can end up in the same ISM. Performance warning if set to false. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	bool bAllowMergeDifferentDataInSameInstancedComponents = true;

	/** Opt-in option to silence errors when the property override attributes are not found. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bSilenceOverrideAttributeNotFoundErrors = false;

	/** Adds a warning to the node on repeated spawning with identical conditions (ie. same mesh descriptor at same spawn location, etc). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bWarnOnIdenticalSpawn = true;

protected:
	/** Experimental: Divide instances into cells to allow renderer to cull subranges of instances early on the CPU. */
	UPROPERTY(EditAnywhere, Category = GPU, AdvancedDisplay, meta = (EditCondition = "bExecuteOnGPU", EditConditionHides))
	bool bSetupCullingCells = false;

	UPROPERTY(EditAnywhere, Category = GPU, AdvancedDisplay, meta = (EditCondition = "bExecuteOnGPU && bSetupCullingCells", EditConditionHides, ClampMin = "64", UIMin = "64", UIMax = "65536"))
	int32 CullingCellSize = 2048;

	UE_API void RefreshMeshSelector();
	UE_API void RefreshInstancePacker();
};

class FPCGStaticMeshSpawnerElement : public IPCGElement
{
public:
	UE_API virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	UE_API virtual FPCGContext* CreateContext() override;
	UE_API virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	UE_API virtual bool ExecuteInternal(FPCGContext* Context) const override;
	UE_API virtual void AbortInternal(FPCGContext* Context) const override;
	UE_API void SpawnStaticMeshInstances(FPCGStaticMeshSpawnerContext* Context, const FPCGStaticMeshSpawnerContext::FPackedInstanceListData& InstanceListData, const FPCGMeshInstanceList& InstanceList, const FPCGPackedCustomData* InPackedCustomData) const;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
	UE_API virtual EPCGDataOverridePhase GetDataOverridePhase() const override;
};

#undef UE_API
