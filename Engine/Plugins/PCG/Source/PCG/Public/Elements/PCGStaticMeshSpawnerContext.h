// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGManagedResource.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "MeshSelectors/PCGMeshSelectorBase.h"
#include "InstanceDataPackers/PCGInstanceDataPackerBase.h"

#include "PCGStaticMeshSpawnerContext.generated.h"

#define UE_API PCG_API

class AActor;
class UPCGSpatialData;

struct FPCGInstancesAndWeights
{
	TArray<TArray<FPCGMeshInstanceList>> MeshInstances;
	TArray<int> CumulativeWeights;
};

USTRUCT(BlueprintType)
struct FPCGStaticMeshSpawnerContext : public FPCGContext
#if CPP
	, public IPCGAsyncLoadingContext
#endif
{
	GENERATED_BODY()

	struct FPackedInstanceListData
	{
		UE_API FPackedInstanceListData();
		UE_API ~FPackedInstanceListData();
		AActor* TargetActor = nullptr;
		const UPCGSpatialData* SpatialData = nullptr;
		TArray<FPCGMeshInstanceList> MeshInstances;
		TArray<FPCGPackedCustomData> PackedCustomData;
		FPCGCrc DataCrc;
		bool bSkippedDueToReuse = false;
	};

	TArray<FPackedInstanceListData> MeshInstancesData;
	// Index of input in the context (for selection)
	int32 CurrentInputIndex = 0;
	// Index of MeshInstances/PackedCustomData in last MeshInstancesData element
	int32 CurrentDataIndex = 0;

	// Whole-execution variables
	bool bReuseCheckDone = false;
	bool bSkippedDueToReuse = false; // fully skipped

	// Per-input context variables
	bool bCurrentInputSetup = false;
	bool bSelectionDone = false;
	bool bPartitionDone = false;
	bool bCurrentDataSkippedDueToReuse = false;

	const UPCGBasePointData* CurrentPointData = nullptr;
	UPCGBasePointData* CurrentOutputPointData = nullptr;
	FPCGMeshMaterialOverrideHelper MaterialOverrideHelper;
	int32 CurrentPointIndex = 0;
	int32 CurrentWriteIndex = 0;

	// Used in all selectors if we have to change the out points bounds by the mesh bounds. Will be empty otherwise.
	// We need to keep all point indices that will spawn this mesh, in all output point data.

	TMap<TSoftObjectPtr<UStaticMesh>, TMap<UPCGBasePointData*, TArray<int32>>> MeshToOutPoints;

	// Used in weighted selector
	TArray<TArray<FPCGMeshInstanceList>> WeightedMeshInstances;
	TMap<TSoftObjectPtr<UStaticMesh>, PCGMetadataValueKey> MeshToValueKey;
	TArray<int> CumulativeWeights;

	// Used in the weighted by category selector
	TMap<PCGMetadataValueKey, FPCGInstancesAndWeights> CategoryEntryToInstancesAndWeights;

	// Used for mesh property overrides
	TArray<TArray<int32>> AttributeOverridePartition;
	TArray<FPCGSoftISMComponentDescriptor> OverriddenDescriptors;

	// Keeping track of all touched resources to allow for correct cleanup on abort
	TSet<TWeakObjectPtr<UPCGManagedISMComponent>> TouchedResources;
	
	// struct use to maintain the used resources and if those resources have been merged with another spawner
	struct FResourceReuseInfo
	{
		bool bMergedResources = false;
		TSet<TWeakObjectPtr<UPCGManagedISMComponent>> Resources;
	};
	
	// information about all the resources used by this spawner for this execution
	FResourceReuseInfo UsedResources;

	// information about all the resources used per input (data crc) by this spawner for this execution
	TMap<uint32, FResourceReuseInfo> DataCrcToUsedResources;

	UE_API void ResetInputIterationData();
};

#undef UE_API
