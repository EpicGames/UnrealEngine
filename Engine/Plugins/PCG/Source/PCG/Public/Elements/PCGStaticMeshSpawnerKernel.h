// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compute/PCGComputeKernel.h"

#include "PCGStaticMeshSpawnerKernel.generated.h"

class UPCGStaticMeshSpawnerSettings;

namespace PCGStaticMeshSpawnerKernelConstants
{
	const FName InstanceCountsPinLabel = TEXT("InstanceCounts");
	const FName PrimitiveTablePinLabel = TEXT("PrimitiveTable");
	const FName CullingCellMinMaxPositionsPinLabel = TEXT("CullingCellMinMaxPositions");
}

UCLASS()
class UPCGStaticMeshSpawnerKernel : public UPCGComputeKernel
{
	GENERATED_BODY()

public:
	virtual TSharedPtr<const FPCGDataCollectionDesc> ComputeOutputBindingDataDesc(FName InOutputPinLabel, UPCGDataBinding* InBinding) const override;
	virtual int ComputeThreadCount(const UPCGDataBinding* Binding) const override;
#if WITH_EDITOR
	virtual const TCHAR* GetSourceFilePath() const override { return TEXT("/Plugin/PCG/Private/Elements/PCGStaticMeshSpawner.usf"); }
	virtual FString GetCookedSource(FPCGGPUCompilationContext& InOutContext) const override;
	virtual FString GetEntryPoint() const override { return TEXT("PCGStaticMeshSpawnerCS"); }
	virtual void CreateAdditionalInputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
	virtual void CreateAdditionalOutputDataInterfaces(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, TArray<TObjectPtr<UComputeDataInterface>>& OutDataInterfaces) const override;
	virtual bool UploadInputPinDataToGPU(FName InPinLabel) const override;
#endif
	virtual void AddStaticCreatedStrings(TArray<FString>& InOutStringTable) const override;
	virtual void GetKernelAttributeKeys(TArray<FPCGKernelAttributeKey>& OutKeys) const override;

	virtual void GetInputPins(TArray<FPCGPinProperties>& OutPins) const override;
	virtual void GetOutputPins(TArray<FPCGPinPropertiesGPU>& OutPins) const override;

	void SetUseRawInstanceCountsBuffer(bool bInUseRawInstanceCountsBuffer) { bUseRawInstanceCountsBuffer = bInUseRawInstanceCountsBuffer; }

	bool GetProduceOutputPoints() const { return bProduceOutputPoints; }
	void SetProduceOutputPoints(bool bInProduceOutputPoints) { bProduceOutputPoints = bInProduceOutputPoints; }

	int32 GetTargetCullingGridSize() const { return TargetCullingGridSize; }
	void SetTargetCullingGridSize(int32 InTargetCullingGridSize) { TargetCullingGridSize = InTargetCullingGridSize; }

protected:
#if WITH_EDITOR
	virtual bool PerformStaticValidation() override;
#endif

protected:
	/** Consume the instance counts as a raw buffer instead of an attribute set. */
	UPROPERTY()
	bool bUseRawInstanceCountsBuffer = false;

	UPROPERTY()
	bool bProduceOutputPoints = true;

	/** Optionally divide primitive instances into cells and pass to renderer to aid culling. */
	UPROPERTY()
	int32 TargetCullingGridSize = 0;
};
