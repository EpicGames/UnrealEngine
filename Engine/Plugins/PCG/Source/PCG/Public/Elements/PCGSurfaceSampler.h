// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "PCGTimeSlicedElementBase.h"

#include "Math/Box.h"

#include "PCGSurfaceSampler.generated.h"

class UPCGNode;
class UPCGPointData;
class UPCGBasePointData;
class UPCGSurfaceData;
class UPCGSurfaceSamplerSettings;

namespace PCGSurfaceSamplerConstants
{
	const FName SurfaceLabel = TEXT("Surface");
	const FName BoundingShapeLabel = TEXT("Bounding Shape");
}

namespace PCGSurfaceSampler
{
	struct FSurfaceSamplerParams
	{
		void InitializeFromSettings(const UPCGSurfaceSamplerSettings* Settings);

		bool bUseLegacyGridCreationMethod = false;
		float PointsPerSquaredMeter = 1.0f;
		FVector PointExtents = FVector::One() * 0.5f;
		float Looseness = 0.0f;
		bool bApplyDensityToPoints = false;
		float PointSteepness = 0.0f;
#if WITH_EDITORONLY_DATA
		bool bKeepZeroDensityPoints = false;
#endif
	};

	struct FSurfaceSamplerData
	{
		bool Initialize(const UPCGSurfaceSamplerSettings* Settings, const FPCGContext* Context, const FBox& InEffectiveGridBounds, const FTransform& InSurfaceTransform = FTransform::Identity);
		PCG_API bool Initialize(const FPCGContext* Context, const FBox& InEffectiveGridBounds, const FTransform& InSurfaceTransform = FTransform::Identity);

		FIntVector2 ComputeCellIndices(int32 Index) const;

		FSurfaceSamplerParams Params;

		/** Whether the pre-projected points need a local->world transformation. */
		bool bNeedsLocalTransformation = false;

		/** Computed values **/
		FVector InterstitialDistance = FVector::ZeroVector;
		FVector InnerCellSize = FVector::ZeroVector;
		FVector InnerCellOffset = FVector::ZeroVector;
		FVector CellSize = FVector::ZeroVector;

		int32 CellMinX = 0;
		int32 CellMaxX = 0;
		int32 CellMinY = 0;
		int32 CellMaxY = 0;
		int32 CellCount = 0;
		float Ratio = 0.f;
		int Seed = 0;

		/** The displacement of the pre-projected, pre-sampled points. */
		FVector::FReal PreProjectionDisplacement = 0.0;

		/** Will be applied to pre-sampled points to transform them onto the pre-projection plane. */
		FMatrix PreProjectionTransform = FMatrix::Identity;
	};

	struct FSurfaceSamplerExecutionState
	{
		const UPCGSpatialData* BoundingShape = nullptr;
		FBox BoundingShapeBounds = FBox(EForceInit::ForceInit);
		TArray<const UPCGSurfaceData*> GeneratingShapes;
	};

	struct FSurfaceSamplerIterationState
	{
		FSurfaceSamplerData SamplerData;

		UPCGBasePointData* OutputData = nullptr;

		// This will be set to the same value as OutputData when using UPCGPointData type to support deprecated code
		UE_DEPRECATED(5.6, "Use OutputData instead")
		UPCGPointData* OutputPoints = nullptr;
	};

	/** Sample a surface and returns the resulting point data. Can't be timesliced. */
	PCG_API UPCGBasePointData* SampleSurface(FPCGContext* Context, const FSurfaceSamplerParams& ExecutionParams, const UPCGSurfaceData* InSurface, const UPCGSpatialData* InBoundingShape, const FBox& EffectiveBounds, TSubclassOf<UPCGBasePointData> PointDataClass = {});
		
	PCG_API UPCGPointData* SampleSurface(FPCGContext* Context, const UPCGSurfaceData* InSurface, const UPCGSpatialData* InBoundingShape, const FBox& EffectiveBounds, const FSurfaceSamplerParams& ExecutionParams);

	/** Sample a surface and write the results in the given point data. Can be timesliced, and will return false if the processing is not done, true otherwise. */
	bool SampleSurface(FPCGContext* Context, const FSurfaceSamplerData& SamplerData, const UPCGSurfaceData* InSurface, const UPCGSpatialData* InBoundingShape, UPCGBasePointData* SampledData, const bool bTimeSlicingIsEnabled = false);
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSurfaceSamplerSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGSurfaceSamplerSettings();

	//~Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SurfaceSampler")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGSurfaceSamplerSettings", "NodeTitle", "Surface Sampler"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Sampler; }
	virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const override;
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
#endif
	virtual bool UseSeed() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface

public:
	/** Controls the grid cell size, down to a minimum size defined by the Point Extents. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(ClampMin="0", PCG_Overridable))
	float PointsPerSquaredMeter = 0.1f;

	/** Extents of the points to create (e.g. half their total size) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FVector PointExtents = FVector(50.0f);

	/** Controls how points are placed in their cell with 0 being in the center and 1 being anywhere inside the full cell size. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta=(ClampMin="0", PCG_Overridable))
	float Looseness = 1.0f;

	/** If no Bounding Shape input is provided, the actor bounds are used to limit the sample generation domain.
	* This option allows ignoring the actor bounds and generating over the entire surface. Use with caution as this
	* may generate a lot of points.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUnbounded = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Points", meta = (PCG_Overridable))
	bool bApplyDensityToPoints = true;

	/** Each PCG point represents a discretized, volumetric region of world space. The points' Steepness value [0.0 to
	 * 1.0] establishes how "hard" or "soft" that volume will be represented. From 0, it will ramp up linearly
	 * increasing its influence over the density from the point's center up to two times the bounds. At 1, it will
	 * represent a binary box function with the size of the point's bounds.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Points", meta=(ClampMin="0", ClampMax="1", PCG_Overridable))
	float PointSteepness = 0.5f;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug", meta = (PCG_Overridable))
	bool bKeepZeroDensityPoints = false;

	UPROPERTY()
	float PointRadius_DEPRECATED = 0.0f;
#endif

	/** Controls whether the prior grid creation mechanism (with cells being extents * (1 + steepness), then filtered by points by squared meters) is used. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bUseLegacyGridCreationMethod = false;
};

class FPCGSurfaceSamplerElement : public TPCGTimeSlicedElementBase<PCGSurfaceSampler::FSurfaceSamplerExecutionState, PCGSurfaceSampler::FSurfaceSamplerIterationState>
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams&, FPCGCrc& OutCrc) const override;
	// Might be sampling landscape or other external data, worth computing a full CRC in case we can halt change propagation/re-executions
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override { return true; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
