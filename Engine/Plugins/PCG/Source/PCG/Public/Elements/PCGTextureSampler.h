// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Data/PCGTextureData.h"

#include "PCGTextureSampler.generated.h"

class UTexture;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGTextureSamplerSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
#endif
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetTextureData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGTextureSamplerSettings", "NodeTitle", "Get Texture Data"); }
	virtual FText GetNodeTooltipText() const override;
	virtual TOptional<FPCGNodeThumbnailProxy> GetNodeThumbnail() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; }
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

#if WITH_EDITOR
	void UpdateDisplayTextureArrayIndex();
#endif

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UFUNCTION(BlueprintGetter, meta = (BlueprintInternalUseOnly = "true"))
	EPCGTextureDensityFunction GetDensityFunctionEquivalent() const;

	UFUNCTION(BlueprintSetter, meta = (BlueprintInternalUseOnly = "true"))
	void SetDensityFunctionEquivalent(EPCGTextureDensityFunction DensityFunction);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	PCG_API void SetTexture(TSoftObjectPtr<UTexture> InTexture);
	TSoftObjectPtr<UTexture> GetTexture() const { return Texture; }

	/** Surface transform */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FTransform Transform = FTransform::Identity;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseAbsoluteTransform = false;

	/** Index of texture array slice. Only used when built with editor and if the type of Texture is UTexture2DArray. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = bDisplayTextureArrayIndex, EditConditionHides, HideEditConditionToggle, ClampMin = '0', PCG_Overridable))
	int TextureArrayIndex = 0;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "DensityFunction has been deprecated in favor of bUseDensitySourceChannel.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(BlueprintGetter=GetDensityFunctionEquivalent, BlueprintSetter=SetDensityFunctionEquivalent, Category = SpatialData, meta = (DeprecatedProperty, DeprecatedMessage = "Density function on GetTextureData is deprecated in favor of bUseDensitySourceChannel."))
	EPCGTextureDensityFunction DensityFunction = EPCGTextureDensityFunction::Multiply;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SpatialData, meta = (InlineEditConditionToggle, PCG_Overridable, PCG_OverrideAliases="DensityFunction"))
	bool bUseDensitySourceChannel = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition="bUseDensitySourceChannel", DisplayName = "Density Source Channel", PCG_Overridable))
	EPCGTextureColorChannel ColorChannel = EPCGTextureColorChannel::Alpha;

	/** Method used to determine the value for a sample based on the value of nearby texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGTextureFilter Filter = EPCGTextureFilter::Bilinear;

	/** The size of one texel in cm, used when calling ToPointData. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (UIMin = "1.0", ClampMin = "1.0", PCG_Overridable))
	float TexelSize = 50.0f;

	/** Whether to tile the source or to stretch it to fit target area. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUseAdvancedTiling = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Tiling", meta = (EditCondition = "bUseAdvancedTiling", PCG_Overridable))
	FVector2D Tiling = FVector2D(1.0, 1.0);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Tiling", meta = (EditCondition = "bUseAdvancedTiling", PCG_Overridable))
	FVector2D CenterOffset = FVector2D::ZeroVector;

	/** Rotation to apply when sampling texture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Tiling", meta = (UIMin = -360, ClampMin = -360, UIMax = 360, ClampMax = 360, Units = deg, EditCondition = "bUseAdvancedTiling", PCG_Overridable))
	float Rotation = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Tiling", meta = (EditionCondition = "bUseAdvancedTiling", PCG_Overridable))
	bool bUseTileBounds = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Tiling", meta = (EditCondition = "bUseAdvancedTiling && bUseTileBounds", PCG_Overridable))
	FVector2D TileBoundsMin = FVector2D(-0.5, -0.5);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Tiling", meta = (EditCondition = "bUseAdvancedTiling && bUseTileBounds", PCG_Overridable))
	FVector2D TileBoundsMax = FVector2D(0.5, 0.5);

#if WITH_EDITORONLY_DATA
	/** Even if the texture is not set to CPU-available, it can still be accessed from CPU memory under certain conditions (sRGB disabled, no mipmaps, and non-compressed format).
	 * Reading from CPU memory will be faster and more accurate than reading from GPU memory, since the texture will not be subject to compression or resolution clamping. Enable
	 * this flag to force a duplicate of the texture with the correct settings for CPU memory access. This is editor-only.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (DisplayName = "Force Editor Only CPU Sampling"))
	bool bForceEditorOnlyCPUSampling = false;
#endif

	/** If enabled, output points with a density value of zero will NOT be automatically filtered out. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bKeepZeroDensityPoints = false;

	/** By default, texture loading is asynchronous, can force it synchronous if needed. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;

protected:
	/** Texture specific parameters */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	TSoftObjectPtr<UTexture> Texture = nullptr;

#if WITH_EDITORONLY_DATA
	// Used to hide the 'TextureArrayIndex' property.
	UPROPERTY(Transient)
	bool bDisplayTextureArrayIndex = false;
#endif

	friend class FPCGTextureSamplerElement;

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "bSkipReadbackToCPU has been removed. CPU readback is now always deferred.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecatedMessage = "bSkipReadbackToCPU has been removed. CPU readback is now always deferred."))
	bool bSkipReadbackToCPU = false;
};

struct FPCGTextureSamplerContext : public FPCGContext, public IPCGAsyncLoadingContext
{
public:
	TObjectPtr<UPCGTextureData> TextureData;

	bool bTransformInitialized = false;

	FTransform Transform;

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;
};

class FPCGTextureSamplerElement : public IPCGElement
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;
	// Loading needs to be done on the main thread and accessing objects outside of PCG might not be thread safe, so taking the safe approach
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	virtual FPCGContext* CreateContext() override;
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
