// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"

/**
 * Rundown for utilizing the Time Slice Element and Context:
 * 
 * Create a struct to contain the state data for the element that will contain "per-execution" state data to be calculated only once
 * Create a struct to contain the state data for the element that will contain "per-iteration" state data for each item (usually validated inputs) to iterate through
 * Note: Either struct is optional and can be substituted with an empty struct if that aspect of the element is stateless
 * Override the element type with TTimeSlicedPCGElement with two template arguments, the first being the static struct,and the second being the "per-iteration"
 * Pass a function or lambda matching the correct signature to InitializePerExecutionState and initialize the static struct within
 * Do the same for InitializePerIterationStates, but also pass the number of iterations. This will iterate through a state initialization for each iteration
 * [Optional] If needed, mark UObjects with TrackObject or TrackObjectByName if you need to retrieve it later and to prevent their garbage collection.
 * DataIsPrepared SHOULD BE used to verify initialization was successful, such as in the ExecuteInternal if data was previously initialized in PrepareDataInternal
 * Call ExecuteSlice with an execution function or lambda that returns a boolean that is true once the full execution is completed, or false otherwise
 *
 * Note: See PCGSurfaceSampler.h/.cpp as an example.
 */

// Forward declaration for friending
template <typename PerExecutionStateT, typename PerIterationStateT, typename BaseContext = FPCGContext> class TPCGTimeSlicedElementBase;

/** The result of initializing the Per Execution or Per Iteration Time Slice States */
enum class EPCGTimeSliceInitResult : uint8
{
	Uninitialized = 0, // Initialization has not occurred yet
	Success,           // Initialization was a success
	NoOperation,       // Initialization was a success, but resulted in no operation
	AbortExecution     // Initialization was a failure. Should abort.
};

namespace PCGTimeSlice
{
	struct FEmptyStruct {};
}

/**
 * A PCG context with helper utility to enable element authors to more easily implement timeslicing.
 * @tparam PerExecutionStateT Struct type of the "per-execution" static data state
 * @tparam PerIterationStateT Struct type of the "per-iteration" data state
 * @tparam BaseContext        To derive from something else than FPCGContext (useful for extra interfaces or even a custom context)
 */
template <typename PerExecutionStateT = PCGTimeSlice::FEmptyStruct, typename PerIterationStateT = PCGTimeSlice::FEmptyStruct, typename BaseContext = FPCGContext>
struct TPCGTimeSlicedContext : public BaseContext
{
	virtual bool TimeSliceIsEnabled() const override final { return bTimeSliceIsEnabled; }
	void SetTimeSliceIsEnabled(const bool bEnableTimeSlice = true) { bTimeSliceIsEnabled = bEnableTimeSlice; }

	/** Retrieves the number of times this context was executed */
	uint32 GetExecutionCount() const { return ExecutionCount; }

	using InitExecSignature = EPCGTimeSliceInitResult(TPCGTimeSlicedContext* Context, PerExecutionStateT& OutState);

	/** Initializes per execution state data if required. */
	EPCGTimeSliceInitResult InitializePerExecutionState(TFunctionRef<InitExecSignature> InitFunc = [](TPCGTimeSlicedContext*, PerExecutionStateT&) -> EPCGTimeSliceInitResult { return EPCGTimeSliceInitResult::Success; });

	using InitIterSignature = EPCGTimeSliceInitResult(PerIterationStateT& OutState, const PerExecutionStateT& ExecState, const uint32 IterationIndex);

	/** Initializes per execution state data if required. An array will be created with a state element for every execution iteration in the context. Returns the Init Result for each iteration's initialization. */
	const TArray<EPCGTimeSliceInitResult>& InitializePerIterationStates(int32 NumIterations = 1, TFunctionRef<InitIterSignature> IterFunc = [](PerIterationStateT&, const uint32){ return TArray({EPCGTimeSliceInitResult::Success}); });

	/** Will return the result of the attempt to initialize the per execution state */
	EPCGTimeSliceInitResult GetExecutionStateResult() const { return ExecutionStateResult; }

	/** Will return the result of the attempt to initialize a specific iteration, by Index */
	EPCGTimeSliceInitResult GetIterationStateResult(const int32 Index) const
	{
		check(Index >= 0 && Index < PerIterationStateResultArray.Num());

		return PerIterationStateResultArray[Index];
	}

	const TArray<EPCGTimeSliceInitResult>& GetPerIterationStateInitResultArray() { return PerIterationStateResultArray; }

	PerExecutionStateT& GetPerExecutionState() { return PerExecutionStateData; }
	const PerExecutionStateT& GetPerExecutionState() const { return PerExecutionStateData; }

	TArray<PerIterationStateT>& GetPerIterationStateArray() { return PerIterationStateArray; }
	const TArray<PerIterationStateT>& GetPerIterationStateArray() const { return PerIterationStateArray; }

	/** Returns true if both the execution state and iteration state were fully initialized and no problems arose that should abort the execution. Can be used to bypass any further initialization. */
	bool DataIsPreparedForExecution() const
	{
		if (ExecutionStateResult == EPCGTimeSliceInitResult::Uninitialized ||
			ExecutionStateResult == EPCGTimeSliceInitResult::AbortExecution ||
			!bPerIterationStateIsInitialized)
		{
			return false;
		}

		for (const EPCGTimeSliceInitResult Result : PerIterationStateResultArray)
		{
			if (Result == EPCGTimeSliceInitResult::Uninitialized || Result == EPCGTimeSliceInitResult::AbortExecution)
			{
				return false;
			}
		}

		return true;
	}

	/** Fire and forget function to make sure a UObject will not be GC'ed for the duration of the Context. */
	void TrackObject(const UObject* Object);

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;

private:
	// Allow exposure to iteration index, etc
	friend class TPCGTimeSlicedElementBase<PerExecutionStateT, PerIterationStateT, BaseContext>;

	/** The number of times this context has been time sliced */
	uint32 ExecutionCount = 0u;
	/** The index of which iteration being processed. Ie. If a volume sampler has two volume inputs, it will be processed twice. */
	int32 IterationIndex = 0;

	/** True if time slicing is enabled for this context */
	bool bTimeSliceIsEnabled = true;

	/** The result of initializing the per execution state */
	EPCGTimeSliceInitResult ExecutionStateResult = EPCGTimeSliceInitResult::Uninitialized;

	/** True if an attempt has been made to initialize the per iteration states */
	bool bPerIterationStateIsInitialized = false;

	/** Tracks the results of each attempt to initialize an iteration state */
	TArray<EPCGTimeSliceInitResult> PerIterationStateResultArray;

	/** The state of the timesliced context that won't change each iteration. Ie. the node settings. */
	PerExecutionStateT PerExecutionStateData;

	/** An array of the various states of timesliced context that will change per iteration. Ie. generating shape */
	TArray<PerIterationStateT> PerIterationStateArray;

	/** Tracks rooted UObjects that need to avoid garbage collection during the lifetime of the context */
	TArray<TObjectPtr<const UObject>> TrackedObjectArray;
};

/**
 * A PCG Element that will utilize a Time Slice Context.
 * @tparam PerExecutionStateT Struct type of the "per-execution" static data state
 * @tparam PerIterationStateT Struct type of the "per-iteration" data state
 * @tparam BaseContext        To derive from something else than FPCGContext (useful for extra interfaces or even a custom context)
 */
template <typename PerExecutionStateT, typename PerIterationStateT, typename BaseContext>
class TPCGTimeSlicedElementBase : public IPCGElement
{
public:
	// Aliases, for ease of use with the template
	using ExecStateType = PerExecutionStateT;
	using IterStateType = PerIterationStateT;
	using ContextType = TPCGTimeSlicedContext<ExecStateType, IterStateType, BaseContext>;

	virtual FPCGContext* CreateContext() override { return new ContextType(); }

	using ExecSignature = bool(ContextType* Context, const PerExecutionStateT& PerExecutionState, PerIterationStateT& PerIterationState, const uint32 IterationIndex);

	/** Executes the delegate for every iteration. Will return false while still processing and true only when all tasks for all iterations are complete. */
	bool ExecuteSlice(ContextType* Context, TFunctionRef<ExecSignature> ExecFunc) const;

	// TODO: [FUTURE WORK] Consider callback 'on completion' and other Execution styles, like parallel iterations
};

template <typename PerExecutionStateT, typename PerIterationStateT, typename BaseContext>
void TPCGTimeSlicedContext<PerExecutionStateT, PerIterationStateT, BaseContext>::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(TrackedObjectArray);
}

template <typename PerExecutionStateT, typename PerIterationStateT, typename BaseContext>
EPCGTimeSliceInitResult TPCGTimeSlicedContext<PerExecutionStateT, PerIterationStateT, BaseContext>::InitializePerExecutionState(TFunctionRef<InitExecSignature> InitFunc)
{
	// Should only ever be initialized once, so if its called again, ignore it. This allows flexibility for the call to be somewhere that might be invoked numerous times
	if (ExecutionStateResult == EPCGTimeSliceInitResult::Uninitialized)
	{
		ExecutionStateResult = InitFunc(this, PerExecutionStateData);
	}

	return ExecutionStateResult;
}

template <typename PerExecutionStateT, typename PerIterationStateT, typename BaseContext>
const TArray<EPCGTimeSliceInitResult>& TPCGTimeSlicedContext<PerExecutionStateT, PerIterationStateT, BaseContext>::InitializePerIterationStates(int32 NumIterations, TFunctionRef<InitIterSignature> IterFunc)
{
	// Same as InitializePerExecutionState. Should only ever be initialized once, so if its called again, ignore it.
	if (bPerIterationStateIsInitialized)
	{
		return PerIterationStateResultArray;
	}

	bPerIterationStateIsInitialized = true;

	// Should be guaranteed to be uninitialized
	check(PerIterationStateResultArray.IsEmpty() && PerIterationStateArray.IsEmpty());

	// An empty iteration state is still valid
	if (NumIterations < 1)
	{
		return PerIterationStateResultArray;
	}

	PerIterationStateResultArray.AddUninitialized(NumIterations);
	PerIterationStateArray.Reserve(NumIterations);

	for (int32 I = 0; I < NumIterations; ++I)
	{
		PerIterationStateResultArray[I] = IterFunc(PerIterationStateArray.Emplace_GetRef(), PerExecutionStateData, I);

		if (PerIterationStateResultArray[I] == EPCGTimeSliceInitResult::AbortExecution)
		{
			// If it fails, remove the new state from the array and continue
			PerIterationStateArray.RemoveAt(PerIterationStateArray.Num() - 1);
		}
	}

	// Returns a complete array of the results
	return PerIterationStateResultArray;
}

template <typename PerExecutionStateT, typename PerIterationStateT, typename BaseContext>
void TPCGTimeSlicedContext<PerExecutionStateT, PerIterationStateT, BaseContext>::TrackObject(const UObject* Object)
{
	check(Object && IsValid(Object));
	TrackedObjectArray.AddUnique(Object);
}

template <typename PerExecutionStateT, typename PerIterationStateT, typename BaseContext>
bool TPCGTimeSlicedElementBase<PerExecutionStateT, PerIterationStateT, BaseContext>::ExecuteSlice(ContextType* Context, TFunctionRef<ExecSignature> ExecFunc) const
{
	++Context->ExecutionCount;

	// The user is responsible to check for this before execution, but just in case
	if (!ensureMsgf(Context->DataIsPreparedForExecution(), TEXT("State data was not properly initialized.")) || Context->PerIterationStateArray.IsEmpty())
	{
		return true;
	}

	do // The elements must execute at least once
	{
		if (ExecFunc(Context, Context->PerExecutionStateData, Context->PerIterationStateArray[Context->IterationIndex], Context->IterationIndex))
		{
			++Context->IterationIndex;
		}
		else
		{
			return false;
		}
	} while (Context->IterationIndex < Context->PerIterationStateArray.Num() && !Context->ShouldStop());

	return Context->IterationIndex == Context->PerIterationStateArray.Num();
}
