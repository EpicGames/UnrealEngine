// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGTransformPoints.generated.h"

class FPCGTransformPointsElement;
class UPCGTransformPointsKernel;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGTransformPointsSettings : public UPCGSettings
{
	friend UPCGTransformPointsKernel;
	friend FPCGTransformPointsElement;
	
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("TransformPoints")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGTransformPointsSettings", "NodeTitle", "Transform Points"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::PointOps; }
	virtual bool DisplayExecuteOnGPUSetting() const override { return true; }
	virtual void CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const override;
#endif
	virtual bool UseSeed() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
	
	EPCGPointNativeProperties GetPropertiesToAllocate() const;

public:
	/** If the operation should be applied to a Transform attribute instead of the point transform. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bApplyToAttribute = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition="bApplyToAttribute", EditConditionHides, PCG_Overridable))
	FName AttributeName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU))
	FVector OffsetMin = FVector::Zero();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU))
	FVector OffsetMax = FVector::Zero();

	/** Set offset in world space */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU))
	bool bAbsoluteOffset = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU))
	FRotator RotationMin = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU))
	FRotator RotationMax = FRotator::ZeroRotator;

	/** Set rotation directly instead of additively */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU))
	bool bAbsoluteRotation = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (AllowPreserveRatio, PCG_OverridableCPUAndGPU))
	FVector ScaleMin = FVector::One();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (AllowPreserveRatio, PCG_OverridableCPUAndGPU))
	FVector ScaleMax = FVector::One();

	/** Set scale directly instead of multiplicatively */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU))
	bool bAbsoluteScale = false;

	/** Scale uniformly on each axis. Uses the X component of ScaleMin and ScaleMax. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_OverridableCPUAndGPU))
	bool bUniformScale = true;

	/** Recompute the seed for each new point using its new location */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bApplyToAttribute", PCG_OverridableCPUAndGPU))
	bool bRecomputeSeed = false;
};

class FPCGTransformPointsElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
