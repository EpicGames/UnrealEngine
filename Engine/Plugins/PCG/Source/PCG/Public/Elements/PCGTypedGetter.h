// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGLandscapeData.h"
#include "Elements/PCGDataFromActor.h"

#include "PCGTypedGetter.generated.h"

#define UE_API PCG_API

/** Builds a collection of landscape data from the selected actors. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetLandscapeSettings : public UPCGDataFromActorSettings
{
	GENERATED_BODY()

public:
	UPCGGetLandscapeSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetLandscapeData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetLandscapeElement", "NodeTitle", "Get Landscape Data"); }
	virtual FText GetNodeTooltipText() const override;
#endif

	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings

	//~Begin UPCGDataFromActorSettings interface
public:
	virtual EPCGDataType GetDataFilter() const override { return EPCGDataType::Landscape; }
	virtual TSubclassOf<AActor> GetDefaultActorSelectorClass() const override;

protected:
#if WITH_EDITOR
	virtual bool DisplayModeSettings() const override { return false; }
#endif
	//~End UPCGDataFromActorSettings

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ShowOnlyInnerProperties))
	FPCGLandscapeDataProps SamplingProperties;

#if WITH_EDITORONLY_DATA
	// Editor only: If true, the intersected landscape bounds are going to be used to prepare the landscape cache, otherwise the PCG Component's grid bounds will be used.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUnbounded = true;

	UPROPERTY()
	bool bGetHeightOnly_DEPRECATED = false;

	UPROPERTY()
	bool bGetLayerWeights_DEPRECATED = true;
#endif // WITH_EDITORONLY_DATA
};

class FPCGGetLandscapeDataElement : public FPCGDataFromActorElement
{
protected:
	virtual void ProcessActors(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, const TArray<AActor*>& FoundActors, TArray<FPCGTaskId>& OutDynamicDependencies) const override;
	virtual void ProcessActors(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, const TArray<AActor*>& FoundActors) const override;
	virtual void ProcessActor(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, AActor* FoundActor) const override;
};

/** Builds a collection of spline data from the selected actors. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetSplineSettings : public UPCGDataFromActorSettings
{
	GENERATED_BODY()

public:
	UE_API UPCGGetSplineSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetSplineData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetSplineElement", "NodeTitle", "Get Spline Data"); }
	UE_API virtual FText GetNodeTooltipText() const override;
#endif

protected:
	UE_API virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings

	//~Begin UPCGDataFromActorSettings interface
public:
	virtual EPCGDataType GetDataFilter() const override { return EPCGDataType::PolyLine; }

protected:
#if WITH_EDITOR
	virtual bool DisplayModeSettings() const override { return false; }
#endif
	//~End UPCGDataFromActorSettings
};

/** Builds a collection of volume data from the selected actors. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetVolumeSettings : public UPCGDataFromActorSettings
{
	GENERATED_BODY()

public:
	UPCGGetVolumeSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetVolumeData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetVolumeElement", "NodeTitle", "Get Volume Data"); }
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings

	//~Begin UPCGDataFromActorSettings interface
public:
	virtual EPCGDataType GetDataFilter() const override { return EPCGDataType::Volume; }

protected:
#if WITH_EDITOR
	virtual bool DisplayModeSettings() const override { return false; }
#endif
	//~End UPCGDataFromActorSettings
};

/** Builds a collection of primitive data from primitive components on the selected actors. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetPrimitiveSettings : public UPCGDataFromActorSettings
{
	GENERATED_BODY()

public:
	UPCGGetPrimitiveSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetPrimitiveData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetPrimitiveElement", "NodeTitle", "Get Primitive Data"); }
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings

	//~Begin UPCGDataFromActorSettings interface
public:
	virtual EPCGDataType GetDataFilter() const override { return EPCGDataType::Primitive; }

protected:
#if WITH_EDITOR
	virtual bool DisplayModeSettings() const override { return false; }
#endif
	//~End UPCGDataFromActorSettings
};

/**
 * Builds a collection of data from other PCG components on the selected actors. Automatically tags each output with the grid size it was collected
 * from, prefixed by "PCG_GridSize_" (e.g. PCG_GridSize_12800).
 *
 * Note: a component cannot get component data from itself or other components in its execution context, as it could create a circular dependency.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetPCGComponentSettings : public UPCGDataFromActorSettings
{
	GENERATED_BODY()

public:
	UPCGGetPCGComponentSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetPCGComponentData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetPCGComponentElement", "NodeTitle", "Get PCG Component Data"); }
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings

	//~Begin UPCGDataFromActorSettings interface
public:
	virtual EPCGDataType GetDataFilter() const override { return EPCGDataType::Any; }

protected:
#if WITH_EDITOR
	virtual bool DisplayModeSettings() const override { return false; }
#endif
	//~End UPCGDataFromActorSettings
};

/** Builds a collection of virtual texture data from the selected actors. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetVirtualTextureSettings : public UPCGDataFromActorSettings
{
	GENERATED_BODY()

public:
	UPCGGetVirtualTextureSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetVirtualTextureData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetVirtualTextureElement", "NodeTitle", "Get Virtual Texture Data"); }
	virtual FText GetNodeTooltipText() const override;
#endif

	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings

	//~Begin UPCGDataFromActorSettings interface
public:
	virtual EPCGDataType GetDataFilter() const override { return EPCGDataType::VirtualTexture; }
	virtual TSubclassOf<AActor> GetDefaultActorSelectorClass() const override;

protected:
#if WITH_EDITOR
	virtual bool DisplayModeSettings() const override { return false; }
#endif
	//~End UPCGDataFromActorSettings
};

#undef UE_API
