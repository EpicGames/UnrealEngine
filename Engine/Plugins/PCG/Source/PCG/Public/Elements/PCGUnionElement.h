// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettingsWithDynamicInputs.h"
#include "Data/PCGUnionData.h"

#include "PCGUnionElement.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGUnionSettings : public UPCGSettingsWithDynamicInputs
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Union")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGUnionSettings", "NodeTitle", "Union"); }
	FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGUnionSettings", "NodeTooltip", "Combine spatial data into a union of all inputs. Order of inputs is respected, beginning with the dynamic pin inputs."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin UPCGSettingsWithDynamicInputs interface
	FName GetDynamicInputPinsBaseLabel() const override;
	TArray<FPCGPinProperties> StaticInputPinProperties() const override;

public:
#if WITH_EDITOR
	void AddDefaultDynamicInputPin() override;
#endif // WITH_EDITOR
	//~End UPCGSettingsWithDynamicInputs interface

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGUnionType Type = EPCGUnionType::LeftToRightPriority;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGUnionDensityFunction DensityFunction = EPCGUnionDensityFunction::Maximum;
};

class FPCGUnionElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};
