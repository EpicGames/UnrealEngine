// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "PCGElement.h"
#include "PCGSettings.h"
#include "Helpers/PCGPropertyHelpers.h"

#include "PCGUserParameterGet.generated.h"

UENUM()
enum class EPCGUserParameterSource : uint8
{
	Current,
	Upstream,
	Root
};

/**
* Getter for user parameters defined in PCGGraph, by the user.
* Will pick up the value from the graph instance.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGUserParameterGetSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FGuid PropertyGuid;

	UPROPERTY()
	FName PropertyName;

	/** If the output attribute name has special characters, remove them. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bSanitizeOutputAttributeName = true;
	
	/** How we should extract the structs. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGStructExtractorBehavior StructExtractorBehavior = EPCGStructExtractorBehavior::ExtractRootOnly;
		
	/** How we should extract the objects. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGObjectExtractorBehavior ObjectExtractorBehavior = EPCGObjectExtractorBehavior::NoExtract;
		
	/** How we should extract the containers. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGContainerExtractorBehavior ContainerExtractorBehavior = EPCGContainerExtractorBehavior::FlattenLastAndDiscardNested;
	
	/** If the property is a struct/object supported by metadata, this option can be toggled to force extracting all (compatible) properties contained in this property. Automatically true if unsupported by metadata. For now, only supports direct child properties (and not deeper). */
	UE_DEPRECATED(5.8, "Use the different extractor behavior options")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the different extractor behavior options"))
	bool bForceObjectAndStructExtraction = false;
	
	void UpdatePropertyName(FName InNewName);

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
	virtual bool RequiresDataFromPreTask() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return {}; }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	virtual bool HasExecutionDependencyPin() const override { return false; }

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName("GetGraphParameter"); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGUserParameterGetSettings", "NodeTitle", "Get Graph Parameter"); }
	virtual bool ShouldDrawNodeCompact() const override { return true; }
	virtual bool CanUserEditTitle() const override { return true; }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::GraphParameters; }
	virtual TArray<FPCGPreconfiguredInfo> GetConversionInfo() const override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
	virtual bool GroupPreconfiguredSettings() const override { return false; }
#endif
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;

	virtual bool ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo) override;
	//~End UPCGSettings interface
};

/**
* Generic getter for user parameter defined in the PCG Graph, by the user.
* Will pick up the value from the graph instance.
* This getter allows to set manually the user parameter they want to get, and add extractor, the same way than GetActorProperty or GetPropertyFromObjectPath
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGenericUserParameterGetSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FString PropertyPath;

	/** If the output attribute name has special characters, remove them. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bSanitizeOutputAttributeName = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FName OutputAttributeName = NAME_None;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGUserParameterSource Source = EPCGUserParameterSource::Current;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bQuiet = false;
	
	/** How we should extract the structs. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGStructExtractorBehavior StructExtractorBehavior = EPCGStructExtractorBehavior::ExtractRootOnly;
		
	/** How we should extract the objects. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGObjectExtractorBehavior ObjectExtractorBehavior = EPCGObjectExtractorBehavior::NoExtract;
		
	/** How we should extract the containers. (cf. enum tooltips) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGContainerExtractorBehavior ContainerExtractorBehavior = EPCGContainerExtractorBehavior::FlattenLastAndDiscardNested;
	
	/** If the property is a struct/object supported by metadata, this option can be toggled to force extracting all (compatible) properties contained in this property. Automatically true if unsupported by metadata. For now, only supports direct child properties (and not deeper). */
	UE_DEPRECATED(5.8, "Use the different extractor behavior options")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the different extractor behavior options"))
	bool bForceObjectAndStructExtraction = false;

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
	virtual bool RequiresDataFromPreTask() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override;

#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif // WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return {}; }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName("GetGenericGraphParameter"); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("UPCGGenericUserParameterGetSettings", "NodeTitle", "Get Graph Parameter"); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::GraphParameters; }
	virtual TArray<FPCGPreconfiguredInfo> GetConversionInfo() const override;
#endif

	virtual bool ConvertNode(const FPCGPreconfiguredInfo& ConversionInfo) override;
	//~End UPCGSettings interface
};

class FPCGUserParameterGetElement : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
