// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGElement.h"
#include "PCGSettings.h"

#include "DebugRenderSceneProxy.h"

#include "PCGVisualizeAttribute.generated.h"

/**
 * Visualizes a selected attribute on screen at each point's transform.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGVisualizeAttributeSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("VisualizeAttribute")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGVisualizeAttributeElement", "NodeTitle", "Visualize Attribute"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGVisualizeAttributeElement", "NodeTooltip", "Visualizes a selected attribute on screen at each point's transform."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Debug; }
#endif

protected:
	virtual FPCGElementPtr CreateElement() const override;
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return Super::DefaultPointInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	//~End UPCGSettings interface

public:
	/** This attribute will be have it's value printed in proximity to each input point's transform. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector AttributeSource;

	/** A custom added prefix to which the attribute value will be appended. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FString CustomPrefixString;

	/** Prefix the printed value with the point's index. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bPrefixWithIndex = true;

	/** Prefix the printed value with the attribute's name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bPrefixWithAttributeName = false;

	/** A local offset from the point's location to draw the text. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FVector LocalOffset = FVector(0, 0, 0);

	/** The color of the on displayed value. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FColor Color = FColor::Cyan;

	/** The duration (in seconds) of the displayed value. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = "0.0"))
	double Duration = 30.0;

	/** The limit of points to draw debug messages. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ClampMin = "1", ClampMax="4096"))
	int32 PointLimit = 4096;

	/** The visualizer is enabled. Useful for dynamically overriding. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bVisualizeEnabled = true;
};

class FPCGVisualizeAttribute : public IPCGElement
{
public:
	// Creating components (like the debug component for this node) is not thread safe.
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }

protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};