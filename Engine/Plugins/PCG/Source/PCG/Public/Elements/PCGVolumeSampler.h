// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "PCGTimeSlicedElementBase.h"

#include "Math/Box.h"

#include "PCGVolumeSampler.generated.h"

class UPCGPointData;
class UPCGSpatialData;

namespace PCGVolumeSamplerConstants
{
	const FName VolumeLabel = TEXT("Volume");
	const FName BoundingShapeLabel = TEXT("Bounding Shape");
}

namespace PCGVolumeSampler
{
	const FVector DefaultVoxelSize = FVector(100.0, 100.0, 100.0);

	struct FVolumeSamplerParams
	{
		FVector VoxelSize = DefaultVoxelSize;
		float PointSteepness = 0.0f;
		FBox Bounds{EForceInit::ForceInit};
	};

	struct FVolumeSamplerExecutionState
	{
		const UPCGSpatialData* BoundingShape = nullptr;
		FBox BoundingShapeBounds = FBox(EForceInit::ForceInit);
		TArray<const UPCGSpatialData*> GeneratingShapes;
	};

	struct FVolumeSamplerIterationState
	{
		FVolumeSamplerParams Settings;

		const UPCGSpatialData* Volume = nullptr;

		UPCGBasePointData* OutputPointData = nullptr;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.6, "Use OutputPointData instead")
		UPCGPointData* OutputData = nullptr;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	/** Sample a volume and returns the resulting point data. */
	PCG_API UPCGBasePointData* SampleVolume(FPCGContext* Context, TSubclassOf<UPCGBasePointData> PointDataClass, const FVolumeSamplerParams& SamplerSettings, const UPCGSpatialData* Volume, const UPCGSpatialData* BoundingShape = nullptr);

	UE_DEPRECATED(5.6, "Use SampleVolume returning UPCGBasePointData")
	PCG_API UPCGPointData* SampleVolume(FPCGContext* Context, const FVolumeSamplerParams& SamplerSettings, const UPCGSpatialData* Volume, const UPCGSpatialData* BoundingShape = nullptr);

	PCG_API UPCGBasePointData* SampleVolume(FPCGContext* Context, const UPCGSpatialData* Volume, const FVolumeSamplerParams& SamplerSettings, const UPCGSpatialData* BoundingShape = nullptr);

	/** Sample a volume and write the results in the given point data. Can be timesliced and will return false if the processing is not done, true otherwise. */
	PCG_API bool SampleVolume(FPCGContext* Context, const FVolumeSamplerParams& SamplerSettings, const UPCGSpatialData* Volume, const UPCGSpatialData* BoundingShape, UPCGBasePointData* OutputData, const bool bTimeSlicingIsEnabled = false);
}

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGVolumeSamplerSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGVolumeSamplerSettings();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data", meta = (PCG_Overridable))
	FVector VoxelSize = PCGVolumeSampler::DefaultVoxelSize;

	/** If no Bounding Shape input is provided, the actor bounds are used to limit the sample generation domain.
	* This option allows ignoring the actor bounds and generating over the entire volume. Use with caution as this
	* may generate a lot of points.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUnbounded = false;

	/** Each PCG point represents a discretized, volumetric region of world space. The points' Steepness value [0.0 to
	 * 1.0] establishes how "hard" or "soft" that volume will be represented. From 0, it will ramp up linearly
	 * increasing its influence over the density from the point's center up to two times the bounds. At 1, it will
	 * represent a binary box function with the size of the point's bounds.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Points", meta=(ClampMin="0", ClampMax="1", PCG_Overridable))
	float PointSteepness = 0.5f;
	
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("VolumeSampler")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGVolumeSamplerSettings", "NodeTitle", "Volume Sampler"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Sampler; }
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return Super::DefaultPointOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGVolumeSamplerElement : public TPCGTimeSlicedElementBase<PCGVolumeSampler::FVolumeSamplerExecutionState, PCGVolumeSampler::FVolumeSamplerIterationState>
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;
	// Might be sampling external data like brush, worth computing a full CRC in case we can halt change propagation/re-executions
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override { return true; }

protected:
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};
