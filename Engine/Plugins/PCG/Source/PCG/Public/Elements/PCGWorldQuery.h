// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Data/PCGCollisionShape.h"
#include "Data/PCGWorldData.h"

#include "PCGWorldQuery.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGWorldQuerySettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("WorldVolumetricQuery")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGWorldQuerySettings", "NodeTitle", "World Volumetric Query"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;

#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
#endif // WITH_EDITOR
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, ShowOnlyInnerProperties))
	FPCGWorldVolumetricQueryParams QueryParams;
};

class FPCGWorldVolumetricQueryElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGWorldRayHitSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("WorldRayHitQuery")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGWorldRayHitSettings", "NodeTitle", "World Ray Hit Query"); }
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;

#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
#endif // WITH_EDITOR
	//~End UPCGSettings interface

public:
	/** Parameters for either using a line trace or specifying a collision shape for a sweep. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGCollisionShape CollisionShape;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Data, meta = (PCG_Overridable, ShowOnlyInnerProperties))
	FPCGWorldRayHitQueryParams QueryParams;
};

class FPCGWorldRayHitQueryElement : public IPCGElement
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};
