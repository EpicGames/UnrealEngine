// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PCGSettings.h"
#include "PCGTimeSlicedElementBase.h"
#include "Data/PCGCollisionShape.h"
#include "Data/PCGWorldData.h"
#include "Helpers/PCGWorldQueryHelpers.h"

#include "CollisionQueryParams.h"

#include "PCGWorldRaycast.generated.h"

namespace PCGWorldRaycastElement::Constants::Labels
{
	extern const FName EndPointsInput;
}

/**
 * Casts rays from provided points along a given direction and transform points to the impact point.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGWorldRaycastElementSettings : public UPCGSettings
{
	GENERATED_BODY()

	UPCGWorldRaycastElementSettings();

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("WorldRaycast")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGWorldRaycastElement", "NodeTooltip", "Casts a line trace or collision shape sweep from provided points along a given direction returning the location of the impact."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual TArray<FText> GetNodeTitleAliases() const override;
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return false; }
	virtual bool GroupPreconfiguredSettings() const override { return true; }
	virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
	virtual bool DisplayExecuteOnGPUSetting() const override { return true; }
	virtual void CreateKernels(FPCGGPUCompilationContext& InOutContext, UObject* InObjectOuter, const UPCGNode* InNode, TArray<UPCGComputeKernel*>& OutKernels, TArray<FPCGKernelEdge>& OutEdges) const override;
#endif // WITH_EDITOR

	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;
	virtual bool IsPinUsedByNodeExecution(const UPCGPin* InPin) const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual bool IsGPUFriendly(const FPCGPreConfiguredSettingsInfo* PreconfiguredInfo = nullptr) const override;
#endif
	//~End UPCGSettings interface

public:
	/** Parameters for either using a line trace or specifying a collision shape. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGCollisionShape CollisionShape;

	/** Determines how the ray's direction and distance will be calculated. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGWorldRaycastMode RaycastMode = EPCGWorldRaycastMode::Infinite;

	/** The selected attribute determines the ray origin. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector OriginInputAttribute;

	/** Use a selected attribute as the ray direction. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "RaycastMode != EPCGWorldRaycastMode::Segments", EditConditionHides))
	bool bOverrideRayDirections = false;

	/** A ray direction that will be used for all raycasts. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "RaycastMode != EPCGWorldRaycastMode::Segments && !bOverrideRayDirections", EditConditionHides))
	FVector RayDirection = -FVector::UnitZ();

	/** The selected attribute determines the ray direction. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "RaycastMode != EPCGWorldRaycastMode::Segments && bOverrideRayDirections", EditConditionHides))
	FPCGAttributePropertyInputSelector RayDirectionAttribute;

	/** The selected attribute determines the ray terminal point. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "RaycastMode == EPCGWorldRaycastMode::Segments", EditConditionHides))
	FPCGAttributePropertyInputSelector EndPointAttribute;

	/** Use a selected attribute as the ray length. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "RaycastMode == EPCGWorldRaycastMode::NormalizedWithLength", EditConditionHides))
	bool bOverrideRayLengths = false;

	/** A ray length that will be used for all raycasts. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "RaycastMode == EPCGWorldRaycastMode::NormalizedWithLength && !bOverrideRayLengths", EditConditionHides))
	double RayLength = 100000.0;

	/** The selected attribute determines the ray length. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "RaycastMode == EPCGWorldRaycastMode::NormalizedWithLength && bOverrideRayLengths", EditConditionHides))
	FPCGAttributePropertyInputSelector RayLengthAttribute;

	/** World ray trace parameters. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, ShowOnlyInnerProperties, EditCondition = "!bExecuteOnGPU", EditConditionHides))
	FPCGWorldRaycastQueryParams WorldQueryParams;

	/** Will keep the original points at their location if the raycast misses or if the hit result is out of bounds. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bKeepOriginalPointOnMiss = false;

	/** If no Bounding Shape input is provided, the actor bounds are used to limit the sample generation domain. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bUnbounded = false;

	/** [EXPERIMENTAL] Create a Vector2 attribute ('ImpactUVCoords') for UV Coordinates of the surface hit. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bExecuteOnGPU", EditConditionHides))
	bool bGetTextureCoordinates = false;

	/** [EXPERIMENTAL] Which channel to sample UV Coordinates from. If this channel does not exist, UV coordinates will be (0, 0). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bExecuteOnGPU && bGetTextureCoordinates", EditConditionHides))
	int TextureCoordinatesChannelIndex = 0;

	/** [EXPERIMENTAL] Create a Vector attribute ('ImpactNormal') for normals of the surface hit. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bExecuteOnGPU", EditConditionHides))
	bool bGetNormals = false;
};

namespace FPCGWorldRaycast
{
	struct FExecutionState
	{
		FCollisionQueryParams CollisionQueryParams;
		FCollisionObjectQueryParams CollisionObjectQueryParams;
		FBox Bounds = FBox(EForceInit::ForceInit);
	};

	struct FIterationState
	{
		// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FIterationState() = default;
		FIterationState(const FIterationState&) = default;
		FIterationState(FIterationState&&) = default;
		FIterationState& operator=(const FIterationState&) = default;
		FIterationState& operator=(FIterationState&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		TArray<FVector> CachedRayOrigins;
		TArray<FVector> CachedRayVectors;
		TArray<double> CachedRayLengths;
		TSet<FObjectKey> CachedFilterObjects;

		UE_DEPRECATED(5.8, "Use CachedFilterObjects instead")
		TSet<TObjectKey<AActor>> CachedFilterActors;
	};
}

class FPCGWorldRaycastElement : public TPCGTimeSlicedElementBase<FPCGWorldRaycast::FExecutionState, FPCGWorldRaycast::FIterationState>
{
public:
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	// Physics required to run on main thread
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
};
