// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGWriteDataIndex.generated.h"

class FPCGWriteDataIndexElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	// @todo_pcg: support gpu resident data
};

/**
* Writes the data's index in the input collection to a tag or to an attribute
*/
UCLASS(BlueprintType, ClassGroup=(Procedural))
class UPCGWriteDataIndexSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGWriteDataIndexSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("WriteDataIndex")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGWriteDataIndexElement", "NodeTitle", "Write Data Index"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGWriteDataIndexElement", "NodeTooltip", "Writes the data's index in the input collection to a tag or an attribute."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Generic; }
#endif // WITH_EDITOR
	virtual bool HasDynamicPins() const override { return true; }

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGWriteDataIndexElement>(); }
	//~End UPCGSettings interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (InlineEditConditionToggle, PCG_Overridable))
	bool bAddTag = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bAddTag", PCG_Overridable))
	FString IndexTag = "Index";

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (InlineEditConditionToggle, PCG_Overridable))
	bool bAddAttribute = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bAddAttribute", PCG_Overridable, PCG_DiscardPropertySelection, PCG_DiscardExtraSelection))
	FPCGAttributePropertyOutputSelector IndexAttribute;
};