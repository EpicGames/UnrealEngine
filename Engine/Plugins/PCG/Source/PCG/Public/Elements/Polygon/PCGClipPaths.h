// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGClipPaths.generated.h"

UENUM()
enum class EPCGClipPathOperation : uint8
{
	Intersection = 0,
	Difference
};

class FPCGClipPathsElement : public IPCGElement
{
public:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Setings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};

/**
* Clips paths (points or splines) using provided polygons
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGClipPathsSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ClipPaths")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGClipPathsElement>(); }
	//~End UPCGSettings interface

public:
	/** Controls whether the paths will be clipped to be inside or outside the clip polygons. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	EPCGClipPathOperation Operation = EPCGClipPathOperation::Intersection;

	/** Maximum squared distance before we need to subdivide a segment again as part of the spline discretization to a polygon. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (ClampMin = 0.001, PCG_Overridable))
	double SplineMaxDiscretizationError = 1.0;
};