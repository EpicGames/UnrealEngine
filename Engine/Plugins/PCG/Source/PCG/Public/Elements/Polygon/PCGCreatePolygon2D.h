// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/Polygon/PCGPolygon2DUtils.h"

#include "PCGCreatePolygon2D.generated.h"

UENUM()
enum class EPCGCreatePolygonInputType : uint8
{
	Automatic UMETA(Tooltip="Assumes that point data represent a closed polygon, but for splines will respect spline type."),
	ForceOpen UMETA(Tooltip="Always considers inputs as open polygons."),
	ForceClosed UMETA(Tooltip="Always considers inputs as closed polygons.")
};

UENUM()
enum class EPCGCreatePolygonDefaultNormalBehavior : uint8
{
	OnlyForDegenerateCases UMETA(Tooltip="Uses the default normal only for polygons that are degenerate (collinear or too small)."),
	OnlyForOpenPolygons UMETA(Tooltip="Uses the default normal for all open polygons (including forced open polygons) and degenerate polygons."),
	Always UMETA(Tooltip="Uses the default normal for all polygons.")
};

class FPCGCreatePolygonElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};

/**
* Creates polygon(s) from the specified point data.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup=(Procedural))
class UPCGCreatePolygon2DSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGCreatePolygon2DSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CreatePolygon2D")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCreatePolygon2DElement", "NodeTitle", "Create Polygon 2D"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGCreatePolygon2DElement", "NodeTooltip", "Creates a (closed) 2d polygon per input."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return PCGPolygon2DUtils::DefaultPolygonOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGCreatePolygonElement>(); }
	//~End UPCGSettings interface

public:
	/** Controls the way input data will be considered - either as closed or open paths. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	EPCGCreatePolygonInputType InputType = EPCGCreatePolygonInputType::Automatic;

	/** Controls whether we will use the hole attribute to separate outer from hole segments. Note that the attribute will be used only when the input is a point data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bUseHoleAttribute = false;

	/** Attribute that describes the hole index of the points, where -1 is the outer polygon, and other values are for holes. Expects values on inputs to be sequential and increasing only. Note that this will be used only on point data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable, EditCondition = "bUseHoleAttribute"))
	FPCGAttributePropertyInputSelector HoleIndexAttribute;

	/** Controls whether target polygon width (when open/forced open) will be driven by an attribute. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bUsePolygonWidthAttribute = false;

	/** Attribute from which to get the width value */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable, EditCondition = "bUsePolygonWidthAttribute"))
	FPCGAttributePropertyInputSelector PolygonWidthAttribute;

	/** Target width of the polygon generated from an open path. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "!bUsePolygonWidthAttribute && InputType != EPCGCreatePolygonInputType::ForceClosed", PCG_Overridable))
	double OpenPolygonWidth = 100.0;

	/** Maximum squared distance before we need to subdivide a segment again as part of the spline discretization to a polygon. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (ClampMin = 0.001, PCG_Overridable))
	double SplineMaxDiscretizationError = 1.0;

	/** Default axis to use when the input has an ambiguous shape (lines, collinear polygons, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	FVector DefaultPolygonNormal = FVector::ZAxisVector;

	/** Controls whether we will use the default polygon normal : always/only for open/only for degenerate data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	EPCGCreatePolygonDefaultNormalBehavior DefaultPolygonNormalBehavior = EPCGCreatePolygonDefaultNormalBehavior::OnlyForDegenerateCases;

	/** Controls behavior of how joining points are resolved. See GeometryScript for more details. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (EditCondition = "InputType != EPCGCreatePolygonInputType::ForceClosed", PCG_Overridable))
	EPCGPolygonJoinType JoinType = EPCGPolygonJoinType::Square;

	/** Controls behavior of how end points are resolved. See GeometryScript for more details. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (EditCondition = "InputType != EPCGCreatePolygonInputType::ForceClosed", PCG_Overridable))
	EPCGPolygonEndType EndType = EPCGPolygonEndType::Square;
};