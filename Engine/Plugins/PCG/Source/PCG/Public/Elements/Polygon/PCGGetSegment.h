// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGGetSegment.generated.h"

class FPCGGetSegmentElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::PrimaryPinAndBroadcastablePins; }
};

/** Gets segments from point data, spline or polygon2d. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetSegmentSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetSegment")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetSegmentElement", "NodeTitle", "Get Segment"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGGetSegmentElement", "NodeTooltip", "Gets a specific segment from the input."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGGetSegmentElement>(); }
	//~End UPCGSettings interface

public:
	/** Controls whether the segment index will be provided from a matching data source or use a constant. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bUseInputSegmentData = false;

	/** Controls whether the hole index will be provided from a matching data source or use a constant. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bUseInputHoleData = false;

	/** Controls whether the output is a spline or points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bOutputSplineData = false;

	/** Specifies the attribute from which to extract the segment index value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bUseInputSegmentData", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector SegmentIndexAttribute;

	/** Specifies the segment index to extract from the input data. Supports negative indices (-1 being the last, etc.). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "!bUseInputSegmentData", EditConditionHides, PCG_Overridable))
	int32 SegmentIndex = 0;

	/** Specifies the attribute from which to extract the hole index value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bUseInputHoleData", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector HoleIndexAttribute;

	/** Specifies the hole index to use for polygons in the input data. Note that -1 denotes the outer polygon. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "!bUseInputHoleData", EditConditionHides, PCG_Overridable))
	int32 HoleIndex = -1;
};