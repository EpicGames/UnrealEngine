// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/Polygon/PCGPolygon2DUtils.h"

#include "PCGOffsetPolygon2D.generated.h"

UENUM()
enum class EPCGPolygonOffsetOperation
{
	Offset UMETA(Tooltip = "Expands (positive) or contracts (negative) the polygon"),
	Open UMETA(Tooltip = "Morphological open on the polygon based on the offset distance (contracts then expands)."),
	Close UMETA(Tooltip = "Morphological close on the polygon based on the offset distance (expands then contracts).")
};

class FPCGOffsetPolygonElement : public IPCGElement
{
public:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::PrimaryPinAndBroadcastablePins; }
};

/**
* Offsets polygons, depending on the offset quantity and join type.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGOffsetPolygon2DSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("OffsetPolygon2D")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGOffsetPolygon2D", "NodeTitle", "Offset Polygon"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGOffsetPolygon2D", "NodeTooltip", "Offsets polygon to either make it larger or smaller, or open/close holes based on the offset quantity."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	// @todo_pcg : presets for offset/open/close
	virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return PCGPolygon2DUtils::DefaultPolygonOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGOffsetPolygonElement>(); }
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGPolygonOffsetOperation Operation = EPCGPolygonOffsetOperation::Offset;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bInheritMetadata = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bUseOffsetFromInput = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bUseOffsetFromInput", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector OffsetAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "!bUseOffsetFromInput", EditConditionHides, PCG_Overridable))
	double Offset = 100.0;

	/** Controls behavior of how joining points are resolved. See GeometryScript for more details. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	EPCGPolygonJoinType JoinType = EPCGPolygonJoinType::Square;
};