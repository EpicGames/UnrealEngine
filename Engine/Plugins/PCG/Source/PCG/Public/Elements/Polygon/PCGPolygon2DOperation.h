// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/Polygon/PCGPolygon2DUtils.h"

#include "PCGPolygon2DOperation.generated.h"

UENUM()
enum class EPCGPolygonOperation : uint8
{
	Union UMETA(Tooltip="Computes the union of all the input polygons"),
	Difference UMETA(Tooltip="Computes the difference between each input polygon and the union of the clip polygons."),
	Intersection UMETA(Tooltip="Computes the intersection of each input polygon with the union of the clip polygons."),
	PairwiseIntersection UMETA(Tooltip="Computes the intersection of each input polygon and each clip polygon."),
	InnerIntersection UMETA(Tooltip="Computes the intersection between all the input polygons."),
	ExclusiveOr UMETA(Tooltip="Computes the exclusive OR of each input polygon and the union of the clip polygons."),
	CutWithPaths UMETA(Tooltip="Cuts polygons with paths by completing them using the polygon bounds. If both ends of the path are not outside the input polygons, the results might be incorrect.")
};

UENUM()
enum class EPCGPolygonOperationMetadataMode : uint8
{
	None UMETA(Tooltip="Output polygons will have their default attribute values only."),
	SourceOnly UMETA(Tooltip="Output polygons will have their attribute values based on the input polygons only."),
	Full UMETA(Tooltip="Output polygons will have their attribute values be based on input polygons first, then on clip polygons.")
};

class FPCGPolygon2DOperationElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const { return true; }
};

/**
* Applies polygon operations between polygons
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPolygon2DOperationSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Polygon2DOperation")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual TArray<FPCGPreConfiguredSettingsInfo> GetPreconfiguredInfo() const override;
	virtual bool OnlyExposePreconfiguredSettings() const override { return true; }
	virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins) override;
#endif
	virtual FString GetAdditionalTitleInformation() const override;
	virtual void ApplyPreconfiguredSettings(const FPCGPreConfiguredSettingsInfo& PreconfigureInfo) override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return PCGPolygon2DUtils::DefaultPolygonOutputPinProperties(); }
	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGPolygon2DOperationElement>(); }
	//~End UPCGSettings interface

public:
	/** Controls operation to be performed on the input polygons (vs. the clip polygons). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGPolygonOperation Operation = EPCGPolygonOperation::Intersection;

	/** Controls whether metadata on the resulting output will be either not computed & defaulted ('None'), based on the input polygons only ('SourceOnly') or using both the source and the clip polygons. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	EPCGPolygonOperationMetadataMode MetadataMode = EPCGPolygonOperationMetadataMode::Full;

	/** Maximum squared distance before we need to subdivide a segment again as part of the spline discretization to a path. **/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (ClampMin = 0.001, PCG_Overridable, EditCondition = "Operation==EPCGPolygonOperation::CutWithPaths", EditConditionHides))
	double SplineMaxDiscretizationError = 1.0;

	/** Controls whether the operation can log warnings/errors if it fails. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bQuiet = false;
};