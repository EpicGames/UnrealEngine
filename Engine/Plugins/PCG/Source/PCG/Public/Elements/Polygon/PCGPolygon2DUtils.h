// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "Curve/PolygonOffsetUtils.h"

#include "PCGPolygon2DUtils.generated.h"

UENUM(BlueprintType)
enum class EPCGPolygonJoinType : uint8
{
	/* Uniform squaring on all convex edge joins. */
	Square,
	/* Arcs on all convex edge joins. */
	Round,
	/* Squaring of convex edge joins with acute angles ("spikes"). Use in combination with MiterLimit. */
	Miter,
};

UENUM(BlueprintType)
enum class EPCGPolygonEndType : uint8
{
	/* Offsets only one side of a closed path */
	Polygon UMETA(Hidden),
	/* Offsets both sides of a path, with square blunt ends */
	Butt,
	/* Offsets both sides of a path, with square extended ends */
	Square,
	/* Offsets both sides of a path, with round extended ends */
	Round,
};

namespace PCGPolygon2DUtils
{
	UE::Geometry::EPolygonOffsetJoinType GetJoinType(EPCGPolygonJoinType JoinType);
	UE::Geometry::EPolygonOffsetEndType GetEndType(EPCGPolygonEndType EndType);

	namespace Constants
	{
		const FLazyName ClipPolysLabel = TEXT("ClipPolygons");
		const FLazyName ClipPathsLabel = TEXT("ClipPaths");

		// Since these were introduced in 5.7, they will be removed as of 5.8.
		namespace Deprecated
		{
			const FLazyName OldClipPolysLabel = TEXT("Clip Polygons");
			const FLazyName OldClipPathsLabel = TEXT("Clip Paths");
		}
	}

	TArray<FPCGPinProperties> DefaultPolygonInputPinProperties();
	TArray<FPCGPinProperties> DefaultPolygonOutputPinProperties();
}