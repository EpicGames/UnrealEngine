// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/Polygon/PCGPolygon2DUtils.h"

#include "PCGSurfaceFromPolygon2D.generated.h"

class FPCGCreateSurfaceFromPolygon2DElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::SinglePrimaryPin; }
};

/** 
* Creates a surface representation from a polygon data. 
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCreateSurfaceFromPolygon2DSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("CreateSurfaceFromPolygon2D")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGCreateSurfaceFromPolygon2DElement", "NodeTitle", "Create Surface From Polygon2D"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGCreateSurfaceFromPolygon2DElement", "NodeTooltip", "Creates a surface data from a Polygon 2D data."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual bool ShouldDrawNodeCompact() const override { return false; }
#endif
	virtual bool HasExecutionDependencyPin() const override { return false; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return PCGPolygon2DUtils::DefaultPolygonInputPinProperties(); }
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGCreateSurfaceFromPolygon2DElement>(); }
	//~End UPCGSettings interface
};