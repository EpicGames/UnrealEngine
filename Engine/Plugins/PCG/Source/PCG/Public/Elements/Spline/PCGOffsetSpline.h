// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGSettings.h"

#include "PCGOffsetSpline.generated.h"

class FPCGOffsetSplineElement : public IPCGElement
{
public:
	virtual EPCGElementExecutionLoopMode ExecutionLoopMode(const UPCGSettings* Settings) const override { return EPCGElementExecutionLoopMode::PrimaryPinAndBroadcastablePins; }
	virtual bool ExecuteInternal(FPCGContext* Context) const;
};

/**
* Offsets a spline based on provided direction & magnitude, and corrects tangents lengths after the fact.
* Note that this node is not currently robust to extreme offsets that would make the spline self-intersect.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGOffsetSplineSettings : public UPCGSettings
{
	GENERATED_BODY()

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("OffsetSpline")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGOffsetSplineElement", "NodeTitle", "Offset Spline"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGOffsetSplineElement", "NodeTooltip", "Offsets spline control points according to a direction and a magnitude."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif // WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override { return MakeShared<FPCGOffsetSplineElement>(); }
	// ~End UPCGSettings interface implementation

public:
	/** Specifies what attribute from the input that will drive the offset direction. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector DirectionAttribute;

	/** Specifies what attribute from the magnitude input that will drive the offset magnitude. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyInputSelector MagnitudeAttribute;
};