// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Grammar/PCGGrammarParser.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGGrammar.generated.h"

namespace PCGGrammar
{
	// Coupled concrete module information with a grammar module descriptor (node)
	struct FTokenizedModule
	{
		FTokenizedModule() = default;
		FTokenizedModule(const PCGGrammar::FModuleDescriptor* InDescriptor)
			: Descriptor(InDescriptor)
		{
		}

		bool IsValid() const { return bIsValid; }
		double GetMinSize() const { return UnitSize * (Descriptor ? Descriptor->GetMinNumberOfRepetitions() : 0); }
		double GetUnitSize() const { return UnitSize; }
		double GetMinConcreteSize() const { return ConcreteUnitSize * (Descriptor ? Descriptor->GetMinConcreteNumberOfRepetitions() : 0); }
		double GetConcreteUnitSize() const { return ConcreteUnitSize; }
		int GetNumRepeat() const { return Descriptor ? Descriptor->Repetitions : 0; }
		int8 GetWeight() const { return Descriptor ? Descriptor->Weight : 0; }
		bool IsScalable() const { return bScalable; }
		EModuleType GetType() const { return Descriptor ? Descriptor->Type : EModuleType::Unknown; }

		TArray<FTokenizedModule> Submodules;
		const PCGGrammar::FModuleDescriptor* Descriptor = nullptr;
		double UnitSize = 0.0; // minimum size not including repetitions
		double ConcreteUnitSize = 0.0; // minimum size of the expansion of a unit of this module
		bool bScalable = false;
		bool bIsValid = false;
	};

	struct FTokenizedGrammar
	{
		TSharedPtr<FModuleDescriptor> ParsedGrammar;
		TSharedPtr<FTokenizedModule> ModuleGrammar;

		bool IsValid() const { return ModuleGrammar.IsValid() && ModuleGrammar.Get()->IsValid(); }
	};
}

USTRUCT(Blueprintable, BlueprintType)
struct FPCGGrammarSelection
{
	GENERATED_BODY()

	/** Read the grammar as an attribute rather than directly from the settings.
	* Grammar syntax:
	* - Each symbol can have multiple characters
	* - Modules are defined in '[]', multiple symbols in a module are separated with ','
	* - Modules can be repeated a fixed number of times, by adding a number after it (like [A,B]3 will produce ABABAB)
	* - Modules can be marked repeated an indefinite number of times, with '*'. (like [A,B]* will produce ABABABAB... while it fits the allowed size).
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bGrammarAsAttribute = false;

	/** An encoded string that represents how to apply a set of rules to a series of defined modules. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (MultiLine, EditCondition = "!bGrammarAsAttribute", EditConditionHides, PCG_Overridable))
	FString GrammarString;

	/** Attribute to be taken from the input spline containing the grammar to use. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "bGrammarAsAttribute", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector GrammarAttribute;
};