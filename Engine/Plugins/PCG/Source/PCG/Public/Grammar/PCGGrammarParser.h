// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

#define UE_API PCG_API

namespace PCGGrammar
{
	enum class EModuleType
	{
		Root,
		Literal,
		Sequence,
		Stochastic,
		Priority,
		Unknown // used when there are no matching descriptors
	};

	static constexpr int InfiniteRepetition = -1;
	static constexpr int AtLeastOneRepetition = -2;

	struct FModuleDescriptor
	{
		FModuleDescriptor() = default;
		FModuleDescriptor(FName InLiteral)
		{
			Type = EModuleType::Literal;
			Symbol = InLiteral;
		}

		explicit FModuleDescriptor(EModuleType InType)
		{
			Type = InType;
		}

		int32 GetMinNumberOfRepetitions() const { return (Repetitions >= 0 ? Repetitions : (Repetitions == AtLeastOneRepetition ? 1 : 0)); }
		int32 GetMinConcreteNumberOfRepetitions() const { return ((Repetitions >= 0) ? Repetitions : 1); }

		TArray<FModuleDescriptor> Submodules;
		FModuleDescriptor* Parent = nullptr;
		FName Symbol;
		int32 Repetitions = 1;
		EModuleType Type = EModuleType::Literal;
		int8 Weight = 1; // TODO: review the need for this based on struct size
	};
}

struct FPCGGrammarResult
{
	enum class ELogType
	{
		Log = ELogVerbosity::Log,
		Warning = ELogVerbosity::Warning,
		Error = ELogVerbosity::Error
	};

	struct FLog
	{
		FLog(FText InMessage, ELogType InVerbosity = ELogType::Log)
			: Message(std::move(InMessage))
			, Verbosity(InVerbosity)
		{}

		FText Message;
		ELogType Verbosity = ELogType::Log;

		bool operator==(const FLog& Other) const { return Message.EqualTo(Other.Message) && Verbosity == Other.Verbosity; }
	};

	UE_API void AddLog(FText Message, ELogType Verbosity = ELogType::Log);

	const TArray<FLog>& GetLogs() const{ return Logs; }

	bool bSuccess = true;
	PCGGrammar::FModuleDescriptor Root = PCGGrammar::FModuleDescriptor(PCGGrammar::EModuleType::Root);

private:
	TArray<FLog> Logs;
};

namespace PCGGrammar
{
	PCG_API FPCGGrammarResult Parse(const FString& Grammar);
}

#undef UE_API
