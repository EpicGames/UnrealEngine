// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/Parsing/PCGIndexing.h"
#include "StructUtils/InstancedStruct.h"

#include "Hash/xxhash.h"

#include "PCGDataOverride.generated.h"

class UPCGData;
struct FPCGStack;
struct FPCGContext;
struct FPCGPoint;

namespace PCG::DataOverride
{
	namespace Constants
	{
		inline constexpr FLazyName DefaultOverrideLabel = "Override";
	}
} // namespace PCG::DataOverride

/** How the overridden data is tied to its storage. Used for mapping. */
UENUM()
enum class EPCGDataOverrideKeyPolicy : uint8
{
	None = 0,
	Spatial,
	ByIndex,
	ByAttribute
};

UENUM()
enum class EPCGSpatialKeyTarget : uint8
{
	Position,
	Transform,
	AABB
};

/** Phase of element execution to apply overrides. Usually in PostExecute, but artifact generating elements may require the PrepareData phase. */
UENUM(Meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EPCGDataOverridePhase : uint8
{
	None        = 0,
	PrepareData = 1 << 0,
	PostExecute = 1 << 1,
};
ENUM_CLASS_FLAGS(EPCGDataOverridePhase);

/** If there are multiple collisions. Note: NoOp may have added cost, requiring the key matching to go through the entire set of candidates. */
UENUM()
enum class EPCGKeyCollisionResolution : uint8
{
	NoOp,      // Do not apply the override
	TakeFirst, // Apply the first override and ignore any further collisions
	Fail       // Fail and cancel the operation
};

/** In the event of a collision, this is how the user will be notified. */
UENUM()
enum class EPCGKeyCollisionResponse : uint8
{
	DoNothing = 0,
	Log,
	Warning,
	Error
};

/** Configuration for how the override should be applied. */
USTRUCT()
struct FPCGDeltaSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Key Composition")
	EPCGDataOverrideKeyPolicy KeyPolicy = EPCGDataOverrideKeyPolicy::None;

	UPROPERTY(EditAnywhere, Category = "Key Composition", meta = (EditCondition = "KeyPolicy == EPCGDataOverrideKeyPolicy::ByAttribute", EditConditionHides))
	FName AttributeName = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Key Composition")
	FName PinLabel = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Key Composition", meta = (EditCondition = "KeyPolicy == EPCGDataOverrideKeyPolicy::Spatial", EditConditionHides))
	EPCGSpatialKeyTarget KeyTarget = EPCGSpatialKeyTarget::Position;

	UPROPERTY(EditAnywhere, Category = "Key Composition", meta = (EditCondition = "KeyPolicy == EPCGDataOverrideKeyPolicy::Spatial", EditConditionHides, ClampMin = "1.0"))
	double SpatialTolerance = 10.0;
};

/** Wrapper helper around an element's signature. Used for storing deltas. */
USTRUCT()
struct FPCGDeltaKey
{
	GENERATED_BODY()

	FPCGDeltaKey() = default;
	PCG_API explicit FPCGDeltaKey(const FXxHash64 InHash, FName InDeltaLabel = NAME_None);

	bool operator==(const FPCGDeltaKey& Other) const { return Hash == Other.Hash; }
	bool operator!=(const FPCGDeltaKey& Other) const { return Hash != Other.Hash; }
	friend uint32 GetTypeHash(const FPCGDeltaKey& Key) { return GetTypeHash(Key.Hash); }

	UPROPERTY()
	uint64 Hash = 0u;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName DeltaLabel;
#endif // WITH_EDITORONLY_DATA
};

/** Base class for deltas to inherit. Contains debug metadata or other useful information. */
USTRUCT()
struct FPCGDeltaBase
{
	GENERATED_BODY()

	virtual ~FPCGDeltaBase() = default;
	PCG_API virtual FName GetDeltaName() const;
	PCG_API virtual PCGIndexing::FPCGIndexCollection FilterCandidates(const UPCGData* InData) const;
	PCG_API virtual int32 Resolve(const UPCGData* InData, const PCGIndexing::FPCGIndexCollection& FilteredCandidates, const FPCGDeltaSettings& DeltaSettings) const;
	PCG_API virtual bool Apply(UPCGData* InData, int32 ResolvedIndex) const;
	
	/** Returns true if Delta application is a full data replacement. In this case, Apply() should not be called, UPCGData should simply be replaced by the return value of GetReplacementData() */
	virtual bool UsesReplacementData() const { return false; }
	
	/** Returns the Data that should replace the current one, returns nullptr if the delta does not support full replacement.*/
	virtual UPCGData* GetReplacementData() const { return nullptr; } 

#if WITH_EDITORONLY_DATA
	// @todo_pcg: The per-point spatial key used as the key for this specific delta. Keep for debugging, potentially permanently.
	UPROPERTY()
	FPCGDeltaKey ComputedKey;
#endif // WITH_EDITORONLY_DATA
	
	static FPCGDeltaKey ComputeKey(const FPCGDeltaSettings& DeltaSettings, const FTransform& InTransform, const FBox& InWorldBounds);
};

/** A collection of deltas, usually tied directly to a node element or more specifically, to a pin output. */
USTRUCT()
struct FPCGDeltaCollection
{
	GENERATED_BODY()

	/** Find the mutable Delta with the provided Key. */
	PCG_API TInstancedStruct<FPCGDeltaBase>* Find(const FPCGDeltaKey& InKey);
	/** Find the Delta with the provided Key. */
	PCG_API const TInstancedStruct<FPCGDeltaBase>* Find(const FPCGDeltaKey& InKey) const;
	/** Returns true if the collection contains the provided Key. */
	PCG_API bool Contains(const FPCGDeltaKey& InKey) const;

	// For now, this will be editor only for storage mutation.
#if WITH_EDITOR
	/** Add a new delta entry to the collection, as a TInstancedStruct<FPCGDeltaBase>. */
	PCG_API void Add(const FPCGDeltaKey& InKey, TInstancedStruct<FPCGDeltaBase>&& InDelta);
	/** Add a new delta entry to the collection, as a TInstancedStruct<FPCGDeltaBase>. Returns a mutable reference to the delta. */
	PCG_API TInstancedStruct<FPCGDeltaBase>& Add_GetRef(const FPCGDeltaKey& InKey, TInstancedStruct<FPCGDeltaBase>&& InDelta);

	/** Remove a delta entry by key. */
	PCG_API bool Remove(const FPCGDeltaKey& InKey);

	/** Empty the collection. */
	PCG_API void Empty();
#endif // WITH_EDITOR

	/** The number of Deltas in the collection. */
	PCG_API int32 Num() const;
	PCG_API bool IsEmpty() const;

	/** Iterates over all deltas, calling the provided function for each.
	 * @param Func Lambda returning true to continue iteration, false to stop early
	 * @return true if all deltas were processed, false if iteration stopped early
	 */
	PCG_API bool ForEachDelta(TFunctionRef<bool(const FPCGDeltaKey&, TInstancedStruct<FPCGDeltaBase>&)> Func);
	PCG_API bool ForEachDelta(TFunctionRef<bool(const FPCGDeltaKey&, const TInstancedStruct<FPCGDeltaBase>&)> Func) const;
	// @todo_pcg: ForEachDeltaAs

	UPROPERTY()
	FPCGDeltaSettings Settings;

private:
	UPROPERTY()
	TMap<FPCGDeltaKey, TInstancedStruct<FPCGDeltaBase>> Deltas;
};

