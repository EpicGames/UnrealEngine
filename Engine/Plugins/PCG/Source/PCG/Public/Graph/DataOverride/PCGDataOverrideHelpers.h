// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGDataOverride.h"
#include "Graph/PCGSourceDataContainer.h"
#include "Metadata/PCGAttributePropertySelector.h"

struct FXxHash64;
struct FPCGTaggedData;
class IPCGGraphExecutionSource;
class UPCGNode;
class UPCGPin;

namespace PCG::DataOverride
{
	namespace Constants
	{
		constexpr double SpatialToleranceDefault = 10.0;
	}

	namespace Helpers
	{
		/** Compute a node-level override key from an execution stack and node. Pushes the node frame and hashes. */
		PCG_API FXxHash64 ComputeNodeOverrideKey(const FPCGStack& InExecutionStack, const UPCGNode* InNode);
		/** Compute a pin-level override key from an execution stack, node, and pin label. */
		PCG_API FXxHash64 ComputePinOverrideKey(const FPCGStack& InExecutionStack, const UPCGNode* InNode, FName PinLabel);
		/** Compute a pin-level override key by appending a pin label to a precomputed node key. */
		PCG_API FXxHash64 ComputePinOverrideKey(FXxHash64 NodeKey, FName PinLabel);

		/** Returns the pins that carry override data for the given node (input or output, depending on the element's override phase). */
		PCG_API const TArray<TObjectPtr<UPCGPin>>* GetOverridePins(const UPCGNode* InNode);

#if WITH_EDITOR
		/** Returns every storage key (pin-level and node-level) that could host overrides for the node across executed stacks. */
		PCG_API TArray<FPCGSourceDataStorageKey> CollectNodeStorageKeys(const UPCGNode* InNode, const IPCGGraphExecutionSource* InExecutionSource);
#endif

		/** Returns the const data container for the execution source, or nullptr if either is missing. */
		PCG_API const FPCGSourceDataContainer* GetSourceDataContainer(const IPCGGraphExecutionSource* InExecutionSource);

		/** Returns the mutable data container for the execution source, or nullptr if either is missing. */
		PCG_API FPCGSourceDataContainer* GetMutableSourceDataContainer(IPCGGraphExecutionSource* InExecutionSource);

		/** Refresh a single execution source. Routes runtime-gen sources through the runtime gen scheduler; otherwise re-runs Generate so only that source regenerates. */
		PCG_API void RefreshExecutionSource(IPCGGraphExecutionSource* InExecutionSource);
	} // namespace Helpers

	namespace Keys
	{
		PCG_API FXxHash64 ComputePositionHash(const FVector& InLocation, double Tolerance = Constants::SpatialToleranceDefault);
		PCG_API FXxHash64 ComputeTransformHash(const FTransform& InTransform, double Tolerance = Constants::SpatialToleranceDefault);
		PCG_API FXxHash64 ComputeAABBHash(const FBox& InBounds, double Tolerance = Constants::SpatialToleranceDefault);
		// @todo_pcg: Could do a radial sphere bounds key as well.

		/** Create a key directly with a custom label. */
		struct FPCGLabelDeltaKey
		{
			explicit FPCGLabelDeltaKey(const FName InLabel) : Label(InLabel) {}

			operator FPCGDeltaKey() const { return FPCGDeltaKey(FXxHash64{}, Label); }

		private:
			FName Label;
		};

		struct FPCGLocationDeltaKey
		{
			explicit FPCGLocationDeltaKey(const FVector& InLocation, const double InTolerance = Constants::SpatialToleranceDefault, const FName DeltaLabel = NAME_None)
				: Key(ComputePositionHash(InLocation, InTolerance), DeltaLabel) {}

			operator FPCGDeltaKey() const { return Key; }

		private:
			FPCGDeltaKey Key;
		};

		struct FPCGAABBDeltaKey
		{
			explicit FPCGAABBDeltaKey(const FBox& InBounds, const double InTolerance = Constants::SpatialToleranceDefault, const FName DeltaLabel = NAME_None)
				: Key(ComputeAABBHash(InBounds, InTolerance), DeltaLabel) {}

			operator FPCGDeltaKey() const { return Key; }

		private:
			FPCGDeltaKey Key;
		};

		struct FPCGTransformDeltaKey
		{
			explicit FPCGTransformDeltaKey(const FTransform& InTransform, const double InTolerance = Constants::SpatialToleranceDefault, const FName DeltaLabel = NAME_None)
				: Key(ComputeTransformHash(InTransform, InTolerance), DeltaLabel) {}

			operator FPCGDeltaKey() const { return Key; }

		private:
			FPCGDeltaKey Key;
		};

		/** Compute a hash from selected transform components. Short-circuits to existing functions when possible. */
		PCG_API FXxHash64 ComputeCompositeHash(const FTransform& InTransform, double Tolerance, bool bIncludePosition, bool bIncludeRotation, bool bIncludeScale);

		struct FPCGCompositeTransformDeltaKey
		{
			explicit FPCGCompositeTransformDeltaKey(
				const FTransform& InTransform,
				const double InTolerance,
				const bool bPosition,
				const bool bRotation,
				const bool bScale,
				const FName DeltaLabel = NAME_None)
				: Key(ComputeCompositeHash(InTransform, InTolerance, bPosition, bRotation, bScale), DeltaLabel) {}

			operator FPCGDeltaKey() const { return Key; }

		private:
			FPCGDeltaKey Key;
		};

		// @todo_pcg: still to implement in future pass
		PCG_API FPCGDeltaKey ComputeAttributeKey(const FPCGAttributePropertyInputSelector& InSelector);
	} // namespace Keys

	/** Returns true if the node has node-level data overrides that apply to all outputs. */
	bool HasNodeLevelDataOverrides(FPCGContext* Context);

	/** Returns true if the specific tagged data entry has pin-level data overrides. */
	bool HasPinLevelDataOverrides(FPCGContext* Context, const FPCGTaggedData& InTaggedData);

	/** Returns true if node-level or pin-level data overrides exist. */
	bool HasAnyDataOverrides(FPCGContext* Context, const TArrayView<const FPCGTaggedData> InTaggedData);

	/** Applies override deltas from the container to the tagged data. Data must be owned/duplicated before calling. */
	void ApplyDataOverrides(FPCGContext* Context, const TArrayView<FPCGTaggedData> OutTaggedData);
} // namespace PCG::DataOverride
