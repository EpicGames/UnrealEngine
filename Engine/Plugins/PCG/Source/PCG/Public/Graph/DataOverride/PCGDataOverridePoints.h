// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataOverride.h"
#include "PCGPoint.h"

#include "PCGDataOverridePoints.generated.h"

namespace PCG::DataOverride::Points::Constants
{
	inline constexpr FLazyName PointTransformDeltaName = "PointTransform";
	inline constexpr FLazyName PointTransformOffsetDeltaName = "PointTransformOffset";
	inline constexpr FLazyName PointDeleteDeltaName = "PointDelete";
	inline constexpr FLazyName PointInsertDeltaName = "PointInsert";
}

/** Base delta for point data overrides. Uses an octree bounds query to filter candidate points spatially. */
USTRUCT()
struct FPCGPointDeltaBase : public FPCGDeltaBase
{
	GENERATED_BODY()

	/** Queries the point octree with Bounds to find candidate points. */
	PCG_API virtual PCGIndexing::FPCGIndexCollection FilterCandidates(const UPCGData* InData) const override;

	/** Computes the spatial key for each candidate and compares against this delta's key to find the matching point. */
	PCG_API virtual int32 Resolve(const UPCGData* InData, const PCGIndexing::FPCGIndexCollection& FilteredCandidates, const FPCGDeltaSettings& DeltaSettings) const override;

	/** The original transform of the point. For use with matching the key during override. */
	UPROPERTY()
	FTransform OriginalTransform;

	/** World-space bounding box used for octree filtering and key computation. */
	UPROPERTY()
	FBox Bounds = FBox(ForceInit);
};

/** Delta that overrides the transform of a specific point. */
USTRUCT()
struct FPCGPointTransformDelta : public FPCGPointDeltaBase
{
	GENERATED_BODY()

	static FName GetDeltaNameStatic() { return PCG::DataOverride::Points::Constants::PointTransformDeltaName; }
	PCG_API virtual FName GetDeltaName() const override;
	PCG_API virtual bool Apply(UPCGData* InData, int32 ResolvedIndex) const override;

	/** The desired transform to apply as the override. */
	UPROPERTY()
	FTransform TransformOverride;

	/** Override the point's position. */
	UPROPERTY()
	bool bOverridePosition = true;

	/** Override the point's rotation. */
	UPROPERTY()
	bool bOverrideRotation = true;

	/** Override the point's scale. */
	UPROPERTY()
	bool bOverrideScale = true;

#if WITH_EDITORONLY_DATA
	/** The original element index. For debugging purposes only. */
	UPROPERTY()
	int32 ElementIndex = INDEX_NONE;
#endif // WITH_EDITORONLY_DATA
};

/** Delta that adds a local offset to the transform of a specific point. Position is additive, rotation is composed, scale is multiplicative. */
USTRUCT()
struct FPCGPointTransformOffsetDelta : public FPCGPointDeltaBase
{
	GENERATED_BODY()

	static FName GetDeltaNameStatic() { return PCG::DataOverride::Points::Constants::PointTransformOffsetDeltaName; }
	PCG_API virtual FName GetDeltaName() const override;
	PCG_API virtual bool Apply(UPCGData* InData, int32 ResolvedIndex) const override;

	// @todo_pcg: The offset is currently applied in world space. Explore local-to-point and local-to-actor offset spaces.
	// Local-to-point would rotate the offset by the point's original transform, making offsets survive orientation changes
	// in the source data. Local-to-actor would use the owning component's transform as the reference frame.

	/** The offset to compose with the point's transform. Applied in world space. */
	UPROPERTY()
	FTransform TransformOffset;

	/** Offset the point's position. */
	UPROPERTY()
	bool bOffsetPosition = true;

	/** Offset the point's rotation. */
	UPROPERTY()
	bool bOffsetRotation = true;

	/** Offset the point's scale. */
	UPROPERTY()
	bool bOffsetScale = true;

#if WITH_EDITORONLY_DATA
	/** The original element index. For debugging purposes only. */
	UPROPERTY()
	int32 ElementIndex = INDEX_NONE;
#endif // WITH_EDITORONLY_DATA
};

/** Delta that marks a specific point for deletion. */
USTRUCT()
struct FPCGPointDeletionDelta : public FPCGPointDeltaBase
{
	GENERATED_BODY()

	static FName GetDeltaNameStatic() { return PCG::DataOverride::Points::Constants::PointDeleteDeltaName; }
	PCG_API virtual FName GetDeltaName() const override;
	PCG_API virtual bool Apply(UPCGData* InData, int32 ResolvedIndex) const override;

#if WITH_EDITORONLY_DATA
	/** The original element index. For debugging purposes only. */
	UPROPERTY()
	int32 ElementIndex = INDEX_NONE;
#endif // WITH_EDITORONLY_DATA
};

/** Delta that inserts a new point. */
USTRUCT()
struct FPCGPointInsertionDelta : public FPCGDeltaBase
{
	GENERATED_BODY()

	static FName GetDeltaNameStatic() { return PCG::DataOverride::Points::Constants::PointInsertDeltaName; }
	PCG_API virtual FName GetDeltaName() const override;
	PCG_API virtual PCGIndexing::FPCGIndexCollection FilterCandidates(const UPCGData* InData) const override;
	PCG_API virtual bool Apply(UPCGData* InData, int32 ResolvedIndex) const override;

	/** Points to be appended to the point data. */
	UPROPERTY()
	TArray<FPCGPoint> InsertedPoints;
};
