// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/DataOverride/PCGDataOverride.h"
#include "Data/PCGPolygon2DData.h"

#include "PCGDataOverridePolygon.generated.h"

USTRUCT()
struct FPCGPolygon2DDataDelta : public FPCGDeltaBase
{
	GENERATED_BODY()

	static FName GetDeltaNameStatic();

	//~ Begin FPCGDeltaBase Interface
	PCG_API virtual PCGIndexing::FPCGIndexCollection FilterCandidates(const UPCGData* InData) const override;
	PCG_API virtual int32 Resolve(const UPCGData* InData, const PCGIndexing::FPCGIndexCollection& FilteredCandidates, const FPCGDeltaSettings& DeltaSettings) const override;
	PCG_API virtual FName GetDeltaName() const override;
	PCG_API virtual bool Apply(UPCGData* InData, int32 ResolvedIndex) const override;
	virtual bool UsesReplacementData() const override { return true; }
	PCG_API virtual UPCGData* GetReplacementData() const override;
	//~ End FPCGDeltaBase Interface
	
	/** The original transform of the first point of the polygon. For use with matching the key during override. */
	UPROPERTY()
	FTransform OriginalStartTransform = FTransform::Identity;

	/** The polygon that should replace the other polygon.*/
	UPROPERTY()
	TObjectPtr<UPCGPolygon2DData> PolygonOverride;
};
