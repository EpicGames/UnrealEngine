// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/DataOverride/PCGDataOverride.h"
#include "Data/PCGSplineData.h"

#include "PCGDataOverrideSpline.generated.h"

USTRUCT()
struct FPCGSplineDataDelta : public FPCGDeltaBase
{
	GENERATED_BODY()

	static FName GetDeltaNameStatic();

	//~ Begin FPCGDeltaBase Interface
	PCG_API virtual PCGIndexing::FPCGIndexCollection FilterCandidates(const UPCGData* InData) const override;
	PCG_API virtual int32 Resolve(const UPCGData* InData, const PCGIndexing::FPCGIndexCollection& FilteredCandidates, const FPCGDeltaSettings& DeltaSettings) const override;
	PCG_API virtual FName GetDeltaName() const override;
	PCG_API virtual bool Apply(UPCGData* InData, int32 ResolvedIndex) const override;
	virtual bool UsesReplacementData() const override { return true; }
	PCG_API virtual UPCGData* GetReplacementData() const override;
	//~ End FPCGDeltaBase Interface
	
	/** The original transform of the starting point of the spline. For use with matching the key during override. */
	UPROPERTY()
	FTransform OriginalStartTransform = FTransform::Identity;

	/** The spline data that should replace the spline*/
	UPROPERTY()
	TObjectPtr<UPCGSplineData> SplineOverride;
};
