// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCrc.h"

class IPCGElement;
class IPCGGraphExecutionSource;
class UPCGComponent;
class UPCGNode;
struct FPCGDataCollection;

struct FPCGGetFromCacheParams
{
	const UPCGNode* Node = nullptr;
	const IPCGElement* Element = nullptr;

	const IPCGGraphExecutionSource* ExecutionSource = nullptr;
	FPCGCrc Crc;
};

struct FPCGStoreInCacheParams
{
	const IPCGElement* Element = nullptr;
	FPCGCrc Crc;
};

/** Interface to encapsulate use of the cache in PCG elements */
class IPCGGraphCache
{
public:
	virtual bool IsEnabled() const = 0;
	virtual bool GetFromCache(const FPCGGetFromCacheParams& Params, FPCGDataCollection& OutCollection) const = 0;
	virtual void StoreInCache(const FPCGStoreInCacheParams& Params, const FPCGDataCollection& InCollection) = 0;
};
