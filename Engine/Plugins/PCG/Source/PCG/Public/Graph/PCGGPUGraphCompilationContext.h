// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "PCGCommon.h"

class FPCGGraphCompiler;
class UPCGComputeGraph;
struct FPCGKernelAttributeTable;

/** Current context/state used during compilation of GPU graphs. */
struct FPCGGPUCompilationContext
{
	explicit FPCGGPUCompilationContext(FPCGGraphCompiler& InGraphCompiler)
		: GraphCompiler(InGraphCompiler)
	{
	}

	~FPCGGPUCompilationContext()
	{
		// Remove Async Flags from objects created during compilation so that they can get tracked by the GarbageCollector properly.
		for (TObjectPtr<UObject>& AsyncObject : AsyncObjects)
		{
			if (ensure(AsyncObject->HasAnyInternalFlags(EInternalObjectFlags::Async)))
			{
				AsyncObject->ClearInternalFlags(EInternalObjectFlags::Async);
				ForEachObjectWithOuter(
					AsyncObject,
					[&AsyncObjects = AsyncObjects](UObject* SubObject)
					{
						if (AsyncObjects.Contains(SubObject))
						{
							return;
						}

						SubObject->ClearInternalFlags(EInternalObjectFlags::Async);
					},
					EGetObjectsFlags::IncludeNestedObjects);
			}
		}
	}

	/** Thread safe NewObject. If object created off game thread, object added to AsyncObjects. */
	template<class T, typename... Args>
	T* NewObject_AnyThread(Args&&... InArgs)
	{
		if (!IsInGameThread())
		{
			T* Object = nullptr;
			{
				FGCScopeGuard Scope;
				Object = ::NewObject<T>(std::forward<Args>(InArgs)...);
			}

			check(Object);
			AsyncObjects.Add(Object);
			return Object;
		}

		return ::NewObject<T>(std::forward<Args>(InArgs)...);
	}

	FPCGGraphCompiler& GetGraphCompiler() { return GraphCompiler; }
	TArray<TObjectPtr<UPCGComputeGraph>>& GetCompiledComputeGraphs() { return CompiledComputeGraphs; }

	const FPCGKernelAttributeTable* GetStaticAttributeTable() { return StaticAttributeTable; }
	void SetStaticAttributeTable(const FPCGKernelAttributeTable* InStaticAttributeTable) { StaticAttributeTable = InStaticAttributeTable; }

	/** Returns the index of the newly added compute graph. */
	int32 AddCompiledComputeGraph(TObjectPtr<UPCGComputeGraph> NewComputeGraph) { return CompiledComputeGraphs.Add(NewComputeGraph); }

private:
	// List of objects created during compilation, we need to track them so we can remove their Async flags when compilation is done
	// so that they can be considered as existing on the main thread (and get properly GCed).
	TSet<TObjectPtr<UObject>> AsyncObjects;

	FPCGGraphCompiler& GraphCompiler;
	TArray<TObjectPtr<UPCGComputeGraph>> CompiledComputeGraphs;

	/** Attribute table of the currently compiling compute graph. */
	const FPCGKernelAttributeTable* StaticAttributeTable = nullptr;
};

#endif // WITH_EDITOR
