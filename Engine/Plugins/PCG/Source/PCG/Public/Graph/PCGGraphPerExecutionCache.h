// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGModule.h"
#include "PCGGraphExecutionStateInterface.h"

#include "StructUtils/InstancedStruct.h"

#include "PCGGraphPerExecutionCache.generated.h"

class UPCGData;

USTRUCT()
struct FPCGPerExecutionCacheData
{
	GENERATED_BODY()

	FPCGPerExecutionCacheData() = default;
	virtual ~FPCGPerExecutionCacheData() = default;

	virtual void AddStructReferencedObjects(FReferenceCollector& Collector) {}
};

USTRUCT()
struct FPCGPerExecutionCachePCGData : public FPCGPerExecutionCacheData
{
	GENERATED_BODY()

	using ValueType = UPCGData*;

	FPCGPerExecutionCachePCGData() = default;
	FPCGPerExecutionCachePCGData(ValueType InData)
		: Data(InData)
	{
	}

	virtual void AddStructReferencedObjects(FReferenceCollector& Collector) override;

	ValueType GetValue() const { return Data; }

	// stored as TObjectPtr for AddStructReferencedOjects
	TObjectPtr<UPCGData> Data = nullptr;
};

USTRUCT()
struct FPCGPerExecutionCacheBounds : public FPCGPerExecutionCacheData
{
	GENERATED_BODY()

	using ValueType = FBox;

	FPCGPerExecutionCacheBounds() = default;
	FPCGPerExecutionCacheBounds(const ValueType& InBounds)
		: Bounds(InBounds)
	{
	}

	const ValueType& GetValue() const { return Bounds; }

	ValueType Bounds;
};

/** Typed GUID wrapper associating a cache identifier with a concrete FPCGPerExecutionCacheData
 *  subclass. Using TPCGPerExecutionCacheId<T> instead of a raw FGuid in Get/Set prevents passing a
 *  Bounds GUID to a PCGData getter — the type mismatch is caught at compile time. */
template<typename T>
struct TPCGPerExecutionCacheId
{
	FGuid Guid;
	const TCHAR* DebugName = nullptr;
	explicit constexpr TPCGPerExecutionCacheId(FGuid InGuid, const TCHAR* InDebugName = nullptr) : Guid(InGuid), DebugName(InDebugName) {}
};

namespace PCGPerExecutionCacheGuids
{
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCachePCGData>    PCGData{						FGuid(0x8D1077E7, 0x32AD4876, 0xA86D33D4, 0xE6618E25), TEXT("CachedPCGData") };
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCachePCGData>    InputData{					FGuid(0x410F0D9A, 0x10504945, 0x916FAEC0, 0xAC4CCDF5), TEXT("CachedInputData") };
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCachePCGData>    SelfData{					FGuid(0xDB4FEE15, 0x0B924BB9, 0x920E17D0, 0xB68A439A), TEXT("CachedActorData") };
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCachePCGData>    LandscapeData{				FGuid(0xBD2F8989, 0xF911433C, 0x9150CF07, 0xD25A2A3F), TEXT("CachedLandscapeData") };
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCachePCGData>    LandscapeHeightData{			FGuid(0x697872EE, 0x90624718, 0x85248924, 0xC5C3C2E2), TEXT("CachedLandscapeHeightData") };
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCachePCGData>    OriginalSelfData{			FGuid(0xE2A8E4C1, 0x18B444BD, 0xABA60F8C, 0x78FFC972), TEXT("CachedOriginalActorData") };
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCacheBounds>     Bounds{						FGuid(0xEEB1FE5D, 0xCA5B4EDA, 0xA96B1448, 0x49F7FFAE), TEXT("GridBounds") };
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCacheBounds>     OriginalBounds{				FGuid(0x7A77D800, 0x5E9045D3, 0xB69CD774, 0xE6CC5F0C), TEXT("Original Bounds") };
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCacheBounds>     LocalSpaceBounds{			FGuid(0xDA7B1698, 0x19A54A97, 0xB53C66E4, 0x126319FC), TEXT("Local Space Bounds") };
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCacheBounds>     OriginalLocalSpaceBounds{	FGuid(0x4F358809, 0x6FB24086, 0xB977FC0C, 0x8BC291D3), TEXT("Original Local Space Bounds") };
	constexpr TPCGPerExecutionCacheId<FPCGPerExecutionCacheBounds>     TotalBounds{					FGuid(0x65B51D8B, 0xA70348BE, 0x894CDDF1, 0x8D318171), TEXT("Total Bounds") };
}

struct FPCGPerExecutionCacheEntry
{
	void AddStructReferencedObjects(FReferenceCollector& Collector);

	TMap<FGuid, TInstancedStruct<FPCGPerExecutionCacheData>> Data;
};

struct FPCGPerExecutionCache
{
	/** Returns the cached value for the source's current task, unset otherwise.
	 *  T is deduced from InCacheId — mismatched IDs are a compile error. */
	template<typename T>
	TOptional<typename T::ValueType> GetExecutionCacheEntry(const IPCGGraphExecutionSource* InSource, TPCGPerExecutionCacheId<T> InCacheId);

	/** Stores a new value for the source's current task and typed ID. Asserts if the slot is already occupied. */
	template<typename T>
	void SetExecutionCacheEntry(const IPCGGraphExecutionSource* InSource, TPCGPerExecutionCacheId<T> InCacheId, typename T::ValueType InValue, bool bInValidateWritable = true, bool bEnableLogging = false);

	/** Returns the cached value if present, otherwise calls InMakeEntry() to produce one.
	 *  If the source's task is valid the produced value is stored in the cache before returning.
	 *  InMakeEntry must return T::ValueType by value. */
	template<typename T, typename FuncType>
	typename T::ValueType GetOrCreateExecutionCacheValue(const IPCGGraphExecutionSource* InSource, TPCGPerExecutionCacheId<T> InCacheId, const FuncType& InMakeEntry, bool bInValidateWritable = true, bool bEnableLogging = false);

	void Clear();
	void AddStructReferencedObjects(FReferenceCollector& Collector);

	/** Per graph execution cache, gets emptied when executor has no more work to do */
	PCG::FRecursiveSharedLock Lock;
	TMap<FPCGTaskId, FPCGPerExecutionCacheEntry> Entries;
};

template<typename T>
TOptional<typename T::ValueType> FPCGPerExecutionCache::GetExecutionCacheEntry(const IPCGGraphExecutionSource* InSource, TPCGPerExecutionCacheId<T> InCacheId)
{
	check(InSource);
	const FPCGTaskId TaskId = InSource->GetExecutionState().GetGenerationTaskId();
	if (TaskId != InvalidPCGTaskId)
	{
		PCG::TSharedScopeLock ReadLock(Lock, !Lock.IsLockedByCurrentThread());
		if (FPCGPerExecutionCacheEntry* Entry = Entries.Find(TaskId))
		{
			if (TInstancedStruct<FPCGPerExecutionCacheData>* StructData = Entry->Data.Find(InCacheId.Guid))
			{
				if (StructData->IsValid())
				{
					return StructData->Get<T>().GetValue();
				}
			}
		}
	}
	return {};
}

template<typename T>
void FPCGPerExecutionCache::SetExecutionCacheEntry(const IPCGGraphExecutionSource* InSource, TPCGPerExecutionCacheId<T> InCacheId, typename T::ValueType InValue, bool bValidateWritable, bool bEnableLogging)
{
	check(InSource);
	const FPCGTaskId TaskId = InSource->GetExecutionState().GetGenerationTaskId();
	if (TaskId != InvalidPCGTaskId)
	{
		PCG::TUniqueScopeLock WriteLock(Lock);
		FPCGPerExecutionCacheEntry& Entry = Entries.FindOrAdd(TaskId);
		TInstancedStruct<FPCGPerExecutionCacheData>& StructData = Entry.Data.FindOrAdd(InCacheId.Guid);
		check(!StructData.IsValid());
		
		if (bValidateWritable)
		{
			PCG_EXECUTION_CACHE_VALIDATION_CHECK(InSource);
		}

		StructData.InitializeAs<T>(InValue);

		UE_CLOGF(bEnableLogging, LogPCG, Log, "         [%ls] CACHE SET %ls",
			*InSource->GetExecutionState().GetDebugName(),
			InCacheId.DebugName ? InCacheId.DebugName : TEXT("unknown"));
	}
}

template<typename T, typename FuncType>
typename T::ValueType FPCGPerExecutionCache::GetOrCreateExecutionCacheValue(const IPCGGraphExecutionSource* InSource, TPCGPerExecutionCacheId<T> InCacheId, const FuncType& InMakeEntry, bool bValidateWritable, bool bEnableLogging)
{
	check(InSource);
	const FPCGTaskId TaskId = InSource->GetExecutionState().GetGenerationTaskId();
	if (TaskId != InvalidPCGTaskId)
	{
		{
			PCG::TSharedScopeLock ReadLock(Lock, !Lock.IsLockedByCurrentThread());
			if (FPCGPerExecutionCacheEntry* Entry = Entries.Find(TaskId))
			{
				if (TInstancedStruct<FPCGPerExecutionCacheData>* StructData = Entry->Data.Find(InCacheId.Guid))
				{
					if (StructData->IsValid())
					{
						return StructData->GetMutable<T>().GetValue();
					}
				}
			}
		}

		PCG::TUniqueScopeLock WriteLock(Lock);
		FPCGPerExecutionCacheEntry* Entry = &Entries.FindOrAdd(TaskId);
		TInstancedStruct<FPCGPerExecutionCacheData>* StructData = &Entry->Data.FindOrAdd(InCacheId.Guid);
		if (!StructData->IsValid())
		{
			if (bValidateWritable)
			{
				PCG_EXECUTION_CACHE_VALIDATION_CHECK(InSource);
			}

			typename T::ValueType EntryValue = InMakeEntry();

			// InMakeEntry() call can cause a reentrant GetOrCreateExecutionCacheValue() call which is supported because FPCGPerExecutionCache::Lock is a PCG::FRecursiveSharedLock 
			// but it can invalidate our cached pointers (Entry & StructData) so find them again before intializing the struct data.
			Entry = &Entries.FindChecked(TaskId);
			StructData = &Entry->Data.FindChecked(InCacheId.Guid);

			StructData->InitializeAs<T>(MoveTemp(EntryValue));
			
			UE_CLOGF(bEnableLogging, LogPCG, Log, "         [%ls] CACHE SET %ls",
				*InSource->GetExecutionState().GetDebugName(),
				InCacheId.DebugName ? InCacheId.DebugName : TEXT("unknown"));
		}
		return StructData->GetMutable<T>().GetValue();
	}
	return InMakeEntry();
}
