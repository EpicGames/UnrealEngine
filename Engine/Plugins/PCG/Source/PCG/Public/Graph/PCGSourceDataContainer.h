// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGModule.h"
#include "StructUtils/SharedStruct.h"

#include "PCGSourceDataContainer.generated.h"

// Concept for guaranteeing compatible script struct types for the source data types.
template <typename T>
concept CPCGSourceDataCompatibleType = requires { T::StaticStruct(); };

/**
 * This data storage key merges an associated label with a predefined hash to create a unique key.
 * Note: The Label can be used independently if desired, with the Hash left as 0.
 */
USTRUCT()
struct FPCGSourceDataStorageKey
{
	GENERATED_BODY()

	FPCGSourceDataStorageKey() = default;
	PCG_API FPCGSourceDataStorageKey(FName InLabel, uint64 InHash = 0u);

	bool operator==(const FPCGSourceDataStorageKey& Other) const = default;

	PCG_API friend uint32 GetTypeHash(const FPCGSourceDataStorageKey& Key);

	/** A label associated with this storage key. (Ex. "MyCustomToolData") */
	UPROPERTY()
	FName Label;

	/** An additional hash to associate with a unique identifier. */
	UPROPERTY()
	uint64 Hash = 0;
};

/** Type-erased storage value. Wraps a FSharedStruct payload for runtime verification and key collision avoidance. */
USTRUCT()
struct FPCGSourceDataStorageValue
{
	GENERATED_BODY()

	/** A FSharedStruct container of data to store within the Execution Source. */
	UPROPERTY()
	FSharedStruct Data;

	template <CPCGSourceDataCompatibleType T>
	FConstSharedStruct GetAs() const;

	template <CPCGSourceDataCompatibleType T>
	FSharedStruct GetAsMutable();
};

/**
 * A container for data storage on an Execution Source, supporting unique keying and Crc compatibility.
 * NOTE: This container is READ-ONLY at runtime.
 */
USTRUCT()
struct FPCGSourceDataContainer
{
	GENERATED_BODY()

	FPCGSourceDataContainer() = default;
	~FPCGSourceDataContainer() = default;
	// Required by USTRUCT
	// NOT THREAD SAFE: These must only be called from a single thread with no concurrent readers/writers.
	PCG_API FPCGSourceDataContainer(const FPCGSourceDataContainer& Other);
	PCG_API FPCGSourceDataContainer(FPCGSourceDataContainer&& Other);
	PCG_API FPCGSourceDataContainer& operator=(const FPCGSourceDataContainer& Other);
	PCG_API FPCGSourceDataContainer& operator=(FPCGSourceDataContainer&& Other);

	/** Custom serialization for the container. Serializes type-erased FSharedStruct payloads via FInstancedStruct. */
	bool Serialize(FArchive& Ar);

#if WITH_EDITOR
	/** Store data of type T within the storage container. */
	template <CPCGSourceDataCompatibleType T>
	void Store(const FPCGSourceDataStorageKey& DataKey, const T& Data);

	/** Remove the data attached to a specific Data Key. */
	PCG_API bool Remove(const FPCGSourceDataStorageKey& DataKey);

	/** Get a mutable shared reference to stored data by type. Returns an empty FSharedStruct if key or type doesn't exist. */
	template <CPCGSourceDataCompatibleType T>
	FSharedStruct GetMutable(const FPCGSourceDataStorageKey& DataKey);

	/** Mark the container dirty to invalidate the Crc cache. */
	PCG_API void MarkDirty();

	/** Set the container to automatically dirty to invalidate the Crc cache. */
	PCG_API void SetShouldAutoDirty(bool bShouldAutoDirty);

	/** Empty the contents of the container. */
	PCG_API void Empty();
#endif // WITH_EDITOR

	/** Get the dirty generation value for Crc combination. */
	PCG_API uint32 GetDirtyGeneration() const;

	/** Get the number of storage values. */
	PCG_API int32 Num() const;

	/** True if the storage is empty. */
	PCG_API bool IsEmpty() const;

	/** Report referenced UObjects from all stored shared structs to the GC. */
	PCG_API void AddStructReferencedObjects(FReferenceCollector& Collector) const;

	/** Get a const shared reference to stored data by type. Returns an empty FConstSharedStruct if key or type doesn't exist. */
	template <CPCGSourceDataCompatibleType T>
	FConstSharedStruct Get(const FPCGSourceDataStorageKey& DataKey) const;

private:
	UPROPERTY()
	TMap<FPCGSourceDataStorageKey, FPCGSourceDataStorageValue> DataStorage;

	/** Automatically increments the dirty generation to invalidate the Crc node cache. */
	TAtomic<bool> bAutoDirty = false;

	/** Mark the source data container dirty by incrementing this, which will invalidate the Crc cache. */
	UPROPERTY()
	uint32 DirtyGeneration = 0;

#if WITH_EDITOR
	mutable PCG::FSharedLock DataStorageLock;
#endif // WITH_EDITOR
};

template<>
struct TStructOpsTypeTraits<FPCGSourceDataContainer> : public TStructOpsTypeTraitsBase2<FPCGSourceDataContainer>
{
	enum
	{
		WithSerializer = true
	};
};

template <CPCGSourceDataCompatibleType T>
FConstSharedStruct FPCGSourceDataStorageValue::GetAs() const
{
#if WITH_EDITOR
	if (!(Data.GetPtr<T>() || !Data.IsValid()))
	{
		const FString RequestedType = T::StaticStruct()->GetName();
		const FString StoredType = Data.GetScriptStruct()->GetName();
		UE_LOGF(LogPCG, Error, "FPCGSourceDataStorageValue type mismatch: requested '%ls', stored '%ls'", *RequestedType, *StoredType);
		return {};
	}
#endif // WITH_EDITOR
	return FConstSharedStruct(Data);
}

template <CPCGSourceDataCompatibleType T>
FSharedStruct FPCGSourceDataStorageValue::GetAsMutable()
{
#if WITH_EDITOR
	if (!(Data.GetPtr<T>() || !Data.IsValid()))
	{
		const FString RequestedType = T::StaticStruct()->GetName();
		const FString StoredType = Data.GetScriptStruct()->GetName();
		UE_LOGF(LogPCG, Error, "FPCGSourceDataStorageValue type mismatch: requested '%ls', stored '%ls'", *RequestedType, *StoredType);
		return {};
	}
#endif // WITH_EDITOR
	return FSharedStruct(Data);
}

#if WITH_EDITOR
template <CPCGSourceDataCompatibleType T>
void FPCGSourceDataContainer::Store(const FPCGSourceDataStorageKey& DataKey, const T& Data)
{
	PCG::TUniqueScopeLock Lock(DataStorageLock);

	FPCGSourceDataStorageValue& Value = DataStorage.FindOrAdd(DataKey);
	Value.Data = FSharedStruct::Make(Data);

	if (bAutoDirty)
	{
		++DirtyGeneration;
	}
}

template <CPCGSourceDataCompatibleType T>
FSharedStruct FPCGSourceDataContainer::GetMutable(const FPCGSourceDataStorageKey& DataKey)
{
	PCG::TSharedScopeLock Lock(DataStorageLock);

	if (FPCGSourceDataStorageValue* StoredValue = DataStorage.Find(DataKey))
	{
		return StoredValue->GetAsMutable<T>();
	}

	return {};
}
#endif // WITH_EDITOR

template <CPCGSourceDataCompatibleType T>
FConstSharedStruct FPCGSourceDataContainer::Get(const FPCGSourceDataStorageKey& DataKey) const
{
#if WITH_EDITOR
	PCG::TSharedScopeLock Lock(DataStorageLock);
#endif // WITH_EDITOR

	if (const FPCGSourceDataStorageValue* Value = DataStorage.Find(DataKey))
	{
		return Value->GetAs<T>();
	}

	return {};
}
