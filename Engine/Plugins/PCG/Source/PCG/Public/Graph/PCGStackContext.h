// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCrc.h"

#include "Utils/PCGExtraCapture.h"

#include "Containers/Array.h"
#include "UObject/WeakObjectPtr.h"

#include "PCGStackContext.generated.h"

#define UE_API PCG_API

class FString;
class UObject;
class UPCGComponent;
class UPCGGraph;
class UPCGNode;
class UPCGPin;

/** A single frame of a call stack, represented as a pointer to the associated object (graph/subgraph or node) or a loop index. */
USTRUCT()
struct FPCGStackFrame
{
	GENERATED_BODY()

	FPCGStackFrame() = default;

	explicit FPCGStackFrame(const UObject* InObject)
	{
		SetObject(InObject);
	}

	explicit FPCGStackFrame(int32 InLoopIndex)
	{
		SetLoopIndex(InLoopIndex);
	}

	bool operator==(const FPCGStackFrame& Other) const { return Object == Other.Object && LoopIndex == Other.LoopIndex; }
	bool operator!=(const FPCGStackFrame& Other) const { return !(*this == Other); };

	friend uint32 GetTypeHash(const FPCGStackFrame& In) { return In.Hash; }

	// A valid frame should either point to an object or have a loop index >= 0.
	bool IsValid() const { return LoopIndex != INDEX_NONE || Object.IsValid(); }

	/** Loop index frames are injected for any dynamic subgraph invocation and have Object set explicitly null. */
	bool IsLoopIndexFrame() const { return Object.IsNull(); }

	void SetObject(const UObject* InObject)
	{
		Object = InObject;
		LoopIndex = INDEX_NONE;
		ComputeHash();
	}

	template <class T = UObject>
	const T* GetObject_AnyThread() const
	{
		FGCScopeGuard Guard;
		return Cast<T>(Object.Get());
	}

	template <class T = UObject>
	const T* GetObject_NoGuard() const
	{
		return Cast<T>(Object.Get());
	}

	template <class T = UObject>
	const T* GetObject_GameThread() const
	{
		check(IsInGameThread());
		return Cast<T>(Object.Get());
	}

	void SetLoopIndex(int32 InLoopIndex)
	{
		Object = nullptr;
		LoopIndex = InLoopIndex;
		ComputeHash();
	}

	UE_API void PostSerialize(const FArchive& Ar);

	// Stores object this frame refers to. Use SetObject to change this and properly update the hash.
	// Prefer using a variation of GetObject to avoid direct access.
	UPROPERTY()
	TSoftObjectPtr<const UObject> Object;

	// Stores the loop index (if any) that this frame refers to. Use SetLoopIndex to change this and properly update the hash.
	UPROPERTY()
	int32 LoopIndex = INDEX_NONE;

private:
	UE_API void ComputeHash();

	uint32 Hash = 0;
};

template<>
struct TStructOpsTypeTraits<FPCGStackFrame> : public TStructOpsTypeTraitsBase2<FPCGStackFrame>
{
	enum
	{
		WithPostSerialize = true,
	};
};

/** A call stack, represented as an array of stack frames. */
USTRUCT()
struct FPCGStack
{
	GENERATED_BODY()

	friend struct FPCGStackContext;

public:
#if WITH_EDITOR
	// Disable deprecation warnings on the rule of 5 because of the Timer member. To be removed when that that member is removed.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPCGStack() = default;
	FPCGStack(const FPCGStack&) = default;
	FPCGStack(FPCGStack&&) = default;
	FPCGStack& operator=(const FPCGStack&) = default;
	FPCGStack& operator=(FPCGStack&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

	/** Returns root graph execution id */
	FPCGTaskId GetGraphExecutionTaskId() const { return GraphExecutionTaskId; }

	/** Push frame onto top of stack. */
	void PushFrame(const FPCGStackFrame& Frame) { StackFrames.Add(Frame); }
	void PushFrame(const UObject* InFrameObject) { StackFrames.Emplace(InFrameObject); }
	void PushFrame(int32 FrameLoopIndex) { StackFrames.Emplace(FrameLoopIndex); }

	/** Pop frame from the stack. */
	UE_API void PopFrame();

	/** Construct a string version of this stack. Postfixed by optional node/pin if provided. */
	UE_API bool CreateStackFramePath(FString& OutString, const UPCGNode* InNode = nullptr, const UPCGPin* InPin = nullptr) const;

	/** Returns how many graphs the stack contains (top level graph stacks will return 1). */
	UE_API uint32 GetNumGraphLevels() const;

	/** Returns true if given stack is a prefix of this stack. */
	UE_API bool BeginsWith(const FPCGStack& Other) const;

	const TArray<FPCGStackFrame>& GetStackFrames() const { return StackFrames; }
	TArray<FPCGStackFrame>& GetStackFramesMutable() { return StackFrames; }

	/** Component given by first stack frame. */
	UE_DEPRECATED(5.7, "Use GetRootSource instead")
	UE_API const UPCGComponent* GetRootComponent() const;

	/** Execution source given by first stack frame. */
	UE_API const IPCGGraphExecutionSource* GetRootSource() const;

	/** First (top) graph frame in stack (or null if no graph frames present). */
	UE_API const UPCGGraph* GetRootGraph(int32* OutRootFrameIndex = nullptr) const;

	/** Returns true if this stack is the top level/root graph, rather than in a subgraph. */
	bool IsCurrentFrameInRootGraph() const { return GetNumGraphLevels() == 1; }

	/** Gets the graph from the graph frame closest to the top of the stack (most recent), or null if no such graph present. */
	UE_API const UPCGGraph* GetGraphForCurrentFrame() const;

	/** Walks up the stack to find nearest dynamic subgraph frame and if found returns the graph. */
	UE_API const UPCGGraph* GetNearestDynamicSubgraphForCurrentFrame() const;

	/** Walks up the stack to find the nearest non-inlined graph (either dynamic subgraph or the top graph). */
	UE_API const UPCGGraph* GetNearestNonInlinedGraphForCurrentFrame() const;

	/** If current frame (top of stack) corresponds to a node returns that node, otherwise returns null. */
	UE_API const UPCGNode* GetCurrentFrameNode() const;

	/** Gets the node from the node frame closest to the top of the stack (most recent), or null if no such node present. */
	UE_API const UPCGNode* GetNodeForCurrentFrame() const;

	/** Stack has a frame corresponding to the given object. */
	UE_API bool HasObject(const UObject* InObject) const;

	UE_DEPRECATED(5.6, "No longer supported")
	bool ReplaceRoot(const TMap<UObject*, UObject*>& ReplacementMap) { return false; }

	UE_API bool operator==(const FPCGStack& Other) const;
	bool operator!=(const FPCGStack& Other) const { return !(*this == Other); }

	friend uint32 GetTypeHash(const FPCGStack& In)
	{
		uint32 Hash = 0;

		for (const FPCGStackFrame& Frame : In.StackFrames)
		{
			Hash = HashCombine(Hash, GetTypeHash(Frame));
		}

		return Hash;
	}

	UE_API FPCGCrc GetCrc() const;

private:
	FPCGTaskId GraphExecutionTaskId = InvalidPCGTaskId;

	UPROPERTY()
	TArray<FPCGStackFrame> StackFrames;

public:
#if WITH_EDITOR
	// Used to store node & hierarchy information
	UE_DEPRECATED(5.5, "The timer has been moved to the FPCGContext struct.")
	PCGUtils::FCallTime Timer;
#endif
};

struct FPCGStackSharedPtr : public TSharedPtr<FPCGStack>
{
	using Super = TSharedPtr<FPCGStack>;
	using Super::operator bool;
	using Super::operator->;
	using Super::operator*;

	FPCGStackSharedPtr() = default;
	FPCGStackSharedPtr(TSharedPtr<FPCGStack>&& Other) : Super(Other) {}

	bool operator==(const FPCGStackSharedPtr& Other) const
	{
		if (IsValid() && Other.IsValid())
		{
			return *Get() == *Other.Get();
		}
		else
		{
			return IsValid() == Other.IsValid();
		}
	}

	bool operator!=(const FPCGStackSharedPtr& Other) const { return !operator==(Other); }

	friend uint32 GetTypeHash(const FPCGStackSharedPtr& In)
	{
		if (In)
		{
			return GetTypeHash(*In);
		}
		else
		{
			return 0;
		}
	}
};

/** A collection of call stacks. */
USTRUCT()
struct FPCGStackContext
{
	GENERATED_BODY()

	friend class FPCGGraphExecutor;

public:
	FPCGStackContext() = default;
	/** Builds a stack context by prepending the stack context by the common parent stack. */
	UE_API FPCGStackContext(const FPCGStackContext& InStackContext, const FPCGStack& InCommonParentStack);

	int32 GetNumStacks() const { return Stacks.Num(); }
	int32 GetCurrentStackIndex() const { return CurrentStackIndex; }
	UE_API const FPCGStack* GetStack(int32 InStackIndex) const;
	const TArray<FPCGStack>& GetStacks() const { return Stacks; }

	/** Create a new stack and create a frame from the provided object (typically graph or node pointer). Returns index of newly added stack. */
	UE_API int32 PushFrame(const UObject* InFrameObject);

	/** Remove a frame from the current stack. Returns current stack index. */
	UE_API int32 PopFrame();

	/** Takes the current stack and appends each of the stacks in InStacks. Called during compilation when inlining a static subgraph. */
	UE_API void AppendStacks(const FPCGStackContext& InStacks);

	/** Called during execution when invoking a dynamic subgraph, to prepend the caller stack to form the complete callstacks. */
	UE_API void PrependParentStack(const FPCGStack* InParentStack);

	TArray<FPCGStack>& GetStacksMutable() { return Stacks; }

	UE_API bool operator==(const FPCGStackContext& Other) const;

	FPCGTaskId GetGraphExecutionTaskId() const { return GraphExecutionTaskId; }

private:
	UE_API void SetGraphExecutionTaskId(FPCGTaskId InGraphExecutionTaskId);

	/** Unique graph execution id of this stack */
	FPCGTaskId GraphExecutionTaskId = InvalidPCGTaskId;

	/** List of all stacks encountered top graph and all (nested) subgraphs. Order is simply order of encountering during compilation. */
	UPROPERTY()
	TArray<FPCGStack> Stacks;

	/** Index of element in Stacks that is the current stack. */
	UPROPERTY()
	int32 CurrentStackIndex = INDEX_NONE;
};

#undef UE_API
