// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#include "CoreMinimal.h"
#include "Misc/Optional.h"
#include "UObject/SoftObjectPtr.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#include "PCGGridDescriptor.generated.h"

class FArchive;
class UExternalDataLayerAsset;

/**
* Descriptor struct used to determine where to output generated resources
*/
USTRUCT()
struct FPCGGridDescriptor
{
	GENERATED_BODY()

private:
	UPROPERTY()
	uint32 GridSize = 0;
		
	UPROPERTY()
	bool bIs2DGrid = true;

	UPROPERTY()
	bool bIsRuntime = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<TSoftObjectPtr<UDataLayerAsset>> DataLayerAssets;

	UPROPERTY()
	TSoftObjectPtr<UHLODLayer> HLODLayer;
#endif // WITH_EDITORONLY_DATA
		
	// Some descriptor properties do not exist in non editor builds / game worlds. This is the hash that represents those editor only properties.
	UPROPERTY()
	uint32 RuntimeHash = 0;

public:
	FPCGGridDescriptor& SetGridSize(uint32 InGridSize) { check(!Hash.IsSet()); GridSize = InGridSize; return *this; }
	uint32 GetGridSize() const { return GridSize; }

	bool IsUnboundedGrid() const { return GetGridSize() == PCGHiGenGrid::UnboundedGridSize(); }

	FPCGGridDescriptor& SetIs2DGrid(bool bInIs2DGrid) { check(!Hash.IsSet()); bIs2DGrid = bInIs2DGrid; return *this; }
	bool Is2DGrid() const { return bIs2DGrid; }

	FPCGGridDescriptor& SetIsRuntime(bool bInIsRuntime) { check(!Hash.IsSet()); bIsRuntime = bInIsRuntime; return *this; }
	bool IsRuntime() const { return bIsRuntime; }
		
	FPCGGridDescriptor& SetRuntimeHash(uint32 InRuntimeHash) { check(!Hash.IsSet()); RuntimeHash = InRuntimeHash; return *this; }
#if WITH_EDITOR
	uint32 GetRuntimeHash() const;

	FPCGGridDescriptor& SetDataLayerAssets(const TArray<TSoftObjectPtr<UDataLayerAsset>>& InDataLayerAssets);
	FPCGGridDescriptor& SetDataLayerAssets(const TArray<const UDataLayerAsset*>& InDataLayerAssets);
	const TArray<TSoftObjectPtr<UDataLayerAsset>>& GetDataLayerAssets() const { return DataLayerAssets; }

	void GetDataLayerAssets(TArray<TSoftObjectPtr<UDataLayerAsset>>& OutDataLayerAssets, const UExternalDataLayerAsset*& OutExternalDataLayerAsset) const;

	FPCGGridDescriptor& SetHLODLayer(const TSoftObjectPtr<UHLODLayer>& InHLODLayer) { check(!Hash.IsSet()); HLODLayer = InHLODLayer; return *this; }
	const TSoftObjectPtr<UHLODLayer>& GetHLODLayer() const { return HLODLayer; }
#endif // WITH_EDITOR

	bool operator==(const FPCGGridDescriptor& Other) const;
	bool operator!=(const FPCGGridDescriptor& Other) const { return !(*this == Other); }
		
	friend uint32 GetTypeHash(const FPCGGridDescriptor& Descriptor)
	{
		if (!Descriptor.Hash.IsSet())
		{
			Descriptor.Hash = Descriptor.ComputeHash();
		}

		return Descriptor.Hash.GetValue();
	}

	FString GetPartitionActorName(const FIntVector& GridCoords) const;

private:
	uint32 ComputeHash() const;

	mutable TOptional<uint32> Hash;

	// Deprecated section
public:
	UE_DEPRECATED(5.8, "Use GetGridSize")
	EPCGHiGenGrid GetHiGenGrid() const { return PCGHiGenGrid::GridSizeToGrid(GetGridSize()); }
};

/**
 * Describes one entry in a Grid
 */
USTRUCT()
struct FPCGGridCellDescriptor
{
	GENERATED_BODY()

	FPCGGridCellDescriptor() = default;
	FPCGGridCellDescriptor(const FPCGGridDescriptor& InDescriptor, const FIntVector& InGridCoords)
		: Descriptor(InDescriptor), GridCoords(InGridCoords)
	{}

	UPROPERTY()
	FPCGGridDescriptor Descriptor;

	/** The specific grid cell this actor lives in. */
	UPROPERTY(VisibleAnywhere, Category = Debug)
	FIntVector GridCoords = FIntVector::ZeroValue;

	bool operator==(const FPCGGridCellDescriptor& InOther) const;
	friend uint32 GetTypeHash(const FPCGGridCellDescriptor& In);
};

