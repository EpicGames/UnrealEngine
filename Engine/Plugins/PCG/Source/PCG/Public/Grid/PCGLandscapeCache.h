// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"

#include "Engine/World.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Serialization/BulkData.h"
#include "WorldPartition/WorldPartitionRuntimeCellInterface.h"

#include "PCGLandscapeCache.generated.h"

class ALandscapeProxy;
class FLandscapeProxyComponentDataChangedParams;
class ULandscapeComponent;
class ULandscapeInfo;
class UPCGLandscapeCache;
class UPCGPointData;
class UPCGMetadata;
struct FPCGPoint;

UENUM()
enum class EPCGLandscapeCacheSerializationMode : uint8
{
	SerializeOnlyAtCook,
	NeverSerialize,
	AlwaysSerialize
};

UENUM()
enum class EPCGLandscapeCacheSerializationContents : uint8
{
	SerializeOnlyPositionsAndNormals,
	SerializeOnlyLayerData,
	SerializeAll
};

USTRUCT(BlueprintType)
struct FPCGLandscapeLayerWeight
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Landscape Attribute")
	FName Name = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Landscape Attribute")
	float Weight = 0.0f;
};

namespace PCGLandscapeCache
{
	struct FSafeIndices
	{
		int32 X0Y0 = 0;
		int32 X1Y0 = 0;
		int32 X0Y1 = 0;
		int32 X1Y1 = 0;

		float XFraction = 0;
		float YFraction = 0;
	};

	// this will ensure all indicies are valid in a Stride*Stride sized array
	FSafeIndices CalcSafeIndices(FVector2D LocalPosition, int32 Stride);
}

struct FPCGLandscapeCacheEntry
{
	friend class UPCGLandscapeCache;

public:
	void GetPoint(int32 PointIndex, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;
	void GetPointHeightOnly(int32 PointIndex, FPCGPoint& OutPoint) const;
	void GetInterpolatedPoint(const FVector2D& LocalPoint, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;
	void GetInterpolatedPointMetadataOnly(const FVector2D& LocalPoint, int64& OutMetadataEntry, UPCGMetadata* OutMetadata) const;
	void GetInterpolatedPointHeightOnly(const FVector2D& LocalPoint, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const;
	void GetInterpolatedLayerWeights(const FVector2D& LocalPoint, TArray<FPCGLandscapeLayerWeight>& OutLayerWeights) const;

private:
	// Private API to remove boilerplate
	void GetInterpolatedPointInternal(const PCGLandscapeCache::FSafeIndices& Indices, FPCGPoint& OutPoint, bool bHeightOnly = false) const;
	void GetInterpolatedPointMetadataInternal(const PCGLandscapeCache::FSafeIndices& Indices, int64& OutMetadataEntry, UPCGMetadata* OutMetadata) const;

	// Private API for UPCGLandscapeCache usage
	bool TouchAndLoad(int32 Touch) const;
	void Unload();
	int32 GetMemorySize() const;

#if WITH_EDITOR
	static TSharedPtr<FPCGLandscapeCacheEntry> CreateCacheEntry(ULandscapeInfo* LandscapeInfo, ULandscapeComponent* InComponent);
#endif

	// Serialize called from the landscape cache
	void Serialize(FArchive& Ar, UObject* Owner, int32 BulkIndex, EPCGLandscapeCacheSerializationContents SerializationContents);

	// Internal usage methods
	void SerializeToBulkData(EPCGLandscapeCacheSerializationContents SerializationContents);
	bool SerializeFromBulkData() const;

	// Serialized data
	TArray<FName> LayerDataNames;
	FVector PointHalfSize = FVector::One();
	int32 Stride = 0;

	// Data built in editor or loaded from the bulk data
	mutable FByteBulkData BulkData;

	// Data stored in the BulkData
	mutable TArray<FVector> PositionsAndNormals;
	mutable TArray<TArray<uint8>> LayerData;

	// Transient data
	mutable PCG::FLock DataLock;
	mutable int32 Touch = 0;
	mutable bool bDataLoaded = false;
	TWeakObjectPtr<UPCGLandscapeCache> OwningCache = nullptr;
};

UCLASS()
class UPCGLandscapeCache : public UObject
{
	GENERATED_BODY()
public:
	//~Begin UObject interface
	virtual void BeginDestroy() override;
	virtual void Serialize(FArchive& Archive) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject interface

	/** Initialize cache. Can be safely called multiple times. */
	void Initialize();

	UFUNCTION(CallInEditor, Category = "Landscape Cache")
	void PrimeCache();

	UFUNCTION(CallInEditor, Category = "Landscape Cache")
	void ClearCache();

	void TakeOwnership(UPCGLandscapeCache* InLandscapeCache);

	void Tick(float DeltaSeconds);

	bool AreCacheEntriesReady(const ULandscapeInfo* InLandscapeInfo, const FIntPoint& InMinComponentKey, const FIntPoint& InMaxComponentKey);

	/** Gets landscape cache entry for specified component key */
	TSharedPtr<const FPCGLandscapeCacheEntry> GetCacheEntry(const ULandscapeInfo* InLandscapeInfo, const FIntPoint& InComponentKey, ALandscapeProxy* InLandscapeProxy = nullptr, bool bInAllowLoadOrCreate = true);

	/** Load/Create cache entries between Min/Max component key inclusively */
	void PrimeCache(const ULandscapeInfo* InLandscapeInfo, const FIntPoint& InMinComponentKey, const FIntPoint& InMaxComponentKey);

	TArray<FName> GetLayerNames(ALandscapeProxy* Landscape);

	/** Convenience method to get metadata from the landscape for a given pair of landscape and position */
	void SampleMetadataOnPoint(ALandscapeProxy* Landscape, const FTransform& InTransform, int64& OutMetadataEntry, UPCGMetadata* OutMetadata);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Landscape Cache")
	EPCGLandscapeCacheSerializationMode SerializationMode = EPCGLandscapeCacheSerializationMode::NeverSerialize;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Landscape Cache")
	EPCGLandscapeCacheSerializationContents CookedSerializedContents = EPCGLandscapeCacheSerializationContents::SerializeAll;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, BlueprintReadOnly, VisibleAnywhere, Category = "Landscape Cache")
	int32 CacheEntryCount = 0;
#endif

private:
#if WITH_EDITOR
	void SetupLandscapeCallbacks();
	void TeardownLandscapeCallbacks();
	void OnLandscapeChanged(ALandscapeProxy* InLandscape, const FLandscapeProxyComponentDataChangedParams& InChangeParams);
	void OnLandscapeMoved(AActor* InActor);
	void OnLandscapeAdded(AActor* Actor);
	void OnLandscapeDeleted(AActor* Actor);
	void OnLandscapeLoaded(AActor& Actor);
	void CacheLayerNames(ALandscapeProxy* InLandscape);
	void CacheLayerNames();
	void RemoveComponentFromCache_Unsafe(const ALandscapeProxy* LandscapeProxy);

	/** Gets (and creates if needed) the cache entry - available only in Editor */
	TSharedPtr<const FPCGLandscapeCacheEntry> GetOrCreateCacheEntryInternal(const ULandscapeInfo* InLandscapeInfo, const FIntPoint& InComponentKey, bool bInAllowLoadOrCreate, bool& bOutValidEntry);
#endif
	/** Gets landscape cache entry, works both in editor (but does not create) but works in game mode too. */
	TSharedPtr<const FPCGLandscapeCacheEntry> GetCacheEntryInternal(AActor* InHintActor, const FGuid& InLandscapeGuid, const FIntPoint& InComponentKey, bool bInAllowLoad);
	TSharedPtr<const FPCGLandscapeCacheEntry> GetCacheEntryInternal(const ULandscapeInfo* InLandscapeInfo, const FIntPoint& InComponentKey, ALandscapeProxy* InLandscapeProxy, bool bInAllowLoadOrCreate, bool& bOutValidEntry);

	struct CacheMapKey
	{
		FGuid LandscapeGuid;
		FIntPoint Coordinate;
		FObjectKey WorldKey;

		// Implementation note: FObjectKey will implicitly convert UObjects, including AActor.
		CacheMapKey(const FGuid& InLandscapeGuid, const FIntPoint& InCoordinate, const FObjectKey& InWorldKey)
			: LandscapeGuid(InLandscapeGuid), Coordinate(InCoordinate), WorldKey(InWorldKey)
		{}

		CacheMapKey(const FGuid& InLandscapeGuid, const FIntPoint& InCoordinate, const AActor* InHintActor)
			: LandscapeGuid(InLandscapeGuid), Coordinate(InCoordinate)
		{
			if (InHintActor && InHintActor->GetLevel())
			{
				if (InHintActor->GetLevel()->GetWorldPartitionRuntimeCell())
				{
					WorldKey = FObjectKey(InHintActor->GetLevel()->GetWorldPartitionRuntimeCell()->GetOuterWorld());
				}
				else
				{
					WorldKey = FObjectKey(InHintActor->GetTypedOuter<UWorld>());
				}
			}
		}

		friend inline uint32 GetTypeHash(const CacheMapKey& Key)
		{
			return HashCombine(HashCombine(GetTypeHash(Key.LandscapeGuid), GetTypeHash(Key.Coordinate)), GetTypeHash(Key.WorldKey));
		}

		bool operator==(const CacheMapKey& Rhs) const { return LandscapeGuid == Rhs.LandscapeGuid && Coordinate == Rhs.Coordinate && WorldKey == Rhs.WorldKey; }
	};

	void UpdateCacheWorldKeys();

	// Mapping of landscape guid + coordinates to entries. This is manually serialized as needed (depends on the serialize options).
	TMap<CacheMapKey, TSharedPtr<FPCGLandscapeCacheEntry>> CachedData;

	//TODO: separate by landscape
	UPROPERTY(VisibleAnywhere, Category = "Landscape Cache")
	TSet<FName> CachedLayerNames;

	std::atomic<int32> CacheMemorySize = 0;
	std::atomic<int32> CacheTouch = 0;
	float TimeSinceLastCleanupInSeconds = 0;
	bool bInitialized = false;
	bool bLoggedNoCacheError = false;

#if WITH_EDITOR
	TSet<TWeakObjectPtr<ALandscapeProxy>> Landscapes;
#endif

#if WITH_EDITOR
	PCG::FSharedLock CacheLock;
#endif
};
