// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorPartition/PartitionActor.h"

#include "PCGCommon.h"
#include "Grid/PCGGridDescriptor.h"

#include "PCGPartitionActor.generated.h"

#define UE_API PCG_API

namespace EEndPlayReason { enum Type : int; }

class UPCGComponent;
class UPCGSubsystem;
class UBoxComponent;

/** 
* The APCGPartitionActor actor is used to store grid cell data
* and its size will be a multiple of the grid size.
*/
UCLASS(MinimalAPI, NotBlueprintable, NotPlaceable)
class APCGPartitionActor : public APartitionActor
{
	GENERATED_BODY()

public:
	UE_API APCGPartitionActor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	//~Begin UObject Interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~End UObject Interface

	//~Begin AActor Interface
	UE_API virtual void BeginPlay();
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void GetActorBounds(bool bOnlyCollidingComponents, FVector& Origin, FVector& BoxExtent, bool bIncludeFromChildActors) const override;
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual void PostUnregisterAllComponents() override;
#if WITH_EDITOR
	UE_API virtual AActor* GetSceneOutlinerParent() const override;
	virtual bool ShouldExport() override { return false; }
	virtual bool ShouldImport(FStringView ActorPropString, bool IsMovingLevel) override { return false; }
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual bool IsSelectable() const override;
#endif
	//~End AActor Interface

#if WITH_EDITOR
	//~Begin APartitionActor Interface
	UE_API virtual uint32 GetDefaultGridSize(UWorld* InWorld) const override;
	virtual bool ShouldIncludeGridSizeInLabel() const override { return true; }
	UE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	UE_API virtual bool IsUserManaged() const override;
	//~End APartitionActor Interface

	// Called from PCGActorAndComponentMapping in case we are dealing with an older PCGPartitionActor
	UE_API void UpdateUse2DGridIfNeeded(bool bInUse2DGrid);
#endif

	UE_API FBox GetFixedBounds() const;
	UE_API FIntVector GetGridCoord() const;

	UFUNCTION(BlueprintCallable, Category = "PCG", meta= (DisplayName="Get PCG Grid Size"))
	int64 BP_GetPCGGridSize() const { return (int64)GetPCGGridSize(); }

	uint32 GetPCGGridSize() const { return PCGGridSize; }
	UE_API FPCGGridDescriptor GetGridDescriptor() const;

	bool IsUsing2DGrid() const { return bUse2DGrid; }

	/** Marks this PartitionActor as managed by the runtime generation system. */
	void SetToRuntimeGenerated() { bIsRuntimeGenerated = true; }
	bool IsRuntimeGenerated() const { return bIsRuntimeGenerated; }

	/** Forces the actor location to change even if its mobility is static. */
	UE_API bool Teleport(const FVector& NewLocation);

	/** Register with the PCG Subsystem. */
	UE_API void RegisterPCG();

	/** Unregister with the PCG Subsystem. */
	UE_API void UnregisterPCG();

	UE_API void AddGraphInstance(UPCGComponent* OriginalComponent);
	UE_API void RemapGraphInstance(const UPCGComponent* OldOriginalComponent, UPCGComponent* NewOriginalComponent);
	UE_API bool RemoveGraphInstance(UPCGComponent* OriginalComponent);

	// Repoint the local-component mappings when a local component is reinstanced (e.g. OnObjectsReplaced).
	void RemapLocalComponent(UPCGComponent* OldLocalComponent, UPCGComponent* NewLocalComponent);

	UE_DEPRECATED(5.6, "This function will be removed")
	UE_API void CleanupDeadGraphInstances(bool bRemoveNonNullOnly = false);

	// When a local component is destroyed. It calls this function. We make sure we don't keep mappings that are dead.
	UE_API void RemoveLocalComponent(UPCGComponent* LocalComponent);

	/** To be called after the creation of a new actor to set the grid guid and size. */
	UE_DEPRECATED(5.5, "Use FPCGGridDescriptor version")
	void PostCreation(const FGuid& InGridGUID, uint32 InGridSize) { }

	UE_API void PostCreation(const FPCGGridDescriptor& GridDescriptor);

	/** [Game thread only] Return if the actor is safe for deletion, meaning no generation is currently running on all original components. */
	UE_API bool IsSafeForDeletion() const;

	/** Whether there are any local PCGCompnents on this actor. */
	UE_API bool HasLocalPCGComponents() const;

	/** Return an array of all the PCGComponents on this actor */
	UE_API TSet<TObjectPtr<UPCGComponent>> GetAllLocalPCGComponents() const;

	/** Return a set of all the PCGComponents linked to this actor */
	UE_API TSet<TObjectPtr<UPCGComponent>> GetAllOriginalPCGComponents() const;

	/** Return true if this PA has any graph instances. */
	bool HasGraphInstances() const { return LocalToOriginal.Num() > 0; }

	/** Changes transient state for the local component matching the given original component. Returns true if PA becomes empty */
	UE_API bool ChangeTransientState(UPCGComponent* OriginalComponent, EPCGEditorDirtyMode EditingMode);

	// TODO: Make this in-editor only; during runtime, we should keep a map of component to bounds/volume only
	// and preferably precompute the intersection, so this would make it easier/possible to not have the original actor in game version.
	UFUNCTION(BlueprintCallable, Category = "PCG|PartitionActor")
	UE_API UPCGComponent* GetLocalComponent(const UPCGComponent* OriginalComponent) const;

	UFUNCTION(BlueprintCallable, Category = "PCG|PartitionActor")
	UE_API UPCGComponent* GetOriginalComponent(const UPCGComponent* LocalComponent) const;

	UE_DEPRECATED(5.5, "PCGGuid is deprecated")
	UPROPERTY()
	FGuid PCGGuid;

private:
	/** 
	 * Cleanup graph instances if Original or Local components are invalid. *
	 * 
	 * @param bCleanupTest					If true, we only test that there are some components to cleanup.
	 * @param OutRemovedLocalComponents		Returns the local components to cleanup if non-null and bCleanupTest is true.
	 * @return True if there are some components to cleanup (when bCleanupTest is true) or if actual cleanup happened (when bCleanupTest is false)
	 */
	UE_API bool CleanupDeadGraphInstancesInternal(bool bCleanupTest = false, TArray<UPCGComponent*>* OutRemovedLocalComponents = nullptr);

	UE_API UPCGSubsystem* GetSubsystem() const;

	friend class FPCGTrackingManager;
#if WITH_EDITOR
	UE_API void UpdateBoundsComponentExtents();
	UE_API void SetInvalidForPCG();

	friend struct FPCGWorldPartitionBuilder;
	UE_API TSoftObjectPtr<UPCGComponent> GetOriginalComponentSoftObjectPtr(UPCGComponent* LocalComponent) const;
#endif // WITH_EDITOR

	UE_API void RebuildOriginalToLocal();
	UE_API UPCGComponent* GetLocalComponent(const UPCGComponent* OriginalComponent, bool bRebuildMappingOnNullEntries) const;

	// Note: this map is not a property and not serialized since we will rebuild it from the LocalToOriginal
	mutable TMap<TObjectKey<UPCGComponent>, TObjectPtr<UPCGComponent>> OriginalToLocal;

	UPROPERTY(NonTransactional)
	TMap<TObjectPtr<UPCGComponent>, TSoftObjectPtr<UPCGComponent>> LocalToOriginal;

	// PCG components that are cleared when in preview-on-load mode are kept aside and put back when serializing to prevent changes
	UPROPERTY(Transient)
	TMap<TObjectPtr<UPCGComponent>, TSoftObjectPtr<UPCGComponent>> LoadedPreviewComponents;

	UPROPERTY(VisibleAnywhere, Category = WorldPartition)
	uint32 PCGGridSize;

	UPROPERTY(VisibleAnywhere, Category = WorldPartition)
	bool bUse2DGrid;

	UPROPERTY()
	uint32 RuntimeGridDescriptorHash = 0;

#if WITH_EDITORONLY_DATA
	/** Box component to draw the Partition actor bounds in the Editor viewport */
	UPROPERTY(Transient)
	TObjectPtr<UBoxComponent> BoundsComponent;

	/** Flag used to ignore some invalid actors so they don't get registered into the PCG Subsystem */
	UPROPERTY(Transient, NonTransactional)
	bool bIsInvalidForPCG = false;

	/** Set from PostLoad so that we can know if this actor needs to update its bUse2DGrid from the old APCGWorldActor::bUse2DGrid flag or not */
	bool bRequiresUse2DGridFixup = false;
#endif // WITH_EDITORONLY_DATA

	/** Tracks the registration status of this PA with the TrackingManager system. Helps us avoid invalid (un)registers. */
	bool bIsRegistered = false;

	/** Tracks if this actor was created by the Runtime Generation system. */
	bool bIsRuntimeGenerated = false;

	/** Utility bool to check if PostCreation/PostLoad was called. */
	bool bWasPostCreatedLoaded = false;

public:
	/** 
	 * Gets the name this partition actor should have.
	 * This does not respect traditional PA name contents like GridGuid, ShouldIncludeGridSizeInName, or ContextHash.
	 */
	UE_DEPRECATED(5.5, "Use FPCGGridDescriptor version")
	static UE_API FString GetPCGPartitionActorName(uint32 GridSize, const FIntVector& GridCoords, bool bRuntimeGenerated);

	static UE_API FString GetPCGPartitionActorName(const FPCGGridDescriptor& GridDescriptor, const FIntVector& GridCoords);

#if WITH_EDITOR
	bool IsInvalidForPCG() const { return bIsInvalidForPCG; }
#endif
};

DEFINE_ACTORDESC_TYPE(APCGPartitionActor, FPCGPartitionActorDesc);

#undef UE_API
