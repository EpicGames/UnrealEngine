// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "Grid/PCGGridDescriptor.h"
#include "UObject/ObjectMacros.h"
#include "WorldPartition/ActorPartition/PartitionActorDesc.h"

class FWorldPartitionActorDescInstance;

class FPCGPartitionActorDesc : public FPartitionActorDesc
{
public:
	FPCGPartitionActorDesc() = default;

	FPCGGridDescriptor GetGridDescriptor(const FWorldPartitionActorDescInstance* ActorDescInstance) const;
	FIntVector GetGridCoord() const;
protected:
	virtual void Init(const AActor* InActor) override;
	virtual uint32 GetSizeOf() const override { return sizeof(FPCGPartitionActorDesc); }
	virtual void Serialize(FArchive& Ar) override;

private:
	friend class FPCGTrackingManager;

	bool bUse2DGrid = true;
	bool bRequiresUse2DGridFixup = false;
	bool bInvalid = false;
};
#endif