// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Hash/PCGObjectHash.h"

class UPCGGraphInterface;
enum class EPCGChangeType : uint32;

class FPCGGraphHashContext : public FPCGObjectHashContext
{
public:
	explicit FPCGGraphHashContext(UPCGGraphInterface* InGraph);
	virtual ~FPCGGraphHashContext();
	
	static FPCGObjectHashContext* MakeInstance(UObject* InObject);

protected:
	void OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType);
};

#endif // WITH_EDITOR