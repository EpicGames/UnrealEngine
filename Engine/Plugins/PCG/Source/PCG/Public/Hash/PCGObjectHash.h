// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Delegates/Delegate.h"
#include "Hash/xxhash.h"
#include "Misc/Optional.h"
#include "UObject/WeakObjectPtr.h"

DECLARE_DELEGATE(FPCGObjectHashContextChanged);

/**
* IPCGObjectHashPolicy is an interface that allows defining specific rules for property hashing.
*/
class IPCGObjectHashPolicy
{
public:
	virtual ~IPCGObjectHashPolicy() = default;

	virtual bool ShouldHashProperty(const UObject* InObject, const FProperty* InProperty) const = 0;
};

/**
 * Base class to be specialized for specific UObject sub-classes that support PCG object hashing
 * 
 * Context is responsible for triggering event (OnChanged) that would invalidate the FPCGObjectHash and also provide logic for what properties need to be hashed.
 */
class FPCGObjectHashContext
{
public:
	explicit FPCGObjectHashContext(UObject* InObject);
	virtual ~FPCGObjectHashContext();

	bool ShouldHashProperty(const UObject* InObject, const FProperty* InProperty) const;
	bool ShouldHashSubObject(const UObject* InObject) const;

	template <class T = UObject>
	T* GetObject() const { return Cast<T>(Object.Get()); }

	bool IsValid() const { return !!GetObject(); }

	FPCGObjectHashContextChanged& OnChanged() { return Changed; }

protected:
	virtual bool ShouldHashPropertyInternal(const UObject* InObject, const FProperty* InProperty) const { return true; }
	virtual bool ShouldHashSubObjectInternal(const UObject* InObject) const { return true; }
	
	void AddHashPolicy(const IPCGObjectHashPolicy* InPolicy) { HashPolicies.Add(InPolicy); }
	virtual bool ShouldHashTransientProperties() const { return false; }

private:
	TWeakObjectPtr<UObject> Object;
	TArray<const IPCGObjectHashPolicy*> HashPolicies;
	FPCGObjectHashContextChanged Changed;
};

/**
* Simple hash policy which includes or excludes properties based on a UPROPERTY metadata tag.
*/
class FPCGObjectHashPolicyPropertyMetaDataFilter : public IPCGObjectHashPolicy
{
public:
	FPCGObjectHashPolicyPropertyMetaDataFilter(const FString& InMetaData, bool bInInclusionFilter)
		: MetaData(InMetaData), bInclusionFilter(bInInclusionFilter)
	{
	}

	virtual bool ShouldHashProperty(const UObject* InObject, const FProperty* InProperty) const override;

private:
	FString MetaData;
	bool bInclusionFilter;
};

/** 
* Simple hash policy which excludes every property with the 'PCGNoHash' metadata tag.
*/
class FPCGObjectHashPolicyPCGNoHash : public FPCGObjectHashPolicyPropertyMetaDataFilter
{
public:
	FPCGObjectHashPolicyPCGNoHash()
		: FPCGObjectHashPolicyPropertyMetaDataFilter(TEXT("PCGNoHash"), /*bInInclusionFilter=*/false)
	{
	}
};

/**
* Class used with a FPCGObjectHashContext specialization to hash an Object and its External Dependencies.
*/
class FPCGObjectHash final
{
public:
	using FHash = FXxHash64;

	explicit FPCGObjectHash(FPCGObjectHashContext* InContext);
	~FPCGObjectHash();

	FHash GetHash(bool bVerbose = false) const;

private:
	FHash GetHashInternal(bool bVerbose, uint32 Level) const;
	void InvalidateHash() { LocalHash.Reset(); }
	
	mutable TOptional<FHash> LocalHash;
	mutable TArray<TWeakObjectPtr<UObject>> ObjectReferences;

	FPCGObjectHashContext* Context = nullptr;
};

/**
* PCG Object hash context factory delegate.
*/ 
DECLARE_DELEGATE_RetVal_OneParam(FPCGObjectHashContext*, FPCGOnCreateObjectHashContext, UObject*);

/**
* Factory accessed through FPCGModule to register new supported Hash context types.
*/
class FPCGObjectHashFactory final
{
public:
	PCG_API void RegisterObjectHashContextFactory(UClass* InClass, FPCGOnCreateObjectHashContext InCreatePCGObjectHashContext);
	PCG_API FPCGObjectHash* GetOrCreateObjectHash(UObject* InObject) const;

private:
	TMap<FName, FPCGOnCreateObjectHashContext> CreateObjectHashContextPerClass;
};

#endif // WITH_EDITOR