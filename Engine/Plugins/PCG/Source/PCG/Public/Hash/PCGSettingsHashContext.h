// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Hash/PCGObjectHash.h"

class UPCGSettings;
class UPCGSettingsInterface;
enum class EPCGChangeType : uint32;

class FPCGSettingsHashContext : public FPCGObjectHashContext
{
public:
	FPCGSettingsHashContext(UPCGSettingsInterface* InSettings);
	virtual ~FPCGSettingsHashContext();

	static FPCGObjectHashContext* MakeInstance(UObject* InObject);

protected:
	void OnSettingsChanged(UPCGSettings* InSettings, EPCGChangeType ChangeType);
};

class FPCGSettingsHashPolicy : public FPCGObjectHashPolicyPCGNoHash
{
public:
	using Super = FPCGObjectHashPolicyPCGNoHash;

	FPCGSettingsHashPolicy() = default;

	virtual bool ShouldHashProperty(const UObject* InObject, const FProperty* InProperty) const override;
};

#endif // WITH_EDITOR