// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/Accessors/PCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

namespace PCG::IO
{
	namespace Constants
	{
		// Custom PCG version for attribute export
		struct FCustomExportVersion
		{
			enum Type
			{
				InitialVersion = 0,

				// New versions can be added above this line
				VersionPlusOne,
				LatestVersion = VersionPlusOne - 1
			};

			static FName GetFriendlyName() { return FName(TEXT("PCG::IO::AttributeExport")); }
			const static FGuid GUID;
		};

		namespace Attribute
		{
			struct UE_DEPRECATED(5.8, "Use PCG::IO::Constants::FCustomExportVersion") FCustomExportVersion : Constants::FCustomExportVersion {};
		}
	} // namespace Constants

	namespace Binary::Constants
	{
		static const FString Extension = TEXT(".bin");
	}

	namespace Helpers::String
	{
		void ToPrecisionString(const double Value, const int32 Precision, FString& OutString);
	}

	namespace Accessor
	{
		namespace Constants
		{
			static constexpr int32 CacheInlineAllocationCount = 16;
		}

		struct FCacheEntry
		{
			FPCGAttributePropertySelector Selector;
			TUniquePtr<const IPCGAttributeAccessor> Accessor = nullptr;
			TUniquePtr<const IPCGAttributeAccessorKeys> Keys = nullptr;
		};

		using FCache = TArray<FCacheEntry, TInlineAllocator<Constants::CacheInlineAllocationCount>>;
	} // namespace Accessor
} // namespace PCG::IO
