// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "Metadata/PCGMetadataAttributeTraits.h"

#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"

class FJsonValue;
class FJsonValueArray;
class UPCGDataViewData;
struct FPCGContext;
struct FPCGAttributePropertySelector;

namespace PCG::IO::Json
{
	template <typename T>
	concept CFloatingPoint = std::is_floating_point_v<std::remove_cv_t<T>>;

	namespace Constants
	{
		static const FString Extension = TEXT(".json");
		static const FString DefaultDataVersionKey = TEXT("data_version");
	}

	namespace Helpers
	{
		// Converts a floating point (float, double, FVector, FRotator, etc) type to a Json value.
		template <CFloatingPoint T>
		TSharedPtr<FJsonValue> ConvertFloatingPointType(const T& Value)
		{
			return MakeShared<FJsonValueNumber>(static_cast<double>(Value));
		}

		PCG_API TSharedPtr<FJsonValue> ConvertFloatingPointType(const FVector2D& Value);
		PCG_API TSharedPtr<FJsonValue> ConvertFloatingPointType(const FVector& Value);
		PCG_API TSharedPtr<FJsonValue> ConvertFloatingPointType(const FVector4& Value);
		PCG_API TSharedPtr<FJsonValue> ConvertFloatingPointType(const FQuat& Value);
		PCG_API TSharedPtr<FJsonValue> ConvertFloatingPointType(const FRotator& Value);
		PCG_API TSharedPtr<FJsonValue> ConvertFloatingPointType(const FTransform& Value);

		template <typename T>
		TSharedPtr<FJsonValue> ConvertFloatingPointType(const T&)
		{
			static_assert(!std::is_same_v<T, T>, "Unsupported type");
			return nullptr;
		}

		// Pass a JsonObject and set a field inside the object directly
		template <typename T>
		void SetValue(TSharedPtr<FJsonObject>& InOutJsonObject, FString&& ValueName, const T& Value)
		{
			check(InOutJsonObject);
			if constexpr (std::is_same_v<T, bool>)
			{
				InOutJsonObject->SetBoolField(std::forward<FString>(ValueName), Value);
			}
			else if constexpr (std::is_unsigned_v<std::remove_cv_t<T>> || std::is_signed_v<std::remove_cv_t<T>>)
			{
				InOutJsonObject->SetNumberField(std::forward<FString>(ValueName), Value);
			}
			else if constexpr (Private::MetadataTraits<T>::IsFloatingPoint)
			{
				InOutJsonObject->SetField(std::forward<FString>(ValueName), ConvertFloatingPointType(Value));
			}
			else if constexpr (std::is_same_v<T, FString>)
			{
				InOutJsonObject->SetStringField(std::forward<FString>(ValueName), Value);
			}
			else
			{
				InOutJsonObject->SetStringField(std::forward<FString>(ValueName), Private::MetadataTraits<T>::ToString(Value));
			}
		}

		UE_DEPRECATED(5.8, "AppendHeader is deprecated, please use AppendVersionToHeader instead.")
		void AppendHeader(const TSharedPtr<FJsonObject>& InOutJsonObject, int32 DataVersion, int32 CustomDataVersion = -1);

		// Append PCG Json data version header object to a Json object
		void AppendVersionToHeader(const TSharedPtr<FJsonObject>& InOutJsonObject, int32 DataVersion);

		// Append the selected properties/attributes to the provided Json object on an attribute-wise axis.
		void AppendSelectionByAttribute(TSharedPtr<FJsonObject>& InOutJsonObject, const UPCGData* InData, const TArray<FPCGAttributePropertySelector>& Selectors, const FPCGContext* InOptionalContext = nullptr);

		// Append the selected properties/attributes to the provided Json object on an element-wise axis.
		void AppendSelectionByElement(TSharedPtr<FJsonObject>& InOutJsonObject, const UPCGData* InData, const TArray<FPCGAttributePropertySelector>& Selectors, const FPCGContext* InOptionalContext = nullptr);

		// Helper to convert a Json object into a string
		FString ToJsonString(const TSharedPtr<FJsonObject>& JsonObject, bool bPrettyJson = false);

		// @todo_pcg: Add a fast track for AppendAllAttributes
	} // namespace Helpers
} // namespace PCG::IO::Json
