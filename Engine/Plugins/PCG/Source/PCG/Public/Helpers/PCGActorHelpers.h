// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCrc.h"
#include "Algo/Compare.h"
#include "Elements/PCGSplineMeshParams.h"
#include "MeshSelectors/PCGISMDescriptor.h"
#include "MeshSelectors/PCGSkinnedMeshDescriptor.h"
#include "Engine/EngineTypes.h"
#include "Engine/SplineMeshComponentDescriptor.h"
#include "Engine/World.h"
#include "Animation/AnimBank.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"

#include "PCGActorHelpers.generated.h"

class AActor;
class APCGIsolatedActor;
class IPCGGraphExecutionSource;
class UActorComponent;
class UDataLayerInstance;
class UHLODLayer;
class UInstancedStaticMeshComponent;
class UInstancedSkinnedMeshComponent;
class ULevel;
class UMaterialInterface;
class USplineMeshComponent;
class UPCGComponent;
class UPCGManagedISMComponent;
class UPCGManagedISKMComponent;
class UPCGManagedSkinnedMeshComponent;
class UPCGManagedSplineMeshComponent;
class UStaticMesh;
class UWorld;
struct FPCGContext;
struct FPCGGetExecutionSourcesFromSelectionKeyParams;
struct FPCGMoveResourceParams;

struct UE_DEPRECATED(5.5, "Use FPCGISMComponentBuilderParams instead.") FPCGISMCBuilderParameters
{
	FISMComponentDescriptor Descriptor;
	int32 NumCustomDataFloats = 0;
	bool bAllowDescriptorChanges = true;

	friend inline uint32 GetTypeHash(const FPCGISMCBuilderParameters& Key)
	{
		return HashCombine(GetTypeHash(Key.Descriptor), 1 + Key.NumCustomDataFloats);
	}

	inline bool operator==(const FPCGISMCBuilderParameters& Other) const { return Descriptor == Other.Descriptor && NumCustomDataFloats == Other.NumCustomDataFloats && bAllowDescriptorChanges == Other.bAllowDescriptorChanges; }
};

struct FPCGISMComponentBuilderParams
{
	FPCGISMComponentBuilderParams() = default;

	FPCGSoftISMComponentDescriptor Descriptor;
	int32 NumCustomDataFloats = 0;
	TConstArrayView<float> CustomPrimitiveData;
	FPCGCrc SettingsCrc;
	FPCGCrc DataCrc;
	bool bAllowDescriptorChanges = true;
	bool bTransient = false;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	explicit FPCGISMComponentBuilderParams(const FPCGISMCBuilderParameters& Params)
	: Descriptor(Params.Descriptor), NumCustomDataFloats(Params.NumCustomDataFloats), bAllowDescriptorChanges(Params.bAllowDescriptorChanges)
	{
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	friend inline uint32 GetTypeHash(const FPCGISMComponentBuilderParams& Key)
	{
		uint32 Result = GetTypeHash(Key.Descriptor);
		Result = HashCombine(Result, 1 + Key.NumCustomDataFloats);
		Result = HashCombine(Result, (Key.bAllowDescriptorChanges ? 2 : 1));
		Result = GetArrayHash(Key.CustomPrimitiveData.GetData(), Key.CustomPrimitiveData.Num(), Result);

		return Result;
	}

	inline bool operator==(const FPCGISMComponentBuilderParams& Other) const
	{
		return Descriptor == Other.Descriptor
			&& NumCustomDataFloats == Other.NumCustomDataFloats
			&& bAllowDescriptorChanges == Other.bAllowDescriptorChanges
			&& Algo::Compare(CustomPrimitiveData, Other.CustomPrimitiveData);
	}
};

struct FPCGSkinnedMeshComponentBuilderParams
{
	FPCGSkinnedMeshComponentBuilderParams() = default;

	FPCGSoftSkinnedMeshComponentDescriptor Descriptor;
	int32 NumCustomDataFloats = 0;
	FPCGCrc SettingsCrc;
	bool bTransient = false;

	friend inline uint32 GetTypeHash(const FPCGSkinnedMeshComponentBuilderParams& Key)
	{
		return HashCombine(GetTypeHash(Key.Descriptor), 1 + Key.NumCustomDataFloats);
	}

	inline bool operator==(const FPCGSkinnedMeshComponentBuilderParams& Other) const { return Descriptor == Other.Descriptor && NumCustomDataFloats == Other.NumCustomDataFloats; }
};

struct FPCGSplineMeshComponentBuilderParameters
{
	FSplineMeshComponentDescriptor Descriptor;
	FPCGSplineMeshParams SplineMeshParams;
	FPCGCrc SettingsCrc;

	friend inline uint32 GetTypeHash(const FPCGSplineMeshComponentBuilderParameters& Key)
	{
		return HashCombine(GetTypeHash(Key.Descriptor), GetTypeHash(Key.SplineMeshParams));
	}

	inline bool operator==(const FPCGSplineMeshComponentBuilderParameters& Other) const { return Descriptor == Other.Descriptor && SplineMeshParams == Other.SplineMeshParams; }
};

UCLASS(MinimalAPI, BlueprintType)
class UPCGActorHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static PCG_API UInstancedStaticMeshComponent* GetOrCreateISMC(AActor* InTargetActor, UPCGComponent* SourceComponent, uint64 SettingsUID, const FPCGISMCBuilderParameters& Params);
	static PCG_API UPCGManagedISMComponent* GetOrCreateManagedISMC(AActor* InTargetActor, UPCGComponent* SourceComponent, uint64 SettingsUID, const FPCGISMCBuilderParameters& Params);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.6, "Use signature with no SettingsUID. Make sure to fill Params.SettingsCRC to enable component reuse.")
	static PCG_API UInstancedStaticMeshComponent* GetOrCreateISMC(AActor* InTargetActor, UPCGComponent* SourceComponent, uint64 SettingsUID, const FPCGISMComponentBuilderParams& Params);
	UE_DEPRECATED(5.6, "Use signature with no SettingsUID. Make sure to fill Params.SettingsCRC to enable component reuse.")	
	static PCG_API UPCGManagedISMComponent* GetOrCreateManagedISMC(AActor* InTargetActor, UPCGComponent* SourceComponent, uint64 SettingsUID, const FPCGISMComponentBuilderParams& Params);
	UE_DEPRECATED(5.6, "Use signature with no SettingsUID. Make sure to fill Params.SettingsCRC to enable component reuse.")
	static PCG_API USplineMeshComponent* GetOrCreateSplineMeshComponent(AActor* InTargetActor, UPCGComponent* SourceComponent, uint64 SettingsUID, const FPCGSplineMeshComponentBuilderParameters& Params);
	UE_DEPRECATED(5.6, "Use signature with no SettingsUID. Make sure to fill Params.SettingsCRC to enable component reuse.")
	static PCG_API UPCGManagedSplineMeshComponent* GetOrCreateManagedSplineMeshComponent(AActor* InTargetActor, UPCGComponent* SourceComponent, uint64 SettingsUID, const FPCGSplineMeshComponentBuilderParameters& Params);
	
	static PCG_API UInstancedStaticMeshComponent* GetOrCreateISMC(AActor* InTargetActor, IPCGGraphExecutionSource* ExecutionSource, const FPCGISMComponentBuilderParams& Params, FPCGContext* OptionalContext = nullptr);
	static PCG_API UPCGManagedISMComponent* GetOrCreateManagedISMC(AActor* InTargetActor, IPCGGraphExecutionSource* ExecutionSource, const FPCGISMComponentBuilderParams& Params, FPCGContext* OptionalContext = nullptr);
	static PCG_API UPCGManagedISKMComponent* GetOrCreateManagedABMC(AActor* InTargetActor, IPCGGraphExecutionSource* ExecutionSource, const FPCGSkinnedMeshComponentBuilderParams& Params, FPCGContext* OptionalContext = nullptr);
	static PCG_API USplineMeshComponent* GetOrCreateSplineMeshComponent(AActor* InTargetActor, IPCGGraphExecutionSource* ExecutionSource, const FPCGSplineMeshComponentBuilderParameters& Params, FPCGContext* OptionalContext = nullptr);
	static PCG_API UPCGManagedSplineMeshComponent* GetOrCreateManagedSplineMeshComponent(AActor* InTargetActor, IPCGGraphExecutionSource* ExecutionSource, const FPCGSplineMeshComponentBuilderParameters& Params, FPCGContext* OptionalContext = nullptr);

	static PCG_API bool DeleteActors(UWorld* World, const TArray<TSoftObjectPtr<AActor>>& ActorsToDelete);

	template <typename T, typename = typename std::enable_if_t<std::is_base_of_v<AActor, T>>>
	inline static void ForEachActorInLevel(ULevel* Level, TFunctionRef<bool(AActor*)> Callback)
	{
		return ForEachActorInLevel(Level, T::StaticClass(), Callback);
	}

	/**
	* Iterate over all actors in the level, from the given class, and pass them to a callback
	* @param Level The level
	* @param ActorClass class of AActor to pass to the callback
	* @param Callback Function to call with the found actor. Needs to return a bool, to indicate if it needs to continue (true = yes)
	*/
	static PCG_API void ForEachActorInLevel(ULevel* Level, TSubclassOf<AActor> ActorClass, TFunctionRef<bool(AActor*)> Callback);

	template <typename T, typename = typename std::enable_if_t<std::is_base_of_v<AActor, T>>>
	inline static void ForEachActorInWorld(UWorld* World, TFunctionRef<bool(AActor*)> Callback)
	{
		return ForEachActorInWorld(World, T::StaticClass(), Callback);
	}

	/**
	* Iterate over all actors in the world, from the given class, and pass them to a callback
	* @param World The world
	* @param ActorClass class of AActor to pass to the callback
	* @param Callback Function to call with the found actor. Needs to return a bool, to indicate if it needs to continue (true = yes)
	*/
	static PCG_API void ForEachActorInWorld(UWorld* World, TSubclassOf<AActor> ActorClass, TFunctionRef<bool(AActor*)> Callback);

	/**
	* Spawn a new actor of type T and attach it to the parent (if not null)
	* @param World The world
	* @param Level The level to spawn into
	* @param BaseName Base name for the actor, will have a unique name
	* @param Transform The transform for the new actor
	* @param Parent Optional parent to attach to.
	*/
	template <typename T = AActor, typename = typename std::enable_if_t<std::is_base_of_v<AActor, T>>>
	inline static AActor* SpawnDefaultActor(UWorld* World, ULevel* Level, FName BaseName, const FTransform& Transform, AActor* Parent = nullptr)
	{
		return SpawnDefaultActor(World, Level, T::StaticClass(), BaseName, Transform, Parent);
	}

	/**
	* Spawn a new actor and attach it to the parent (if not null)
	* @param World The world
	* @param Level The level to spawn into
	* @param ActorClass Class of the actor to spawn
	* @param BaseName Base name for the actor, will have a unique name
	* @param Transform The transform for the new actor
	* @param Parent Optional parent to attach to.
	*/
	static PCG_API AActor* SpawnDefaultActor(UWorld* World, ULevel* Level, TSubclassOf<AActor> ActorClass, FName BaseName, const FTransform& Transform, AActor* Parent = nullptr);

	/**
	* Spawn a new actor and attach it to the parent (if not null)
	* @param World The world
	* @param Level The level to spawn into
	* @param ActorClass Class of the actor to spawn
	* @param Transform The transform for the new actor
	* @param SpawnParams The spawn parameters
	* @param Parent Optional parent to attach to.
	*/
	static PCG_API AActor* SpawnDefaultActor(UWorld* World, ULevel* Level, TSubclassOf<AActor> ActorClass, const FTransform& Transform, const FActorSpawnParameters& SpawnParams, AActor* Parent = nullptr);

	/**
	* Struct containing all parameters needed to spawn the actor
	*/
	struct FSpawnDefaultActorParams
	{
		FSpawnDefaultActorParams(UWorld* InWorld, TSubclassOf<AActor> InActorClass, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
			: World(InWorld), ActorClass(InActorClass), Transform(InTransform), SpawnParams(InSpawnParams)
		{
		}

		UWorld* World = nullptr;
		TSubclassOf<AActor> ActorClass;
		FTransform Transform;
		FActorSpawnParameters SpawnParams;
		AActor* Parent = nullptr;
		bool bForceStaticMobility = true;
		bool bIsPreviewActor = false;
		bool bIsRuntime = false;
#if WITH_EDITOR
		UHLODLayer* HLODLayer = nullptr;
		TArray<const UDataLayerInstance*> DataLayerInstances;
#endif
	};

	/**
	* Spawn a new actor
	* @param Params struct containing all the parameters needed to spawn the actor
	*/
	static PCG_API AActor* SpawnDefaultActor(const FSpawnDefaultActorParams& Params);
	
	/**
	 * Return the grid cell coordinates on the PCG partition grid given a position and the grid size.
	 */
	static PCG_API FIntVector GetCellCoord(FVector InPosition, uint32 InGridSize, bool bUse2DGrid);

	/**
	 * Return the center of the PCG partition grid cell given a position and the grid size.
	 */
	static PCG_API FVector GetCellCenter(const FVector& InPosition, uint32 InGridSize, bool bUse2DGrid);

	/** Extract the tags and the actor reference of the given actor and hash it. Useful for CRC dependencies that depends on the tags or the instance of the actor.*/
	static PCG_API int ComputeHashFromActorTagsAndReference(const AActor* InActor, const bool bIncludeTags, const bool bIncludeActorReference);

#if WITH_EDITOR
	static UPackage* CreatePreviewPackage(ULevel* InLevel, const FString& InActorName);
#endif

	/** Returns all isolated actors that originate from the given actor. */
	static TArray<APCGIsolatedActor*> GetIsolatedActorsOriginatingFrom(const AActor* InActor);
	
	/** Extracts (like Clear PCG Link) content from the actor, assuming it has a PCG component, into a PCG Isolated Actor. */
	static PCG_API AActor* IsolateFromActor(AActor* Actor, const TArray<FName>& RequiredTags);

	/** Extracts (like Clear PCG Link) content from the actor, assuming it has a PCG component, into a provided target actor or a PCG Isolated Actor created by this call. */
	static PCG_API AActor* IsolateFromActor(AActor* Actor, const FPCGMoveResourceParams& Params);

	/** Finds all execution sources using the passed in param as context for search. */
	static PCG_API TArray<IPCGGraphExecutionSource*> GetExecutionSourcesFromSelectionKey(const FPCGGetExecutionSourcesFromSelectionKeyParams& InParams);
};
