// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPin.h"

#include "PCGAssetHelpers.generated.h"

struct FAssetData;

// Struct to hold asset registry searchable data (w/ deprecation support) to be used in graphs and blueprints.
// Modeled after the pattern in PCG Data Assets.
USTRUCT(MinimalAPI)
struct FPCGAssetCachedPins
{
	GENERATED_BODY()

	/** Cached input pins to be able to know if a graph is compatible with a specific type without loading it first. */
	UPROPERTY()
	TArray<FPCGPinProperties> InputPins;

	/** Cached output pins to be able to know if a graph is compatible with a specific type without loading it first. */
	UPROPERTY()
	TArray<FPCGPinProperties> OutputPins;

	/** For deprecation, to know if the asset has cached pins. */
	UPROPERTY()
	bool bHasCachedPins = false;
};

#if WITH_EDITOR
class FPCGAssetHelpers
{
public:
	struct FGraphAssetOutput
	{
		FText Category;
		FText Description;
		FPCGAssetCachedPins CachedPins;
	};

	struct FBlueprintAssetOutput
	{
		FText Category;
		FText Description;
		FPCGAssetCachedPins CachedPins;
		FSoftClassPath GeneratedClass;
		bool bOnlyExposePreconfiguredSettings = false;
		bool bEnabledPreconfiguredSettings = false;
	};

	struct FSettingsAssetOutput
	{
		FText Category;
		FText Description;
		FPCGAssetCachedPins CachedPins;
	};

	static PCG_API bool GetGraphAssetRegistryData(const FAssetData& AssetData, FGraphAssetOutput& Output);
	static PCG_API bool GetBlueprintAssetRegistryData(const FAssetData& AssetData, FBlueprintAssetOutput& Output);
	static PCG_API bool GetSettingsAssetRegistryData(const FAssetData& AssetData, FSettingsAssetOutput& Output);
};
#endif