// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/PCGAsyncState.h"

#include "Async/Async.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/MpscQueue.h"
#include "Tasks/Task.h"
#include "Templates/Function.h"

#include <atomic>

struct FPCGContext;
struct FPCGPoint;

namespace FPCGAsync
{
	namespace ConsoleVar
	{
		extern PCG_API TAutoConsoleVariable<bool> CVarDisableAsyncTimeSlicing;
		extern PCG_API TAutoConsoleVariable<bool> CVarDisableAsyncTimeSlicingOnGameThread;
		extern PCG_API TAutoConsoleVariable<float> CVarAsyncOutOfTickBudgetInMilliseconds;
		extern PCG_API TAutoConsoleVariable<int32> CVarAsyncOverrideChunkSize;
	};

	/** 
	* Helper to do simple point processing loops
	* @param Context - The context containing the information about how many tasks to launch
	* @param NumIterations - The number of calls that will be done to the provided function, also an upper bound on the number of points generated
	* @param OutPoints - The array in which the results will be written to. Note that the array will be cleared before execution
	* @param PointFunc - A function that has the index [0; NumIterations] and has to write to the point & return true when the current call generates a point
	*/
	PCG_API void AsyncPointProcessing(FPCGContext* Context, int32 NumIterations, TArray<FPCGPoint>& OutPoints, const TFunction<bool(int32, FPCGPoint&)>& PointFunc);
	
	/** 
	* Helper to do simple point processing loops
	* @param Context - The context containing the information about how many tasks to launch
	* @param InPoints - The array in which the source points will be read from
	* @param OutPoints - The array in which the results will be written to. Note that the array will be cleared before execution
	* @param PointFunc - A function that has the input point and has to write to the output point & return true when the current call generates a point
	*/
	PCG_API void AsyncPointProcessing(FPCGContext* Context, const TArray<FPCGPoint>& InPoints, TArray<FPCGPoint>& OutPoints, const TFunction<bool(const FPCGPoint&, FPCGPoint&)>& PointFunc);

	/**
	* Helper to do simple point filtering loops
	* @param Context - The context containing the information about how many tasks to launch
	* @param NumIterations - The number of calls that will be done to the provided function, also an upper bound on the number of points generated
	* @param InFilterPoints - The array in which the in-filter results will be written to. Note that the array will be cleared before execution
	* @param OutFilterPoints - The array in which the out-filter results will be written to. Note that the array will be cleared before execution
	* @param PointFunc - A function that has the index [0; NumIterations] and has to write to the point & return true when the current call generates a point
	*/
	PCG_API void AsyncPointFilterProcessing(FPCGContext* Context, int32 NumIterations, TArray<FPCGPoint>& InFilterPoints, TArray<FPCGPoint>& OutFilterPoints, const TFunction<bool(int32, FPCGPoint&, FPCGPoint&)>& PointFunc);

	/**
	* Helper to do simple 1:N point processing loops
	* @param Context - The context containing the information about how many tasks to launch
	* @param NumIterations - The number of calls that will be done to the provided function, also an upper bound on the number of points generated
	* @param OutPoints - The array in which the results will be written to. Note that the array will be cleared before execution
	* @param PointFunc - A function that has the index [0; NumIterations] and has to write to the point & return true when the current call generates a point
	*/
	PCG_API void AsyncMultiPointProcessing(FPCGContext* Context, int32 NumIterations, TArray<FPCGPoint>& OutPoints, const TFunction<TArray<FPCGPoint>(int32)>& PointFunc);

	namespace Private
	{
		PCG_API bool AsyncProcessing(FPCGAsyncState& AsyncState, int32 NumIterations, TFunctionRef<void(void)> Initialize, TFunctionRef<int32(int32, int32, int32)> IterationInnerLoop, TFunctionRef<void(int32, int32, int32)> MoveDataRange, TFunctionRef<void(int32)> Finished, const bool bInEnableTimeSlicing, const int32 InChunkSize, const bool bAllowChunkSizeOverride);
	}

	/**
	* A Helper for generic parallel loops, with support for timeslicing, specialized to work on ranges. This version only uses indexes allowing more flexible usage
	* to process many arrays in parallel or use it for other batch updating.
	* 
	* Work will be separated in chunks, that will be processed in parallel. Main thread will then collapse incoming data from async tasks.
	* Will use AsyncState.ShouldStop() to stop execution if timeslicing is enabled.
	* Important info: 
	*   - We will finish to process and collapse data for all data already in process, even if we need to stop. To mitigate this, try to use small chunk sizes.
	*   - To avoid infinite loops (when we should stop even before starting working), we will at least process 1 chunk of data per thread.
	*   - To have async tasks, you need to have at least 3 available threads (main thread + 2 futures). Otherwise, we will only process on the main thread, without collapse.
	* 
	* @param AsyncState - The context containing the information about how many tasks we can launch, async read/write index for the current job and a function to know if we need to stop processing.
	* @param NumIterations - The number of calls that will be done to the provided function, also an upper bound on the number of data generated.
	* @param Initialize - Signature: void(). A function that will be called once on the first timeslice, where you can reserve data for processing
	* @param ProcessRange - Signature: int32(int32 StartReadIndex, int32 StartWriteIndex, int32 Count). A function that processes a range of values and returns the number of written values 
	* @param MoveDataRange - Signature: void(int32 ReadIndex, int32 WriteIndex, int32 Count). If the processing filters points, this will be used to move elements in chunk from one range to another 
	* @param Finished - Signature: void(int32 Count). Called once on finished, and tells you the total count of points written.
	* @param bEnableTimeSlicing - If false, we will not stop until all the processing is done.
	* @param ChunkSize - Size of the chunks to cut the input data with
	* @param bAllowChunkSizeOverride - If true, ChunkSize can be overridden by 'pcg.AsyncOverrideChunkSize' CVar
	* @returns true if the processing is done, false otherwise. Use this to know if you need to reschedule the task.
	*/
	template <typename InitializeFunc, typename ProcessRangeFunc, typename MoveDataRangeFunc, typename FinishedFunc>
	bool AsyncProcessingRangeEx(FPCGAsyncState* AsyncState, int32 NumIterations, InitializeFunc&& Initialize, ProcessRangeFunc&& ProcessRange, MoveDataRangeFunc&& MoveDataRange, FinishedFunc&& Finished, const bool bEnableTimeSlicing, const int32 ChunkSize = 64, const bool bAllowChunkSizeOverride = true)
	{
		if (AsyncState && !AsyncState->bIsRunningAsyncCall)
		{
			AsyncState->bIsRunningAsyncCall = true;
			const bool bIsDone = Private::AsyncProcessing(*AsyncState, NumIterations, Initialize, ProcessRange, MoveDataRange, Finished, bEnableTimeSlicing, ChunkSize, bAllowChunkSizeOverride);
			AsyncState->bIsRunningAsyncCall = false;
			return bIsDone;
		}
		else
		{
			// Can't use time slicing without an async state or while running in another async call (it will mess up with async indexes). 
			// We also force using one thread (the current one).
			FPCGAsyncState DummyState;
			DummyState.NumAvailableTasks = 1;
			DummyState.bIsRunningAsyncCall = true;
			return Private::AsyncProcessing(DummyState, NumIterations, Initialize, ProcessRange, MoveDataRange, Finished, /*bEnableTimeSlicing=*/false, ChunkSize, bAllowChunkSizeOverride);
		}
	}

	/**
	* A Helper for generic parallel loops, with support for timeslicing, specialized to work on ranges. This version only uses indexes allowing more flexible usage
	* to process many arrays in parallel or use it for other batch updating.
	* 
	* This version also does not support removing elements and is expects a 1:1 mapping. This just removes some lambdas required to support
	* that use case.
	* 
	* Work will be separated in chunks, that will be processed in parallel. Main thread will then collapse incoming data from async tasks.
	* Will use AsyncState.ShouldStop() to stop execution if timeslicing is enabled.
	* Important info: 
	*   - We will finish to process and collapse data for all data already in process, even if we need to stop. To mitigate this, try to use small chunk sizes.
	*   - To avoid infinite loops (when we should stop even before starting working), we will at least process 1 chunk of data per thread.
	*   - To have async tasks, you need to have at least 3 available threads (main thread + 2 futures). Otherwise, we will only process on the main thread, without collapse.
	* 
	* @param AsyncState - The context containing the information about how many tasks we can launch, async read/write index for the current job and a function to know if we need to stop processing.
	* @param NumIterations - The number of calls that will be done to the provided function, also an upper bound on the number of data generated.
	* @param Initialize - Signature: void(). A function that will be called once on the first timeslice, where you can reserve data for processing
	* @param Finished - Signature: void(int32 Count). Called once on finished, and tells you the total count of points written.
	* @param bEnableTimeSlicing - If false, we will not stop until all the processing is done.
	* @param ChunkSize - Size of the chunks to cut the input data with
	* @param bAllowChunkSizeOverride - If true, ChunkSize can be overridden by 'pcg.AsyncOverrideChunkSize' CVar
	* @returns true if the processing is done, false otherwise. Use this to know if you need to reschedule the task.
	*/
	template <typename InitializeFunc, typename ProcessRangeFunc>
	bool AsyncProcessingOneToOneRangeEx(FPCGAsyncState* AsyncState, int32 NumIterations, InitializeFunc&& Initialize, ProcessRangeFunc&& ProcessRange, const bool bEnableTimeSlicing, const int32 ChunkSize = 64, const bool bAllowChunkSizeOverride = true)
	{
		auto MoveDataRange = [](int32, int32, int32) { ensure(false); };
		auto Finished = [NumIterations](int32 Count) { ensure(NumIterations == Count); };

		if (AsyncState && !AsyncState->bIsRunningAsyncCall)
		{
			AsyncState->bIsRunningAsyncCall = true;
			const bool bIsDone = Private::AsyncProcessing(*AsyncState, NumIterations, Initialize, ProcessRange, MoveDataRange, Finished, bEnableTimeSlicing, ChunkSize, bAllowChunkSizeOverride);
			AsyncState->bIsRunningAsyncCall = false;
			return bIsDone;
		}
		else
		{
			// Can't use time slicing without an async state or while running in another async call (it will mess up with async indexes). 
			// We also force using one thread (the current one).
			FPCGAsyncState DummyState;
			DummyState.NumAvailableTasks = 1;
			DummyState.bIsRunningAsyncCall = true;
			return Private::AsyncProcessing(DummyState, NumIterations, Initialize, ProcessRange, MoveDataRange, Finished, /*bEnableTimeSlicing=*/false, ChunkSize, bAllowChunkSizeOverride);
		}
	}

	/**
	* A Helper for generic parallel loops, with support for timeslicing. This version only uses indexes allowing more flexible usage
	* to process many arrays in parallel or use it for other batch updating.
	* 
	* Work will be separated in chunks, that will be processed in parallel. Main thread will then collapse incoming data from async tasks.
	* Will use AsyncState.ShouldStop() to stop execution if timeslicing is enabled.
	* Important info: 
	*   - We will finish to process and collapse data for all data already in process, even if we need to stop. To mitigate this, try to use small chunk sizes.
	*   - To avoid infinite loops (when we should stop even before starting working), we will at least process 1 chunk of data per thread.
	*   - To have async tasks, you need to have at least 3 available threads (main thread + 2 futures). Otherwise, we will only process on the main thread, without collapse.
	* 
	* @param AsyncState - The context containing the information about how many tasks we can launch, async read/write index for the current job and a function to know if we need to stop processing.
	* @param NumIterations - The number of calls that will be done to the provided function, also an upper bound on the number of data generated.
	* @param Initialize - Signature: void(). A function that will be called once on the first timeslice, where you can reserve data for processing
	* @param ProcessElement - Signature: bool(int32 ReadIndex, int32 WriteIndex). A function that processes  
	* @param MoveData - Signature: void(int32 ReadIndex, int32 WriteIndex). If the processing filters points, this will be used to move elements from one index to another 
	* @param Finished - Signature: void(int32 Count). Called once on finished, and tells you the total count of points written.
	* @param bEnableTimeSlicing - If false, we will not stop until all the processing is done.
	* @param ChunkSize - Size of the chunks to cut the input data with 
	* @param bAllowChunkSizeOverride - If true, ChunkSize can be overridden by 'pcg.AsyncOverrideChunkSize' CVar
	* @returns true if the processing is done, false otherwise. Use this to know if you need to reschedule the task.
	*/
	template <typename InitializeFunc, typename ProcessElementFunc, typename MoveFunc, typename FinishedFunc>
	bool AsyncProcessingEx(FPCGAsyncState* AsyncState, int32 NumIterations, InitializeFunc&& Initialize, ProcessElementFunc&& ProcessElement, MoveFunc&& MoveData, FinishedFunc&& Finished, const bool bEnableTimeSlicing, const int32 ChunkSize = 64, const bool bAllowChunkSizeOverride = true)
	{
		auto IterationInnerLoop = [Func = MoveTemp(ProcessElement)](int32 StartReadIndex, int32 StartWriteIndex, int32 Count) -> int32
		{
			int32 NumPointsWritten = 0;

			for (int32 i = 0; i < Count; ++i)
			{
				if (Func(StartReadIndex + i, StartWriteIndex + NumPointsWritten))
				{
					++NumPointsWritten;
				}
			}

			return NumPointsWritten;
		};

		auto MoveDataRange = [Func = MoveTemp(MoveData)](int32 ReadIndex, int32 WriteIndex, int32 Count)
		{
			for (int Index = 0; Index < Count; ++Index)
			{
				Func(ReadIndex + Index, WriteIndex + Index);
			}
		};

		return AsyncProcessingRangeEx(AsyncState, NumIterations, Initialize, IterationInnerLoop, MoveDataRange, Finished, bEnableTimeSlicing, ChunkSize, bAllowChunkSizeOverride);
	}
 
 	/**
	* A Helper for generic parallel loops, with support for timeslicing. This version only uses indexes allowing more flexible usage
	* to process any data format.
	* 
	* This version also does not support removing elements and is expects a 1:1 mapping. This just removes some lambdas required to support
	* that use case.
	* 
	* Work will be separated in chunks, that will be processed in parallel. Main thread will then collapse incoming data from async tasks.
	* Will use AsyncState.ShouldStop() to stop execution if timeslicing is enabled.
	* Important info: 
	*   - We will finish to process and collapse data for all data already in process, even if we need to stop. To mitigate this, try to use small chunk sizes.
	*   - To avoid infinite loops (when we should stop even before starting working), we will at least process 1 chunk of data per thread.
	*   - To have async tasks, you need to have at least 3 available threads (main thread + 2 futures). Otherwise, we will only process on the main thread, without collapse.
	* 
	* @param AsyncState - The context containing the information about how many tasks we can launch, async read/write index for the current job and a function to know if we need to stop processing.
	* @param NumIterations - The number of calls that will be done to the provided function, also an upper bound on the number of data generated.
	* @param Initialize - Signature: void(). A function that will be called once on the first timeslice, where you can reserve data.
	* @param ProcessElement - Signature: bool(int32 ReadIndex, int32 WriteIndex). A function that processes  
	* @param bEnableTimeSlicing - If false, we will not stop until all the processing is done.
	* @param ChunkSize - Size of the chunks to cut the input data with
	* @param bAllowChunkSizeOverride - If true, ChunkSize can be overridden by 'pcg.AsyncOverrideChunkSize' CVar
	* @returns true if the processing is done, false otherwise. Use this to know if you need to reschedule the task.
	*/
 	template <typename InitializeFunc, typename ProcessElementFunc>
	bool AsyncProcessingOneToOneEx(FPCGAsyncState* AsyncState, int32 NumIterations, InitializeFunc&& Initialize, ProcessElementFunc&& ProcessElement, const bool bEnableTimeSlicing, const int32 ChunkSize = 64, const bool bAllowChunkSizeOverride = true)
	{
		auto IterationInnerLoop = [Func = MoveTemp(ProcessElement)](int32 StartReadIndex, int32 StartWriteIndex, int32 Count) -> int32
		{
			for (int32 i = 0; i < Count; ++i)
			{
				Func(StartReadIndex + i, StartWriteIndex + i);
			}

			return Count;
		};

		auto MoveDataRange = [](int32, int32, int32) { ensure(false); };
		auto Finished = [NumIterations](int32 Count) { ensure(NumIterations == Count); };

		if (AsyncState && !AsyncState->bIsRunningAsyncCall)
		{
			AsyncState->bIsRunningAsyncCall = true;
			const bool bIsDone = Private::AsyncProcessing(*AsyncState, NumIterations, Initialize, IterationInnerLoop, MoveDataRange, Finished, bEnableTimeSlicing, ChunkSize, bAllowChunkSizeOverride);
			AsyncState->bIsRunningAsyncCall = false;
			return bIsDone;
		}
		else
		{
			// Can't use time slicing without an async state or while running in another async call (it will mess up with async indexes). 
			// We also force using one thread (the current one).
			FPCGAsyncState DummyState;
			DummyState.NumAvailableTasks = 1;
			DummyState.bIsRunningAsyncCall = true;
			return Private::AsyncProcessing(DummyState, NumIterations, Initialize, IterationInnerLoop, MoveDataRange, Finished, /*bEnableTimeSlicing=*/false, ChunkSize, bAllowChunkSizeOverride);
		}
	}

	/**
	* Helper for generic parallel loops, with support for timeslicing. This version takes a data type, expecting processing on an
	* array of that type.
	* 
	* Work will be separated in chunks, that will be processed in parallel. Main thread will then collapse incoming data from async tasks.
	* Will use AsyncState.ShouldStop() to stop execution if timeslicing is enabled.
	* Important info: 
	*   - We will finish to process and collapse data for all data already in process, even if we need to stop. To mitigate this, try to use small chunk sizes.
	*   - To avoid infinite loops (when we should stop even before starting working), we will at least process 1 chunk of data per thread.
	*   - To have async tasks, you need to have at least 3 available threads (main thread + 2 futures). Otherwise, we will only process on the main thread, without collapse.
	* 
	* @param AsyncState - The context containing the information about how many tasks we can launch, async read/write index for the current job and a function to know if we need to stop processing.
	* @param NumIterations - The number of calls that will be done to the provided function, also an upper bound on the number of data generated.
	* @param OutData - The array in which the results will be written to. Note that the array will be cleared before execution.
	* @param Func - Signature: bool(int32, OutputType&). A function that has the index [0; NumIterations] and has to write to some data & return false if the result should be discarded. 
	* @param bEnableTimeSlicing - If false, we will not stop until all the processing is done.
	* @param ChunkSize - Size of the chunks to cut the input data with
	* @param bAllowChunkSizeOverride - If true, ChunkSize can be overridden by 'pcg.AsyncOverrideChunkSize' CVar
	* @returns true if the processing is done, false otherwise. Use this to know if you need to reschedule the task.
	*/
	template <typename OutputType, typename Func, typename AllocatorType = FDefaultAllocator>
	bool AsyncProcessing(FPCGAsyncState* AsyncState, int32 NumIterations, TArray<OutputType, AllocatorType>& OutData, Func&& InFunc, const bool bEnableTimeSlicing, const int32 ChunkSize = 64, bool bAllowChunkSizeOverride = true)
	{
		auto Initialize = [&OutData, NumIterations]()
		{
			// Array will be shrunk at the end of the processing.
			OutData.SetNumUninitialized(NumIterations, EAllowShrinking::No);
		};
		
		auto IterationInnerLoop = [Func = MoveTemp(InFunc), &OutData](int32 ReadIndex, int32 WriteIndex) -> int32
		{
			return Func(ReadIndex, OutData[WriteIndex]);
		};

		auto MoveData = [&OutData](int32 ReadIndex, int32 WriteIndex)
		{
			OutData[WriteIndex] = std::move(OutData[ReadIndex]);
		};

		auto Finished = [&OutData](int32 Count)
		{
			// Shrinking can have a big impact on the performance, but without it, we can also hold a big chunk of wasted memory.
			// Might revisit later if the performance impact is too big.
			OutData.SetNum(Count);
		};

		return AsyncProcessingEx(AsyncState, NumIterations, Initialize, IterationInnerLoop, MoveData, Finished, bEnableTimeSlicing, ChunkSize, bAllowChunkSizeOverride);
	}
}
