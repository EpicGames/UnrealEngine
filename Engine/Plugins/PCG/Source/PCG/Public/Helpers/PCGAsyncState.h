// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Templates/Tuple.h"

/**
* Helper class to gather information about asynchronus execution. Will be held in the PCGContext.
*/
struct FPCGAsyncState
{
	// Deprecating a property requires to have deprecation warning disable for the default copy ctor...
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPCGAsyncState() = default;
	~FPCGAsyncState() = default;

	FPCGAsyncState(const FPCGAsyncState&) = default;
	FPCGAsyncState(FPCGAsyncState&&) = default;

	FPCGAsyncState& operator=(const FPCGAsyncState&) = default;
	FPCGAsyncState& operator=(FPCGAsyncState&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** For timeslicing, it will be set by the graph executor to know when we should stop. */
	double EndTime = 0.0;

	/** How many tasks are available to run async. */
	int32 NumAvailableTasks = 0;

	UE_DEPRECATED(5.5, "Not used anymore")
	int32 AsyncCurrentReadIndex = 0;

	/** For async process, keep track of where the write processing is in output data. */
	int32 AsyncCurrentWriteIndex = 0;

	/** List of the current chunk and last chunk a given task have to process. There is one entry per future, and the last one is for the main thread. */
	TArray<TTuple<int32, int32>> TasksChunksStartEnd;

	/** Keep information if the async has started or not. */
	bool bStarted = false;

	/** Keep information on the initial number of iteration, so that if we have timeslicing we make sure that the number of iterations matches. */
	int32 InitialNumIterations = 0;

	/** Keep track of the current chunk we need to collapse. Important to collapse in order. */
	int32 CurrentChunkToCollapse = 0;

	/** Map between the processed chunks but not yet processed and the number of elements written for that chunk. */
	TMap<int32, int32> ChunkToNumElementsWrittenMap;

	/** For multithreading, track if the current element is run on the main thread. */
	bool bIsRunningOnMainThread = true;

	/** True if currently inside a PCGAsync scope - will prevent further async processing. */
	bool bIsRunningAsyncCall = false;

	/** True if the async process will call BP functions, which will require a BP context reset. */
	bool bIsCallingBlueprint = false;

	/** True if we can run out of tick */
	bool bIsRunningOutOfTick = false;

	/** True if out of tick buddget was set */
	bool bIsOutOfTickBudgetSet = false;

	/** Returns true if we reached end time. */
	PCG_API bool ShouldStop() const;

	/** To be called at the end of an async processing. */
	PCG_API void Reset();
};