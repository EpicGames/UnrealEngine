// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Math/Box.h"

#include "PCGBlueprintHelpers.generated.h"

class IPCGGraphExecutionSource;

class UPCGComponent;
class UPCGData;
class UPCGGraphInterface;
class UPCGParamData;
class UPCGSettings;

struct FPCGBlueprintContextHandle;
struct FPCGContext;
struct FPCGLandscapeLayerWeight;
struct FPCGPoint;

enum class EPCGGenerationStatus : uint8;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPCGOnGenerateGraphAsyncCompleted, EPCGGenerationStatus, Status);

UCLASS(MinimalAPI)
class UPCGGenerateGraphAsync : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()

	/**
	 * Triggers async generation of a PCG Standalone Graph.
	 * 
	 * @param Graph Graph to generate.
	 * @param Seed Seed used to generate graph.
	 */
	UFUNCTION(BlueprintCallable, Category="PCG|Helpers", meta = (BlueprintInternalUseOnly = "true", DisplayName = "Generate Graph Async"))
	static UPCGGenerateGraphAsync* GenerateGraphAsync(UPCGGraphInterface* Graph, int32 Seed = 42);

	/**
	 * Called when graph generation completes or aborts. 
	 */
	UPROPERTY(BlueprintAssignable)
	FPCGOnGenerateGraphAsyncCompleted OnCompleted;

	virtual void Activate() override;

private:
	void OnGenerationDone(IPCGGraphExecutionSource* InExecutionSource, EPCGGenerationStatus InStatus);

	UPROPERTY()
	TObjectPtr<UPCGGraphInterface> Graph = nullptr;

	UPROPERTY()
	int32 Seed = 42;
};

UCLASS(MinimalAPI)
class UPCGBlueprintHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	static PCG_API void ThrowBlueprintException(const FText& ErrorMessage);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers")
	static PCG_API int ComputeSeedFromPosition(const FVector& InPosition);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers")
	static PCG_API void SetSeedFromPosition(UPARAM(ref) FPCGPoint& InPoint);

	/** Creates a random stream from a point's seed and settings/component's seed (optional) */
	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API FRandomStream GetRandomStreamFromPoint(const FPCGPoint& InPoint, const UPCGSettings* OptionalSettings = nullptr, const UPCGComponent* OptionalComponent = nullptr);

	/** Creates a random stream from using the random seeds from two points, as well as settings/component's seed (optional) */
	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API FRandomStream GetRandomStreamFromTwoPoints(const FPCGPoint& InPointA, const FPCGPoint& InPointB, const UPCGSettings* OptionalSettings = nullptr, const UPCGComponent* OptionalComponent = nullptr);

	/** Prefer using GetSettingsWithContext */
	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API const UPCGSettings* GetSettings(UPARAM(ref) FPCGContext& Context);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API const UPCGSettings* GetSettingsWithContext(const FPCGBlueprintContextHandle& ContextHandle);

	/** Prefer using GetActorDataWithContext */
	UFUNCTION(BlueprintCallable, Category = "PCG|Temporary", meta = (ScriptMethod))
	static PCG_API UPCGData* GetActorData(UPARAM(ref) FPCGContext& Context);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API UPCGData* GetActorDataWithContext(const FPCGBlueprintContextHandle& ContextHandle);

	/** Prefer using GetInputDataWithContext */
	UFUNCTION(BlueprintCallable, Category = "PCG|Temporary", meta = (ScriptMethod))
	static PCG_API UPCGData* GetInputData(UPARAM(ref) FPCGContext& Context);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API UPCGData* GetInputDataWithContext(const FPCGBlueprintContextHandle& ContextHandle);

	/** Prefer using GetComponentWithContext */
	UFUNCTION(BlueprintCallable, Category = "PCG|Temporary", meta = (ScriptMethod))
	static PCG_API UPCGComponent* GetComponent(UPARAM(ref) FPCGContext& Context);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API UPCGComponent* GetComponentWithContext(const FPCGBlueprintContextHandle& ContextHandle);

	/** Prefer using GetOriginalComponentWithContext */
	UFUNCTION(BlueprintCallable, Category = "PCG|Temporary", meta = (ScriptMethod))
	static PCG_API UPCGComponent* GetOriginalComponent(UPARAM(ref) FPCGContext& Context);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API UPCGComponent* GetOriginalComponentWithContext(const FPCGBlueprintContextHandle& ContextHandle);

	/** Prefer using GetTargetActorWithContext */
	UFUNCTION(BlueprintCallable, Category = "PCG|Temporary", meta = (ScriptMethod))
	static PCG_API AActor* GetTargetActor(UPARAM(ref) FPCGContext& Context, UPCGSpatialData* SpatialData);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API AActor* GetTargetActorWithContext(const FPCGBlueprintContextHandle& ContextHandle, UPCGSpatialData* SpatialData);

	UFUNCTION(BlueprintCallable, Category = "PCG|Points", meta = (ScriptMethod))
	static PCG_API void SetExtents(UPARAM(ref) FPCGPoint& InPoint, const FVector& InExtents);

	UFUNCTION(BlueprintCallable, Category = "PCG|Points", meta = (ScriptMethod))
	static PCG_API FVector GetExtents(const FPCGPoint& InPoint);

	UFUNCTION(BlueprintCallable, Category = "PCG|Points", meta = (ScriptMethod))
	static PCG_API void SetLocalCenter(UPARAM(ref) FPCGPoint& InPoint, const FVector& InLocalCenter);

	UFUNCTION(BlueprintCallable, Category = "PCG|Points", meta = (ScriptMethod))
	static PCG_API FVector GetLocalCenter(const FPCGPoint& InPoint);

	UFUNCTION(BlueprintCallable, Category = "PCG|Points", meta = (ScriptMethod))
	static PCG_API FBox GetTransformedBounds(const FPCGPoint& InPoint);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API FBox GetActorBoundsPCG(AActor* InActor, bool bIgnorePCGCreatedComponents = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API FBox GetActorLocalBoundsPCG(AActor* InActor, bool bIgnorePCGCreatedComponents = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API UPCGData* CreatePCGDataFromActor(AActor* InActor, bool bParseActor = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (WorldContext="WorldContextObject"))
	static PCG_API TArray<FPCGLandscapeLayerWeight> GetInterpolatedPCGLandscapeLayerWeights(UObject* WorldContextObject, const FVector& Location);

	/** Prefer using GetTaskIdWithContext */
	UFUNCTION(BlueprintCallable, Category = "PCG|Temporary", meta = (ScriptMethod))
	static PCG_API int64 GetTaskId(UPARAM(ref) FPCGContext& Context);

	UFUNCTION(BlueprintCallable, Category = "PCG|Helpers", meta = (ScriptMethod))
	static PCG_API int64 GetTaskIdWithContext(const FPCGBlueprintContextHandle& ContextHandle);

	/** Flush the cache, to be used if you have changed something PCG depends on at runtime. Same as `pcg.FlushCache` command. Returns true if it succeeded. */
	UFUNCTION(BlueprintCallable, Category = "PCG", meta=(DisplayName = "Flush PCG Cache"))
	static PCG_API bool FlushPCGCache();

	/** Refresh a component set to Generate At Runtime, if some parameters changed. Can also flush the cache. */
	UFUNCTION(BlueprintCallable, Category = "PCG|Runtime", meta = (ScriptMethod, DisplayName = "Refresh PCG Runtime Component"))
	static PCG_API void RefreshPCGRuntimeComponent(UPCGComponent* InComponent, const bool bFlushCache = false);

	/** Prefer using DuplicateDataWithContext */
	UFUNCTION(BlueprintCallable, Category="PCG|Temporary", meta = (ScriptMethod))
	static PCG_API UPCGData* DuplicateData(const UPCGData* InData, UPARAM(ref) FPCGContext& Context, bool bInitializeMetadata = true);

	// Implementation note: Needs to be done outside of UPCGData because of circular dependency between PCGContext.h and PCGData.h
	/** Return a copy of the data, with Metadata inheritance for spatial data. */
	UFUNCTION(BlueprintCallable, Category = "PCG|Data", meta = (ScriptMethod))
	static PCG_API UPCGData* DuplicateDataWithContext(const UPCGData* InData, const FPCGBlueprintContextHandle& ContextHandle, bool bInitializeMetadata = true);

private:
	// ~ This method cannot be called outside of BP, use PCGPropertyHelpers::ExtractPropertyAsAttributeSet.
	/**
	 * Extract a given property from a structure into an Attribute Set. If the Property is None, will extract the full structure. Only works with a structure as input.
	 * @param Struct Structure to extract
	 * @param PropertyPath Path to the property in the structure to extract. Must be a valid property. If None, will extract the structure as input.
	 * @param OutputAttributeName If the property to extract is not a struct/object, can provide an override to the output attribute name. Use @Source to use the PropertyName.
	 * @param bExtract If the property path is not None and reference an object/struct, give the possibility to extract all its members in one go.
	 * @param bStrictlySanitizeNames Output attribute name are already sanitized by default but it keeps some characters like spaces.
	 * This is a stricter option and if the name is not following the object name rule (pretty much only alphanumerical and "_"), it will be sanitized.
	 * Necessary if we have cases with spaces, which doesn't play nicely with PCG attributes.
	 * @return Extracted param data. Can be null if something went wrong.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "PCG|Data", meta = (CustomStructureParam = "Struct", AllowAbstract = "false"))
	static PCG_API UPCGParamData* ExtractStructureAsAttributeSet(UPARAM(DisplayName = "Structure To Convert") const int32& Struct, const FName PropertyPath = NAME_None, const FName OutputAttributeName = "@Source", const bool bExtract = true, const bool bStrictlySanitizeNames = true);
	DECLARE_FUNCTION(execExtractStructureAsAttributeSet);
};
