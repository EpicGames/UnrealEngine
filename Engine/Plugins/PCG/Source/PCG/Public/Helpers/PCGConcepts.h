// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "UObject/Class.h"

namespace PCG::Concepts
{
	template <typename T>
	struct TIsStaticArray : std::false_type {};
	
	template <typename T, uint32 ...Extra>
	struct TIsStaticArray<TStaticArray<T, Extra...>> : std::true_type {};
	
	template <typename Container>
	concept CIsArrayLike = 
		TIsTArray_V<Container> 
		|| TIsTArrayView_V<Container>
		|| TIsStaticArray<Container>::value;

	template <typename LhsType, typename RhsType>
	concept CIsComparable = requires(const LhsType& Lhs, const RhsType& Rhs)
	{
		{ Lhs == Rhs } -> UE::CSameAs<bool>;
	};
	
	// Concept to catch all "basic types" that are also structs.
	template <typename T>
	concept CIsBasicStruct = requires (T)
	{
		TBaseStructure<T>::Get();
	};
}
