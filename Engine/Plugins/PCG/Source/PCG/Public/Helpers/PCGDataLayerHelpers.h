// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPin.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "Engine/EngineTypes.h"

class UDataLayerAsset;

#if WITH_EDITOR
class AActor;
class UDataLayerInstance;
class UPCGData;

struct FPCGDataLayerSettings;
struct FPCGContext;
#endif

#include "PCGDataLayerHelpers.generated.h"

namespace PCGDataLayerHelpers
{
	namespace Constants
	{
		const FName IncludedDataLayersAttribute = TEXT("IncludedDataLayers");
		const FName ExcludedDataLayersAttribute = TEXT("ExcludedDataLayers");
		const FName AddDataLayersAttribute = TEXT("AddDataLayers");
		const FName DataLayerReferenceAttribute = TEXT("DataLayerReference");
	}

#if WITH_EDITOR
	TArray<TSoftObjectPtr<UDataLayerAsset>> GetDataLayerAssetsFromInput(FPCGContext* Context, FName InputPinName, const FPCGDataLayerReferenceSelector& DataLayerSelector);
	TArray<TSoftObjectPtr<UDataLayerAsset>> GetDataLayerAssetsFromInput(FPCGContext* Context, FName InputPinName, const FPCGAttributePropertyInputSelector& InputSelector);
	TArray<FSoftObjectPath> GetDataLayerAssetsFromActorReferences(FPCGContext* Context, const UPCGData* ParamData, const FPCGAttributePropertyInputSelector& ActorReferenceAttribute);
	TArray<const UDataLayerInstance*> GetDataLayerInstancesAndCrc(FPCGContext* Context, const FPCGDataLayerSettings& DataLayerSettings, AActor* DefaulDataLayerSource, int32& OutCrc);
	TArray<const UDataLayerAsset*> GetDatalayerAssetsForActor(AActor* InActor);
#endif
};

UENUM()
enum class EPCGDataLayerSource : uint8
{
	Self,
	DataLayerReferences,
};

USTRUCT(BlueprintType)
struct FPCGDataLayerReferenceSelector
{
	GENERATED_BODY()

	/** Set it to true to get Data Layers through input attribute set. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	bool bAsInput = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default, meta = (EditCondition = "bAsInput", EditConditionHides))
	FPCGAttributePropertyInputSelector Attribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default, meta = (EditCondition = "!bAsInput", EditConditionHides))
	TArray<TSoftObjectPtr<UDataLayerAsset>> DataLayers;
};


USTRUCT(BlueprintType)
struct FPCGDataLayerSettings
{
	GENERATED_BODY()

	FPCGDataLayerSettings();

	/** What source should be used to assign Data Layers to the spawned actor */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	EPCGDataLayerSource DataLayerSourceType = EPCGDataLayerSource::Self;
		
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default, meta = (EditCondition = "DataLayerSourceType==EPCGDataLayerSource::DataLayerReferences", EditConditionHides, DisplayAfter = DataLayerSourceType))
	FPCGAttributePropertyInputSelector DataLayerReferenceAttribute;
		
	/** When left empty, all Data Layers from the Data Layer Source are included, if any Data Layers are specified, only those will be included */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	FPCGDataLayerReferenceSelector IncludedDataLayers;
		
	/** Specified Data Layers will get excluded from the Data Layer Source */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	FPCGDataLayerReferenceSelector ExcludedDataLayers;
		
	/** Specified Data Layers will get added */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	FPCGDataLayerReferenceSelector AddDataLayers;

	TArray<FPCGPinProperties> InputPinProperties() const;

#if WITH_EDITOR
	EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const;
#endif
};