// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/ArchiveCrc32.h"
#include "StructUtils/PropertyBag.h"

#include "PCGDefaultValueContainer.generated.h"

enum class EPCGMetadataTypes : uint8;
struct FPCGContext;
class UPCGParamData;

/** A struct used to store a default value locally for use with inline constant values. */
USTRUCT(BlueprintType)
struct FPCGDefaultValueContainer
{
	GENERATED_BODY()

	/** Creates a new FProperty with the specified type. Will overwrite if it already exists. Returns nullptr if the type is invalid. */
	const FProperty* CreateNewProperty(FName PropertyName, EPCGMetadataTypes Type);

	/** Finds an FProperty in the property bag. Returns nullptr if it does not exist. */
	const FProperty* FindProperty(FName PropertyName) const;

	/** Removes the property in the property bag, if it exists. */
	void RemoveProperty(FName PropertyName);

	/** Find the current metadata subtype of the specified FProperty. */
	EPCGMetadataTypes GetCurrentPropertyType(FName PropertyName) const;

	/** Get the current value of the property as a string. */
	FString GetPropertyValueAsString(FName PropertyName) const;

	/** Serialize the properties into a Crc32 to be used for caching purposes. */
	void SerializeCrc(FArchiveCrc32& Crc32) const;

	/** Helper that creates a Param Data with the typed property as an attribute attached to the metadata. */
	const UPCGParamData* CreateParamData(FPCGContext* Context, FName PropertyKey, FName AttributeName = NAME_None) const;

	/** The property exists and is set to active. */
	bool IsPropertyActivated(FName PropertyName) const;

#if WITH_EDITOR
	/** Change the metadata subtype for the specified FProperty. Will overwrite. */
	const FProperty* ConvertPropertyType(FName PropertyName, EPCGMetadataTypes Type);

	/** Sets the FProperty's value directly with a string input. Returns true if successful. */
	bool SetPropertyValueFromString(FName PropertyName, const FString& ValueString);

	/** Sets the property as currently active or inactive. Returns true if a change was made. */
	bool SetPropertyActivated(FName PropertyName, bool bIsActivated);

	/** Reset the container to empty. */
	void Reset();
#endif // WITH_EDITOR

private:
	/** A mapping of which properties are currently active. */
	UPROPERTY()
	TSet<FName> ActivatedProperties;

	/** The handling and storage of dynamic properties. */
	UPROPERTY()
	FInstancedPropertyBag PropertyBag;
};
