// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Elements/PCGActorSelector.h"
#include "Containers/Array.h"
#include "UObject/WeakInterfacePtr.h"

struct FPCGContext;
class IPCGGraphExecutionSource;

/**
* Simple helper class to factorize the logic for gathering dynamic tracking keys and pushing them to the component.
* Only work for settings that override CanDynamicalyTrackKeys.
*/
struct FPCGDynamicTrackingHelper
{
public:
	/** Enable dynamic tracking, will cache the weak ptr of the component and optionally resize the array for keys. */
	PCG_API void EnableAndInitialize(const FPCGContext* InContext, int32 OptionalNumElements = 0);

	/** Add the key to the tracking, will be uniquely added to the array. */
	PCG_API void AddToTracking(FPCGSelectionKey&& InKey, bool bIsCulled);

	/** Push all the tracked keys to the ceched component if still valid and the same as the context. */
	PCG_API void Finalize(const FPCGContext* InContext);

	/** Convenient function to push just a single tracking key to the component. */
	static PCG_API void AddSingleDynamicTrackingKey(FPCGContext* InContext, FPCGSelectionKey&& InKey, bool bIsCulled);

	/** Convenience function to push just a single selector as a tracking key to the component. */
	static PCG_API void AddSingleDynamicTrackingKey(FPCGContext* InContext, const FPCGActorSelectorSettings& InSelector);

private:
	bool bDynamicallyTracked = false;
	TWeakInterfacePtr<IPCGGraphExecutionSource> CachedExecutionSource;
	TArray<TPair<FPCGSelectionKey, bool>, TInlineAllocator<16>> DynamicallyTrackedKeysAndCulling;
};

#endif // WITH_EDITOR
