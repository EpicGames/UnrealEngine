// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"

#define UE_API PCG_API

/** Extension helper class to allow templatization without dirtying the PCGGraph.h file */
class FPCGGraphParameterExtension
{
	template <typename T> static bool constexpr StaticFail{false};

public:
	// Catch all implicit templates for unsupported types
	template <typename T>
	static TValueOrError<T, EPropertyBagResult> GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName) { static_assert(StaticFail<T>, "Invalid graph parameter type"); return MakeError(EPropertyBagResult::PropertyNotFound); }
	template <typename T>
	static EPropertyBagResult SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const T& Value) { static_assert(StaticFail<T>, "Invalid graph parameter type"); return EPropertyBagResult::PropertyNotFound; }

	// Enum Objects are a special signature
	static UE_API TValueOrError<uint64, EPropertyBagResult> GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName, const UEnum* Enum);
	static UE_API EPropertyBagResult SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const uint64& Value, const UEnum* Enum);
};

// Template declarations necessary for DLL API
template<> PCG_API TValueOrError<float, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<double, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<bool, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<uint8, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<int32, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<int64, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<FName, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<FString, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<FSoftObjectPath, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<FSoftClassPath, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<TSoftObjectPtr<UObject>, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<TSoftClassPtr<UObject>, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<UObject*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<UClass*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<FVector*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<FRotator*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<FTransform*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<FVector4*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<FVector2D*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);
template<> PCG_API TValueOrError<FQuat*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);

template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const float& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const double& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const bool& bValue);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const uint8& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const int32& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const int64& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FName& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FString& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FSoftObjectPath& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FSoftClassPath& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const TSoftObjectPtr<UObject>& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const TSoftClassPtr<UObject>& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, UObject* const& Value); // UObject* const& to respect template specialization
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, UClass* const& Value); // UClass* const& to respect template specialization
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FVector& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FRotator& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FTransform& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FVector4& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FVector2D& Value);
template<> PCG_API EPropertyBagResult FPCGGraphParameterExtension::SetGraphParameter(FInstancedPropertyBag& PropertyBag, const FName PropertyName, const FQuat& Value);


template<> 
UE_DEPRECATED(5.7, "This function is not working properly and the non-pointer version should be used instead.")
PCG_API TValueOrError<FSoftObjectPath*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);

template<> 
UE_DEPRECATED(5.7, "This function is not working properly and the non-pointer version should be used instead.")
PCG_API TValueOrError<FSoftClassPath*, EPropertyBagResult> FPCGGraphParameterExtension::GetGraphParameter(const FInstancedPropertyBag& PropertyBag, const FName PropertyName);

#undef UE_API
