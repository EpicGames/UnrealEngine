// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "PCGGraphParametersHelpers.generated.h"

#define UE_API PCG_API

enum class EPropertyBagAlterationResult : uint8;

class UPCGGraphInterface;

namespace PCGGraphParameter::Helpers
{
	PCG_API bool GenerateUniqueName(const UPCGGraphInterface* InGraph, FName& InOutName);

	PCG_API void LogGraphParamNamingErrors(FName IntendedName, const EPropertyBagAlterationResult Result, const FPCGContext* InContext = nullptr);
}

/**
* Blueprint Library to get or set graph parameters on graphs and graph instances
*/
UCLASS(MinimalAPI)
class UPCGGraphParametersHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API bool IsOverridden(const UPCGGraphInterface* GraphInterface, const FName Name);

	////////////
	// Getters
	////////////

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API float GetFloatParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API double GetDoubleParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API bool GetBoolParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API uint8 GetByteParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API int32 GetInt32Parameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API int64 GetInt64Parameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API FName GetNameParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API FString GetStringParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API uint8 GetEnumParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API FSoftObjectPath GetSoftObjectPathParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API TSoftObjectPtr<UObject> GetSoftObjectParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API TSoftClassPtr<UObject> GetSoftClassParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API UObject* GetObjectParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API UClass* GetClassParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API FVector GetVectorParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API FRotator GetRotatorParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API FTransform GetTransformParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API FVector4 GetVector4Parameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API FVector2D GetVector2DParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API FQuat GetQuaternionParameter(const UPCGGraphInterface* GraphInterface, const FName Name);

	////////////
	// Setters
	////////////

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetFloatParameter(UPCGGraphInterface* GraphInterface, const FName Name, const float Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetDoubleParameter(UPCGGraphInterface* GraphInterface, const FName Name, const double Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetBoolParameter(UPCGGraphInterface* GraphInterface, const FName Name, const bool bValue);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetByteParameter(UPCGGraphInterface* GraphInterface, const FName Name, const uint8 Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetInt32Parameter(UPCGGraphInterface* GraphInterface, const FName Name, const int32 Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetInt64Parameter(UPCGGraphInterface* GraphInterface, const FName Name, const int64 Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetNameParameter(UPCGGraphInterface* GraphInterface, const FName Name, const FName Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetStringParameter(UPCGGraphInterface* GraphInterface, const FName Name, const FString Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetEnumParameter(UPCGGraphInterface* GraphInterface, const FName Name, const uint8 Value, const UEnum* Enum = nullptr);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetSoftObjectPathParameter(UPCGGraphInterface* GraphInterface, const FName Name, UPARAM(ref) const FSoftObjectPath& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetSoftObjectParameter(UPCGGraphInterface* GraphInterface, const FName Name, UPARAM(ref) const TSoftObjectPtr<UObject>& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetSoftClassParameter(UPCGGraphInterface* GraphInterface, const FName Name, UPARAM(ref) const TSoftClassPtr<UObject>& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetObjectParameter(UPCGGraphInterface* GraphInterface, const FName Name, UObject* Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetClassParameter(UPCGGraphInterface* GraphInterface, const FName Name, UClass* Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetVectorParameter(UPCGGraphInterface* GraphInterface, const FName Name, UPARAM(ref) const FVector& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetRotatorParameter(UPCGGraphInterface* GraphInterface, const FName Name, UPARAM(ref) const FRotator& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetTransformParameter(UPCGGraphInterface* GraphInterface, const FName Name, UPARAM(ref) const FTransform& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetVector4Parameter(UPCGGraphInterface* GraphInterface, const FName Name, UPARAM(ref) const FVector4& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetVector2DParameter(UPCGGraphInterface* GraphInterface, const FName Name, UPARAM(ref) const FVector2D& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Graph Parameters")
	static UE_API void SetQuaternionParameter(UPCGGraphInterface* GraphInterface, const FName Name, UPARAM(ref) const FQuat& Value);
};

#undef UE_API
