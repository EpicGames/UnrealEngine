// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"

class UHLODLayer;

class AActor;

struct FPCGContext;
struct FPCGHLODSettings;

#include "PCGHLODHelpers.generated.h"

namespace PCGHLODHelpers
{
	namespace Constants
	{
		const FName HLODLayerAttribute = TEXT("HLODLayer");
	}

#if WITH_EDITOR
	UHLODLayer* GetHLODLayerAndCrc(FPCGContext* Context, const FPCGHLODSettings& HLODSettings, AActor* DefaultHLODLayerSource, AActor* TemplateActor, int32& OutCrc);
#endif
};

UENUM()
enum class EPCGHLODSource : uint8
{
	Self,
	Reference,
	Template,
};

USTRUCT(BlueprintType)
struct FPCGHLODSettings
{
	GENERATED_BODY()

	/** What source should be used to assign HLOD Layer to the spawned actor */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default)
	EPCGHLODSource HLODSourceType = EPCGHLODSource::Self;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Default, meta = (EditCondition = "HLODSourceType==EPCGHLODSource::Reference", EditConditionHides))
	TObjectPtr<UHLODLayer> HLODLayer;
};