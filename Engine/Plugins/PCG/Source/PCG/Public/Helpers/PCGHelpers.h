// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGPoint.h"
#include "Metadata/PCGMetadataCommon.h"

#include "EngineDefines.h" // For UE_ENABLE_DEBUG_DRAWING
#include "Containers/Ticker.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Math/Box.h"
#include "Math/RandomStream.h"

#include "PCGHelpers.generated.h"

class AActor;
class APCGWorldActor;
class ALandscape;
class ALandscapeProxy;
class IPCGGraphExecutionSource;
class UPCGComponent;
class UPCGGraph;
class UPCGData;
class UPCGMetadata;
class UPCGSettings;
class UWorld;
struct FPCGContext;
struct FPCGMetadataDomainID;
struct FPCGPoint;

namespace PCGHelpers
{
	/** Tag that will be added on every component generated through the PCG system */
	const FName DefaultPCGTag = TEXT("PCG Generated Component");
	const FName DefaultPCGDebugTag = TEXT("PCG Generated Debug Component");
	const FName DefaultPCGActorTag = TEXT("PCG Generated Actor");
	const FName MarkedForCleanupPCGTag = TEXT("PCG Marked For Cleanup");

	PCG_API int ComputeSeed(int A);
	PCG_API int ComputeSeed(int A, int B);
	PCG_API int ComputeSeed(int A, int B, int C);
	PCG_API int ComputeSeedFromPosition(const FVector& InPosition);
	PCG_API FRandomStream GetRandomStreamFromSeed(int32 Seed, const UPCGSettings* OptionalSettings = nullptr, const IPCGGraphExecutionSource* OptionalExecutionSource = nullptr);
	PCG_API FRandomStream GetRandomStreamFromTwoSeeds(int32 SeedA, int32 SeedB, const UPCGSettings* OptionalSettings = nullptr, const IPCGGraphExecutionSource* OptionalExecutionSource = nullptr);

	PCG_API bool IsInsideBounds(const FBox& InBox, const FVector& InPosition);
	PCG_API bool IsInsideBoundsXY(const FBox& InBox, const FVector& InPosition);

	PCG_API FBox OverlapBounds(const FBox& InBoxA, const FBox& InBoxB);

	/** Returns the bounds of InActor, intersected with the execution source if InActor is a partition actor */
	PCG_API FBox GetGridBounds(const AActor* InActor, const IPCGGraphExecutionSource* InSource);

	PCG_API FBox GetActorBounds(const AActor* InActor, bool bIgnorePCGCreatedComponents = true);
	PCG_API FBox GetActorLocalBounds(const AActor* InActor, bool bIgnorePCGCreatedComponents = true);
	PCG_API FBox GetLandscapeBounds(const ALandscapeProxy* InLandscape);

	PCG_API ALandscape* GetLandscape(UWorld* InWorld, const FBox& InActorBounds);
	PCG_API TArray<TWeakObjectPtr<ALandscapeProxy>> GetLandscapeProxies(UWorld* InWorld, const FBox& InActorBounds);
	PCG_API TArray<TWeakObjectPtr<ALandscapeProxy>> GetAllLandscapeProxies(UWorld* InWorld);

	PCG_API bool IsRuntimeOrPIE();

	PCG_API APCGWorldActor* GetPCGWorldActor(UWorld* InWorld);
	PCG_API APCGWorldActor* FindPCGWorldActor(UWorld* InWorld);

	UE_DEPRECATED(5.5, "This function was incorrectly parsing by whitespace. Please use 'GetStringArrayFromCommaSeparatedList' instead.")
	PCG_API TArray<FString> GetStringArrayFromCommaSeparatedString(const FString& InCommaSeparatedString, const FPCGContext* InOptionalContext = nullptr);
	PCG_API TArray<FString> GetStringArrayFromCommaSeparatedList(const FString& InCommaSeparatedString);

#if WITH_EDITOR
	PCG_API void GatherDependencies(UObject* Object, TSet<TObjectPtr<UObject>>& OutDependencies, int32 MaxDepth = -1, const TArray<UClass*>& InExcludedClasses = {});
	PCG_API void GatherDependencies(FProperty* Property, const void* InContainer, TSet<TObjectPtr<UObject>>& OutDependencies, int32 MaxDepth, const TArray<UClass*>& InExcludedClasses = {});

	/** Returns true if it's running a World Partition builder commandlet. */
	PCG_API bool IsInWorldPartitionBuilderCommandletContext();
#endif

	/** 
	* Check if an object is a new object and not the CDO.
	*
	* Some objects might not have the appropriate flags if they are embedded inside of other objects. 
	* Use the bCheckHierarchy flag to true to go up the object hierarchy if you want to check for this situation.
	*/
	PCG_API bool IsNewObjectAndNotDefault(const UObject* InObject, bool bCheckHierarchy = false);

	/** If hierarchical generation is enabled, returns all relevant grid sizes for the graph, otherwise returns partition grid size from world actor. */
	PCG_API bool GetGenerationGridSizes(const UPCGGraph* InGraph, const APCGWorldActor* InWorldActor, PCGHiGenGrid::FSizeArray& OutGridSizes, bool& bOutHasUnbounded);

	PCG_API int32 GetGenerationGridSize(const IPCGGraphExecutionSource* InExecutionSource);
	PCG_API bool IsRuntimeGeneration(const IPCGGraphExecutionSource* InExecutionSource);

#if WITH_EDITOR
	PCG_API void GetGeneratedActorsFolderPath(const AActor* InTargetActor, FString& OutFolderPath);
	PCG_API void GetGeneratedActorsFolderPath(const AActor* InTargetActor, const FPCGContext* InContext, EPCGAttachOptions AttachOptions, FString& OutFolderPath);
#endif

	UE_DEPRECATED(5.5, "This function has been deprecated in favor of the version with a context.")
	PCG_API void AttachToParent(AActor* InActorToAttach, AActor* InParent, EPCGAttachOptions AttachOptions, const FString& GeneratedPath = FString());

	PCG_API void AttachToParent(AActor* InActorToAttach, AActor* InParent, EPCGAttachOptions AttachOptions, const FPCGContext* InContext, const FString& GeneratedPath = FString());

	/**
	 * Finds functions on the actor matching the provided function names. Functions must be marked as CallInEditor
	 * and have parameters matching one of the provided prototypes. Some prototypes are provided in UPCGFunctionPrototypes.
	 */
	PCG_API TArray<UFunction*> FindUserFunctions(TSubclassOf<UObject> ActorClass, const TArray<FName>& FunctionNames, const TArray<const UFunction*>& FunctionPrototypes, const FPCGContext* InContext = nullptr);

	PCG_API TFunction<float(float, float)> GetDensityMergeFunction(EPCGDensityMergeOperation InOperation);

	/** Get an array of a randomized, uniformly distributed indices to a provided array view. */
	PCG_API TArray<int32> GetRandomIndices(FRandomStream& RandomStream, const int32 ArraySize, const int32 NumSelections);

	PCG_API bool IsServer(const UWorld* InWorld);
	PCG_API bool IsDedicatedServer(const UWorld* InWorld);
	PCG_API bool IsListenServer(const UWorld* InWorld);

	/** Shuffles the elements of an array randomly and uniformly. */
	template <typename T>
	void ShuffleArray(FRandomStream& RandomStream, TArray<T>& Array)
	{
		const int32 LastIndex = Array.Num() - 1;
		for (int32 i = 0; i < LastIndex; ++i)
		{
			const int32 Index = RandomStream.RandRange(i, LastIndex);

			if (i != Index)
			{
				Array.Swap(i, Index);
			}
		}
	}

	/** Shifts the elements of an array a number of times. */
	template <typename T>
	void ShiftArrayElements(TArrayView<T> Array, int32 NumShifts = 1)
	{
		if (Array.Num() < 2 || NumShifts == 0)
		{
			return;
		}

		const int32 Count = Array.Num();
		NumShifts %= Count;
		if (NumShifts < 0)
		{
			NumShifts += Count;
		}

		TArray<T> TempArray;
		if constexpr (std::is_trivially_copyable_v<T>)
		{
			TempArray.SetNumUninitialized(Count);
		}
		else
		{
			TempArray.SetNum(Count);
		}

		for (int32 i = 0; i < NumShifts; ++i)
		{
			TempArray[i] = std::move(Array[i + Count - NumShifts]);
		}

		for (int32 i = NumShifts; i < Count; ++i)
		{
			TempArray[i] = std::move(Array[i - NumShifts]);
		}

		// @todo: there is a better implementation where we need to allocate at most the shift so we can move things in place.
		for (int32 i = 0; i < Count; ++i)
		{
			Array[i] = std::move(TempArray[i]);
		}
	}

	/** Execute given functor on game thread. If called from game thread, executes immediately. */
	template<typename FunctorType>
	void ExecuteOnGameThread(const TCHAR* DebugName, FunctorType&& Functor)
	{
		if (IsInGameThread())
		{
			Functor();
		}
		else
		{
			::ExecuteOnGameThread(DebugName, std::forward<FunctorType>(Functor));
		}
	}

	/**
	 * Create keys for a hypothetical attribute in the provided domain ('Elements' by default), get its count, and create
	 * a const value accessor for retrieving it. Only available if the data supports the provided domain.
	 */
	PCG_API int32 GetNumberOfElements(const UPCGData* InData, const FPCGMetadataDomainID& InDomainID = PCGMetadataDomainID::Elements);

#if UE_ENABLE_DEBUG_DRAWING 
	PCG_API void DebugDrawGenerationVolume(FPCGContext* InContext, const FColor* InOverrideColor = nullptr);
#endif
}

/** Holds function prototypes used to match against actor function signatures. */
UCLASS(MinimalAPI)
class UPCGFunctionPrototypes : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	static UFunction* GetPrototypeWithNoParams() { FGCScopeGuard Guard; return FindObject<UFunction>(StaticClass(), TEXT("PrototypeWithNoParams")); }
	static UFunction* GetPrototypeWithPointAndMetadata() { FGCScopeGuard Guard; return FindObject<UFunction>(StaticClass(), TEXT("PrototypeWithPointAndMetadata")); }

private:
	UFUNCTION()
	void PrototypeWithNoParams() {}

	UFUNCTION()
	void PrototypeWithPointAndMetadata(FPCGPoint Point, const UPCGMetadata* Metadata) {}
};
