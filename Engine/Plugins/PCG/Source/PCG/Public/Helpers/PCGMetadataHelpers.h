// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/PCGMetadataCommon.h"

#include "HAL/Platform.h"

class FPCGMetadataAttributeBase;
class FPCGMetadataDomain;
class UPCGData;
class UPCGMetadata;
struct FPCGContext;
struct FSoftObjectPath;
template <typename FuncType> class TFunction;

namespace PCGMetadataHelpers
{
	PCG_API bool HasSameRoot(const UPCGMetadata* Metadata1, const UPCGMetadata* Metadata2);
	PCG_API bool HasSameRoot(const FPCGMetadataDomain* Metadata1, const FPCGMetadataDomain* Metadata2);
	PCG_API const UPCGMetadata* GetParentMetadata(const UPCGMetadata* Metadata);
	PCG_API const FPCGMetadataDomain* GetParentMetadata(const FPCGMetadataDomain* Metadata);

	/** Useful for data that doesn't support metadata parenting (like ParamData), where we need to copy the data domain,
	 * but only want to add the attributes for the elements domain, because we will copy the values manually.
	 */
	PCG_API void InitializeMetadataWithDataDomainCopyAndElementsNoCopy(const UPCGData* InData, UPCGData* OutData);

	// Utility function to break circular dependency for UPCGMetadata deprecation in PCGMetadataAttributeTpl.
	PCG_API FPCGMetadataDomain* GetDefaultMetadataDomain(UPCGMetadata* InMetadata);

	// Helpers functions to cast in spatial/param and return metadata. Nullptr if data doesn't have metadata
	PCG_API const UPCGMetadata* GetConstMetadata(const UPCGData* InData);
	PCG_API UPCGMetadata* GetMutableMetadata(UPCGData* InData);
	
	/** Create a lambda that will construct a soft object path from an underlying attribute of type FSoftObjectPath or FString. Returns true if successful. */
	[[nodiscard]] PCG_API bool CreateObjectPathGetter(const FPCGMetadataAttributeBase* InAttributeBase, TFunction<void(int64, FSoftObjectPath&)>& OutGetter);

	/** Create a lambda that will construct a soft object path from an underlying attribute of type FSoftObjectPath or FString. Returns true if successful. */
	[[nodiscard]] PCG_API bool CreateObjectOrClassPathGetter(const FPCGMetadataAttributeBase* InAttributeBase, TFunction<void(int64, FSoftObjectPath&)>& OutGetter);

	struct FPCGCopyAttributeParams
	{
		/** Source data where the attribute is coming from */
		const UPCGData* SourceData = nullptr;

		/** Target data to write to */
		UPCGData* TargetData = nullptr;

		/** Selector for the attribute in SourceData */
		FPCGAttributePropertyInputSelector InputSource;

		/** Selector for the attribute in TargetData */
		FPCGAttributePropertyOutputSelector OutputTarget;

		/** Optional context for logging */
		FPCGContext* OptionalContext = nullptr;

		/** Will convert the output attribute to this type if not Unknown */
		EPCGMetadataTypes OutputType = EPCGMetadataTypes::Unknown;

		/** If SourceData and TargetData have the same origin (if TargetData was initialized from SourceData) */
		bool bSameOrigin = false;
	};

	/** Copy the attribute coming from Source Data into Target Data. */
	PCG_API bool CopyAttribute(const FPCGCopyAttributeParams& InParams);

	struct FPCGCopyAllAttributesParams
	{
		/** Source data where the attribute is coming from */
		const UPCGData* SourceData = nullptr;

		/** Target data to write to */
		UPCGData* TargetData = nullptr;

		/**
		 * Metadata domains mapping. Empty means copying all domains from target to source, as long as they are compatible.
		 * For retro-compatibility, it is initialized as Default -> Default only.
		 */
		TMap<FPCGMetadataDomainID, FPCGMetadataDomainID> DomainMapping = {{PCGMetadataDomainID::Default, PCGMetadataDomainID::Default}}; 

		/** Optional context for logging */
		FPCGContext* OptionalContext = nullptr;

		/** Will initialize DomainMapping using SourceData and TargetData to convert names into domain IDs. If the mapping is empty, will be default -> default.*/
		PCG_API void InitializeMappingFromDomainNames(const TMap<FName, FName>& MetadataDomainsMapping);

		/** Will map all matching domains.*/
		PCG_API void InitializeMappingForAllDomains();
	};

	UE_DEPRECATED(5.6, "Use the version with FPCGCopyAllAttributesParams")
	PCG_API bool CopyAllAttributes(const UPCGData* SourceData, UPCGData* TargetData, FPCGContext* OptionalContext = nullptr);

	/** Copy all the attributes coming from Source Data into Target Data. */
	PCG_API bool CopyAllAttributes(const FPCGCopyAllAttributesParams& InParams);
	
	PCG_API void SetPointAttributes(FPCGMetadataDomain* InOutMetadata, const TArrayView<const FPCGPoint>& InPoints, const FPCGMetadataDomain* InMetadata, const TArrayView<FPCGPoint>& OutPoints, FPCGContext* OptionalContext = nullptr);

	PCG_API void ComputePointWeightedAttribute(FPCGMetadataDomain* InOutMetadata, FPCGPoint& OutPoint, const TArrayView<TPair<const FPCGPoint*, float>>& InWeightedPoints, const FPCGMetadataDomain* InMetadata);

	// [EXPERIMENTAL] - This function may be renamed or deprecated in the future once it is decoupled with default values.
	/** Helper that checks that Type is supported by default values. */
	inline bool MetadataTypeSupportsDefaultValues(const EPCGMetadataTypes Type)
	{
		switch (Type)
		{
			case EPCGMetadataTypes::Double:         // fall-through
			case EPCGMetadataTypes::Integer32:      // fall-through
			case EPCGMetadataTypes::Integer64:      // fall-through
			case EPCGMetadataTypes::Vector:         // fall-through
			case EPCGMetadataTypes::Vector2:        // fall-through
			case EPCGMetadataTypes::Vector4:        // fall-through
			case EPCGMetadataTypes::String:         // fall-through
			case EPCGMetadataTypes::Name:           // fall-through
			case EPCGMetadataTypes::Boolean:        // fall-through
			case EPCGMetadataTypes::Rotator:        // fall-through
			case EPCGMetadataTypes::Quaternion:     // fall-through
			case EPCGMetadataTypes::Transform:      // fall-through
			case EPCGMetadataTypes::SoftObjectPath: // fall-through
			case EPCGMetadataTypes::SoftObject:     // fall-through
			case EPCGMetadataTypes::SoftClassPath:  // fall-through
			case EPCGMetadataTypes::SoftClass:
				return true;
			// This is automatically converted in the accessor to a Double
			case EPCGMetadataTypes::Float: // fall-through
			// Anything else is not valid.
			case EPCGMetadataTypes::Unknown: // fall-through
			default:
				return false;
		}
	}
}
