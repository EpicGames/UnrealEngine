// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGModule.h"

/** 
* Convenience class to do data partitioning, i.e. building a TMap<Value, TArray<indices>>
* Uses CRTP to remove the need for virtual calls.
* Has some behavior options depending on the derived class.
* Currently supports time-slicing (caveat for rooting data, not done yet)
* Can easily write to a common output (for match & set cases)
* Can write to separate data (for partitioning)
* Supports multiple data (indices won't be useful though)
* 
* The intended implementation is that derived classes will implement the same functions (e.g. name hiding) and those will
* be appropriately called through using the derived class type.
* Methods that are not re-implemented/hidden will retain their default implementation, which is mostly trivial.
*/
template<typename Derived, typename KeyType>
class FPCGDataPartitionBase
{
public:
	struct Element
	{
		TArray<int32> Indices;
		UPCGData* PartitionData = nullptr;
	};

	FPCGDataPartitionBase() = default;
	FPCGDataPartitionBase(const TArrayView<KeyType>& InKeys)
	{
		for (const KeyType& Key : InKeys)
		{
			ElementMap.Emplace(Key, Element());
		}
	}

	Derived* This() { return static_cast<Derived*>(this); }
	const Derived* This() const { return static_cast<const Derived*>(this); }

	/** Overridable behavior methods */
	bool Initialize() { return true; }
	bool InitializeForData(const UPCGData* Data) { return true; }
	void AddToPartitionData(Element* SelectedElement, const UPCGData* ParentData, int32 Index) {}
	void WriteToOutputData(const UPCGData* ParentData, UPCGData* OutData, Element* SelectedElement, int32 Index) {}
	void Finalize(const UPCGData* InData, UPCGData* OutData) {}
	Element* Select(int32 Index) { return nullptr; }
	int32 TimeSlicingCheckFrequency() const { return 1024; }

	/** API */
	bool SelectMultiple(FPCGContext& Context, const UPCGData* InData, int32& InCurrentIndex, int32 InMaxIndex, UPCGData* OutData);
	void Reset() { ElementMap.Reset(); }

	// Data access
	TMap<KeyType, Element> ElementMap;
};

template<typename Derived, typename KeyType>
bool FPCGDataPartitionBase<Derived, KeyType>::SelectMultiple(FPCGContext& Context, const UPCGData* InData, int32& InCurrentIndex, int32 InMaxIndex, UPCGData* OutData)
{
	if (!InData)
	{
		PCGE_LOG_C(Error, GraphAndLog, &Context, NSLOCTEXT("PCGDataPartition", "InputMissingData", "Missing input data"));
		return true;
	}

	if (InCurrentIndex == 0)
	{
		if (!This()->InitializeForData(InData, OutData))
		{
			return true;
		}
	}

	int32 CurrentIndex = InCurrentIndex;
	int32 LastCheckpointIndex = InCurrentIndex;
	const int32 TimeSlicingCheckFrequency = This()->TimeSlicingCheckFrequency();

	while (CurrentIndex < InMaxIndex)
	{
		Element* SelectedElement = This()->Select(CurrentIndex);
		if (SelectedElement)
		{
			SelectedElement->Indices.Add(CurrentIndex);
			This()->AddToPartitionData(SelectedElement, InData, CurrentIndex);
		}

		This()->WriteToOutputData(InData, OutData, SelectedElement, CurrentIndex);

		++CurrentIndex;

		if (CurrentIndex - LastCheckpointIndex >= TimeSlicingCheckFrequency)
		{
			if (Context.ShouldStop())
			{
				break;
			}
			else
			{
				LastCheckpointIndex = CurrentIndex;
			}
		}
	}

	InCurrentIndex = CurrentIndex;

	if (CurrentIndex == InMaxIndex)
	{
		This()->Finalize(InData, OutData);
	}

	return (CurrentIndex == InMaxIndex);
}