// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Box.h"
#include "Math/BoxSphereBounds.h"
#include "Math/UnrealMathUtility.h"

namespace PCGPointHelpers
{
	inline FBox GetLocalBounds(const FVector& InBoundsMin, const FVector& InBoundsMax)
	{
		return FBox(InBoundsMin, InBoundsMax);
	}

	inline void SetLocalBounds(const FBox& InBounds, FVector& OutBoundsMin, FVector& OutBoundsMax)
	{
		OutBoundsMin = InBounds.Min;
		OutBoundsMax = InBounds.Max;
	}

	inline FBox GetLocalDensityBounds(float InSteepness, const FVector& InBoundsMin, const FVector& InBoundsMax)
	{
		return FBox((2 - InSteepness) * InBoundsMin, (2 - InSteepness) * InBoundsMax);
	}

	inline FVector GetLocalCenter(const FVector& InBoundsMin, const FVector& InBoundsMax)
	{
		return (InBoundsMax + InBoundsMin) / 2.0;
	}

	inline FVector GetExtents(const FVector& InBoundsMin, const FVector& InBoundsMax)
	{
		return (InBoundsMax - InBoundsMin) / 2.0;
	}

	inline FBoxSphereBounds GetDensityBounds(const FTransform& InTransform, float InSteepness, const FVector& InBoundsMin, const FVector& InBoundsMax)
	{
		if (InTransform.IsRotationNormalized())
		{
			return FBoxSphereBounds(PCGPointHelpers::GetLocalDensityBounds(InSteepness, InBoundsMin, InBoundsMax).TransformBy(InTransform));
		}
		else
		{
			FTransform TranslationAndScale = InTransform;
			TranslationAndScale.SetRotation(FQuat::Identity);
			return FBoxSphereBounds(PCGPointHelpers::GetLocalDensityBounds(InSteepness, InBoundsMin, InBoundsMax).TransformBy(TranslationAndScale));
		}
	}
	
	inline void SetExtents(const FVector& InExtents, FVector& InOutBoundsMin, FVector& InOutBoundsMax)
	{
		const FVector Center = PCGPointHelpers::GetLocalCenter(InOutBoundsMin, InOutBoundsMax);
		InOutBoundsMin = Center - InExtents;
		InOutBoundsMax = Center + InExtents;
	}
		
	inline FVector GetScaledExtents(const FTransform& InTransform, const FVector& InBoundsMin, const FVector& InBoundsMax)
	{ 
		return PCGPointHelpers::GetExtents(InBoundsMin, InBoundsMax) * InTransform.GetScale3D();
	}

	inline void SetLocalCenter(const FVector& InCenter, FVector& InOutBoundsMin, FVector& InOutBoundsMax)
	{
		const FVector Delta = InCenter - PCGPointHelpers::GetLocalCenter(InOutBoundsMin, InOutBoundsMax);
		InOutBoundsMin += Delta;
		InOutBoundsMax += Delta;
	}

	inline FVector GetLocalSize(const FVector& InBoundsMin, const FVector& InBoundsMax)
	{ 
		return InBoundsMax - InBoundsMin; 
	}
	
	inline FVector GetScaledLocalSize(const FTransform& InTransform, const FVector& InBoundsMin, const FVector& InBoundsMax)
	{ 
		return PCGPointHelpers::GetLocalSize(InBoundsMin, InBoundsMax) * InTransform.GetScale3D();
	}

	inline void ApplyScaleToBounds(FTransform& InOutTransform, FVector& InOutBoundsMin, FVector& InOutBoundsMax)
	{
		const FVector PointScale = InOutTransform.GetScale3D();
		InOutTransform.SetScale3D(PointScale.GetSignVector());
		InOutBoundsMin *= PointScale.GetAbs();
		InOutBoundsMax *= PointScale.GetAbs();
	}

	inline void ResetPointCenter(const FVector& BoundsRatio, FTransform& InOutTransform, FVector& InOutBoundsMin, FVector& InOutBoundsMax)
	{
		const FVector NewCenterLocal = FMath::Lerp(InOutBoundsMin, InOutBoundsMax, BoundsRatio);

		InOutBoundsMin -= NewCenterLocal;
		InOutBoundsMax -= NewCenterLocal;

		InOutTransform.SetLocation(InOutTransform.GetLocation() + InOutTransform.TransformVector(NewCenterLocal));
	}
}