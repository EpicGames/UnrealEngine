// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGParamData.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "StructUtils/PropertyBag.h"
#include "UObject/UnrealType.h" // IWYU pragma: keep

#include "PCGPropertyHelpers.generated.h"

enum class EPCGMetadataTypes : uint8;

struct FPCGContext;
class UPCGData;
class UPCGParamData;
class UUserDefinedStruct;

UENUM()
enum class EPCGObjectExtractorBehavior
{
	NoExtract UMETA(ToolTip="Do not extract object members, single soft object/class path attribute"),
	Extract UMETA(ToolTip="Extract object members at the first level (first struct property), nested objects will be a soft object/class path attribute"),
};
	
UENUM()
enum class EPCGStructExtractorBehavior
{
	ExtractRootOnly UMETA(ToolTip="Legacy behavior that was extracting the first level if the top was a struct. Nested structs were discarded."),
	NoExtract UMETA(ToolTip="Do not extract struct members, single attribute of the struct type."),
	Extract UMETA(ToolTip="Extract struct members at the first level (first struct property), nested structs will be an attribute of the struct type."),
};
	
UENUM()
enum class EPCGContainerExtractorBehavior
{
	FlattenLastAndDiscardNested UMETA(ToolTip="Do not extract containers that are nested in structs, but flatten all encountered containers in the property path, including the last array property."),
	NoFlattenLast UMETA(ToolTip="If the property to extract is an array, do not flatten it, make it a single attribute value. All encountered containers in the path will be flatten though."),
	FlattenLast UMETA(ToolTip="If the property to extract is an array, flatten it, make it a multi entry attribute value. All encountered containers in the path will be flatten though."),
};


namespace PCGPropertyHelpers
{
	/**
	* Get a property value and pass it as a parameter to a callback function.
	* @param InObject - The object to read from
	* @param InProperty - The property to look for
	* @param InFunc - Callback function that can return anything, and should have a single templated argument, where the property will be.
	* @returns Forward the result of the callback.
	*/
	template <typename ObjectType, typename Func>
	decltype(auto) GetPropertyValueWithCallback(const ObjectType* InObject, const FProperty* InProperty, Func InFunc);

	/**
	* Set a property value given by a callback function.
	* @param InObject - The object to write to
	* @param InProperty - The property to look for
	* @param InFunc - Callback function that take a reference to a templated type. It will set the property with this value. returns true if we should set, false otherwise.
	* @returns Forward the result of the callback
	*/
	template <typename ObjectType, typename Func>
	bool SetPropertyValueFromCallback(ObjectType* InObject, const FProperty* InProperty, Func InFunc);

	/**
	* Conversion between property type and PCG type.
	* @param InProperty - The property to look for
	* @returns PCG type if the property is supported, Unknown otherwise.
	*/
	PCG_API EPCGMetadataTypes GetMetadataTypeFromProperty(const FProperty* InProperty);

	/**
	 * Conversion between StructUtils' PropertyBag types and PCG type.
	 * @param InPropertyName - The name of the property.
	 * @param Type - The PCG metadata type.
	 * @returns The property bag property type.
	 */
	PCG_API FPropertyBagPropertyDesc CreatePropertyBagDescWithMetadataType(FName InPropertyName, EPCGMetadataTypes Type);
	
	/**
	 * Conversion between StructUtils' PropertyBag types and PCG Attribute Desc.
	 * @param InPropertyName - The name of the property.
	 * @param AttributeDesc - The PCG metadata attribute descriptor.
	 * @returns The property bag property type.
	 */
	PCG_API FPropertyBagPropertyDesc CreatePropertyBagDescWithMetadataType(FName InPropertyName, const FPCGMetadataAttributeDesc& AttributeDesc);
	
	struct FExtractorParameters
	{
		// Disable deprecation warning on the rule of 5 because of the PropertySelector member.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FExtractorParameters() = default;
		FExtractorParameters(const FExtractorParameters&) = default;
		FExtractorParameters(FExtractorParameters&&) = default;
		FExtractorParameters& operator=(const FExtractorParameters&) = default;
		FExtractorParameters& operator=(FExtractorParameters&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FExtractorParameters(const void* InContainer, const UStruct* InClass, const FPCGAttributePropertySelector& InPropertySelector, FName InOutputAttributeName, bool bInPropertyNeedsToBeVisible);
		FExtractorParameters(const void* InContainer, const UStruct* InClass, const TArray<FPCGAttributePropertySelector>& InPropertySelectors, FName InOutputAttributeName, bool bInPropertyNeedsToBeVisible);
		FExtractorParameters(const void* InContainer, const UStruct* InClass, const FString& InPropertySelectorString, FName InOutputAttributeName, bool bInPropertyNeedsToBeVisible);
		
		// Pointer to the container containing the data we want to extract
		const void* Container = nullptr;

		// Class or ScriptStruct of the object/struct we want to extract the property from.
		const UStruct* Class = nullptr;

		// Selector of the properties we want to extract
		TArray<FPCGAttributePropertySelector> PropertySelectors;

		// Optional name of the attribute that will receive the extracted property. If @Source or @SourceName, will take the property name. 
		// Also not used for Structs/Object extraction, as we will create multiple attributes, and they will be the name of all the extracted members.
		FName OutputAttributeName = NAME_None;
		
		// If the property we want to extract is an object/struct, give the possibility to extract all their member in one go.
		// Only work if their members are not structs or containers (like array).
		UE_DEPRECATED(5.8, "Should set the enum EPCGStructExtractorBehavior/EPCGObjectExtractorBehavior")
		bool bShouldExtract = false;
		
		// Extra flag if the property needs to be visible to be extractable.
		bool bPropertyNeedsToBeVisible = false;

		// Output attribute name are already sanitized by default but it keeps some characters like spaces.
		// This is a stricter option and if the name is not following the object name rule (pretty much only alphanumerical and "_"), it will be sanitized.
		// Necessary if we have cases with spaces, which doesn't play nicely with PCG attributes.
		bool bStrictSanitizeOutputAttributeNames = false;
		
		// How we should extract the structs. (cf. enum tooltips)
		EPCGStructExtractorBehavior StructExtractorBehavior = EPCGStructExtractorBehavior::ExtractRootOnly;
		
		// How we should extract the objects. (cf. enum tooltips)
		EPCGObjectExtractorBehavior ObjectExtractorBehavior = EPCGObjectExtractorBehavior::NoExtract;
		
		// How we should extract the containers. (cf. enum tooltips)
		EPCGContainerExtractorBehavior ContainerExtractorBehavior = EPCGContainerExtractorBehavior::FlattenLastAndDiscardNested;
	};

	/**
	* Extract a given property in an Attribute Set.
	* @param Parameters - Parameters for extraction, cf above.
	* @param OptionalContext - Optional context if the extraction is done in a PCG Node, so errors are using the context to log.
	* @param OptionalObjectTraversed - Optional set to store all objects that we traversed, to be able to react to those objects changes.
	*/
	PCG_API UPCGParamData* ExtractPropertyAsAttributeSet(const FExtractorParameters& Parameters, FPCGContext* OptionalContext = nullptr, TSet<FSoftObjectPath>* OptionalObjectTraversed = nullptr, bool bQuiet = false);

	/**
	* Extract an attribute set in a array of structures. T MUST be a UStruct. Also, it must only contain supported types (so no arrays nor other structures)
	* @param InParamData - Attribute set that contains the data.
	* @param OptionalNameMapping - Optional mapping for the name in the structure and the name in the attribute set. Can also say if this property is required, or not and should be defaulted. By default all are defaulted if not found.
	* @param OptionalContext - Optional context if the extraction is done in a PCG Node, so errors are using the context to log.
	*/
	template <typename T> requires TModels_V<CStaticStructProvider, T>
	TArray<T> ExtractAttributeSetAsArrayOfStructs(const UPCGParamData* InParamData, const TMap<FName, TTuple<FName, bool>>* OptionalNameMapping = nullptr, FPCGContext* OptionalContext = nullptr);
	
	/**
	* Extract an attribute set in a array of structures. Also, it must only contain supported types (so no arrays nor other structures)
	* @param InParamData - Attribute set that contains the data.
	* @param InStruct - Structure to extract the param to.
	* @param OptionalNameMapping - Optional mapping for the name in the structure and the name in the attribute set. Can also say if this property is required, or not and should be defaulted. By default all are defaulted if not found.
	* @param OptionalContext - Optional context if the extraction is done in a PCG Node, so errors are using the context to log.
	*/
	PCG_API TArray<FInstancedStruct> ExtractAttributeSetAsArrayOfStructs(const UPCGParamData* InParamData, const UScriptStruct* InStruct, const TMap<FName, TTuple<FName, bool>>* OptionalNameMapping = nullptr, FPCGContext* OptionalContext = nullptr);
	
	/**
	* Extract an attribute set in a array of structures. Also, it must only contain supported types (so no arrays nor other structures)
	* @param InParamData - Attribute set that contains the data.
	* @param InStruct - Structure to extract the param to.
	* @param InContainers - Containers of the given structs. The number MUST match the number of entries in the ParamData.
	* @param OptionalNameMapping - Optional mapping for the name in the structure and the name in the attribute set. Can also say if this property is required, or not and should be defaulted. By default all are defaulted if not found.
	* @param OptionalContext - Optional context if the extraction is done in a PCG Node, so errors are using the context to log.
	*/
	PCG_API bool ExtractAttributeSetToContainers(const UPCGParamData* InParamData, const UStruct* InStruct, TArrayView<void*> InContainers, const TMap<FName, TTuple<FName, bool>>* OptionalNameMapping = nullptr, FPCGContext* OptionalContext = nullptr);

	/**
	* More permissive search for a property name if the InStruct is a UserDefinedStruct.
	* Since the names in editor and at runtime can be different for the properties defined in a UserDefinedStruct, the search will also look for sanitized version of the name if it is not a valid name.
	* Otherwise just return FindFProperty
	*/
	PCG_API const FProperty* FindPropertyByName(const UStruct* InStruct, const FName InName);

	/**
	* More permissive search for a property name in a UserDefinedStruct/UVerseClass.
	* Since the names in editor and at runtime can be different for the properties defined in a UserDefinedStruct/VerseClass, the search will also look for sanitized version of the name if it is not a valid name.
	*/
	const FProperty* FindPropertyByNameEx(const UStruct* InStruct, const FName InName);

	namespace Constants
	{
		const FName EnableCategoriesMetadataName("EnableCategories");
		const FName CategoryMetadataName("Category");
	}
}

//////
/// PCGPropertyHelpers Implementation
//////

// Func signature : auto(auto&&)
template <typename ObjectType, typename Func>
inline decltype(auto) PCGPropertyHelpers::GetPropertyValueWithCallback(const ObjectType* InObject, const FProperty* InProperty, Func InFunc)
{
	if (!InObject || !InProperty)
	{
		return false;
	}

	TUniquePtr<IPCGAttributeAccessor> PropertyAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(InProperty);

	if (!PropertyAccessor.IsValid())
	{
		return false;
	}

	auto Getter = [&PropertyAccessor, &InFunc, InObject](auto Dummy)
	{
		using Type = decltype(Dummy);
		Type Value{};
		FPCGAttributeAccessorKeysSingleObjectPtr<ObjectType> Key(InObject);

		if (PropertyAccessor->Get<Type>(Value, Key))
		{
			return InFunc(Value);
		}
		else
		{
			using ReturnType = decltype(InFunc(0.0));
			if constexpr (std::is_same_v<ReturnType, void>)
			{
				return;
			}
			else
			{
				return ReturnType{};
			}
		}
	};

	return PCGMetadataAttribute::CallbackWithRightType(PropertyAccessor->GetUnderlyingType(), Getter);
}

// Func signature : bool(auto&)
// Will have property value in first arg, and boolean return if we should set the property after.
// Returns true if the set succeeded
template <typename ObjectType, typename Func>
inline bool PCGPropertyHelpers::SetPropertyValueFromCallback(ObjectType* InObject, const FProperty* InProperty, Func InFunc)
{
	if (!InObject || !InProperty)
	{
		return false;
	}

	TUniquePtr<IPCGAttributeAccessor> PropertyAccessor = PCGAttributeAccessorHelpers::CreatePropertyAccessor(InProperty);

	if (!PropertyAccessor.IsValid())
	{
		return false;
	}

	auto Setter = [&PropertyAccessor, &InFunc, InObject](auto Dummy) -> bool
	{
		using Type = decltype(Dummy);
		Type Value{};
		FPCGAttributeAccessorKeysSingleObjectPtr<std::remove_const_t<ObjectType>> Key(InObject);

		if (InFunc(Value))
		{
			return PropertyAccessor->Set<Type>(Value, Key);
		}
		else
		{
			return false;
		}
	};

	return PCGMetadataAttribute::CallbackWithRightType(PropertyAccessor->GetUnderlyingType(), Setter);
}

template <typename T> requires TModels_V<CStaticStructProvider, T>
TArray<T> PCGPropertyHelpers::ExtractAttributeSetAsArrayOfStructs(const UPCGParamData* InParamData, const TMap<FName, TTuple<FName, bool>>* OptionalNameMapping, FPCGContext* OptionalContext)
{
	if (!ensure(InParamData))
	{
		return {};
	}
	
	const int32 NumElements = InParamData->ConstMetadata()->GetItemCountForChild();
	
	if (NumElements == 0)
	{
		return {};
	}
	
	TArray<T> Result;
	TArray<void*> ResultContainers;
	Result.SetNum(NumElements);
	ResultContainers.Reserve(NumElements);
	for (int32 i = 0; i < NumElements; i++)
	{
		ResultContainers.Add(&Result[i]);
	}
	
	return ExtractAttributeSetToContainers(InParamData, T::StaticStruct(), ResultContainers, OptionalNameMapping, OptionalContext) ? Result : TArray<T>{};
}
