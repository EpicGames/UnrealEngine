// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGModule.h"
#include "PCGParamData.h"
#include "PCGSettings.h"
#include "Data/PCGSpatialData.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "PCGSettingsHelpers.generated.h"

#define UE_API PCG_API

class UPCGComponent;
class UPCGNode;
class UPCGPin;
struct FPCGDataCollection;


USTRUCT(BlueprintType)
struct FPCGSettingsPropertyDefinition
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = Node)
	FString Name;

	UPROPERTY(BlueprintReadOnly, Category = Node)
	FString FullName;

	UPROPERTY(BlueprintReadOnly, Category = Node)
	FString Tooltip;

	UPROPERTY(BlueprintReadOnly, Category = Node)
	FString CppType;
};

UCLASS(MinimalAPI, BlueprintType)
class UPCGSettingsHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Returns the list of overridable properties from the settings class. */
	UFUNCTION(BlueprintCallable, Category = Node)
	static PCG_API TArray<FPCGSettingsPropertyDefinition> GetCommonlyUsedProperties(TSubclassOf<UPCGSettings> InSettingsClass);
};

namespace PCGSettingsHelpers
{
	/** Utility to call from before-node-update deprecation. A dedicated pin for params will be added when the pins are updated. Here we detect any params
	*   connections to the In pin and disconnect them, and move the first params connection to a new params pin.
	*/
	void DeprecationBreakOutParamsToNewPin(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins, TArray<TObjectPtr<UPCGPin>>& OutputPins);

	/**
	* Advanced method to gather override params when you don't have access to FPCGContext (and therefore don't have access to automatic
	* param override).
	* Limitation: Only support metadata types for T.
	*/
	template <typename T>
	bool GetOverrideValue(const FPCGDataCollection& InInputData, const UPCGSettings* InSettings, const FName InPropertyName, const T& InDefaultValue, T& OutValue)
	{
		check(InSettings);

		// Limitation: Only support metadata types
		static_assert(PCG::Private::IsPCGType<T>());

		// Try to find the override param associated with the property.
		const FPCGSettingsOverridableParam* Param = InSettings->OverridableParams().FindByPredicate([InPropertyName](const FPCGSettingsOverridableParam& InParam) { return !InParam.PropertiesNames.IsEmpty() && (InParam.PropertiesNames.Last() == InPropertyName);});

		OutValue = InDefaultValue;

		if (!Param)
		{
			return false;
		}

		PCGAttributeAccessorHelpers::AccessorParamResult AccessorResult{};
		TUniquePtr<const IPCGAttributeAccessor> AttributeAccessor = PCGAttributeAccessorHelpers::CreateConstAccessorForOverrideParamWithResult(InInputData, *Param, &AccessorResult);

		const FName AttributeName = AccessorResult.AttributeName;

		if (!AttributeAccessor)
		{
			return false;
		}

		return PCGMetadataAttribute::CallbackWithRightType(PCG::Private::MetadataTypes<T>::Id, [&AttributeAccessor, &Param, &AttributeName, &InDefaultValue, &OutValue](auto Dummy) -> bool
		{
			using PropertyType = decltype(Dummy);

			// Override were using the first entry (0) by default.
			FPCGAttributeAccessorKeysEntries FirstEntry(PCGFirstEntryKey);

			if (!AttributeAccessor->Get<T>(OutValue, FirstEntry, EPCGAttributeAccessorFlags::AllowBroadcast | EPCGAttributeAccessorFlags::AllowConstructible))
			{
				UE_LOGF(LogPCG, Warning, "[PCGSettingsHelpers::GetOverrideValue] '%ls' parameter cannot be converted from '%ls' attribute, incompatible types.", *Param->Label.ToString(), *AttributeName.ToString());
				return false;
			}

			return true;
		});
	}

	inline const UPCGSpatialData* ComputeBoundingShape(FPCGContext* Context, FName BoundingShapeLabel, bool& bOutUnionWasCreated)
	{
		const UPCGSpatialData* BoundingShape = Context->InputData.GetSpatialUnionOfInputsByPin(Context, BoundingShapeLabel, bOutUnionWasCreated);

		// Fallback to getting bounds from actor
		if (!BoundingShape && Context->ExecutionSource.IsValid())
		{
			check(bOutUnionWasCreated == false);
			BoundingShape = Cast<UPCGSpatialData>(Context->ExecutionSource->GetExecutionState().GetSelfData());
		}

		return BoundingShape;
	}

	struct FPCGGetAllOverridableParamsConfig
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FPCGGetAllOverridableParamsConfig() = default;
		FPCGGetAllOverridableParamsConfig(const FPCGGetAllOverridableParamsConfig&) = default;
		FPCGGetAllOverridableParamsConfig(FPCGGetAllOverridableParamsConfig&&) = default;
		FPCGGetAllOverridableParamsConfig& operator=(const FPCGGetAllOverridableParamsConfig&) = default;
		FPCGGetAllOverridableParamsConfig& operator=(FPCGGetAllOverridableParamsConfig&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// If we don't use the seed, don't add it as override.
		bool bUseSeed = false;

		// Don't look for properties from parents
		bool bExcludeSuperProperties = false;

		// Don't look for properties from 'StopClass' and its parents (Should be a subclass of the looked-up class) nullptr means go all the way to root.
		//  if bExcludeSuperProperties is true, this is ignored.
		const UStruct* StopClass = nullptr;

#if WITH_EDITOR
		// List of metadata values to find in property metadata. Only works in editor builds as metadata on property is not available elsewise.
		UE_DEPRECATED(5.8, "Replaced by custom logic in ShouldKeepPropertyFunc")
		TArray<FName> IncludeMetadataValues;

		// If the metadata values is a conjunction (all values need to be present to keep), or disjunction (any values needs to be present to keep)
		UE_DEPRECATED(5.8, "Replaced by custom logic in ShouldKeepPropertyFunc")
		bool bIncludeMetadataIsConjunction = false;

		// List of metadata values to find in property metadata. Only works in editor builds as metadata on property is not available elsewise.
		UE_DEPRECATED(5.8, "Replaced by custom logic in ShouldKeepPropertyFunc")
		TArray<FName> ExcludeMetadataValues;

		// If the exclude values is a conjunction (all values need to be present to discard), or disjunction (any values needs to be present to discard)
		UE_DEPRECATED(5.8, "Replaced by custom logic in ShouldKeepPropertyFunc")
		bool bExcludeMetadataIsConjunction = false;
#endif // WITH_EDITOR

		UE_DEPRECATED(5.8, "Replaced by custom logic in ShouldKeepPropertyFunc")
		// Flags to exclude in property flags
		uint64 ExcludePropertyFlags = 0;

		// If the exclude flags is a conjunction (all flags need to be present to discard), or disjunction (any flag needs to be present to discard)
		UE_DEPRECATED(5.8, "Replaced by custom logic in ShouldKeepPropertyFunc")
		bool bExcludePropertyFlagsIsConjunction = false;

		// Flags to include in property flags. Note that if exclusion says discard, it will be discarded.
		UE_DEPRECATED(5.8, "Replaced by custom logic in ShouldKeepPropertyFunc")
		uint64 IncludePropertyFlags = 0;

		// If the include flags is a conjunction (all flags need to be present to keep), or disjunction (any flag needs to be present to keep)
		UE_DEPRECATED(5.8, "Replaced by custom logic in ShouldKeepPropertyFunc")
		bool bIncludePropertyFlagsIsConjunction = false;

		// To accomodate all the different combinations for property flags and metadata flags, the caller can provide a function to evaluate
		// if the property should be kept or discarded.
		TFunction<bool(const FProperty* /*Property*/, int32 /*Depth*/)> ShouldKeepPropertyFunc;

		// Max depth for structs of structs. -1 = no limit
		int32 MaxStructDepth = -1;

		// If you want to go through objects too
		bool bExtractObjects = false;

		// If you want to also extract arrays
		bool bExtractArrays = false;
		
		// If we should discard any leaf struct property that was not supported before generic attributes.
		bool bDiscardLeafStructProperty = true;

		// When extracting arrays of arrays, discard any property that is within this number or more containers.
		// For example, a value of 1 will discard any property that is in an array of array.
		// -1 = no limit
		int32 MaxContainersNum = -1;
	};

	PCG_API TArray<FPCGSettingsOverridableParam> GetAllOverridableParams(const UStruct* InClass, const FPCGGetAllOverridableParamsConfig& InConfig);

	/* Small helper that use thread local storage to build a map of pins to their current type. Meant to prevent reentrancy issues (combinatorics) in dynamic types.
	* This struct is not meant to be allocated dynamically as the intention is to have a well-defined construction-destruction scope. */
	struct FPinTypeScopeHelper
	{
		UE_API FPinTypeScopeHelper();
		UE_API ~FPinTypeScopeHelper();
		FPinTypeScopeHelper(const FPinTypeScopeHelper&) = delete;
		FPinTypeScopeHelper(FPinTypeScopeHelper&&) = delete;
		FPinTypeScopeHelper& operator=(const FPinTypeScopeHelper&) = delete;
		FPinTypeScopeHelper& operator=(FPinTypeScopeHelper&&) = delete;

		UE_DEPRECATED(5.7, "Use GetCurrentPinTypeID version")
		UE_API EPCGDataType GetCurrentPinType(const UPCGPin* InPin) const;

		UE_API FPCGDataTypeIdentifier GetCurrentPinTypeID(const UPCGPin* InPin) const;
	private:
		bool bClearMapOnDestruction = false;
	};
}

// Deprecated macro, not necessary anymore. Cf. GetValue
#define PCG_GET_OVERRIDEN_VALUE(Settings, Variable, Params) PCGSettingsHelpers::GetValue(GET_MEMBER_NAME_CHECKED(std::remove_pointer_t<std::remove_const_t<decltype(Settings)>>, Variable), (Settings)->Variable, Params)

#undef UE_API
