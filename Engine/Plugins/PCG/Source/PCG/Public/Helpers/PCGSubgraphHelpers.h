// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SubclassOf.h"

class UPCGGraph;
class UPCGNode;
class UPCGPin;
class UPCGSettings;
class UPCGBaseSubgraphSettings;
class UObject;

class FPCGSubgraphHelpers
{
public:
	static PCG_API UPCGGraph* CollapseIntoSubgraphWithReason(UPCGGraph* InOriginalGraph, const TArray<UPCGNode*>& InNodesToCollapse, const TArray<UObject*>& InExtraNodesToCollapse, FText& OutFailReason, UPCGGraph* OptionalPreAllocatedGraph = nullptr, TSubclassOf<UPCGBaseSubgraphSettings> InSubgraphSettingsClass = nullptr);
	static PCG_API UPCGGraph* CollapseIntoSubgraph(UPCGGraph* InOriginalGraph, const TArray<UPCGNode*>& InNodesToCollapse, const TArray<UObject*>& InExtraNodesToCollapse, UPCGGraph* OptionalPreAllocatedGraph = nullptr);

	/** Generic function to spawn a node and connect it to the default in and out pins. */
	static PCG_API UPCGNode* SpawnNodeAndConnect(UPCGGraph* InGraph, UPCGPin* InUpstreamPin, UPCGPin* InDownstreamPin, TSubclassOf<UPCGSettings> InSettingsClass, const TFunction<bool(UPCGSettings*)>& PostSettingsCreation = {});
};