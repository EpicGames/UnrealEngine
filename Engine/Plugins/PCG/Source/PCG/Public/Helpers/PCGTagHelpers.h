// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "Metadata/PCGMetadata.h"

#define UE_API PCG_API

namespace PCG
{
namespace Private
{
	struct FParseTagResult
	{
		FParseTagResult() = default;
		UE_API explicit FParseTagResult(const FString& InTag);
		UE_API explicit FParseTagResult(FName InTag);

		bool IsValid() const { return !Attribute.IsEmpty(); }
		bool HasBeenSanitized() const { return OriginalAttribute.IsSet(); }
		bool HasValue() const { return Value.IsSet(); }
		bool HasNumericValue() const { return NumericValue.IsSet(); }
		bool HasBooleanValue() const { return BooleanValue.IsSet(); }
		const FString& GetOriginalAttribute() const { return HasBeenSanitized() ? OriginalAttribute.GetValue() : Attribute; }

		FString Attribute;
		TOptional<FString> OriginalAttribute;
		TOptional<FString> Value;
		TOptional<double> NumericValue;
		TOptional<bool> BooleanValue;
	};

	enum class ESetAttributeFromTagFlags : uint8
	{
		None = 0,
		CreateAttribute = 1,
		OverwriteAttributeIfDifferentType = 2,
		SetDefaultValue = 4
	};
	ENUM_CLASS_FLAGS(ESetAttributeFromTagFlags);

	// Builds the tag result structure from the provided tag.
	PCG_API FParseTagResult ParseTag(const FString& InTag);
	PCG_API FParseTagResult ParseTag(FName InTag);

	// Parses a tag and creates the corresponding attribute on the provided metadata. If the name is invalid or the predicate rejects it, will return false.
	PCG_API bool CreateAttributeFromTag(const FString& InTag, UPCGMetadata* InMetadata, FParseTagResult* OutResult = nullptr);
	PCG_API bool CreateAttributeFromTag(const FParseTagResult& InTagData, UPCGMetadata* InMetadata);

	// Parses a tag, optionally creates the corresponding attribute on the provided metadata and sets the value. If the name is invalid or the predicate rejects it, will return false.
	PCG_API bool SetAttributeFromTag(const FString& InTag, UPCGMetadata* InMetadata, PCGMetadataEntryKey InKey, ESetAttributeFromTagFlags Flags = ESetAttributeFromTagFlags::None, FParseTagResult* OutResult = nullptr, TOptional<FName> OptionalAttributeName = {});
	PCG_API bool SetAttributeFromTag(const FParseTagResult& InTagData, UPCGMetadata* InMetadata, PCGMetadataEntryKey InKey, ESetAttributeFromTagFlags Flags = ESetAttributeFromTagFlags::None, TOptional<FName> OptionalAttributeName = {});

	PCG_API bool SetAttributeFromTag(const FString& InTag, FPCGMetadataDomain* InMetadata, PCGMetadataEntryKey InKey, ESetAttributeFromTagFlags Flags = ESetAttributeFromTagFlags::None, FParseTagResult* OutResult = nullptr, TOptional<FName> OptionalAttributeName = {});
	PCG_API bool SetAttributeFromTag(const FParseTagResult& InTagData, FPCGMetadataDomain* InMetadata, PCGMetadataEntryKey InKey, ESetAttributeFromTagFlags Flags = ESetAttributeFromTagFlags::None, TOptional<FName> OptionalAttributeName = {});

	// Deprecated methods
	UE_DEPRECATED(5.6, "Use the version with the flags parameters")
	PCG_API bool SetAttributeFromTag(const FString& InTag, UPCGMetadata* InMetadata, PCGMetadataEntryKey InKey, bool bInCanCreateAttribute, FParseTagResult* OutResult = nullptr);

	UE_DEPRECATED(5.6, "Use the version with the flags parameter")
	PCG_API bool SetAttributeFromTag(const FParseTagResult& InTagData, UPCGMetadata* InMetadata, PCGMetadataEntryKey InKey, bool bInCanCreateAttribute);
}
}

#undef UE_API
