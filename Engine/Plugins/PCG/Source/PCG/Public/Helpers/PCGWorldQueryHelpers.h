// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Physics/PCGPhysicsRegistry.h"

#include "Containers/ArrayView.h"
#include "Engine/HitResult.h"
#include "Misc/Optional.h"
#include "UObject/ObjectKey.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakInterfacePtr.h"

#include "PCGWorldQueryHelpers.generated.h"

class AActor;
class IPCGGraphExecutionSource;
class FPCGMetadataDomain;
class UPCGComponent;
class UPCGMetadata;
struct FHitResult;
struct FOverlapResult;
struct FPCGPoint;
struct FPCGWorldCommonQueryParams;
struct FPCGWorldRaycastQueryParams;
struct FPCGWorldVolumetricQueryParams;

UENUM()
enum class EPCGWorldRaycastMode : uint8
{
	Infinite UMETA(ToolTip = "Use the direction vector with 'infinite' magnitude."),
	ScaledVector UMETA(ToolTip = "Use the direction vector 'as-is' for casting the ray with its current magnitude."),
	NormalizedWithLength UMETA(ToolTip = "Normalize the direction vector and apply the length directly."),
	Segments UMETA(ToolTip = "User provided end points. Must match input points N:N, N:1, or 1:N.")
};

namespace PCGWorldQueryConstants
{
	const FName ImpactAttribute = TEXT("ImpactResult");
	const FName ImpactPointAttribute = TEXT("ImpactPoint");
	const FName ImpactNormalAttribute = TEXT("ImpactNormal");
	const FName ImpactReflectionAttribute = TEXT("ImpactReflection");
	const FName ImpactDistanceAttribute = TEXT("ImpactDistance");
	const FName LocalImpactPointAttribute = TEXT("ImpactLocalPoint");
	const FName PhysicalMaterialReferenceAttribute = TEXT("PhysicalMaterial");
	const FName RenderMaterialReferenceAttribute = TEXT("ImpactRenderMaterial");
	const FName StaticMeshReferenceAttribute = TEXT("ImpactStaticMesh");
	const FName ElementIndexAttribute = TEXT("ImpactElementIndex");
	const FName UVCoordAttribute = TEXT("ImpactUVCoords");
	const FName FaceIndexAttribute = TEXT("ImpactFaceIndex");
	const FName SectionIndexAttribute = TEXT("ImpactSectionIndex");
	const FName RenderMaterialIndexAttribute = TEXT("ImpactRenderMaterialIndex");
}

namespace PCGWorldQueryHelpers
{
	FTransform GetOrthonormalImpactTransform(const FHitResult& Hit);

	
	/** Returns true if hit was handled, in this case bOutPassesFilter will be set. */
	bool FilterRayHitResult(const FPCGFilterHitResultParams& InParams, bool& bOutPassesFilter);
	
	/** Returns true if overlap was handled, in this case bOutPassesFilter will be set. */
	bool FilterOverlapResult(const FPCGFilterOverlapResultParams& InParams, bool& bOutPassesFilter);
	
	/** Returns true if hit was handled, in this case bOutSuccess will be set. */
	bool ApplyHitResultAttributes(const FPCGApplyHitResultAttributesParams& InParams, bool& bOutSuccess);

	/** Returns false if a physics query doesn't meet the requirements of common query parameters. */
	bool FilterCommonQueryResults(
		const FPCGWorldCommonQueryParams& QueryParams,
		const UPrimitiveComponent* TriggeredComponent,
		TWeakInterfacePtr<IPCGGraphExecutionSource> OriginatingExecutionSource,
		const TSet<FObjectKey>& FilteredObjectReferences);

	/** Filters through an array of hit results, testing them against raycast query parameters. Optional extra filter predicate. */
	TOptional<FHitResult> FilterRayHitResults(
		const FPCGWorldRaycastQueryParams& QueryParams,
		TWeakInterfacePtr<IPCGGraphExecutionSource> OriginatingExecutionSource,
		const TArray<FHitResult>& HitResults,
		const TSet<FObjectKey>& FilteredObjectReferences);

	/** Filters through an array of overlap results, testing them against raycast query parameters. Optional extra filter predicate. */
	TOptional<FOverlapResult> FilterOverlapResults(
		const FPCGWorldVolumetricQueryParams& QueryParams,
		TWeakInterfacePtr<IPCGGraphExecutionSource> OriginatingExecutionSource,
		const TArray<FOverlapResult>& OverlapResults,
		const TSet<FObjectKey>& FilteredObjectReferences);

	/** Creates hit result attributes based off query params. Can be called before ApplyRayHitMetadata. */
	bool CreateRayHitAttributes(const FPCGWorldRaycastQueryParams& QueryParams, UPCGMetadata* OutMetadata);
	
	/** Creates hit result attributes based off query params. Can be called before ApplyRayHitMetadata. */
	bool CreateRayHitAttributes(const FPCGWorldRaycastQueryParams& QueryParams, FPCGMetadataDomain* OutMetadata);

	/** Applies a 'miss' hit result to the metadata. To be called if the ray misses the target and the point should be kept. */
	bool ApplyRayMissMetadata(const FPCGWorldRaycastQueryParams& QueryParams, int64& OutMetadataEntry, UPCGMetadata* OutMetadata);

	/** Applies common world ray hit results to attributes. */
	bool ApplyRayHitMetadata(const FHitResult& HitResult,
		const FPCGWorldRaycastQueryParams& QueryParams,
		const FVector& RayDirection,
		const FTransform& InTransform, 
		int64& OutMetadataEntry,
		UPCGMetadata* OutMetadata,
		TWeakObjectPtr<UWorld> World);
}
