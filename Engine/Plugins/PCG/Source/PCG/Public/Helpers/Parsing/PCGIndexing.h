// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Helper namespace for handling index based evaluations. */
namespace PCGIndexing
{
	/** A simple data structure to represent a range of indices [X,Y). Validating ranges are client responsibility. */
	struct FPCGIndexRange
	{
		FPCGIndexRange(const int32 InStartIndex, const int32 InEndIndex) : StartIndex(InStartIndex), EndIndex(InEndIndex) {}

		/** Returns true if the index can be found in this range. */
		bool ContainsIndex(int32 Index) const;

		/** Returns the total discrete indices within the range. */
		int32 GetIndexCount() const;

		bool operator<(const FPCGIndexRange& OtherRange) const;
		bool operator==(const FPCGIndexRange& OtherRange) const;
		bool operator!=(const FPCGIndexRange& OtherRange) const;

		int32 StartIndex;
		int32 EndIndex;
	};

	/** An abstract collection of FPCGIndexRange data that represents a concrete set of indices. */
	class FPCGIndexCollection
	{
	public:
		static FPCGIndexCollection Invalid() { return {}; }

		/** The constructor must accept the size of the array to support negative terminating indices. */
		explicit FPCGIndexCollection(const int32 InArraySize) : ArraySize(FMath::Max(InArraySize, -1)) {}

		/** Direct conversion from an array view of indices. Size is implicitly calculated from max value. */
		explicit FPCGIndexCollection(const TArrayView<const int32> InArrayView);

		/** Direct conversion from an array view of indices. Size is explicit. */
		FPCGIndexCollection(const TArrayView<const int32> InArrayView, int32 InArraySize);

		/** Add a new index range to the collection directly via start and end indices. */
		bool AddRange(int32 StartIndex, int32 EndIndex);

		/** Add a new index range to the collection directly via an index range. */
		bool AddRange(const FPCGIndexRange& NewRange);

		/** Validates that a range is acceptable for this collection. Returns true if valid, false otherwise */
		bool RangeIsValid(const FPCGIndexRange& Range) const;

		/** Returns true if this collection contains the given index. */
		bool ContainsIndex(int32 InIndex) const;

		/** Returns the abstract size of the array tied to the collection. */
		int32 GetArraySize() const;

		/** Gets the number of range structures currently in the collection. */
		int32 GetTotalRangeCount() const;

		/** Gets the total number of concrete indices within the collection. */
		int32 GetTotalIndexCount() const;

		/** The index collection has a valid array size. */
		bool IsValid() const;

		/** The index collection has no ranges. */
		bool IsEmpty() const;

		/** Iterate over the entire range of indices. Action should return true if the loop should terminate */
		void ForEach(const TFunctionRef<bool(FPCGIndexRange)> Action) const;

		bool operator==(const FPCGIndexCollection& Other) const;
		FPCGIndexCollection& operator+=(const FPCGIndexCollection& Other);
		FPCGIndexCollection& operator+=(const TArrayView<const int32>& InArrayView);

	private:
		FPCGIndexCollection() = default;

		/** Adjusts indices into a positive range, including offsetting the EndIndex if they are equal. */
		FPCGIndexRange AdjustIndicesAndCreateRange(int32 StartIndex, int32 EndIndex) const;

		/** Checks two ranges for an overlap and returns true if they overlap. */
		bool CheckOverlap(const FPCGIndexRange& FirstRange, const FPCGIndexRange& SecondRange) const;

		/** Returns the resultant range after merging two overlapping ranges. */
		FPCGIndexRange MergeRanges(const FPCGIndexRange& FirstRange, const FPCGIndexRange& SecondRange) const;

		/** Helper to add an array of indices. If marked to grow, the max value will be used to determine the new array size. */
		void AddIndices(const TArrayView<const int32> InArrayView, bool bGrowIfNeeded = false);

		/** The size of the representative array associated with this collection. Needed for negative terminating indices. */
		int32 ArraySize = 0;

		/** A collection of the abstract index ranges in the collection. */
		TArray<FPCGIndexRange, TInlineAllocator<64>> IndexRanges;
	};
}
