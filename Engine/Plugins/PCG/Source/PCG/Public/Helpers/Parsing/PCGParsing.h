// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGIndexing.h"

namespace PCGIndexing
{
	class FPCGIndexCollection;
}

/** This helper namespace to be used for helper functions dedicated to parsing various input strings. */
namespace PCGParser
{
	enum class EPCGParserResult
	{
		Success,
		InvalidCharacter,
		InvalidExpression,
		EmptyExpression,
	};

	/** Indices should be included individually or with ranges using a delimiter. Negative terminating ranges are accepted.
	* For example, on an array of size 10: '0,2,4:5,7:-1' will include indices: 0,2,4,5,7,8
	*/
	PCG_API EPCGParserResult ParseIndexRanges(PCGIndexing::FPCGIndexCollection& OutIndexCollection, const FStringView& InputString);
}
