// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataCommon.h"

#include "PCGInstanceDataPackerBase.generated.h"

struct FPCGAttributePropertyInputSelector;
struct FPCGContext;
struct FPCGMeshInstanceList;
class FPCGMetadataAttributeBase;
class FPCGMetadataDomain;
class IPCGAttributeAccessor;
class IPCGAttributeAccessorKeys;
class UPCGData;
class UPCGMetadata;
class UPCGSpatialData;

USTRUCT(BlueprintType)
struct FPCGPackedCustomData
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	int NumCustomDataFloats = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<float> CustomData;

	/** If the custom data was sparsely set, mark the custom data index corresponding bit. */
	TBitArray<> Mask;

	PCG_API TConstArrayView<float> GetViewForIndex(int32 Index) const;
	PCG_API void Reset();
};

namespace PCGInstanceDataPackerBase
{
	struct FPackedDataCommonParams
	{
		/** Number of different entries in the OutPackedCustomData. */
		int32 NumInstances = 0;

		/**
		 * Out structure containing the packed data. Must be filled beforehand for its member NumCustomDataFloats for the accessor case or if OptionalOffsets is not empty. Otherwise it can be left at 0, it will be automatically filled.
		 * If it is not 0, it'll be verified if the number is valid.
		 */
		FPCGPackedCustomData* OutPackedCustomData = nullptr;

		/** If the packing is sparse, offsets can be provided. It is mandatory that those offsets are not overlapping, in ascending order and there must be as many offsets that there are different values. */
		TConstArrayView<int32> OptionalOffsets;

		/** Optional context for logging. */
		FPCGContext* OptionalContext = nullptr;

		/** For validation that the offset at OffsetIndex is not overlapping with NumOfCustomFloats. */
		PCG_API bool ValidateOffsets(int32 OffsetIndex, int32 NumOfCustomFloats) const;

		/** For validation that the computed number of custom floats is less or equal than the expected value set in OutPackedCustomData. */
		PCG_API bool ValidateNumOfFloats(int32 NumOfCustomFloats) const;

		/** If there is any kind of error, zero out the buffer in OutPackedCustomData. */
		PCG_API void ZeroOutCustomData();
	};
	
	struct FPackedDataFromAccessorParams
	{
		FPackedDataCommonParams CommonParams;
		
		/** List of accessors to read from. */
		TConstArrayView<TUniquePtr<const IPCGAttributeAccessor>> Accessors;

		/** List of accessor keys. Can provide just a single key if there is one key for all the accessors. */
		TConstArrayView<TUniquePtr<const IPCGAttributeAccessorKeys>> AccessorKeys;
	};

	PCG_API void PackCustomDataFromAccessors(FPackedDataFromAccessorParams& InParams);

	struct FPackedDataFromAttributesParams
	{
		FPackedDataCommonParams CommonParams;

		/**
		 * If not all values from the data are read from, can provide a list of indices. Note: It is mandatory that the attributes are all on
		 * the same domain in that case, otherwise the mapping is ambiguous. The packing will fail if we have OptionalIndices and multi domain attributes.
		 */
		TConstArrayView<int32> OptionalIndices;

		/** Input data to read attributes from. */
		const UPCGData* InData = nullptr;

		/** List of attributes to read from. Has precedence on AttributeIdentifiers. */
		TConstArrayView<const FPCGMetadataAttributeBase*> Attributes;

		/** List of attribute identifiers to read from. Ignored if Attributes is not empty. */
		TConstArrayView<FPCGAttributeIdentifier> AttributeIdentifiers;
	};

	PCG_API void PackCustomDataFromAttributes(FPackedDataFromAttributesParams& InParams);

	struct FPackedDataParams
	{
		FPackedDataCommonParams CommonParams;

		/** Input data to read attributes from. */
		const UPCGData* InData = nullptr;

		/** List of selectors to read from. */
		TConstArrayView<FPCGAttributePropertyInputSelector> Selectors;
	};

	PCG_API void PackCustomData(FPackedDataParams& InParams);

	/** Will add the type packing size to OutPackedCustomData. Return false if the type is not supported. */
	PCG_API bool AddTypeToPacking(int TypeId, int32& OutNumCustomDataFloats);

	/** Return the type packing size. 0 if unsupported. */
	PCG_API int32 GetTypePackingSize(int TypeId);
}

UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, ClassGroup = (Procedural))
class UPCGInstanceDataPackerBase : public UObject 
{
	GENERATED_BODY()

public:
	/** Defines the strategy for (H)ISM custom float data packing */
	UFUNCTION(BlueprintNativeEvent, Category = InstancePacking)
	PCG_API void PackInstances(FPCGContext& Context, const UPCGSpatialData* InSpatialData, const FPCGMeshInstanceList& InstanceList, FPCGPackedCustomData& OutPackedCustomData) const;

	PCG_API virtual void PackInstances_Implementation(FPCGContext& Context, const UPCGSpatialData* InSpatialData, const FPCGMeshInstanceList& InstanceList, FPCGPackedCustomData& OutPackedCustomData) const;

	/** Interprets Metadata TypeId and increments OutPackedCustomData.NumCustomDataFloats appropriately. Returns false if the type could not be interpreted. */
	UFUNCTION(BlueprintCallable, Category = InstancePacking)
	bool AddTypeToPacking(int TypeId, FPCGPackedCustomData& OutPackedCustomData) const { return PCGInstanceDataPackerBase::AddTypeToPacking(TypeId, OutPackedCustomData.NumCustomDataFloats); }

	/** Build a PackedCustomData by processing each attribute in order for each point in the InstanceList */
	UFUNCTION(BlueprintCallable, Category = InstancePacking) 
	PCG_API void PackCustomDataFromAttributes(const FPCGMeshInstanceList& InstanceList, const UPCGMetadata* Metadata, const TArray<FName>& AttributeNames, FPCGPackedCustomData& OutPackedCustomData) const;

	/** Build a PackedCustomData by processing each attribute in order for each point in the InstanceList */
	PCG_API void PackCustomDataFromAttributes(const FPCGMeshInstanceList& InstanceList, const TArray<const FPCGMetadataAttributeBase*>& Attributes, FPCGPackedCustomData& OutPackedCustomData) const;

	/** Build a PackedCustomData by processing each accessor in order for each point in the InstanceList */
	PCG_API void PackCustomDataFromAccessors(const FPCGMeshInstanceList& InstanceList, TArray<TUniquePtr<const IPCGAttributeAccessor>> Accessors, TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> AccessorKeys, FPCGPackedCustomData& OutPackedCustomData) const;

	/** If OutNames is not null, returns a list of all attributes that will be packed. Returns true if this list can be statically determined (prior to execution). */
	virtual bool GetAttributeNames(TArray<FName>* OutNames) { return false; }

	/** Returns a list of all attribute selectors that will be packed, if they can be statically determined (prior to execution). */
	virtual TOptional<TConstArrayView<FPCGAttributePropertyInputSelector>> GetAttributeSelectors() const { return {}; }
};

