// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGInstanceDataPackerBase.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGInstanceDataPackerByAttribute.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGInstanceDataPackerByAttribute : public UPCGInstanceDataPackerBase 
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface
#endif
	virtual void PackInstances_Implementation(UPARAM(ref) FPCGContext& Context, const UPCGSpatialData* InSpatialData, UPARAM(ref) const FPCGMeshInstanceList& InstanceList, FPCGPackedCustomData& OutPackedCustomData) const override;
	virtual bool GetAttributeNames(TArray<FName>* OutNames) override;
	virtual TOptional<TConstArrayView<FPCGAttributePropertyInputSelector>> GetAttributeSelectors() const override;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = InstanceDataPacker)
	TArray<FPCGAttributePropertyInputSelector> AttributeSelectors;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "AttributeNames has been deprecated. Use AttributeSelectors instead.")
	UPROPERTY(BlueprintReadWrite, Category = InstanceDataPacker, meta = (DeprecatedProperty, DeprecationMessage = "AttributeNames has been deprecated. Use AttributeSelectors instead."))
	TArray<FName> AttributeNames;
#endif
};
