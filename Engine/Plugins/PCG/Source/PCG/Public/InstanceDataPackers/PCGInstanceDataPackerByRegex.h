// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGInstanceDataPackerBase.h"

#include "PCGInstanceDataPackerByRegex.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGInstanceDataPackerByRegex : public UPCGInstanceDataPackerBase 
{
	GENERATED_BODY()

public:
	virtual void PackInstances_Implementation(UPARAM(ref) FPCGContext& Context, const UPCGSpatialData* InSpatialData, UPARAM(ref) const FPCGMeshInstanceList& InstanceList, FPCGPackedCustomData& OutPackedCustomData) const override;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = InstanceDataPacker)
	TArray<FString> RegexPatterns; 
};
