// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSkinnedMeshInstanceDataPackerBase.generated.h"

#define UE_API PCG_API

class FPCGMetadataAttributeBase;
class IPCGAttributeAccessor;
class IPCGAttributeAccessorKeys;
class UPCGMetadata;
class UPCGSpatialData;
struct FPCGAttributePropertyInputSelector;
struct FPCGContext;
struct FPCGSkinnedMeshInstanceList;

USTRUCT(BlueprintType)
struct FPCGSkinnedMeshPackedCustomData
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	int SequenceIndex = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	int NumCustomDataFloats = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<float> CustomData;
};

UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, ClassGroup = (Procedural))
class UPCGSkinnedMeshInstanceDataPackerBase : public UObject 
{
	GENERATED_BODY()

public:
	/** Defines the strategy for custom float data packing */
	UFUNCTION(BlueprintNativeEvent, Category = InstancePacking)
	UE_API void PackInstances(FPCGContext& Context, const UPCGSpatialData* InSpatialData, const FPCGSkinnedMeshInstanceList& InstanceList, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const;

	UE_API virtual void PackInstances_Implementation(FPCGContext& Context, const UPCGSpatialData* InSpatialData, const FPCGSkinnedMeshInstanceList& InstanceList, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const;

	/** Interprets Metadata TypeId and increments OutPackedCustomData.NumCustomDataFloats appropriately. Returns false if the type could not be interpreted. */
	UFUNCTION(BlueprintCallable, Category = InstancePacking)
	UE_API bool AddTypeToPacking(int TypeId, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const;

	/** Build a PackedCustomData by processing each attribute in order for each point in the InstanceList */
	UFUNCTION(BlueprintCallable, Category = InstancePacking) 
	UE_API void PackCustomDataFromAttributes(const FPCGSkinnedMeshInstanceList& InstanceList, const UPCGMetadata* Metadata, const TArray<FName>& AttributeNames, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const;

	/** Build a PackedCustomData by processing each attribute in order for each point in the InstanceList */
	UE_API void PackCustomDataFromAttributes(const FPCGSkinnedMeshInstanceList& InstanceList, const TArray<const FPCGMetadataAttributeBase*>& Attributes, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const;

	/** Build a PackedCustomData by processing each accessor in order for each point in the InstanceList */
	UE_API void PackCustomDataFromAccessors(const FPCGSkinnedMeshInstanceList& InstanceList, TArray<TUniquePtr<const IPCGAttributeAccessor>> Accessors, TArray<TUniquePtr<const IPCGAttributeAccessorKeys>> AccessorKeys, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const;

	/** If OutNames is not null, returns a list of all attributes that will be packed. Returns true if this list can be statically determined (prior to execution). */
	virtual bool GetAttributeNames(TArray<FName>* OutNames) { return false; }

	/** Returns a list of all attribute selectors that will be packed, if they can be statically determined (prior to execution). */
	virtual TOptional<TConstArrayView<FPCGAttributePropertyInputSelector>> GetAttributeSelectors() const { return {}; }
};

#undef UE_API
