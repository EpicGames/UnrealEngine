// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSkinnedMeshInstanceDataPackerBase.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGSkinnedMeshInstanceDataPackerByAttribute.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSkinnedMeshInstanceDataPackerByAttribute : public UPCGSkinnedMeshInstanceDataPackerBase
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface
#endif
	virtual void PackInstances_Implementation(UPARAM(ref) FPCGContext& Context, const UPCGSpatialData* InSpatialData, UPARAM(ref) const FPCGSkinnedMeshInstanceList& InstanceList, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const override;
	virtual bool GetAttributeNames(TArray<FName>* OutNames) override;
	virtual TOptional<TConstArrayView<FPCGAttributePropertyInputSelector>> GetAttributeSelectors() const override;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = InstanceDataPacker)
	TArray<FPCGAttributePropertyInputSelector> AttributeSelectors;
};
