// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSkinnedMeshInstanceDataPackerBase.h"
#include "PCGSkinnedMeshInstanceDataPackerByRegex.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSkinnedMeshInstanceDataPackerByRegex : public UPCGSkinnedMeshInstanceDataPackerBase
{
	GENERATED_BODY()

public:
	virtual void PackInstances_Implementation(UPARAM(ref) FPCGContext& Context, const UPCGSpatialData* InSpatialData, UPARAM(ref) const FPCGSkinnedMeshInstanceList& InstanceList, FPCGSkinnedMeshPackedCustomData& OutPackedCustomData) const override;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = InstanceDataPacker)
	TArray<FString> RegexPatterns; 
};
