// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataTypesConstantStruct.h"

#include "PCGMatchAndSetBase.generated.h"

#define UE_API PCG_API

class UPCGPointData;

struct FPCGAttributePropertySelector;
struct FPCGContext;
struct FPCGMetadataTypesConstantStruct;
struct FPCGPoint;
class UPCGMetadata;
class UPCGPointMatchAndSetSettings;

/** Base class for Match & Set objects. Note that while it currently deals with points, it might be extended in the future.
* This class is extensible and can be implemented in different ways, but its role should be simple:
* For a given point, if it matches some criteria ("Match"), apply it some value ("Set").
* It can be a lookup, a random process or something more involved. 
*/
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, ClassGroup = (Procedural))
class UPCGMatchAndSetBase : public UObject
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
#if WITH_EDITOR
	UE_API virtual void PostLoad() override;
#endif
	//~End UObject interface

	/** Queries whether this object uses a random process - note that this is expected to be static through the life of the object */
	virtual bool UsesRandomProcess() const { return false; }

	/** Queries whether we should mutate the seeds as a post-process */
	virtual bool ShouldMutateSeed() const { return false; }

	/** Sets & propagates type change from the owner settings object */
	UE_API virtual void SetType(EPCGMetadataTypes InType);

	/** Early check to prevent issues when the data does not contain the required information to perform the operation */
	UFUNCTION(BlueprintNativeEvent, Category = Selection)
	UE_API bool ValidatePreconditions(const UPCGPointData* InPointData) const;

	virtual bool ValidatePreconditions_Implementation(const UPCGPointData* InPointData) const { return true; }

	/** Main function to process points, and pass them through the Match & Set logic. */
	UFUNCTION(BlueprintNativeEvent, Category = Selection)
	UE_API void MatchAndSet(
		UPARAM(ref) FPCGContext& Context,
		const UPCGPointMatchAndSetSettings* InSettings,
		const UPCGPointData* InPointData,
		UPCGPointData* OutPointData) const;

	UE_API virtual void MatchAndSet_Implementation(
		FPCGContext& Context,
		const UPCGPointMatchAndSetSettings* InSettings,
		const UPCGPointData* InPointData,
		UPCGPointData* OutPointData) const;

protected:
	UE_API bool CreateAttributeIfNeeded(
		FPCGContext& Context,
		const FPCGAttributePropertySelector& Selector,
		const FPCGMetadataTypesConstantStruct& Value,
		UPCGPointData* OutPointData,
		const UPCGPointMatchAndSetSettings* InSettings) const;

	/** For the sake of managing internal state a bit better, we keep a copy of the Set type & string subtype. */
	UPROPERTY()
	EPCGMetadataTypes Type = EPCGMetadataTypes::Double;

	UPROPERTY()
	EPCGMetadataTypesConstantStructStringMode StringMode_DEPRECATED;
};

#undef UE_API
