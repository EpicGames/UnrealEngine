// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MatchAndSet/PCGMatchAndSetBase.h"

#include "PCGMatchAndSetByAttribute.generated.h"

#define UE_API PCG_API

struct FPropertyChangedEvent;

USTRUCT(BlueprintType)
struct FPCGMatchAndSetByAttributeEntry
{
	GENERATED_BODY()

	UE_API FPCGMatchAndSetByAttributeEntry();

#if WITH_EDITOR
	UE_API void OnPostLoad();
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties))
	FPCGMetadataTypesConstantStruct ValueToMatch;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties))
	FPCGMetadataTypesConstantStruct Value;
};

/** 
* This Match & Set object looks up an attribute on a given point,
* then looks up its entries to find a match; if there is one, then it sets it value. 
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMatchAndSetByAttribute : public UPCGMatchAndSetBase
{
	GENERATED_BODY()

public:
	virtual void SetType(EPCGMetadataTypes InType) override;

	/** Propagates (does not set) the Match type to the entries */
	virtual void SetSourceType(EPCGMetadataTypes InType);

	virtual void MatchAndSet_Implementation(
		FPCGContext& Context,
		const UPCGPointMatchAndSetSettings* InSettings,
		const UPCGPointData* InPointData,
		UPCGPointData* OutPointData) const override;

	virtual bool ValidatePreconditions_Implementation(const UPCGPointData* InPointData) const override;

	//~Begin UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
#endif
	//~End UObject interface

public:
	/** Attribute to match on the data */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MatchAndSet)
	FName MatchSourceAttribute;

	/** Type of the attribute to match against. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MatchAndSet)
	EPCGMetadataTypes MatchSourceType = EPCGMetadataTypes::Double;

	/** String type of the attribute to match against (if required). */
	UPROPERTY()
	EPCGMetadataTypesConstantStructStringMode MatchSourceStringMode_DEPRECATED;

	/** Lookup entries (key-value pairs) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MatchAndSet)
	TArray<FPCGMatchAndSetByAttributeEntry> Entries;
};

#undef UE_API
