// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MatchAndSet/PCGMatchAndSetBase.h"

#include "PCGMatchAndSetWeighted.generated.h"

#define UE_API PCG_API

enum class EPCGMetadataTypes : uint8;
struct FPropertyChangedEvent;

USTRUCT(BlueprintType)
struct FPCGMatchAndSetWeightedEntry
{
	GENERATED_BODY()

#if WITH_EDITOR
	UE_API void OnPostLoad();
#endif

	UE_API FPCGMatchAndSetWeightedEntry();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties))
	FPCGMetadataTypesConstantStruct Value;

	/** Relative weight of this entry */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0"))
	int Weight = 1;
};

/**
* This Match & Set object assigns randomly a value based on weighted ratios,
* provided in the entries.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMatchAndSetWeighted : public UPCGMatchAndSetBase
{
	GENERATED_BODY()

public:
	virtual bool UsesRandomProcess() const { return true; }
	virtual bool ShouldMutateSeed() const { return bShouldMutateSeed; }
	UE_API virtual void SetType(EPCGMetadataTypes InType) override;

	UE_API virtual void MatchAndSet_Implementation(
		FPCGContext& Context,
		const UPCGPointMatchAndSetSettings* InSettings,
		const UPCGPointData* InPointData,
		UPCGPointData* OutPointData) const override;

	//~Begin UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostLoad() override;
#endif
	//~End UObject interface

public:
	/** Values and their respective weights */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MatchAndSet)
	TArray<FPCGMatchAndSetWeightedEntry> Entries;

	/** Controls whether the output data should mutate its seed - prevents issues when doing multiple random processes in a row */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MatchAndSet)
	bool bShouldMutateSeed = true;
};

#undef UE_API
