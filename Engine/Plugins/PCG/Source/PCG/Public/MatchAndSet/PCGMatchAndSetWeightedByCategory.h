// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MatchAndSet/PCGMatchAndSetBase.h"
#include "MatchAndSet/PCGMatchAndSetWeighted.h"

#include "PCGMatchAndSetWeightedByCategory.generated.h"

#define UE_API PCG_API

struct FPCGMatchAndSetWeightedEntry;
struct FPropertyChangedEvent;

USTRUCT(BlueprintType)
struct FPCGMatchAndSetWeightedByCategoryEntryList
{
	GENERATED_BODY()

	UE_API FPCGMatchAndSetWeightedByCategoryEntryList();

#if WITH_EDITOR
	UE_API void OnPostLoad();
#endif

	UE_API void SetType(EPCGMetadataTypes InType);
	UE_API int GetTotalWeight() const;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties))
	FPCGMetadataTypesConstantStruct CategoryValue;

	/** If the category is the default, if the input does not match to anything, it will use this category. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool bIsDefault = false;

	/** Values and their weights */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FPCGMatchAndSetWeightedEntry> WeightedEntries;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMatchAndSetWeightedByCategory : public UPCGMatchAndSetBase
{
	GENERATED_BODY()

public:
	virtual bool UsesRandomProcess() const { return true; }
	virtual bool ShouldMutateSeed() const { return bShouldMutateSeed; }
	UE_API virtual void SetType(EPCGMetadataTypes InType) override;
	/** Propagates (does not set) category (e.g. Match) type to entries. */
	UE_API virtual void SetCategoryType(EPCGMetadataTypes InType);

	UE_API virtual void MatchAndSet_Implementation(
		FPCGContext& Context,
		const UPCGPointMatchAndSetSettings* InSettings,
		const UPCGPointData* InPointData,
		UPCGPointData* OutPointData) const override;

	UE_API virtual bool ValidatePreconditions_Implementation(const UPCGPointData* InPointData) const override;

	//~Begin UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostLoad() override;
#endif
	//~End UObject interface

public:
	/** Attribute to match against */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MatchAndSet)
	FName CategoryAttribute;

	/** Type of the attribute to match against. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MatchAndSet)
	EPCGMetadataTypes CategoryType = EPCGMetadataTypes::Double;

	UPROPERTY()
	EPCGMetadataTypesConstantStructStringMode CategoryStringMode_DEPRECATED;

	/** Lookup entries (key -> weighted list) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MatchAndSet)
	TArray<FPCGMatchAndSetWeightedByCategoryEntryList> Categories;

	/** Controls whether the output data should mutate its seed - prevents issues when doing multiple random processes in a row */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MatchAndSet)
	bool bShouldMutateSeed = true;
};

#undef UE_API
