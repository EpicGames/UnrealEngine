// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/CollisionProfile.h"
#include "ISMPartition/ISMComponentDescriptor.h"

#include "PCGISMDescriptor.generated.h"

/** Convenience PCG-side component descriptor so we can adjust defaults to the most common use cases. */
// Implementation note: the tags don't really need to contribute to the hash, so we will retain the base class !=, == and ComputeHash implementations
USTRUCT()
struct FPCGSoftISMComponentDescriptor : public FSoftISMComponentDescriptor
{
	GENERATED_BODY()

	PCG_API FPCGSoftISMComponentDescriptor();
	PCG_API explicit FPCGSoftISMComponentDescriptor(const FISMComponentDescriptor& Other);
	PCG_API virtual void InitFrom(const UStaticMeshComponent* Component, bool bInitBodyInstance = true) override;
	PCG_API virtual void InitComponent(UInstancedStaticMeshComponent* ISMComponent) const override;
	bool HasTags() const { return !ComponentTags.IsEmpty() || !AdditionalCommaSeparatedTags.IsEmpty(); }

public:
	UPROPERTY(EditAnywhere, Category="Component Settings|Tags", meta = (DisplayAfter = "ComponentClass"))
	TArray<FName> ComponentTags;

	UPROPERTY(EditAnywhere, Category="Component Settings|Tags", meta = (DisplayAfter = "ComponentTags", Tooltip="Comma-separated tag list. Normally should be left empty, and used as a property override instead."))
	FString AdditionalCommaSeparatedTags;

	PCG_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

template<>
struct TStructOpsTypeTraits<FPCGSoftISMComponentDescriptor> : public TStructOpsTypeTraitsBase2<FPCGSoftISMComponentDescriptor>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};
