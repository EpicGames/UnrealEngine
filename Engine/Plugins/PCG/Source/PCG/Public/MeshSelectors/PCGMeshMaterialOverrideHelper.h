// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadata.h"

#include "PCGMeshMaterialOverrideHelper.generated.h"

#define UE_API PCG_API

class UMaterialInterface;

UENUM()
enum class EPCGMeshSelectorMaterialOverrideMode : uint8
{
	NoOverride UMETA(Tooltip = "Does not apply any material overrides to the spawned mesh(es)"),
	StaticOverride UMETA(Tooltip = "Applies the material overrides provided in the Static Material Overrides array"),
	ByAttributeOverride UMETA(Tooltip = "Applies the materials overrides using the point data attribute(s) specified in the By Attribute Material Overrides array")
};

/** Struct used to efficiently gather overrides and cache them during instance packing */
struct FPCGMeshMaterialOverrideHelper
{
	FPCGMeshMaterialOverrideHelper() = default;

	// Use this constructor when you have a 1:1 mapping between attributes or static overrides
	UE_API void Initialize(
		FPCGContext& InContext,
		bool bUseMaterialOverrideAttributes,
		const TArray<TSoftObjectPtr<UMaterialInterface>>& InStaticMaterialOverrides,
		const TArray<FName>& InMaterialOverrideAttributeNames,
		const UPCGMetadata* InMetadata);

	// Use this constructor when you have common attribute usage or separate static overrides
	UE_API void Initialize(
		FPCGContext& InContext,
		bool bInByAttributeOverride,
		const TArray<FName>& InMaterialOverrideAttributeNames,
		const UPCGMetadata* InMetadata);

	UE_API void Reset();

	bool IsInitialized() const { return bIsInitialized; }
	bool IsValid() const { return bIsValid; }
	bool OverridesMaterials() const { return bUseMaterialOverrideAttributes; }
	UE_API const TArray<TSoftObjectPtr<UMaterialInterface>>& GetMaterialOverrides(PCGMetadataEntryKey EntryKey);

private:
	// Cached data
	TArray<const FPCGMetadataAttributeBase*> MaterialAttributes;
	TArray<TMap<PCGMetadataValueKey, TSoftObjectPtr<UMaterialInterface>>> ValueKeyToOverrideMaterials;
	TArray<TSoftObjectPtr<UMaterialInterface>> WorkingMaterialOverrides;

	// Data needed to perform operations
	bool bIsInitialized = false;
	bool bIsValid = false;
	bool bUseMaterialOverrideAttributes = false;

	TArray<TSoftObjectPtr<UMaterialInterface>> StaticMaterialOverrides;
	TArray<FName> MaterialOverrideAttributeNames;
	const UPCGMetadata* Metadata = nullptr;

	UE_API void Initialize(FPCGContext& InContext);
};

#undef UE_API
