// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPoint.h"
#include "Data/PCGPointData.h"
#include "MeshSelectors/PCGISMDescriptor.h"
#include "MeshSelectors/PCGMeshMaterialOverrideHelper.h"
#include "Metadata/PCGMetadata.h"

#include "Engine/CollisionProfile.h"

#include "PCGMeshSelectorBase.generated.h"

#define UE_API PCG_API

class UPCGPointData;
class UPCGSpatialData;
class UStaticMesh;
struct FPCGContext;
struct FPCGStaticMeshSpawnerContext;
class UPCGStaticMeshSpawnerSettings;

USTRUCT(BlueprintType)
struct FPCGMeshInstanceList
{
	GENERATED_BODY()

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	explicit FPCGMeshInstanceList(const FPCGSoftISMComponentDescriptor& InDescriptor)
		: Descriptor(InDescriptor)
		, AttributePartitionIndex(INDEX_NONE)
	{}
	
	FPCGMeshInstanceList() = default;
	~FPCGMeshInstanceList() = default;
	FPCGMeshInstanceList(const FPCGMeshInstanceList&) = default;
	FPCGMeshInstanceList(FPCGMeshInstanceList&&) = default;
	FPCGMeshInstanceList& operator=(const FPCGMeshInstanceList&) = default;
	FPCGMeshInstanceList& operator=(FPCGMeshInstanceList&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY(EditAnywhere, Category = Settings)
	FPCGSoftISMComponentDescriptor Descriptor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FTransform> Instances;

	/** Tracks which partition the instance list belongs to. */
	int64 AttributePartitionIndex;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TWeakObjectPtr<const UPCGBasePointData> PointData;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<int32> InstancesIndices;

	UPROPERTY()
	TArray<float> CustomPrimitiveData;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "Use PointData + InstanceIndices instead.")
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<int64> InstancesMetadataEntry;
#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = (Procedural))
class UPCGMeshSelectorBase : public UObject 
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.6, "Call / Override SelectMeshInstances with UPCGBasePointData parameter")
	virtual bool SelectInstances(
		FPCGStaticMeshSpawnerContext& Context,
		const UPCGStaticMeshSpawnerSettings* Settings,
		const UPCGPointData* InPointData,
		TArray<FPCGMeshInstanceList>& OutMeshInstances,
		UPCGPointData* OutPointData) const { return true; }

	UE_API virtual bool SelectMeshInstances(
		FPCGStaticMeshSpawnerContext& Context,
		const UPCGStaticMeshSpawnerSettings* Settings,
		const UPCGBasePointData* InPointData,
		TArray<FPCGMeshInstanceList>& OutMeshInstances,
		UPCGBasePointData* OutPointData) const;
};

#undef UE_API
