// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshSelectors/PCGMeshSelectorBase.h"
#include "MeshSelectors/PCGISMDescriptor.h"

#include "PCGMeshSelectorByAttribute.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMeshSelectorByAttribute : public UPCGMeshSelectorBase 
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	virtual void PostLoad() override;
	// ~End UObject interface

	virtual bool SelectMeshInstances(
		FPCGStaticMeshSpawnerContext& Context,
		const UPCGStaticMeshSpawnerSettings* Settings,
		const UPCGBasePointData* InPointData,
		TArray<FPCGMeshInstanceList>& OutMeshInstances,
		UPCGBasePointData* OutPointData) const override;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSelector)
	FName AttributeName;

	UPROPERTY(EditAnywhere, Category = MeshSelector)
	FPCGSoftISMComponentDescriptor TemplateDescriptor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSelector, meta = (InlineEditConditionToggle))
	bool bUseAttributeMaterialOverrides = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, DisplayName = "By Attribute Material Overrides", Category = MeshSelector, meta = (EditCondition = "bUseAttributeMaterialOverrides"))
	TArray<FName> MaterialOverrideAttributes;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bOverrideCollisionProfile_DEPRECATED = false;

	UPROPERTY()
	FCollisionProfileName CollisionProfile_DEPRECATED = UCollisionProfile::NoCollision_ProfileName;

	UPROPERTY()
	EPCGMeshSelectorMaterialOverrideMode MaterialOverrideMode_DEPRECATED = EPCGMeshSelectorMaterialOverrideMode::NoOverride;

	UPROPERTY()
	bool bOverrideMaterials_DEPRECATED = false;

	UPROPERTY()
	TArray<TSoftObjectPtr<UMaterialInterface>> MaterialOverrides_DEPRECATED;

	/** Distance at which instances begin to fade. */
	UPROPERTY()
	float CullStartDistance_DEPRECATED = 0;
	
	/** Distance at which instances are culled. Use 0 to disable. */
	UPROPERTY()
	float CullEndDistance_DEPRECATED = 0;

	UPROPERTY()
	int32 WorldPositionOffsetDisableDistance_DEPRECATED = 0;
#endif // WITH_EDITORONLY_DATA
};