// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshSelectors/PCGMeshSelectorBase.h"

#include "Metadata/PCGObjectPropertyOverride.h"

#include "MeshSelectors/PCGISMDescriptor.h"

#include "PCGMeshSelectorPrimitiveData.generated.h"

class UPCGBasePointData;
class UPCGStaticMeshSpawnerSettings;
struct FPCGMeshInstanceList;
struct FPCGPackedCustomData;
struct FPCGStaticMeshSpawnerContext;

/** Selector type that uses a table of primitive data (one row per primitive) and expects a table row index on the input points. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMeshSelectorPrimitiveData : public UPCGMeshSelectorBase
{
	GENERATED_BODY()

public:
	PCG_API virtual bool SelectMeshInstances(
		FPCGStaticMeshSpawnerContext& Context,
		const UPCGStaticMeshSpawnerSettings* Settings,
		const UPCGBasePointData* InPointData,
		TArray<FPCGMeshInstanceList>& OutMeshInstances,
		UPCGBasePointData* OutPointData) const override;

	void PackCustomPrimitiveData(const UPCGData* InputData, int32 NumInstances, FPCGPackedCustomData& OutCustomPrimitiveData, FPCGContext* OptionalContext = nullptr) const;

public:
	/** Base/default settings for the spawned primitive before overrides are applied. */
	UPROPERTY(EditAnywhere, Category = "Primitive Setup")
	FPCGSoftISMComponentDescriptor TemplateDescriptor;

	/** Attribute on input source data that serves as a row index into the primitive table. */
	UPROPERTY(EditAnywhere, Category = "Primitive Setup")
	FName PrimitiveIndexAttribute;

	/** Attribute in primitive data that provides the mesh for the spawned primitives. */
	UPROPERTY(EditAnywhere, Category = "Primitive Setup")
	FName MeshAttribute = TEXT("Mesh");

	/** Per-material-slot list of attributes in primitive data that specify materials to override on the spawned primitives. */
	UPROPERTY(EditAnywhere, Category = "Primitive Setup")
	TArray<FName> MaterialOverrideAttributes;

	/** Array of attributes in primitive data that drive properties on the spawned primitives. */
	UPROPERTY(EditAnywhere, Category = "Primitive Setup")
	TArray<FPCGObjectPropertyOverrideDescription> PrimitiveOverrideAttributes;

	UPROPERTY(EditAnywhere, Category = "Primitive Setup")
	TArray<FPCGAttributePropertyInputSelector> CustomPrimitiveDataAttributes;
};
