// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshSelectors/PCGMeshSelectorBase.h"
#include "MeshSelectors/PCGISMDescriptor.h"

#include "PCGMeshSelectorWeighted.generated.h"

#define UE_API PCG_API

namespace PCGMeshSelectorWeighted
{
	// Returns variation matching the overrides & the reverse culling flag
	FPCGMeshInstanceList& GetInstanceList(
		TArray<FPCGMeshInstanceList>& InstanceLists,
		bool bUseMaterialOverrides,
		const TArray<TSoftObjectPtr<UMaterialInterface>>& InMaterialOverrides,
		bool bInIsLocalToWorldDeterminantNegative,
		const UPCGBasePointData* InPointData);
}

USTRUCT(BlueprintType)
struct FPCGMeshSelectorWeightedEntry
{
	GENERATED_BODY()

	FPCGMeshSelectorWeightedEntry() = default;
	UE_API FPCGMeshSelectorWeightedEntry(TSoftObjectPtr<UStaticMesh> InMesh, int InWeight);

#if WITH_EDITOR
	UE_API void ApplyDeprecation();
#endif // WITH_EDITOR

	UPROPERTY(EditAnywhere, Category = Settings)
	FPCGSoftISMComponentDescriptor Descriptor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0"))
	int Weight = 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, VisibleAnywhere, Category = Settings)
	FName DisplayName = NAME_None;

	UPROPERTY()
	TSoftObjectPtr<UStaticMesh> Mesh_DEPRECATED;

	UPROPERTY()
	bool bOverrideCollisionProfile_DEPRECATED = false;

	UPROPERTY()
	FCollisionProfileName CollisionProfile_DEPRECATED = UCollisionProfile::NoCollision_ProfileName;

	UPROPERTY()
	bool bOverrideMaterials_DEPRECATED = false;

	UPROPERTY()
	TArray<TSoftObjectPtr<UMaterialInterface>> MaterialOverrides_DEPRECATED;

	/** Distance at which instances begin to fade. */
	UPROPERTY()
	float CullStartDistance_DEPRECATED = 0;
	
	/** Distance at which instances are culled. Use 0 to disable. */
	UPROPERTY()
	float CullEndDistance_DEPRECATED = 0;

	UPROPERTY()
	int32 WorldPositionOffsetDisableDistance_DEPRECATED = 0;
#endif
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMeshSelectorWeighted : public UPCGMeshSelectorBase 
{
	GENERATED_BODY()

public:
	virtual bool SelectMeshInstances(
		FPCGStaticMeshSpawnerContext& Context,
		const UPCGStaticMeshSpawnerSettings* Settings, 
		const UPCGBasePointData* InPointData,
		TArray<FPCGMeshInstanceList>& OutMeshInstances,
		UPCGBasePointData* OutPointData) const override;

	// ~Begin UObject interface
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostEditImport() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// ~End UObject interface

	/** Refresh MeshEntries display names */
	PCG_API void RefreshDisplayNames();
#endif // WITH_EDITOR

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSelector, meta = (TitleProperty = "DisplayName"))
	TArray<FPCGMeshSelectorWeightedEntry> MeshEntries;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSelector, meta = (InlineEditConditionToggle))
	bool bUseAttributeMaterialOverrides = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, DisplayName = "By Attribute Material Overrides", Category = MeshSelector, meta = (EditCondition = "bUseAttributeMaterialOverrides"))
	TArray<FName> MaterialOverrideAttributes;
};

#undef UE_API
