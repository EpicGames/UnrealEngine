// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshSelectors/PCGMeshSelectorBase.h"
#include "PCGMeshSelectorWeighted.h"

#include "PCGMeshSelectorWeightedByCategory.generated.h"

#define UE_API PCG_API

USTRUCT(BlueprintType)
struct FPCGWeightedByCategoryEntryList
{
	GENERATED_BODY()

	FPCGWeightedByCategoryEntryList() = default;

	FPCGWeightedByCategoryEntryList(const FString& InCategoryEntry, const TArray<FPCGMeshSelectorWeightedEntry>& InWeightedMeshEntries)
		: CategoryEntry(InCategoryEntry), WeightedMeshEntries(InWeightedMeshEntries)
	{}

#if WITH_EDITOR
	UE_API void ApplyDeprecation();
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	FString CategoryEntry;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	bool IsDefault = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (TitleProperty = "DisplayName"))
	TArray<FPCGMeshSelectorWeightedEntry> WeightedMeshEntries;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMeshSelectorWeightedByCategory : public UPCGMeshSelectorBase 
{
	GENERATED_BODY()

public:
	virtual bool SelectMeshInstances(
		FPCGStaticMeshSpawnerContext& Context,
		const UPCGStaticMeshSpawnerSettings* Settings,
		const UPCGBasePointData* InPointData,
		TArray<FPCGMeshInstanceList>& OutMeshInstances,
		UPCGBasePointData* OutPointData) const override;

	// ~Begin UObject interface
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostEditImport() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// ~End UObject interface

	// Refresh MeshEntries display names
	PCG_API void RefreshDisplayNames();
#endif // WITH_EDITOR

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSelector)
	FName CategoryAttribute;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSelector)
	TArray<FPCGWeightedByCategoryEntryList> Entries;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSelector, meta = (InlineEditConditionToggle))
	bool bUseAttributeMaterialOverrides = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, DisplayName = "By Attribute Material Overrides", Category = MeshSelector, meta = (EditCondition = "bUseAttributeMaterialOverrides"))
	TArray<FName> MaterialOverrideAttributes;
};

#undef UE_API
