// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/CollisionProfile.h"
#include "Engine/DataTable.h"
#include "Animation/AnimBank.h"

#include "PCGSkinnedMeshDescriptor.generated.h"

USTRUCT()
struct FPCGAnimBankDataRow : public FTableRowBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=AnimBank)
	TObjectPtr<USkeletalMesh> Mesh = nullptr;

	UPROPERTY(EditAnywhere, Category=AnimBank)
	TObjectPtr<UAnimBank> Bank = nullptr;

	UPROPERTY(EditAnywhere, Category=AnimBank)
	uint32 SequenceIndex = 0;
};

/** Convenience PCG-side component descriptor so we can adjust defaults to the most common use cases. */
// Implementation note: the tags don't really need to contribute to the hash, so we will retain the base class !=, == and ComputeHash implementations
USTRUCT()
struct FPCGSoftSkinnedMeshComponentDescriptor : public FSoftSkinnedMeshComponentDescriptor
{
	GENERATED_BODY()

	PCG_API FPCGSoftSkinnedMeshComponentDescriptor();
	PCG_API explicit FPCGSoftSkinnedMeshComponentDescriptor(const FSkinnedMeshComponentDescriptor& Other);
	PCG_API virtual void InitFrom(const UInstancedSkinnedMeshComponent* Component, bool bInitBodyInstance = true) override;
	PCG_API virtual void InitComponent(UInstancedSkinnedMeshComponent* Component) const override;
	bool HasTags() const { return !ComponentTags.IsEmpty() || !AdditionalCommaSeparatedTags.IsEmpty(); }

	// GW-TODO: 
	PCG_API int32 GetOrAddAnimationIndex(const FSoftAnimBankItem& BankItem);

public:
	UPROPERTY(EditAnywhere, Category="Component Settings|Tags", meta = (DisplayAfter = "ComponentClass"))
	TArray<FName> ComponentTags;

	UPROPERTY(EditAnywhere, Category = "Component Settings|Tags", meta = (DisplayAfter = "ComponentTags", Tooltip = "Comma-separated tag list. Normally should be left empty, and used as a property override instead."))
	FString AdditionalCommaSeparatedTags;
};
