// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPoint.h"
#include "Data/PCGPointData.h"
#include "Engine/CollisionProfile.h"
#include "MeshSelectors/PCGSkinnedMeshDescriptor.h"
#include "MeshSelectors/PCGMeshMaterialOverrideHelper.h"
#include "Metadata/PCGMetadata.h"
#include "PCGSkinnedMeshSelector.generated.h"

class UPCGPointData;
class UPCGSpatialData;
class UAnimBank;
struct FPCGContext;
struct FPCGSkinnedMeshSpawnerContext;
class UPCGSkinnedMeshSpawnerSettings;

USTRUCT(BlueprintType)
struct FPCGSkinnedMeshInstance
{
	GENERATED_BODY()

	FTransform Transform;
	int32 AnimationIndex = 0;
};

USTRUCT(BlueprintType)
struct FPCGSkinnedMeshInstanceList
{
	GENERATED_BODY()

	explicit FPCGSkinnedMeshInstanceList(const FPCGSoftSkinnedMeshComponentDescriptor& InDescriptor)
	: Descriptor(InDescriptor)
	, AttributePartitionIndex(INDEX_NONE)
	{}
	
	FPCGSkinnedMeshInstanceList() = default;
	~FPCGSkinnedMeshInstanceList() = default;
	FPCGSkinnedMeshInstanceList(const FPCGSkinnedMeshInstanceList&) = default;
	FPCGSkinnedMeshInstanceList(FPCGSkinnedMeshInstanceList&&) = default;
	FPCGSkinnedMeshInstanceList& operator=(const FPCGSkinnedMeshInstanceList&) = default;
	FPCGSkinnedMeshInstanceList& operator=(FPCGSkinnedMeshInstanceList&&) = default;

	UPROPERTY(EditAnywhere, Category = Settings)
	FPCGSoftSkinnedMeshComponentDescriptor Descriptor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FPCGSkinnedMeshInstance> Instances;

	/** Tracks which partition the instance list belongs to. */
	int64 AttributePartitionIndex;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TWeakObjectPtr<const UPCGPointData> PointData;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<int32> InstancePointIndices;
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSkinnedMeshSelector : public UObject 
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
	PCG_API virtual void PostLoad() override;
	// ~End UObject interface

	PCG_API virtual bool SelectInstances(
		FPCGSkinnedMeshSpawnerContext& Context,
		const UPCGSkinnedMeshSpawnerSettings* Settings,
		const UPCGPointData* InPointData,
		TArray<FPCGSkinnedMeshInstanceList>& OutMeshInstances,
		UPCGPointData* OutPointData) const;

public:
	/** Attribute in the input data that contains the path to the skinned mesh to spawn. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSelector)
	FPCGAttributePropertyInputSelector MeshAttribute;

	UPROPERTY(EditAnywhere, Category = MeshSelector)
	FPCGSoftSkinnedMeshComponentDescriptor TemplateDescriptor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSelector, meta = (InlineEditConditionToggle))
	bool bUseAttributeMaterialOverrides = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, DisplayName = "By Attribute Material Overrides", Category = MeshSelector, meta = (EditCondition = "bUseAttributeMaterialOverrides"))
	TArray<FName> MaterialOverrideAttributes;
};
