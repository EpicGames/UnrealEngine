// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataAccessorVariants.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/PCGMetadataContainerTypes.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Misc/Zip.h"

#include "IPCGAttributeAccessor.generated.h"

class IPCGAttributeAccessorKeys;

//////////////////////////////////////////////////////////////////////////////

UENUM(Meta = (Bitflags))
enum class EPCGAttributeAccessorFlags
{
	// Always require that the underlying type of the accessor match the expected type, 1 for 1.
	StrictType = 1 << 0,

	// Allow to broadcast the expected type to the underlying type (or vice versa, depending on the operation)
	AllowBroadcast = 1 << 1,

	// Allow to construct the expected type from the underlying type (or vice versa, depending on the operation)
	AllowConstructible = 1 << 2,

	// By default, if the key is a PCGInvalidEntryKey, it will add a new entry. With this set, it will override the default value.
	// USE WITH CAUTION
	AllowSetDefaultValue = 1 << 3,

	// New writes usually create a new metadata entry key when we write. In most cases, that's not mandatory, so use this flag to re-use an existing key.
	// Only useful for writing to attributes.
	// USE WITH CAUTION
	AllowReuseMetadataEntryKey = 1 << 4,
	
	// Allow to write multiple values in a single container. This is only used by the Set operation if we try to write a range of value into
	// a container accessor (array or set). Only if the accessors keys contains a single value.
	AllowRangeOfValuesIntoSingleContainer = 1 << 5,

	AllowBroadcastAndConstructible = AllowBroadcast | AllowConstructible
};
ENUM_CLASS_FLAGS(EPCGAttributeAccessorFlags);


namespace PCG::Private
{
	enum class EPCGAttributeAccessorOperation
	{
		SameType,
		Construct,
		Broadcast,
		Invalid
	};
	
	inline EPCGAttributeAccessorOperation GetOpType(const FPCGMetadataAttributeDesc& ReadDesc, const FPCGMetadataAttributeDesc& WriteDesc, EPCGAttributeAccessorFlags Flags)
	{
		if (IsEquivalentDesc<FString>(WriteDesc))
		{
			// Special case - always allow broadcasting from soft object path to string, so that legacy code that grabbed soft path attributes
			// as FStrings will still work.
			if (IsEquivalentDesc<FSoftObjectPath>(ReadDesc) || IsEquivalentDesc<FSoftClassPath>(ReadDesc))
			{
				return EPCGAttributeAccessorOperation::Broadcast;
			}
		}

		// Priority list if SameType -> Broadcast -> Constructible
		EPCGAttributeAccessorOperation Result = EPCGAttributeAccessorOperation::Invalid;
			
		if (ReadDesc.IsSameType(WriteDesc))
		{
			Result = EPCGAttributeAccessorOperation::SameType;
		}
		else if (!!(Flags & EPCGAttributeAccessorFlags::AllowBroadcast) && PCG::Private::IsBroadcastable(ReadDesc, WriteDesc))
		{
			Result = EPCGAttributeAccessorOperation::Broadcast;
		}
		else if (!!(Flags & EPCGAttributeAccessorFlags::AllowConstructible) && PCG::Private::IsConstructible(WriteDesc, ReadDesc))
		{
			Result = EPCGAttributeAccessorOperation::Construct;
		}
		else
		{
			return EPCGAttributeAccessorOperation::Invalid;
		}
			
		// When we are writing between objects, make sure it is always valid
		if (ReadDesc.ContainsObject() && WriteDesc.ContainsObject())
		{
			return ReadDesc.IsCompatible(WriteDesc) ? Result : EPCGAttributeAccessorOperation::Invalid;
		}

		return Result;
	}
}

//////////////////////////////////////////////////////////////////////////////

/**
* Base class for accessor. GetRange and SetRange will be sepcialized for all supported types, and there are a GetRange and SetRange virtual
* for each supported type.
* For Get/Set, you need a key, that will represent how you can access the value wanted. cf PCGAttributeAccessorKeys.
* NOTE: This is not threadsafe and not intended to be used unprotected.
*/
class IPCGAttributeAccessor
{
public:
	virtual ~IPCGAttributeAccessor() = default;
	
	// Both GetRange and SetRange will be specialized for all supported types. (declared below and defined in the cpp). 
	// By default, all unsupported types will return false.

	/**
	* Get a value from the accessor for a given type. Not threadsafe for all accessors.
	* @param OutValue - Where the value will be written to
	* @param Index - The index to look for in the keys
	* @param Keys - Identification to know how to retrieve the value
	* @param Flags - Optional flag to allow for specific operations. Cf EPCGAttributeAccessorFlags.
	* @return true if the get succeeded, false otherwise
	*/
	template <typename T>
	bool Get(T& OutValue, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType) const
	{
		return GetRange(TArrayView<T>(&OutValue, 1), Index, Keys, Flags);
	}

	/**
	* Get a value from the accessor for a given type at index 0. Not threadsafe for all accessors.
	* @param OutValue - Where the value will be written to
	* @param Keys - Identification to know how to retrieve the value
	* @param Flags - Optional flag to allow for specific operations. Cf EPCGAttributeAccessorFlags.
	* @return true if the get succeeded, false otherwise
	*/
	template <typename T>
	bool Get(T& OutValue, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType) const
	{
		return Get(OutValue, 0, Keys, Flags);
	}

	/**
	* Set a value to the accessor for a given type. Not threadsafe for all accessors. 
	* If Index is greater than the number of keys, it will fail (return false).
	* @param InValue - the value to write
	* @param Index - The index to look for in the keys
	* @param Keys - Identification to know how to identify the value.
	* @param Flags - Optional flag to allow for specific operations. Cf EPCGAttributeAccessorFlags.
	* @return true if the set succeeded, false otherwise
	*/
	template <typename T>
	bool Set(const T& InValue, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType)
	{
		return SetRange(TArrayView<const T>(&InValue, 1), Index, Keys, Flags);
	}

	/**
	* Set a value to the accessor for a given type at index 0. Not threadsafe for all accessors.
	* @param InValue - the value to write
	* @param Keys - Identification to know how to identify the value.
	* @param Flags - Optional flag to allow for specific operations. Cf EPCGAttributeAccessorFlags.
	* @return true if the set succeeded, false otherwise
	*/
	template <typename T>
	bool Set(const T& InValue, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType)
	{
		return Set(InValue, 0, Keys, Flags);
	}

	virtual void Prepare(IPCGAttributeAccessorKeys& Keys, int32 Count, const bool bCanReuseEntryKeys) {}

	// Since underlying type was only working for Single Values (and not arrays or other containers), make sure we never return a valid value if this is not a single value.
	int16 GetUnderlyingType() const { return UnderlyingDesc.IsSingleValue() ? UnderlyingType : static_cast<int16>(EPCGMetadataTypes::Unknown); }
	const FPCGMetadataAttributeDesc& GetUnderlyingDesc() const { return UnderlyingDesc; }
	bool IsReadOnly() const { return bReadOnly; }

	// To know if we can do default value operations
	virtual bool IsAttribute() const { return false; }
	
	/**
	* Get a range of values from the accessor for a given type. Not threadsafe for all accessors.
	* If the number of elements asked is greater that the number of keys, it will wrap around.
	* @param OutValues - View on memory where to write values to. Its "Num" will determine how many elements we will read.
	* @param Index - The index to start looking for in the keys
	* @param Keys - Identification to know how to retrieve the value
	* @param Flags - Optional flag to allow for specific operations. Cf EPCGAttributeAccessorFlags.
	* @return true if the get succeeded, false otherwise
	*/
	template <typename T>
	bool GetRange(TArrayView<T> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType) const
	{
		if (OutValues.IsEmpty())
		{
			// Nothing to do
			return true;
		}
		
		// Have a layer of indirection to first resolve T within the view then match it against the different versions.
		// For example:
		// ```
		// TArray<const FString*> Values;
		// GetRange<const FString*>(Values, Index, Keys, Flags).
		// ```
		// First T resolve will resolve to `const FString*`, then GetRange_Internal will match the pointer version.
		return GetRange_Internal(OutValues, Index, Keys, Flags);
	}
	
	/**
	* Set a range of values to the accessor for a given type. Not threadsafe for all accessors.
	* If the number of elements asked is greater that the number of keys, it will fail. It is to avoid writing at the same memory place multiple times.
	* @param InValues - View on memory where to read values from. Its "Num" will determine how many elements we will write.
	* @param Index - The index to start looking for in the keys
	* @param Keys - Identification to know how to retrieve the value
	* @param Flags - Optional flag to allow for specific operations. Cf EPCGAttributeAccessorFlags.
	* @return true if the set succeeded, false otherwise
	*/
	template <typename T>
	bool SetRange(TArrayView<const T> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType)
	{
		// Can't set if read only, and discard any set that goes beyond the number of keys.
		if (bReadOnly || Index >= Keys.GetNum())
		{
			return false;
		}
		
		if (InValues.IsEmpty())
		{
			// Nothing to do
			return true;
		}
		
		// Have a layer of indirection to first resolve T within the view then match it against the different versions.
		// For example:
		// ```
		// TArray<TArray<FString>> Values;
		// SetRange<TArray<FString>>(Values, Index, Keys, Flags).
		// ```
		// First T resolve will resolve to `TArray<FString>`, then SetRange_Internal will match the array version.
		return SetRange_Internal(InValues, Index, Keys, Flags);
	}

	/**
	* Copy an accessor into another.
	* @param SrcKeys - Identification to know how to retrieve the value in this accessor
	* @param DestAccessor - Accessor to write into
	* @param DestKeys - Identification to know how to write the value in destination accessor
	* @param Index - The index to start looking for in the keys
	* @param Count - Number of values to copy
	* @param Flags - Optional flag to allow for specific operations. Cf EPCGAttributeAccessorFlags.
	* @return true if the copy succeeded, false otherwise
	*/
	bool CopyTo(const IPCGAttributeAccessorKeys& SrcKeys, IPCGAttributeAccessor& DestAccessor, IPCGAttributeAccessorKeys& DestKeys, int32 Index, int32 Count, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType) const
	{
		return CopyTo(SrcKeys, DestAccessor, DestKeys, /*InputIndex=*/Index, /*OutputIndex=*/Index, Count, Flags);
	}
	
	/**
	* Partial copy an accessor into another.
	* @param SrcKeys - Identification to know how to retrieve the value in this accessor
	* @param DestAccessor - Accessor to write into
	* @param DestKeys - Identification to know how to write the value in destination accessor
	* @param InputIndex - The index to start looking for in the input keys
	* @param OutputIndex - The index to start looking for in the output keys
	* @param Count - Number of values to copy
	* @param Flags - Optional flag to allow for specific operations. Cf EPCGAttributeAccessorFlags.
	* @return true if the copy succeeded, false otherwise
	*/
	PCG_API bool CopyTo(const IPCGAttributeAccessorKeys& SrcKeys, IPCGAttributeAccessor& DestAccessor, IPCGAttributeAccessorKeys& DestKeys, int32 InputIndex, int32 OutputIndex, int32 Count, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType) const;

	/**
	 * Type erased version of GetRange. To be used with caution, it is the caller responsibility to allocate the right buffer.
	 * If conversion is possible but the buffers are not allocated in OutValues, it will error out.
	 * @param OutValues Type erased variant to write the values to.
	 * @param WriteDesc Type of the underlying descriptor to write to. Must match the OutValues data.
	 * @param Count     Number of values to copy
	 * @param Index     The index to look for in the keys
	 * @param Keys      Identification to know how to retrieve the value
	 * @param Flags		Optional flag to allow for specific operations. Cf EPCGAttributeAccessorFlags.
	 * @return true if the get succeeded, false otherwise
	 */
	PCG_API bool GetRange(PCG::Private::FOutValues OutValues, const FPCGMetadataAttributeDesc& WriteDesc, int32 Count, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) const;
	
	/**
	 * Type erased version of SetRange. To be used with caution, it is the caller responsibility to allocate the right buffer. 
	 * @param InValues Type erased variant to read the values from.
	 * @param ReadDesc Type of the underlying descriptor to read from. Must match the InValues data.
	 * @param Count    Number of values to copy
	 * @param Index    The index to look for in the keys
	 * @param Keys     Identification to know how to retrieve the value
	 * @param Flags	   Optional flag to allow for specific operations. Cf EPCGAttributeAccessorFlags.
	 * @return true if the set succeeded, false otherwise
	 */
	PCG_API bool SetRange(PCG::Private::FInValues InValues, const FPCGMetadataAttributeDesc& ReadDesc, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags);
	
	// To not be used directly, used by other accessors to chain calls.
	virtual bool GetRangeVirtual(PCG::Private::FOutValues OutValues, int32 Count, int32 Index, const IPCGAttributeAccessorKeys& Keys) const = 0;
	virtual bool SetRangeVirtual(PCG::Private::FInValues InValues, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) = 0;
	
	// To be overridden by child classes to know if we can process the Out/InValues
	// By default, it is if the underlying desc matches, but if we support more (like pointers), it needs to be explicitly set.
	PCG_API virtual bool SupportsGet(const PCG::Private::FOutValues& OutValues) const;
	PCG_API virtual bool SupportsSet(const PCG::Private::FInValues& InValues) const;

protected:
	IPCGAttributeAccessor(bool bInReadOnly, int16 InUnderlyingType)
		: bReadOnly(bInReadOnly)
		, UnderlyingType(InUnderlyingType)
		, UnderlyingDesc()
	{
		// Should be set by the child class, but have a catch all here.
		UnderlyingDesc.ValueType = static_cast<EPCGMetadataTypes>(UnderlyingType);
	}
	
	template <typename T> requires (!std::is_pointer_v<T>)
	bool GetRange_Internal(TArrayView<T> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) const
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<T>();
		PCG::Private::EPCGAttributeAccessorOperation OpType = PCG::Private::GetOpType(UnderlyingDesc, WriteDesc, Flags);
		
		if (OpType == PCG::Private::EPCGAttributeAccessorOperation::Invalid)
		{
			return false;
		}
		
		return GetRange_Internal(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, OutValues.GetData(), OutValues.Num(), &WriteDesc},
			WriteDesc,
			OutValues.Num(),
			Index,
			Keys,
			Flags,
			OpType
		);
	}

	// Pointer version
	template <typename T>
	bool GetRange_Internal(TArrayView<const T*> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) const
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<T>();
		PCG::Private::EPCGAttributeAccessorOperation OpType = PCG::Private::GetOpType(UnderlyingDesc, WriteDesc, Flags);
		
		if (OpType != PCG::Private::EPCGAttributeAccessorOperation::SameType)
		{
			return false;
		}
		
		// We need to type erase the pointers, but to be fully C++ compliant about aliasing rules we can't reinterpret_cast. So make a copy of the pointers.
		TArray<const void*, TInlineAllocator<256>> Temp;
		Temp.SetNumUninitialized(OutValues.Num());
		const bool bSuccess = GetRange_Internal(
			PCG::Private::FOutValues{ TInPlaceType<PCG::Private::FOutValuesByPtr>{}, Temp, /*Buffer=*/nullptr, &WriteDesc },
			WriteDesc,
			OutValues.Num(),
			Index,
			Keys,
			Flags,
			OpType
		);
		
		if (bSuccess)
		{
			for (int32 i = 0; i < OutValues.Num(); ++i)
			{
				OutValues[i] = static_cast<const T*>(Temp[i]);
			}
		}
		
		return bSuccess;
	}
	
	// Array version
	template <typename T> 
	bool GetRange_Internal(TArrayView<PCG::TPCGArrayAccessorWrapper<T>> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) const
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TArray<T>>();
		PCG::Private::EPCGAttributeAccessorOperation OpType = PCG::Private::GetOpType(UnderlyingDesc, WriteDesc, Flags);
		
		if (OpType == PCG::Private::EPCGAttributeAccessorOperation::Invalid)
		{
			return false;
		}
		
		// Get the values as if it is exactly the same type
		TArray<TTuple<const void*, int32>, TInlineAllocator<256>> Temp;
		Temp.SetNumUninitialized(OutValues.Num());
		const bool bSuccess = GetRange_Internal(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, Temp, /*Buffers=*/TArrayView<PCG::FPCGAccessorBuffer>{}, &WriteDesc},
			WriteDesc,
			OutValues.Num(),
			Index,
			Keys,
			Flags,
			PCG::Private::EPCGAttributeAccessorOperation::SameType
		);
		
		if (bSuccess)
		{
			// Then if it is the same type, just copy the pointers
			if (OpType == PCG::Private::EPCGAttributeAccessorOperation::SameType)
			{
				// Note that OutValue and PulledValue are both references here and are not copied by Zip nor the loop.
				for (auto [OutValue, PulledValue] : UE::Zip(OutValues, Temp))
				{
					auto [DataPtr, ArrayNum] = PulledValue;
					OutValue.SetupView(MakeConstArrayView<T>(static_cast<const T*>(DataPtr), ArrayNum));
				}
			}
			else
			{
				// Otherwise allocate the data to hold the converted value, and convert it.
				PCG::Private::FTransformFunc TransformFunc = OpType == PCG::Private::EPCGAttributeAccessorOperation::Broadcast
					? PCG::Private::GetBroadcastTransformFunc(static_cast<int16>(UnderlyingDesc.ValueType), static_cast<int16>(WriteDesc.ValueType))
					: PCG::Private::GetConstructibleTransformFunc(static_cast<int16>(WriteDesc.ValueType), static_cast<int16>(UnderlyingDesc.ValueType));
				
				check(TransformFunc);
				
				for (int32 i = 0; i < OutValues.Num(); ++i)
				{
					auto [DataPtr, ArrayNum] = Temp[i];
					TArrayView<T> MutableView = OutValues[i].AllocateOwnMemory(ArrayNum);
					check(MutableView.Num() == ArrayNum);
					
					if (!MutableView.IsEmpty())
					{
						TransformFunc(MutableView.GetData(), DataPtr, ArrayNum);
					}
				}
			}
		}
		
		return bSuccess;
	}
	
	// Sets version
	template <typename T> 
	bool GetRange_Internal(TArrayView<PCG::TScriptSetWrapper<T>> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) const
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TSet<T>>();
		PCG::Private::EPCGAttributeAccessorOperation OpType = PCG::Private::GetOpType(UnderlyingDesc, WriteDesc, Flags);
		
		if (OpType != PCG::Private::EPCGAttributeAccessorOperation::SameType)
		{
			return false;
		}
		
		TArray<FScriptSetHelper*, TInlineAllocator<256>> Temp;
		Temp.Reserve(OutValues.Num());
		for (PCG::TScriptSetWrapper<T>& Value : OutValues)
		{
			Temp.Add(&Value.Helper);
		}

		return GetRange_Internal(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsSet>{}, Temp, &WriteDesc},
			WriteDesc,
			OutValues.Num(),
			Index,
			Keys,
			Flags,
			OpType
		);
	}
	
	// Maps version
	template <typename KeyType, typename ValueType> 
	bool GetRange_Internal(TArrayView<PCG::TScriptMapWrapper<KeyType, ValueType>> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) const
	{
		const FPCGMetadataAttributeDesc& WriteDesc = PCG::Private::GetDefaultAttributeDesc<TMap<KeyType, ValueType>>();
		PCG::Private::EPCGAttributeAccessorOperation OpType = PCG::Private::GetOpType(UnderlyingDesc, WriteDesc, Flags);
		
		if (OpType != PCG::Private::EPCGAttributeAccessorOperation::SameType)
		{
			return false;
		}
		
		TArray<FScriptMapHelper*, TInlineAllocator<256>> Temp;
		Temp.Reserve(OutValues.Num());
		for (PCG::TScriptMapWrapper<KeyType, ValueType>& Value : OutValues)
		{
			Temp.Add(&Value.Helper);
		}

		return GetRange_Internal(
			PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsMap>{}, Temp, &WriteDesc},
			WriteDesc,
			OutValues.Num(),
			Index,
			Keys,
			Flags,
			OpType
		);
	}
	
	template <typename T>
	bool SetRange_Internal(TArrayView<const T> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags = EPCGAttributeAccessorFlags::StrictType)
	{
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<T>();

		return SetRange(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, InValues.GetData(), InValues.Num(), &ReadDesc},
			ReadDesc,
			InValues.Num(),
			Index,
			Keys,
			Flags
		);
	}
	
	/**
	 * Version that accept anything that is array like, like `TArray<T>`, `TArrayView<T>`, `TStaticArray<T, 5>`, `TArray<T, TInlineAllocator<5>>`, etc...
	 */
	template <typename ArrayContainer> requires (PCG::Concepts::CIsArrayLike<ArrayContainer>)
	bool SetRange_Internal(TConstArrayView<ArrayContainer> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		using ElementType = std::remove_const_t<typename ArrayContainer::ElementType>;
		
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<TArray<ElementType>>();
		PCG::Private::EPCGAttributeAccessorOperation OpType = PCG::Private::GetOpType(ReadDesc, UnderlyingDesc, Flags);
		
		if (OpType == PCG::Private::EPCGAttributeAccessorOperation::Invalid)
		{
			return false;
		}
		
		TArray<TTuple<const void*, int32>, TInlineAllocator<256>> InValuesTuple;
		InValuesTuple.Reserve(InValues.Num());
		Algo::Transform(InValues, InValuesTuple, [](const auto& Item) { return TTuple<const void*, int32>{Item.GetData(), Item.Num()}; });

		return SetRange_Internal(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, InValuesTuple, &ReadDesc},
			ReadDesc,
			InValues.Num(),
			Index,
			Keys,
			Flags,
			OpType
		);
	}

	/**
	 * Version that accepts Sets
	 */
	template <typename T, typename ...Extra>
	bool SetRange_Internal(TConstArrayView<TSet<T, Extra...>> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		using ElementType = std::remove_const_t<T>;
		
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<TSet<ElementType>>();
		PCG::Private::EPCGAttributeAccessorOperation OpType = PCG::Private::GetOpType(ReadDesc, UnderlyingDesc, Flags);
		
		if (OpType != PCG::Private::EPCGAttributeAccessorOperation::SameType)
		{
			return false;
		}
		
		TArray<TArray<const void*>, TInlineAllocator<256>> InSetValues;
		InSetValues.Reserve(InValues.Num());
		Algo::Transform(InValues, InSetValues, [](const auto& Item) -> TArray<const void*>
		{
			TArray<const void*> SetValues;
			SetValues.Reserve(Item.Num());
			Algo::Transform(Item, SetValues, [](const ElementType& Value) -> const ElementType* { return &Value;});
			return SetValues;
		});
		
		return SetRange_Internal(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsSet>{}, InSetValues, &ReadDesc},
			ReadDesc,
			InValues.Num(),
			Index,
			Keys,
			Flags,
			OpType
		);
	}
	
	/**
	 * Version that accepts Maps
	 */
	template <typename KeyType, typename ValueType, typename ...Extra>
	bool SetRange_Internal(TConstArrayView<TMap<KeyType, ValueType, Extra...>> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		const FPCGMetadataAttributeDesc& ReadDesc = PCG::Private::GetDefaultAttributeDesc<TMap<std::remove_const_t<KeyType>, std::remove_const_t<ValueType>>>();
		PCG::Private::EPCGAttributeAccessorOperation OpType = PCG::Private::GetOpType(ReadDesc, UnderlyingDesc, Flags);
		
		if (OpType != PCG::Private::EPCGAttributeAccessorOperation::SameType)
		{
			return false;
		}
		
		TArray<TArray<TPair<const void*, const void*>>, TInlineAllocator<256>> InMapValues;
		InMapValues.Reserve(InValues.Num());
		Algo::Transform(InValues, InMapValues, [](const auto& Item) -> TArray<TPair<const void*, const void*>>
		{
			TArray<TPair<const void*, const void*>> MapValues;
			MapValues.Reserve(Item.Num());
			Algo::Transform(Item, MapValues, [](const auto& It) -> TPair<const void*, const void*> { return {&It.Key, &It.Value};});
			return MapValues;
		});
		
		return SetRange_Internal(
			PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsMap>{}, InMapValues, &ReadDesc},
			ReadDesc,
			InValues.Num(),
			Index,
			Keys,
			Flags,
			OpType
		);
	}
	
	PCG_API bool GetRange_Internal(PCG::Private::FOutValues OutValues, const FPCGMetadataAttributeDesc& WriteDesc, int32 Count, int32 Index, const IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags, PCG::Private::EPCGAttributeAccessorOperation OpType) const;	
	PCG_API bool SetRange_Internal(PCG::Private::FInValues InValues, const FPCGMetadataAttributeDesc& ReadDesc, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags, PCG::Private::EPCGAttributeAccessorOperation OpType);

	bool bReadOnly = true;
	int16 UnderlyingType = (int16)EPCGMetadataTypes::Unknown;
	FPCGMetadataAttributeDesc UnderlyingDesc{};
};
