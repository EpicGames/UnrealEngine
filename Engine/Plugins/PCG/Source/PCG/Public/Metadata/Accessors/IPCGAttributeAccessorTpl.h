// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

/**
* Use of curiously recursive template pattern (CRTP) to dispatch GetRangeImpl and SetRangeImpl at compile time.
* Override all virtual functions for the supported types and will handle the conversion between
* "U" the incoming type and "T" the underlying type.
* 
* Class that inherit this one needs to define:
* -> bool GetRangeImpl(TArrayView<T>& OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
* -> bool SetRangeImpl(TArrayView<const T>& InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
* -> The underlying type: (using Type = ...);
*/
template <typename Derived>
class IPCGAttributeAccessorT : public IPCGAttributeAccessor
{
protected:
	IPCGAttributeAccessorT(bool bInReadOnly)
		: IPCGAttributeAccessor(bInReadOnly, /*UnderlyingType=*/ PCG::Private::MetadataTypes<typename Derived::Type>::Id)
	{}
	
	// Default implementation
	virtual bool GetRangeVirtual(PCG::Private::FOutValues OutValues, int32 Count, int32 Index, const IPCGAttributeAccessorKeys& Keys) const override
	{
		using T = typename Derived::Type;
		const Derived* This = static_cast<const Derived*>(this);
		
		if (PCG::Private::FOutValuesByValue* Values = OutValues.TryGet<PCG::Private::FOutValuesByValue>())
		{
			return This->GetRangeImpl(MakeArrayView(static_cast<T*>(Values->OutValues), Count), Index, Keys);
		}
		else
		{
			return false;
		}
	}
	
	virtual bool SetRangeVirtual(PCG::Private::FInValues InValues, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) override
	{
		using T = typename Derived::Type;
		Derived* This = static_cast<Derived*>(this);
		
		if (PCG::Private::FInValuesByValue* Values = InValues.TryGet<PCG::Private::FInValuesByValue>())
		{
			return This->SetRangeImpl(MakeArrayView(static_cast<const T*>(Values->InValues), Count), Index, Keys, Flags);
		}
		else
		{
			return false;
		}
	}
};
