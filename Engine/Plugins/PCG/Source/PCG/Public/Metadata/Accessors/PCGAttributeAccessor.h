// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPCGAttributeAccessorTpl.h"

#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/PCGMetadataDomain.h"

#define LOCTEXT_NAMESPACE "PCGAttributeAccessor"

/**
* Templated accessor class for attributes. Will wrap around a generic attribute.
* Key supported: MetadataEntryKey and Points
*/
class FPCGAttributeGenericAccessor : public IPCGAttributeAccessor
{
public:
	using Super = IPCGAttributeAccessor;

	// Can't write if metadata is null
	FPCGAttributeGenericAccessor(FPCGMetadataAttributeBase* InAttribute, FPCGMetadataDomain* InMetadataDomain, bool bForceReadOnly = false)
		: Super(/*bInReadOnly=*/ InMetadataDomain == nullptr || bForceReadOnly, static_cast<int16>(InAttribute->GetAttributeDesc().ValueType))
		, Attribute(InAttribute)
		, MetadataDomain(InMetadataDomain)
	{
		check(InAttribute && (!InMetadataDomain || InAttribute->GetMetadataDomain() == InMetadataDomain));
		UnderlyingDesc = InAttribute->GetAttributeDesc();
	}

	FPCGAttributeGenericAccessor(const FPCGMetadataAttributeBase* InAttribute, const FPCGMetadataDomain* InMetadataDomain, bool bForceReadOnly = false)
		: Super(/*bInReadOnly=*/ true, static_cast<int16>(InAttribute->GetAttributeDesc().ValueType))
		, Attribute(const_cast<FPCGMetadataAttributeBase*>(InAttribute))
		, MetadataDomain(const_cast<FPCGMetadataDomain*>(InMetadataDomain))
	{
		check(InAttribute && (!InMetadataDomain || InAttribute->GetMetadataDomain() == InMetadataDomain));
		UnderlyingDesc = InAttribute->GetAttributeDesc();
	}
	
	virtual bool SupportsGet(const PCG::Private::FOutValues& OutValues) const override
	{
		using namespace PCG::Private;
		if (OutValues.IsType<FOutValuesByPtr>())
		{
			return UnderlyingDesc.IsValid() && UnderlyingDesc.IsSingleValue();
		}
		else
		{
			return Super::SupportsGet(OutValues);
		}
	}
	
	virtual bool SupportsSet(const PCG::Private::FInValues& InValues) const override
	{
		using namespace PCG::Private;
		if (InValues.IsType<FInValuesByPtr>())
		{
			return UnderlyingDesc.IsValid() && UnderlyingDesc.IsSingleValue();
		}
		else
		{
			return Super::SupportsSet(InValues);
		}
	}

	virtual bool GetRangeVirtual(PCG::Private::FOutValues OutValues, int32 Count, int32 Index, const IPCGAttributeAccessorKeys& Keys) const override
	{
		TArray<const PCGMetadataEntryKey*, TInlineAllocator<256>> EntryKeyPtrs;
		EntryKeyPtrs.SetNumUninitialized(Count);

		TArrayView<const PCGMetadataEntryKey*> EntryKeysView(EntryKeyPtrs);
		if (!Keys.GetKeys<PCGMetadataEntryKey>(Index, EntryKeysView))
		{
			return false;
		}

		TArray<PCGMetadataEntryKey, TInlineAllocator<256>> EntryKeys;
		EntryKeys.Reserve(EntryKeyPtrs.Num());
		Algo::Transform(EntryKeyPtrs, EntryKeys, [](const PCGMetadataEntryKey* KeyPtr) { return *KeyPtr; });
		
		TArray<PCGMetadataValueKey> ValueKeys;
		Attribute->GetValueKeys(PCGValueRangeHelpers::MakeConstValueRange<PCGMetadataEntryKey>(EntryKeys), ValueKeys);
		Attribute->GetValues_Internal(ValueKeys, MoveTemp(OutValues));

		return true;
	}
	
	virtual bool SetRangeVirtual(PCG::Private::FInValues InValues, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) override
	{
		if (EnumHasAnyFlags(Flags, EPCGAttributeAccessorFlags::AllowSetDefaultValue) && Keys.GetNum() == 1)
		{
			PCGMetadataEntryKey* EntryKey = nullptr;
			Keys.GetKey(EntryKey);

			if (EntryKey && *EntryKey == PCGInvalidEntryKey)
			{
				check(Count > 0);
				Attribute->AddValues_Internal(InValues, 1, /*bExistingIndex=*/PCG::Private::ExistingIndexForDefaultValue);
				return true;
			}
		}

		TArray<PCGMetadataEntryKey*, TInlineAllocator<512>> EntryKeys;
		EntryKeys.SetNumUninitialized(Count);
		TArrayView<PCGMetadataEntryKey*> EntryKeysView(EntryKeys);
		
		if (!bWasPrepared || Attribute->DoesCompressData())
		{
			if (!Prepare(Keys, Count, /*bCanReuseEntryKeys=*/EnumHasAnyFlags(Flags, EPCGAttributeAccessorFlags::AllowReuseMetadataEntryKey), Index, /*bPreallocateValues=*/false, &EntryKeysView))
			{
				return false;
			}

			TArray<PCGMetadataValueKey> ValueKeys = Attribute->DoesCompressData() ? Attribute->AddCompressedValues(InValues, Count) : Attribute->AddValues_Internal(InValues, Count);
			Attribute->SetValuesFromValueKeys(EntryKeys, ValueKeys);
		}
		else
		{
			check(StartIndex != INDEX_NONE);
			Attribute->AddValues_Internal(InValues, Count, /*bExistingIndex=*/StartIndex + Index, /*bLockless=*/true);
		}

		return true;
	}

	virtual bool IsAttribute() const override { return true; }

	bool Prepare(IPCGAttributeAccessorKeys& Keys, int32 Count, const bool bCanReuseEntryKeys, int32 Index, bool bPreallocateValues, TArrayView<PCGMetadataEntryKey*>* OutEntryKeys = nullptr)
	{
		if (!MetadataDomain || IsReadOnly())
		{
			PCGLog::LogWarningOnGraph(LOCTEXT("ReadOnlyInvalidPrepare", "Try to prepare an attribute accessor with a read only accessor. This is invalid."));
			return false;		
		}

		const bool bSupportsMultiEntries = MetadataDomain->SupportsMultiEntries();
		if (!bSupportsMultiEntries && Count > 1)
		{
			PCGLog::LogWarningOnGraph(FText::Format(LOCTEXT("InvalidDomain", "Try to prepare multiple values to set in an attribute from a domain that doesn't support multi entries ({0})"), FText::FromString(MetadataDomain->GetDomainID().DebugName.ToString())));
			return false;
		}

		check(!OutEntryKeys || OutEntryKeys->Num() == Count);

		TArray<PCGMetadataEntryKey*, TInlineAllocator<512>> EntryKeys;
		TArrayView<PCGMetadataEntryKey*> EntryKeysView;
		if (!OutEntryKeys)
		{
			EntryKeys.SetNumUninitialized(Count);
			EntryKeysView = TArrayView<PCGMetadataEntryKey*>(EntryKeys);
		}
		else
		{
			EntryKeysView = *OutEntryKeys;
		}

		if (!Keys.GetKeys<PCGMetadataEntryKey>(Index, EntryKeysView))
		{
			return false;
		}

		TArray<PCGMetadataEntryKey*, TInlineAllocator<512>> EntriesToSet;
		EntriesToSet.Reserve(Count);

		// Implementation note: this is a stripped down version of UPCGMetadata::InitializeOnSet
		for (int EntryIndex = 0; EntryIndex < EntryKeysView.Num(); ++EntryIndex)
		{
			PCGMetadataEntryKey& EntryKey = *EntryKeysView[EntryIndex];
			if (EntryKey == PCGInvalidEntryKey || (EntryKey < MetadataDomain->GetItemKeyCountForParent() && !bCanReuseEntryKeys))
			{
				EntriesToSet.Add(&EntryKey);
			}
		}

		if (!EntriesToSet.IsEmpty())
		{
			if (bSupportsMultiEntries || (MetadataDomain->GetItemCountForChild() == 0 && ensure(EntriesToSet.Num() == 1)))
			{
				MetadataDomain->AddEntriesInPlace(EntriesToSet);
			}
			else if (!bSupportsMultiEntries && ensure(EntriesToSet.Num() == 1))
			{
				*(EntriesToSet[0]) = PCGFirstEntryKey;
			}
		}

		if (bPreallocateValues && ensure(!bWasPrepared))
		{
			StartIndex = Attribute->PreallocateValues(EntryKeysView, /*bLockless=*/false);
		}

		return true;
	}

	virtual void Prepare(IPCGAttributeAccessorKeys& Keys, int32 Count, const bool bCanReuseEntryKeys) override
	{
		if (!ensure(!bWasPrepared))
		{
			return;
		}

		Attribute->Prepare(Count);
		bWasPrepared = Prepare(Keys, Count, bCanReuseEntryKeys, /*Index=*/0, /*bPreallocateValues=*/true, /*OutEntryKeys=*/nullptr);
	}

protected:
	FPCGMetadataAttributeBase* Attribute = nullptr;
	FPCGMetadataDomain* MetadataDomain = nullptr;
	bool bWasPrepared = false;
	int32 StartIndex = INDEX_NONE;
};

/** Deprecated accessor, use the generic one.*/
template <typename T>
class UE_DEPRECATED(5.8, "Use the Generic Accessor.") FPCGAttributeAccessor : public FPCGAttributeGenericAccessor
{
	using FPCGAttributeGenericAccessor::FPCGAttributeGenericAccessor;
};

#undef LOCTEXT_NAMESPACE