// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGAttributePropertySelector.h"

#include "Templates/Function.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"

struct FPCGAttributePropertySelector;
class FPCGMetadataAttributeBase;
class FPCGModule;
class IPCGAttributeAccessor;
class IPCGAttributeAccessorKeys;
class UPCGData;
class UPCGMetadata;

#if WITH_EDITOR
struct FPCGAttributeSelectorMenuEntry;

struct FPCGAttributeSelectorMenu
{
	FText Label;
	FText Tooltip;
	
	TArray<FPCGAttributeSelectorMenu> SubMenus;
	TArray<FPCGAttributeSelectorMenuEntry> Entries;
};

struct FPCGAttributeSelectorMenuEntry
{
	FPCGAttributeSelectorMenuEntry(FText InLabel, FText InTooltip, FPCGAttributePropertySelector InSelector, bool bInReadOnly = false)
		: Label(std::move(InLabel))
		, Tooltip(std::move(InTooltip))
		, Selector(std::move(InSelector))
		, bReadOnly(bInReadOnly)
	{}

	/** Label for the entry. */
	FText Label;

	/** Tooltip to display for the entry. */
	FText Tooltip;

	/** Selector to copy if this entry is selected. */
	FPCGAttributePropertySelector Selector;

	/** If the entry is read-only, it will only show for FPCGAttributePropertyInputSelector. */
	bool bReadOnly = false;
};
#endif // WITH_EDITOR

struct FPCGAttributeAccessorMethods
{
	TFunction<TUniquePtr<IPCGAttributeAccessor>(UPCGData*, const FPCGAttributePropertySelector&, const bool)> CreateAccessorFunc;
	TFunction<TUniquePtr<const IPCGAttributeAccessor>(const UPCGData*, const FPCGAttributePropertySelector&, const bool)> CreateConstAccessorFunc;
	TFunction<TUniquePtr<IPCGAttributeAccessorKeys>(UPCGData*, const FPCGAttributePropertySelector&, const bool)> CreateAccessorKeysFunc;
	TFunction<TUniquePtr<const IPCGAttributeAccessorKeys>(const UPCGData*, const FPCGAttributePropertySelector&, const bool)> CreateConstAccessorKeysFunc;

#if WITH_EDITOR
	template <typename EnumType>
	void FillSelectorMenuEntryFromEnum(const TArrayView<const FText> InMenuHierarchy = {}) { FillSelectorMenuEntryFromEnum(StaticEnum<EnumType>(), InMenuHierarchy); }

	PCG_API void FillSelectorMenuEntryFromEnum(const UEnum* EnumType, const TArrayView<const FText> InMenuHierarchy = {});
	
	// Special option to gather all the possible options for a given data type in the Selector context menu.
	FPCGAttributeSelectorMenu AttributeSelectorMenu;
#endif // WITH_EDITOR
};

class FPCGAttributeAccessorFactory
{
	friend FPCGModule;

public:	
	static PCG_API FPCGAttributeAccessorFactory& GetMutableInstance();
	static PCG_API const FPCGAttributeAccessorFactory& GetInstance();

	/** Create a simple accessor based on the data passed as input. No chain of extraction. Internal function deliberately not exposed. Use PCGAttributeAccessorHelpers::CreateAccessor. */
	TUniquePtr<IPCGAttributeAccessor> CreateSimpleAccessor(UPCGData* InData, const FPCGAttributePropertySelector& InSelector, const bool bQuiet) const;
	
	/** Create a simple const accessor based on the data passed as input. No chain of extraction. Internal function deliberately not exposed. Use PCGAttributeAccessorHelpers::CreateConstAccessor. */
	TUniquePtr<const IPCGAttributeAccessor> CreateSimpleConstAccessor(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, const bool bQuiet) const;
	
	/** Create a simple key based on the data passed as input. Internal function deliberately not exposed. Use PCGAttributeAccessorHelpers::CreateKeys. */
	TUniquePtr<IPCGAttributeAccessorKeys> CreateSimpleKeys(UPCGData* InData, const FPCGAttributePropertySelector& InSelector, const bool bQuiet) const;
	
	/** Create a simple key based on the data passed as input. Internal function deliberately not exposed. Use PCGAttributeAccessorHelpers::CreateConstKeys. */
	TUniquePtr<const IPCGAttributeAccessorKeys> CreateSimpleConstKeys(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, const bool bQuiet) const;

#if WITH_EDITOR
	/** Call a Callback on all the FPCGAttributeSelectorMenu currently registered. */
	PCG_API void ForEachSelectorMenu(TFunctionRef<void(const FPCGAttributeSelectorMenu&)> Callback) const;
#endif // WITH_EDITOR
	
	template <typename T, typename std::enable_if_t<std::is_base_of_v<UPCGData, T>, bool> = false>
	void RegisterMethods(FPCGAttributeAccessorMethods&& InAccessorMethods) { RegisterMethods(T::StaticClass(), std::move(InAccessorMethods)); }
	
	PCG_API void RegisterMethods(TSubclassOf<UPCGData> PCGDataClass, FPCGAttributeAccessorMethods&& InAccessorMethods);

	template <typename T, typename std::enable_if_t<std::is_base_of_v<UPCGData, T>, bool> = false>
	void UnregisterMethods() { UnregisterMethods(T::StaticClass()); }
	
	PCG_API void UnregisterMethods(TSubclassOf<UPCGData> PCGDataClass);

private:
	/** To be called by the FPCGModule on register. Will register the default UPCGData accessor (Metadata), keys and the spatial const keys (special case). */
	void RegisterDefaultMethods();
	void UnregisterDefaultMethods();
	
	template <typename PCGData, typename Func>
	decltype(auto) CallOnMethod(PCGData* InData, Func Callback) const;
	
	TMap<const TSubclassOf<UPCGData>, FPCGAttributeAccessorMethods> AccessorMethods;
};
