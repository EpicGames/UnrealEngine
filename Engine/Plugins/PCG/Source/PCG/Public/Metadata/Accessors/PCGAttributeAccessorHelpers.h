// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGAttributePropertySelector.h"
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/Accessors/IPCGAttributeAccessor.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "Containers/Array.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

enum class EPCGExtraProperties : uint8;
struct FPCGContext;
struct FPCGDataCollection;
struct FPCGSettingsOverridableParam;
class FProperty;
class IPCGAttributeAccessor;
class IPCGAttributeAccessorKeyIterator;
class IPCGAttributeAccessorKeys;
class UClass;
class UPCGData;
class UStruct;

namespace PCGAttributeAccessorHelpers
{
	/** Returns true if the property is supported by PCG types and has a conversion to a Metadata type. For deprecation, the default behavior will be to return the old accessors. For the new ones, set bUseGenericAccessor to true. */
	PCG_API bool IsPropertyAccessorSupported(const FProperty* InProperty, bool bUseGenericAccessor = false);

	/** Returns true if the property is supported by PCG types and has a conversion to a Metadata type.  For deprecation, the default behavior will be to return the old accessors. For the new ones, set bUseGenericAccessor to true. */
	PCG_API bool IsPropertyAccessorSupported(const FName InPropertyName, const UStruct* InStruct, bool bUseGenericAccessor = false);

	/** Returns true if the property chain exists,  and the last property is supported by PCG types and has a conversion to a Metadata type.  For deprecation, the default behavior will be to return the old accessors. For the new ones, set bUseGenericAccessor to true. */
	PCG_API bool IsPropertyAccessorChainSupported(const TArray<FName>& InPropertyNames, const UStruct* InStruct, bool bUseGenericAccessor = false);

	/** Returns the metadata type associated with this property. For deprecation, the default behavior will be to return the old accessors. For the new ones, set bUseGenericAccessor to true. */
	PCG_API EPCGMetadataTypes GetMetadataTypeForProperty(const FProperty* InProperty, bool bUseGenericAccessor = false);

	/** Create an accessor for the given property. Property needs to be supported by PCG (cf. IsPropertyAccessorSupported). For deprecation, the default behavior will be to return the old accessors. For the new ones, set bUseGenericAccessor to true. */
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreatePropertyAccessor(const FProperty* InProperty, bool bUseGenericAccessor = false);
	
	/** Look for a property in the provided class/struct and create an accessor for it. Property needs to be supported by PCG (cf. IsPropertyAccessorSupported). For deprecation, the default behavior will be to return the old accessors. For the new ones, set bUseGenericAccessor to true. */
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreatePropertyAccessor(const FName InPropertyName, const UStruct* InStruct, bool bUseGenericAccessor = false);

	/**
	 * Create a chain accessor for the given properties. Last property needs to be supported by PCG (cf. IsPropertyAccessorSupported). For deprecation, the default behavior will be to return the old accessors. For the new ones, set bUseGenericAccessor to true.
	 * USE WITH CAUTION: There is no validation that the properties are related and may corrupt memory if used incorrectly.
	 */
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreatePropertyChainAccessor(TArray<const FProperty*>&& InProperties, bool bUseGenericAccessor = false);

	/** Create a chain accessor for the given properties, starting from the provided class/struct. Last property needs to be supported by PCG (cf. IsPropertyAccessorSupported). For deprecation, the default behavior will be to return the old accessors. For the new ones, set bUseGenericAccessor to true. */
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreatePropertyChainAccessor(const TArray<FName>& InPropertyNames, const UStruct* InStruct, bool bUseGenericAccessor = false);

	/** Create a special accessor for all the supported extra properties (cf. EPCGExtraProperties). */
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreateExtraAccessor(EPCGExtraProperties InExtraProperties);

	/** Create a chain accessor to extract the "Name" out of the Accessor. Example: InAccessor is of type Vector, and name is `X`, it will create an accessor to access the X component on vectors. */
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreateChainAccessor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
	
	/** Create a chain accessor using the extra names from the selector. Example: InAccessor is of type Vector, and extra names are [`X`], it will create an accessor to access the X component on vectors. */
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreateChainAccessor(TUniquePtr<IPCGAttributeAccessor> InAccessor, const FPCGAttributePropertySelector& InSelector, bool& bOutSuccess, bool bQuiet = false);

	/** From a list of property names, starting from provided class/struct, return the property chain. */
	PCG_API bool GetPropertyChain(const TArray<FName>& InPropertyNames, const UStruct* InStruct, TArray<const FProperty*>& OutProperties);

	struct AccessorParamResult
	{
		FName AttributeName = NAME_None;
		FName AliasUsed = NAME_None;
		bool bUsedAliases = false;
		bool bPinConnected = false;
		bool bHasMultipleAttributeSetsOnOverridePin = false;
		bool bHasMultipleDataInAttributeSet = false;
		bool bHasNoEntry = false;

		// If the param is in a container, we'll need the keys
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys;
	};

	/**
	* Create a const accessor depending on an overridable param
	*/
	PCG_API TUniquePtr<const IPCGAttributeAccessor> CreateConstAccessorForOverrideParamWithResult(const FPCGDataCollection& InInputData, const FPCGSettingsOverridableParam& InParam, AccessorParamResult* OutResult = nullptr);

	/**
	* Creates a const accessor to the property or attribute pointed at by the InSelector.
	* Note that InData must not be null if the selector points to an attribute,
	* but in the case of properties, it either has to be the appropriate type or null.
	* Make sure to update your selector before-hand if you want to support "@Last"
	*/
	PCG_API TUniquePtr<const IPCGAttributeAccessor> CreateConstAccessor(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet = false);

	/** 
	* Creates a const accessor to an attribute without requiring a selector.
	*/
	PCG_API TUniquePtr<const IPCGAttributeAccessor> CreateConstAccessor(const FPCGMetadataAttributeBase* InAttribute, const UPCGMetadata* InMetadata, bool bQuiet = false);
	PCG_API TUniquePtr<const IPCGAttributeAccessor> CreateConstAccessor(const FPCGMetadataAttributeBase* InAttribute, const FPCGMetadataDomain* InMetadata, bool bQuiet = false);

	/**
	* Creates an accessor to the property or attribute pointed at by the InSelector.
	* Note that InData must not be null if the selector points to an attribute,
	* but in the case of properties, it either has to be the appropriate type or null.
	* Make sure to update your selector before-hand if you want to support "@Source". Otherwise the creation will fail.
	*/
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreateAccessor(UPCGData* InData, const FPCGAttributePropertySelector& InSelector, bool bQuiet = false);

	/**
	* Creates an accessor to the property or attribute pointed at by the InSelector. If the selector points to a base attribute and the attribute doesn't exist 
	* or the types doesn't match with the Matching Accessor or provided type, create the attribute using the Matching Accessor. The default value of this new attribute will either
	* be the first value get from the provided keys (if provided), the default value of the matching accessor (if it is an attribute) or the zero value (if no matching accessor or not supporting default values).
	* Note that InData must not be null if the selector points to an attribute,
	* but in the case of properties, it either has to be the appropriate type or null.
	* Make sure to update your selector before-hand if you want to support "@Source". Otherwise the creation will fail.
	*/
	struct FPCGCreateAccessorWithAttributeCreationParams
	{
		UPCGData* InData = nullptr;
		const FPCGAttributePropertySelector* InSelector = nullptr;

		const IPCGAttributeAccessor* InMatchingAccessor = nullptr;
		const IPCGAttributeAccessorKeys* InMatchingKeysForDefaultValue = nullptr;
		// Discarded if InMatchingAccessor is not null.
		FPCGMetadataAttributeDesc InExpectedDesc;

		EPCGAttributeAccessorFlags InTypeMatching = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible;
		bool bQuiet = false;
	};

	PCG_API TUniquePtr<IPCGAttributeAccessor> CreateAccessorWithAttributeCreation(const FPCGCreateAccessorWithAttributeCreationParams& InParams);

	UE_DEPRECATED(5.8, "Use the version with the params")
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreateAccessorWithAttributeCreation(UPCGData* InData, const FPCGAttributePropertySelector& InSelector, const IPCGAttributeAccessor* InMatchingAccessor, EPCGAttributeAccessorFlags InTypeMatching = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible, bool bQuiet = false);

	/**
	* Creates an accessor to an attribute without requiring a selector.
	*/
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreateAccessor(FPCGMetadataAttributeBase* InAttribute, UPCGMetadata* InMetadata, bool bQuiet = false);
	PCG_API TUniquePtr<IPCGAttributeAccessor> CreateAccessor(FPCGMetadataAttributeBase* InAttribute, FPCGMetadataDomain* InMetadata, bool bQuiet = false);

	PCG_API TUniquePtr<const IPCGAttributeAccessorKeys> CreateConstKeys(const UPCGData* InData, const FPCGAttributePropertySelector& InSelector);
	PCG_API TUniquePtr<IPCGAttributeAccessorKeys> CreateKeys(UPCGData* InData, const FPCGAttributePropertySelector& InSelector);

	PCG_API TUniquePtr<const IPCGAttributeAccessorKeys> CreateConstKeys(const FPCGMetadataAttributeBase* InAttribute);
	PCG_API TUniquePtr<IPCGAttributeAccessorKeys> CreateKeys(FPCGMetadataAttributeBase* InAttribute);

	/** 
	 * Utility function to extract all the values through the accessor for all the keys.
	 *
	 * @param InAccessor  Attribute accessor from which to extract the values.
	 * @param InKeys      Attribute accessor keys for the values.
	 * @param OutArray    Output array where the values will be written.
	 * @param GetFlags    Flags for the get accessor. By default, is broadcast and constructible.
	 * @return true if it succeeded.
	 */
	template <typename T, typename AllocatorType>
	bool ExtractAllValues(const IPCGAttributeAccessor* InAccessor, const IPCGAttributeAccessorKeys* InKeys, TArray<T, AllocatorType>& OutArray, EPCGAttributeAccessorFlags GetFlags = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible)
	{
		if (!InAccessor || !InKeys)
		{
			return false;
		}

		if (InKeys->GetNum() < 1)
		{
			return true;
		}

		if constexpr (std::is_trivially_copyable_v<T>)
		{
			OutArray.SetNumUninitialized(InKeys->GetNum());
		}
		else
		{
			OutArray.SetNum(InKeys->GetNum());
		}

		return InAccessor->GetRange<T>(OutArray, 0, *InKeys, GetFlags);
	}

	/** 
	 * Utility function to create the accessor and keys and extract all the values for all the keys. Also, automatically resolve the selector for @Last.
	 *
	 * @param InData      Input data to extract the value from. Will also be used to resolve @Last on the selector.
	 * @param InSelector  Selector to know which value to extract. Will be resolved if it is @Last.
	 * @param OutArray    Output array where the values will be written.
	 * @param Context     Optional context for logging. Can be null.
	 * @param GetFlags    Flags for the get accessor. By default, is broadcast and constructible.
	 * @return true if it succeeded.
	 */
	template <typename T, typename AllocatorType>
	bool ExtractAllValues(const UPCGData* InData, const FPCGAttributePropertyInputSelector& InSelector, TArray<T, AllocatorType>& OutArray, FPCGContext* Context = nullptr, EPCGAttributeAccessorFlags GetFlags = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible, bool bQuiet = false)
	{
		// TODO: Provide a version that take advantage of compressed attributes (Keys + Values)
		
		const FPCGAttributePropertyInputSelector Selector = InSelector.CopyAndFixLast(InData);

		TUniquePtr<const IPCGAttributeAccessor> Accessor = CreateConstAccessor(InData, Selector, bQuiet);
		TUniquePtr<const IPCGAttributeAccessorKeys> Keys = CreateConstKeys(InData, Selector);
		if (!Accessor.IsValid())
		{
			if (!bQuiet)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, Context);
			}
			return false;
		}

		if (!Keys.IsValid())
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(Selector, Context);
			return false;
		}

		const bool bResult = ExtractAllValues<T, AllocatorType>(Accessor.Get(), Keys.Get(), OutArray, GetFlags);

		if (!bResult && !bQuiet)
		{
			PCGLog::Metadata::LogFailToGetAttributeError(Selector, Context);
		}

		return bResult;
	}

	template <typename T>
	bool ExtractParamValue(const UPCGData* InData, const FPCGAttributePropertyInputSelector& InSelector, T& OutValue, FPCGContext* Context = nullptr, EPCGAttributeAccessorFlags GetFlags = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible, bool bQuiet = false)
	{
		const FPCGAttributePropertyInputSelector Selector = InSelector.CopyAndFixLast(InData);

		TUniquePtr<const IPCGAttributeAccessor> Accessor = CreateConstAccessor(InData, Selector, bQuiet);
		if (!Accessor.IsValid())
		{
			if (!bQuiet)
			{
				PCGLog::Metadata::LogFailToCreateAccessorError(Selector, Context);
			}
			return false;
		}

		FPCGAttributeAccessorKeysEntries FirstEntry(PCGFirstEntryKey);
		bool bResult = Accessor->Get<T>(OutValue, FirstEntry, GetFlags);

		if (!bResult && !bQuiet)
		{
			PCGLog::Metadata::LogFailToGetAttributeError(Selector, Context);
		}

		return bResult;
	}
	
	/** 
	 * Utility function to create the accessor and keys and write all the values for all the keys coming from InValues. Also, automatically resolve the selector for @Last.
	 *
	 * @param OutputData      Input data to extract the value from. Will also be used to resolve @Source on the selector.
	 * @param OutputSelector  Selector to know which value to write to. Will be resolved if it is @Source.
	 * @param InValues        Input view where the values will be read from. InValues MUST be the same size as the keys that will be created by CreateKeys on the OutputData. 
	 * @param SourceSelector  Optional source selector for @Source resolve. Can be null.
	 * @param Context         Optional context for logging. Can be null.
	 * @param SetFlags        Flags for the set accessor. By default, is broadcast and constructible.
	 * @return true if it succeeded.
	 */
	template <typename T>
	bool WriteAllValues(UPCGData* OutputData, const FPCGAttributePropertyOutputSelector& OutputSelector, TArrayView<const T> InValues, const FPCGAttributePropertyInputSelector* SourceSelector = nullptr, FPCGContext* Context = nullptr, EPCGAttributeAccessorFlags SetFlags = EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible)
	{
		// TODO: Provide a version that take advantage of compressed attributes (Keys + Values)
		
		const FPCGAttributePropertyOutputSelector Selector = OutputSelector.CopyAndFixSource(SourceSelector, OutputData);

		TUniquePtr<IPCGAttributeAccessor> Accessor = CreateAccessor(OutputData, Selector);
		TUniquePtr<IPCGAttributeAccessorKeys> Keys = CreateKeys(OutputData, Selector);
		if (!Accessor.IsValid() || !Keys.IsValid())
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(Selector, Context);
			return false;
		}

		if (!ensureMsgf(Keys->GetNum() == InValues.Num(), TEXT("Number of values passed (%d) mismatches with the number of keys (%d)"), Keys->GetNum(), InValues.Num()))
		{
			return false;
		}

		if (!Accessor->SetRange<T>(InValues, 0, *Keys, SetFlags))
		{
			PCGLog::Metadata::LogFailToSetAttributeError<T>(Selector, Accessor.Get(), Context);
			return false;
		}

		return true;
	}

	namespace Private
	{
		// Use a lambda to have this code more likely to be inlined in SortByAttribute.
		static inline auto DefaultIndexGetter = [](int32 Index) -> int32 { return Index; };

		// Use bAscending bool to know if you need to negate the condition for equal, since CompareDescending is !CompareAscending
		template<typename T>
		bool DefaultStableCompareLess(const T& A, const T& B, int32 IndexA, int32 IndexB, bool bAscending)
		{
			if (PCG::Private::MetadataTraits<T>::Equal(A, B))
			{
				return (bAscending == (IndexA < IndexB));
			}

			return PCG::Private::MetadataTraits<T>::Less(A, B);
		}

		// We need the lambda, because we can't use a templated function as a default parameter without specifying the template, but we can with a generic lambda.
		static inline auto DefaultStableCompareLessLambda = [](const auto& A, const auto& B, int32 IndexA, int32 IndexB, bool bAscending) -> bool { return DefaultStableCompareLess(A, B, IndexA, IndexB, bAscending); };
	}

	/**
	* Returns a sorted key indices array.
	* A custom function can be used to extract the index of the elements of the array,
	* and sort ascending or descending by the values associated with that index.
	* CompareLess method can also be provided, method signature needs to follow DefaultStableCompareLess.
	* We need the decltypes on both function types to allow to provide default values for both callbacks.
	* Check DefaultIndexGetter and DefaultStableCompareLess for their signatures.
	*/
	template <typename GetIndexFunc = decltype(Private::DefaultIndexGetter), typename CompareLessFunc = decltype(Private::DefaultStableCompareLessLambda)>
	TArray<int32> SortKeyIndicesByAttribute(const IPCGAttributeAccessor& InAccessor, const IPCGAttributeAccessorKeys& InKeys, int32 KeyCount, bool bAscending, GetIndexFunc CustomGetIndex = Private::DefaultIndexGetter, CompareLessFunc CompareLess = Private::DefaultStableCompareLessLambda)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PCGAttributeAccessorHelpers::SortKeyIndicesByAttribute);
		check(KeyCount <= InKeys.GetNum())

		if (KeyCount == 0)
		{
			return {};
		}

		// Prepare integer sequence
		TArray<int32> ElementsIndexes;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGAttributeAccessorHelpers::SortByAttribute::PrepareIndexes);
			ElementsIndexes.Reserve(KeyCount);
			for (int i = 0; i < KeyCount; ++i)
			{
				ElementsIndexes.Add(i);
			}
		}

		auto SortIndexes = [&InAccessor, &InKeys, bAscending, &CustomGetIndex, &CompareLess, &ElementsIndexes](auto Dummy)
		{
			using ValueType = decltype(Dummy);

			if constexpr (PCG::Private::MetadataTraits<ValueType>::CanCompare)
			{
				TArray<ValueType> CachedValues;

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(PCGAttributeAccessorHelpers::SortByAttribute::AllocateCachedValues);
					if constexpr (std::is_trivially_copyable_v<ValueType>)
					{
						CachedValues.SetNumUninitialized(InKeys.GetNum());
					}
					else
					{
						CachedValues.SetNum(InKeys.GetNum());
					}
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(PCGAttributeAccessorHelpers::SortByAttribute::GatherCachedValues);
					InAccessor.GetRange(TArrayView<ValueType>(CachedValues), 0, InKeys);
				}


				// Pass bAscending bool to the compare function to be able to negate the condition on equal, for it to be also stable in descending mode.
				auto CompareAscending = [&CachedValues, &CustomGetIndex, &CompareLess, bAscending](int LHS, int RHS)
				{
					const int32 LHSIndex = CustomGetIndex(LHS);
					const int32 RHSIndex = CustomGetIndex(RHS);
					return CompareLess(CachedValues[LHSIndex], CachedValues[RHSIndex], LHSIndex, RHSIndex, bAscending);
				};

				auto CompareDescending = [&CompareAscending](int LHS, int RHS) { return !CompareAscending(LHS, RHS); };

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(PCGAttributeAccessorHelpers::SortByAttribute::SortIndexes);
					if (bAscending)
					{
						ElementsIndexes.Sort(CompareAscending);
					}
					else
					{
						ElementsIndexes.Sort(CompareDescending);
					}
				}
			}
		};

		// Perform the sorting on the indexes using the attribute values provided by the accessor.
		PCGMetadataAttribute::CallbackWithRightType(InAccessor.GetUnderlyingType(), SortIndexes);

		return ElementsIndexes;
	}

	/**
	* Sorts array given the accessors and keys of the array. Sort is stable by default.
	* A custom function can be used to extract the index of the elements of the array,
	* and sort ascending or descending by the values associated with that index.
	* CompareLess method can also be provided, method signature needs to follow DefaultStableCompareLess.
	* We need the decltypes on both function types to allow to provide default values for both callbacks.
	* Check DefaultIndexGetter and DefaultStableCompareLess for their signatures.
	*/
	template <typename T, typename GetIndexFunc = decltype(Private::DefaultIndexGetter), typename CompareLessFunc = decltype(Private::DefaultStableCompareLessLambda)>
	void SortByAttribute(const IPCGAttributeAccessor& InAccessor, const IPCGAttributeAccessorKeys& InKeys, TArray<T>& InArray, bool bAscending, GetIndexFunc CustomGetIndex = Private::DefaultIndexGetter, CompareLessFunc CompareLess = Private::DefaultStableCompareLessLambda)
	{
		TArray<int32> SortedKeyIndices = SortKeyIndicesByAttribute(InAccessor, InKeys, InArray.Num(), bAscending, CustomGetIndex, CompareLess);

		// Write back the values according to the sorted indexes.
		TArray<T> SortedArray;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGAttributeAccessorHelpers::SortByAttribute::WriteBackInSortedArray);
			SortedArray.Reserve(InArray.Num());

			for (int i = 0; i < InArray.Num(); ++i)
			{
				SortedArray.Add(MoveTemp(InArray[SortedKeyIndices[i]]));
			}
		}

		InArray = MoveTemp(SortedArray);
	}

	template <typename T>
	bool GetOverrideParamValue(const IPCGAttributeAccessor& InAccessor, T& OutValue)
	{
		// Override were using the first entry (0) by default.
		FPCGAttributeAccessorKeysEntries FirstEntry(PCGFirstEntryKey);
		return InAccessor.Get<T>(OutValue, FirstEntry, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);
	}
}
