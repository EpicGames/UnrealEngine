// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataCommon.h"

#define UE_API PCG_API

class FPCGMetadataAttributeBase;
class FPCGMetadataDomain;
struct FPCGPoint;
class UPCGMetadata;
class UPCGSplineData;
class UPCGPolygon2DData;
class UPCGPolyLineData;
class UPCGBasePointData;

///////////////////////////////////////////////////////////////////////

namespace PCGAttributeAccessorKeys
{
	template <typename T, typename Container, typename Func>
	bool GetKeys(Container& InContainer, int32 InStart, TArrayView<T*> OutItems, Func&& Transform)
	{
		if (InContainer.Num() == 0)
		{
			return false;
		}

		int32 Current = InStart;
		if (Current >= InContainer.Num())
		{
			Current %= InContainer.Num();
		}

		for (int32 i = 0; i < OutItems.Num(); ++i)
		{
			OutItems[i] = Transform(InContainer[Current++]);
			if (Current >= InContainer.Num())
			{
				Current = 0;
			}
		}

		return true;
	}

	/**
	 * Utility wrapper around IsChildOf to check if a class passed as argument is compatible with the templated class, which is what the keys are storing.
	 * Can only be checked if ObjectType is a UClass or UStruct.
	 * @tparam ObjectType Class of the underlying object. For example for the points keys, ObjectType will be FPCGPoint.
	 * @param InClass Class of the requested object. A point property could check that the keys is supporting FPCGPoint.
	 * @return True if it is supported. False otherwise.
	 */
	template <typename ObjectType>
	bool IsClassSupported(const UStruct* InClass)
	{		
		if constexpr (TModels_V<CStaticClassProvider, ObjectType> || TModels_V<CStaticStructProvider, ObjectType>)
		{
			return InClass && InClass->IsChildOf<ObjectType>();
		}
		else
		{
			return false;
		}
	}
}

///////////////////////////////////////////////////////////////////////

/**
* Base class to identify keys to use with an accessor.
*/
class IPCGAttributeAccessorKeys
{
public:
	explicit IPCGAttributeAccessorKeys(bool bInReadOnly)
		: bIsReadOnly(bInReadOnly)
	{}

	virtual ~IPCGAttributeAccessorKeys() = default;

	/**
	* Retrieve in the given view pointers of the wanted type
	* Need to be a supported type, such as FPCGPoint, PCGMetadataEntryKey or void.
	* It will wrap around if the index/range goes outside the number of keys.
	* @param InStart - Index to start looking in the keys.
	* @param OutKeys - View on the out keys. Its size will indicate the number of elements to get.
	* @return true if it succeeded, false otherwise. (like num == 0,unsupported type or read only)
	*/
	template <typename ObjectType>
	bool GetKeys(int32 InStart, TArrayView<ObjectType*> OutKeys);

	// Same function but const.
	template <typename ObjectType>
	bool GetKeys(int32 InStart, TArrayView<const ObjectType*> OutKeys) const;

	/**
	 * Retrieve in the given array indices that can be accessed for Accessor/AccessorKeys that support it
	 * It will wrap around if the index/range goes outside the number of keys.
	 * @param InStart - Index to start looking in the keys.
	 * @param InCount - Number of indices to get.
	 * @param OutKeyIndices - Out array of key indices. 
	 * @param OutContiguous - Out value notifying caller that OutKeyIndices is empty but key indices are contiguous so [InStart, InStart+Count-1] is a valid range for the accessor
	 * @return true if it succeeded, false otherwise.
	 */
	virtual bool GetKeyIndices(int32 InStart, int32 InCount, TArray<int32>& OutKeyIndices, bool& bOutContiguous) const { return false; }

	/**
	* Retrieve in the given argument pointer of the wanted type.
	* Need to be a supported type, such as FPCGPoint, PCGMetadataEntryKey or void.
	* It will wrap around if the index goes outside the number of keys.
	* @param InStart - Index to start looking in the keys.
	* @param OutObject - Reference on a pointer, where the key will be written.
	* @return true if it succeeded, false otherwise. (like num == 0, unsupported type or read only)
	*/
	template <typename ObjectType>
	bool GetKey(int32 InStart, ObjectType*& OutObject)
	{
		return GetKeys(InStart, TArrayView<ObjectType*>(&OutObject, 1));
	}

	// Same function but const
	template <typename ObjectType>
	bool GetKey(int32 InStart, ObjectType const*& OutObject) const
	{
		return GetKeys(InStart, TArrayView<const ObjectType*>(&OutObject, 1));
	}

	/**
	* Retrieve in the given argument pointer of the wanted type at the index 0.
	* Need to be a supported type, such as FPCGPoint, PCGMetadataEntryKey or void.
	* @param OutObject - Reference on a pointer, where the key will be written.
	* @return true if it succeeded, false otherwise. (like num == 0, unsupported type or read only)
	*/
	template <typename ObjectType>
	bool GetKey(ObjectType*& OutObject)
	{
		return GetKeys(/*InStart=*/ 0, TArrayView<ObjectType*>(&OutObject, 1));
	}

	// Same function but const
	template <typename ObjectType>
	bool GetKey(ObjectType const*& OutObject) const
	{
		return GetKeys(/*InStart=*/ 0, TArrayView<const ObjectType*>(&OutObject, 1));
	}

	/*
	* Returns the number of keys.
	*/
	virtual int32 GetNum() const = 0;

	bool IsReadOnly() const { return bIsReadOnly; }

	/** Returns true if GetGenericObjectKeys would return this class/struct */
	virtual bool IsClassSupported(const UStruct* InClass) const { return false; }

protected:
	virtual bool GetPointKeys(int32 InStart, TArrayView<FPCGPoint*> OutPoints) { return false; }
	virtual bool GetPointKeys(int32 InStart, TArrayView<const FPCGPoint*> OutPoints) const { return false; }

	virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<void*> OutObjects) { return false; }
	virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<const void*> OutObjects) const { return false; }

	virtual bool GetMetadataEntryKeys(int32 InStart, TArrayView<PCGMetadataEntryKey*> OutEntryKeys) { return false; }
	virtual bool GetMetadataEntryKeys(int32 InStart, TArrayView<const PCGMetadataEntryKey*> OutEntryKeys) const { return false; }

	bool bIsReadOnly = false;
};

///////////////////////////////////////////////////////////////////////

/**
* Key around a metadata entry key
*/
class FPCGAttributeAccessorKeysEntries : public IPCGAttributeAccessorKeys
{
public:
	UE_DEPRECATED(5.5, "This key accessor is deprecated and replaced by the one taking a const or non-const UPCGMetadata object instead")
	UE_API explicit FPCGAttributeAccessorKeysEntries(const FPCGMetadataAttributeBase* Attribute);

	UE_API explicit FPCGAttributeAccessorKeysEntries(PCGMetadataEntryKey EntryKey);
	UE_API explicit FPCGAttributeAccessorKeysEntries(const TArrayView<PCGMetadataEntryKey>& InEntries);
	UE_API explicit FPCGAttributeAccessorKeysEntries(const TArrayView<const PCGMetadataEntryKey>& InEntries);

	// Iterates on all the entries in the metadata. By default, const keys don't have the default value if empty, non-const have it if empty.
	UE_API explicit FPCGAttributeAccessorKeysEntries(const UPCGMetadata* Metadata, bool bAddDefaultValueIfEmpty = false);
	UE_API explicit FPCGAttributeAccessorKeysEntries(UPCGMetadata* Metadata, bool bAddDefaultValueIfEmpty = true);

	UE_API explicit FPCGAttributeAccessorKeysEntries(const FPCGMetadataDomain* Metadata, bool bAddDefaultValueIfEmpty = false);
	UE_API explicit FPCGAttributeAccessorKeysEntries(FPCGMetadataDomain* Metadata, bool bAddDefaultValueIfEmpty = true);

	virtual int32 GetNum() const override { return Entries.Num(); }

protected:
	/** For subclasses that have their own initialization logic, but protected to not make it default constructible. */
	UE_API FPCGAttributeAccessorKeysEntries();

	UE_API virtual bool GetMetadataEntryKeys(int32 InStart, TArrayView<PCGMetadataEntryKey*> OutEntryKeys) override;
	UE_API virtual bool GetMetadataEntryKeys(int32 InStart, TArrayView<const PCGMetadataEntryKey*> OutEntryKeys) const override;

	UE_API void InitializeFromMetadata(const FPCGMetadataDomain* Metadata, bool bAddDefaultValueIfEmpty);

	TArrayView<PCGMetadataEntryKey> Entries;
	TArray<PCGMetadataEntryKey> ExtractedEntries;
};

///////////////////////////////////////////////////////////////////////

/**
* Key around points
*/
class FPCGAttributeAccessorKeysPoints : public IPCGAttributeAccessorKeys
{
public:
	UE_API FPCGAttributeAccessorKeysPoints(const TArrayView<FPCGPoint>& InPoints);
	UE_API FPCGAttributeAccessorKeysPoints(const TArrayView<const FPCGPoint>& InPoints);

	UE_API explicit FPCGAttributeAccessorKeysPoints(FPCGPoint& InPoint);
	UE_API explicit FPCGAttributeAccessorKeysPoints(const FPCGPoint& InPoint);

	virtual int32 GetNum() const override { return Points.Num(); }

	UE_API virtual bool IsClassSupported(const UStruct* InClass) const override;

protected:
	UE_API virtual bool GetKeyIndices(int32 InStart, int32 InCount, TArray<int32>& OutKeyIndices, bool& bOutContiguous) const override;

	UE_API virtual bool GetPointKeys(int32 InStart, TArrayView<FPCGPoint*> OutPoints) override;
	UE_API virtual bool GetPointKeys(int32 InStart, TArrayView<const FPCGPoint*> OutPoints) const override;

	UE_API virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<void*> OutObjects) override;
	UE_API virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<const void*> OutObjects) const override;

	UE_API virtual bool GetMetadataEntryKeys(int32 InStart, TArrayView<PCGMetadataEntryKey*> OutEntryKeys) override;
	UE_API virtual bool GetMetadataEntryKeys(int32 InStart, TArrayView<const PCGMetadataEntryKey*> OutEntryKeys) const override;

	TArrayView<FPCGPoint> Points;
};

///////////////////////////////////////////////////////////////////////

/**
* Key around subset of points
*/
class FPCGAttributeAccessorKeysPointsSubset : public IPCGAttributeAccessorKeys
{
public:
	UE_API FPCGAttributeAccessorKeysPointsSubset(const TArrayView<FPCGPoint>& InPoints, const TArrayView<const int32>& InPointIndices);
	UE_API FPCGAttributeAccessorKeysPointsSubset(const TArrayView<const FPCGPoint>& InPoints, const TArrayView<const int32>& InPointIndices);

	UE_API FPCGAttributeAccessorKeysPointsSubset(TArray<FPCGPoint*> InPointPtrs);
	UE_API FPCGAttributeAccessorKeysPointsSubset(TArray<const FPCGPoint*> InPointPtrs);

	UE_API FPCGAttributeAccessorKeysPointsSubset(const UPCGBasePointData* InPointData, const TArrayView<const int32>& InPointIndices);
	UE_API FPCGAttributeAccessorKeysPointsSubset(UPCGBasePointData* InPointData, const TArrayView<const int32>& InPointIndices);

	virtual int32 GetNum() const override { return Points.Num() > 0 ? Points.Num() : PointIndices.Num(); }

	UE_API virtual bool IsClassSupported(const UStruct* InClass) const override;

protected:
	UE_API virtual bool GetKeyIndices(int32 InStart, int32 InCount, TArray<int32>& OutKeyIndices, bool& bOutContiguous) const override;

	UE_API virtual bool GetPointKeys(int32 InStart, TArrayView<FPCGPoint*> OutPoints) override;
	UE_API virtual bool GetPointKeys(int32 InStart, TArrayView<const FPCGPoint*> OutPoints) const override;

	UE_API virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<void*> OutObjects) override;
	UE_API virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<const void*> OutObjects) const override;

	UE_API virtual bool GetMetadataEntryKeys(int32 InStart, TArrayView<PCGMetadataEntryKey*> OutEntryKeys) override;
	UE_API virtual bool GetMetadataEntryKeys(int32 InStart, TArrayView<const PCGMetadataEntryKey*> OutEntryKeys) const override;

	TArray<FPCGPoint*> Points;

	UPCGBasePointData* PointData = nullptr;
	TArray<int32> PointIndices;
};

/////////////////////////////////////////////////////////////////

/**
* Key around generic objects. 
* Make sure ObjectType is not a pointer nor a reference, since we convert those to void*, it could lead to
* very bad situations if we try to convert a T** to a void*.
*/
template <typename ObjectType, typename = typename std::enable_if_t<!std::is_pointer_v<ObjectType> && !std::is_reference_v<ObjectType>>>
class FPCGAttributeAccessorKeysGeneric : public IPCGAttributeAccessorKeys
{
public:
	FPCGAttributeAccessorKeysGeneric(const TArrayView<ObjectType>& InObjects)
		: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ false)
		, Objects(InObjects)
	{}

	FPCGAttributeAccessorKeysGeneric(const TArrayView<const ObjectType>& InObjects)
		: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ true)
		, Objects(const_cast<ObjectType*>(InObjects.GetData()), InObjects.Num())
	{}

	explicit FPCGAttributeAccessorKeysGeneric(ObjectType& InObject)
		: FPCGAttributeAccessorKeysGeneric(TArrayView<ObjectType>(&InObject, 1))
	{}

	explicit FPCGAttributeAccessorKeysGeneric(const ObjectType& InObject)
		: FPCGAttributeAccessorKeysGeneric(TArrayView<const ObjectType>(&InObject, 1))
	{}

	virtual int32 GetNum() const override { return Objects.Num(); }

	virtual bool IsClassSupported(const UStruct* InClass) const override
	{
		return PCGAttributeAccessorKeys::IsClassSupported<ObjectType>(InClass);
	}

protected:
	virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<void*> OutObjects) override
	{
		return PCGAttributeAccessorKeys::GetKeys(Objects, InStart, OutObjects, [](ObjectType& Obj) -> ObjectType* { return &Obj; });
	}

	virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<const void*> OutObjects) const override
	{
		return PCGAttributeAccessorKeys::GetKeys(Objects, InStart, OutObjects, [](const ObjectType& Obj) -> const ObjectType* { return &Obj; });
	}

	TArrayView<ObjectType> Objects;
};

/////////////////////////////////////////////////////////////////

/**
* Unique Key around a single object.
* Necessary if ObjectType is void, but keep a template version for completeness.
* Useful when you want to use the accessors Get/Set methods on a single object.
* Make sure ObjectType is not a pointer nor a reference, since we convert those to void*, it could lead to
* very bad situations if we try to convert a T** to a void*.
*/
template <typename ObjectType, typename = typename std::enable_if_t<!std::is_pointer_v<ObjectType> && !std::is_reference_v<ObjectType>>>
class FPCGAttributeAccessorKeysSingleObjectPtr : public IPCGAttributeAccessorKeys
{
public:

	FPCGAttributeAccessorKeysSingleObjectPtr()
		: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ true)
		, Ptr(nullptr)
	{}

	explicit FPCGAttributeAccessorKeysSingleObjectPtr(ObjectType* InPtr)
		: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ false)
		, Ptr(InPtr)
	{}

	explicit FPCGAttributeAccessorKeysSingleObjectPtr(const ObjectType* InPtr)
		: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ true)
		, Ptr(const_cast<ObjectType*>(InPtr))
	{}

	virtual int32 GetNum() const override { return 1; }

	virtual bool IsClassSupported(const UStruct* InClass) const override
	{
		return PCGAttributeAccessorKeys::IsClassSupported<ObjectType>(InClass);
	}

protected:
	virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<void*> OutObjects) override
	{
		if (Ptr == nullptr)
		{
			return false;
		}

		for (int32 i = 0; i < OutObjects.Num(); ++i)
		{
			OutObjects[i] = Ptr;
		}

		return true;
	}

	virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<const void*> OutObjects) const override
	{
		if (Ptr == nullptr)
		{
			return false;
		}

		for (int32 i = 0; i < OutObjects.Num(); ++i)
		{
			OutObjects[i] = Ptr;
		}

		return true;
	}

	ObjectType* Ptr = nullptr;
};

/////////////////////////////////////////////////////////////////

/**
* Type erasing generic keys. Allow to store void* keys, if we are dealing with addresses instead
* of plain objects.
* We can't use FPCGAttributeAccessorKeysGeneric since it has a constructor taking a reference on a object,
* and you can't have void&.
*/
class FPCGAttributeAccessorKeysGenericPtrs : public IPCGAttributeAccessorKeys
{
public:
	FPCGAttributeAccessorKeysGenericPtrs(const TArrayView<void*>& InPtrs)
		: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ false)
		, Ptrs(InPtrs)
	{}

	FPCGAttributeAccessorKeysGenericPtrs(const TArrayView<const void*>& InPtrs)
		: IPCGAttributeAccessorKeys(/*bInReadOnly=*/ true)
		, Ptrs(const_cast<void**>(InPtrs.GetData()), InPtrs.Num())
	{}

	virtual int32 GetNum() const override { return Ptrs.Num(); }

protected:
	virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<void*> OutObjects) override
	{
		return PCGAttributeAccessorKeys::GetKeys(Ptrs, InStart, OutObjects, [](void* Ptr) -> void* { return Ptr; });
	}

	virtual bool GetGenericObjectKeys(int32 InStart, TArrayView<const void*> OutObjects) const override
	{
		return PCGAttributeAccessorKeys::GetKeys(Ptrs, InStart, OutObjects, [](const void* Ptr) -> const void* { return Ptr; });
	}

	TArrayView<void*> Ptrs;
};

/////////////////////////////////////////////////////////////////
/**
* Unique Key around a polyline data.
*/
template<typename PolyLineType>
class FPCGAttributeAccessorKeysPolyLineData : public FPCGAttributeAccessorKeysSingleObjectPtr<PolyLineType>
{
public:
	FPCGAttributeAccessorKeysPolyLineData();
	explicit FPCGAttributeAccessorKeysPolyLineData(PolyLineType* InPtr, bool bInGlobalData);
	explicit FPCGAttributeAccessorKeysPolyLineData(const PolyLineType* InPtr, bool bInGlobalData);

	virtual int32 GetNum() const override;

private:
	bool bGlobalData = false;
};

/**
* Aliases for polyline keys
*/
typedef FPCGAttributeAccessorKeysPolyLineData<UPCGSplineData> FPCGAttributeAccessorKeysSplineData;
typedef FPCGAttributeAccessorKeysPolyLineData<UPCGPolygon2DData> FPCGAttributeAccessorKeysPolygon2DData;

/**
* Keys for metadata on a spline data
*/
template<typename PolyLineType>
class FPCGAttributeAccessorKeysPolyLineDataEntries : public FPCGAttributeAccessorKeysEntries
{
public:
	explicit FPCGAttributeAccessorKeysPolyLineDataEntries(const PolyLineType* InPolyLineData);
	explicit FPCGAttributeAccessorKeysPolyLineDataEntries(PolyLineType* InPolyLineData);

	virtual int32 GetNum() const override;

private:
	const PolyLineType* Ptr = nullptr;
};

/**
* Aliases for polyline keys
*/
typedef FPCGAttributeAccessorKeysPolyLineDataEntries<UPCGSplineData> FPCGAttributeAccessorKeysSplineDataEntries;
typedef FPCGAttributeAccessorKeysPolyLineDataEntries<UPCGPolygon2DData> FPCGAttributeAccessorKeysPolygon2DDataEntries;

template <typename ObjectType>
inline bool IPCGAttributeAccessorKeys::GetKeys(int32 InStart, TArrayView<ObjectType*> OutKeys)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IPCGAttributeAccessorKeys::GetKeys);

	if (bIsReadOnly)
	{
		return false;
	}

	if constexpr (std::is_same_v<ObjectType, FPCGPoint>)
	{
		return GetPointKeys(InStart, OutKeys);
	}
	else if constexpr (std::is_same_v<ObjectType, PCGMetadataEntryKey>)
	{
		return GetMetadataEntryKeys(InStart, OutKeys);
	}
	else if constexpr (std::is_same_v<ObjectType, void>)
	{
		return GetGenericObjectKeys(InStart, OutKeys);
	}
	else
	{
		return false;
	}
}

template <typename ObjectType>
inline bool IPCGAttributeAccessorKeys::GetKeys(int32 InStart, TArrayView<const ObjectType*> OutKeys) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IPCGAttributeAccessorKeys::GetKeys);

	if constexpr (std::is_same_v<ObjectType, FPCGPoint>)
	{
		return GetPointKeys(InStart, OutKeys);
	}
	else if constexpr (std::is_same_v<ObjectType, PCGMetadataEntryKey>)
	{
		return GetMetadataEntryKeys(InStart, OutKeys);
	}
	else if constexpr (std::is_same_v<ObjectType, void>)
	{
		return GetGenericObjectKeys(InStart, OutKeys);
	}
	else
	{
		return false;
	}
}

#undef UE_API
