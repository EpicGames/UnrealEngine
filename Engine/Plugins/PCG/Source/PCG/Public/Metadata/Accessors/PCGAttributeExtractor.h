// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

class IPCGAttributeAccessor;

namespace PCGAttributeExtractorConstants
{
	// Vector/Quat
	const FName VectorX = TEXT("X");
	const FName VectorY = TEXT("Y");
	const FName VectorZ = TEXT("Z");
	const FName VectorW = TEXT("W");
	const FName VectorR = TEXT("R");
	const FName VectorG = TEXT("G");
	const FName VectorB = TEXT("B");
	const FName VectorA = TEXT("A");
	const FName VectorLength = TEXT("Length");
	const FName VectorSquaredLength = TEXT("SquaredLength");
	const FName VectorSize = TEXT("Size");
	const FName VectorNormalized = TEXT("Normalized");

	const FName VectorExtractorNames[] =
	{
		VectorX,
		VectorY,
		VectorZ,
		VectorW,
		VectorR,
		VectorG,
		VectorB,
		VectorA,
		VectorLength,
		VectorSize,
		VectorSquaredLength,
		VectorNormalized
	};

	// Transform
	const FName TransformLocation = TEXT("Location");
	const FName TransformRotation = TEXT("Rotation");
	const FName TransformScale = TEXT("Scale");

	const FName TransformExtractorNames[] =
	{
		TransformLocation,
		TransformRotation,
		TransformScale
	};

	// Rotator/Quat
	const FName RotatorPitch = TEXT("Pitch");
	const FName RotatorYaw = TEXT("Yaw");
	const FName RotatorRoll = TEXT("Roll");
	const FName RotatorForward = TEXT("Forward");
	const FName RotatorRight = TEXT("Right");
	const FName RotatorUp = TEXT("Up");
	const FName RotatorEulerAngles = TEXT("EulerAngles");

	const FName RotatorExtractorNames[] =
	{
		RotatorPitch,
		RotatorYaw,
		RotatorRoll,
		RotatorForward,
		RotatorRight,
		RotatorUp,
		RotatorEulerAngles
	};

	// String
	const FName StringLength = TEXT("Length");
}

namespace PCGAttributeExtractor
{
	// Only instantiated for FVector2D, FVector, FVector4 and FQuat
	template <typename VectorType>
	TUniquePtr<IPCGAttributeAccessor> CreateVectorExtractor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);

	TUniquePtr<IPCGAttributeAccessor> CreateRotatorExtractor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
	TUniquePtr<IPCGAttributeAccessor> CreateQuatExtractor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);

	TUniquePtr<IPCGAttributeAccessor> CreateTransformExtractor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);

	template <typename StringType>
	TUniquePtr<IPCGAttributeAccessor> CreateStringExtractor(TUniquePtr<IPCGAttributeAccessor> InAccessor, FName Name, bool& bOutSuccess);
}
