// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPCGAttributeAccessorTpl.h"

#include "PCGPoint.h"
#include "Data/PCGBasePointData.h"

/**
* Templated accessor class for custom point properties. Need a getter and a setter, defined in the
* FPCGPoint class.
* Key supported: Points
*/
template <typename T>
class FPCGCustomPointAccessor : public IPCGAttributeAccessorT<FPCGCustomPointAccessor<T>>
{
public:
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGCustomPointAccessor<T>>;

	FPCGCustomPointAccessor(const FPCGPoint::PointCustomPropertyGetter& InGetter, const FPCGPoint::PointCustomPropertySetter& InSetter)
		: Super(/*bInReadOnly=*/ false)
		, Getter(InGetter)
		, Setter(InSetter)
	{}

	FPCGCustomPointAccessor(const FPCGPoint::PointCustomPropertyGetter& InGetter)
	: Super(/*bInReadOnly=*/ true)
	, Getter(InGetter)
	{}

	bool GetRangeImpl(TArrayView<T> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		TArray<const FPCGPoint*> PointKeys;
		PointKeys.SetNum(OutValues.Num());
		TArrayView<const FPCGPoint*> PointKeysView(PointKeys);
		if (!Keys.GetKeys(Index, PointKeysView))
		{
			return false;
		}

		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			Getter(*PointKeys[i], &OutValues[i]);
		}

		return true;
	}

	bool SetRangeImpl(TArrayView<const T> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		TArray<FPCGPoint*> PointKeys;
		PointKeys.SetNum(InValues.Num());
		TArrayView<FPCGPoint*> PointKeysView(PointKeys);
		if (!Keys.GetKeys(Index, PointKeysView))
		{
			return false;
		}

		for (int32 i = 0; i < InValues.Num(); ++i)
		{
			Setter(*PointKeys[i], &InValues[i]);
		}

		return true;
	}

private:
	FPCGPoint::PointCustomPropertyGetter Getter;
	FPCGPoint::PointCustomPropertySetter Setter;
};

/**
* Very simple accessor that returns a constant value. Read only
* Key supported: All
*/
template <typename T>
class FPCGConstantValueAccessor : public IPCGAttributeAccessorT<FPCGConstantValueAccessor<T>>
{
public:
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGConstantValueAccessor<T>>;

	FPCGConstantValueAccessor(const T& InValue)
		: Super(/*bInReadOnly=*/ true)
		, Value(InValue)
	{}

	bool GetRangeImpl(TArrayView<T> OutValues, int32, const IPCGAttributeAccessorKeys&) const
	{
		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			OutValues[i] = Value;
		}

		return true;
	}

	bool SetRangeImpl(TArrayView<const T>, int32, IPCGAttributeAccessorKeys&, EPCGAttributeAccessorFlags)
	{
		return false;
	}

private:
	T Value;
};

/**
* To chain accessors. T is the type of this accessor. U is the type of the underlying accessor.
* Key supported: Same as the underlying accessor
*/
template <typename T, typename U>
class FPCGChainAccessor : public IPCGAttributeAccessorT<FPCGChainAccessor<T,U>>
{
public:
	using ChainGetter = TFunction<T(const U&)>;
	using ChainSetter = TFunction<void(U&, const T&)>;

	using Type = T;
	using OtherType = U;
	using Super = IPCGAttributeAccessorT<FPCGChainAccessor<T, U>>;

	FPCGChainAccessor(TUniquePtr<IPCGAttributeAccessor> InAccessor, const ChainGetter& InGetter)
		: Super(/*bInReadOnly=*/ true)
		, Accessor(std::move(InAccessor))
		, Getter(InGetter)
		, Setter()
	{
		check(Accessor);
	}

	FPCGChainAccessor(TUniquePtr<IPCGAttributeAccessor> InAccessor, const ChainGetter& InGetter, const ChainSetter& InSetter)
		: Super(/*bInReadOnly=*/ !InAccessor || InAccessor->IsReadOnly())
		, Accessor(std::move(InAccessor))
		, Getter(InGetter)
		, Setter(InSetter)
	{
		check(Accessor);
	}

	bool GetRangeImpl(TArrayView<T> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		TArray<U> TempValues;
		if constexpr (std::is_trivially_copyable_v<U>)
		{
			TempValues.SetNumUninitialized(OutValues.Num());
		}
		else
		{
			TempValues.SetNum(OutValues.Num());
		}

		if (!Accessor->GetRange<U>(TempValues, Index, Keys))
		{
			return false;
		}

		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			OutValues[i] = Getter(TempValues[i]);
		}

		return true;
	}

	bool SetRangeImpl(TArrayView<const T> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		TArray<U> TempValues;
		if constexpr (std::is_trivially_copyable_v<U>)
		{
			TempValues.SetNumUninitialized(InValues.Num());
		}
		else
		{
			TempValues.SetNum(InValues.Num());
		}

		if (!Accessor->GetRange<U>(TempValues, Index, Keys))
		{
			return false;
		}

		for (int32 i = 0; i < InValues.Num(); ++i)
		{
			Setter(TempValues[i], InValues[i]);
		}

		return Accessor->SetRange<U>(TempValues, Index, Keys, Flags);
	}

private:
	TUniquePtr<IPCGAttributeAccessor> Accessor;
	ChainGetter Getter;
	ChainSetter Setter;
};

/**
* Very simple accessor that returns the index. Read only
* Key supported: All
*/
class FPCGIndexAccessor : public IPCGAttributeAccessorT<FPCGIndexAccessor>
{
public:
	using Type = int32;
	using Super = IPCGAttributeAccessorT<FPCGIndexAccessor>;

	FPCGIndexAccessor()
		: Super(/*bInReadOnly=*/ true)
	{}

	bool GetRangeImpl(TArrayView<int32> OutValues, int32 Index, const IPCGAttributeAccessorKeys& InKeys) const
	{
		const int32 NumKeys = InKeys.GetNum();
		int32 Counter = Index;

		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			OutValues[i] = Counter++;
			if (Counter >= NumKeys)
			{
				Counter = 0;
			}
		}

		return true;
	}

	bool SetRangeImpl(TArrayView<const int32>, int32, IPCGAttributeAccessorKeys&, EPCGAttributeAccessorFlags)
	{
		return false;
	}
};

class FPCGAttributeAccessorKeysPointIndices : public IPCGAttributeAccessorKeys
{
public:
	FPCGAttributeAccessorKeysPointIndices(UPCGBasePointData* InPointData, bool bAllocateMetadataEntries = false)
		: IPCGAttributeAccessorKeys(/*bInReadOnly=*/false)
		, PointData(InPointData)
	{
		// By default don't allocate Metadata Entries, since we don't know if the keys are going to be used to write into attributes.
		// If it is known that it will write into attributes, set bAllocateMetadataEntries to true.
		MetadataEntryKeys = PointData->GetMetadataEntryValueRange(bAllocateMetadataEntries);
		ConstMetadataEntryKeys = PointData->GetConstMetadataEntryValueRange();
	}

	FPCGAttributeAccessorKeysPointIndices(const UPCGBasePointData* InPointData)
		: IPCGAttributeAccessorKeys(/*bInReadOnly=*/true)
		, PointData(const_cast<UPCGBasePointData*>(InPointData))
	{
		ConstMetadataEntryKeys = PointData->GetConstMetadataEntryValueRange();
	}

	virtual bool GetKeyIndices(int32 InStart, int32 InCount, TArray<int32>& OutKeyIndices, bool& bOutContiguous) const override
	{
		if (PointData->GetNumPoints() == 0)
		{
			return false;
		}

		// Optimization (avoid allocating indices memory)
		if (InStart + InCount <= PointData->GetNumPoints())
		{
			bOutContiguous = true;
			return true;
		}

		OutKeyIndices.SetNumUninitialized(InCount);
		for (int32 Index = 0; Index < InCount; ++Index)
		{
			OutKeyIndices[Index] = (InStart + Index) % PointData->GetNumPoints();
		}

		return true;
	}

	virtual int32 GetNum() const override
	{
		return PointData ? PointData->GetNumPoints() : 0;
	}

protected:
	virtual bool GetMetadataEntryKeys(int32 InStart, TArrayView<PCGMetadataEntryKey*> OutEntryKeys) override
	{
		if (!PointData || PointData->GetNumPoints() == 0)
		{
			return false;
		}

		check(!MetadataEntryKeys.IsEmpty());

		for (int32 Index = 0; Index < OutEntryKeys.Num(); ++Index)
		{
			int32 EntryIndex = (InStart + Index) % MetadataEntryKeys.Num();
			OutEntryKeys[Index] = &MetadataEntryKeys[EntryIndex];
		}

		return true;
	}

	virtual bool GetMetadataEntryKeys(int32 InStart, TArrayView<const PCGMetadataEntryKey*> OutEntryKeys) const override
	{
		if (!PointData || PointData->GetNumPoints() == 0)
		{
			return false;
		}

		check(!ConstMetadataEntryKeys.IsEmpty());

		for (int32 Index = 0; Index < OutEntryKeys.Num(); ++Index)
		{
			int32 EntryIndex = (InStart + Index) % ConstMetadataEntryKeys.Num();
			OutEntryKeys[Index] = &ConstMetadataEntryKeys[EntryIndex];
		}

		return true;
	}

private:
	UPCGBasePointData* PointData = nullptr;
	TPCGValueRange<int64> MetadataEntryKeys;
	TConstPCGValueRange<int64> ConstMetadataEntryKeys;
};

namespace PCGCustomAccessor
{
	template<typename T>
	bool GetRange(TArrayView<T> OutValues, int32 StartIndex, const IPCGAttributeAccessorKeys& Keys, const auto& Range)
	{
		TArray<int32> KeyIndices;
		bool bContiguous = false;
		if (!Keys.GetKeyIndices(StartIndex, OutValues.Num(), KeyIndices, bContiguous))
		{
			return false;
		}

		if (bContiguous)
		{
			for (int32 OutIndex = 0; OutIndex < OutValues.Num(); ++OutIndex)
			{
				OutValues[OutIndex] = static_cast<T>(Range[StartIndex + OutIndex]);
			}
		}
		else
		{
			for (int32 OutIndex = 0; OutIndex < OutValues.Num(); ++OutIndex)
			{
				OutValues[OutIndex] = static_cast<T>(Range[KeyIndices[OutIndex]]);
			}
		}

		return true;
	}
}

template <typename T, typename R = T>
class UE_DEPRECATED(5.7, "Use FPCGNativePointPropertyEnumConstAccessor instead") FPCGNativePointPropertyConstAccessor : public IPCGAttributeAccessorT<FPCGNativePointPropertyConstAccessor<T, R>>
{
public:
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGNativePointPropertyConstAccessor<T, R>>;

	FPCGNativePointPropertyConstAccessor(const UPCGBasePointData* InPointData, EPCGPointNativeProperties InNativeProperty)
		: Super(/*bReadOnly=*/true)
	{
		ValueRange = InPointData->GetConstValueRange<R>(InNativeProperty);
	}

	FPCGNativePointPropertyConstAccessor(UPCGBasePointData* InPointData, EPCGPointNativeProperties InNativeProperty)
		: Super(/*bReadOnly=*/true)
	{
		ValueRange = InPointData->GetConstValueRange<R>(InNativeProperty);
	}

	bool GetRangeImpl(TArrayView<T> OutValues, int32 StartIndex, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGCustomAccessor::GetRange(OutValues, StartIndex, Keys, ValueRange);
	}

	bool SetRangeImpl(TArrayView<const T> InValues, int32 StartIndex, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		return false;
	}

private:
	TConstPCGValueRange<R> ValueRange;
};

template <typename T, typename R = T>
class UE_DEPRECATED(5.7, "Use FPCGNativePointPropertyEnumAccessor instead") FPCGNativePointPropertyAccessor : public IPCGAttributeAccessorT<FPCGNativePointPropertyAccessor<T, R>>
{
public:
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGNativePointPropertyAccessor<T, R>>;

	FPCGNativePointPropertyAccessor(const UPCGBasePointData* InPointData, EPCGPointNativeProperties InNativeProperty)
		: Super(/*bReadOnly=*/false)
	{
		ValueRange = const_cast<UPCGBasePointData*>(InPointData)->GetValueRange<R>(InNativeProperty, /*bAllocate=*/ true);
	}

	FPCGNativePointPropertyAccessor(UPCGBasePointData* InPointData, EPCGPointNativeProperties InNativeProperty)
		: Super(/*bReadOnly=*/false)
	{
		ValueRange = InPointData->GetValueRange<R>(InNativeProperty, /*bAllocate=*/true);
	}

	bool GetRangeImpl(TArrayView<T> OutValues, int32 StartIndex, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGCustomAccessor::GetRange<T>(OutValues, StartIndex, Keys, ValueRange);
	}

	bool SetRangeImpl(TArrayView<const T> InValues, int32 StartIndex, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		if (Keys.IsReadOnly())
		{
			return false;
		}

		TArray<int32> KeyIndices;
		bool bContiguous = false;
		if (!Keys.GetKeyIndices(StartIndex, InValues.Num(), KeyIndices, bContiguous))
		{
			return false;
		}

		if (bContiguous)
		{
			for (int32 InIndex = 0; InIndex < InValues.Num(); ++InIndex)
			{
				ValueRange[StartIndex + InIndex] = (R)InValues[InIndex];
			}
		}
		else
		{
			for (int32 InIndex = 0; InIndex < InValues.Num(); ++InIndex)
			{
				ValueRange[KeyIndices[InIndex]] = (R)InValues[InIndex];
			}
		}

		return true;
	}

private:
	TPCGValueRange<R> ValueRange;
};

template <EPCGPointNativeProperties Property>
class FPCGNativePointPropertyEnumConstAccessor : public IPCGAttributeAccessorT<FPCGNativePointPropertyEnumConstAccessor<Property>>
{
public:
	using Type = typename TPCGPointNativeProperty<Property>::AccessorType;
	using PropertyType = typename TPCGPointNativeProperty<Property>::Type;
	using Super = IPCGAttributeAccessorT<FPCGNativePointPropertyEnumConstAccessor<Property>>;

	FPCGNativePointPropertyEnumConstAccessor(const UPCGBasePointData* InPointData)
		: Super(/*bReadOnly=*/true)
	{
		ValueRange = InPointData->GetConstValueRange<Property>();
	}

	FPCGNativePointPropertyEnumConstAccessor(UPCGBasePointData* InPointData)
		: Super(/*bReadOnly=*/true)
	{
		ValueRange = InPointData->GetConstValueRange<Property>();
	}

	bool GetRangeImpl(TArrayView<Type> OutValues, int32 StartIndex, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGCustomAccessor::GetRange(OutValues, StartIndex, Keys, ValueRange);
	}

	bool SetRangeImpl(TArrayView<const Type> InValues, int32 StartIndex, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		return false;
	}

private:
	typename TPCGPointNativeProperty<Property>::ConstValueRange ValueRange;
};

template <EPCGPointNativeProperties Property>
class FPCGNativePointPropertyEnumAccessor : public IPCGAttributeAccessorT<FPCGNativePointPropertyEnumAccessor<Property>>
{
public:
	using Type = typename TPCGPointNativeProperty<Property>::AccessorType;
	using PropertyType = typename TPCGPointNativeProperty<Property>::Type;
	using Super = IPCGAttributeAccessorT<FPCGNativePointPropertyEnumAccessor<Property>>;

	FPCGNativePointPropertyEnumAccessor(const UPCGBasePointData* InPointData)
		: Super(/*bReadOnly=*/false)
	{
		ValueRange = const_cast<UPCGBasePointData*>(InPointData)->GetValueRange<Property>(/*bAllocate=*/ true);
	}

	FPCGNativePointPropertyEnumAccessor(UPCGBasePointData* InPointData)
		: Super(/*bReadOnly=*/false)
	{
		ValueRange = InPointData->GetValueRange<Property>(/*bAllocate=*/true);
	}

	bool GetRangeImpl(TArrayView<Type> OutValues, int32 StartIndex, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGCustomAccessor::GetRange<Type>(OutValues, StartIndex, Keys, ValueRange);
	}

	bool SetRangeImpl(TArrayView<const Type> InValues, int32 StartIndex, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		if (Keys.IsReadOnly())
		{
			return false;
		}

		TArray<int32> KeyIndices;
		bool bContiguous = false;
		if (!Keys.GetKeyIndices(StartIndex, InValues.Num(), KeyIndices, bContiguous))
		{
			return false;
		}

		if (bContiguous)
		{
			for (int32 InIndex = 0; InIndex < InValues.Num(); ++InIndex)
			{
				ValueRange[StartIndex + InIndex] = (PropertyType)InValues[InIndex];
			}
		}
		else
		{
			for (int32 InIndex = 0; InIndex < InValues.Num(); ++InIndex)
			{
				ValueRange[KeyIndices[InIndex]] = (PropertyType)InValues[InIndex];
			}
		}

		return true;
	}

private:
	typename TPCGPointNativeProperty<Property>::ValueRange ValueRange;
};

template <typename T, typename... Args>
using FPointCustomPropertyGetter = TFunction<bool(int32, T&, Args...)>;

template <typename T, typename... Args>
using FPointCustomPropertySetter = TFunction<bool(int32, const T&, Args...)>;

template <typename T, typename... Args>
class FPCGCustomPointPropertyAccessor : public IPCGAttributeAccessorT<FPCGCustomPointPropertyAccessor<T, Args...>>
{
public:
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGCustomPointPropertyAccessor<T, Args...>>;

	FPCGCustomPointPropertyAccessor(const UPCGBasePointData* InPointData, const FPointCustomPropertyGetter<T, Args...>& InGetter, const FPointCustomPropertySetter<T, Args...>& InSetter, Args&& ...InValueRanges)
		: Super(/*bInReadOnly=*/ false)
		, PointData(const_cast<UPCGBasePointData*>(InPointData))
		, Getter(InGetter)
		, Setter(InSetter)
		, ValueRanges(std::forward<Args>(InValueRanges)...)
	{
	}

	FPCGCustomPointPropertyAccessor(const UPCGBasePointData* InPointData, const FPointCustomPropertyGetter<T, Args...>& InGetter, Args&& ...InValueRanges)
		: Super(/*bInReadOnly=*/ true)
		, PointData(const_cast<UPCGBasePointData*>(InPointData))
		, Getter(InGetter)
		, ValueRanges(std::forward<Args>(InValueRanges)...)
	{
	}

	bool GetRangeImpl(TArrayView<T> OutValues, int32 StartIndex, const IPCGAttributeAccessorKeys& Keys) const
	{
		TArray<int32> KeyIndices;
		bool bContiguous = false;
		if (!Keys.GetKeyIndices(StartIndex, OutValues.Num(), KeyIndices, bContiguous))
		{
			return false;
		}

		if (bContiguous)
		{
			for (int32 OutIndex = 0; OutIndex < OutValues.Num(); ++OutIndex)
			{
				ValueRanges.ApplyAfter(Getter, StartIndex + OutIndex, OutValues[OutIndex]);
			}
		}
		else
		{
			for (int32 OutIndex = 0; OutIndex < OutValues.Num(); ++OutIndex)
			{
				ValueRanges.ApplyAfter(Getter, KeyIndices[OutIndex], OutValues[OutIndex]);
			}
		}

		return true;
	}

	bool SetRangeImpl(TArrayView<const T> InValues, int32 StartIndex, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		if (Keys.IsReadOnly())
		{
			return false;
		}

		TArray<int32> KeyIndices;
		bool bContiguous = false;
		if (!Keys.GetKeyIndices(StartIndex, InValues.Num(), KeyIndices, bContiguous))
		{
			return false;
		}

		if (bContiguous)
		{
			for (int32 InIndex = 0; InIndex < InValues.Num(); ++InIndex)
			{
				ValueRanges.ApplyAfter(Setter, StartIndex + InIndex, InValues[InIndex]);
			}
		}
		else
		{
			for (int32 InIndex = 0; InIndex < InValues.Num(); ++InIndex)
			{
				ValueRanges.ApplyAfter(Setter, KeyIndices[InIndex], InValues[InIndex]);
			}
		}

		return true;
	}

private:
	UPCGBasePointData* PointData = nullptr;
	FPointCustomPropertyGetter<T, Args...> Getter;
	FPointCustomPropertySetter<T, Args...> Setter;
	TTuple<Args...> ValueRanges;
};
