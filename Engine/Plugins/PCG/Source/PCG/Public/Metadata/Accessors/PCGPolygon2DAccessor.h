// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGPolygon2DData.h"
#include "Metadata/Accessors/PCGPropertyAccessor.h"

/**
* Templated accessor for polygon vertices accessor.
* There is an assumption that no vertex will be set by 2 different threads at the same time and that the polygon itself should not be modified (by adding/removing vertices)
* while the accessor is getting/setting values.
* You must not use this accessor directly.
*/
template<typename T, EPCGPolygon2DProperties Target>
class FPCGPolygon2DVerticesAccessor : public IPCGAttributeAccessorT<FPCGPolygon2DVerticesAccessor<T, Target>>
{
public:
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGPolygon2DVerticesAccessor<T, Target>>;

	FPCGPolygon2DVerticesAccessor(bool bIsReadOnly)
		: Super(bIsReadOnly)
	{
	}

	bool GetRangeImpl(TArrayView<Type> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		const void* ContainerKeys = nullptr;
		TArrayView<const void*> ContainerKeysView(&ContainerKeys, 1);
		if (!Keys.GetKeys(Index, ContainerKeysView))
		{
			return false;
		}

		// Validation to not access keys that are not the expected type. Done after the GetKeys, as we also want to discard other type of incompatible
		// keys (like a Default Metadata entry key)
		if (!ensure(Keys.IsClassSupported(UPCGPolygon2DData::StaticClass())))
		{
			return false;
		}

		const UPCGPolygon2DData* Data = static_cast<const UPCGPolygon2DData*>(ContainerKeys);
		const FTransform& Transform = Data->GetTransform();
		const UE::Geometry::FGeneralPolygon2d& Polygon = Data->GetPolygon();
		const TMap<int, TPair<int, int>>& SegmentIndexToSegmentAndHoleIndices = Data->GetSegmentIndexToSegmentAndHoleIndices();

		const int32 NumPoints = SegmentIndexToSegmentAndHoleIndices.Num();
		if (NumPoints == 0)
		{
			return false;
		}

		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			const int32 CurrIndex = (Index + i) % NumPoints;
			auto [SegmentIndex, HoleIndex] = SegmentIndexToSegmentAndHoleIndices[CurrIndex];

			if constexpr (Target == EPCGPolygon2DProperties::Position)
			{
				static_assert(std::is_same_v<Type, FVector>);
				const UE::Geometry::FSegment2d Segment = Polygon.Segment(SegmentIndex, HoleIndex);
				OutValues[i] = Transform.TransformPosition(FVector(Segment.StartPoint(), 0.0));
			}
			else if constexpr (Target == EPCGPolygon2DProperties::Rotation)
			{
				static_assert(std::is_same_v<Type, FQuat>);
				//@todo_pcg: there is a Polygon.GetNormal that we could use, but it blends normals across vertices.
				const UE::Geometry::FSegment2d Segment = Polygon.Segment(SegmentIndex, HoleIndex);
				OutValues[i] = Transform.TransformRotation(FRotationMatrix::MakeFromXZ(FVector(Segment.Direction, 0.0), FVector::UpVector).ToQuat());
			}
			else if constexpr (Target == EPCGPolygon2DProperties::SegmentIndex)
			{
				static_assert(std::is_same_v<Type, int32>);
				OutValues[i] = SegmentIndex;
			}
			else if constexpr (Target == EPCGPolygon2DProperties::HoleIndex)
			{
				static_assert(std::is_same_v<Type, int32>);
				OutValues[i] = HoleIndex;
			}
			else if constexpr (Target == EPCGPolygon2DProperties::SegmentLength)
			{
				static_assert(std::is_same_v<Type, double>);
				const UE::Geometry::FSegment2d Segment = Polygon.Segment(SegmentIndex, HoleIndex);
				OutValues[i] = Segment.Length();
			}
			else if constexpr (Target == EPCGPolygon2DProperties::LocalPosition)
			{
				static_assert(std::is_same_v<Type, FVector2d>);
				const UE::Geometry::FSegment2d Segment = Polygon.Segment(SegmentIndex, HoleIndex);
				OutValues[i] = Segment.StartPoint();
			}
			else if constexpr (Target == EPCGPolygon2DProperties::LocalRotation)
			{
				static_assert(std::is_same_v<Type, FQuat>);
				const UE::Geometry::FSegment2d Segment = Polygon.Segment(SegmentIndex, HoleIndex);
				OutValues[i] = FRotationMatrix::MakeFromXZ(FVector(Segment.Direction, 0.0), FVector::UpVector).ToQuat();
			}
			else
			{
				// Pitfall static assert
				static_assert(!std::is_same_v<Type, Type>);
			}
		}

		return true;
	}

	bool SetRangeImpl(TArrayView<const Type> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
	{
		void* ContainerKeys = nullptr;
		TArrayView<void*> ContainerKeysView(&ContainerKeys, 1);
		if (!Keys.GetKeys(Index, ContainerKeysView))
		{
			return false;
		}

		// Validation to not access keys that are not the expected type. Done after the GetKeys, as we also want to discard other type of incompatible
		// keys (like a Default Metadata entry key)
		if (!ensure(Keys.IsClassSupported(UPCGPolygon2DData::StaticClass())))
		{
			return false;
		}

		UPCGPolygon2DData* Data = static_cast<UPCGPolygon2DData*>(ContainerKeys);
		const FTransform& Transform = Data->GetTransform();
		const TMap<int, TPair<int, int>>& SegmentIndexToSegmentAndHoleIndices = Data->GetSegmentIndexToSegmentAndHoleIndices();

		const int32 NumPoints = SegmentIndexToSegmentAndHoleIndices.Num();
		if (NumPoints == 0)
		{
			return false;
		}

		if constexpr (Target != EPCGPolygon2DProperties::Position && Target != EPCGPolygon2DProperties::LocalPosition)
		{
			return false;
		}
		else
		{
			FTransform InverseTransform;

			if constexpr (Target == EPCGPolygon2DProperties::Position)
			{
				InverseTransform = Transform.Inverse();
			}

			for (int i = 0; i < InValues.Num(); ++i)
			{
				const int32 CurrIndex = (Index + i) % NumPoints;
				auto [SegmentIndex, HoleIndex] = SegmentIndexToSegmentAndHoleIndices[CurrIndex];

				if constexpr (Target == EPCGPolygon2DProperties::Position)
				{
					static_assert(std::is_same_v<Type, FVector3d>);
					FVector LocalPosition = InverseTransform.TransformPosition(InValues[i]);
					Data->UpdateVertexInPlace(FVector2d(LocalPosition.X, LocalPosition.Y), HoleIndex, SegmentIndex);
				}
				else if constexpr (Target == EPCGPolygon2DProperties::LocalPosition)
				{
					static_assert(std::is_same_v<Type, FVector2d>);
					Data->UpdateVertexInPlace(InValues[i], HoleIndex, SegmentIndex);
				}
				else
				{
					// Pitfall static assert
					static_assert(!std::is_same_v<Type, Type>);
				}
			}

			return true;
		}
	}
};

/**
* Templated accessor for global polygon2d data.
*/
template<typename T, EPCGPolygon2DDataProperties Target>
class FPCGPolygon2DDataAccessor : public IPCGAttributeAccessorT<FPCGPolygon2DDataAccessor<T, Target>>
{
public:
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGPolygon2DDataAccessor<T, Target>>;

	FPCGPolygon2DDataAccessor(bool bIsReadOnly)
		: Super(bIsReadOnly)
	{
	}

	bool GetRangeImpl(TArrayView<Type> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		const void* ContainerKeys = nullptr;
		TArrayView<const void*> ContainerKeysView(&ContainerKeys, 1);
		if (!Keys.GetKeys(Index, ContainerKeysView))
		{
			return false;
		}

		// Validation to not access keys that are not the expected type. Done after the GetKeys, as we also want to discard other type of incompatible
		// keys (like a Default Metadata entry key)
		if (!ensure(Keys.IsClassSupported(UPCGPolygon2DData::StaticClass())))
		{
			return false;
		}

		const UPCGPolygon2DData* Data = static_cast<const UPCGPolygon2DData*>(ContainerKeys);
		const FTransform& Transform = Data->GetTransform();
		const UE::Geometry::FGeneralPolygon2d& Polygon = Data->GetPolygon();

		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			if constexpr (Target == EPCGPolygon2DDataProperties::Transform)
			{
				static_assert(std::is_same_v<Type, FTransform>);
				OutValues[i] = Transform;
			}
			else if constexpr (Target == EPCGPolygon2DDataProperties::Area)
			{
				static_assert(std::is_same_v<Type, double>);
				OutValues[i] = Polygon.SignedArea();
			}
			else if constexpr (Target == EPCGPolygon2DDataProperties::Perimeter)
			{
				static_assert(std::is_same_v<Type, double>);
				OutValues[i] = Polygon.Perimeter();
			}
			else if constexpr (Target == EPCGPolygon2DDataProperties::BoundsMin)
			{
				static_assert(std::is_same_v<Type, FVector2d>);
				OutValues[i] = Polygon.Bounds().Min;
			}
			else if constexpr (Target == EPCGPolygon2DDataProperties::BoundsMax)
			{
				static_assert(std::is_same_v<Type, FVector2d>);
				OutValues[i] = Polygon.Bounds().Max;
			}
			else if constexpr (Target == EPCGPolygon2DDataProperties::SegmentCount)
			{
				static_assert(std::is_same_v<Type, int32>);
				OutValues[i] = Data->GetNumSegments();
			}
			else if constexpr (Target == EPCGPolygon2DDataProperties::OuterSegmentCount)
			{
				static_assert(std::is_same_v<Type, int32>);
				OutValues[i] = Polygon.GetOuter().GetVertices().Num();
			}
			else if constexpr (Target == EPCGPolygon2DDataProperties::HoleCount)
			{
				static_assert(std::is_same_v<Type, int32>);
				OutValues[i] = Polygon.GetHoles().Num();
			}
			else if constexpr (Target == EPCGPolygon2DDataProperties::LongestOuterSegmentIndex)
			{
				int32 LongestOuterSegmentIndex = INDEX_NONE;
				double MaxLength = -DBL_MAX;
				const int32 OuterSegmentCount = Polygon.GetOuter().GetVertices().Num();
				for (int SegIndex = 0; SegIndex < OuterSegmentCount; ++SegIndex)
				{
					const double SegLength = Polygon.Segment(SegIndex, -1).Length();
					if (SegLength > MaxLength)
					{
						MaxLength = SegLength;
						LongestOuterSegmentIndex = SegIndex;
					}
				}

				static_assert(std::is_same_v<Type, int32>);
				OutValues[i] = LongestOuterSegmentIndex;
			}
			else if constexpr (Target == EPCGPolygon2DDataProperties::IsClockwise)
			{
				static_assert(std::is_same_v<Type, bool>);
				OutValues[i] = Polygon.OuterIsClockwise();
			}
			else
			{
				// Pitfall static assert
				static_assert(!std::is_same_v<Type, Type>);
			}
		}

		return true;
	}

	bool SetRangeImpl(TArrayView<const Type> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags AccessorFlags)
	{
		void* ContainerKeys = nullptr;
		TArrayView<void*> ContainerKeysView(&ContainerKeys, 1);
		if (!Keys.GetKeys(Index, ContainerKeysView))
		{
			return false;
		}

		// Validation to not access keys that are not the expected type. Done after the GetKeys, as we also want to discard other type of incompatible
		// keys (like a Default Metadata entry key)
		if (!ensure(Keys.IsClassSupported(UPCGPolygon2DData::StaticClass())))
		{
			return false;
		}

		UPCGPolygon2DData* Data = static_cast<UPCGPolygon2DData*>(ContainerKeys);

		if constexpr (Target == EPCGPolygon2DDataProperties::Transform)
		{
			static_assert(std::is_same_v<Type, FTransform>);
			for (int i = 0; i < InValues.Num(); ++i)
			{
				Data->SetTransform(InValues[i], /*bCheckWinding=*/true);
			}

			return true;
		}
		else
		{
			return false;
		}
	}
};