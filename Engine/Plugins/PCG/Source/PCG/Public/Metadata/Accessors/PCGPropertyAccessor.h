// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPCGAttributeAccessorTpl.h"

#include "PCGModule.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "Containers/UnrealString.h" // IWYU pragma: keep
#include "Math/Color.h" // IWYU pragma: keep
#include "UObject/SoftObjectPath.h" // IWYU pragma: keep
#include "UObject/UnrealType.h" // IWYU pragma: keep

class FEnumProperty;

namespace PCGPropertyAccessor
{
	template <typename KeyType, typename ...ArrayArgs>
	void GetContainerKeys(int32 Index, int32 Range, IPCGAttributeAccessorKeys& Keys, TArray<KeyType*, ArrayArgs...>& OutContainer)
	{
		OutContainer.SetNumUninitialized(Range);
		TArrayView<KeyType*> ContainerKeysView(OutContainer);
		if (!Keys.GetKeys(Index, ContainerKeysView))
		{
			OutContainer.Reset();
		}
	}

	template <typename KeyType, typename ...ArrayArgs>
	void GetContainerKeys(int32 Index, int32 Range, const IPCGAttributeAccessorKeys& Keys, TArray<const KeyType*, ArrayArgs...>& OutContainer)
	{
		OutContainer.SetNumUninitialized(Range);
		TArrayView<const KeyType*> ContainerKeysView(OutContainer);
		if (!Keys.GetKeys(Index, ContainerKeysView))
		{
			OutContainer.Reset();
		}
	}

	template <typename T>
	void AddressOffset(const TArray<const FProperty*>& InProperties, TArrayView<T> InContainerKeys, bool bIgnoreLastProperty = false)
	{
		for (int32 j = 0; j < (bIgnoreLastProperty ? InProperties.Num() - 1 : InProperties.Num()); ++j)
		{
			const FProperty* Property = InProperties[j];
			check(Property);
			// No indirection for last property
			const FObjectProperty* ObjectProperty = (j < InProperties.Num() - 1) ? CastField<const FObjectProperty>(Property) : nullptr;
			if (ObjectProperty)
			{
				for (int32 i = 0; i < InContainerKeys.Num(); ++i)
				{
					InContainerKeys[i] = InContainerKeys[i] ? ObjectProperty->GetObjectPropertyValue_InContainer(InContainerKeys[i]) : nullptr;
				}
			}
			else
			{
				for (int32 i = 0; i < InContainerKeys.Num(); ++i)
				{
					InContainerKeys[i] = InContainerKeys[i] ? Property->ContainerPtrToValuePtr<void>(InContainerKeys[i]) : nullptr;
				}
			}
		}
	}

	template <typename T, typename Func>
	bool IterateGet(const TArray<const FProperty*>& Properties, TArrayView<T>& OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys, Func&& Getter)
	{
		TArray<const void*, TInlineAllocator<256>> ContainerKeys;
		GetContainerKeys(Index, OutValues.Num(), Keys, ContainerKeys);
		if (ContainerKeys.IsEmpty())
		{
			return false;
		}

		// Update the addresses of all
		AddressOffset<const void*>(Properties, ContainerKeys);

		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			OutValues[i] = ContainerKeys[i] ? Getter(ContainerKeys[i]) : PCG::Private::MetadataTraits<T>::ZeroValue();
		}

		return true;
	}

	template <typename T, typename Func>
	bool IterateSet(const TArray<const FProperty*>& Properties, TArrayView<const T>& InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, Func&& Setter)
	{
		TArray<void*, TInlineAllocator<256>> ContainerKeys;
		GetContainerKeys(Index, InValues.Num(), Keys, ContainerKeys);
		if (ContainerKeys.IsEmpty())
		{
			return false;
		}

		// Update the addresses of all
		AddressOffset<void*>(Properties, ContainerKeys);

		for (int32 i = 0; i < InValues.Num(); ++i)
		{
			if (ContainerKeys[i])
			{
				Setter(ContainerKeys[i], InValues[i]);
			}
		}

		return true;
	}

	void LogErrorForInvalidObjectWrite(const UClass* PropertyClass, const UClass* ObjectClass, const FSoftObjectPath& Value);
	void LogErrorForInvalidClassWrite(const UClass* MetaClass, const UClass* ObjectMetaClass, const FSoftClassPath& Value);
}

/**
* Interface for Property chain to factorize ctor, fix the chain and storing the property chain
*/
class IPCGPropertyChain
{
public:
	virtual ~IPCGPropertyChain() = default;

protected:
	IPCGPropertyChain(const FProperty* Property, TArray<const FProperty*>&& ExtraProperties);

	const TArray<const FProperty*>& GetPropertyChain() const { return PropertyChain; }

	/** Return the owner class/struct of the top property (if it is a class/struct). Useful to make sure the top property matches the keys type. */
	const UStruct* GetTopPropertyStruct() const;

private:
	TArray<const FProperty*> PropertyChain;
};

/**
* Templated accessor class for numeric properties. Will wrap around a numeric property.
* Do not instanciate it manually, use PCGAttributeAccessorHelpers::CreatePropertyAccessor.
* Only support integral and floating point types.
* Key supported: Generic object
*/
template <typename T>
class FPCGNumericPropertyAccessor : public IPCGAttributeAccessorT<FPCGNumericPropertyAccessor<T>>, IPCGPropertyChain
{
public:
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGNumericPropertyAccessor<T>>;

	FPCGNumericPropertyAccessor(const FNumericProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
		, Property(InProperty)
	{
		static_assert(std::is_integral_v<T> || std::is_floating_point_v<T>);
		check(Property);
	}

	bool GetRangeImpl(TArrayView<T> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> T
			{
				if constexpr (std::is_integral_v<T>)
				{
					return T(Property->GetSignedIntPropertyValue(PropertyAddressData));
				}
				else
				{
					return T(Property->GetFloatingPointPropertyValue(PropertyAddressData));
				}
			});
	}

	bool SetRangeImpl(TArrayView<const T> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
	{
		return PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this](void* PropertyAddressData, const T& Value) -> void
			{
				if constexpr (std::is_integral_v<T>)
				{
					Property->SetIntPropertyValue(PropertyAddressData, static_cast<int64>(Value));
				}
				else
				{
					Property->SetFloatingPointPropertyValue(PropertyAddressData, Value);
				}
			});
	}

private:
	const FNumericProperty* Property = nullptr;
};

/**
* Templated accessor class for enum properties. Will wrap around an enum property.
* Will always convert to int64 for PCG
* Do not instanciate it manually, use PCGAttributeAccessorHelpers::CreatePropertyAccessor.
* Note that setting the property value will be slower, because we need to make sure that the value is a valid value for the enum.
* Key supported: Generic object
*/
class FPCGEnumPropertyAccessor : public IPCGAttributeAccessorT<FPCGEnumPropertyAccessor>, IPCGPropertyChain
{
public:
	using Type = int64;
	using Super = IPCGAttributeAccessorT<FPCGEnumPropertyAccessor>;

	FPCGEnumPropertyAccessor(const FEnumProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {});

	bool GetRangeImpl(TArrayView<int64> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const;
	bool SetRangeImpl(TArrayView<const int64> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags);

private:
	const FEnumProperty* Property = nullptr;
	bool bHasMaxValue = false;
	int64 EnumMaxValue = -1;
};

/**
* Templated accessor class for struct properties. Will wrap around a struct property.
* Do not instanciate it manually, use PCGAttributeAccessorHelpers::CreatePropertyAccessor.
* WARNING: Getting the address of the underlying data from a container using this property
* should point to a "T" object in memory, otherwise it is UD.
* Key supported: Generic object
*/
template <typename T>
class FPCGPropertyStructAccessor : public IPCGAttributeAccessorT<FPCGPropertyStructAccessor<T>>, IPCGPropertyChain
{
public:
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGPropertyStructAccessor<T>>;

	FPCGPropertyStructAccessor(const FStructProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
		, Property(InProperty)
	{
		static_assert(PCG::Private::IsPCGType<T>());
		check(Property);
	}

	bool GetRangeImpl(TArrayView<T> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> T
			{
				return *reinterpret_cast<const T*>(PropertyAddressData);
			});
	}

	bool SetRangeImpl(TArrayView<const T> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
	{
		return PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this](void* PropertyAddressData, const T& Value) -> void
			{
				*reinterpret_cast<T*>(PropertyAddressData) = Value;
			});
	}

private:
	const FStructProperty* Property = nullptr;
};

/**
* Templated accessor class for properties that has a (Get/Set)PropertyValue. Will wrap around a property.
* For example String/Name/Bool
* Do not instanciate it manually, use PCGAttributeAccessorHelpers::CreatePropertyAccessor.
* Key supported: Generic object
*/
template <typename T, typename PropertyType>
class FPCGPropertyAccessor : public IPCGAttributeAccessorT<FPCGPropertyAccessor<T, PropertyType>>, IPCGPropertyChain
{
public:
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGPropertyAccessor<T, PropertyType>>;

	FPCGPropertyAccessor(const PropertyType* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
		, Property(InProperty)
	{
		static_assert(PCG::Private::IsPCGType<T>());
		check(Property);
	}

	bool GetRangeImpl(TArrayView<T> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> T
			{
				return Property->GetPropertyValue(PropertyAddressData);
			});
	}

	bool SetRangeImpl(TArrayView<const T> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
	{
		return PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this](void* PropertyAddressData, const T& Value) -> void
			{
				Property->SetPropertyValue(PropertyAddressData, Value);
			});
	}

private:
	const PropertyType* Property = nullptr;
};

/**
* Templated accessor class for path properties. Will wrap around a soft path property.
* Do not instanciate it manually, use PCGAttributeAccessorHelpers::CreatePropertyAccessor.
* Will always convert to FString for PCG
* Key supported: Generic object
*/
template <typename T>
class FPCGPropertyPathAccessor : public IPCGAttributeAccessorT<FPCGPropertyPathAccessor<T>>, IPCGPropertyChain
{
public:
	using Type = FString;
	using Super = IPCGAttributeAccessorT<FPCGPropertyPathAccessor<T>>;

	FPCGPropertyPathAccessor(const FProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
		, Property(InProperty)
	{
		static_assert(std::is_same_v<FSoftObjectPath, T> || std::is_same_v<FSoftClassPath, T>);
		check(Property);
	}

	bool GetRangeImpl(TArrayView<Type> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> Type
			{
				return reinterpret_cast<const T*>(PropertyAddressData)->ToString();
			});
	}

	bool SetRangeImpl(TArrayView<const Type> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
	{
		return PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this](void* PropertyAddressData, const Type& Value) -> void
			{
				reinterpret_cast<T*>(PropertyAddressData)->SetPath(Value);
			});
	}

private:
	const FProperty* Property = nullptr;
};

/**
* Templated accessor class for soft object ptr properties - produces soft object path.
* Do not instantiate it manually, use PCGAttributeAccessorHelpers::CreatePropertyAccessor.
* Key supported: Generic object
*/
class FPCGPropertySoftObjectPathAccessor : public IPCGAttributeAccessorT<FPCGPropertySoftObjectPathAccessor>, IPCGPropertyChain
{
public:
	using Type = FSoftObjectPath;
	using Super = IPCGAttributeAccessorT<FPCGPropertySoftObjectPathAccessor>;

	FPCGPropertySoftObjectPathAccessor(const FSoftObjectProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
		, Property(InProperty)
	{
		check(Property);
	}

	bool GetRangeImpl(TArrayView<Type> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> Type
		{
			return Property->GetPropertyValue(PropertyAddressData).ToSoftObjectPath();
		});
	}

	bool SetRangeImpl(TArrayView<const Type> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
	{
		return PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this](void* PropertyAddressData, const Type& Value) -> void
		{
			Property->SetPropertyValue(PropertyAddressData, FSoftObjectPtr(Value));
		});
	}

private:
	const FSoftObjectProperty* Property = nullptr;
};

/**
* Templated accessor class for soft class ptr properties - produces soft class path.
* Do not instantiate it manually, use PCGAttributeAccessorHelpers::CreatePropertyAccessor.
* Key supported: Generic object
*/
class FPCGPropertySoftClassPathAccessor : public IPCGAttributeAccessorT<FPCGPropertySoftClassPathAccessor>, IPCGPropertyChain
{
public:
	using Type = FSoftClassPath;
	using Super = IPCGAttributeAccessorT<FPCGPropertySoftClassPathAccessor>;

	FPCGPropertySoftClassPathAccessor(const FSoftClassProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
		, Property(InProperty)
	{
		check(Property);
	}

	bool GetRangeImpl(TArrayView<Type> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> Type
		{
			return FSoftClassPath(Property->GetPropertyValue(PropertyAddressData).ToString());
		});
	}

	bool SetRangeImpl(TArrayView<const Type> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
	{
		return PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this](void* PropertyAddressData, const Type& Value) -> void
		{
			Property->SetPropertyValue(PropertyAddressData, FSoftObjectPtr(Value));
		});
	}

private:
	const FSoftClassProperty* Property = nullptr;
};

/**
* Templated accessor class for object/class ptr properties.
* Do not instantiate it manually, use PCGAttributeAccessorHelpers::CreatePropertyAccessor.
* Will always convert to FSoftObjectPath/FSoftClassPath for PCG
* Note that setting the property value will be slower, because we need to make sure that the Soft Object/Class Path is of the right type.
* It will also force the object to load, as ObjectPtrs are expected to be loaded.
* Key supported: Generic object
*/
template <typename PropertyType> requires std::is_base_of_v<FObjectProperty, PropertyType>
class FPCGPropertyObjectPtrAccessor : public IPCGAttributeAccessorT<FPCGPropertyObjectPtrAccessor<PropertyType>>, IPCGPropertyChain
{
public:
	using Type = std::conditional_t<std::is_same_v<PropertyType, FClassProperty>, FSoftClassPath, FSoftObjectPath>;
	using Super = IPCGAttributeAccessorT<FPCGPropertyObjectPtrAccessor>;

	FPCGPropertyObjectPtrAccessor(const PropertyType* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
		, Property(InProperty)
	{
		check(Property);
	}

	bool GetRangeImpl(TArrayView<Type> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> Type
		{
			return Property->GetPropertyValue(PropertyAddressData).GetPath();
		});
	}

	bool SetRangeImpl(TArrayView<const Type> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
	{
		bool bWriteOK = true;

		const bool bIterateOK = PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this, &bWriteOK](void* PropertyAddressData, const Type& Value) -> void
		{
			UObject* Object = Value.TryLoad();
			if (Object && Property->PropertyClass && !Object->IsA(Property->PropertyClass))
			{
				PCGPropertyAccessor::LogErrorForInvalidObjectWrite(Property->PropertyClass, Object->GetClass(), Value);
				bWriteOK = false;
				return;
			}

			if constexpr (std::is_base_of_v<FClassProperty, PropertyType>)
			{
				const UClass* Class = Cast<const UClass>(Object);
				if (Class && Property->MetaClass && !Class->IsChildOf(Property->MetaClass))
				{
					PCGPropertyAccessor::LogErrorForInvalidObjectWrite(Property->MetaClass, Class, Value);
					bWriteOK = false;
					return;
				}
			}

			Property->SetPropertyValue(PropertyAddressData, Object);
		});

		return bIterateOK && bWriteOK;
	}

private:
	const PropertyType* Property = nullptr;
};

/**
* Special accessor to support attribute selector overrides. Interface with a string.
* Key supported: Generic object
*/
class FPCGAttributePropertySelectorAccessor : public IPCGAttributeAccessorT<FPCGAttributePropertySelectorAccessor>, IPCGPropertyChain
{
public:
	using Type = FString;
	using Super = IPCGAttributeAccessorT<FPCGAttributePropertySelectorAccessor>;

	FPCGAttributePropertySelectorAccessor(const FStructProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
		, Property(InProperty)
	{
		ensure(InProperty && InProperty->Struct && InProperty->Struct->IsA(FPCGAttributePropertySelector::StaticStruct()->GetClass()));
	}

	bool GetRangeImpl(TArrayView<FString> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> FString
		{
			return reinterpret_cast<const FPCGAttributePropertySelector*>(PropertyAddressData)->GetDisplayText().ToString();
		});
	}

	bool SetRangeImpl(TArrayView<const FString> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		return PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this](void* PropertyAddressData, const FString& Value) -> void
		{
			reinterpret_cast<FPCGAttributePropertySelector*>(PropertyAddressData)->Update(Value);
		});
	}

private:
	const FStructProperty* Property = nullptr;
};

/**
* Special accessor to support linear color overrides. Interface with a vector 4. Will be output as RGBA.
* Key supported: Generic object
*/
class FPCGLinearColorAccessor : public IPCGAttributeAccessorT<FPCGLinearColorAccessor>, IPCGPropertyChain
{
public:
	using Type = FVector4;
	using Super = IPCGAttributeAccessorT<FPCGLinearColorAccessor>;

	FPCGLinearColorAccessor(const FStructProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
		, Property(InProperty)
	{
		ensure(InProperty && InProperty->Struct && InProperty->Struct == TBaseStructure<FLinearColor>::Get());
	}

	bool GetRangeImpl(TArrayView<FVector4> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> FVector4
		{
			const FLinearColor* Value = reinterpret_cast<const FLinearColor*>(PropertyAddressData);
			return FVector4(Value->R, Value->G, Value->B, Value->A);
		});
	}

	bool SetRangeImpl(TArrayView<const FVector4> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		return PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this](void* PropertyAddressData, const FVector4& Value) -> void
		{
			*reinterpret_cast<FLinearColor*>(PropertyAddressData) = FLinearColor(Value);
		});
	}

private:
	const FStructProperty* Property = nullptr;
};

/**
* Special accessor to support color overrides. Interface with a vector 4. Will remap [0;255] to [0.0,1.0]
* Key supported: Generic object
*/
class FPCGColorAccessor : public IPCGAttributeAccessorT<FPCGColorAccessor>, IPCGPropertyChain
{
public:
	using Type = FVector4;
	using Super = IPCGAttributeAccessorT<FPCGColorAccessor>;

	FPCGColorAccessor(const FStructProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
		, Property(InProperty)
	{
		ensure(InProperty && InProperty->Struct && InProperty->Struct == TBaseStructure<FColor>::Get());
	}

	bool GetRangeImpl(TArrayView<FVector4> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		return PCGPropertyAccessor::IterateGet(GetPropertyChain(), OutValues, Index, Keys, [this](const void* PropertyAddressData) -> FVector4
		{
			constexpr double Inv255 = 1.0 / 255.0;
			const FColor* Value = reinterpret_cast<const FColor*>(PropertyAddressData);
			return FVector4(Value->R * Inv255, Value->G * Inv255, Value->B * Inv255, Value->A * Inv255);
		});
	}

	bool SetRangeImpl(TArrayView<const FVector4> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags)
	{
		return PCGPropertyAccessor::IterateSet(GetPropertyChain(), InValues, Index, Keys, [this](void* PropertyAddressData, const FVector4& Value) -> void
		{
			*reinterpret_cast<FColor*>(PropertyAddressData) = FLinearColor(Value).QuantizeRound();
		});
	}

private:
	const FStructProperty* Property = nullptr;
};

/**
* Generic accessor class for properties.
* Do not instantiate it manually, use PCGAttributeAccessorHelpers::CreatePropertyAccessor.
* Key supported: Generic object
*/
class FPCGPropertyGenericAccessor : public IPCGAttributeAccessor, IPCGPropertyChain
{
public:
	using Super = IPCGAttributeAccessor;

	PCG_API FPCGPropertyGenericAccessor(const FProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {});
	
	PCG_API virtual bool SupportsGet(const PCG::Private::FOutValues& OutValues) const override;
	PCG_API virtual bool SupportsSet(const PCG::Private::FInValues& InValues) const override;
	PCG_API virtual bool GetRangeVirtual(PCG::Private::FOutValues OutValues, int32 Count, int32 Index, const IPCGAttributeAccessorKeys& Keys) const override;
	PCG_API virtual bool SetRangeVirtual(PCG::Private::FInValues InValues, int32 Count, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags Flags) override;

private:
	/** Functions that validate and copy objects. */
	bool SetRange_Objects_Values(const PCG::Private::FInValuesByValue& InValues, TArrayView<void*> ContainerKeys) const;
	bool SetRange_Objects_Ptr(const PCG::Private::FInValuesByPtr& InValues, TArrayView<void*> ContainerKeys) const;
	bool SetRange_Objects_Arrays(FScriptArrayHelper& Helper, const PCG::Private::FInValuesAsArray& InValues, int32 Index) const;
	bool SetRange_Objects_Sets(FScriptSetHelper& Helper, const PCG::Private::FInValuesAsSet& InValues, int32 Index) const;
	bool SetRange_Objects_Maps(FScriptMapHelper& Helper, const PCG::Private::FInValuesAsMap& InValues, int32 Index) const;

	bool ValidateValue(const TObjectPtr<UObject>& InValue, const bool bUseKeyProperty) const;

	const FProperty* Property = nullptr;
	const FObjectProperty* ObjectProperty = nullptr;
	const FClassProperty* ClassProperty = nullptr;
	const FObjectProperty* KeyObjectProperty = nullptr;
	const FClassProperty* KeyClassProperty = nullptr;
	bool bNeedSlowPath = false;
};
