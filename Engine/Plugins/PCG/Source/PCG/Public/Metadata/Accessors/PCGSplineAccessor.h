// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGSplineStruct.h"
#include "Metadata/Accessors/PCGPropertyAccessor.h"

#include "Components/SplineComponent.h"

/**
* Templated accessor for location/rotation/scale in world coordinates. It's important that the keys only have a single value, the struct that holds the spline data/struct, since
* there is a single transform per spline.
* Keys supported: PCGSplineData, FPCGSplineStruct
*/
template <typename T, EPCGControlPointsAccessorTarget Target, bool bWorldCoordinates>
class FPCGControlPointsAccessor : public IPCGAttributeAccessorT<FPCGControlPointsAccessor<T, Target, bWorldCoordinates>>, IPCGPropertyChain
{
public:
	// The underlying type is a quat if we target the rotation, otherwise a vector
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGControlPointsAccessor<Type, Target, bWorldCoordinates>>;
	
	FPCGControlPointsAccessor(const FStructProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ false)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
	{
		static_assert(PCG::Private::IsPCGType<Type>());		
		check(InProperty && InProperty->Struct->IsChildOf<FPCGSplineStruct>());
		TopPropertyStruct = GetTopPropertyStruct();
	}

	bool GetRangeImpl(TArrayView<Type> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		// We want to access the transform on the spline struct, there is just a single one.
		const void* ContainerKeys = nullptr;
		TArrayView<const void*> ContainerKeysView(&ContainerKeys, 1);
		if (!Keys.GetKeys(Index, ContainerKeysView))
		{
			return false;
		}

		// Validation to not access keys that are not the expected type. Done after the GetKeys, as we also want to discard other type of incompatible
		// keys (like a Default Metadata entry key)
		if (!ensure(Keys.IsClassSupported(TopPropertyStruct)))
		{
			return false;
		}

		PCGPropertyAccessor::AddressOffset(GetPropertyChain(), ContainerKeysView);

		const FPCGSplineStruct& SplineStruct = *static_cast<const FPCGSplineStruct*>(ContainerKeys);
		const FTransform SplineTransform = SplineStruct.GetTransform();

		const int32 NumPoints = SplineStruct.GetNumberOfPoints();
		if (NumPoints == 0)
		{
			return false;
		}
		
		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			const int32 CurrIndex = (Index + i) % NumPoints;
			
			if constexpr (Target == EPCGControlPointsAccessorTarget::Location)
			{
				static_assert(std::is_same_v<Type, FVector>);
				OutValues[i] = SplineStruct.GetLocation(CurrIndex);
				if constexpr (bWorldCoordinates)
				{
					OutValues[i] = SplineTransform.TransformPosition(OutValues[i]);
				}
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::Rotation)
			{
				static_assert(std::is_same_v<Type, FQuat>);
				OutValues[i] = SplineStruct.GetRotation(CurrIndex);
				if constexpr (bWorldCoordinates)
				{
					OutValues[i] = SplineTransform.TransformRotation(OutValues[i]);
				}
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::Scale)
			{
				
				static_assert(std::is_same_v<Type, FVector>);
				OutValues[i] = SplineStruct.GetScale(CurrIndex);
				if constexpr (bWorldCoordinates)
				{
					OutValues[i] = (FTransform(FQuat::Identity, FVector::ZeroVector, OutValues[i]) * SplineTransform).GetScale3D();
				}
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::Transform)
			{
				static_assert(std::is_same_v<Type, FTransform>);
				OutValues[i] = FTransform(SplineStruct.GetRotation(CurrIndex), SplineStruct.GetLocation(CurrIndex), SplineStruct.GetScale(CurrIndex));
				if constexpr (bWorldCoordinates)
				{
					OutValues[i] = OutValues[i] * SplineTransform;
				}
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::ArriveTangent)
			{
				static_assert(std::is_same_v<Type, FVector>);
				OutValues[i] = SplineStruct.GetArriveTangent(CurrIndex);
				if constexpr (bWorldCoordinates)
				{
					OutValues[i] = SplineTransform.TransformVector(OutValues[i]);
				}
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::LeaveTangent)
			{
				static_assert(std::is_same_v<Type, FVector>);
				OutValues[i] = SplineStruct.GetLeaveTangent(CurrIndex);
				if constexpr (bWorldCoordinates)
				{
					OutValues[i] = SplineTransform.TransformVector(OutValues[i]);
				}
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::InterpMode)
			{
				static_assert(std::is_same_v<Type, FPCGEnumPropertyAccessor::Type>);
				OutValues[i] = static_cast<Type>(ConvertInterpCurveModeToSplinePointType(SplineStruct.GetSplinePointType(CurrIndex)));
			}
			else
			{
				// Pitfall static assert
				static_assert(!std::is_same_v<Type, Type>);
			}
		}

		return true;
	}

	bool SetRangeImpl(TArrayView<const Type> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
	{		
		// We want to access the transform on the spline struct, there is just a single one.
		void* ContainerKeys = nullptr;
		TArrayView<void*> ContainerKeysView(&ContainerKeys, 1);
		if (!Keys.GetKeys(Index, ContainerKeysView))
		{
			return false;
		}

		// Validation to not access keys that are not the expected type. Done after the GetKeys, as we also want to discard other type of incompatible
		// keys (like a Default Metadata entry key)
		if (!ensure(Keys.IsClassSupported(TopPropertyStruct)))
		{
			return false;
		}

		PCGPropertyAccessor::AddressOffset(GetPropertyChain(), ContainerKeysView);

		FPCGSplineStruct& SplineStruct = *static_cast<FPCGSplineStruct*>(ContainerKeys);
		const FTransform InverseSplineTransform = SplineStruct.GetTransform().Inverse();
		
		const int32 NumPoints = SplineStruct.GetNumberOfPoints();
		if (NumPoints == 0)
		{
			return false;
		}
		
		for (int32 i = 0; i < InValues.Num(); ++i)
		{
			const int32 CurrIndex = (Index + i) % NumPoints;
			
			if constexpr (Target == EPCGControlPointsAccessorTarget::Location)
			{
				static_assert(std::is_same_v<Type, FVector>);
				FVector OutPosition = InValues[i];
				if constexpr (bWorldCoordinates)
				{
					OutPosition = InverseSplineTransform.TransformPosition(OutPosition);
				}
				SplineStruct.SetLocation(CurrIndex, OutPosition);
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::Rotation)
			{
				static_assert(std::is_same_v<Type, FQuat>);
				FQuat OutRotation = InValues[i];
				if constexpr (bWorldCoordinates)
				{
					OutRotation = InverseSplineTransform.TransformRotation(OutRotation);
				}
				SplineStruct.SetRotation(CurrIndex, OutRotation);
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::Scale)
			{
				static_assert(std::is_same_v<Type, FVector>);
				FVector OutScale = InValues[i];
				if constexpr (bWorldCoordinates)
				{
					OutScale = (FTransform(FQuat::Identity, FVector::ZeroVector, OutScale) * InverseSplineTransform).GetScale3D();
				}
				SplineStruct.SetScale(CurrIndex, OutScale);
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::Transform)
			{
				static_assert(std::is_same_v<Type, FTransform>);
				FTransform Transform = InValues[i];
				if constexpr (bWorldCoordinates)
				{
					Transform = Transform * InverseSplineTransform;
				}
				SplineStruct.SetLocation(CurrIndex, Transform.GetLocation());
				SplineStruct.SetRotation(CurrIndex, Transform.GetRotation());
				SplineStruct.SetScale(CurrIndex, Transform.GetScale3D());
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::ArriveTangent)
			{
				static_assert(std::is_same_v<Type, FVector>);
				FVector OutArriveTangent = InValues[i];
				if constexpr (bWorldCoordinates)
				{
					OutArriveTangent = InverseSplineTransform.TransformVector(OutArriveTangent);
				}
				SplineStruct.SetArriveTangent(CurrIndex, OutArriveTangent);
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::LeaveTangent)
			{
				static_assert(std::is_same_v<Type, FVector>);
				FVector OutLeaveTangent = InValues[i];
				if constexpr (bWorldCoordinates)
				{
					OutLeaveTangent = InverseSplineTransform.TransformVector(OutLeaveTangent);
				}
				SplineStruct.SetLeaveTangent(CurrIndex, OutLeaveTangent);
			}
			else if constexpr (Target == EPCGControlPointsAccessorTarget::InterpMode)
			{
				static_assert(std::is_same_v<Type, FPCGEnumPropertyAccessor::Type>);
				SplineStruct.SetSplinePointType(CurrIndex, ConvertSplinePointTypeToInterpCurveMode(static_cast<ESplinePointType::Type>(InValues[i])));
			}
			else
			{
				// Pitfall static assert
				static_assert(!std::is_same_v<Type, Type>);
			}
		}

		return true;
	}

private:
	const UStruct* TopPropertyStruct = nullptr;
};

/**
* Templated accessor for global spline data. Note that only the spline transform value is writable.
* Keys supported: PCGSplineData, FPCGSplineStruct
*/
template <typename T, EPCGSplineDataProperties Target>
class FPCGSplineAccessorV2 : public IPCGAttributeAccessorT<FPCGSplineAccessorV2<T, Target>>, IPCGPropertyChain
{
public:
	// The underlying type is a quat if we target the rotation, otherwise a vector
	using Type = T;
	using Super = IPCGAttributeAccessorT<FPCGSplineAccessorV2<T, Target>>;
	
	FPCGSplineAccessorV2(const FStructProperty* InProperty, TArray<const FProperty*>&& ExtraProperties = {})
		: Super(/*bInReadOnly=*/ Target != EPCGSplineDataProperties::SplineTransform)
		, IPCGPropertyChain(InProperty, std::move(ExtraProperties))
	{
		static_assert(PCG::Private::IsPCGType<Type>());		
		check(InProperty && InProperty->Struct->IsChildOf<FPCGSplineStruct>());
		TopPropertyStruct = GetTopPropertyStruct();
	}

	bool GetRangeImpl(TArrayView<Type> OutValues, int32 Index, const IPCGAttributeAccessorKeys& Keys) const
	{
		// We want to access the transform on the spline struct, there is just a single one.
		const void* ContainerKeys = nullptr;
		TArrayView<const void*> ContainerKeysView(&ContainerKeys, 1);
		if (!Keys.GetKeys(Index, ContainerKeysView))
		{
			return false;
		}

		// Validation to not access keys that are not the expected type. Done after the GetKeys, as we also want to discard other type of incompatible
		// keys (like a Default Metadata entry key)
		if (!ensure(Keys.IsClassSupported(TopPropertyStruct)))
		{
			return false;
		}

		PCGPropertyAccessor::AddressOffset(GetPropertyChain(), ContainerKeysView);

		const FPCGSplineStruct& SplineStruct = *static_cast<const FPCGSplineStruct*>(ContainerKeys);
		
		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			if constexpr (Target == EPCGSplineDataProperties::SplineTransform)
			{
				using SplineTransformType = decltype(SplineStruct.GetTransform());
				static_assert(std::is_same_v<Type, SplineTransformType>);
				OutValues[i] = SplineStruct.GetTransform();
			}
			else if constexpr (Target == EPCGSplineDataProperties::IsClosed)
			{
				using SplineIsClosedType = decltype(SplineStruct.IsClosedLoop());
				static_assert(std::is_same_v<Type, SplineIsClosedType>);
				OutValues[i] = SplineStruct.IsClosedLoop();
			}
			else if constexpr (Target == EPCGSplineDataProperties::Length)
			{
				using SplineLengthType = decltype(SplineStruct.GetSplineLength());
				static_assert(std::is_same_v<Type, SplineLengthType>);
				OutValues[i] = SplineStruct.GetSplineLength();
			}
			else
			{
				// Pitfall static assert
				static_assert(!std::is_same_v<Type, Type>);
			}
		}

		return true;
	}

	bool SetRangeImpl(TArrayView<const Type> InValues, int32 Index, IPCGAttributeAccessorKeys& Keys, EPCGAttributeAccessorFlags)
	{
		if constexpr (Target != EPCGSplineDataProperties::SplineTransform)
		{
			// Not supported, as we need to update the spline when we change it, which is not threadsafe.
			return false;
		}
		else
		{
			// We want to access the transform on the spline struct, there is just a single one.
			void* ContainerKeys = nullptr;
			TArrayView<void*> ContainerKeysView(&ContainerKeys, 1);
			if (!Keys.GetKeys(Index, ContainerKeysView))
			{
				return false;
			}

			// Validation to not access keys that are not the expected type. Done after the GetKeys, as we also want to discard other type of incompatible
			// keys (like a Default Metadata entry key)
			if (!ensure(Keys.IsClassSupported(TopPropertyStruct)))
			{
				return false;
			}

			PCGPropertyAccessor::AddressOffset(GetPropertyChain(), ContainerKeysView);

			FPCGSplineStruct& SplineStruct = *static_cast<FPCGSplineStruct*>(ContainerKeys);
			
			if constexpr (std::is_same_v<Type, decltype(SplineStruct.Transform)>)
			{
				for (int32 i = 0; i < InValues.Num(); ++i)
				{
					SplineStruct.Transform = InValues[i];
				}
			}
			else
			{
				// Pitfall static assert
				static_assert(!std::is_same_v<Type, Type>);
			}
			
			return true;
		}
	}

private:
	const UStruct* TopPropertyStruct = nullptr;
};

// DEPRECATION

enum class UE_DEPRECATED(5.8, "Not used anymore. Replaced by EPCGSplineDataProperties") EPCGSplineAccessorTarget
{
	Transform,
	ClosedLoop
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Consteval function to convert the old enum to the new one, to be used in a template (hence the consteval)
template <EPCGSplineAccessorTarget Target>
consteval EPCGSplineDataProperties DeprecationSplineAccessorTarget()
{
	if constexpr (Target == EPCGSplineAccessorTarget::Transform)
	{
		return EPCGSplineDataProperties::SplineTransform;
	}
	else if constexpr (Target == EPCGSplineAccessorTarget::ClosedLoop)
	{
		return EPCGSplineDataProperties::IsClosed;
	}
	else
	{
		// Pitfall assert
		static_assert(!std::is_same_v<decltype(Target), EPCGSplineAccessorTarget>);
		return {};
	}
}

template <typename T, EPCGSplineAccessorTarget Target>
class UE_DEPRECATED(5.8, "Use FPCGSplineAccessorV2") FPCGSplineAccessor : public FPCGSplineAccessorV2<T, DeprecationSplineAccessorTarget<Target>()>
{
	using FPCGSplineAccessorV2<T, DeprecationSplineAccessorTarget<Target>()>::FPCGSplineAccessorV2;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
