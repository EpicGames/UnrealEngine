// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGPoint.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "PCGAttributePropertySelector.generated.h"

#define UE_API PCG_API

UENUM()
enum class EPCGAttributePropertySelection
{
	Attribute,
	PointProperty UE_DEPRECATED(5.6, "Please use Property flag now") UMETA(Hidden, DeprecatedProperty, DeprecationMessage="Please use Property flag now."),
	ExtraProperty,
	Property,
};

UENUM()
enum class EPCGExtraProperties : uint8
{
	Index UMETA(PCG_PropertyReadOnly),
	NumElements UMETA(ToolTip = "Data domain property. Only supported on data that have elements.", PCG_PropertyReadOnly, PCG_MetadataDomain="Data")
};

class FArchiveCrc32;
struct FPCGCustomVersion;
class UPCGData;

#if WITH_EDITOR
// Friend for unit tests
class FPCGAttributePropertySelectorTests;
#endif // WITH_EDITOR

/**
* Blueprint class to allow to select an attribute or a property.
* It will handle the logic and can only be modified using the blueprint library defined below.
* Also has a custom detail view in the PCGEditor plugin.
* 
* Note: This class should not be used as is, but need to be referenced by either an "InputSelector" or an "OutputSelector" (defined below).
* The reason for that is to provide 2 different default values for input and output. Input will have the "@Last" default value (meaning last attribute written to)
* and the Output will have "@Source" default value (meaning, same thing as input).
*/
USTRUCT(BlueprintType, meta = (Hidden))
struct FPCGAttributePropertySelector
{
	GENERATED_BODY()

public:
	FPCGAttributePropertySelector() = default;
	virtual ~FPCGAttributePropertySelector() = default;

	UE_API bool operator==(const FPCGAttributePropertySelector& Other) const;
	UE_API bool IsSame(const FPCGAttributePropertySelector& Other, bool bIncludeExtraNames = true) const;

	UE_API bool Reset();
	UE_API bool ResetExtraNames();
	
	// Setters, return true if something changed.
	UE_API bool SetPointProperty(EPCGPointProperties InPointProperty, bool bResetExtraNames = true);
	UE_API bool SetExtraProperty(EPCGExtraProperties InExtraProperty, bool bResetExtraNames = true);
	UE_API bool SetAttributeName(FName InAttributeName, bool bResetExtraNames = true);
	UE_API bool SetPropertyName(FName InPropertyName, bool bResetExtraNames = true);
	UE_API bool SetDomainName(FName InDomainName, bool bResetExtraNames = true);

	// Getters
	EPCGAttributePropertySelection GetSelection() const { return Selection; }
	const TArray<FString>& GetExtraNames() const { return ExtraNames; }
	TArray<FString>& GetExtraNamesMutable() { return ExtraNames; }
	FName GetAttributeName() const { return AttributeName; }
	UE_API EPCGPointProperties GetPointProperty() const;
	EPCGExtraProperties GetExtraProperty() const { return ExtraProperty; }
	FName GetPropertyName() const { return PropertyName; }
	FName GetDomainName() const { return DomainName; }

	// Convenient function to know if it is a basic attribute (attribute and no extra names)
	UE_API bool IsBasicAttribute() const;

	// Return the name of the selector.
	UE_API FName GetName() const;

	// Returns qualified attribute/property name with the accessors.
	UE_API FString ToString(bool bSkipDomain = false) const;

	// Returns attribute/property name only, with optional '@' qualifier for domains.
	UE_API FString GetDomainString(bool bAddLeadingQualifier = true) const;
	
	// Returns attribute/property name only, with optional '$' qualifier for properties and '.'.
	UE_API FString GetAttributePropertyString(bool bAddPropertyQualifier = true) const;

	// Returns the accessor part of the selector, with optional leading '.' separator.
	UE_API FString GetAttributePropertyAccessorsString(bool bAddLeadingSeparator) const;

	// Returns the text to display in the widget.
	FText GetDisplayText(bool bSkipDomain = false) const { return FText::FromString(ToString(bSkipDomain)); }

	// Return true if the underlying name is valid.
	UE_API bool IsValid() const;

	// Update the selector with an incoming string.
	UE_API bool Update(const FStringView NewValue);

	// For deprecation of Update(const FString&)
	// @todo_pcg in another pass
	// UE_DEPRECATED(5.8, "Make sure to use the string view version")
	UE_API bool Update(const char* NewValue);

	template <typename T>
	static T CreateFromOtherSelector(const FPCGAttributePropertySelector& InOther)
	{
		static_assert(std::is_base_of_v<FPCGAttributePropertySelector, T>, "The type must be of base class 'FPCGAttributePropertySelector'");
		T OutSelector;
		OutSelector.ImportFromOtherSelector(InOther);
		return OutSelector;
	}

	// Convenience templated static constructors
	template <typename T = FPCGAttributePropertySelector>
	static T CreateAttributeSelector(const FName AttributeName, const FName DomainName = NAME_None, const TArrayView<const FString>& InExtraNames = {})
	{
		static_assert(std::is_base_of_v<FPCGAttributePropertySelector, T>, "The type must be of base class 'FPCGAttributePropertySelector'");
		T Selector;
		Selector.SetDomainName(DomainName);
		Selector.SetAttributeName(AttributeName);
		Selector.GetExtraNamesMutable() = InExtraNames;
		return Selector;
	}
	
	template <typename T = FPCGAttributePropertySelector>
	static T CreatePropertySelector(FName PropertyName, const FName DomainName = NAME_None, const TArrayView<const FString>& InExtraNames = {})
	{
		static_assert(std::is_base_of_v<FPCGAttributePropertySelector, T>, "The type must be of base class 'FPCGAttributePropertySelector'");
		T Selector;
		Selector.SetDomainName(DomainName);
		Selector.SetPropertyName(PropertyName);
		Selector.GetExtraNamesMutable() = InExtraNames;
		return Selector;
	}
	
	template <typename T = FPCGAttributePropertySelector>
	static T CreatePointPropertySelector(EPCGPointProperties PointProperty, const FName DomainName = NAME_None, const TArrayView<const FString>& InExtraNames = {})
	{
		static_assert(std::is_base_of_v<FPCGAttributePropertySelector, T>, "The type must be of base class 'FPCGAttributePropertySelector'");
		T Selector;
		Selector.SetDomainName(DomainName);
		Selector.SetPointProperty(PointProperty);
		Selector.GetExtraNamesMutable() = InExtraNames;
		return Selector;
	}

	template <typename T = FPCGAttributePropertySelector>
	static T CreateExtraPropertySelector(EPCGExtraProperties ExtraProperty, const FName DomainName = NAME_None, const TArrayView<const FString>& InExtraNames = {})
	{
		static_assert(std::is_base_of_v<FPCGAttributePropertySelector, T>, "The type must be of base class 'FPCGAttributePropertySelector'");
		T Selector;
		Selector.SetDomainName(DomainName);
		Selector.SetExtraProperty(ExtraProperty);
		Selector.GetExtraNamesMutable() = InExtraNames;
		return Selector;
	}

	template <typename T = FPCGAttributePropertySelector>
	static T CreateSelectorFromString(const FStringView String)
	{
		static_assert(std::is_base_of_v<FPCGAttributePropertySelector, T>, "The type must be of base class 'FPCGAttributePropertySelector'");
		T Selector;
		Selector.Update(String);
		return Selector;
	}

	// For deprecation purposes
	explicit FPCGAttributePropertySelector(const FName InName)
	{
		SetAttributeName(InName);
	}
	
	FPCGAttributePropertySelector& operator=(const FName InName)
	{
		SetAttributeName(InName);
		return *this;
	}

	explicit operator FName() const
	{
		return GetName();
	}

	UE_API void ImportFromOtherSelector(const FPCGAttributePropertySelector& InOther);

	UE_API virtual void AddToCrc(FArchiveCrc32& Ar) const;

	friend uint32 GetTypeHash(const FPCGAttributePropertySelector& Selector);

	UE_API bool ExportTextItem(FString& ValueStr, FPCGAttributePropertySelector const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	UE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

	UE_API bool Serialize(FArchive& Ar);
	UE_API void PostSerialize(const FArchive& Ar);

protected:
	UPROPERTY()
	EPCGAttributePropertySelection Selection = EPCGAttributePropertySelection::Attribute;

	UPROPERTY()
	FName DomainName = NAME_None;

	UPROPERTY()
	FName AttributeName = NAME_None;

	UPROPERTY()
	FName PropertyName = NAME_None;

	UPROPERTY()
	TArray<FString> ExtraNames;

	UPROPERTY()
	EPCGExtraProperties ExtraProperty = EPCGExtraProperties::Index;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	EPCGPointProperties PointProperty_DEPRECATED = EPCGPointProperties::Position;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	// To unit test deprecation
	friend FPCGAttributePropertySelectorTests;
#endif // WITH_EDITOR
};

/** Struct that will default on @Last (or @LastCreated for previously created selectors). */
USTRUCT(BlueprintType)
struct FPCGAttributePropertyInputSelector : public FPCGAttributePropertySelector
{
	GENERATED_BODY()

	UE_API FPCGAttributePropertyInputSelector();

	/** Get a copy of the selector, with @Last replaced by the right selector. */
	UE_API FPCGAttributePropertyInputSelector CopyAndFixLast(const UPCGData* InData) const;

	/** To support previously saved nodes, that used FPCGAttributePropertySelector, we need to define this function to de-serialize the new class using the old. And add a trait (see below). */
	UE_API bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/** For older nodes, before the split between Input and Output, force any last attribute to be last created to preserve the old behavior. Will be called by the PCGSetting deprecation function. Not meant to be used otherwise. */
	UE_API void ApplyDeprecation(int32 InPCGCustomVersion);
};

// Version where it doesn't make sense to have @Source, alias for FPCGAttributePropertySelector
USTRUCT(BlueprintType)
struct FPCGAttributePropertyOutputNoSourceSelector : public FPCGAttributePropertySelector
{
	GENERATED_BODY()

	/** To support previously saved nodes, that used FPCGAttributePropertySelector, we need to define this function to de-serialize the new class using the old. And add a trait (see below). */
	UE_API virtual bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

/** Struct that will default on @Source. */
USTRUCT(BlueprintType)
struct FPCGAttributePropertyOutputSelector : public FPCGAttributePropertyOutputNoSourceSelector
{
	GENERATED_BODY()

	UE_API FPCGAttributePropertyOutputSelector();

	/** Get a copy of the selector, with @Source replaced by the right selector. Can add extra data for specific deprecation cases. */
	UE_API FPCGAttributePropertyOutputSelector CopyAndFixSource(const FPCGAttributePropertyInputSelector* InSourceSelector, const UPCGData* InOptionalData = nullptr) const;
};


// All selectors have a custom Export/Import text to use the Update function to serialize/deserialize.
// It fixes errors of serialization when duplicating/copying.
template<>
struct TStructOpsTypeTraits<FPCGAttributePropertySelector> : public TStructOpsTypeTraitsBase2<FPCGAttributePropertySelector>
{
	enum
	{
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithPostSerialize = true,
		WithSerializer = true
	};
};

template<> struct TStructOpsTypeTraits<FPCGAttributePropertyOutputNoSourceSelector> : public TStructOpsTypeTraits<FPCGAttributePropertySelector>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true
	};
};

// Also extra trait to specify to the deserializer that those two classes can be deserialized using the old class.
template<> 
struct TStructOpsTypeTraits<FPCGAttributePropertyInputSelector> : public TStructOpsTypeTraits<FPCGAttributePropertySelector>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true
	};
};

template<>
struct TStructOpsTypeTraits<FPCGAttributePropertyOutputSelector> : public TStructOpsTypeTraits<FPCGAttributePropertyOutputNoSourceSelector> {};

/**
* Helper class to allow the BP to call the custom setters and getters on FPCGAttributePropertySelector.
*/
UCLASS()
class UPCGAttributePropertySelectorBlueprintHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static bool SetPointProperty(UPARAM(ref) FPCGAttributePropertySelector& Selector, EPCGPointProperties InPointProperty, bool bResetExtraNames = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static bool SetAttributeName(UPARAM(ref) FPCGAttributePropertySelector& Selector, FName InAttributeName, bool bResetExtraNames = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static bool SetPropertyName(UPARAM(ref) FPCGAttributePropertySelector& Selector, FName InPropertyName, bool bResetExtraNames = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static bool SetExtraProperty(UPARAM(ref) FPCGAttributePropertySelector& Selector, EPCGExtraProperties InExtraProperty, bool bResetExtraNames = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static bool SetDomainName(UPARAM(ref) FPCGAttributePropertySelector& Selector, FName InDomainName, bool bResetExtraNames = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static EPCGAttributePropertySelection GetSelection(UPARAM(const, ref) const FPCGAttributePropertySelector& Selector);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static EPCGPointProperties GetPointProperty(UPARAM(const, ref) const FPCGAttributePropertySelector& Selector);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static FName GetAttributeName(UPARAM(const, ref) const FPCGAttributePropertySelector& Selector);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static FName GetPropertyName(UPARAM(const, ref) const FPCGAttributePropertySelector& Selector);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static FName GetDomainName(UPARAM(const, ref) const FPCGAttributePropertySelector& Selector);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static EPCGExtraProperties GetExtraProperty(UPARAM(const, ref) const FPCGAttributePropertySelector& Selector);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static const TArray<FString>& GetExtraNames(UPARAM(const, ref) const FPCGAttributePropertySelector& Selector);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static FName GetName(UPARAM(const, ref) const FPCGAttributePropertySelector& Selector);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static FPCGAttributePropertyInputSelector CopyAndFixLast(UPARAM(const, ref) const FPCGAttributePropertyInputSelector& Selector, const UPCGData* InData);

	UFUNCTION(BlueprintCallable, Category = "PCG|PCGAttributePropertySelector", meta = (ScriptMethod))
	static FPCGAttributePropertyOutputSelector CopyAndFixSource(UPARAM(const, ref) const FPCGAttributePropertyOutputSelector& OutputSelector, const FPCGAttributePropertyInputSelector& InputSelector, const UPCGData* InOptionalData = nullptr);
};

#undef UE_API
