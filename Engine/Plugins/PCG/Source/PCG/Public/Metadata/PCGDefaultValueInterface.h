// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTraits.h"

#include "UObject/Interface.h"

#include "PCGDefaultValueInterface.generated.h"

UENUM()
enum class EPCGSettingDefaultValueExtraFlags : uint64
{
	None          = 0,
	WideText      = 1 << 0, // The text box will need to be large enough to support a long text block.
	MultiLineText = 1 << 1  // The text block may span multiple lines.
};

ENUM_CLASS_FLAGS(EPCGSettingDefaultValueExtraFlags);

USTRUCT()
struct FPCGPinDefaultValueInfo
{
	GENERATED_BODY()

#if WITH_EDITOR
	struct FInitParams
	{
		bool bStartActivated = true;
		EPCGMetadataTypes InitialType = EPCGMetadataTypes::Double;
		FString InitialValue = PCG::Private::MetadataTraits<double>::ZeroValueString();
		TArray<EPCGMetadataTypes> AvailableTypes;
	};

	FPCGPinDefaultValueInfo() = default;

	explicit FPCGPinDefaultValueInfo(FInitParams InParams)
		: CurrentType(InParams.InitialType)
		, AvailableTypes(MoveTemp(InParams.AvailableTypes))
		, InitialType(InParams.InitialType)
		, InitialValue(MoveTemp(InParams.InitialValue))
	{}
#endif // WITH_EDITOR

	UPROPERTY()
	EPCGMetadataTypes CurrentType = EPCGMetadataTypes::Unknown;

#if WITH_EDITORONLY_DATA
	/** Metadata types this pin accepts. Empty = all types are supported. */
	UPROPERTY()
	TArray<EPCGMetadataTypes> AvailableTypes;

	UPROPERTY()
	EPCGMetadataTypes InitialType = EPCGMetadataTypes::Unknown;

	UPROPERTY()
	FString InitialValue;

	UPROPERTY()
	EPCGSettingDefaultValueExtraFlags ExtraFlags = EPCGSettingDefaultValueExtraFlags::None;
#endif // WITH_EDITORONLY_DATA
};

UINTERFACE(MinimalAPI)
class UPCGSettingsDefaultValueProvider : public UInterface
{
	GENERATED_BODY()
};

class IPCGSettingsDefaultValueProvider
{
	GENERATED_BODY()

public:
	/** One or more pins on this node has a 'default value' and can be adjusted via an inline constant. */
	virtual bool DefaultValuesAreEnabled() const { return false; }

	/** The specified pin can accommodate 'default value' inline constants. */
	virtual bool IsPinDefaultValueEnabled(FName PinLabel) const { return false; }

	/** The specified pin has a 'default value' currently activated. */
	virtual bool IsPinDefaultValueActivated(FName PinLabel) const { return false; }

	/** Gets the current 'default value' type, if supported, for the pin. */
	virtual EPCGMetadataTypes GetPinDefaultValueType(FName PinLabel) const { return EPCGMetadataTypes::Unknown; }

	/** Add an attribute to a given metadata for the initial default value. In most cases, this will be the Zero Value. */
	virtual bool CreateInitialDefaultValueAttribute(FName PinLabel, UPCGMetadata* OutMetadata) const
	{
		EPCGMetadataTypes AttributeType = GetPinDefaultValueType(PinLabel);
		return OutMetadata && PCGMetadataAttribute::CallbackWithRightType(static_cast<uint16>(AttributeType), [OutMetadata]<typename T>(T) -> bool
		{
			T Value = PCG::Private::MetadataTraits<T>::ZeroValue();
			return nullptr != OutMetadata->CreateAttribute(NAME_None, Value, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/false);
		});
	}

#if WITH_EDITOR
	/** Returns true if the pin supports the provided metadata type. */
	virtual bool IsPinDefaultValueMetadataTypeValid(FName PinLabel, EPCGMetadataTypes DataType) const { return false; }

	/** Resets all default values to their 'reset' value and deactivates them. */
	virtual void ResetDefaultValues() {}

	/** Resets the pin's default value to the initial value. */
	virtual void ResetDefaultValue(FName PinLabel) {}

	/** Sets the pin's default value string directly. */
	virtual void SetPinDefaultValue(FName PinLabel, const FString& DefaultValue, bool bCreateIfNeeded = false) {}

	/** Attempts a metadata type conversion of the pin's default value inline constant. */
	virtual void ConvertPinDefaultValueMetadataType(FName PinLabel, EPCGMetadataTypes DataType) {}

	/** Whether the default value can be toggled on/off by the user. */
	virtual bool CanDefaultValueBeToggled() const { return true; }

	/** Sets the default value to active. Must be overridden by the subclass. */
	virtual void SetPinDefaultValueIsActivated(FName PinLabel, bool bIsActivated, bool bDirtySettings = true) PURE_VIRTUAL(IPCGSettingsDefaultValueProvider::SetPinDefaultValueIsActivated, return;);

	/** Gets the 'default value', if supported, for the pin. */
	virtual FString GetPinDefaultValueAsString(FName PinLabel) const { return FString(); }

	/** For the initial 'default value' of the pin. */
	virtual FString GetPinInitialDefaultValueString(FName PinLabel) const { return FString(); }

	/** Extra flags related to displaying the value. */
	virtual EPCGSettingDefaultValueExtraFlags GetDefaultValueExtraFlags(FName PinLabel) const { return EPCGSettingDefaultValueExtraFlags::None; }
#endif // WITH_EDITOR

protected:
	// @todo_pcg: GetPinInitialDefaultValueType should likely be Editor Only
	/** For the initial 'default value' type of the pin. */
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const { return EPCGMetadataTypes::Unknown; }
};
