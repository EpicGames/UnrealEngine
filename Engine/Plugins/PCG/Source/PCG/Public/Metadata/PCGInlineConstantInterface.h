// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/PCGDefaultValueContainer.h"
#include "Metadata/PCGDefaultValueInterface.h"

#include "PCGInlineConstantInterface.generated.h"

class UPCGParamData;
struct FPCGContext;

/**
 * Composable state struct for inline constant values on pins.
 * Any UPCGSettings class implementing IPCGSettingsInlineConstant can embed this as a UPROPERTY member and override
 * GetInlineConstantState()/GetMutableInlineConstantState() to get full inline constant support with minimal boilerplate.
 */
USTRUCT()
struct FPCGInlineConstantState
{
	GENERATED_BODY()

	/** One or more pins on this node has a 'default value' and can be adjusted via an inline constant. */
	PCG_API bool DefaultValuesAreEnabled() const;

	/** The specified pin can accommodate 'default value' inline constants. */
	PCG_API bool IsPinDefaultValueEnabled(FName PinLabel) const;

	/** The specified pin has a 'default value' currently activated. */
	PCG_API bool IsPinDefaultValueActivated(FName PinLabel) const;

	/** Get the current 'default value' type, if supported, for the pin. */
	PCG_API EPCGMetadataTypes GetPinDefaultValueType(FName PinLabel) const;

	/** Create a Param Data with the inline constant default value properties inserted as metadata. */
	PCG_API const UPCGParamData* CreateDefaultValueParamData(FPCGContext* InContext, FName PropertyKey, FName AttributeName = NAME_None) const;

#if WITH_EDITOR
	/** Returns true if the pin supports the provided metadata type. */
	PCG_API bool IsPinDefaultValueMetadataTypeValid(FName PinLabel, EPCGMetadataTypes DataType) const;

	/** Returns true if the pin has been registered for default values. */
	PCG_API bool PinIsMapped(FName PinLabel) const;

	/** For the initial 'default value' type of the pin. */
	PCG_API EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const;

	/** Get the 'default value', if supported, for the pin. */
	PCG_API FString GetPinDefaultValueAsString(FName PinLabel) const;

	/** Get the initial 'default value' of the pin. */
	PCG_API FString GetPinInitialDefaultValueString(FName PinLabel) const;

	/** Extra flags related to displaying the value. */
	PCG_API EPCGSettingDefaultValueExtraFlags GetDefaultValueExtraFlags(FName PinLabel) const;

	/** Register a pin for inline constant support. Returns false if already registered. */
	PCG_API bool AddDefaultValueToPin(FName PinLabel, FPCGPinDefaultValueInfo::FInitParams InitParams = FPCGPinDefaultValueInfo::FInitParams{});

	/** Unregister a pin from inline constant support. Returns false if not registered. */
	PCG_API bool RemoveDefaultValueFromPin(FName PinLabel);

	/** Reset all values to their initial type/value. Returns true if any reset occurred. */
	PCG_API bool ResetDefaultValues();

	/** Reset a single pin's value. Returns true if reset occurred. */
	PCG_API bool ResetDefaultValue(FName PinLabel);

	/** Set the pin's default value string directly. Returns true if value changed. */
	PCG_API bool SetPinDefaultValue(FName PinLabel, const FString& DefaultValue, bool bCreateIfNeeded = false);

	/** Attempt a metadata type conversion. Returns true if conversion occurred. */
	PCG_API bool ConvertPinDefaultValueMetadataType(FName PinLabel, EPCGMetadataTypes DataType);

	/** Set the default value to active/inactive. Returns true if activation state changed. */
	PCG_API bool SetPinDefaultValueIsActivated(FName PinLabel, bool bIsActivated);
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	void ResetDefaultValueInternal(FName PinLabel);
#endif // WITH_EDITOR

	UPROPERTY()
	TMap<FName, FPCGPinDefaultValueInfo> DefaultValueMap;

	UPROPERTY()
	FPCGDefaultValueContainer DefaultValues;
};

/**
 * Interface for settings that support inline constants via FPCGInlineConstantState.
 * Inherits from IPCGSettingsDefaultValueProvider and overrides all methods with boilerplate implementations that
 * delegate to the composable state struct.
 *
 * Inheriting this interface:
 *   1. Embed an FPCGInlineConstantState as a UPROPERTY
 *   2. Override GetInlineConstantState() / GetMutableInlineConstantState() to return it
 *   3. Override GetPinInitialDefaultValueType() to specify eligible pins and their initial types
 *
 * Note: Do NOT inherit IPCGSettingsDefaultValueProvider manually alongside this interface.
 */
UINTERFACE(MinimalAPI)
class UPCGSettingsInlineConstant : public UPCGSettingsDefaultValueProvider
{
	GENERATED_BODY()
};

class IPCGSettingsInlineConstant : public IPCGSettingsDefaultValueProvider
{
	GENERATED_BODY()

public:
	/** Creates a Param Data with the inline constant default value properties inserted as metadata. */
	PCG_API const UPCGParamData* CreateDefaultValueParamData(FPCGContext* InContext, FName PropertyKey, FName AttributeName = NAME_None) const;

	// ~Begin IPCGSettingsDefaultValueProvider interface
	PCG_API virtual bool DefaultValuesAreEnabled() const override;
	PCG_API virtual bool IsPinDefaultValueEnabled(FName PinLabel) const override;
	PCG_API virtual bool IsPinDefaultValueActivated(FName PinLabel) const override;
	PCG_API virtual EPCGMetadataTypes GetPinDefaultValueType(FName PinLabel) const override;

#if WITH_EDITOR
	PCG_API virtual bool IsPinDefaultValueMetadataTypeValid(FName PinLabel, EPCGMetadataTypes DataType) const override;
	PCG_API virtual void ResetDefaultValues() override;
	PCG_API virtual void ResetDefaultValue(FName PinLabel) override;
	PCG_API virtual void SetPinDefaultValue(FName PinLabel, const FString& DefaultValue, bool bCreateIfNeeded = false) override;
	PCG_API virtual void ConvertPinDefaultValueMetadataType(FName PinLabel, EPCGMetadataTypes DataType) override;
	PCG_API virtual void SetPinDefaultValueIsActivated(FName PinLabel, bool bIsActivated, bool bDirtySettings = true) override;
	PCG_API virtual FString GetPinDefaultValueAsString(FName PinLabel) const override;
	PCG_API virtual FString GetPinInitialDefaultValueString(FName PinLabel) const override;
	PCG_API virtual EPCGSettingDefaultValueExtraFlags GetDefaultValueExtraFlags(FName PinLabel) const override;
#endif // WITH_EDITOR
	// ~End IPCGSettingsDefaultValueProvider interface

protected:
	/** Return the embedded const FPCGInlineConstantState. Must be overridden by subclasses. */
	virtual const FPCGInlineConstantState* GetInlineConstantState() const = 0;

	/** Return the embedded mutable FPCGInlineConstantState. Must be overridden by subclasses. */
	virtual FPCGInlineConstantState* GetMutableInlineConstantState() = 0;
};
