// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGMetadataAttributeTraits.h"
#include "PCGMetadataCommon.h" // IWYU pragma: keep
#include "Metadata/PCGMetadataAttributeTpl.h"
#include "Metadata/PCGMetadataDomain.h"

#include "Misc/SpinLock.h"

#include "PCGMetadata.generated.h"

#define UE_API PCG_API

struct FPCGPoint;
struct FPCGContext;

// @todo_pcg: Support blueprint
struct FPCGMetadataInitializeParams
{
	explicit FPCGMetadataInitializeParams(const UPCGMetadata* InParent)
		: Parent(InParent)
	{}

	// Will create a new DomainInitializeParams with the Default metadata domain, to ease deprecation.
	PCG_API explicit FPCGMetadataInitializeParams(const UPCGMetadata* InParent, const TArray<PCGMetadataEntryKey>* InOptionalEntriesToCopy);

	// Will create a new DomainInitializeParams with the Default metadata domain, to ease deprecation.
	PCG_API explicit FPCGMetadataInitializeParams(const UPCGMetadata* InParent,
		const TSet<FName>& InFilteredAttributes,
		EPCGMetadataFilterMode InFilterMode = EPCGMetadataFilterMode::ExcludeAttributes,
		EPCGStringMatchingOperator InMatchOperator = EPCGStringMatchingOperator::Equal,
		const TArray<PCGMetadataEntryKey>* InOptionalEntriesToCopy = nullptr);

	PCG_API void PopulateDomainInitializeParamsFromParent();

	/** Parent to initialize from */
	const UPCGMetadata* Parent = nullptr;

	/** Optional mapping for cross domain. In an array since we'll probably never have a lot of them. */
	TArray<TTuple<FPCGMetadataDomainID, FPCGMetadataDomainID>> DomainMapping;

	/**
	 * Optional mapping for each domain initialization. In an array since we'll probably never have a lot of them.
	 * If not set explicitly, it will use the default FPCGMetadataDomainInitializeParams, with the right parent.
	 */
	TArray<TTuple<FPCGMetadataDomainID, FPCGMetadataDomainInitializeParams>> DomainInitializeParams;
};

UCLASS(MinimalAPI, BlueprintType)
class UPCGMetadata : public UObject
{
	friend FPCGMetadataDomain;
	friend FPCGMetadataInitializeParams;
	
	GENERATED_BODY()

public:
	UE_API UPCGMetadata(const FObjectInitializer& ObjectInitializer);
	
	//~ Begin UObject interface
	UE_API virtual void Serialize(FArchive& InArchive) override;
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject interface

	/** To be called by the outer pcg data to initialize the domains supported for this data. Can mark this one as default, will override previously set default. */
	UE_API void SetupDomain(FPCGMetadataDomainID DomainID, bool bIsDefault);

	/** If we have floating metadata , setup it to match a given data type. */
	template <typename DataType, std::enable_if_t<std::is_base_of_v<UPCGData, DataType>, int> = 0>
	void SetupDomainsFromPCGDataType() { SetupDomainsFromPCGDataType(DataType::StaticClass()); }

	UE_API void SetupDomainsFromPCGDataType(const TSubclassOf<UPCGData>& PCGDataType);

	/** Initializes the metadata from a parent metadata, if any (can be null). Copies attributes and values. */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void Initialize(const UPCGMetadata* InParent);

	/** Initializes the metadata from a parent metadata, if any (can be null) with the option to not add attributes from the parent. */
	UE_API void Initialize(const UPCGMetadata* InParent, bool bAddAttributesFromParent);

	UE_API void Initialize(const FPCGMetadataInitializeParams& InParams);

	/**
	 * Initializes the metadata from a parent metadata. Copies attributes and values.
	 * @param InParent              The parent metadata to use as a template, if any (can be null).
	 * @param InFilteredAttributes  Optional list of attributes to exclude or include when adding the attributes from the parent.
	 * @param InFilterMode          Defines attribute filter operation.
	 * @param InMatchOperator       How to match the names for the filtering
	 */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void InitializeWithAttributeFilter(const UPCGMetadata* InParent, const TSet<FName>& InFilteredAttributes, EPCGMetadataFilterMode InFilterMode = EPCGMetadataFilterMode::ExcludeAttributes, EPCGStringMatchingOperator InMatchOperator = EPCGStringMatchingOperator::Equal);

	UE_DEPRECATED(5.6, "Use the FPCGMetadataInitializeParams version")
	UE_API void InitializeAsCopy(const UPCGMetadata* InMetadataToCopy, const TArray<PCGMetadataEntryKey>* InOptionalEntriesToCopy = nullptr);

	/** Initializes the metadata from a parent metadata by copying all attributes to it. */
	UE_API void InitializeAsCopy(const FPCGMetadataInitializeParams& InParams);

	UE_DEPRECATED(5.6, "Use InitializeAsCopy with FPCGMetadataInitializeParams version")
	UE_API void InitializeAsCopyWithAttributeFilter(const UPCGMetadata* InMetadataToCopy, const TSet<FName>& InFilteredAttributes, EPCGMetadataFilterMode InFilterMode = EPCGMetadataFilterMode::ExcludeAttributes, const TArray<PCGMetadataEntryKey>* InOptionalEntriesToCopy = nullptr, EPCGStringMatchingOperator InMatchOperator = EPCGStringMatchingOperator::Equal);	

	/** Creates missing attributes from another metadata if they are not currently present - note that this does not copy values */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void AddAttributes(const UPCGMetadata* InOther);

	/**
	 * Creates missing attributes from another metadata if they are not currently present - note that this does not copy values.
	 * @param InOther               The other metadata to obtain a list of attributes from.
	 * @param InFilteredAttributes  Optional list of attributes to exclude or include when adding the attributes.
	 * @param InFilterMode          Defines attribute filter operation.
	 * @param InMatchOperator       How to match the names for the filtering
	 */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void AddAttributesFiltered(const UPCGMetadata* InOther, const TSet<FName>& InFilteredAttributes, EPCGMetadataFilterMode InFilterMode = EPCGMetadataFilterMode::ExcludeAttributes, EPCGStringMatchingOperator InMatchOperator = EPCGStringMatchingOperator::Equal);

	UE_API void AddAttributes(const FPCGMetadataInitializeParams& InParams);
	
	/** Creates missing attribute from another metadata if it is not currently present - note that this does not copy values */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (DisplayName = "Add Attribute"))
	UE_API void BP_AddAttribute(const UPCGMetadata* InOther, FName AttributeName);
	
	UE_API void AddAttribute(const UPCGMetadata* InOther, FPCGAttributeIdentifier AttributeName);

	/** Copies attributes from another metadata, including entries & values. Warning: this is intended when dealing with the same data set */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata|Advanced")
	UE_API void CopyAttributes(const UPCGMetadata* InOther);

	/** Copies an attribute from another metadata, including entries & values. Warning: this is intended when dealing with the same data set */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata|Advanced", meta = (DisplayName = "Copy Attribute"))
	UE_API void BP_CopyAttribute(const UPCGMetadata* InOther, FName AttributeToCopy, FName NewAttributeName);

	UE_API void CopyAttribute(const UPCGMetadata* InOther, FPCGAttributeIdentifier AttributeToCopy, FName NewAttributeName);
	
	/** Copies another attribute, with options to keep its parent and copy entries/values */
	UE_API FPCGMetadataAttributeBase* CopyAttribute(const FPCGMetadataAttributeBase* OriginalAttribute, FPCGAttributeIdentifier NewAttributeName, bool bKeepParent, bool bCopyEntries, bool bCopyValues);

	/** Returns this metadata's parent */
	TWeakObjectPtr<const UPCGMetadata> GetParentPtr() const { return Parent; }
	const UPCGMetadata* GetParent() const { return Parent.Get(); }
	UE_API const UPCGMetadata* GetRoot() const;
	UE_API bool HasParent(const UPCGMetadata* InTentativeParent) const;

	/** Unparents current metadata by flattening the attributes (values, entries, etc.) and potentially compress the data to remove unused values. */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata|Advanced")
	UE_API void Flatten();

	/** Unparents current metadata by flattening the attributes (values, entries, etc.) */
	UE_API void FlattenImpl();

	UE_DEPRECATED(5.6, "Use the version with the mapping")
	UE_API bool FlattenAndCompress(const TArray<PCGMetadataEntryKey>& InEntryKeysToKeep);

	/** Unparents current metadata, flatten attribute and only keep the entries specified. Return true if something has changed and keys needs be updated. */
	UE_API bool FlattenAndCompress(const TMap<FPCGMetadataDomainID, TArrayView<const PCGMetadataEntryKey>>& InEntryKeysToKeepMapping);
	
	// TODO: Will be moved at the end of the class before commit, as they are functions planned to be deprecated.
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateInteger32Attribute(FName AttributeName, int32 DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateInteger64Attribute(FName AttributeName, int64 DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateFloatAttribute(FName AttributeName, float DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateDoubleAttribute(FName AttributeName, double DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateVectorAttribute(FName AttributeName, FVector DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta=(DisplayName = "Create Vector4 Attribute"))
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateVector4Attribute(FName AttributeName, FVector4 DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateVector2Attribute(FName AttributeName, FVector2D DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateRotatorAttribute(FName AttributeName, FRotator DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateQuatAttribute(FName AttributeName, FQuat DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateTransformAttribute(FName AttributeName, FTransform DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateStringAttribute(FName AttributeName, FString DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateNameAttribute(FName AttributeName, FName DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateBoolAttribute(FName AttributeName, bool DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateSoftObjectPathAttribute(FName AttributeName, const FSoftObjectPath& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API UPARAM(DisplayName = "Metadata") UPCGMetadata* CreateSoftClassPathAttribute(FName AttributeName, const FSoftClassPath& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent = true);

	/** Creates an attribute given a property.
	* @param AttributeName: Target attribute to create
	* @param Object: Object to get the property value from
	* @param Property: The property to set from
	* @returns true if the attribute creation succeeded
	*/
	UE_API bool CreateAttributeFromProperty(FPCGAttributeIdentifier AttributeName, const UObject* Object, const FProperty* Property);

	/** Creates an attribute given a property.
	* @param AttributeName: Target attribute to create
	* @param Data: Data pointer to get the property value from
	* @param Property: The property to set from
	* @returns true if the attribute creation succeeded
	*/
	UE_API bool CreateAttributeFromDataProperty(FPCGAttributeIdentifier AttributeName, const void* Data, const FProperty* Property);

	/** Set an attribute given a property and its value.
	* @param AttributeName: Target attribute to set the property's value to
	* @param EntryKey: Metadata entry key to set the value to
	* @param Object: Object to get the property value from
	* @param Property: The property to set from
	* @param bCreate: If true and the attribute doesn't exists, it will create an attribute based on the property type
	* @returns true if the attribute creation (if required) and the value set succeeded
	*/
	UE_API bool SetAttributeFromProperty(FPCGAttributeIdentifier AttributeName, PCGMetadataEntryKey& EntryKey, const UObject* Object, const FProperty* Property, bool bCreate);

	/** Set an attribute given a property and its value.
	* @param AttributeName: Target attribute to set the property's value to
	* @param EntryKey: Metadata entry key to set the value to
	* @param Data: Data pointer to get the property value from
	* @param Property: The property to set from
	* @param bCreate: If true and the attribute doesn't exists, it will create an attribute based on the property type
	* @returns true if the attribute creation (if required) and the value set succeeded
	*/
	UE_API bool SetAttributeFromDataProperty(FPCGAttributeIdentifier AttributeName, PCGMetadataEntryKey& EntryKey, const void* Data, const FProperty* Property, bool bCreate);
	// End of move

	/** Get attributes */
	UE_API FPCGMetadataAttributeBase* GetMutableAttribute(FPCGAttributeIdentifier AttributeName);
	UE_API const FPCGMetadataAttributeBase* GetConstAttribute(FPCGAttributeIdentifier AttributeName) const;

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (DisplayName = "Has Attribute"))
	UE_API bool BP_HasAttribute(FName AttributeName) const;
	UE_API bool HasAttribute(FPCGAttributeIdentifier AttributeName) const;

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API bool HasCommonAttributes(const UPCGMetadata* InMetadata) const;

	/** Return the number of attributes in this metadata. */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API int32 GetAttributeCount() const;

	template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
	FPCGMetadataAttribute<T>* GetMutableTypedAttribute(FPCGAttributeIdentifier AttributeName);

	template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
	UE_DEPRECATED(5.8, "Should not be used anymore.")
	FPCGMetadataAttribute<T>* GetMutableTypedAttribute_Unsafe(FPCGAttributeIdentifier AttributeName);

	template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
	const FPCGMetadataAttribute<T>* GetConstTypedAttribute(FPCGAttributeIdentifier AttributeName) const;

	/** Get all the attributes names and type on the default domain. If you need all the attributes on all domains, use GetAllAttributes. */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void GetAttributes(TArray<FName>& AttributeNames, TArray<EPCGMetadataTypes>& AttributeTypes) const;

	/** Get all the attributes identifiers and their type for all domains. */
	UE_API void GetAllAttributes(TArray<FPCGAttributeIdentifier>& AttributeNames, TArray<EPCGMetadataTypes>& AttributeTypes) const;

	/** Returns name of the most recently created attribute, or none if no attributes are present. */
	UE_API FName GetLatestAttributeNameOrNone() const;

	/** Delete/Hide attribute */
	// Due to stream inheriting, we might want to consider "hiding" parent stream and deleting local streams only
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (DisplayName = "DeleteAttribute"))
	UE_API void BP_DeleteAttribute(FName AttributeName);

	UE_API void DeleteAttribute(FPCGAttributeIdentifier AttributeName);
	
	/** Copy attribute */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (DisplayName = "CopyExistingAttribute"))
	UE_API bool BP_CopyExistingAttribute(FName AttributeToCopy, FName NewAttributeName, bool bKeepParent = true);
	
	UE_API bool CopyExistingAttribute(FPCGAttributeIdentifier AttributeToCopy, FName NewAttributeName, bool bKeepParent = true);

	/** Rename attribute */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (DisplayName = "RenameAttribute"))
	UE_API bool BP_RenameAttribute(FName AttributeToRename, FName NewAttributeName);

	UE_API bool RenameAttribute(FPCGAttributeIdentifier AttributeToRename, FName NewAttributeName);
	
	/** Clear/Reinit attribute */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (DisplayName = "ClearAttributeW"))
	UE_API void BP_ClearAttribute(FName AttributeToClear);
	
	UE_API void ClearAttribute(FPCGAttributeIdentifier AttributeToClear);
	
	/** Change type of an attribute */
	UE_API bool ChangeAttributeType(FPCGAttributeIdentifier AttributeName, const FPCGMetadataAttributeDesc& AttributeNewDesc);
	
	UE_DEPRECATED(5.8, "Use the version with the Attribute Desc")
	UE_API bool ChangeAttributeType(FPCGAttributeIdentifier AttributeName, int16 AttributeNewType);

	/** Adds a unique entry key to the metadata */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API int64 AddEntry(int64 ParentEntryKey = -1);

	/** Adds a unique entry key to the metadata for all the parent entry keys. */
	UE_API TArray<int64> AddEntries(TArrayView<const int64> ParentEntryKeys);

	/** Adds a unique entry key to the metadata for all the parent entry keys, in place. */
	UE_API void AddEntriesInPlace(TArrayView<int64*> ParentEntryKeys);

	/** Advanced method.
	*   In a MT context, we might not want to add the entry directly (because of write lock). Call this to generate an unique index in the MT context
	*   And call AddDelayedEntries at the end when you want to add all the entries.
	*   Make sure to not call AddEntry in a different thread or even in the same thread if you use this function, or it will change all the indexes.
	*/
	UE_API int64 AddEntryPlaceholder();

	/** Advanced method.
	*   If you used AddEntryPlaceholder, call this function at the end of your MT processing to add all the entries in one shot.
	*   Make sure to call this one at the end!!
	*   @param AllEntries Array of pairs. First item is the EntryIndex generated by AddEntryPlaceholder, the second the ParentEntryKey (cf AddEntry)
	*/
	UE_API void AddDelayedEntries(const TArray<TTuple<int64, int64>>& AllEntries);

	/** Initializes the metadata entry key. Returns true if key set from either parent */
	UE_API bool InitializeOnSet(PCGMetadataEntryKey& InOutKey, PCGMetadataEntryKey InParentKeyA = PCGInvalidEntryKey, const UPCGMetadata* InParentMetadataA = nullptr, PCGMetadataEntryKey InParentKeyB = PCGInvalidEntryKey, const UPCGMetadata* InParentMetadataB = nullptr);

	/** Metadata chaining mechanism */
	UE_API PCGMetadataEntryKey GetParentKey(PCGMetadataEntryKey LocalItemKey) const;

	/** Metadata chaining mechanism for bulk version. Can provide a mask to only update only a subset of the passed keys. */
	UE_API void GetParentKeys(TArrayView<PCGMetadataEntryKey> LocalItemKeys, const TBitArray<>* Mask = nullptr) const;

	/** Attributes operations */
	UE_API void MergeAttributes(PCGMetadataEntryKey InKeyA, const UPCGMetadata* InMetadataA, PCGMetadataEntryKey InKeyB, const UPCGMetadata* InMetadataB, PCGMetadataEntryKey& OutKey, EPCGMetadataOp Op);
	UE_API void MergeAttributesSubset(PCGMetadataEntryKey InKeyA, const UPCGMetadata* InMetadataA, const UPCGMetadata* InMetadataSubetA, PCGMetadataEntryKey InKeyB, const UPCGMetadata* InMetadataB, const UPCGMetadata* InMetadataSubsetb, PCGMetadataEntryKey& OutKey, EPCGMetadataOp Op);

	UE_API void ResetWeightedAttributes(PCGMetadataEntryKey& OutKey);
	UE_API void AccumulateWeightedAttributes(PCGMetadataEntryKey InKey, const UPCGMetadata* InMetadata, float Weight, bool bSetNonInterpolableAttributes, PCGMetadataEntryKey& OutKey);

	UE_API void SetAttributes(PCGMetadataEntryKey InKey, const UPCGMetadata* InMetadata, PCGMetadataEntryKey& OutKey);
	UE_API void SetAttributes(const TArrayView<const PCGMetadataEntryKey>& InOriginalKeys, const UPCGMetadata* InMetadata, const TArrayView<PCGMetadataEntryKey>* InOutOptionalKeys = nullptr, FPCGContext* OptionalContext = nullptr);
	UE_API void SetAttributes(const TArrayView<const PCGMetadataEntryKey>& InKeys, const UPCGMetadata* InMetadata, const TArrayView<PCGMetadataEntryKey>& OutKeys, FPCGContext* OptionalContext = nullptr);

	// TODO: Will be moved at the end of the class before commit, as they are functions planned to be deprecated.
	/** Attributes operations - shorthand for points */
	UE_API void MergePointAttributes(const FPCGPoint& InPointA, const FPCGPoint& InPointB, FPCGPoint& OutPoint, EPCGMetadataOp Op);
	UE_API void MergePointAttributesSubset(const FPCGPoint& InPointA, const UPCGMetadata* InMetadataA, const UPCGMetadata* InMetadataSubetA, const FPCGPoint& InPointB, const UPCGMetadata* InMetadataB, const UPCGMetadata* InMetadataSubsetb, FPCGPoint& OutPoint, EPCGMetadataOp Op);
	UE_API void SetPointAttributes(const TArrayView<const FPCGPoint>& InPoints, const UPCGMetadata* InMetadata, const TArrayView<FPCGPoint>& OutPoints, FPCGContext* OptionalContext = nullptr);

	/** Blueprint-friend versions */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void MergeAttributesByKey(int64 KeyA, const UPCGMetadata* MetadataA, int64 KeyB, const UPCGMetadata* MetadataB, int64 TargetKey, EPCGMetadataOp Op, int64& OutKey);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void ResetWeightedAttributesByKey(int64 TargetKey, int64& OutKey);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void AccumulateWeightedAttributesByKey(int64 Key, const UPCGMetadata* Metadata, float Weight, bool bSetNonInterpolableAttributes, int64 TargetKey, int64& OutKey);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void SetAttributesByKey(int64 Key, const UPCGMetadata* InMetadata, int64 TargetKey, int64& OutKey);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void MergePointAttributes(const FPCGPoint& PointA, const UPCGMetadata* MetadataA, const FPCGPoint& PointB, const UPCGMetadata* MetadataB, UPARAM(ref) FPCGPoint& TargetPoint, EPCGMetadataOp Op);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void SetPointAttributes(const FPCGPoint& Point, const UPCGMetadata* Metadata, UPARAM(ref) FPCGPoint& OutPoint);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void ResetPointWeightedAttributes(FPCGPoint& OutPoint);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	UE_API void AccumulatePointWeightedAttributes(const FPCGPoint& InPoint, const UPCGMetadata* InMetadata, float Weight, bool bSetNonInterpolableAttributes, UPARAM(ref) FPCGPoint& OutPoint);

	UE_API void ComputePointWeightedAttribute(FPCGPoint& OutPoint, const TArrayView<TPair<const FPCGPoint*, float>>& InWeightedPoints, const UPCGMetadata* InMetadata);
	UE_API void ComputeWeightedAttribute(PCGMetadataEntryKey& OutKey, const TArrayView<TPair<PCGMetadataEntryKey, float>>& InWeightedKeys, const UPCGMetadata* InMetadata);
	// End of move

	UE_API int64 GetItemKeyCountForParent() const;
	UE_API int64 GetLocalItemCount() const;

	/** Return the number of entries in metadata including the parent entries. */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (DisplayName = "Get Number of Entries"))
	UE_API int64 GetItemCountForChild() const;

	/**
	* Create a new attribute. If the attribute already exists, it will raise a warning (use FindOrCreateAttribute if this usecase can arise)
	* If the attribute already exists but is of the wrong type, it will fail and return nullptr. Same if the name is invalid.
	* Return a typed attribute pointer, of the requested type T.
	*/
	template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
	FPCGMetadataAttribute<T>* CreateAttribute(FPCGAttributeIdentifier AttributeName, const T& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent);
	
	/**
	* Create a new attribute, that will be generic and not typed. This should never be static cast to a FPCGMetadataAttribute<T>* as this would
	* be undefined behavior.
	* If the attribute already exists, it will raise a warning (use FindOrCreateAttribute if this usecase can arise)
	* If the attribute already exists but is of the wrong type, it will fail and return nullptr. Same if the name is invalid.
	*/
	template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == false)
	FPCGMetadataAttributeBase* CreateAttribute(FPCGAttributeIdentifier AttributeName, const T& DefaultValue, bool bAllowsInterpolation = true, bool bOverrideParent = true);
	
	/**
	* Create a new attribute, that will be either generic or typed, depending the type given by the desc. The default value will not be set.
	* If the attribute already exists, it will raise a warning (use FindOrCreateAttribute if this usecase can arise)
	* If the attribute already exists but is of the wrong type, it will fail and return nullptr. Same if the name is invalid.
	*/
	UE_API FPCGMetadataAttributeBase* CreateAttribute(FPCGAttributeIdentifier AttributeName, const FPCGMetadataAttributeDesc& AttributeDesc, bool bAllowsInterpolation = true, bool bOverrideParent = true);

	/**
	* Find or create an attribute. Follows CreateAttribute signature.
	* Extra boolean bOverwriteIfTypeMismatch allows to overwrite an existing attribute if the type mismatch.
	* Same as CreateAttribute, it will return nullptr if the attribute name is invalid.
	* Return a typed attribute pointer, of the requested type T.
	*/
	template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
	FPCGMetadataAttribute<T>* FindOrCreateAttribute(FPCGAttributeIdentifier AttributeName, const T& DefaultValue = T{}, bool bAllowsInterpolation = true, bool bOverrideParent = true, bool bOverwriteIfTypeMismatch = true);
	
	/**
	* Find or create an attribute. Follows CreateAttribute signature.
	* Extra boolean bOverwriteIfTypeMismatch allows to overwrite an existing attribute if the type mismatch.
	* Same as CreateAttribute, it will return nullptr if the attribute name is invalid.
	* Return a generic attribute pointer
	*/
	template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == false)
	FPCGMetadataAttributeBase* FindOrCreateAttribute(FPCGAttributeIdentifier AttributeName, const T& DefaultValue = T{}, bool bAllowsInterpolation = true, bool bOverrideParent = true, bool bOverwriteIfTypeMismatch = true);

	// Need this gymnastic for those 2 functions, because blueprints doesn't support default arguments for Arrays, and we don't want to force c++ user to provide OptionalNewEntriesOrder.

	/** Initializes the metadata from a parent metadata by copying all attributes to it.
	* @param InMetadataToCopy Metadata to copy from
	* @param InOptionalEntriesToCopy Optional array that contains the keys to copy over. This array order will be respected, so it can also be used to re-order entries. If empty, copy them all.
	*/
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (DisplayName = "Initialize As Copy", AutoCreateRefTerm = "InOptionalEntriesToCopy"))
	UE_API void K2_InitializeAsCopy(const UPCGMetadata* InMetadataToCopy, const TArray<int64>& InOptionalEntriesToCopy);

	/** Initializes the metadata from a parent metadata by copy filtered attributes only to it
	* @param InMetadataToCopy         Metadata to copy from
	* @param InFilteredAttributes     Attributes to keep/exclude, can be empty.
	* @param InOptionalEntriesToCopy  Optional array that contains the keys to copy over. This array order will be respected, so it can also be used to re-order entries. If empty, copy them all.
	* @param InFilterMode             Filter to know if we should keep or exclude InFilteredAttributes.
	* @param InMatchOperator          How to match the names for the filtering
	*/
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (DisplayName = "Initialize As Copy With Attribute Filter", AutoCreateRefTerm = "InFilteredAttributes,InOptionalEntriesToCopy"))
	UE_API void K2_InitializeAsCopyWithAttributeFilter(const UPCGMetadata* InMetadataToCopy, const TSet<FName>& InFilteredAttributes, const TArray<int64>& InOptionalEntriesToCopy, EPCGMetadataFilterMode InFilterMode = EPCGMetadataFilterMode::ExcludeAttributes, EPCGStringMatchingOperator InMatchOperator = EPCGStringMatchingOperator::Equal);

	/** Computes Crc from all attributes & keys from outer's data. */
	UE_API void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const;

	/** Expected to be used when metadata domains are already setup. */
	UE_API FPCGMetadataDomain* GetMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID);
	UE_API FPCGMetadataDomain* GetMetadataDomainFromSelector(const FPCGAttributePropertySelector& InSelector);
	
	/** Expected to be used when metadata domain are already setup. */
	UE_API const FPCGMetadataDomain* GetConstMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID) const;
	UE_API const FPCGMetadataDomain* GetConstMetadataDomainFromSelector(const FPCGAttributePropertySelector& InSelector) const;

	FPCGMetadataDomain* GetDefaultMetadataDomain() { return GetMetadataDomain(PCGMetadataDomainID::Default); }
	const FPCGMetadataDomain* GetConstDefaultMetadataDomain() const { return GetConstMetadataDomain(PCGMetadataDomainID::Default); }

	/** Mirror functions to be called on the outer data. If there is no outer, we will use the default object for a UPCGData. */
	UE_API bool MetadataDomainSupportsMultiEntries(const FPCGMetadataDomainID& InDomainID) const;
	UE_API bool MetadataDomainSupportsParenting(const FPCGMetadataDomainID& InDomainID) const;

private:
	/** Return a mutable metadata domain. Will allow for Create/Get/Set attributes. Should never be called on input data (as input data is const). */
	UFUNCTION(BlueprintPure, Category = "PCG|Metadata", meta = (DisplayName = "Mutable Metadata Domain"))
	FPCGMetadataDomainHandle BP_GetMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID);

	/** Return a const metadata domain. Will allow for Get attributes. Create/Set will fail. */
	UFUNCTION(BlueprintPure, Category = "PCG|Metadata", meta = (DisplayName = "Const Metadata Domain"))
	FPCGMetadataDomainHandle BP_GetConstMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID) const;

	// Utility functions to forward calls to FPCGMetadataDomain.
	template <typename Func, typename ...Args>
	decltype(auto) WithMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID, Func InFunc, Args&& ...InArgs);
	
	template <typename Func>
	decltype(auto) WithMetadataDomain_Lambda(const FPCGMetadataDomainID& InMetadataDomainID, Func InFunc);

	template <typename Func, typename ...Args>
	decltype(auto) WithConstMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID, Func InFunc, Args&& ...InArgs) const;

	template <typename Func>
	void ForEachValidUniqueConstMetadataDomain(Func InFunc) const;

	template <typename Func>
	void ForEachValidUniqueMetadataDomain(Func InFunc);

	template <typename Func>
	void FindFixOrCreateDomainInitializeParams(const FPCGMetadataInitializeParams& InParams, const FPCGMetadataDomainID DomainID, const FPCGMetadataDomain& OtherMetadataDomain, Func&& InFunc);

protected:
	UE_API FPCGMetadataDomain* FindOrCreateMetadataDomain(const FPCGMetadataDomainID& InMetadataDomainID);
	
	UE_API FPCGMetadataDomain* CreateMetadataDomain_Unsafe(const FPCGMetadataDomainID& InMetadataDomainID);
	
	UE_API FPCGMetadataAttributeBase* CopyAttribute(FPCGAttributeIdentifier AttributeToCopy, FName NewAttributeName, bool bKeepParent, bool bCopyEntries, bool bCopyValues);

	UE_API bool ParentHasAttribute(FPCGAttributeIdentifier AttributeName) const;

	UE_API void SetLastCachedSelectorOnOwner(FName AttributeName, FPCGMetadataDomainID DomainID);

	UE_API void SetupDomainsFromOtherMetadataIfNeeded(const UPCGMetadata* OtherMetadata);

	UPROPERTY()
	TObjectPtr<const UPCGMetadata> Parent;

	FPCGMetadataDomainID DefaultDomain = PCGMetadataDomainID::Default;
	TMap<FPCGMetadataDomainID, TSharedPtr<FPCGMetadataDomain>> MetadataDomains;
	PCG::FLock MetadataDomainsLock;
};

template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
FPCGMetadataAttribute<T>* UPCGMetadata::CreateAttribute(FPCGAttributeIdentifier AttributeName, const T& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent)
{
	FPCGMetadataDomain* FoundSubMetadata = FindOrCreateMetadataDomain(AttributeName.MetadataDomain);
	return FoundSubMetadata ? FoundSubMetadata->CreateAttribute<T>(AttributeName.Name, DefaultValue, bAllowsInterpolation, bOverrideParent) : nullptr;
}

template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == false)
FPCGMetadataAttributeBase* UPCGMetadata::CreateAttribute(FPCGAttributeIdentifier AttributeName, const T& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent)
{
	FPCGMetadataDomain* FoundSubMetadata = FindOrCreateMetadataDomain(AttributeName.MetadataDomain);
	return FoundSubMetadata ? FoundSubMetadata->CreateAttribute<T>(AttributeName.Name, DefaultValue, bAllowsInterpolation, bOverrideParent) : nullptr;
}

template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
FPCGMetadataAttribute<T>* UPCGMetadata::FindOrCreateAttribute(FPCGAttributeIdentifier AttributeName, const T& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent, bool bOverwriteIfTypeMismatch)
{
	FPCGMetadataDomain* FoundSubMetadata = FindOrCreateMetadataDomain(AttributeName.MetadataDomain);
	return FoundSubMetadata ? FoundSubMetadata->FindOrCreateAttribute<T>(AttributeName.Name, DefaultValue, bAllowsInterpolation, bOverrideParent, bOverwriteIfTypeMismatch) : nullptr;
}

template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == false)
FPCGMetadataAttributeBase* UPCGMetadata::FindOrCreateAttribute(FPCGAttributeIdentifier AttributeName, const T& DefaultValue, bool bAllowsInterpolation, bool bOverrideParent, bool bOverwriteIfTypeMismatch)
{
	FPCGMetadataDomain* FoundSubMetadata = FindOrCreateMetadataDomain(AttributeName.MetadataDomain);
	return FoundSubMetadata ? FoundSubMetadata->FindOrCreateAttribute<T>(AttributeName.Name, DefaultValue, bAllowsInterpolation, bOverrideParent, bOverwriteIfTypeMismatch) : nullptr;
}

template<typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
FPCGMetadataAttribute<T>* UPCGMetadata::GetMutableTypedAttribute_Unsafe(FPCGAttributeIdentifier AttributeName)
{
	FPCGMetadataDomain* FoundSubMetadata = GetMetadataDomain(AttributeName.MetadataDomain);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FoundSubMetadata ? FoundSubMetadata->GetMutableTypedAttribute_Unsafe<T>(AttributeName.Name) : nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
FPCGMetadataAttribute<T>* UPCGMetadata::GetMutableTypedAttribute(FPCGAttributeIdentifier AttributeName)
{
	FPCGMetadataDomain* FoundSubMetadata = GetMetadataDomain(AttributeName.MetadataDomain);
	return FoundSubMetadata ? FoundSubMetadata->GetMutableTypedAttribute<T>(AttributeName.Name) : nullptr;
}

template <typename T> requires (PCG::Private::TIsBasicType<T>::Value == true)
const FPCGMetadataAttribute<T>* UPCGMetadata::GetConstTypedAttribute(FPCGAttributeIdentifier AttributeName) const
{
	const FPCGMetadataDomain* FoundSubMetadata = GetConstMetadataDomain(AttributeName.MetadataDomain);
	return FoundSubMetadata ? FoundSubMetadata->GetConstTypedAttribute<T>(AttributeName.Name) : nullptr;
}

#undef UE_API
