// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGMetadataCommon.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "PCGMetadataAccessor.generated.h"

struct FPCGPoint;

class UPCGMetadata;

UCLASS()
class UPCGMetadataAccessorHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Id-based metadata functions */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static int32 GetInteger32AttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetInteger32AttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, int32 Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static int64 GetInteger64AttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetInteger64AttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, int64 Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static float GetFloatAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetFloatAttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, float Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static double GetDoubleAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetDoubleAttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, double Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static FVector GetVectorAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetVectorAttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FVector& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static FVector4 GetVector4AttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetVector4AttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FVector4& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static FVector2D GetVector2AttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetVector2AttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FVector2D& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static FRotator GetRotatorAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetRotatorAttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FRotator& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static FQuat GetQuatAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetQuatAttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FQuat& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static FTransform GetTransformAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetTransformAttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FTransform& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static FString GetStringAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetStringAttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FString& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static bool GetBoolAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetBoolAttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, bool Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static FSoftObjectPath GetSoftObjectPathAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetSoftObjectPathAttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FSoftObjectPath& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static FSoftClassPath GetSoftClassPathAttributeByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static void SetSoftClassPathAttributeByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, const FSoftClassPath& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static bool SetAttributeFromPropertyByMetadataKey(UPARAM(ref) int64& Key, UPCGMetadata* Metadata, FName AttributeName, const UObject* Object, FName PropertyName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata")
	static PCG_API bool HasAttributeSetByMetadataKey(int64 Key, const UPCGMetadata* Metadata, FName AttributeName);

	/** Point functions */
	UFUNCTION(BlueprintCallable, Category = "PCG", meta = (ScriptMethod))
	static PCG_API void CopyPoint(const FPCGPoint& InPoint, FPCGPoint& OutPoint, bool bCopyMetadata = true, const UPCGMetadata* InMetadata = nullptr, UPCGMetadata* OutMetadata = nullptr);

	static void InitializeMetadata(FPCGPoint& Point, UPCGMetadata* Metadata);

	/** Assigns a metadata entry but does not copy values if from a non-parented metadata */
	static PCG_API void InitializeMetadataWithParent(FPCGPoint& Point, UPCGMetadata* Metadata, const FPCGPoint& ParentPoint, const UPCGMetadata* ParentMetadata);

	/** Assigns a metadata entry key and will copy attribute values if from an unrelated metadata. Note: a null ParentMetadata assumes this is the same as Metadata */
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static PCG_API void InitializeMetadata(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, const FPCGPoint& ParentPoint, const UPCGMetadata* ParentMetadata = nullptr);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static int32 GetInteger32Attribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetInteger32Attribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, int32 Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static int64 GetInteger64Attribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetInteger64Attribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, int64 Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static float GetFloatAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetFloatAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, float Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static double GetDoubleAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetDoubleAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, double Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static FVector GetVectorAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetVectorAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FVector& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static FVector4 GetVector4Attribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetVector4Attribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FVector4& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static FVector2D GetVector2Attribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetVector2Attribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FVector2D& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static FRotator GetRotatorAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetRotatorAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FRotator& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static FQuat GetQuatAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetQuatAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FQuat& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static FTransform GetTransformAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetTransformAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FTransform& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static FString GetStringAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetStringAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FString& Value);
	
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static FName GetNameAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);
	
	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetNameAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FName& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static bool GetBoolAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetBoolAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, bool Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static FSoftObjectPath GetSoftObjectPathAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetSoftObjectPathAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FSoftObjectPath& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static FSoftClassPath GetSoftClassPathAttribute(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static void SetSoftClassPathAttribute(UPARAM(ref) FPCGPoint& Point, UPCGMetadata* Metadata, FName AttributeName, const FSoftClassPath& Value);

	UFUNCTION(BlueprintCallable, Category = "PCG|Metadata", meta = (ScriptMethod))
	static PCG_API bool HasAttributeSet(const FPCGPoint& Point, const UPCGMetadata* Metadata, FName AttributeName);

private:
	/**
	 * Set an arbitrary value to an attribute, using a given metadata entry key. The value needs to be the same type of the attribute or able
	 * to convert to it.
	 * @param Key Metadata entry key to set. The key can be modified.
	 * @param Metadata Pointer to a mutable metadata
	 * @param AttributeName Name of the attribute to set the value on
	 * @param Value Wildcard pin to hold the value to set.
	 * @return true if the set succeeded, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "PCG|Metadata", meta = (ScriptMethod, CustomStructureParam = "Value", AllowAbstract = "false"))
	static PCG_API bool SetAttributeValueByMetadataKey(UPCGMetadata* Metadata, UPARAM(ref) int64& Key, FName AttributeName, const int32& Value);
	DECLARE_FUNCTION(execSetAttributeValueByMetadataKey);
	
	/**
	 * Set an arbitrary value to an attribute on a given point. The value needs to be the same type of the attribute or able
	 * to convert to it.
	 * @param Point Ref of the point to set the attribute on.
	 * @param Metadata Pointer to a mutable metadata
	 * @param AttributeName Name of the attribute to set the value on
	 * @param Value Wildcard pin to hold the value to set.
	 * @return true if the set succeeded, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "PCG|Metadata", meta = (ScriptMethod, CustomStructureParam = "Value", AllowAbstract = "false"))
	static PCG_API bool SetAttributeValue(UPCGMetadata* Metadata, UPARAM(ref) FPCGPoint& Point, FName AttributeName, const int32& Value);
	DECLARE_FUNCTION(execSetAttributeValue);
	
	/**
	 * Get an arbitrary value from an attribute, using a given metadata entry key. The value needs to be the same type of the attribute or able
	 * to convert from it.
	 * @param Key Metadata entry key to get.
	 * @param Metadata Pointer to a const metadata
	 * @param AttributeName Name of the attribute to get the value from
	 * @param Value Wildcard pin to retrieve the read value.
	 * @return true if the get succeeded, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "PCG|Metadata", meta = (ScriptMethod, CustomStructureParam = "Value", AllowAbstract = "false"))
	static PCG_API bool GetAttributeValueByMetadataKey(const UPCGMetadata* Metadata, int64 Key, FName AttributeName, UPARAM(Ref) int32& Value);
	DECLARE_FUNCTION(execGetAttributeValueByMetadataKey);
	
	/**
	 * Get an arbitrary value from an attribute on a given point. The value needs to be the same type of the attribute or able
	 * to convert from it.
	 * @param Point Const ref of the point to get the attribute from.
	 * @param Metadata Pointer to a const metadata
	 * @param AttributeName Name of the attribute to get the value from
	 * @param Value Wildcard pin to retrieve the read value.
	 * @return true if the get succeeded, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "PCG|Metadata", meta = (ScriptMethod, CustomStructureParam = "Value", AllowAbstract = "false"))
	static PCG_API bool GetAttributeValue(const UPCGMetadata* Metadata, const FPCGPoint& Point, FName AttributeName, UPARAM(Ref) int32& Value);
	DECLARE_FUNCTION(execGetAttributeValue);
	
	/**
	 * Set an arbitrary value to an attribute, using a given metadata entry key. The value needs to be the same type of the attribute or able
	 * to convert to it.
	 * @param Key Metadata entry key to set. The key can be modified.
	 * @param MetadataDomain Pointer to a mutable metadata domain handle
	 * @param AttributeName Name of the attribute to set the value on
	 * @param Value Wildcard pin to hold the value to set.
	 * @return true if the set succeeded, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "PCG|MetadataDomain", meta = (ScriptMethod, CustomStructureParam = "Value", AllowAbstract = "false"))
	static PCG_API bool SetAttributeValueByMetadataKeyOnDomain(FPCGMetadataDomainHandle MetadataDomain, UPARAM(ref) int64& Key, FName AttributeName, const int32& Value);
	DECLARE_FUNCTION(execSetAttributeValueByMetadataKeyOnDomain);
	
	/**
	 * Get an arbitrary value from an attribute, using a given metadata entry key. The value needs to be the same type of the attribute or able
	 * to convert from it.
	 * @param Key Metadata entry key to get.
	 * @param Metadata Pointer to a const metadata domain handle
	 * @param AttributeName Name of the attribute to get the value from
	 * @param Value Wildcard pin to retrieve the read value.
	 * @return true if the get succeeded, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "PCG|MetadataDomain", meta = (ScriptMethod, CustomStructureParam = "Value", AllowAbstract = "false"))
	static PCG_API bool GetAttributeValueByMetadataKeyOnDomain(FPCGMetadataDomainHandle MetadataDomain, int64 Key, FName AttributeName, UPARAM(Ref) int32& Value);
	DECLARE_FUNCTION(execGetAttributeValueByMetadataKeyOnDomain);
	
	UFUNCTION(BlueprintPure, Category = "PCG|MetadataDomain")
	static FPCGMetadataDomainID GetElementsDomainID() { return PCGMetadataDomainID::Elements; }
	
	UFUNCTION(BlueprintPure, Category = "PCG|MetadataDomain")
	static FPCGMetadataDomainID GetDataDomainID() { return PCGMetadataDomainID::Data; }
	
	UFUNCTION(BlueprintPure, Category = "PCG|MetadataDomain")
	static FPCGMetadataDomainID GetDefaultDomainID() { return PCGMetadataDomainID::Default; }

protected:
	template<typename T>
	static T GetAttribute(PCGMetadataEntryKey Key, const UPCGMetadata* Metadata, FName AttributeName);

	template<typename T>
	static void SetAttribute(PCGMetadataEntryKey& Key, UPCGMetadata* Metadata, FName AttributeName, const T& Value);
};