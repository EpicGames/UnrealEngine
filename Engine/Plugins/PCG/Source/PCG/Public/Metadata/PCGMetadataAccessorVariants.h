// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Metadata/PCGMetadataCommon.h"
#include "Metadata/PCGMetadataContainerTypes.h"

#include "Misc/TVariant.h"

/**
* Variants to type erase chunk of data to be read/write in attributes or accessors
*/
namespace PCG::Private
{
	//////////////////////
	// OutValues Section
	//////////////////////
	struct FOutValuesByValue
	{
		void* OutValues = nullptr;
		int32 Count = 0;
		
		// Optional descriptor that contains the type of OutValues
		const FPCGMetadataAttributeDesc* UnderlyingDesc = nullptr;
	};
	
	struct FOutValuesByPtr
	{
		TArrayView<const void*> OutValues;
		
		// Optional buffer to store the values if there is a type mismatch.
		FPCGAccessorBuffer* Buffer = nullptr;
		
		// Optional descriptor that contains the type of OutValues
		const FPCGMetadataAttributeDesc* UnderlyingDesc = nullptr;
	};
	
	struct FOutValuesAsArray
	{
		TArrayView<TTuple<const void*, int32>> OutValues;
		
		// Optional buffers to store the values if there is a type mismatch.
		TArrayView<FPCGAccessorBuffer> Buffers;
		
		// Optional descriptor that contains the type of OutValues
		const FPCGMetadataAttributeDesc* UnderlyingDesc = nullptr;
	};
	
	struct FOutValuesAsSet
	{
		TArrayView<FScriptSetHelper*> OutValues;
		
		// Optional descriptor that contains the type of OutValues
		const FPCGMetadataAttributeDesc* UnderlyingDesc = nullptr;
	};
	
	struct FOutValuesAsMap
	{
		TArrayView<FScriptMapHelper*> OutValues;
		
		// Optional descriptor that contains the type of OutValues
		const FPCGMetadataAttributeDesc* UnderlyingDesc = nullptr;
	};
	
	using FOutValues = TVariant<FOutValuesByValue, FOutValuesByPtr, FOutValuesAsArray, FOutValuesAsSet, FOutValuesAsMap>;
	
	//////////////////////
	// InValues Section
	//////////////////////
	struct FInValuesByValue
	{
		const void* InValues = nullptr;
		int32 Count = 0;
		
		// Optional descriptor that contains the type of InValues
		const FPCGMetadataAttributeDesc* UnderlyingDesc = nullptr;
	};
	
	struct FInValuesByPtr
	{
		TConstArrayView<const void*> InValues;
		
		// Optional descriptor that contains the type of InValues
		const FPCGMetadataAttributeDesc* UnderlyingDesc = nullptr;
	};

	struct FInValuesAsArray
	{
		TArrayView<TTuple<const void*, int32>> InValues;
		
		// Optional descriptor that contains the type of InValues
		const FPCGMetadataAttributeDesc* UnderlyingDesc = nullptr;
	};

	struct FInValuesAsSet
	{
		TArrayView<TArray<const void*>> InValues;
		
		// Optional descriptor that contains the type of InValues
		const FPCGMetadataAttributeDesc* UnderlyingDesc = nullptr;
	};

	struct FInValuesAsMap
	{
		TArrayView<TArray<TPair<const void*, const void*>>> InValues;
		
		// Optional descriptor that contains the type of InValues
		const FPCGMetadataAttributeDesc* UnderlyingDesc = nullptr;
	};
	
	struct FInValuesSubset
	{
		const void* InValues = nullptr; // It is expected to be a FInValues underneath. Anything else will crash.
		TArray<int32> Indices;
	};

	using FInValues = TVariant<FInValuesByValue, FInValuesByPtr, FInValuesAsArray, FInValuesAsSet, FInValuesAsMap, FInValuesSubset>;
}