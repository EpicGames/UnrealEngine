// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGCommon.h"
#include "PCGModule.h"
#include "Helpers/PCGConcepts.h"
#include "Helpers/PCGMetadataHelpers.h"
#include "Metadata/PCGMetadataAccessorVariants.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/PCGMetadataCommon.h"
#include "Metadata/PCGMetadataContainerTypes.h"
#include "Utils/PCGValueRange.h"

#include "Algo/Compare.h"
#include "Containers/StaticArray.h"
#include "Misc/TVariant.h"
#include "StructUtils/PropertyBag.h"

#define UE_API PCG_API

class FPCGMetadataDomain;
class UPCGMetadata;

namespace PCGMetadataAttributeConstants
{
	extern PCG_API const FName LastAttributeName;
	extern PCG_API const FName LastCreatedAttributeName;
	extern PCG_API const FName SourceAttributeName;
	extern PCG_API const FName SourceNameAttributeName;
}

struct FPCGAttributeDefaultValue;

namespace PCG::Private
{
	/** Magic number for AddValues to know if we need to write to the DefaultValue or not. */
	constexpr int32 ExistingIndexForDefaultValue = -2;
}

// Simple wrapper around a raw pointer to hold the default value to mimic how it was stored in the typed attributes
struct FPCGAttributeDefaultValue
{
	FPCGAttributeDefaultValue() = default;
	PCG_API ~FPCGAttributeDefaultValue();

	// Since this is an internal struct and attributes are never copied with constructor/assignment, just delete the copy/move
	UE_NONCOPYABLE(FPCGAttributeDefaultValue);

	PCG_API void Init(TSharedPtr<FPCGAttributeProperty> InUnderlyingProperty);
	PCG_API void CopyFrom(const void* SrcData);

	void* GetRawPtr() { return Memory; }
	const void* GetRawPtr() const { return Memory; };
	bool IsValid() const { return Memory != nullptr; }

private:
	void Destroy();

	TSharedPtr<FPCGAttributeProperty> UnderlyingProperty;
	void* Memory = nullptr;
};

class FPCGMetadataAttributeBase
{
public:
	UE_DEPRECATED(5.8, "Metadata attributes should never be constructed individually and should be created from the Metadata domain.")
	UE_API FPCGMetadataAttributeBase(FPCGMetadataDomain* InMetadata, FName InName, const FPCGMetadataAttributeBase* InParent, bool bInAllowsInterpolation);

	UE_API virtual ~FPCGMetadataAttributeBase();

	UE_API virtual void Serialize(FPCGMetadataDomain* InMetadata, FArchive& InArchive);
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const;
	UE_API void AddReferencedObjects(FReferenceCollector& Collector);

	UE_API void SerializeValuesForEntryKeys(TConstArrayView<PCGMetadataEntryKey> EntryKeys, FArchive& InArchive) const;
	UE_API void SerializeValues(TConstArrayView<PCGMetadataValueKey> ValueKeys, FArchive& InArchive) const;

	const FPCGMetadataAttributeDesc& GetAttributeDesc() const { return CachedDesc; }
	bool DoesCompressData() const { return UnderlyingProperty && UnderlyingProperty->CompressData(); }

	template <typename T>
	bool IsOfType() const
	{
		return PCG::Private::IsEquivalentDesc<T>(CachedDesc);
	}

	/** Unparents current attribute by flattening the values, entries, etc. */
	UE_API virtual void Flatten();
	/** Unparents current attribute by flattening the values, entries, etc while only keeping the entries referenced in InEntryKeysToKeep. There must be NO invalid entry keys. */
	UE_API virtual void FlattenAndCompress(const TArrayView<const PCGMetadataEntryKey>& InEntryKeysToKeep);

	/** Remove all entries, values and parenting. */
	UE_API virtual void Reset();

	UE_API const UPCGMetadata* GetMetadata() const;
	const FPCGMetadataDomain* GetMetadataDomain() const { return Metadata; }

	int16 GetTypeId() const { return TypeId; }

	UE_API virtual FPCGMetadataAttributeBase* Copy(FName NewName, FPCGMetadataDomain* InMetadata, bool bKeepParent, bool bCopyEntries = true, bool bCopyValues = true) const;
	UE_API virtual FPCGMetadataAttributeBase* CopyToAnotherType(int16 Type) const;

	UE_API virtual PCGMetadataValueKey GetValueKeyOffsetForChild() const;
	UE_API virtual void SetValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttribute, PCGMetadataEntryKey InEntryKey);
	UE_API virtual void SetZeroValue(PCGMetadataEntryKey ItemKey);
	UE_API virtual void AccumulateValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA, PCGMetadataEntryKey InEntryKeyA, float Weight);
	UE_API virtual void SetWeightedValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttribute, const TArrayView<const TPair<PCGMetadataEntryKey, float>>& InWeightedKeys);
	UE_API virtual void SetValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA, PCGMetadataEntryKey InEntryKeyA, const FPCGMetadataAttributeBase* InAttributeB, PCGMetadataEntryKey InEntryKeyB, EPCGMetadataOp Op);
	UE_API virtual bool IsEqualToDefaultValue(PCGMetadataValueKey ValueKey) const;
	/** In the case of multi entry attribute and after some operations, we might have a single entry attribute with a default value that is different than the first entry. Use this function to fix that. Only valid if there is one and only one value. */
	UE_API virtual void SetDefaultValueToFirstEntry();

	UE_API virtual bool UsesValueKeys() const;
	UE_API virtual bool AreValuesEqualForEntryKeys(PCGMetadataEntryKey EntryKey1, PCGMetadataEntryKey EntryKey2) const;
	UE_API virtual bool AreValuesEqual(PCGMetadataValueKey ValueKey1, PCGMetadataValueKey ValueKey2) const;

	UE_API void SetValueFromValueKey(PCGMetadataEntryKey EntryKey, PCGMetadataValueKey ValueKey, bool bResetValueOnDefaultValueKey = false);
	UE_API PCGMetadataValueKey GetValueKey(PCGMetadataEntryKey EntryKey) const;
	UE_API bool HasNonDefaultValue(PCGMetadataEntryKey EntryKey) const;
	UE_API void ClearEntries();

	/** Bulk getter, to lock in read only once per parent. */
	UE_API void GetValueKeys(const TArrayView<const PCGMetadataEntryKey>& EntryKeys, TArray<PCGMetadataValueKey>& OutValueKeys) const;

	/** Bulk getter, to lock in read only once per parent. */
	UE_API void GetValueKeys(TConstPCGValueRange<PCGMetadataEntryKey> EntryKeys, TArray<PCGMetadataValueKey>& OutValueKeys) const;

	/** Optimized version that take ownership on the Entries passed.*/
	UE_API void GetValueKeys(TArrayView<PCGMetadataEntryKey> EntryKeys, TArray<PCGMetadataValueKey>& OutValueKeys) const;

	/** Optimized version that take ownership on the Entries passed.*/
	UE_API void GetValueKeys(TPCGValueRange<PCGMetadataEntryKey> EntryKeys, TArray<PCGMetadataValueKey>& OutValueKeys) const;

	/** Bulk setter to lock in write only once. */
	UE_API void SetValuesFromValueKeys(const TArrayView<const TTuple<PCGMetadataEntryKey, PCGMetadataValueKey>>& EntryValuePairs, bool bResetValueOnDefaultValueKey = true);

	/** Two arrays version of bulk setter to lock in write only once. Both arrays must be the same size. */
	UE_API void SetValuesFromValueKeys(const TArrayView<const PCGMetadataEntryKey>& EntryKeys, const TArrayView<const PCGMetadataValueKey>& ValueKeys, bool bResetValueOnDefaultValueKey = true);
	UE_API void SetValuesFromValueKeys(TConstPCGValueRange<PCGMetadataEntryKey> EntryKeys, const TArrayView<const PCGMetadataValueKey>& ValueKeys, bool bResetValueOnDefaultValueKey = true);
	UE_API void SetValuesFromValueKeys(const TArrayView<const PCGMetadataEntryKey* const>& EntryKeys, const TArrayView<const PCGMetadataValueKey>& ValueKeys, bool bResetValueOnDefaultValueKey = true);

	/**
	 * Allow partitioning of the entries. Container can be either TArray<TArray<int32>> or TArray<TBitArray<>>
	 * If the underlying type is not comparable, it will return an empty array, meaning that all the entry keys are effectively all different.
	*/
	template <typename BucketType>
	TArray<BucketType> AttributePartition(TConstArrayView<PCGMetadataEntryKey> EntryKeys) const;

	bool AllowsInterpolation() const { return bAllowsInterpolation; }

	int32 GetNumberOfEntries() const { return EntryToValueKeyMap.Num(); }
	int32 GetNumberOfEntriesWithParents() const { return EntryToValueKeyMap.Num() + (Parent ? Parent->GetNumberOfEntries() : 0); }

	// This call is not thread safe
	const TMap<PCGMetadataEntryKey, PCGMetadataValueKey>& GetEntryToValueKeyMap_NotThreadSafe() const { return EntryToValueKeyMap; }

	const FPCGMetadataAttributeBase* GetParent() const { return Parent; }

	/** Returns true if for valid attribute names, which are alphanumeric with some special characters allowed. */
	static UE_API bool IsValidName(const FString& Name);
	static UE_API bool IsValidName(const FName& Name);

	/** Replaces any invalid characters in name with underscores. Returns true if Name was changed. */
	static UE_API bool SanitizeName(FString& InOutName);

	/**
	 * GetValues section
	 * Templated code to support all container types.
	 * If the provided output container does not match the underlying type, it will fail.
	 * It is also mandatory that the number of value keys and output container num match.
	 *
	 * Supported types are:
	 * - Single types that are valid for metadata
	 * - Pointers to single types that are valid for metadata
	 * - Typed ConstArrayViews for arrays
	 * - PCG::TScriptSetWrapper<T> for set, wrapper that can be manipulated like a set
	 * - PCG::TScriptMapWrapper<T> for map, wrapper that can be manipulated like a map
	 */
	template <typename T> requires (!std::is_pointer_v<T>)
	void GetValues(TArrayView<const PCGMetadataValueKey> ValueKeys, TArrayView<T> OutValues) const
	{
		if (ValueKeys.IsEmpty())
		{
			return;
		}

		if (!IsOfType<T>())
		{
			// @todo_pcg Better logging
			UE_LOGF(LogPCG, Error, "Type mismatch on GetValues");
			return;
		}

		if (!ensure(ValueKeys.Num() == OutValues.Num()))
		{
			return;
		}

		GetValues_Internal(ValueKeys, PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesByValue>{}, OutValues.GetData(), OutValues.Num()});
	}

	template <typename T>
	void GetValues(TArrayView<const PCGMetadataValueKey> ValueKeys, TArrayView<const T*> OutValues) const
	{
		if (ValueKeys.IsEmpty())
		{
			return;
		}

		if (!IsOfType<T>())
		{
			// @todo_pcg Better logging
			UE_LOGF(LogPCG, Error, "Type mismatch on GetValues");
			return;
		}

		if (!ensure(ValueKeys.Num() == OutValues.Num()))
		{
			return;
		}

		// We need to type erase the pointers, but to be fully C++ compliant about aliasing rules we can't reinterpret_cast. So make a copy of the pointers.
		TArray<const void*, TInlineAllocator<256>> Temp;
		Temp.SetNumUninitialized(OutValues.Num());
		GetValues_Internal(ValueKeys, PCG::Private::FOutValues{ TInPlaceType<PCG::Private::FOutValuesByPtr>{}, Temp });
		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			OutValues[i] = static_cast<const T*>(Temp[i]);
		}
	}

	template <typename T>
	void GetValues(TArrayView<const PCGMetadataValueKey> ValueKeys, TArrayView<TConstArrayView<T>> OutValues) const
	{
		if (ValueKeys.IsEmpty())
		{
			return;
		}

		if (!IsOfType<TArray<T>>())
		{
			// @todo_pcg Better logging
			UE_LOGF(LogPCG, Error, "Type mismatch on GetValues");
			return;
		}

		if (!ensure(ValueKeys.Num() == OutValues.Num()))
		{
			return;
		}

		TArray<TTuple<const void*, int32>, TInlineAllocator<256>> Temp;
		Temp.SetNumUninitialized(OutValues.Num());
		GetValues_Internal(ValueKeys, PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsArray>{}, Temp});
		for (int32 i = 0; i < OutValues.Num(); ++i)
		{
			OutValues[i] = MakeConstArrayView<T>(static_cast<const T*>(Temp[i].Get<0>()), Temp[i].Get<1>());
		}
	}

	// To support sets
	template <typename T>
	void GetValues(TArrayView<const PCGMetadataValueKey> ValueKeys, TArrayView<PCG::TScriptSetWrapper<T>> OutValues) const
	{
		if (ValueKeys.IsEmpty())
		{
			return;
		}

		if (!IsOfType<TSet<T>>())
		{
			// @todo_pcg Better logging
			UE_LOGF(LogPCG, Error, "Type mismatch on GetValues");
			return;
		}

		if (!ensure(ValueKeys.Num() == OutValues.Num()))
		{
			return;
		}

		TArray<FScriptSetHelper*, TInlineAllocator<256>> Temp;
		Temp.Reserve(OutValues.Num());
		for (PCG::TScriptSetWrapper<T>& Value : OutValues)
		{
			Temp.Add(&Value.Helper);
		}
		GetValues_Internal(ValueKeys, PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsSet>{}, Temp});
	}

	// To support maps
	template <typename KeyType, typename ValueType>
	void GetValues(TArrayView<const PCGMetadataValueKey> ValueKeys, TArrayView<PCG::TScriptMapWrapper<KeyType, ValueType>> OutValues) const
	{
		if (ValueKeys.IsEmpty())
		{
			return;
		}

		if (!IsOfType<TMap<KeyType, ValueType>>())
		{
			// @todo_pcg Better logging
			UE_LOGF(LogPCG, Error, "Type mismatch on GetValues");
			return;
		}

		if (!ensure(ValueKeys.Num() == OutValues.Num()))
		{
			return;
		}

		TArray<FScriptMapHelper*, TInlineAllocator<256>> Temp;
		Temp.Reserve(OutValues.Num());
		for (PCG::TScriptMapWrapper<KeyType, ValueType>& Value : OutValues)
		{
			Temp.Add(&Value.Helper);
		}
		GetValues_Internal(ValueKeys, PCG::Private::FOutValues{TInPlaceType<PCG::Private::FOutValuesAsMap>{}, Temp});
	}

	////////////// End GetValues Section

	// Fill the OutValues using the array of entry keys.
	// cf. GetValues for more info on supported types.
	template <typename T>
	void GetValuesFromItemKeys(TConstArrayView<PCGMetadataEntryKey> EntryKeys, TArrayView<T> OutValues) const
	{
		GetValuesFromItemKeys(PCGValueRangeHelpers::MakeConstValueRange(EntryKeys), OutValues);
	}

	// Fill the OutValues using the range of entry keys.
	// cf. GetValues for more info on supported types.
	template <typename T>
	void GetValuesFromItemKeys(TConstPCGValueRange<PCGMetadataEntryKey> EntryKeys, TArrayView<T> OutValues) const
	{
		if (EntryKeys.IsEmpty() || !ensure(EntryKeys.Num() == OutValues.Num()))
		{
			return;
		}

		TArray<PCGMetadataValueKey> ValueKeys;
		GetValueKeys(EntryKeys, ValueKeys);
		GetValues(ValueKeys, OutValues);
	}

	template <typename T>
	T GetValueFromItemKey(const PCGMetadataEntryKey InEntryKey) const
	{
		T OutValue{};
		GetValuesFromItemKeys(MakeArrayView(&InEntryKey, 1), MakeArrayView(&OutValue, 1));
		return OutValue;
	}

	// Override the default value for this attribute.
	// Will fail if the type mismatch.
	template <typename T>
	void SetDefaultValue(const T& Value)
	{
		if (TOptional<FScriptArrayHelper> Helper = MakeArrayHelper(); Helper.IsSet())
		{
			check(DefaultValue.IsValid());
			AddValues(MakeArrayView(&Value, 1), /*ExistingIndex=*/PCG::Private::ExistingIndexForDefaultValue);
		}
	}

	template <typename T>
	void SetValue(const PCGMetadataEntryKey InEntryKey, const T& InValue)
	{
		SetValuesFromValueKeys(MakeArrayView(&InEntryKey, 1), AddValues<T>({InValue}));
	}

	template <typename T>
	void SetValues(TArrayView<const PCGMetadataEntryKey * const> ItemKeys, TArrayView<const T> InValues)
	{
		SetValuesFromValueKeys(ItemKeys, AddValues(InValues));
	}

	template <typename T>
	void SetValues(TArrayView<const PCGMetadataEntryKey> ItemKeys, TArrayView<const T> InValues)
	{
		SetValuesFromValueKeys(ItemKeys, AddValues(InValues));
	}

	template <typename T>
	void SetValues(TConstPCGValueRange<PCGMetadataEntryKey> ItemKeys, TArrayView<const T> InValues)
	{
		SetValuesFromValueKeys(ItemKeys, AddValues(InValues));
	}

	template <typename T>
	void SetValues(TPCGValueRange<PCGMetadataEntryKey> ItemKeys, TArrayView<const T> InValues)
	{
		SetValuesFromValueKeys(PCGValueRangeHelpers::MakeConstValueRange(ItemKeys), AddValues(InValues));
	}

	// Special version to write on pre-allocated data if we do not compress, in order to be able to write in concurrency.
	template <typename T>
	void SetValues_TryLockless(TArrayView<PCGMetadataEntryKey*> ItemKeys, const TArrayView<const T>& InValues, int32 StartIndex)
	{
		if (InValues.IsEmpty())
		{
			return;
		}

		if (UnderlyingProperty->CompressData())
		{
			SetValues(ItemKeys, InValues);
		}
		else
		{
			check(StartIndex != INDEX_NONE && Values.IsValidIndex(StartIndex + InValues.Num() - 1));

			// Since values are already preallocated, we can take advantage of the AddValues with existing index.
			AddValues(InValues, StartIndex, /*bLockless=*/true);
		}
	}

	// Copy values from SrcPtr to the attribute. Property must match the attribute property, and the SrcPtr is expected to be
	// reinterpreted as the property type. The ItemKey can be the invalid entry key to set the default value.
	UE_API void SetValueFromProperty(PCGMetadataEntryKey ItemKey, const void* SrcPtr, const FProperty* Property);

	UE_API int32 PreallocateValues(TArrayView<PCGMetadataEntryKey*> EntryKeys, bool bLockless);

	UE_API void Prepare(int32 Count);

	// @todo_pcg: Temporary for debug. Build a struct that will contain the value at the entry key.
	UE_API FInstancedPropertyBag BuildStructForDebug(PCGMetadataEntryKey EntryKey) const;

	// Utility function to get the pointer to a value using a value key. Return nullptr if invalid. Advanced function, use at your own risk.
	UE_API const void* GetReadAddressForValueKey_Unsafe(const PCGMetadataValueKey InValueKey) const;

	// Utility function to get the pointer to a value using an entry key. Return nullptr if invalid. Advanced function, use at your own risk.
	UE_API const void* GetReadAddressFromEntryKey_Unsafe(const PCGMetadataEntryKey InEntryKey) const;

protected:
	// Base serialize for deprecation, what was done before the merge with generic
	UE_API void BaseSerialize(FPCGMetadataDomain* InMetadata, FArchive& InArchive);
	
	/** Attributes can't be created individually, they need to be created through a Metadata Domain. */
	friend FPCGMetadataDomain;
	friend class FPCGAttributeGenericAccessor;

	// New attribute with no parent, name is contained in the InAttributeDesc.
	UE_API FPCGMetadataAttributeBase(const FPCGMetadataAttributeDesc& InAttributeDesc, FPCGMetadataDomain* InMetadata, bool bInAllowsInterpolation);

	// New attribute with an existing parent, can override the name.
	UE_API FPCGMetadataAttributeBase(TNotNull<const FPCGMetadataAttributeBase*> InParent, FPCGMetadataDomain* InMetadata, bool bInAllowsInterpolation, TOptional<FName> InName = {});

	// For deprecation purposes of the typed attributes
	UE_API FPCGMetadataAttributeBase(const FPCGMetadataAttributeDesc& InAttributeDesc, const FPCGMetadataAttributeBase* InParent, FPCGMetadataDomain* InMetadata, bool bInAllowsInterpolation);

	// Allocate the values (without initialization if it can) and return the first index of that new memory space.
	// Return INDEX_NONE if it didn't work.
	UE_API int32 AllocateValues_Unsafe(int32 Count);

	// Used to make an array helper of the local FScriptArray.
	// Optional, since the attribute can be ill-formed, so the user should always check if it is set before
	// using it.
	UE_API TOptional<FScriptArrayHelper> MakeArrayHelper() const;

	/**
	 * GetValues implementation section
	 * GetValues_Internal is working on a Variant that will type erase the underlying output container
	 * in order to reduce template code size.
	 */
	void GetValues_Internal(const TArrayView<const PCGMetadataValueKey> ValueKeys, PCG::Private::FOutValues OutValues) const
	{
		// Bitset with all unretrieved values. If we have any unretrieved value, we will ask the parent for those.
		TBitArray<> UnretrievedValues(true, ValueKeys.Num());

		return GetValues_Internal(ValueKeys, MoveTemp(OutValues), UnretrievedValues);
	}

	/**
	 * AddValues section
	 * Values will be dispatched to a type erased variant to reduce template code size
	 * Variant contains all supported containers: Values/Array/Set/Map
	 *
	 * Note that to support setting the default value (and potentially write on existing values if it
	 * was preallocated), all AddValues have an extra parameter `ExistingIndex` that will indicate if allocation
	 * is necessary or not. This is a very niche feature, and it is expected that everything is already allocated
	 * or it will crash. There is also an option for lockless, if we expect to be able to write without any lock.
	 */

	/**
	 * Basic version that will accept all single values (as an array view).
	 */
	template <typename T>
	TArray<PCGMetadataValueKey> AddValues(TArrayView<const T> InValues, int32 ExistingIndex = INDEX_NONE, bool bLockless = false)
	{
		if (InValues.IsEmpty())
		{
			return {};
		}

		if (!IsOfType<T>())
		{
			// @todo_pcg Better logging
			UE_LOGF(LogPCG, Error, "Type mismatch on AddValues");
			return {};
		}

		// For compressed data, we need to analyze which values actually need to be added. If existing index is not none, ignore it.
		// Need the concept for Equals, since `AddUnique` require the operator '==` to be defined.
		if constexpr (PCG::Concepts::CIsComparable<T, T>)
		{
			if (UnderlyingProperty->CompressData() && ExistingIndex == INDEX_NONE)
			{
				// Since we're getting raw values here, we might have duplicates
				// so we should aim to remove duplicates here so we preserve our 'compress data' idea, otherwise it will break other foundational blocks (e.g. partition)
				TArray<T, TInlineAllocator<256>> UniqueValues;

				// Initially, fill with mapping to unique values so we can remap them at the end if needed
				TArray<PCGMetadataValueKey> FoundValueKeys;
				FoundValueKeys.Reserve(InValues.Num());

				for (int ValueIndex = 0; ValueIndex < InValues.Num(); ++ValueIndex)
				{
					FoundValueKeys.Emplace(UniqueValues.AddUnique(InValues[ValueIndex]));
				}

				return AddCompressedValues(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, UniqueValues.GetData(), UniqueValues.Num()}, UniqueValues.Num(), FoundValueKeys);
			}
		}

		return AddValues_Internal(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, InValues.GetData(), InValues.Num()}, InValues.Num(), ExistingIndex, bLockless);
	}

	/**
	 * Version that accept anything that is array like, like `TArray<T>`, `TArrayView<T>`, `TStaticArray<T, 5>`, `TArray<T, TInlineAllocator<5>>`, etc...
	 */
	template <typename ArrayContainer> requires (PCG::Concepts::CIsArrayLike<ArrayContainer>)
	TArray<PCGMetadataValueKey> AddValues(TConstArrayView<ArrayContainer> InValues, int32 ExistingIndex = INDEX_NONE, bool bLockless = false)
	{
		if (InValues.IsEmpty())
		{
			return {};
		}

		using T = std::remove_const_t<typename ArrayContainer::ElementType>;
		if (!IsOfType<TArray<T>>())
		{
			// @todo_pcg Better logging
			UE_LOGF(LogPCG, Error, "Type mismatch on AddValues");
			return {};
		}

		// For compressed data, we need to analyze which values actually need to be added. If existing index is not none, ignore it.
		// Need the concept for Equals, since `AddUnique` require the operator '==` to be defined.
		if constexpr (PCG::Concepts::CIsComparable<T, T>)
		{
			if (UnderlyingProperty->CompressData() && ExistingIndex == INDEX_NONE)
			{
				// Since we're getting raw values here, we might have duplicates
				// so we should aim to remove duplicates here so we preserve our 'compress data' idea, otherwise it will break other foundational blocks (e.g. partition)
				TArray<TTuple<const void*, int32>, TInlineAllocator<256>> UniqueValues;

				// Initially, fill with mapping to unique values so we can remap them at the end if needed
				TArray<PCGMetadataValueKey> FoundValueKeys;
				FoundValueKeys.Reserve(InValues.Num());

				for (int ValueIndex = 0; ValueIndex < InValues.Num(); ++ValueIndex)
				{
					const ArrayContainer& Value = InValues[ValueIndex];
					int32 UniqueIndex = UniqueValues.IndexOfByPredicate([&Value](const TTuple<const void*, int32>& UniqueValue) -> bool
					{
						TConstArrayView<T> TypedUniqueValue = MakeConstArrayView(static_cast<const T*>(UniqueValue.Key), UniqueValue.Value);
						return (Value.Num() == TypedUniqueValue.Num()) && Algo::Compare(Value, TypedUniqueValue);
					});
					
					if (UniqueIndex == INDEX_NONE)
					{
						UniqueIndex = UniqueValues.Emplace(Value.GetData(), Value.Num());
					}

					FoundValueKeys.Emplace(UniqueIndex);
				}

				return AddCompressedValues(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, UniqueValues}, UniqueValues.Num(), FoundValueKeys);
			}
		}

		TArray<TTuple<const void*, int32>, TInlineAllocator<256>> InValuesTuple;
		InValuesTuple.Reserve(InValues.Num());
		Algo::Transform(InValues, InValuesTuple, [](const auto& Item) { return TTuple<const void*, int32>{Item.GetData(), Item.Num()}; });
		return AddValues_Internal(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsArray>{}, InValuesTuple}, InValues.Num(), ExistingIndex, bLockless);
	}

	/**
	 * Version that accepts Sets
	 */
	template <typename T, typename ...Extra>
	TArray<PCGMetadataValueKey> AddValues(TConstArrayView<TSet<T, Extra...>> InValues, int32 ExistingIndex = INDEX_NONE, bool bLockless = false)
	{
		if (InValues.IsEmpty())
		{
			return {};
		}

		if (!IsOfType<TSet<std::remove_const_t<T>>>())
		{
			// @todo_pcg Better logging
			UE_LOGF(LogPCG, Error, "Type mismatch on AddValues");
			return {};
		}

		TArray<TArray<const void*>, TInlineAllocator<256>> InSetValues;
		InSetValues.Reserve(InValues.Num());
		Algo::Transform(InValues, InSetValues, [](const auto& Item) -> TArray<const void*>
			{
				TArray<const void*> SetValues;
				SetValues.Reserve(Item.Num());
				Algo::Transform(Item, SetValues, [](const T& Value) -> const T* { return &Value;});
				return SetValues;
			});
		return AddValues_Internal(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsSet>{}, InSetValues}, InValues.Num(), ExistingIndex, bLockless);
	}

	/**
	 * Version that accepts Maps
	 */
	template <typename KeyType, typename ValueType, typename ...Extra>
	TArray<PCGMetadataValueKey> AddValues(TConstArrayView<TMap<KeyType, ValueType, Extra...>> InValues, int32 ExistingIndex = INDEX_NONE, bool bLockless = false)
	{
		if (InValues.IsEmpty())
		{
			return {};
		}

		if (!IsOfType<TMap<std::remove_const_t<KeyType>, std::remove_const_t<ValueType>>>())
		{
			// @todo_pcg Better logging
			UE_LOGF(LogPCG, Error, "Type mismatch on AddValues");
			return {};
		}

		TArray<TArray<TPair<const void*, const void*>>, TInlineAllocator<256>> InMapValues;
		InMapValues.Reserve(InValues.Num());
		Algo::Transform(InValues, InMapValues, [](const auto& Item) -> TArray<TPair<const void*, const void*>>
			{
				TArray<TPair<const void*, const void*>> MapValues;
				MapValues.Reserve(Item.Num());
				Algo::Transform(Item, MapValues, [](const auto& It) -> TPair<const void*, const void*> { return {&It.Key, &It.Value};});
				return MapValues;
			});
		return AddValues_Internal(PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesAsMap>{}, InMapValues}, InValues.Num(), ExistingIndex, bLockless);
	}

	// Internal implementation that will do the dispatch using the typed erased variant.
	UE_API TArray<PCGMetadataValueKey> AddValues_Internal(const PCG::Private::FInValues& InValues, int32 Count, int32 ExistingIndex = INDEX_NONE, bool bLockless = false, TOptional<TConstArrayView<int32>> OptionalIndices = {});

	// Internal implementation for compressed data
	UE_API TArray<PCGMetadataValueKey> AddCompressedValues(PCG::Private::FInValues InUniqueValues, int32 Count, TArray<PCGMetadataValueKey>& FoundValueKeys);

	// Internal implementation for typed-erased compressed data. Will create a temporary storage to find all the unique values then will call AddCompressedValues with those.
	UE_API TArray<PCGMetadataValueKey> AddCompressedValues(const PCG::Private::FInValues& InValues, int32 Count);

	/**
	 * Find all the value keys associated to InValues for compressed data. Return true if all the values were found.
	 * If the ValueLock is already held, we can set bShouldLock to false. 
	 * Can also provide an out value ValueKeysSet to initialize/store the number of values found. Useful for partial find.
	 */
	UE_API bool FindValues(TArray<PCGMetadataValueKey>& OutValueKeys, const PCG::Private::FInValues& InValues, int32 Count, int32* OutValueKeysSet = nullptr, bool bShouldLock = true) const;

	/**
	 * Find all the value keys associated to InValues for compressed data. Return true if all the values were found. Preallocated version.
	 * If the ValueLock is already held, we can set bShouldLock to false. 
	 * Can also provide an out value ValueKeysSet to initialize/store the number of values found. Useful for partial find.
	 */
	UE_API bool FindValues(TArrayView<PCGMetadataValueKey> OutValueKeys, const PCG::Private::FInValues& InValues, int32 Count, int32* OutValueKeysSet = nullptr, bool bShouldLock = true) const;

	template <typename T>
	void InitForDeprecation(TArray<T> InValues, const T& InDefaultValue)
	{
		CachedDesc.Name = Name;

		if (TOptional<FScriptArrayHelper> Helper = MakeArrayHelper(); ensure(Helper))
		{
			Helper->EmptyValues(InValues.Num());
			SetDefaultValue<T>(InDefaultValue);

			// FScriptArray and TArray are interchangeable, if and only if they are the exact same type
			Helper->MoveAssign(&InValues);

			ValueKeyOffset = GetParent() ? GetParent()->GetValueKeyOffsetForChild() : 0;
		}
	}

	using FCopyFunctor = TFunction<void(void* /*DestData*/, const void* /*SrcData*/, const int32 /*Count*/)>;
	UE_API void CopyInternal(FPCGMetadataAttributeBase* NewAttribute, bool bKeepParent, bool bCopyEntries = true, bool bCopyValues = true, FCopyFunctor CopyFunctor = {}) const;

private:
	void Init();

	// Unsafe version, needs to be write lock protected.
	UE_API void SetValueFromValueKey_Unsafe(PCGMetadataEntryKey EntryKey, PCGMetadataValueKey ValueKey, bool bResetValueOnDefaultValueKey, bool bAllowInvalidEntries = false);

	// Gather the value keys for the list of entry keys.
	// Because we need to update the entry keys if we look for the value keys in the parent, but because the EntryKeys are coming from the outside, we can't modify them.
	// (as entry keys need to be put into the parent referential when looking for value keys).
	// So we will copy internally the EntryKeys to modify them (and only once) if we are not owner of the memory.
	UE_API void GetValueKeys_Internal(TConstPCGValueRange<PCGMetadataEntryKey> EntryKeys, TArrayView<PCGMetadataValueKey> OutValueKeys, TBitArray<>& UnsetValues, bool bOwnerOfEntryKeysView = false) const;

	UE_API void GetValues_Internal(const TArrayView<const PCGMetadataValueKey> ValueKeys, PCG::Private::FOutValues OutValues, TBitArray<>& UnretrievedValues) const;
	void FindValuesInternal(const PCG::Private::FInValues& InValues, int32 Count, TArrayView<PCGMetadataValueKey> ValueKeys, int& ValueKeysSet, bool bIsRoot, bool bShouldLock) const;

protected:
	TMap<PCGMetadataEntryKey, PCGMetadataValueKey> EntryToValueKeyMap;
	mutable PCG::FSharedLock EntryMapLock;

	FPCGMetadataDomain* Metadata = nullptr;
	const FPCGMetadataAttributeBase* Parent = nullptr;
	bool bAllowsInterpolation = false;
	
	// Legacy Id that is only set if the attribute is typed. This is purely for deprecation purposes, and the only reliable source to
	// know if you can `static_cast` or not. All the info is now contained in the Desc that can be queried with `GetAttributeDesc`
	int16 TypeId = static_cast<int16>(EPCGMetadataTypes::Unknown);

private:
	FPCGMetadataAttributeDesc CachedDesc;
	TSharedPtr<FPCGAttributeProperty> UnderlyingProperty;
	FScriptArray Values;
	FPCGAttributeDefaultValue DefaultValue;

	mutable PCG::FSharedLock ValueLock;
	PCGMetadataValueKey ValueKeyOffset = 0;

public:
	FName Name = NAME_None;
	PCGMetadataAttributeKey AttributeId = -1;
};

#undef UE_API
