// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "PCGModule.h"
#include "Helpers/PCGMetadataHelpers.h"
#include "Metadata/PCGMetadataAttribute.h"
#include "Metadata/PCGMetadataAttributeTraits.h"
#include "Metadata/PCGMetadataCommon.h"

#include "Serialization/ArchiveCountMem.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

class FPCGMetadataDomain;
class UPCGMetadata;

/**
 * Not supposed to be used directly. Only supports legacy types, such as int32 or float.
 * It does not support arrays or complex structs.
 */
template<typename T>
class FPCGMetadataAttribute : public FPCGMetadataAttributeBase
{
	static_assert(PCG::Private::TIsBasicType<T>::Value,
		"FPCGMetadataAttribute<T> only supports basic types (int32, float, FVector, etc.). "
		"Use FPCGMetadataAttributeBase for non-basic types.");

	template<typename U>
	friend class FPCGMetadataAttribute;
	
	friend FPCGMetadataDomain;

protected:
	// Constructors were public before 5.8, but are now protected. Use this sentinel token to guard the new protected constructors.
	struct FProtectedToken{};
	
	FPCGMetadataAttribute(FProtectedToken, FPCGMetadataDomain* InMetadata, FName InName, const FPCGMetadataAttributeBase* InParent, const T& InDefaultValue, bool bInAllowsInterpolation)
		: FPCGMetadataAttributeBase(PCG::Private::MakeAttributeDesc<T>(InName), InParent, InMetadata, bInAllowsInterpolation)
	{
		static_assert(PCG::Private::TIsBasicType<T>::Value, "Only supports legacy types, such as int32 or float");
		TypeId = PCG::Private::MetadataTypes<T>::Id;
		
		if (!InParent)
		{
			SetDefaultValue(InDefaultValue);
		}
	}

	// This constructor is used only during serialization
	FPCGMetadataAttribute(FProtectedToken)
		: FPCGMetadataAttributeBase(PCG::Private::MakeAttributeDesc<T>(NAME_None), nullptr, false)
	{
		static_assert(PCG::Private::TIsBasicType<T>::Value, "Only supports legacy types, such as int32 or float");
		TypeId = PCG::Private::MetadataTypes<T>::Id;
	}
	
public:
	UE_DEPRECATED(5.8, "Metadata attributes should never be constructed individually and should be created from the Metadata domain.")
	FPCGMetadataAttribute(FPCGMetadataDomain* InMetadata, FName InName, const FPCGMetadataAttributeBase* InParent, const T& InDefaultValue, bool bInAllowsInterpolation)
		: FPCGMetadataAttribute(FProtectedToken{}, InMetadata, InName, InParent, InDefaultValue, bInAllowsInterpolation)
	{}

	// This constructor is used only during serialization
	UE_DEPRECATED(5.8, "Metadata attributes should never be constructed individually and should be created from the Metadata domain.")
	FPCGMetadataAttribute()
		: FPCGMetadataAttribute(FProtectedToken{})
	{}

	virtual void Serialize(FPCGMetadataDomain* InMetadata, FArchive& InArchive) override
	{
		InArchive.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
		
		if (InArchive.IsLoading() && InArchive.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ConvertFPCGMetadataAttributeToGenericAttributes)
		{
			// Need to forward all the values coming from the previous serialization, and follow the same order of initialization
			FPCGMetadataAttributeBase::BaseSerialize(InMetadata, InArchive);

			TArray<T> PreviousValues;
			T PreviousDefaultValue{};
			InArchive << PreviousValues;
			InArchive << PreviousDefaultValue;

			InitForDeprecation<T>(MoveTemp(PreviousValues), PreviousDefaultValue);
		}
		else
		{
			FPCGMetadataAttributeBase::Serialize(InMetadata, InArchive);
		}
	}

	FPCGMetadataAttribute* TypedCopy(FName NewName, FPCGMetadataDomain* InMetadata, bool bKeepParent, bool bCopyEntries = true, bool bCopyValues = true)
	{
		return static_cast<FPCGMetadataAttribute*>(Copy(NewName, InMetadata, bKeepParent, bCopyEntries, bCopyValues));
	}
	
	virtual FPCGMetadataAttributeBase* Copy(FName NewName, FPCGMetadataDomain* InMetadata, bool bKeepParent, bool bCopyEntries = true, bool bCopyValues = true) const override
	{
		return CopyInternal<T>(NewName, InMetadata, bKeepParent, bCopyEntries, bCopyValues);
	}

	virtual FPCGMetadataAttributeBase* CopyToAnotherType(int16 TargetType) const override;

protected:
	// TODO: add enable if only on compatible types, but this has some repercussion on using metadata on types that aren't normally supported.
	// TODO: The UPCGMetadata version was exposed, but should have not, mark the FPCGMetadataDomain version protected. Will be moved at the end, before commit.
	template<typename U>
	FPCGMetadataAttributeBase* CopyInternal(FName NewName, FPCGMetadataDomain* InMetadata, bool bKeepParent, bool bCopyEntries, bool bCopyValues) const
	{
		// If we copy an attribute where we don't want to keep the parent, while copying entries and/or values, we'll lose data.
		// In that case, we will copy all the data from this attribute and all its ancestors.

		// We can't keep the parent if we don't have the same root.
		checkSlow(!bKeepParent || PCGMetadataHelpers::HasSameRoot(Metadata, InMetadata));

		// Validate that the new name is valid
		if (!IsValidName(NewName))
		{
			UE_LOGF(LogPCG, Error, "Try to create a new attribute with an invalid name: %ls", *NewName.ToString());
			return nullptr;
		}
		
		// This copies to a new attribute.
		FPCGMetadataAttribute<U>* AttributeCopy = new FPCGMetadataAttribute<U>(typename FPCGMetadataAttribute<U>::FProtectedToken(), InMetadata, NewName, bKeepParent ? this : nullptr, U{}, bAllowsInterpolation);
		
		auto CopyFunctor = [](void* DestData, const void* SrcData, const int32 Count)
		{
			if constexpr (std::is_same_v<T, U> && std::is_trivially_copyable_v<T>)
			{
				FMemory::Memcpy(DestData, SrcData, Count * sizeof(T));
			}
			else if constexpr (PCG::Private::IsBroadcastableOrConstructible<T, U>())
			{
				TArrayView<U> DestDataView(static_cast<U*>(DestData), Count);
				TConstArrayView<T> SrcDataView(static_cast<const T*>(SrcData), Count);
				
				for (int32 i = 0; i < Count; ++i)
				{
					PCG::Private::GetValueWithBroadcastAndConstructible(SrcDataView[i], DestDataView[i]);
				}
			}
			else
			{
				static_assert(!std::is_same_v<T, T>, "Copy Internal should never be called with a pair of types that are incompatible.");
			}
		};
		
		FPCGMetadataAttributeBase::CopyInternal(AttributeCopy, bKeepParent, bCopyEntries, bCopyValues, CopyFunctor);

		return AttributeCopy;
	}

public:
	virtual void SetZeroValue(PCGMetadataEntryKey ItemKey) override
	{
		check(ItemKey != PCGInvalidEntryKey);
		ZeroValue(ItemKey);
	}
	
	virtual void AccumulateValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttribute, PCGMetadataEntryKey InEntryKey, float Weight) override
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanInterpolate)
		{
			check(ItemKey != PCGInvalidEntryKey);

			if (InAttribute && !InAttribute->IsOfType<T>())
			{
				return;
			}

			Accumulate(ItemKey, InAttribute, InEntryKey, Weight);
		}
	}

	virtual void SetWeightedValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttribute, const TArrayView<const TPair<PCGMetadataEntryKey, float>>& InWeightedKeys) override
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanInterpolate)
		{
			check(ItemKey != PCGInvalidEntryKey);

			if (InAttribute && !InAttribute->IsOfType<T>())
			{
				return;
			}

			Accumulate(ItemKey, InAttribute, InWeightedKeys);
		}
	}
	
	
	//~ Note that this override is purely to disambiguate between all the different versions of SetValue. It just forward the call to the super method.
	virtual void SetValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttribute, PCGMetadataEntryKey InEntryKey) override
	{
		return FPCGMetadataAttributeBase::SetValue(ItemKey, InAttribute, InEntryKey);
	}

	virtual void SetValue(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA, PCGMetadataEntryKey InEntryKeyA, const FPCGMetadataAttributeBase* InAttributeB, PCGMetadataEntryKey InEntryKeyB, EPCGMetadataOp Op) override
	{
		check(ItemKey != PCGInvalidEntryKey);
		bool bAppliedValue = false;

		if ((InAttributeA && !InAttributeA->IsOfType<T>()) || (InAttributeB && !InAttributeB->IsOfType<T>()))
		{
			return;
		}

		if (Op == EPCGMetadataOp::TargetValue && InAttributeB)
		{
			// Take value of second attribute.
			if (InAttributeB == this)
			{
				SetValueFromValueKey(ItemKey, GetValueKey(InEntryKeyB));
			}
			else
			{
				SetValue(ItemKey, InAttributeB->GetValueFromItemKey<T>(InEntryKeyB));
			}

			bAppliedValue = true;
		}
		else if (Op == EPCGMetadataOp::SourceValue && InAttributeA)
		{
			// Take value of first attribute.
			if (InAttributeA == this)
			{
				SetValueFromValueKey(ItemKey, GetValueKey(InEntryKeyA));
			}
			else
			{
				SetValue(ItemKey, InAttributeA->GetValueFromItemKey<T>(InEntryKeyA));
			}

			bAppliedValue = true;
		}
		else if (InAttributeA && InAttributeB && bAllowsInterpolation)
		{
			// Combine attributes using specified operation.
			if (Op == EPCGMetadataOp::Min)
			{
				bAppliedValue = SetMin(ItemKey, InAttributeA, InEntryKeyA, InAttributeB, InEntryKeyB);
			}
			else if (Op == EPCGMetadataOp::Max)
			{
				bAppliedValue = SetMax(ItemKey, InAttributeA, InEntryKeyA, InAttributeB, InEntryKeyB);
			}
			else if (Op == EPCGMetadataOp::Sub)
			{
				bAppliedValue = SetSub(ItemKey, InAttributeA, InEntryKeyA, InAttributeB, InEntryKeyB);
			}
			else if (Op == EPCGMetadataOp::Add)
			{
				bAppliedValue = SetAdd(ItemKey, InAttributeA, InEntryKeyA, InAttributeB, InEntryKeyB);
			}
			else if (Op == EPCGMetadataOp::Mul)
			{
				bAppliedValue = SetMul(ItemKey, InAttributeA, InEntryKeyA, InAttributeB, InEntryKeyB);
			}
			else if (Op == EPCGMetadataOp::Div)
			{
				bAppliedValue = SetDiv(ItemKey, InAttributeA, InEntryKeyA, InAttributeB, InEntryKeyB);
			}
		}
		else if (InAttributeA && InAttributeB && HasNonDefaultValue(ItemKey))
		{
			// In this case, the current already has a value, in which case we should not update it
			bAppliedValue = true;
		}

		if (bAppliedValue)
		{
			// Nothing to do
		}
		else if (InAttributeA)
		{
			if (InAttributeA == this)
			{
				SetValueFromValueKey(ItemKey, GetValueKey(InEntryKeyA));
			}
			else
			{
				SetValue(ItemKey, InAttributeA->GetValueFromItemKey<T>(InEntryKeyA));
			}
		}
		else if (InAttributeB)
		{
			if (InAttributeB == this)
			{
				SetValueFromValueKey(ItemKey, GetValueKey(InEntryKeyB));
			}
			else
			{
				SetValue(ItemKey, InAttributeB->GetValueFromItemKey<T>(InEntryKeyB));
			}
		}
	}

	/** Adds the value, returns the value key for the given value */
	PCGMetadataValueKey AddValue(const T& InValue)
	{
		TArray<PCGMetadataValueKey> Result = FPCGMetadataAttributeBase::AddValues<T>(MakeArrayView<const T>(&InValue, 1));
		return !Result.IsEmpty() ? Result[0] : PCGDefaultValueKey; 
	}

	TArray<PCGMetadataValueKey> AddValues(const TArrayView<const T>& InValues)
	{
		return FPCGMetadataAttributeBase::AddValues<T>(InValues);
	}

	void SetValue(PCGMetadataEntryKey ItemKey, const T& InValue)
	{
		FPCGMetadataAttributeBase::SetValue<T>(ItemKey, InValue);
	}

	void SetValues(const TArrayView<const PCGMetadataEntryKey>& ItemKeys, const TArrayView<const T>& InValues)
	{
		FPCGMetadataAttributeBase::SetValues<T>(ItemKeys, InValues);
	}

	void SetValues(const TArrayView<const PCGMetadataEntryKey * const>& ItemKeys, const TArrayView<const T>& InValues)
	{
		FPCGMetadataAttributeBase::SetValues<T>(ItemKeys, InValues);
	}
	
	void SetValues(TConstPCGValueRange<PCGMetadataEntryKey> ItemKeys, const TArrayView<const T>& InValues)
	{
		FPCGMetadataAttributeBase::SetValues<T>(ItemKeys, InValues);
	}
	
	void SetValues(TPCGValueRange<PCGMetadataEntryKey> ItemKeys, const TArrayView<const T>& InValues)
	{
		FPCGMetadataAttributeBase::SetValues<T>(ItemKeys, InValues);
	}

	template<typename U> requires (!std::is_same_v<T, U> && PCG::Private::IsBroadcastableOrConstructible<U, T>())
	void SetValue(PCGMetadataEntryKey ItemKey, const U& InValue)
	{
		T Value{};
		PCG::Private::GetValueWithBroadcastAndConstructible(InValue, Value);
		FPCGMetadataAttributeBase::SetValue<T>(ItemKey, Value);
	}
	
	// Special version for enum classes, as enums are not directly constructible
	template<typename U> requires (!std::is_same_v<T, U> && !PCG::Private::IsBroadcastableOrConstructible<U, T>() && std::is_enum_v<U> && std::is_convertible_v<std::underlying_type_t<U>, T>)
	void SetValue(PCGMetadataEntryKey ItemKey, const U& InValue)
	{
		FPCGMetadataAttributeBase::SetValue<T>(ItemKey, static_cast<T>(InValue));
	}

	T GetValueFromItemKey(PCGMetadataEntryKey ItemKey) const
	{
		return FPCGMetadataAttributeBase::GetValueFromItemKey<T>(ItemKey);
	}

	T GetValue(PCGMetadataValueKey ValueKey) const
	{
		T Value{};
		FPCGMetadataAttributeBase::GetValues<T>(MakeConstArrayView(&ValueKey, 1), MakeArrayView(&Value, 1));
		return Value;
	}

public:
	/**
	* Write into pre-allocated OutValues the values associated with the given value keys.
	*/
	void GetValues(const TArrayView<const PCGMetadataValueKey> ValueKeys, TArrayView<T> OutValues) const
	{
		return FPCGMetadataAttributeBase::GetValues<T>(ValueKeys, OutValues);
	}

	/** 
	* Write into pre-allocated OutValues the values associated with the given entry keys. 
	* Const version on the Entry Keys, where they won't be modified. It will induce a copy of the entry keys
	* if we ever have to go check the parent attribute, as we need to modify the entry keys for that.
	* If you don't care if the Entry Keys are modified, use the non-const version of the EntryKeys.
	*/
	void GetValuesFromItemKeys(TConstArrayView<PCGMetadataEntryKey> EntryKeys, TArrayView<T> OutValues) const
	{
		FPCGMetadataAttributeBase::GetValuesFromItemKeys<T>(EntryKeys, OutValues);
	}
	
	void GetValuesFromItemKeys(TConstPCGValueRange<PCGMetadataEntryKey> EntryKeys, TArrayView<T> OutValues) const
	{
		FPCGMetadataAttributeBase::GetValuesFromItemKeys<T>(EntryKeys, OutValues);
	}

	/** 
	* Write into pre-allocated OutValues the values associated with the given entry keys.
	* Non-Const version on the Entry Keys, where they can be modified. If you need the Entry Keys to not be modifed,
	* use the const version of the EntryKeys.
	*/
	void GetValuesFromItemKeys(TArrayView<PCGMetadataEntryKey> EntryKeys, TArrayView<T> OutValues) const
	{
		FPCGMetadataAttributeBase::GetValuesFromItemKeys<T>(EntryKeys, OutValues);
	}
	
	PCGMetadataValueKey FindValue(const T& InValue) const 
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CompressData)
		{
			TArray<PCGMetadataValueKey> OutValueKeys;
			const bool bSuccess = FindValues(MakeConstArrayView(&InValue, 1), OutValueKeys);
			return bSuccess && !OutValueKeys.IsEmpty() ? OutValueKeys[0] : PCGDefaultValueKey;
		}
		else
		{
			return PCGNotFoundValueKey;
		}
	}
	
	bool FindValues(const TArrayView<const T>& InValues, TArray<PCGMetadataValueKey>& OutValueKeys) const
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CompressData)
		{
			return FPCGMetadataAttributeBase::FindValues(OutValueKeys, PCG::Private::FInValues{TInPlaceType<PCG::Private::FInValuesByValue>{}, InValues.GetData(), InValues.Num()}, InValues.Num());
		}
		else
		{
			OutValueKeys.Init(PCGNotFoundValueKey, InValues.Num());
			return false;
		}
	}

	void SetValues_TryLockless(TArrayView<PCGMetadataEntryKey*> EntryKeys, TArrayView<const T> InValues, int32 StartIndex)
	{
		FPCGMetadataAttributeBase::SetValues_TryLockless<T>(EntryKeys, InValues, StartIndex);
	}

protected:
	/** Code related to computing compared values (min, max, sub, add) */
	bool SetMin(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA, PCGMetadataEntryKey InEntryKeyA, const FPCGMetadataAttributeBase* InAttributeB, PCGMetadataEntryKey InEntryKeyB)
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanMinMax)
		{
			SetValue(ItemKey, 
				PCG::Private::MetadataTraits<T>::Min(
					InAttributeA->GetValueFromItemKey<T>(InEntryKeyA),
					InAttributeB->GetValueFromItemKey<T>(InEntryKeyB)));

			return true;
		}
		else
		{
			return false;
		}
	}
	
	bool SetMax(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA, PCGMetadataEntryKey InEntryKeyA, const FPCGMetadataAttributeBase* InAttributeB, PCGMetadataEntryKey InEntryKeyB)
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanMinMax)
		{
			SetValue(ItemKey,
				PCG::Private::MetadataTraits<T>::Max(
					InAttributeA->GetValueFromItemKey<T>(InEntryKeyA),
					InAttributeB->GetValueFromItemKey<T>(InEntryKeyB)));
			return true;
		}
		else
		{
			return false;
		}
	}
	
	bool SetAdd(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA, PCGMetadataEntryKey InEntryKeyA, const FPCGMetadataAttributeBase* InAttributeB, PCGMetadataEntryKey InEntryKeyB)
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanSubAdd)
		{
			SetValue(ItemKey,
				PCG::Private::MetadataTraits<T>::Add(
					InAttributeA->GetValueFromItemKey<T>(InEntryKeyA),
					InAttributeB->GetValueFromItemKey<T>(InEntryKeyB)));
			return true;
		}
		else
		{
			return false;
		}
	}
	
	bool SetSub(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA, PCGMetadataEntryKey InEntryKeyA, const FPCGMetadataAttributeBase* InAttributeB, PCGMetadataEntryKey InEntryKeyB)
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanSubAdd)
		{
			SetValue(ItemKey,
				PCG::Private::MetadataTraits<T>::Sub(
					InAttributeA->GetValueFromItemKey<T>(InEntryKeyA),
					InAttributeB->GetValueFromItemKey<T>(InEntryKeyB)));
			return true;
		}
		else
		{
			return false;
		}
	}
	
	bool SetMul(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA, PCGMetadataEntryKey InEntryKeyA, const FPCGMetadataAttributeBase* InAttributeB, PCGMetadataEntryKey InEntryKeyB)
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanMulDiv)
		{
			SetValue(ItemKey,
				PCG::Private::MetadataTraits<T>::Mul(
					InAttributeA->GetValueFromItemKey<T>(InEntryKeyA),
					InAttributeB->GetValueFromItemKey<T>(InEntryKeyB)));
			return true;
		}
		else
		{
			return false;
		}
	}
	
	bool SetDiv(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttributeA, PCGMetadataEntryKey InEntryKeyA, const FPCGMetadataAttributeBase* InAttributeB, PCGMetadataEntryKey InEntryKeyB)
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanMulDiv)
		{
			SetValue(ItemKey,
				PCG::Private::MetadataTraits<T>::Div(
					InAttributeA->GetValueFromItemKey<T>(InEntryKeyA),
					InAttributeB->GetValueFromItemKey<T>(InEntryKeyB)));
			return true;
		}
		else
		{
			return false;
		}
	}

	/** Weighted/interpolated values related code */
	void ZeroValue(PCGMetadataEntryKey ItemKey)
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanInterpolate)
		{
			SetValue(ItemKey, PCG::Private::MetadataTraits<T>::ZeroValue());
		}
	}

	void Accumulate(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttribute, PCGMetadataEntryKey InEntryKey, float Weight)
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanInterpolate)
		{
			T Value = PCG::Private::MetadataTraits<T>::WeightedSum(
					GetValueFromItemKey(ItemKey),
					InAttribute->GetValueFromItemKey<T>(InEntryKey),
					Weight);

			if constexpr (PCG::Private::MetadataTraits<T>::InterpolationNeedsNormalization)
			{
				static_assert(PCG::Private::MetadataTraits<T>::CanNormalize);

				// We need to normalize the resulting value
				PCG::Private::MetadataTraits<T>::Normalize(Value);
			}
		
			SetValue(ItemKey, Value);
		}
	}
	
	void Accumulate(PCGMetadataEntryKey ItemKey, const FPCGMetadataAttributeBase* InAttribute, const TArrayView<const TPair<PCGMetadataEntryKey, float>>& InWeightedKeys)
	{
		if constexpr (PCG::Private::MetadataTraits<T>::CanInterpolate)
		{
			T Value = PCG::Private::MetadataTraits<T>::ZeroValueForWeightedSum();
			for (const TPair<PCGMetadataEntryKey, float>& WeightedEntry : InWeightedKeys)
			{
				Value = PCG::Private::MetadataTraits<T>::WeightedSum(
					Value,
					InAttribute->GetValueFromItemKey<T>(WeightedEntry.Key),
					WeightedEntry.Value);
			}

			if constexpr (PCG::Private::MetadataTraits<T>::InterpolationNeedsNormalization)
			{
				static_assert(PCG::Private::MetadataTraits<T>::CanNormalize);
								
				// We need to normalize the resulting value
				PCG::Private::MetadataTraits<T>::Normalize(Value);
			}

			SetValue(ItemKey, Value);
		}
	}
};

namespace PCGMetadataAttribute
{
	UE_DEPRECATED(5.8, "Should never be used anymore")
	inline FPCGMetadataAttributeBase* AllocateEmptyAttributeFromType(int16 TypeId)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		switch (TypeId)
		{

#define PCG_ALLOCATEEMPTY_DECL(T) case PCG::Private::MetadataTypes<T>::Id: return new FPCGMetadataAttribute<T>();
		PCG_FOREACH_SUPPORTEDTYPES(PCG_ALLOCATEEMPTY_DECL)
#undef PCG_ALLOCATEEMPTY_DECL

		default:
			return nullptr;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	template <typename Func, typename... Args>
	FORCEINLINE decltype(auto) CallbackWithRightType(uint16 TypeId, Func Callback, Args&& ...InArgs)
	{
		using ReturnType = decltype(Callback(double{}, std::forward<Args>(InArgs)...));

		switch (TypeId)
		{

#define PCG_CALLBACKWITHRIGHTTYPE_DECL(T) case (uint16)(PCG::Private::MetadataTypes<T>::Id): CALLSITE_FORCEINLINE return Callback(T{}, std::forward<Args>(InArgs)...);
		PCG_FOREACH_SUPPORTEDTYPES(PCG_CALLBACKWITHRIGHTTYPE_DECL)
#undef PCG_CALLBACKWITHRIGHTTYPE_DECL

		default:
		{
			// ReturnType{} is invalid if ReturnType is void
			if constexpr (std::is_same_v<ReturnType, void>)
			{
				return;
			}
			else
			{
				return ReturnType{};
			}
		}
		}
	}
}

template<typename T>
FPCGMetadataAttributeBase* FPCGMetadataAttribute<T>::CopyToAnotherType(int16 TargetType) const
{
	return PCGMetadataAttribute::CallbackWithRightType(TargetType, [this](auto Dummy) -> FPCGMetadataAttributeBase*
	{
		using U = decltype(Dummy);

		if constexpr (PCG::Private::IsBroadcastableOrConstructible(PCG::Private::MetadataTypes<T>::Id, PCG::Private::MetadataTypes<U>::Id))
		{
			return CopyInternal<U>(Name, Metadata, /*bKeepParent=*/false, /*bCopyEntries=*/true, /*bCopyValues=*/true);
		}
		else
		{
			UE_LOGF(LogPCG, Error, "Metadata attribute '%ls' cannot change its type - delete and create instead", *Name.ToString());
			return nullptr;
		}
	});
}
