// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/PCGConcepts.h"
#include "Metadata/PCGMetadataCommon.h"

#include "Concepts/TSoftClassPtr.h"
#include "Concepts/TSoftObjectPtr.h"
#include "Math/Transform.h" // IWYU pragma: keep
#include "Misc/DefaultValueHelper.h"
#include "UObject/Class.h"
#include "UObject/SoftObjectPath.h"

// Convenient macro to avoid duplicating a lot of code with all our supported types.
#define PCG_FOREACH_SUPPORTEDTYPES(MACRO) \
	MACRO(int32)           \
	MACRO(int64)           \
	MACRO(float)           \
	MACRO(double)          \
	MACRO(FVector2D)       \
	MACRO(FVector)         \
	MACRO(FVector4)        \
	MACRO(FQuat)           \
	MACRO(FTransform)      \
	MACRO(FString)         \
	MACRO(bool)            \
	MACRO(FRotator)        \
	MACRO(FName)           \
	MACRO(FSoftObjectPath) \
	MACRO(FSoftClassPath)

namespace PCG
{
	namespace Private
	{
		// Logging function that doesn't need to be in the headers.
		PCG_API void LogNotLoadedObjectNotOnGameThread(const FSoftObjectPath& InObjectPath);
		
		// Utility structure to be able to extract the types of Key and Values for a map.
		template <typename Container>
		struct TMapTypeExtractor
		{
		};

		template <typename InKeyType, typename InValueType>
		struct TMapTypeExtractor<TMap<InKeyType, InValueType>>
		{
			using KeyType = InKeyType;
			using ValueType = InValueType;
		};

		// Utility struct to know when a type is a basic type, specialized in the macro.
		template <typename T> struct TIsBasicType { enum { Value = false }; };

		// To transform a range of values from one type to another.
		using FTransformFunc = void(*)(void* /*DestPtr*/, const void* /*SrcPtr*/, int32 /*Count*/);

		// To construct a range of values.
		using FConstructMemoryFunc = void(*)(void* /*DestPtr*/, int32 /*Count*/);

		// To destruct a range of values.
		using FDestroyMemoryFunc = void(*)(void* /*DestPtr*/, int32 /*Count*/);

		template <typename T>
		void ConstructMemory(void* DestPtr, int32 Count)
		{
			TArrayView<T> Dest(static_cast<T*>(DestPtr), Count);
			for (T& Value : Dest)
			{
				new (&Value) T();
			}
		}

		template <typename T>
		void DestructMemory(void* DestPtr, int32 Count)
		{
			DestructItems(static_cast<T*>(DestPtr), Count);
		}

		// Forward declare
		template <typename InType, typename OutType>
		inline bool GetValueWithBroadcast(const InType& InValue, OutType& OutValue);

		template <typename InType, typename OutType>
		inline void GetValuesWithBroadcast_Internal(void* DestPtr, const void* SrcPtr, int32 Count);

		template <typename InType, typename OutType>
		inline void GetValuesWithConstructible_Internal(void* DestPtr, const void* SrcPtr, int32 Count);

		// Old way of converting types to the type enum, will be progressively replaced by MakeAttributeDesc/IsSameType.
		template <typename T>
		struct MetadataTypes
		{
			enum { Id = static_cast<uint16>(EPCGMetadataTypes::Unknown) + sizeof(T) };
		};
		
		// Factorized static assert function to indicate to use TObjectPtr instead of UObjects
		// or a generic message that the type is not supported.
		template <typename T>
		consteval void InvalidTypeStaticAssert()
		{
			if constexpr (std::is_base_of_v<UObject, std::remove_pointer_t<std::remove_const_t<T>>>)
			{
				static_assert(!std::is_same_v<T,T>, "For UObjects*, use TObjectPtr<>");
			}
			else
			{
				static_assert(!std::is_same_v<T,T>, "This type is not supported for metadata.");
			}
		}

		/**
		 * Return the default attribute descriptor associated with the type T, without any attribute name.
		 * This version does not compile and is a pitfall for all unsupported types.
		 */
		template <typename T>
		const FPCGMetadataAttributeDesc& GetDefaultAttributeDesc()
		{
			static const FPCGMetadataAttributeDesc EmptyDesc{};
			
			InvalidTypeStaticAssert<T>();

			return EmptyDesc;
		}

		/**
		 * Return true if the provided Desc is matching with type T.
		 * This version does not compile and is a pitfall for all unsupported types.
		 */
		template <typename T>
		bool IsEquivalentDesc(const FPCGMetadataAttributeDesc& Desc)
		{
			InvalidTypeStaticAssert<T>();
			return false;
		}

		/**
		 * Return true if the provided type and type object is matching with type T.
		 * This version does not compile for non-basic types and is a pitfall for all unsupported types.
		 */
		template <typename T>
		bool IsSameType(const EPCGMetadataTypes InType, const UObject* InTypeObject)
		{
			if constexpr (!TIsBasicType<T>::Value)
			{
				InvalidTypeStaticAssert<T>();
				return false;
			}
			else if constexpr (PCG::Concepts::CIsBasicStruct<T>)
			{
				// Basic structures can be represented with their own EPCGMetadataTypes (like Vector) or as a struct.
				return static_cast<uint16>(InType) == MetadataTypes<T>::Id || (InType == EPCGMetadataTypes::Struct && InTypeObject == TBaseStructure<T>::Get());
			}
			else
			{
				return static_cast<uint16>(InType) == MetadataTypes<T>::Id;
			}
		}

		/**
		 * Macro to specialize all known "basic" types.
		 */
#define PCGMetadataGenerateDataTypes(Type, TypeEnum) \
	template<> struct MetadataTypes<Type>{ enum { Id = static_cast<uint16>(EPCGMetadataTypes::TypeEnum)}; }; \
	template<> inline const FPCGMetadataAttributeDesc& GetDefaultAttributeDesc<Type>() { static const FPCGMetadataAttributeDesc TypedDesc{.ValueType=EPCGMetadataTypes::TypeEnum}; return TypedDesc; } \
	template<> struct TIsBasicType<Type> { enum { Value = true }; }; \
	template<> inline bool IsEquivalentDesc<Type>(const FPCGMetadataAttributeDesc& Desc) { return IsSameType<Type>(Desc.ValueType, Desc.ValueTypeObject) && Desc.ContainerTypes.IsEmpty(); }

		PCGMetadataGenerateDataTypes(float, Float);
		PCGMetadataGenerateDataTypes(double, Double);
		PCGMetadataGenerateDataTypes(int32, Integer32);
		PCGMetadataGenerateDataTypes(int64, Integer64);
		PCGMetadataGenerateDataTypes(FVector2D, Vector2);
		PCGMetadataGenerateDataTypes(FVector, Vector);
		PCGMetadataGenerateDataTypes(FVector4, Vector4);
		PCGMetadataGenerateDataTypes(FQuat, Quaternion);
		PCGMetadataGenerateDataTypes(FTransform, Transform);
		PCGMetadataGenerateDataTypes(FString, String);
		PCGMetadataGenerateDataTypes(bool, Boolean);
		PCGMetadataGenerateDataTypes(FRotator, Rotator);
		PCGMetadataGenerateDataTypes(FName, Name);
		PCGMetadataGenerateDataTypes(FSoftObjectPath, SoftObjectPath);
		PCGMetadataGenerateDataTypes(FSoftClassPath, SoftClassPath);

#undef PCGMetadataGenerateDataTypes

		/**
		 * Specialization section of GetDefaultAttributeDesc
		 * Supported generic types:
		 *   - UStruct
		 *   - UEnum
		 *   - TArray<T> if T is a supported type. Does not support nested containers for now.
		 *   - TSet<T> if T is a supported type. Does not support nested containers for now.
		 *   - TMap<T, U> if T and U are supported types. Does not support nested containers for now.
		 */
		template<typename T> requires (TModels_V<CStaticStructProvider, T> && !std::is_base_of_v<FSoftObjectPath, T>)
		const FPCGMetadataAttributeDesc& GetDefaultAttributeDesc()
		{
			static const FPCGMetadataAttributeDesc TypedDesc{
				.ValueType = EPCGMetadataTypes::Struct,
				.ValueTypeObject = T::StaticStruct()
			};

			return TypedDesc;
		}
		
		template<typename T> requires (TIsTObjectPtr_V<T>)
		const FPCGMetadataAttributeDesc& GetDefaultAttributeDesc()
		{
			using ElementType = typename T::ElementType;
			
			static const FPCGMetadataAttributeDesc TypedDesc{
				.ValueType = std::is_base_of_v<UClass, ElementType> ? EPCGMetadataTypes::Class : EPCGMetadataTypes::Object,
				.ValueTypeObject = ElementType::StaticClass()
			};

			return TypedDesc;
		}
		
		template<typename T> requires UE::CTSoftObjectPtr<T>
		const FPCGMetadataAttributeDesc& GetDefaultAttributeDesc()
		{
			using ElementType = typename T::ElementType;
			static const FPCGMetadataAttributeDesc TypedDesc{
				.ValueType = EPCGMetadataTypes::SoftObject,
				.ValueTypeObject = ElementType::StaticClass()
			};

			return TypedDesc;
		}

		template<typename T> requires UE::CTSoftClassPtr<T>
		const FPCGMetadataAttributeDesc& GetDefaultAttributeDesc()
		{
			using ElementType = typename T::ElementType;
			static const FPCGMetadataAttributeDesc TypedDesc{
				.ValueType = EPCGMetadataTypes::SoftClass,
				.ValueTypeObject = ElementType::StaticClass()
			};

			return TypedDesc;
		}

		template<typename T> requires (TIsUEnumClass<T>::Value == true)
		const FPCGMetadataAttributeDesc& GetDefaultAttributeDesc()
		{
			static const FPCGMetadataAttributeDesc TypedDesc{
				.ValueType = EPCGMetadataTypes::Enum,
				.ValueTypeObject = StaticEnum<T>()
			};

			return TypedDesc;
		}

		template <typename Container> requires TIsTArray_V<Container>
		const FPCGMetadataAttributeDesc& GetDefaultAttributeDesc()
		{
			using T = typename Container::ElementType;

			static const FPCGMetadataAttributeDesc TypedDesc = []()
			{
				FPCGMetadataAttributeDesc Result = GetDefaultAttributeDesc<T>();
				Result.ContainerTypes.Add(EPCGMetadataAttributeContainerTypes::Array);
				return Result;
			}();

			return TypedDesc;
		}

		template <typename Container> requires (TIsTSet_V<Container>)
		const FPCGMetadataAttributeDesc& GetDefaultAttributeDesc()
		{
			using T = typename Container::ElementType;

			static const FPCGMetadataAttributeDesc TypedDesc = []()
			{
				FPCGMetadataAttributeDesc Result = GetDefaultAttributeDesc<T>();
				Result.ContainerTypes.Add(EPCGMetadataAttributeContainerTypes::Set);
				return Result;
			}();

			return TypedDesc;
		}

		template<typename Container> requires (TIsTMap_V<Container>)
		const FPCGMetadataAttributeDesc& GetDefaultAttributeDesc()
		{
			using KeyType = typename TMapTypeExtractor<Container>::KeyType;
			using ValueType = typename TMapTypeExtractor<Container>::ValueType;

			static const FPCGMetadataAttributeDesc TypedDesc = []()
			{
				FPCGMetadataAttributeDesc Result = GetDefaultAttributeDesc<ValueType>();
				const FPCGMetadataAttributeDesc& Key = GetDefaultAttributeDesc<KeyType>();

				Result.ContainerTypes.Add(EPCGMetadataAttributeContainerTypes::Map);
				Result.KeyType = Key.ValueType;
				Result.KeyTypeObject = Key.ValueTypeObject;

				return Result;
			}();

			return TypedDesc;
		}

		//////////////////////////////

		/**
		 * Return the attribute descriptor associated with the type T, with the Attribute name provided.
		 */
		template <typename T>
		FPCGMetadataAttributeDesc MakeAttributeDesc(FName AttributeName)
		{
			FPCGMetadataAttributeDesc Result = GetDefaultAttributeDesc<T>();
			Result.Name = AttributeName;
			return Result;
		}

		//////////////////////////////

		template<typename T> requires (TModels_V<CStaticStructProvider, T>  && !std::is_base_of_v<FSoftObjectPath, T>)
		bool IsSameType(const EPCGMetadataTypes InType, const UObject* InTypeObject)
		{
			return InType == EPCGMetadataTypes::Struct &&
				InTypeObject == T::StaticStruct();
		}

		template<typename T> requires (TIsUEnumClass<T>::Value == true)
		bool IsSameType(const EPCGMetadataTypes InType, const UObject* InTypeObject)
		{
			return InType == EPCGMetadataTypes::Enum &&
				InTypeObject == StaticEnum<T>();
		}
		
		template<typename T> requires TIsTObjectPtr_V<T>
		bool IsSameType(const EPCGMetadataTypes InType, const UObject* InTypeObject)
		{
			using ElementType = typename T::ElementType;
			
			return (InType == EPCGMetadataTypes::Object || (std::is_base_of_v<UClass, ElementType> && InType == EPCGMetadataTypes::Class)) &&
				InTypeObject == ElementType::StaticClass();
		}
		
		template<typename T> requires UE::CTSoftObjectPtr<T>
		bool IsSameType(const EPCGMetadataTypes InType, const UObject* InTypeObject)
		{
			using ElementType = typename T::ElementType;

			return InType == EPCGMetadataTypes::SoftObject &&
				InTypeObject == ElementType::StaticClass();
		}
		
		template<typename T> requires UE::CTSoftClassPtr<T>
		bool IsSameType(const EPCGMetadataTypes InType, const UObject* InTypeObject)
		{
			using ElementType = typename T::ElementType;

			return InType == EPCGMetadataTypes::SoftClass &&
				InTypeObject == ElementType::StaticClass();
		}

		//////////////////////////////

		template<typename T> requires ((TModels_V<CStaticStructProvider, T>  && !std::is_base_of_v<FSoftObjectPath, T>) || TIsUEnumClass<T>::Value == true)
		bool IsEquivalentDesc(const FPCGMetadataAttributeDesc& Desc)
		{
			return Desc.IsSingleValue() && IsSameType<T>(Desc.ValueType, Desc.ValueTypeObject);
		}

		template <typename Container> requires TIsTArray_V<Container>
		bool IsEquivalentDesc(const FPCGMetadataAttributeDesc& Desc)
		{
			using T = typename Container::ElementType;
			return Desc.IsArray() && IsSameType<T>(Desc.ValueType, Desc.ValueTypeObject);
		}

		template <typename Container> requires (TIsTSet_V<Container>)
		bool IsEquivalentDesc(const FPCGMetadataAttributeDesc& Desc)
		{
			using T = typename Container::ElementType;
			return Desc.IsSet() && IsSameType<T>(Desc.ValueType, Desc.ValueTypeObject);
		}

		template<typename Container> requires (TIsTMap_V<Container>)
		bool IsEquivalentDesc(const FPCGMetadataAttributeDesc& Desc)
		{
			using KeyType = typename TMapTypeExtractor<Container>::KeyType;
			using ValueType = typename TMapTypeExtractor<Container>::ValueType;

			return Desc.IsMap()
				&& IsSameType<KeyType>(Desc.KeyType, Desc.KeyTypeObject)
				&& IsSameType<ValueType>(Desc.ValueType, Desc.ValueTypeObject);
		}

		////////////////////////
		template <typename T> requires (TIsTObjectPtr_V<T> && std::is_base_of_v<UObject, typename T::ElementType> && !std::is_base_of_v<UClass, typename T::ElementType>)
		struct MetadataTypes<T>
		{
			enum { Id = static_cast<uint16>(EPCGMetadataTypes::Object)};
		};

		template <typename T> requires (TIsTObjectPtr_V<T> && std::is_base_of_v<UClass, typename T::ElementType>)
		struct MetadataTypes<T>
		{
			enum { Id = static_cast<uint16>(EPCGMetadataTypes::Class)};
		};
		
		template <typename T> requires UE::CTSoftObjectPtr<T>
		struct MetadataTypes<T>
		{
			enum { Id = static_cast<uint16>(EPCGMetadataTypes::SoftObject)};
		};
		
		template <typename T> requires UE::CTSoftClassPtr<T>
		struct MetadataTypes<T>
		{
			enum { Id = static_cast<uint16>(EPCGMetadataTypes::SoftClass)};
		};
		
		////////////////////////

		constexpr inline bool IsPCGType(uint16 TypeId)
		{
			return TypeId < (uint16)EPCGMetadataTypes::Count;
		}

		template <typename T>
		constexpr inline bool IsPCGType()
		{
			return IsPCGType(MetadataTypes<T>::Id);
		}

		// Verify if the TypeId is a type known by PCG and matches any types provided in the template.
		// Example: IsOfTypes<FVector2D, FVector, FVector4>(TypeId);
		template <typename ...Types>
		constexpr inline bool IsOfTypes(uint16 TypeId)
		{
			return (TypeId < (uint16)EPCGMetadataTypes::Unknown) && ((TypeId == MetadataTypes<Types>::Id) || ...);
		}

		// Verify if T is a type known by PCG and if it matches another type
		// in all provided types (in Types...)
		// Example: IsOfTypes<T, FVector2D, FVector, FVector4>();
		template <typename T, typename ...Types>
		constexpr inline bool IsOfTypes()
		{
			return IsOfTypes<Types...>(MetadataTypes<T>::Id);
		}

		inline FString GetTypeName(uint16 InType)
		{
			UEnum* PCGDataTypeEnum = StaticEnum<EPCGMetadataTypes>();
			return PCGDataTypeEnum ? PCGDataTypeEnum->GetNameStringByValue(InType) : FString(TEXT("Unknown"));
		}

		template <typename T>
		FString GetTypeName()
		{
			return GetTypeName(MetadataTypes<T>::Id);
		}

		inline FText GetTypeNameText(uint16 InType)
		{
			return FText::FromString(GetTypeName(InType));
		}

		template <typename T>
		FText GetTypeNameText()
		{
			return FText::FromString(GetTypeName<T>());
		}

		template <typename T>
		void Serialize(FArchive& Ar, T& A)
		{
			Ar << A;
		}

		// Serialize must be called on FSoftObjectPath, and this also covers FSoftClassPath.
		template <>
		inline void Serialize<FSoftObjectPath>(FArchive& Ar, FSoftObjectPath& A)
		{
			A.Serialize(Ar);
		}
		
		template <typename T>
		void Serialize(FArchive& Ar, const T& A)
		{
			ensure(!Ar.IsLoading());
			Serialize(Ar, const_cast<T&>(A));
		}

		// Wrapper around a standard 2-dimensional CArray that is constexpr, to know if a type is broadcastable to another.
		// First index is the original type, second index is the wanted type. Returns true if we can broadcast first type into second type.
		struct FBroadcastableTypes
		{
			constexpr FBroadcastableTypes()
				: Values{ {false} }
				, TransformFuncs { {nullptr} }
			{
				// All types are broadcastable to themselves.
				for (uint8 i = 0; i < (uint8)EPCGMetadataTypes::Count; ++i)
				{
					Values[i][i] = true;
				}

#define PCGMetadataBroadcastable(FirstType, SecondType) \
	Values[MetadataTypes<FirstType>::Id][MetadataTypes<SecondType>::Id] = true;\
	TransformFuncs[MetadataTypes<FirstType>::Id][MetadataTypes<SecondType>::Id] = &GetValuesWithBroadcast_Internal<FirstType, SecondType>
				
#define PCGMetadataBroadcastableBothWays(FirstType, SecondType) \
	PCGMetadataBroadcastable(FirstType, SecondType);\
	PCGMetadataBroadcastable(SecondType, FirstType)

				// Finally set all cases where it is broadcastable
				PCGMetadataBroadcastable(bool, int32);
				PCGMetadataBroadcastable(bool, int64);
				PCGMetadataBroadcastable(bool, float);
				PCGMetadataBroadcastable(bool, double);

				PCGMetadataBroadcastable(float, double);
				PCGMetadataBroadcastable(float, FVector2D);
				PCGMetadataBroadcastable(float, FVector);
				PCGMetadataBroadcastable(float, FVector4);

				PCGMetadataBroadcastable(double, FVector2D);
				PCGMetadataBroadcastable(double, FVector);
				PCGMetadataBroadcastable(double, FVector4);

				PCGMetadataBroadcastable(int32, float);
				PCGMetadataBroadcastable(int32, double);
				PCGMetadataBroadcastable(int32, int64);
				PCGMetadataBroadcastable(int32, FVector2D);
				PCGMetadataBroadcastable(int32, FVector);
				PCGMetadataBroadcastable(int32, FVector4);

				PCGMetadataBroadcastable(int64, float);
				PCGMetadataBroadcastable(int64, double);
				PCGMetadataBroadcastable(int64, FVector2D);
				PCGMetadataBroadcastable(int64, FVector);
				PCGMetadataBroadcastable(int64, FVector4);

				PCGMetadataBroadcastable(FVector2D, FVector);
				PCGMetadataBroadcastable(FVector2D, FVector4);

				PCGMetadataBroadcastable(FVector, FVector4);

				PCGMetadataBroadcastable(FQuat, FRotator);
				PCGMetadataBroadcastable(FRotator, FQuat);

				PCGMetadataBroadcastable(float, FString);
				PCGMetadataBroadcastable(double, FString);
				PCGMetadataBroadcastable(int32, FString);
				PCGMetadataBroadcastable(int64, FString);
				PCGMetadataBroadcastable(FVector2D, FString);
				PCGMetadataBroadcastable(FVector, FString);
				PCGMetadataBroadcastable(FVector4, FString);
				PCGMetadataBroadcastable(FQuat, FString);
				PCGMetadataBroadcastable(FTransform, FString);
				PCGMetadataBroadcastable(bool, FString);
				PCGMetadataBroadcastable(FRotator, FString);
				PCGMetadataBroadcastable(FName, FString);
				PCGMetadataBroadcastable(FString, FName);

				PCGMetadataBroadcastable(FSoftObjectPath, FString);
				PCGMetadataBroadcastable(FSoftClassPath, FString);
			
				PCGMetadataBroadcastableBothWays(TObjectPtr<UObject>, FString);
				PCGMetadataBroadcastableBothWays(TObjectPtr<UObject>, FSoftObjectPath);
				PCGMetadataBroadcastableBothWays(TObjectPtr<UObject>, TSoftObjectPtr<>);

				PCGMetadataBroadcastableBothWays(TObjectPtr<UClass>, FString);
				PCGMetadataBroadcastableBothWays(TObjectPtr<UClass>, FSoftClassPath);
				PCGMetadataBroadcastableBothWays(TObjectPtr<UClass>, TSoftClassPtr<>);
				
				PCGMetadataBroadcastableBothWays(TSoftObjectPtr<>, FString);
				PCGMetadataBroadcastableBothWays(TSoftObjectPtr<>, FSoftObjectPath);
				
				PCGMetadataBroadcastableBothWays(TSoftClassPtr<>, FString);
				PCGMetadataBroadcastableBothWays(TSoftClassPtr<>, FSoftClassPath);

#undef PCGMetadataBroadcastableBothWays
#undef PCGMetadataBroadcastable
			}

			constexpr const bool* operator[](int i) const { return Values[i]; }

			bool Values[(uint8)EPCGMetadataTypes::Count][(uint8)EPCGMetadataTypes::Count];

			FTransformFunc TransformFuncs[(uint8)EPCGMetadataTypes::Count][(uint8)EPCGMetadataTypes::Count];
		};

		// Wrapper around a standard 2-dimensional CArray that is constexpr, to know if a type is constructible from another.
		// First index is the wanted type, second index is the original type. Returns true if we can construct first type from second type.
		struct FConstructibleTypes
		{
		private:
			// Empty struct that will hold all our types.
			template <typename... T>
			struct TypeHolder {};

			// Use the macro to have AllTypes that is the Dummy struct templated with all our types.
			// Add void as a sentinel, to know when to stop unrolling (and because TypeHolder<T,U,> is syntax error)
#define PCG_ALL_TYPES(T) T,
			using AllTypes = TypeHolder<PCG_FOREACH_SUPPORTEDTYPES(PCG_ALL_TYPES) void>;
#undef PCG_ALL_TYPES

			// Unroll AllTypes twice, to have all combinations of types <T, U>
			template <typename OutputType> constexpr void UnrollSecond(TypeHolder<void> InTypeHolder) {}

			template <typename OutputType, typename InputType, typename... InputTypes>
			constexpr void UnrollSecond(TypeHolder<InputType, InputTypes...> InTypeHolder)
			{
				if constexpr (std::is_constructible_v<OutputType, InputType>)
				{
					Values[MetadataTypes<OutputType>::Id][MetadataTypes<InputType>::Id] = true;
					TransformFuncs[MetadataTypes<OutputType>::Id][MetadataTypes<InputType>::Id] = &GetValuesWithConstructible_Internal<InputType, OutputType>;
				}
				UnrollSecond<OutputType>(TypeHolder<InputTypes...>{});
			}

			template <typename OutputType, typename... OutputTypes>
			constexpr void UnrollFirst(TypeHolder<OutputType, OutputTypes...> InTypeHolder)
			{
				UnrollSecond<OutputType>(AllTypes{});
				UnrollFirst(TypeHolder<OutputTypes...>{});
			}

			constexpr void UnrollFirst(TypeHolder<void> InTypeHolder) {}

		public:
			constexpr FConstructibleTypes()
				: Values{ {false} }
				, TransformFuncs{ {nullptr} }
			{
				UnrollFirst(AllTypes{});
			}

			constexpr const bool* operator[](int i) const { return Values[i]; }

			// First type = OutputType, Second type = Input type. Can I construct OutputType from InputType.
			bool Values[(uint8)EPCGMetadataTypes::Count][(uint8)EPCGMetadataTypes::Count];
			FTransformFunc TransformFuncs[(uint8)EPCGMetadataTypes::Count][(uint8)EPCGMetadataTypes::Count];
		};

		// Wrapper around a standard 1-dimensional CArray that is constexpr, to know how to construct/destruct/element size
		struct FTypesFundamental
		{
		private:
			// Empty struct that will hold all our types.
			template <typename... T>
			struct TypeHolder {};

			// Use the macro to have AllTypes that is the Dummy struct templated with all our types.
#define PCG_ALL_TYPES(T) T,
			using AllTypes = TypeHolder<PCG_FOREACH_SUPPORTEDTYPES(PCG_ALL_TYPES) TObjectPtr<UObject>, TObjectPtr<UClass>, TSoftObjectPtr<>, TSoftClassPtr<>>;
#undef PCG_ALL_TYPES

			template <typename InputType, typename... InputTypes>
			constexpr void Unroll(TypeHolder<InputType, InputTypes...> InTypeHolder)
			{
				ElementSize[MetadataTypes<InputType>::Id] = sizeof(InputType);
				AlignmentSize[MetadataTypes<InputType>::Id] = alignof(InputType);
				CanBeMemcpy[MetadataTypes<InputType>::Id] = std::is_trivially_copyable_v<InputType>;
				if constexpr (!std::is_trivially_constructible_v<InputType> || !std::is_trivially_destructible_v<InputType>)
				{
					ConstructFuncs[MetadataTypes<InputType>::Id] = &ConstructMemory<InputType>;
					DestroyFuncs[MetadataTypes<InputType>::Id] = &DestructMemory<InputType>;
				}

				Unroll(TypeHolder<InputTypes...>{});
			}

			constexpr void Unroll(TypeHolder<> InTypeHolder) {}

		public:
			constexpr FTypesFundamental()
				: ElementSize{{}}
				, AlignmentSize{{}}
				, CanBeMemcpy{{}}
				, ConstructFuncs{{}}
				, DestroyFuncs{{}}
			{
				Unroll(AllTypes{});
			}

			size_t ElementSize[(uint8)EPCGMetadataTypes::Count];
			size_t AlignmentSize[(uint8)EPCGMetadataTypes::Count];
			bool CanBeMemcpy[(uint8)EPCGMetadataTypes::Count];
			FConstructMemoryFunc ConstructFuncs[(uint8)EPCGMetadataTypes::Count];
			FDestroyMemoryFunc DestroyFuncs[(uint8)EPCGMetadataTypes::Count];
		};

		inline static constexpr FBroadcastableTypes BroadcastableTypes{};
		inline static constexpr FConstructibleTypes ConstructibleTypes{};
		inline static constexpr FTypesFundamental TypesFundamental{};

		constexpr inline bool IsBroadcastable(uint16 FirstType, uint16 SecondType)
		{
			// Unknown types aren't broadcastable
			if (FirstType >= static_cast<uint16>(EPCGMetadataTypes::Count) || SecondType >= static_cast<uint16>(EPCGMetadataTypes::Count))
			{
				return false;
			}

			return BroadcastableTypes[FirstType][SecondType];
		}

		inline bool IsBroadcastable(const FPCGMetadataAttributeDesc& FirstType, const FPCGMetadataAttributeDesc& SecondType)
		{
			// If it is the same type, it is always possible
			if (FirstType.IsSameType(SecondType))
			{
				return true;
			}
			
			// Cross container is not allowed.
			if (FirstType.ContainerTypes != SecondType.ContainerTypes)
			{
				return false;
			}

			// Set/Map are not broadcastable if the types are not matching
			if (FirstType.IsSet() || FirstType.IsMap())
			{
				return false;
			}

			return IsBroadcastable(static_cast<int16>(FirstType.ValueType), static_cast<int16>(SecondType.ValueType));
		}

		template <typename FirstType, typename SecondType>
		constexpr inline bool IsBroadcastable()
		{
			return IsBroadcastable(MetadataTypes<FirstType>::Id, MetadataTypes<SecondType>::Id);
		}

		constexpr inline bool IsConstructible(uint16 FirstType, uint16 SecondType)
		{
			// Unknown types can't be checked for constructible.
			if (FirstType >= static_cast<uint16>(EPCGMetadataTypes::Count) || SecondType >= static_cast<uint16>(EPCGMetadataTypes::Count))
			{
				return false;
			}

			return ConstructibleTypes[FirstType][SecondType];
		}

		inline bool IsConstructible(const FPCGMetadataAttributeDesc& FirstType, const FPCGMetadataAttributeDesc& SecondType)
		{
			// If it is the same type, it is always possible
			if (FirstType.IsSameType(SecondType))
			{
				return true;
			}

			// Cross container is not allowed.
			if (FirstType.ContainerTypes != SecondType.ContainerTypes)
			{
				return false;
			}

			// Set/Map are not constructible if the types are not matching
			if (FirstType.IsSet() || FirstType.IsMap())
			{
				return false;
			}

			return IsConstructible(static_cast<int16>(FirstType.ValueType), static_cast<int16>(SecondType.ValueType));
		}

		// Convenience function for accessor operation, to know if a Get will succeed with AllowBroadcast and AllowConstructible
		// where the SecondType is the target type, and FirstType is the accessor type.
		// ie. Accessor is an int, Accessor->Get<float>(...). First type is int, second type is float.
		template <typename FirstType, typename SecondType>
		constexpr inline bool IsBroadcastableOrConstructible()
		{
			return IsBroadcastable<FirstType, SecondType>() || std::is_constructible_v<SecondType, FirstType>;
		}

		// Convenience function for accessor operation, to know if a Get will succeed with AllowBroadcast and AllowConstructible
		// where the SecondType is the target type, and FirstType is the accessor type.
		// ie. Accessor is an int, Accessor->Get<float>(...). First type is int, second type is float.
		constexpr inline bool IsBroadcastableOrConstructible(uint16 FirstType, uint16 SecondType)
		{
			return IsBroadcastable(FirstType, SecondType) || IsConstructible(SecondType, FirstType);
		}

		// Convenience function for accessor operation, to know if a Get will succeed with AllowBroadcast and AllowConstructible
		// where the SecondType is the target type, and FirstType is the accessor type.
		// ie. Accessor is an int, Accessor->Get<float>(...). First type is int, second type is float.
		inline bool IsBroadcastableOrConstructible(const FPCGMetadataAttributeDesc& FirstType, const FPCGMetadataAttributeDesc& SecondType)
		{
			return IsBroadcastable(FirstType, SecondType) || IsConstructible(SecondType, FirstType);
		}

		/**
		* FirstType is more complex than SecondType, in the sense of PCG types, if both types are different and valid
		* and we can broadcast SecondType to FirstType.
		* It is useful to find the most complex type between operands of a given operation.
		*/
		constexpr inline bool IsMoreComplexType(uint16 FirstType, uint16 SecondType)
		{
			return FirstType != SecondType && FirstType <= (uint16)(EPCGMetadataTypes::Count) && SecondType <= (uint16)(EPCGMetadataTypes::Count) && PCG::Private::BroadcastableTypes[SecondType][FirstType];
		}

		/**
		* cf. non templated version of IsMoreComplexType
		*/
		template <typename FirstType, typename SecondType>
		constexpr inline bool IsMoreComplexType()
		{
			return IsMoreComplexType(MetadataTypes<FirstType>::Id, MetadataTypes<SecondType>::Id);
		}

		inline size_t GetElementSize(uint16 TypeId)
		{
			if (TypeId >= static_cast<uint16>(EPCGMetadataTypes::Count))
			{
				return static_cast<size_t>(-1);
			}

			return TypesFundamental.ElementSize[TypeId];
		}
		
		inline size_t GetAlignmentSize(uint16 TypeId)
		{
			if (TypeId >= static_cast<uint16>(EPCGMetadataTypes::Count))
			{
				return static_cast<size_t>(-1);
			}

			return TypesFundamental.AlignmentSize[TypeId];
		}
		
		inline bool CanBeMemcpy(uint16 TypeId)
		{
			if (TypeId >= static_cast<uint16>(EPCGMetadataTypes::Count))
			{
				return false;
			}

			return TypesFundamental.CanBeMemcpy[TypeId];
		}

		inline FConstructMemoryFunc GetConstructMemoryFunc(uint16 TypeId)
		{
			if (TypeId >= static_cast<uint16>(EPCGMetadataTypes::Count))
			{
				return nullptr;
			}

			return TypesFundamental.ConstructFuncs[TypeId];
		}

		inline FDestroyMemoryFunc GetDestroyMemoryFunc(uint16 TypeId)
		{
			if (TypeId >= static_cast<uint16>(EPCGMetadataTypes::Count))
			{
				return nullptr;
			}

			return TypesFundamental.DestroyFuncs[TypeId];
		}

		inline FTransformFunc GetBroadcastTransformFunc(uint16 FirstType, uint16 SecondType)
		{
			if (FirstType >= static_cast<uint16>(EPCGMetadataTypes::Count) || SecondType >= static_cast<uint16>(EPCGMetadataTypes::Count))
			{
				return nullptr;
			}

			return BroadcastableTypes.TransformFuncs[FirstType][SecondType];
		}

		inline FTransformFunc GetConstructibleTransformFunc(uint16 FirstType, uint16 SecondType)
		{
			if (FirstType >= static_cast<uint16>(EPCGMetadataTypes::Count) || SecondType >= static_cast<uint16>(EPCGMetadataTypes::Count))
			{
				return nullptr;
			}

			return ConstructibleTypes.TransformFuncs[FirstType][SecondType];
		}

		/**
		* Traits
		* - CanCompare: Supports less than, greater than etc for ordering.
		* - CanSearchString: Can do string search operations like matching substrings.
		* - CompressData: Uses value keys, so if two values are identical, we only store one.
		* - NeedsConstruction: Whether type requires constructor for valid state (i.e. can't simply be zero initialized).
		* - IsFloatingPoint: Whether the type itself or the container's value type is floating point.
		*/

		template<typename T>
		struct DefaultStringTraits
		{
			enum { IsFloatingPoint = false };

			static FString ToString(const T& A)
			{
				return A.ToString();
			}

			static T ParseFromString(const FString& InString, bool& bOutSuccess)
			{
				bOutSuccess = true;
				return T(InString);
			}

			static FString ZeroValueString();
		};

		template<typename T>
		struct LexToStringTraits
		{
			static FString ToString(const T& A)
			{
				return LexToString(A);
			}

			static FString ZeroValueString();
		};

		template<typename T>
		struct DefaultOperationTraits
		{
			enum { CanSubAdd = true };
			enum { CanMulDiv = true };
			enum { IsFloatingPoint = false };

			static bool Equal(const T& A, const T& B)
			{
				return A == B;
			}

			static T Sub(const T& A, const T& B)
			{
				return A - B;
			}

			static T Add(const T& A, const T& B)
			{
				return A + B;
			}

			static T Mul(const T& A, const T& B)
			{
				return A * B;
			}

			static T Div(const T& A, const T& B)
			{
				return A / B;
			}
		};

		template<typename T>
		struct DefaultCompareTraits
		{
			enum { CanCompare = true };

			static bool Less(const T& A, const T& B)
			{
				return A < B;
			}

			static bool Greater(const T& A, const T& B)
			{
				return A > B;
			}

			static bool LessOrEqual(const T& A, const T& B)
			{
				return A <= B;
			}

			static bool GreaterOrEqual(const T& A, const T& B)
			{
				return A >= B;
			}
		};

		template<typename T>
		struct DefaultHashTraits
		{
			enum { CanHash = true };

			static uint32 Hash(const T& A)
			{
				return GetTypeHash(A);
			}
		};

		/** "Find Nearest" trait support driven by less/less or equal operation, used for types that do not support distance computation */
		template<typename Traits, typename T>
		struct CompareBasedFindNearestTraits
		{
			enum { CanFindNearest = true};
			enum { CanComputeDistance = false };

			/** Returns true if A is strictly closer to Target than B is. */
			static bool IsCloserTo(const T& A, const T& B, const T& Target)
			{
				// If A is "between" B and Target, then it is closer, e.g. B < A <= Target or Target <= A < B
				return (Traits::Less(B, A) && Traits::LessOrEqual(A, Target)) || (Traits::LessOrEqual(Target, A) && Traits::Less(A, B));
			}
		};

		/** "Find Nearest" trait support driven by distance comparison, used for types that support distance computation */
		template<typename Traits, typename T>
		struct DistanceBasedFindNearestTraits
		{
			enum { CanFindNearest = true };
			enum { CanComputeDistance = true };

			/** Returns true if A is strictly closer to Target than B is. */
			static bool IsCloserTo(const T& A, const T& B, const T& Target)
			{
				return Traits::Distance(A, Target) < Traits::Distance(B, Target);
			}
		};

		template<typename T>
		struct DefaultWeightedSumTraits
		{
			enum { CanInterpolate = true };
			enum { InterpolationNeedsNormalization = false };

			static T WeightedSum(const T& A, const T& B, float Weight)
			{
				return A + B * Weight;
			}

			static T ZeroValue()
			{
				return T{};
			}

			static T ZeroValueForWeightedSum()
			{
				return ZeroValue();
			}
		};

		template <typename T>
		struct DefaultMinMaxTraits
		{
			enum { CanMinMax = true };

			static T Min(const T& A, const T& B)
			{
				return FMath::Min(A, B);
			}

			static T Max(const T& A, const T& B)
			{
				return FMath::Max(A, B);
			}
		};

		template <typename T>
		struct DefaultScalarDistanceTraits : DistanceBasedFindNearestTraits<DefaultScalarDistanceTraits<T>, T>
		{
			using DistanceType = T;

			static DistanceType Distance(const T& A, const T& B)
			{
				return FMath::Abs(A - B);
			}
		};

		template<typename T>
		struct MetadataTraits : DefaultOperationTraits<T>, DefaultWeightedSumTraits<T>, DefaultMinMaxTraits<T>, DefaultCompareTraits<T>, DefaultHashTraits<T>, LexToStringTraits<T>, DefaultScalarDistanceTraits<T>
		{
			enum { CompressData = false };
			enum { CanSearchString = false };
			enum { NeedsConstruction = false };
		};

		template<>
		struct MetadataTraits<int32> : DefaultOperationTraits<int32>, DefaultWeightedSumTraits<int32>, DefaultMinMaxTraits<int32>, DefaultCompareTraits<int32>, DefaultHashTraits<int32>, LexToStringTraits<int32>, DefaultScalarDistanceTraits<int32>
		{
			enum { CompressData = false };
			enum { CanSearchString = false };
			enum { NeedsConstruction = false };
			enum { CanNormalize = false };

			static int32 ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				bOutSuccess = FDefaultValueHelper::IsStringValidInteger(InValue);
				// FDefaultValueHelper::ParseInt/ParseInt64 have this weird behavior where numbers starting with 0, and not followed by 'x' are treated as base 8.
				// So do our own parsing.
				return bOutSuccess ? FCString::Strtoi(*InValue, nullptr, InValue.StartsWith(TEXT("0x")) ? 16 : 10) : 0;
			}
		};

		template<>
		struct MetadataTraits<int64> : DefaultOperationTraits<int64>, DefaultWeightedSumTraits<int64>, DefaultMinMaxTraits<int64>, DefaultCompareTraits<int64>, DefaultHashTraits<int64>, LexToStringTraits<int64>, DefaultScalarDistanceTraits<int64>
		{
			enum { CompressData = false };
			enum { CanSearchString = false };
			enum { NeedsConstruction = false };
			enum { CanNormalize = false };

			static int64 ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				bOutSuccess = FDefaultValueHelper::IsStringValidInteger(InValue);
				// cf int32
				return bOutSuccess ? FCString::Strtoi64(*InValue, nullptr, InValue.StartsWith(TEXT("0x")) ? 16 : 10) : 0;
			}
		};

		template<>
		struct MetadataTraits<float> : DefaultOperationTraits<float>, DefaultWeightedSumTraits<float>, DefaultMinMaxTraits<float>, DefaultCompareTraits<float>, DefaultHashTraits<float>, LexToStringTraits<float>, DefaultScalarDistanceTraits<float>
		{
			enum { CompressData = false };
			enum { CanSearchString = false };
			enum { NeedsConstruction = false };
			enum { IsFloatingPoint = true };
			enum { CanNormalize = false };

			static bool Equal(const float& A, const float& B, const double Tolerance = UE_SMALL_NUMBER)
			{
				return FMath::IsNearlyEqual(A, B, Tolerance);
			}

			static float ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				float OutValue;
				bOutSuccess = FDefaultValueHelper::ParseFloat(InValue, OutValue);
				return bOutSuccess ? OutValue : 0.0f;
			}
		};

		template<>
		struct MetadataTraits<double> : DefaultOperationTraits<double>, DefaultWeightedSumTraits<double>, DefaultMinMaxTraits<double>, DefaultCompareTraits<double>, DefaultHashTraits<double>, LexToStringTraits<double>, DefaultScalarDistanceTraits<double>
		{
			enum { CompressData = false };
			enum { CanSearchString = false };
			enum { NeedsConstruction = false };
			enum { IsFloatingPoint = true };
			enum { CanNormalize = false };

			static bool Equal(const double& A, const double& B, const double Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				return FMath::IsNearlyEqual(A, B, Tolerance);
			}

			static double ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				double OutValue;
				bOutSuccess = FDefaultValueHelper::ParseDouble(InValue, OutValue);
				return bOutSuccess ? OutValue : 0.0;
			}
		};

		template<>
		struct MetadataTraits<bool> : LexToStringTraits<bool>, DefaultHashTraits<bool>
		{
			enum { CompressData = false };
			enum { CanMinMax = true };
			enum { CanSubAdd = true };
			enum { CanMulDiv = false };
			enum { CanInterpolate = false };
			enum { CanNormalize = false };
			enum { CanCompare = true };
			enum { CanFindNearest = false };
			enum { CanComputeDistance = false };
			enum { CanSearchString = false };
			enum { NeedsConstruction = false };
			enum { IsFloatingPoint = false };

			static bool ZeroValue()
			{
				return false;
			}

			static bool Equal(const bool& A, const bool& B)
			{
				return A == B;
			}

			static bool Min(const bool& A, const bool& B)
			{
				return A && B;
			}

			static bool Max(const bool& A, const bool& B)
			{
				return A || B;
			}

			static bool Add(const bool& A, const bool& B)
			{
				return A || B;
			}

			static bool Sub(const bool& A, const bool& B)
			{
				return A && !B;
			}

			static bool Less(const bool& A, const bool& B)
			{
				return !A && B;
			}

			static bool Greater(const bool& A, const bool& B)
			{
				return A && !B;
			}

			static bool LessOrEqual(const bool& A, const bool& B)
			{
				return !Greater(A, B);
			}

			static bool GreaterOrEqual(const bool& A, const bool& B)
			{
				return !Less(A, B);
			}

			static bool ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				bOutSuccess = true;
				return FCString::ToBool(*InValue);
			}
		};

		// Vector types
		template<typename T>
		struct VectorTraits : DefaultOperationTraits<T>, DefaultStringTraits<T>
		{
			enum { CompressData = false };
			enum { CanMinMax = true };
			enum { CanInterpolate = true };
			enum { InterpolationNeedsNormalization = false };
			enum { CanNormalize = true };
			enum { CanCompare = true };
			enum { CanSearchString = false };
			enum { NeedsConstruction = false };
			enum { CanFindNearest = true };
			enum { CanComputeDistance = true };
			enum { IsFloatingPoint = true };

			using DistanceType = typename T::FReal;

			static T ZeroValue()
			{
				return T::Zero();
			}

			static T ZeroValueForWeightedSum()
			{
				return ZeroValue();
			}

			static T WeightedSum(const T& A, const T& B, float Weight)
			{
				return A + B * Weight;
			}

			static DistanceType Distance(const T& A, const T& B)
			{
				return (A - B).Size();
			}

			/** Returns true if A is strictly closer to Target than B is. Implementation note: we don't use the standard distance-based traits to have a more efficient check here. */
			static bool IsCloserTo(const T& A, const T& B, const T& Target)
			{
				return (A - Target).SizeSquared() < (B - Target).SizeSquared();
			}
		};

		template<>
		struct MetadataTraits<FVector2D> : VectorTraits<FVector2D>, DefaultHashTraits<FVector2D>
		{
			static bool Equal(const FVector2D& A, const FVector2D& B, const FVector2D::FReal Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				return A.Equals(B, Tolerance);
			}

			static FVector2D Min(const FVector2D& A, const FVector2D& B)
			{
				return FVector2D::Min(A, B);
			}

			static FVector2D Max(const FVector2D& A, const FVector2D& B)
			{
				return FVector2D::Max(A, B);
			}

			static bool Less(const FVector2D& A, const FVector2D& B)
			{
				return (A.X < B.X) && (A.Y < B.Y);
			}

			static bool Greater(const FVector2D& A, const FVector2D& B)
			{
				return (A.X > B.X) && (A.Y > B.Y);
			}

			static bool LessOrEqual(const FVector2D& A, const FVector2D& B)
			{
				return (A.X <= B.X) && (A.Y <= B.Y);
			}

			static bool GreaterOrEqual(const FVector2D& A, const FVector2D& B)
			{
				return (A.X >= B.X) && (A.Y >= B.Y);
			}

			static FVector2D ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				FVector2D OutValue;
				// FDefaultValueHelper::ParseVector expect commas, even with X=... Y=..., which is not the format of ToString. So try to init from string if it fails.
				bOutSuccess = FDefaultValueHelper::ParseVector2D(InValue, OutValue) || OutValue.InitFromString(InValue);
				return bOutSuccess ? OutValue : FVector2D::ZeroVector;
			}

			static void Normalize(FVector2D& InValue, const FVector2D::FReal Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				InValue.Normalize(Tolerance);
			}
		};

		template<>
		struct MetadataTraits<FVector> : VectorTraits<FVector>, DefaultHashTraits<FVector>
		{
			static bool Equal(const FVector& A, const FVector& B, const FVector::FReal Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				return A.Equals(B, Tolerance);
			}

			static FVector Min(const FVector& A, const FVector& B)
			{
				return FVector::Min(A, B);
			}

			static FVector Max(const FVector& A, const FVector& B)
			{
				return FVector::Max(A, B);
			}

			static bool Less(const FVector& A, const FVector& B)
			{
				return (A.X < B.X) && (A.Y < B.Y) && (A.Z < B.Z);
			}

			static bool Greater(const FVector& A, const FVector& B)
			{
				return (A.X > B.X) && (A.Y > B.Y) && (A.Z > B.Z);
			}

			static bool LessOrEqual(const FVector& A, const FVector& B)
			{
				return (A.X <= B.X) && (A.Y <= B.Y) && (A.Z <= B.Z);
			}

			static bool GreaterOrEqual(const FVector& A, const FVector& B)
			{
				return (A.X >= B.X) && (A.Y >= B.Y) && (A.Z >= B.Z);
			}

			static FVector ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				FVector OutValue;
				// cf Vector2D
				bOutSuccess = FDefaultValueHelper::ParseVector(InValue, OutValue) || OutValue.InitFromString(InValue);
				return bOutSuccess ? OutValue : FVector::ZeroVector;
			}

			static void Normalize(FVector& InValue, const FVector::FReal Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				InValue.Normalize(Tolerance);
			}
		};

		template<>
		struct MetadataTraits<FVector4> : VectorTraits<FVector4>, DefaultHashTraits<FVector4>
		{
			static bool Equal(const FVector4& A, const FVector4& B, const FVector4::FReal Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				return A.Equals(B, Tolerance);
			}

			static FVector4 Min(const FVector4& A, const FVector4& B)
			{
				return FVector4(FMath::Min(A.X, B.X), FMath::Min(A.Y, B.Y), FMath::Min(A.Z, B.Z), FMath::Min(A.W, B.W));
			}

			static FVector4 Max(const FVector4& A, const FVector4& B)
			{
				return FVector4(FMath::Max(A.X, B.X), FMath::Max(A.Y, B.Y), FMath::Max(A.Z, B.Z), FMath::Max(A.W, B.W));
			}

			static bool Less(const FVector4& A, const FVector4& B)
			{
				return (A.X < B.X) && (A.Y < B.Y) && (A.Z < B.Z) && (A.W < B.W);
			}

			static bool Greater(const FVector4& A, const FVector4& B)
			{
				return (A.X > B.X) && (A.Y > B.Y) && (A.Z > B.Z) && (A.W > B.W);
			}

			static bool LessOrEqual(const FVector4& A, const FVector4& B)
			{
				return (A.X <= B.X) && (A.Y <= B.Y) && (A.Z <= B.Z) && (A.W <= B.W);
			}

			static bool GreaterOrEqual(const FVector4& A, const FVector4& B)
			{
				return (A.X >= B.X) && (A.Y >= B.Y) && (A.Z >= B.Z) && (A.W >= B.W);
			}

			static FVector4 ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				FVector4 OutValue;
				// cf Vector2D
				bOutSuccess = FDefaultValueHelper::ParseVector4(InValue, OutValue) || OutValue.InitFromString(InValue);
				return bOutSuccess ? OutValue : FVector4::Zero();
			}

			static void Normalize(FVector4& InValue, const FVector4::FReal Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				InValue = InValue.GetSafeNormal(Tolerance);
			}
		};

		// Quaternion
		template<>
		struct MetadataTraits<FQuat> : DefaultStringTraits<FQuat>, DistanceBasedFindNearestTraits<MetadataTraits<FQuat>, FQuat>, DefaultHashTraits<FQuat>
		{
			enum { CompressData = false };
			enum { CanMinMax = false };
			enum { CanSubAdd = true };
			enum { CanMulDiv = true };
			enum { CanInterpolate = true };
			enum { InterpolationNeedsNormalization = true };
			enum { CanCompare = false };
			enum { CanSearchString = false };
			enum { NeedsConstruction = false };
			enum { IsFloatingPoint = true };
			enum { CanNormalize = true };

			using DistanceType = typename FQuat::FReal;

			static bool Equal(const FQuat& A, const FQuat& B, const DistanceType Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				return A.Equals(B, Tolerance);
			}

			static FQuat Add(const FQuat& A, const FQuat& B)
			{
				return A * B;
			}

			static FQuat Sub(const FQuat& A, const FQuat& B)
			{
				return A * B.Inverse();
			}

			static FQuat Mul(const FQuat& A, const FQuat& B)
			{
				return A * B;
			}

			static FQuat Div(const FQuat& A, const FQuat& B)
			{
				return A * B.Inverse();
			}

			static FQuat WeightedSum(const FQuat& A, const FQuat& B, float Weight)
			{
				// WARNING: the quaternion won't be normalized
				FQuat BlendQuat = B * Weight;

				if ((A | BlendQuat) >= 0.0f)
					return A + BlendQuat;
				else
					return A - BlendQuat;
			}

			static FQuat ZeroValue()
			{
				return FQuat::Identity;
			}

			static FQuat ZeroValueForWeightedSum()
			{
				return FQuat(0.0, 0.0, 0.0, 0.0);
			}

			static FQuat::FReal Distance(const FQuat& A, const FQuat& B)
			{
				return A.AngularDistance(B);
			}

			static FQuat ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				FVector4 Temp;
				// cf Vector2D
				bOutSuccess = FDefaultValueHelper::ParseVector4(InValue, Temp) || Temp.InitFromString(InValue);
				return bOutSuccess ? FQuat(Temp.X, Temp.Y, Temp.Z, Temp.W) : FQuat::Identity;
			}

			static void Normalize(FQuat& InValue, const FQuat::FReal Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				InValue.Normalize(Tolerance);
			}
		};

		// Rotator
		template<>
		struct MetadataTraits<FRotator> : DefaultStringTraits<FRotator>, DistanceBasedFindNearestTraits<MetadataTraits<FRotator>, FRotator>, DefaultHashTraits<FRotator>
		{
			enum { CompressData = false };
			enum { CanMinMax = false };
			enum { CanSubAdd = true };
			enum { CanMulDiv = true };
			enum { CanInterpolate = true };
			enum { CanNormalize = true };
			enum { InterpolationNeedsNormalization = true };
			enum { CanCompare = false };
			enum { CanSearchString = false };
			enum { NeedsConstruction = false };
			enum { IsFloatingPoint = true };

			using DistanceType = typename FRotator::FReal;

			static bool Equal(const FRotator& A, const FRotator& B, const DistanceType Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				return A.Equals(B, Tolerance);
			}

			static FRotator Add(const FRotator& A, const FRotator& B)
			{
				return A + B;
			}

			static FRotator Sub(const FRotator& A, const FRotator& B)
			{
				return A - B;
			}

			static FRotator Mul(const FRotator& A, const FRotator& B)
			{
				return A + B;
			}

			static FRotator Div(const FRotator& A, const FRotator& B)
			{
				return A - B;
			}

			static FRotator WeightedSum(const FRotator& A, const FRotator& B, float Weight)
			{
				// TODO review this, should we use TCustomLerp<UE::Math::TRotator<T>> ?
				return A + (B * Weight);
			}

			static FRotator ZeroValue()
			{
				return FRotator::ZeroRotator;
			}

			static FRotator ZeroValueForWeightedSum()
			{
				return ZeroValue();
			}

			static FRotator::FReal Distance(const FRotator& A, const FRotator& B)
			{
				// Use quaternions for distance calculation, but return degrees instead of radians here
				return 180.0 * FQuat(A).AngularDistance(FQuat(B)) / UE_DOUBLE_PI;
			}

			static FRotator ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				FRotator OutValue;
				bOutSuccess = FDefaultValueHelper::ParseRotator(InValue, OutValue);
				return bOutSuccess ? OutValue : FRotator::ZeroRotator;
			}

			static uint32 Hash(const FRotator& A)
			{
				return GetTypeHash(FVector(A.Pitch, A.Yaw, A.Roll));
			}

			static void Normalize(FRotator& InValue, const FRotator::FReal Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				InValue.Normalize();
			}
		};

		// Transform
		template<>
		struct MetadataTraits<FTransform> : DefaultStringTraits<FTransform>, DefaultHashTraits<FTransform>
		{
			enum { CompressData = false };
			enum { CanMinMax = false };
			enum { CanSubAdd = false };
			enum { CanMulDiv = true };
			enum { CanInterpolate = true };
			enum { CanNormalize = true };
			enum { InterpolationNeedsNormalization = true };
			enum { CanCompare = false };
			enum { CanFindNearest = false };
			enum { CanComputeDistance = false };
			enum { CanSearchString = false };
			enum { NeedsConstruction = false };
			enum { IsFloatingPoint = true };

			static bool Equal(const FTransform& A, const FTransform& B, const FTransform::FReal Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				return A.Equals(B, Tolerance);
			}

			static FTransform Mul(const FTransform& A, const FTransform& B)
			{
				return A * B;
			}

			static FTransform Div(const FTransform& A, const FTransform& B)
			{
				return A * B.Inverse();
			}

			static FTransform WeightedSum(const FTransform& A, const FTransform& B, float Weight)
			{
				// WARNING: the rotation won't be normalized
				return FTransform(
					MetadataTraits<FQuat>::WeightedSum(A.GetRotation(), B.GetRotation(), Weight),
					MetadataTraits<FVector>::WeightedSum(A.GetLocation(), B.GetLocation(), Weight),
					MetadataTraits<FVector>::WeightedSum(A.GetScale3D(), B.GetScale3D(), Weight));
			}

			static FTransform ZeroValue()
			{
				return FTransform::Identity;
			}

			static FTransform ZeroValueForWeightedSum()
			{
				return FTransform(FQuat(0.0, 0.0, 0.0, 0.0), FVector::ZeroVector, FVector::ZeroVector);
			}

			static FTransform ParseFromString(const FString& InValue, bool& bOutSuccess)
			{
				FTransform OutValue;
				bOutSuccess = OutValue.InitFromString(InValue);
				return bOutSuccess ? OutValue : FTransform::Identity;
			}

			static void Normalize(FTransform& InValue, const FTransform::FReal Tolerance = UE_DOUBLE_SMALL_NUMBER)
			{
				InValue.SetRotation(InValue.GetRotation().GetNormalized(Tolerance));
			}
		};

		// Strings
		template<>
		struct MetadataTraits<FString> : DefaultCompareTraits<FString>, DefaultStringTraits<FString>, CompareBasedFindNearestTraits<MetadataTraits<FString>, FString>, DefaultHashTraits<FString>
		{
			enum { CompressData = true };
			enum { CanMinMax = false };
			enum { CanSubAdd = false };
			enum { CanMulDiv = false };
			enum { CanInterpolate = false };
			enum { CanNormalize = false };
			enum { InterpolationNeedsNormalization = false };
			enum { CanSearchString = true };
			enum { NeedsConstruction = true };

			static FString ToString(const FString& A)
			{
				return A;
			}

			static bool Equal(const FString& A, const FString& B)
			{
				return A == B;
			}

			static FString ZeroValue()
			{
				return FString();
			}

			static bool Substring(const FString& A, const FString& B, const ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
			{
				return A.Contains(B, SearchCase);
			}

			static bool Matches(const FString& A, const FString& B, const ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
			{
				return A.MatchesWildcard(B, SearchCase);
			}
		};

		template<>
		struct MetadataTraits<FName> : DefaultStringTraits<FName>, CompareBasedFindNearestTraits<MetadataTraits<FName>, FName>, DefaultHashTraits<FName>
		{
			enum { CompressData = false };
			enum { CanMinMax = false };
			enum { CanSubAdd = false };
			enum { CanMulDiv = false };
			enum { CanInterpolate = false };
			enum { CanNormalize = false };
			enum { InterpolationNeedsNormalization = false };
			enum { CanCompare = true };
			enum { CanSearchString = true };
			enum { NeedsConstruction = false };

			static bool Equal(const FName& A, const FName& B)
			{
				return A == B;
			}

			static bool Less(const FName& A, const FName& B)
			{
				return A.Compare(B) < 0;
			}

			static bool Greater(const FName& A, const FName& B)
			{
				return A.Compare(B) > 0;
			}

			static bool LessOrEqual(const FName& A, const FName& B)
			{
				return A.Compare(B) <= 0;
			}

			static bool GreaterOrEqual(const FName& A, const FName& B)
			{
				return A.Compare(B) >= 0;
			}

			static FName ZeroValue()
			{
				return NAME_None;
			}

			static bool Substring(const FName& A, const FName& B, const ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
			{
				return A.ToString().Contains(B.ToString(), SearchCase);
			}

			static bool Matches(const FName& A, const FName& B, const ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
			{
				return A.ToString().MatchesWildcard(B.ToString(), SearchCase);
			}
		};

		template<typename T>
		struct SoftObjectPathTraits : CompareBasedFindNearestTraits<SoftObjectPathTraits<T>, T>
		{
			enum { CompressData = true };
			enum { CanMinMax = false };
			enum { CanSubAdd = false };
			enum { CanMulDiv = false };
			enum { CanInterpolate = false };
			enum { CanNormalize = false };
			enum { InterpolationNeedsNormalization = false };
			enum { CanSearchString = true };
			// Contains FString objects
			enum { NeedsConstruction = true };
			enum { CanCompare = true };

			static bool Equal(const T& A, const T& B)
			{
				return A == B;
			}

			static T ZeroValue()
			{
				return T();
			}

			static bool Substring(const T& A, const T& B, const ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
			{
				return A.ToString().Contains(B.ToString(), SearchCase);
			}

			static bool Matches(const T& A, const T& B, const ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
			{
				return A.ToString().MatchesWildcard(B.ToString(), SearchCase);
			}

			static bool Less(const T& A, const T& B)
			{
				return A.ToString() < B.ToString();
			}

			static bool Greater(const T& A, const T& B)
			{
				return A.ToString() > B.ToString();
			}

			static bool LessOrEqual(const T& A, const T& B)
			{
				return A.ToString() <= B.ToString();
			}

			static bool GreaterOrEqual(const T& A, const T& B)
			{
				return A.ToString() >= B.ToString();
			}
		};

		// Soft object paths
		template<>
		struct MetadataTraits<FSoftObjectPath> : SoftObjectPathTraits<FSoftObjectPath>, DefaultStringTraits<FSoftObjectPath>, DefaultHashTraits<FSoftObjectPath>
		{
		};

		// Soft class paths
		template<>
		struct MetadataTraits<FSoftClassPath> : SoftObjectPathTraits<FSoftClassPath>, DefaultStringTraits<FSoftClassPath>, DefaultHashTraits<FSoftClassPath>
		{
		};

		/**
		* Generic function to broadcast an InType to an OutType.
		* Supports only PCG types
		* @param InValue - The Value to convert
		* @param OutValue - The converted value
		* @returns true if the conversion worked, false otherwise.
		*/
		template <typename InType, typename OutType>
		inline bool GetValueWithBroadcast(const InType& InValue, OutType& OutValue)
		{
			if constexpr (std::is_same_v<OutType, InType>)
			{
				OutValue = InValue;
				return true;
			}
			else
			{
				if constexpr (!IsBroadcastable<InType, OutType>())
				{
					return false;
				}
				else
				{
					if constexpr (std::is_same_v<OutType, FVector4>)
					{
						// TODO: Should W be 0? 1? Something else? Depending on operation?
						// For now, we set Z to 0 (for vec 2) and we set W to 1.
						if constexpr (std::is_same_v<InType, FVector>)
						{
							OutValue = FVector4(InValue, 1.0);
						}
						else if constexpr (std::is_same_v<InType, FVector2D>)
						{
							OutValue = FVector4(InValue.X, InValue.Y, 0.0, 1.0);
						}
						else
						{
							OutValue = FVector4(InValue, InValue, InValue, InValue);
						}
					}
					else
					{
						// Seems like the && condition is not evaluated correctly on Linux, so we cut the condition in two `if constexpr`.
						if constexpr (std::is_same_v<OutType, FVector>)
						{
							if constexpr (std::is_same_v<InType, FVector2D>)
							{
								OutValue = FVector(InValue, 0.0);
							}
							else
							{
								OutValue = OutType(InValue);
							}
						}
						else
						{
							if constexpr (std::is_same_v<OutType, FString>)
							{
								OutValue = MetadataTraits<InType>::ToString(InValue);
							}
							else
							{
								OutValue = OutType(InValue);
							}
						}
					}

					return true;
				}
			}
		}
		
		// Object versions
		template <typename InType, typename OutType> requires (TIsTObjectPtr_V<InType> || TIsTObjectPtr_V<OutType> || UE::CTSoftObjectPtr<InType> || UE::CTSoftObjectPtr<OutType> || UE::CTSoftClassPtr<InType> || UE::CTSoftClassPtr<OutType>)
		inline bool GetValueWithBroadcast(const InType& InValue, OutType& OutValue)
		{
			if constexpr (std::is_base_of_v<FSoftObjectPath, OutType> || std::is_same_v<OutType, FString>)
			{
				// Use string as intermediate type since we can't construct a FSoftClassPath from a FSoftObjectPath
				FString Value;
				if constexpr (TIsTObjectPtr_V<InType>)
				{
					Value = InValue.GetPath();
				}
				else
				{
					Value = InValue.ToString();
				}

				if constexpr (std::is_same_v<OutType, FString>)
				{
					OutValue = MoveTemp(Value);
				}
				else
				{
					OutValue = OutType(MoveTemp(Value));
				}
				
				return true;
			}
			else if constexpr (UE::CTSoftObjectPtr<InType> || UE::CTSoftClassPtr<InType> || std::is_base_of_v<FSoftObjectPath, InType> || std::is_same_v<InType, FString>)
			{
				const FSoftObjectPath* ValuePtr = nullptr;
				FSoftObjectPath TempValue;
				
				if constexpr (std::is_same_v<InType, FString>)
				{
					TempValue = FSoftObjectPath(InValue);
					ValuePtr = &TempValue;
				}
				else if constexpr (UE::CTSoftObjectPtr<InType> || UE::CTSoftClassPtr<InType>)
				{
					ValuePtr = &InValue.ToSoftObjectPath();
				}
				else
				{
					ValuePtr = &InValue;
				}
				
				if constexpr (TIsTObjectPtr_V<OutType>)
				{
					// Note: It is the responsibility of the caller to protect the call from a GC Scope Guard
					// as it would be expensive to create the guard for each value
					OutValue = Cast<typename OutType::ElementType>(ValuePtr->ResolveObject());
					if (!OutValue)
					{
						if (!IsInGameThread())
						{
							LogNotLoadedObjectNotOnGameThread(*ValuePtr);
						}
						else
						{
							OutValue = Cast<typename OutType::ElementType>(ValuePtr->TryLoad());
						}
					}
				}
				else
				{
					OutValue = OutType(*ValuePtr);
				}
				
				return true;
			}
			else if constexpr (UE::CTSoftObjectPtr<OutType> || UE::CTSoftClassPtr<OutType>)
			{
				if constexpr (TIsTObjectPtr_V<InType> && std::is_base_of_v<UClass, typename InType::ElementType>)
				{
					OutValue = OutType(InValue.Get());
				}
				else
				{
					OutValue = InValue;
				}

				return true;
			}
			else
			{
				static_assert(!std::is_same_v<InType, InType>);
				return false;
			}
		}
		
		template <typename InType, typename OutType>
		inline void GetValuesWithBroadcast_Internal(void* DestPtr, const void* SrcPtr, int32 Count)
		{
			TConstArrayView<InType> Src(static_cast<const InType*>(SrcPtr), Count);
			TArrayView<OutType> Dest(static_cast<OutType*>(DestPtr), Count);
			
			if constexpr (TIsTObjectPtr_V<OutType>)
			{
				// Make sure that if we write to an object that we are guarding the resolve/load against the GC.
				FGCScopeGuard Guard{};
				
				for (int32 i = 0; i < Count; ++i)
				{
					GetValueWithBroadcast<InType, OutType>(Src[i], Dest[i]);
				}
			}
			else
			{
				for (int32 i = 0; i < Count; ++i)
				{
					GetValueWithBroadcast<InType, OutType>(Src[i], Dest[i]);
				}
			}
		}

		template <typename InType, typename OutType>
		inline void GetValuesWithConstructible_Internal(void* DestPtr, const void* SrcPtr, int32 Count)
		{
			static_assert(std::is_constructible_v<OutType, InType>);
			TConstArrayView<InType> Src(static_cast<const InType*>(SrcPtr), Count);
			TArrayView<OutType> Dest(static_cast<OutType*>(DestPtr), Count);

			for (int32 i = 0; i < Count; ++i)
			{
				Dest[i] = OutType(Src[i]);
			}
		}

		template <typename InType, typename OutType>
		inline bool GetValueConstructible(const InType& InValue, OutType& OutValue)
		{
			if constexpr(std::is_constructible_v<OutType, InType>)
			{
				OutValue = OutType(InValue);
				return true;
			}
			else
			{
				return false;
			}
		}

		template<typename InType, typename OutType>
		inline bool GetValueWithBroadcastAndConstructible(const InType& InValue, OutType& OutValue)
		{
			if constexpr (std::is_same_v<OutType, InType>)
			{
				OutValue = InValue;
				return true;
			}
			else
			{
				if constexpr (PCG::Private::IsBroadcastable<InType, OutType>())
				{
					return GetValueWithBroadcast<InType, OutType>(InValue, OutValue);
				}
				else
				{
					return GetValueConstructible<InType, OutType>(InValue, OutValue);
				}
			}
		}

		template <typename T>
		inline FString DefaultStringTraits<T>::ZeroValueString()
		{
			if constexpr (std::is_same_v<T, FString>)
			{
				return FString();
			}
			else
			{
				return MetadataTraits<T>::ZeroValue().ToString();
			}
		}

		template <typename T>
		inline FString LexToStringTraits<T>::ZeroValueString()
		{
			return LexToString(MetadataTraits<T>::ZeroValue());
		}
	}
}
